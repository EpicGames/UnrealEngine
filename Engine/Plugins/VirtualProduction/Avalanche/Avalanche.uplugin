{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Motion Design",
	"Description": "Compositing, designer and broadcasting tool.\n\nPlugin Dependencies: Advanced Renamer, Custom Details View, Dynamic Material, Geometry Cache, Geometry Scripting, Media Compositing, Media IO Framework, Mesh Modeling Toolset Exp, Remote Control, SVG Importer, Text3D and ActorModifierCore.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"Installed": false,
	"EnabledByDefault": false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux",
		"Mac"
	],
	"Modules": [
		{
			"Name": "Avalanche",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheCore",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheCamera",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheEditorCore",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheComponentVisualizers",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheEffectors",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheEffectorsEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheInteractiveTools",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheInteractiveToolsRuntime",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheLevelViewport",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheOutliner",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheMask",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheMaskEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheMedia",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheMediaEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheModifiers",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheModifiersEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalanchePropertyAnimator",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalanchePropertyAnimatorEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheRemoteControl",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheRemoteControlEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheSequence",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheSequencer",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheShapes",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheShapesEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheText",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheTextEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheViewport",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheSceneTree",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheTransition",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheTransitionEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheMRQ",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheMRQEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheSVGEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheTag",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheTagEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheAttribute",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheAttributeEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheSceneRig",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheSceneRigEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheMaterial",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AvalancheFunctionalTest",
			"Type": "DeveloperTool",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		}
	],
	"Plugins": [
		{
			"Name": "AdvancedRenamer",
			"Enabled": true
		},
		{
			"Name": "CustomDetailsView",
			"Enabled": true
		},
		{
			"Name": "DynamicMaterial",
			"Enabled": true
		},
		{
			"Name": "GeometryCache",
			"Enabled": true
		},
		{
			"Name": "GeometryScripting",
			"Enabled": true
		},
		{
			"Name": "LevelSequenceEditor",
			"Enabled": true
		},
		{
			"Name": "MediaCompositing",
			"Enabled": true
		},
		{
			"Name": "MediaIOFramework",
			"Enabled": true
		},
		{
			"Name": "MediaPlate",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolsetExp",
			"Enabled": true
		},
		{
			"Name": "MovieRenderPipeline",
			"Enabled": true
		},
		{
			"Name": "ProceduralMeshComponent",
			"Enabled": true
		},
		{
			"Name": "PropertyAnimator",
			"Enabled": true
		},
		{
			"Name": "PropertyAnimatorCore",
			"Enabled": true
		},
		{
			"Name": "RemoteControl",
			"Enabled": true
		},
		{
			"Name": "RemoteControlComponents",
			"Enabled": true
		},
		{
			"Name": "SequenceNavigator",
			"Enabled": true
		},
		{
			"Name": "SerializationUtils",
			"Enabled": true
		},
		{
			"Name": "StateTree",
			"Enabled": true
		},
		{
			"Name": "SVGImporter",
			"Enabled": true
		},
		{
			"Name": "Text3D",
			"Enabled": true
		},
		{
			"Name": "ActorModifierCore",
			"Enabled": true
		},
		{
			"Name": "ActorModifier",
			"Enabled": true
		},
		{
			"Name": "OperatorStack",
			"Enabled": true
		},
		{
			"Name": "GeometryMask",
			"Enabled": true
		},
		{
			"Name": "ClonerEffector",
			"Enabled": true
		}
	]
}