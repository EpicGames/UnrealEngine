[CoreRedirects]

+ClassRedirects=(OldName="/Script/Avalanche.AvaAnimation",NewName="/Script/AvalancheSequence.AvaSequence")
+ClassRedirects=(OldName="/Script/Avalanche.AvaAnimationActor",NewName="/Script/AvalancheSequence.AvaSequencePlaybackActor")

+EnumRedirects=(OldName="/Script/Avalanche.EAvaAnimationPlayMode",NewName="/Script/AvalancheSequence.EAvaSequencePlayMode")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaMarkRole",NewName="/Script/AvalancheSequence.EAvaMarkRole")

+StructRedirects=(OldName="/Script/Avalanche.AvaLevelSequenceBindingReference",NewName="/Script/AvalancheSequence.AvaAnimBindingReference")
+StructRedirects=(OldName="/Script/Avalanche.AvaLevelSequenceBindingReferenceArray",NewName="/Script/AvalancheSequence.AvaAnimBindingReferenceArray")
+StructRedirects=(OldName="/Script/Avalanche.AvaLevelSequenceBindingReferences",NewName="/Script/AvalancheSequence.AvaAnimBindingReferences")
+StructRedirects=(OldName="/Script/Avalanche.AvaAnimBindingReference",NewName="/Script/AvalancheSequence.AvaAnimBindingReference")
+StructRedirects=(OldName="/Script/Avalanche.AvaAnimBindingReferenceArray",NewName="/Script/AvalancheSequence.AvaAnimBindingReferenceArray")
+StructRedirects=(OldName="/Script/Avalanche.AvaAnimBindingReferences",NewName="/Script/AvalancheSequence.AvaAnimBindingReferences")
+StructRedirects=(OldName="/Script/Avalanche.AvaAnimPlayParams",NewName="/Script/AvalancheSequence.AvaSequencePlayParams")

+PropertyRedirects=(OldName="AvalancheRemoteControlValues.Values", NewName="AvaPlayableRemoveControls.EntityValues")
+PropertyRedirects=(OldName="AvaMediaOutputInfo.HostName", NewName="AvaMediaOutputInfo.ServerName")
+PropertyRedirects=(OldName="/Script/Avalanche.AvalanchePlaylist.Pages", NewName="Pages_DEPRECATED")
+PropertyRedirects=(OldName="/Script/AvalancheSceneTree.AvaSceneItem.ObjectPath", NewName="Id")

+ClassRedirects=(OldName="AvaTextBackgroundModifier",NewName="AvaContentBackgroundModifier")
+ClassRedirects=(OldName="/Script/AvalancheModifiers.AvaContentBackgroundModifier",NewName="/Script/AvalancheModifiers.AvaAutoSizeModifier")
+ClassRedirects=(OldName="/Script/AvalancheModifiers.AvaContentBackgroundModifierFactory",NewName="/Script/AvalancheModifiers.AvaAutoSizeModifierFactory")
+ClassRedirects=(OldName="/Script/AvalancheModifiersEditor.AvaContentBackgroundModifierExtension",NewName="/Script/AvalancheModifiersEditor.AvaAutoSizeModifierExtension")
+ClassRedirects=(OldName="/Script/Avalanche.AvaViewportCameraActor",NewName="/Script/Engine.CameraActor")

+ClassRedirects=(OldName="/Script/Avalanche.MovieSceneAvaToolboxRectCornerSection",NewName="/Script/AvalancheShapes.MovieSceneAvaShapeRectCornerSection")
+ClassRedirects=(OldName="/Script/Avalanche.MovieSceneAvaToolboxRectCornerTrack",NewName="/Script/AvalancheShapes.MovieSceneAvaShapeRectCornerTrack")
+ClassRedirects=(OldName="/Script/Avalanche.AvaToolboxShapeActor",NewName="/Script/AvalancheShapes.AvaShapeActor")
+ClassRedirects=(OldName="/Script/Avalanche.AvaToolbox2DArrowDynamicMesh",NewName="/Script/AvalancheShapes.AvaShape2DArrowDynamicMesh")
+ClassRedirects=(OldName="/Script/Avalanche.AvaToolbox2DDynMeshBase",NewName="/Script/AvalancheShapes.AvaShape2DDynMeshBase")
+ClassRedirects=(OldName="/Script/Avalanche.AvaToolbox3DDynMeshBase",NewName="/Script/AvalancheShapes.AvaShape3DDynMeshBase")
+ClassRedirects=(OldName="/Script/Avalanche.AvaToolboxChevronDynamicMesh",NewName="/Script/AvalancheShapes.AvaShapeChevronDynamicMesh")
+ClassRedirects=(OldName="/Script/Avalanche.AvaToolboxConeDynamicMesh",NewName="/Script/AvalancheShapes.AvaShapeConeDynamicMesh")
+ClassRedirects=(OldName="/Script/Avalanche.AvaToolboxCubeDynamicMesh",NewName="/Script/AvalancheShapes.AvaShapeCubeDynamicMesh")
+ClassRedirects=(OldName="/Script/Avalanche.AvaToolboxDynamicMeshBase",NewName="/Script/AvalancheShapes.AvaShapeDynamicMeshBase")
+ClassRedirects=(OldName="/Script/Avalanche.AvaToolboxEllipseDynamicMesh",NewName="/Script/AvalancheShapes.AvaShapeEllipseDynamicMesh")
+ClassRedirects=(OldName="/Script/Avalanche.AvaToolboxIrregularPolygonDynamicMesh",NewName="/Script/AvalancheShapes.AvaShapeIrregularPolygonDynamicMesh")
+ClassRedirects=(OldName="/Script/Avalanche.AvaToolboxLineDynamicMesh",NewName="/Script/AvalancheShapes.AvaShapeLineDynamicMesh")
+ClassRedirects=(OldName="/Script/Avalanche.AvaToolboxNGonDynamicMesh",NewName="/Script/AvalancheShapes.AvaShapeNGonDynamicMesh")
+ClassRedirects=(OldName="/Script/Avalanche.AvaToolboxRectangleDynamicMesh",NewName="/Script/AvalancheShapes.AvaShapeRectangleDynamicMesh")
+ClassRedirects=(OldName="/Script/Avalanche.AvaToolboxRingDynamicMesh",NewName="/Script/AvalancheShapes.AvaShapeRingDynamicMesh")
+ClassRedirects=(OldName="/Script/Avalanche.AvaToolboxRoundedPolygonDynamicMesh",NewName="/Script/AvalancheShapes.AvaShapeRoundedPolygonDynamicMesh")
+ClassRedirects=(OldName="/Script/Avalanche.AvaToolboxSphereDynamicMesh",NewName="/Script/AvalancheShapes.AvaShapeSphereDynamicMesh")
+ClassRedirects=(OldName="/Script/Avalanche.AvaToolboxStarDynamicMesh",NewName="/Script/AvalancheShapes.AvaShapeStarDynamicMesh")
+ClassRedirects=(OldName="/Script/Avalanche.AvaToolboxTorusDynamicMesh",NewName="/Script/AvalancheShapes.AvaShapeTorusDynamicMesh")
+ClassRedirects=(OldName="/Script/Avalanche.AvalancheText3DActor",NewName="/Script/AvalancheText.AvaTextActor")
+ClassRedirects=(OldName="/Script/Avalanche.AvaText3DCharacterTransform",NewName="/Script/AvalancheText.AvaTextCharacterTransform")
+ClassRedirects=(OldName="/Script/Avalanche.AvaText3DComponent",NewName="/Script/AvalancheText.AvaText3DComponent")
+ClassRedirects=(OldName="/Script/Avalanche.AvalancheFontObject",NewName="/Script/AvalancheText.AvaFontObject")
+ClassRedirects=(OldName="/Script/Avalanche.AvaText3DMaterialHub",NewName="/Script/AvalancheText.AvaTextMaterialHub")
+ClassRedirects=(OldName="/Script/AvalancheText3D.AvalancheText3DActor",NewName="/Script/AvalancheText.AvaTextActor")
+ClassRedirects=(OldName="/Script/AvalancheText3D.AvaText3DCharacterTransform",NewName="/Script/AvalancheText.AvaTextCharacterTransform")
+ClassRedirects=(OldName="/Script/AvalancheText3D.AvaText3DComponent",NewName="/Script/AvalancheText.AvaText3DComponent")
+ClassRedirects=(OldName="/Script/AvalancheText3D.AvaText3DMaterialHub",NewName="/Script/AvalancheText.AvaTextMaterialHub")
+ClassRedirects=(OldName="/Script/AvalancheText3D.AvalancheFontObject",NewName="/Script/AvalancheText.AvaFontObject")
+ClassRedirects=(OldName="/Script/AvalanchePropertyControllers.PropertyControllerTextResolver",NewName="/Script/AvalanchePropertyControllers.AvaPropertyAnimatorTextResolver")
+ClassRedirects=(OldName="/Script/AvalanchePropertyControllers.PropertyControllerSequenceTimeSource",NewName="/Script/AvalanchePropertyControllers.AvaPropertyAnimatorSequenceTimeSource")
+ClassRedirects=(OldName="/Script/AvalancheModifiers.AvaCloneModifier",NewName="/Script/AvalancheModifiers.AvaPatternModifier")

+StructRedirects=(OldName="/Script/Avalanche.AvaGuideInfo",NewName="/Script/Avalanche.AvaViewportGuideInfo")
+StructRedirects=(OldName="/Script/Avalanche.MovieSceneAvaToolboxRectCornerSectionTemplate",NewName="/Script/AvalancheShapes.MovieSceneAvaShapeRectCornerSectionTemplate")
+StructRedirects=(OldName="/Script/Avalanche.AvaToolboxMaskMaterial",NewName="/Script/AvalancheShapes.AvaShapeMaskMaterial") -- ?!!
+StructRedirects=(OldName="/Script/Avalanche.AvaToolboxMesh",NewName="/Script/AvalancheShapes.AvaShapeMesh")
+StructRedirects=(OldName="/Script/Avalanche.AvaToolboxParametricMaterial",NewName="/Script/AvalancheShapes.AvaShapeParametricMaterial")
+StructRedirects=(OldName="/Script/Avalanche.AvaToolboxCachedVertex2D",NewName="/Script/AvalancheShapes.AvaShapeCachedVertex2D")
+StructRedirects=(OldName="/Script/Avalanche.AvaToolboxCachedVertex3D",NewName="/Script/AvalancheShapes.AvaShapeCachedVertex3D")
+StructRedirects=(OldName="/Script/Avalanche.AvaToolboxMaterialUVParameters",NewName="/Script/AvalancheShapes.AvaShapeMaterialUVParameters")
+StructRedirects=(OldName="/Script/Avalanche.AvaToolboxMeshData",NewName="/Script/AvalancheShapes.AvaShapeMeshData")
+StructRedirects=(OldName="/Script/Avalanche.AvaToolboxRoundedCornerSettings",NewName="/Script/AvalancheShapes.AvaShapeRoundedCornerSettings")
+StructRedirects=(OldName="/Script/Avalanche.AvaToolboxRoundedCorner",NewName="/Script/AvalancheShapes.AvaShapeRoundedCorner")
+StructRedirects=(OldName="/Script/Avalanche.AvaToolboxRectangleCornerSettings",NewName="/Script/AvalancheShapes.AvaShapeRectangleCornerSettings")
+StructRedirects=(OldName="/Script/Avalanche.AvaTextAlignment",NewName="/Script/AvalancheText.AvaTextAlignment")
+StructRedirects=(OldName="/Script/Avalanche.AvaLinearGradientSettings",NewName="/Script/AvalancheText.AvaLinearGradientSettings")
+StructRedirects=(OldName="/Script/Avalanche.AvaMaterialMaskSettings",NewName="/Script/AvalancheText.AvaMaterialMaskSettings")
+StructRedirects=(OldName="/Script/Avalanche.AvaTextField",NewName="/Script/AvalancheText.AvaTextField")
+StructRedirects=(OldName="/Script/Avalanche.AvalancheFont",NewName="/Script/AvalancheText.AvaFont")
+StructRedirects=(OldName="/Script/AvalancheText3D.AvalancheFont",NewName="/Script/AvalancheText.AvaFont")
+StructRedirects=(OldName="/Script/AvalancheText3D.AvaTextAlignment",NewName="/Script/AvalancheText.AvaTextAlignment")
+StructRedirects=(OldName="/Script/AvalancheText3D.AvaLinearGradientSettings",NewName="/Script/AvalancheText.AvaLinearGradientSettings")
+StructRedirects=(OldName="/Script/AvalancheText3D.AvaMaterialMaskSettings",NewName="/Script/AvalancheText.AvaMaterialMaskSettings")
+StructRedirects=(OldName="/Script/AvalancheText3D.AvaTextField",NewName="/Script/AvalancheText.AvaTextField")
+StructRedirects=(OldName="/Script/AvalancheModifiers.AvaCloneModifierCircleLayoutOptions",NewName="/Script/AvalancheModifiers.AvaPatternModifierCircleLayoutOptions")
+StructRedirects=(OldName="/Script/AvalancheModifiers.AvaCloneModifierGridLayoutOptions",NewName="/Script/AvalancheModifiers.AvaPatternModifierGridLayoutOptions")
+StructRedirects=(OldName="/Script/AvalancheModifiers.AvaCloneModifierLineLayoutOptions",NewName="/Script/AvalancheModifiers.AvaPatternModifierLineLayoutOptions")

+EnumRedirects=(OldName="/Script/Avalanche.EAvaToolboxGuideState",NewName="/Script/Avalanche.EAvaViewportGuideState")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaToolboxUVMode",NewName="/Script/AvalancheShapes.EAvaShapeUVMode")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaToolboxCornerType",NewName="/Script/AvalancheShapes.EAvaShapeCornerType")
+EnumRedirects=(OldName="/Script/Avalanche.ParametricMaterialStyle",NewName="/Script/AvalancheShapes.EAvaShapeParametricMaterialStyle",ValueChanges=(("SOLID","Solid"),("LINEAR_GRADIENT","LinearGradient")))
+EnumRedirects=(OldName="/Script/Avalanche.EAvaDynamicMeshUpdateState",NewName="/Script/AvalancheShapes.EAvaDynamicMeshUpdateState")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaOutlineType",NewName="/Script/AvalancheText.EAvaOutlineType")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaTextColoringStyle",NewName="/Script/AvalancheText.EAvaTextColoringStyle")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaMaterialMaskOrientation",NewName="/Script/AvalancheText.EAvaMaterialMaskOrientation")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaGradientDirection",NewName="/Script/AvalancheText.EAvaGradientDirection")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaTextLength",NewName="/Script/AvalancheText.EAvaTextLength")
+EnumRedirects=(OldName="/Script/Avalanche.EAvaTextCase",NewName="/Script/AvalancheText.EAvaTextCase")
+EnumRedirects=(OldName="/Script/Avalanche.EMaterialType",ValueChanges=(("DynamicMaterial","MaterialDesigner")))
+EnumRedirects=(OldName="/Script/AvalancheModifiers.EAvaCloneModifierPlane",NewName="/Script/AvalancheModifiers.EAvaPatternModifierPlane")
+EnumRedirects=(OldName="/Script/AvalancheModifiers.EAvaCloneModifierAxis",NewName="/Script/AvalancheModifiers.EAvaPatternModifierAxis")
+EnumRedirects=(OldName="/Script/AvalancheModifiers.EAvaCloneModifierLayout",NewName="/Script/AvalancheModifiers.EAvaPatternModifierLayout")

+PackageRedirects=(OldName="/Script/AvalancheText3D",NewName="/Script/AvalancheText")
+ClassRedirects=(OldName="/Script/AvalancheModifiers.AvaMaskModifier",NewName="/Script/AvalancheModifiers.AvaBooleanModifier")

+StructRedirects=(OldName="/Script/Avalanche.AvalancheFileVersionInfo",NewName="/Script/Avalanche.AvaFileVersionInfo")
+StructRedirects=(OldName="/Script/Avalanche.AvalancheEngineVersionInfo",NewName="/Script/Avalanche.AvaEngineVersionInfo")
+StructRedirects=(OldName="/Script/Avalanche.AvalancheCustomVersionInfo",NewName="/Script/Avalanche.AvaCustomVersionInfo")
+StructRedirects=(OldName="/Script/Avalanche.AvalancheVersionInfo",NewName="/Script/Avalanche.AvaVersionInfo")
+StructRedirects=(OldName="/Script/Avalanche.AvalancheWorldData",NewName="/Script/Avalanche.AvaWorldData")
+StructRedirects=(OldName="/Script/Avalanche.AvalancheSubObjectData",NewName="/Script/Avalanche.AvaSubObjectData")
+StructRedirects=(OldName="/Script/Avalanche.AvalancheObjectData",NewName="/Script/Avalanche.AvaObjectData")
+StructRedirects=(OldName="/Script/Avalanche.AvalancheComponentData",NewName="/Script/Avalanche.AvaComponentData")
+StructRedirects=(OldName="/Script/Avalanche.AvalancheActorData",NewName="/Script/Avalanche.AvaActorData")

+StructRedirects=(OldName="/Script/Avalanche.AvalancheInstanceSettings",NewName="/Script/Avalanche.AvaInstanceSettings")
+PropertyRedirects=(OldName="/Script/AvalancheMedia.AvalancheMediaSettings.AvalancheInstanceSettings",NewName="/Script/AvalancheMedia.AvaMediaSettings.AvaInstanceSettings")

+FunctionRedirects=(OldName="/Script/AvalancheText.AvaText3DComponent.SetAvalancheFont",NewName="/Script/AvalancheText.AvaText3DComponent.SetMotionDesignFont")
+PropertyRedirects=(OldName="/Script/AvalancheText.AvaText3DComponent.AvalancheFont",NewName="/Script/AvalancheText.AvaText3DComponent.MotionDesignFont")
+PropertyRedirects=(OldName="/Script/AvalancheText.AvaFont.AvalancheFontObject",NewName="/Script/AvalancheText.AvaFont.MotionDesignFontObject")
+ClassRedirects=(OldName="/Script/AvalancheOutliner.AvalancheOutlinerSettings",NewName="/Script/AvalancheOutliner.AvaOutlinerSettings")
+ClassRedirects=(OldName="/Script/AvalancheEditor.AvalancheEditorSettings",NewName="/Script/AvalancheEditor.AvaEditorSettings")
+ClassRedirects=(OldName="/Script/AvalancheMask.AvalancheMaskSettings",NewName="/Script/AvalancheMask.AvaMaskSettings")
+ClassRedirects=(OldName="/Script/AvalancheMaskEditor.AvalancheMaskEditorMode",NewName="/Script/AvalancheMaskEditor.AvaMaskEditorMode")

+StructRedirects=(OldName="/Script/Avalanche.AvaSceneContainsAttributeConditionInstanceData",NewName="/Script/Avalanche.AvaSceneContainsTagAttributeConditionInstanceData")

+StructRedirects=(OldName="/Script/AvalancheMedia.AnimPlaySettings",NewName="/Script/AvalancheMedia.AvaPlaybackAnimPlaySettings")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvalancheAnimations",NewName="/Script/AvalancheMedia.AvaPlaybackAnimations")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvalanchePlayable",NewName="/Script/AvalancheMedia.AvaPlayable")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvalancheBroadcast",NewName="/Script/AvalancheMedia.AvaBroadcast")
+FunctionRedirects=(OldName="/Script/AvalancheMedia.AvaBroadcast.GetAvalancheBroadcast",NewName="/Script/AvalancheMedia.AvaBroadcast.GetBroadcast")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvalancheBroadcastComponent",NewName="/Script/AvalancheMedia.AvaBroadcastComponent")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvaBroadcastProfile",NewName="/Script/AvalancheMedia.AvaBroadcastProfile")
+EnumRedirects=(OldName="/Script/AvalancheMedia.EAvaChannelState",NewName="/Script/AvalancheMedia.EAvaBroadcastChannelState")
+EnumRedirects=(OldName="/Script/AvalancheMedia.EAvaBroadcastChannelType",NewName="/Script/AvalancheMedia.EAvaBroadcastChannelType")
+EnumRedirects=(OldName="/Script/AvalancheMedia.EAvalanchePlayableStatus",NewName="/Script/AvalancheMedia.EAvaPlayableStatus")
+EnumRedirects=(OldName="/Script/AvalancheMedia.EAvalanchePlayableSequenceEventType",NewName="/Script/AvalancheMedia.EAvaPlayableSequenceEventType")
+EnumRedirects=(OldName="/Script/AvalancheMedia.EAvaPageListType",NewName="/Script/AvalancheMedia.EAvaRundownPageListType")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvaPageListReference",NewName="/Script/AvalancheMedia.AvaRundownPageListReference")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvaDeviceProviderData",NewName="/Script/AvalancheMedia.AvaBroadcastDeviceProviderData")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvaDisplayMediaCapture",NewName="/Script/AvalancheMedia.AvaBroadcastDisplayMediaCapture")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvaDisplayMediaOutput",NewName="/Script/AvalancheMedia.AvaBroadcastDisplayMediaOutput")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvalancheLevelStreamingPlayable",NewName="/Script/AvalancheMedia.AvaPlayableLevelStreaming")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvalanchePlayableTransition",NewName="/Script/AvalancheMedia.AvaPlayableTransition")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvaLocalPlayableTransition",NewName="/Script/AvalancheMedia.AvaPlayableLocalTransition")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvaOutputChannel",NewName="/Script/AvalancheMedia.AvaBroadcastOutputChannel")
+EnumRedirects=(OldName="/Script/AvalancheMedia.EAvalanchePageStatus",NewName="/Script/AvalancheMedia.EAvaRundownPageStatus")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvalanchePageStatus",NewName="/Script/AvalancheMedia.AvaRundownChannelPageStatus")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvalanchePage",NewName="/Script/AvalancheMedia.AvaRundownPage")
+ClassRedirects=(OldName="/Script/AvalancheMedia.UAvaRundownPlaybackInstancePlayer",NewName="/Script/AvalancheMedia.AvaRundownPlaybackInstancePlayer")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvalanchePagePlayer",NewName="/Script/AvalancheMedia.AvaRundownPagePlayer")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvalanchePageTransition",NewName="/Script/AvalancheMedia.AvaRundownPageTransition")
+EnumRedirects=(OldName="/Script/AvalancheMedia.EAvaPlaybackStopOptions",NewName="/Script/AvalancheMedia.EAvaPlaybackStopOptions")
+EnumRedirects=(OldName="/Script/AvalancheMedia.EAvaPlaybackUnloadOptions",NewName="/Script/AvalancheMedia.EAvaPlaybackUnloadOptions")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvalanchePlaybackPlayableGroup",NewName="/Script/AvalancheMedia.AvaPlaybackPlayableGroup")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvalanchePlayback",NewName="/Script/AvalancheMedia.AvaPlaybackGraph")
+PropertyRedirects=(OldName="/Script/AvalancheMedia.AvaPlaybackNodeBlueprintPlayer.BlueprintAsset",NewName="/Script/AvalancheMedia.AvaPlaybackNodeBlueprintPlayer.Asset")
+EnumRedirects=(OldName="/Script/AvalancheMedia.EAvaRemoteControlChanges",NewName="/Script/AvalancheMedia.EAvaPlayableRemoteControlChanges")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvalancheRemoteControlValue",NewName="/Script/AvalancheMedia.AvaPlayableRemoteControlValue")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvalancheRemoteControlValues",NewName="/Script/AvalancheMedia.AvaPlayableRemoteControlValues")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvalancheRemoteControlValueAsBytes_Legacy",NewName="/Script/AvalancheMedia.AvaPlayableRemoteControlValueAsBytes_Legacy")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvaRemotePlayableTransition",NewName="/Script/AvalancheMedia.AvaPlayableRemoteTransition")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvalancheRemoteProxyPlayable",NewName="/Script/AvalancheMedia.AvaPlayableRemoteProxy")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvaRenderTargetMediaCapture",NewName="/Script/AvalancheMedia.AvaBroadcastRenderTargetMediaCapture")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvaRenderTargetMediaOutput",NewName="/Script/AvalancheMedia.AvaBroadcastRenderTargetMediaOutput")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvalanchePageCollection",NewName="/Script/AvalancheMedia.AvaRundownPageCollection")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvalancheSubList",NewName="/Script/AvalancheMedia.AvaRundownSubList")
+EnumRedirects=(OldName="/Script/AvalancheMedia.EAvaPlaylistPageStopOptions",NewName="/Script/AvalancheMedia.EAvaRundownPageStopOptions")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvalanchePlaylist",NewName="/Script/AvalancheMedia.AvaRundown")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvalanchePlaylistComponent",NewName="/Script/AvalancheMedia.AvaRundownComponent")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvaRundownMacroCommand",NewName="/Script/AvalancheMedia.AvaRundownMacroCommand")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvaRundownMacroKeyBinding",NewName="/Script/AvalancheMedia.AvaRundownMacroKeyBinding")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvaRundownMacroCollection",NewName="/Script/AvalancheMedia.AvaRundownMacroCollection")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvaPlaybackServerLoggingEntry",NewName="/Script/AvalancheMedia.AvaPlaybackServerLoggingEntry")
+StructRedirects=(OldName="/Script/AvalancheMedia.LocalPlaybackServerSettings",NewName="/Script/AvalancheMedia.AvaMediaLocalPlaybackServerSettings")
+ClassRedirects=(OldName="/Script/AvalancheMedia.AvalancheMediaSettings",NewName="/Script/AvalancheMedia.AvaMediaSettings")
+PropertyRedirects=(OldName="/Script/AvalancheMedia.AvaMediaSettings.ManagedAvalancheInstanceCacheMaximumSize",NewName="/Script/AvalancheMedia.AvaMediaSettings.ManagedInstanceCacheMaximumSize")
+EnumRedirects=(OldName="/Script/AvalancheMedia.EAvaSyncResponseResult",NewName="/Script/AvalancheMedia.EAvaMediaSyncResponseResult")
+EnumRedirects=(OldName="/Script/AvalancheMedia.EAvaSyncEngineType",NewName="/Script/AvalancheMedia.EAvaMediaSyncEngineType")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvaSyncConnectionInfo",NewName="/Script/AvalancheMedia.AvaMediaSyncConnectionInfo")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvaSyncResponse",NewName="/Script/AvalancheMedia.AvaMediaSyncResponse")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvaSyncCompareResponse",NewName="/Script/AvalancheMedia.AvaMediaSyncCompareResponse")
+StructRedirects=(OldName="/Script/AvalancheMedia.AvaTransitionPlayableScene",NewName="/Script/AvalancheMedia.AvaPlayableTransitionScene")
+ClassRedirects=(OldName="/Script/AvalancheMediaEditor.AvalancheMediaEditorSettings",NewName="/Script/AvalancheMediaEditor.AvaMediaEditorSettings")
+ClassRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaybackGraph",NewName="/Script/AvalancheMediaEditor.AvaPlaybackEditorGraph")
+ClassRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaybackGraphSchema",NewName="/Script/AvalancheMediaEditor.AvaPlaybackEditorGraphSchema")
+ClassRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaybackGraphNode",NewName="/Script/AvalancheMediaEditor.AvaPlaybackEditorGraphNode")
+ClassRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaybackGraphNode_EventAction",NewName="/Script/AvalancheMediaEditor.AvaPlaybackEditorGraphNode_EventAction")
+ClassRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaybackGraphNode_EventFlow",NewName="/Script/AvalancheMediaEditor.AvaPlaybackEditorGraphNode_EventFlow")
+ClassRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaybackGraphNode_EventTrigger",NewName="/Script/AvalancheMediaEditor.AvaPlaybackEditorGraphNode_EventTrigger")
+ClassRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaybackGraphNode_LevelPlayer",NewName="/Script/AvalancheMediaEditor.AvaPlaybackEditorGraphNode_LevelPlayer")
+ClassRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaybackGraphNode_Root",NewName="/Script/AvalancheMediaEditor.AvaPlaybackEditorGraphNode_Root")
+EnumRedirects=(OldName="/Script/AvalancheMediaEditor.EAvaRundownPageSet",NewName="/Script/AvalancheMediaEditor.EAvaRundownPageSet")
+ClassRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownEditorSettings",NewName="/Script/AvalancheMediaEditor.AvaRundownEditorSettings")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistMsgBase",NewName="/Script/AvalancheMediaEditor.AvaRundownMsgBase")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistServerMsg",NewName="/Script/AvalancheMediaEditor.AvaRundownServerMsg")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistPing",NewName="/Script/AvalancheMediaEditor.AvaRundownPing")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistPong",NewName="/Script/AvalancheMediaEditor.AvaRundownPong")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistGetPlaylists",NewName="/Script/AvalancheMediaEditor.AvaRundownGetRundowns")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistPlaylists",NewName="/Script/AvalancheMediaEditor.AvaRundownRundowns")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistLoadPlaylist",NewName="/Script/AvalancheMediaEditor.AvaRundownLoadRundown")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistGetPages",NewName="/Script/AvalancheMediaEditor.AvaRundownGetPages")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistCreatePage",NewName="/Script/AvalancheMediaEditor.AvaRundownCreatePage")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistDeletePage",NewName="/Script/AvalancheMediaEditor.AvaRundownDeletePage")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistCreateTemplate",NewName="/Script/AvalancheMediaEditor.AvaRundownCreateTemplate")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistDeleteTemplate",NewName="/Script/AvalancheMediaEditor.AvaRundownDeleteTemplate")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistChangeTemplateBP",NewName="/Script/AvalancheMediaEditor.AvaRundownChangeTemplateBP")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistPage",NewName="/Script/AvalancheMediaEditor.AvaRundownPageInfo")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistPages",NewName="/Script/AvalancheMediaEditor.AvaRundownPages")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistGetPageDetails",NewName="/Script/AvalancheMediaEditor.AvaRundownGetPageDetails")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistPageDetails",NewName="/Script/AvalancheMediaEditor.AvaRundownPageDetails")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistPagesStatuses",NewName="/Script/AvalancheMediaEditor.AvaRundownPagesStatuses")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistPageListChanged",NewName="/Script/AvalancheMediaEditor.AvaRundownPageListChanged")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistPageBlueprintChanged",NewName="/Script/AvalancheMediaEditor.AvaRundownPageBlueprintChanged")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistPageChannelChanged",NewName="/Script/AvalancheMediaEditor.AvaRundownPageChannelChanged")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistPageAnimSettingsChanged",NewName="/Script/AvalancheMediaEditor.AvaRundownPageAnimSettingsChanged")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistPageChangeChannel",NewName="/Script/AvalancheMediaEditor.AvaRundownPageChangeChannel")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistUpdatePageFromRCP",NewName="/Script/AvalancheMediaEditor.AvaRundownUpdatePageFromRCP")
+EnumRedirects=(OldName="/Script/AvalancheMediaEditor.EAvaPlaylistPageActions",NewName="/Script/AvalancheMediaEditor.EAvaRundownPageActions")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistPageAction",NewName="/Script/AvalancheMediaEditor.AvaRundownPageAction")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistPagePreviewAction",NewName="/Script/AvalancheMediaEditor.AvaRundownPagePreviewAction")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistPageActions",NewName="/Script/AvalancheMediaEditor.AvaRundownPageActions")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistPagePreviewActions",NewName="/Script/AvalancheMediaEditor.AvaRundownPagePreviewActions")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistPageEvent",NewName="/Script/AvalancheMediaEditor.AvaRundownPageEvent")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistOutputDeviceItem",NewName="/Script/AvalancheMediaEditor.AvaRundownOutputDeviceItem")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistOutputClassItem",NewName="/Script/AvalancheMediaEditor.AvaRundownOutputClassItem")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistDevicesList",NewName="/Script/AvalancheMediaEditor.AvaRundownDevicesList")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistGetChannel",NewName="/Script/AvalancheMediaEditor.AvaRundownGetChannel")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistGetChannels",NewName="/Script/AvalancheMediaEditor.AvaRundownGetChannels")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistChannel",NewName="/Script/AvalancheMediaEditor.AvaRundownChannel")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaBroadcastChannelListChanged",NewName="/Script/AvalancheMediaEditor.AvaRundownChannelListChanged")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistChannelResponse",NewName="/Script/AvalancheMediaEditor.AvaRundownChannelResponse")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistChannels",NewName="/Script/AvalancheMediaEditor.AvaRundownChannels")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistAssetsChanged",NewName="/Script/AvalancheMediaEditor.AvaRundownAssetsChanged")
+EnumRedirects=(OldName="/Script/AvalancheMediaEditor.EAvaPlaylistChannelActions",NewName="/Script/AvalancheMediaEditor.EAvaRundownChannelActions")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistChannelAction",NewName="/Script/AvalancheMediaEditor.AvaRundownChannelAction")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistGetDevices",NewName="/Script/AvalancheMediaEditor.AvaRundownGetDevices")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistAddChannelDevice",NewName="/Script/AvalancheMediaEditor.AvaRundownAddChannelDevice")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistEditChannelDevice",NewName="/Script/AvalancheMediaEditor.AvaRundownEditChannelDevice")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistRemoveChannelDevice",NewName="/Script/AvalancheMediaEditor.AvaRundownRemoveChannelDevice")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistOutputDeviceItemResponse",NewName="/Script/AvalancheMediaEditor.AvaRundownOutputDeviceItemResponse")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistGetChannelImage",NewName="/Script/AvalancheMediaEditor.AvaRundownGetChannelImage")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistChannelImage",NewName="/Script/AvalancheMediaEditor.AvaRundownChannelImage")
+ClassRedirects=(OldName="/Script/AvalancheMediaEditor.AvaPlaylistPageContext",NewName="/Script/AvalancheMediaEditor.AvaRundownPageContext")
+EnumRedirects=(OldName="/Script/AvalancheMediaEditor.EAvaRundownEditorMacroCommand",NewName="/Script/AvalancheMediaEditor.EAvaRundownEditorMacroCommand")
+PropertyRedirects=(OldName="/Script/AvalancheMedia.AvaRundownComponent.Playlist",NewName="/Script/AvalancheMedia.AvaRundownComponent.Rundown")
+EnumRedirects=(OldName="/Script/AvalancheMedia.EAvaPlayType",NewName="/Script/AvalancheMedia.EAvaRundownPagePlayType")
+PropertyRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownRundowns.Playlists",NewName="/Script/AvalancheMediaEditor.AvaRundownRundowns.Rundowns")
+EnumRedirects=(OldName="/Script/AvalancheMediaEditor.EAvaPlaylistPageEvents",NewName="/Script/AvalancheMediaEditor.EAvaRundownPageEvents")
+PropertyRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownLoadRundown.Playlist",NewName="/Script/AvalancheMediaEditor.AvaRundownLoadRundown.Rundown")
+PropertyRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownGetPages.Playlist",NewName="/Script/AvalancheMediaEditor.AvaRundownGetPages.Rundown")
+PropertyRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownCreatePage.Playlist",NewName="/Script/AvalancheMediaEditor.AvaRundownCreatePage.Rundown")
+PropertyRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownDeletePage.Playlist",NewName="/Script/AvalancheMediaEditor.AvaRundownDeletePage.Rundown")
+PropertyRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownCreateTemplate.Playlist",NewName="/Script/AvalancheMediaEditor.AvaRundownCreateTemplate.Rundown")
+PropertyRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownDeleteTemplate.Playlist",NewName="/Script/AvalancheMediaEditor.AvaRundownDeleteTemplate.Rundown")
+PropertyRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownChangeTemplateBP.Playlist",NewName="/Script/AvalancheMediaEditor.AvaRundownChangeTemplateBP.Rundown")
+PropertyRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownGetPageDetails.Playlist",NewName="/Script/AvalancheMediaEditor.AvaRundownGetPageDetails.Rundown")
+PropertyRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownPageDetails.Playlist",NewName="/Script/AvalancheMediaEditor.AvaRundownPageDetails.Rundown")
+PropertyRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownPagesStatuses.Playlist",NewName="/Script/AvalancheMediaEditor.AvaRundownPagesStatuses.Rundown")
+PropertyRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownPageListChanged.Playlist",NewName="/Script/AvalancheMediaEditor.AvaRundownPageListChanged.Rundown")
+PropertyRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownPageBlueprintChanged.Playlist",NewName="/Script/AvalancheMediaEditor.AvaRundownPageBlueprintChanged.Rundown")
+PropertyRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownPageChannelChanged.Playlist",NewName="/Script/AvalancheMediaEditor.AvaRundownPageChannelChanged.Rundown")
+PropertyRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownPageAnimSettingsChanged.Playlist",NewName="/Script/AvalancheMediaEditor.AvaRundownPageAnimSettingsChanged.Rundown")
+PropertyRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownPageChangeChannel.Playlist",NewName="/Script/AvalancheMediaEditor.AvaRundownPageChangeChannel.Rundown")
+EnumRedirects=(OldName="/Script/AvalancheMedia.EAvaRemoteControlResult",NewName="/Script/AvalancheMedia.EAvaPlayableRemoteControlResult")

+PackageRedirects=(OldName="/Avalanche/EditorResources/M_AvalancheBackplate.M_AvalancheBackplate",NewName="/Avalanche/EditorResources/M_Backplate.M_Backplate")
+PackageRedirects=(OldName="/Avalanche/EditorResources/M_AvalancheCheckerboard.M_AvalancheCheckerboard",NewName="/Avalanche/EditorResources/M_Checkerboard.M_Checkerboard")
+PackageRedirects=(OldName="/Avalanche/EditorResources/M_AvalancheChannelVisualizer_Red.M_AvalancheChannelVisualizer_Red",NewName="/Avalanche/EditorResources/M_ChannelVisualizer_Red.M_ChannelVisualizer_Red")
+PackageRedirects=(OldName="/Avalanche/EditorResources/M_AvalancheChannelVisualizer_Green.M_AvalancheChannelVisualizer_Green",NewName="/Avalanche/EditorResources/M_ChannelVisualizer_Green.M_ChannelVisualizer_Green")
+PackageRedirects=(OldName="/Avalanche/EditorResources/M_AvalancheChannelVisualizer_Blue.M_AvalancheChannelVisualizer_Blue",NewName="/Avalanche/EditorResources/M_ChannelVisualizer_Blue.M_ChannelVisualizer_Blue")
+PackageRedirects=(OldName="/Avalanche/EditorResources/M_AvalancheChannelVisualizer_Alpha.M_AvalancheChannelVisualizer_Alpha",NewName="/Avalanche/EditorResources/M_ChannelVisualizer_Alpha.M_ChannelVisualizer_Alpha")

; Redirects for Mask Material functions moved to Geometry Mask plugin
+PackageRedirects=(OldName="/Avalanche/MaskResources/MF_ApplyMask2D", NewName="/GeometryMask/GeometryMask/MF_ApplyMask2D")
+PackageRedirects=(OldName="/Avalanche/MaskResources/MF_ApplyMask2D_Single", NewName="/GeometryMask/GeometryMask/MF_ApplyMask2D_Single")

; Redirects for rundown server requests for legacy clients.
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownPing",NewName="/Script/AvalancheMedia.AvaRundownPing")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownGetRundowns",NewName="/Script/AvalancheMedia.AvaRundownGetRundowns")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownLoadRundown",NewName="/Script/AvalancheMedia.AvaRundownLoadRundown")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownSaveRundown",NewName="/Script/AvalancheMedia.AvaRundownSaveRundown")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownGetPages",NewName="/Script/AvalancheMedia.AvaRundownGetPages")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownCreatePage",NewName="/Script/AvalancheMedia.AvaRundownCreatePage")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownCreateTemplate",NewName="/Script/AvalancheMedia.AvaRundownCreateTemplate")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownDeletePage",NewName="/Script/AvalancheMedia.AvaRundownDeletePage")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownDeleteTemplate",NewName="/Script/AvalancheMedia.AvaRundownDeleteTemplate")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownChangeTemplateBP",NewName="/Script/AvalancheMedia.AvaRundownChangeTemplateBP")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownGetPageDetails",NewName="/Script/AvalancheMedia.AvaRundownGetPageDetails")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownPageChangeChannel",NewName="/Script/AvalancheMedia.AvaRundownPageChangeChannel")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownUpdatePageFromRCP",NewName="/Script/AvalancheMedia.AvaRundownUpdatePageFromRCP")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownPageAction",NewName="/Script/AvalancheMedia.AvaRundownPageAction")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownPagePreviewAction",NewName="/Script/AvalancheMedia.AvaRundownPagePreviewAction")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownPageActions",NewName="/Script/AvalancheMedia.AvaRundownPageActions")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownPagePreviewActions",NewName="/Script/AvalancheMedia.AvaRundownPagePreviewActions")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownGetChannel",NewName="/Script/AvalancheMedia.AvaRundownGetChannel")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownGetChannels",NewName="/Script/AvalancheMedia.AvaRundownGetChannels")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownChannelAction",NewName="/Script/AvalancheMedia.AvaRundownChannelAction")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownAddChannelDevice",NewName="/Script/AvalancheMedia.AvaRundownAddChannelDevice")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownEditChannelDevice",NewName="/Script/AvalancheMedia.AvaRundownEditChannelDevice")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownRemoveChannelDevice",NewName="/Script/AvalancheMedia.AvaRundownRemoveChannelDevice")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownGetChannelImage",NewName="/Script/AvalancheMedia.AvaRundownGetChannelImage")
+StructRedirects=(OldName="/Script/AvalancheMediaEditor.AvaRundownGetDevices",NewName="/Script/AvalancheMedia.AvaRundownGetDevices")

+PropertyRedirects=(OldName="/Script/AvalancheModifiers.AvaDynamicMeshConverterModifier.TransformUpdateInterval",NewName="/Script/AvalancheModifiers.AvaDynamicMeshConverterModifier.UpdateInterval")

; 5.7 Redirects

; Renaming TransitionNode to ContextObject to support any object that can retrieve the transition context (e.g. world context objects)
+PropertyRedirects=(OldName="AvaTransitionLibrary.IsTransitionActiveInLayer.InTransitionNode",NewName="InContextObject")
+PropertyRedirects=(OldName="AvaTransitionLibrary.GetTransitionType.InTransitionNode",NewName="InContextObject")
+PropertyRedirects=(OldName="AvaTransitionLibrary.AreScenesTransitioning.InTransitionNode",NewName="InContextObject")
+PropertyRedirects=(OldName="AvaTransitionLibrary.GetTransitionTree.InTransitionNode",NewName="InContextObject")

; Renaming 'AvaSceneState' to 'AvaAttributeContainer' to avoid confusion with the Scene State system
+ClassRedirects=(OldName="/Script/Avalanche.AvaSceneState",NewName="/Script/Avalanche.AvaAttributeContainer")
+PropertyRedirects=(OldName="/Script/Avalanche.AvaScene.SceneState",NewName="/Script/Avalanche.AvaScene.AttributeContainer")

[/Script/Avalanche.AvaAnimation]
DefaultCompletionMode=KeepState
