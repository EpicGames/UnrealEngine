{
	"Sine": "0.37, 0.0, 0.63, 1.0",
	"Cubic": "0.65, 0.0, 0.35, 1.0",
	"Quintic": "0.83, 0.0, 0.17, 1.0",
	"Circular": "0.85, 0.0, 0.15, 1.0",
	"Quadratic": "0.45, 0.0, 0.55, 1.0",
	"Quartic": "0.76, 0.0, 0.24, 1.0",
	"Exponential": "0.87, 0.0, 0.13, 1.0",
	"Back": "0.68, -0.6, 0.32, 1.6"
}