{
	"Sine": "0.12, 0.0, 0.39, 0.0",
	"Cubic": "0.32, 0.0, 0.67, 0.0",
	"Quintic": "0.64, 0.0, 0.78, 0.0",
	"Circular": "0.55, 0.0, 1.0, 0.45",
	"Quadratic": "0.11, 0.0, 0.5, 0.0",
	"Quartic": "0.5, 0.0, 0.75, 0.0",
	"Exponential": "0.7, 0.0, 0.84, 0.0",
	"Back": "0.36, 0.0, 0.66, -0.56"
}