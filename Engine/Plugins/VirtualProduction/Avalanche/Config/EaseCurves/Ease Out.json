{
	"Sine": "0.61, 1.0, 0.88, 1.0",
	"Cubic": "0.33, 1.0, 0.68, 1.0",
	"Quintic": "0.22, 1.0, 0.36, 1.0",
	"Circular": "0.0, 0.55, 0.45, 1.0",
	"Quadratic": "0.5, 1.0, 0.89, 1.0",
	"Quartic": "0.25, 1.0, 0.5, 1.0",
	"Exponential": "0.16, 1.0, 0.3, 1.0",
	"Back": "0.34, 1.56, 0.64, 1.0"
}