// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/GammaCorrectionCommon.ush"

Texture2D InputTexture;
SamplerState InputSampler;
int SrgbToLinear;
float Gamma;

// Shader to undo the HDR to LDR conversion done inPostProcessCombineLUTs.usf.
void MainPS(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float2 UV = UVAndScreenPos.xy;
	OutColor = InputTexture.Sample(InputSampler, UV);

	if (SrgbToLinear != 0)
	{
		OutColor.xyz = sRGBToLinear(OutColor.xyz);
	}

	OutColor.xyz = pow(max(0, OutColor.xyz), Gamma);
}