// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class Avalanche : ModuleRules
{
	public Avalanche(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AvalancheAttribute",
				"AvalancheCore",
				"AvalancheRemoteControl",
				"AvalancheSceneTree",
				"AvalancheSequence",
				"AvalancheTag",
				"AvalancheTransition",
				"CinematicCamera",
				"Core",
				"Engine",
				"GeometryCore",
				"GeometryFramework",
				"GeometryScriptingCore",
				"Niagara",
				"RawMesh",
				"RemoteControl",
				"RemoteControlLogic",
				"Text3D",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ActorModifierCore",
				"ApplicationCore",
				"AvalancheCamera",
				"AvalancheRemoteControl",
				"CinematicCamera",
				"CoreUObject",
				"DynamicMaterial",
				"DynamicMesh",
				"GeometryCore",
				"GeometryFramework",
				"GeometryScriptingCore",
				"HeadMountedDisplay",
				"InputCore",
				"MediaCompositing",
				"MovieScene",
				"Niagara",
				"NiagaraCore",
				"ProceduralMeshComponent",
				"Projects",
				"RHI",
				"RawMesh",
				"RemoteControlComponents",
				"RenderCore",
				"Slate",
				"SlateCore",
				"StateTreeModule",
				"StaticMeshDescription",
				"Text3D",
				"UMG",
			}
		);

		if (Target.Type == TargetRules.TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"AvalancheOutliner",
				"DynamicMaterialEditor",
				"TypedElementFramework",
				"UnrealEd",
			});

			PublicIncludePathModuleNames.Add("AvalancheEditor");
			PublicIncludePathModuleNames.Add("AvalancheOutliner");
		}
	}
}
