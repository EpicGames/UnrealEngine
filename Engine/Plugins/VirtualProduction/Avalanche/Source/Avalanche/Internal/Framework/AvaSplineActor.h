// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "AvaSplineActor.generated.h"

class USplineComponent;

/**
 * Motion Design Spline Actor
 */
UCLASS(MinimalAPI, DisplayName = "Motion Design Spline Actor")
class AAvaSplineActor : public AActor
{
	GENERATED_BODY()

public:
	AAvaSplineActor();

protected:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design")
	TObjectPtr<USplineComponent> SplineComponent;
};
