// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "AvaTestDynamicMeshActor.generated.h"

class UDynamicMeshComponent;

UCLASS(MinimalAPI, DisplayName = "Motion Design Dynamic Mesh Actor")
class AAvaTestDynamicMeshActor : public AActor
{
	GENERATED_BODY()

public:
	AAvaTestDynamicMeshActor();

	UDynamicMeshComponent* GetMeshComponent() const { return ShapeDynamicMeshComponent; }

protected:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design")
	TObjectPtr<UDynamicMeshComponent> ShapeDynamicMeshComponent = nullptr;
};
