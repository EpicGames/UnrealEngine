// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "AvaTestStaticMeshActor.generated.h"

class UStaticMeshComponent;

UCLASS(NotPlaceable, Hidden, DisplayName = "Motion Design Test Static Mesh Actor")
class AAvaTestStaticMeshActor : public AActor
{
	GENERATED_BODY()

public:
	AAvaTestStaticMeshActor();

	UStaticMeshComponent* GetMeshComponent() const { return StaticMeshComponent; }

protected:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design")
	TObjectPtr<UStaticMeshComponent> StaticMeshComponent = nullptr;
	
private:
	TWeakObjectPtr<UWorld> WorldWeak;
	TWeakObjectPtr<AAvaTestStaticMeshActor> TestStaticMeshActor;
};
