// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/World.h"
#include "Math/Transform.h"
#include "UObject/WeakObjectPtrTemplates.h"

// General Motion Design unit test log
DECLARE_LOG_CATEGORY_EXTERN(LogAvaTest, Log, All);

class AActor;
class AAvaTestDynamicMeshActor;
class AAvaTestStaticMeshActor;
class UWorld;

struct AVALANCHE_API FAvaTestUtils
{
public:
	void Init();
	void Destroy();
	UWorld* CreateWorld();

	template <typename ActorType
		UE_REQUIRES(TIsDerivedFrom<ActorType, AActor>::Value)>
	ActorType* SpawnActor(const FTransform& InTransform = FTransform::Identity)
	{
		if (UWorld* World = WorldWeak.Get())
		{
			FActorSpawnParameters Params;
			Params.ObjectFlags = RF_Transactional;
#if WITH_EDITOR
			Params.bTemporaryEditorActor = false;
#endif
			return World->SpawnActor<ActorType>(
				InTransform.GetLocation(), InTransform.GetRotation().Rotator(), Params);
		}

		return nullptr;
	}

	AAvaTestDynamicMeshActor* SpawnTestDynamicMeshActor(const FTransform& InTransform = FTransform(FVector::ZeroVector));
	
	AAvaTestStaticMeshActor* SpawnTestStaticMeshActor();
	
	TArray<AAvaTestDynamicMeshActor*> SpawnTestDynamicMeshActors(
		int32 InNumberOfActors
		, AAvaTestDynamicMeshActor* InParentActor = nullptr);
		
	void GenerateRectangleForDynamicMesh(
		AAvaTestDynamicMeshActor* InDynamicMeshActor
		, double InHeight = 100
		, double InWidth = 100);

	void LogActorLocation(AActor* InActor);

private:
	TWeakObjectPtr<UWorld> WorldWeak;
	TWeakObjectPtr<AAvaTestDynamicMeshActor> TestDynamicMeshActor;
	TWeakObjectPtr<AAvaTestStaticMeshActor> TestStaticMeshActor;
};
