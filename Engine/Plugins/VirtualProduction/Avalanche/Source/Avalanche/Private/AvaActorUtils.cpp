// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaActorUtils.h"
#include "AvaDefs.h"
#include "AvaSceneItem.h"
#include "AvaSceneSubsystem.h"
#include "AvaSceneTree.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SceneComponent.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "Framework/AvaGameInstance.h"
#include "GameFramework/Actor.h"
#include "IAvaSceneInterface.h"
#include "Math/MathFwd.h"
#include "Math/OrientedBox.h"

#if WITH_EDITOR
#include "AvaOutlinerDefines.h"
#include "AvaOutlinerSubsystem.h"
#include "AvaOutlinerUtils.h"
#include "IAvaOutliner.h"
#endif

FOrientedBox FAvaActorUtils::MakeOrientedBox(const FBox& InLocalBox, const FTransform& InWorldTransform)
{
	FOrientedBox OutOrientedBox;

	OutOrientedBox.Center = InWorldTransform.TransformPosition(InLocalBox.GetCenter());

	OutOrientedBox.AxisX = InWorldTransform.TransformVector(FVector::UnitX());
	OutOrientedBox.AxisY = InWorldTransform.TransformVector(FVector::UnitY());
	OutOrientedBox.AxisZ = InWorldTransform.TransformVector(FVector::UnitZ());

	OutOrientedBox.ExtentX = (InLocalBox.Max.X - InLocalBox.Min.X) / 2.f;
	OutOrientedBox.ExtentY = (InLocalBox.Max.Y - InLocalBox.Min.Y) / 2.f;
	OutOrientedBox.ExtentZ = (InLocalBox.Max.Z - InLocalBox.Min.Z) / 2.f;

	return OutOrientedBox;
}

FBox FAvaActorUtils::GetActorLocalBoundingBox(const AActor* InActor, bool bIncludeFromChildActors, bool bMustBeRegistered)
{
	FBox Box(ForceInit);
	Box.IsValid = 0;

	if (!InActor || !InActor->GetRootComponent())
	{
		return Box;
	}

	FTransform ActorToWorld = InActor->GetTransform();
	ActorToWorld.SetScale3D(FVector::OneVector);
	const FTransform WorldToActor = ActorToWorld.Inverse();

	uint32 FailedComponentCount = 0;
	InActor->ForEachComponent<UPrimitiveComponent>(bIncludeFromChildActors, [&FailedComponentCount, bMustBeRegistered, &WorldToActor, &Box](const UPrimitiveComponent* InPrimComp)
		{
#if WITH_EDITOR
			// Ignore Visualization Components, but don't consider them as failed components.
			if (InPrimComp->IsVisualizationComponent())
			{
				return;
			}
#endif

	if (InPrimComp->IsRegistered() || !bMustBeRegistered)
	{
		const FTransform ComponentToActor = InPrimComp->GetComponentTransform() * WorldToActor;
		Box += InPrimComp->CalcBounds(ComponentToActor).GetBox();
	}
	else
	{
		FailedComponentCount++;
	}
		});

	// Actors with no Failed Primitives should still return a valid Box with no Extents and 0,0,0 origin (local).
	if (FailedComponentCount == 0)
	{
		Box.IsValid = 1;
	}

	return Box;
}

FBox FAvaActorUtils::GetComponentLocalBoundingBox(const USceneComponent* InComponent)
{
	FBox Box(ForceInit);
	Box.IsValid = 0;

	if (!InComponent)
	{
		return Box;
	}

	if (!InComponent->IsRegistered())
	{
		return Box;
	}

#if WITH_EDITOR
	if (InComponent->IsVisualizationComponent())
	{
		return Box;
	}
#endif

	// Pre-scale component to be consistent with actor bounding boxes
	const FTransform ComponentTransform = FTransform(FRotator::ZeroRotator, FVector::ZeroVector, InComponent->GetComponentScale());
	const FBoxSphereBounds BoxSphereBounds = InComponent->CalcBounds(ComponentTransform);
	Box = BoxSphereBounds.GetBox();
	Box.IsValid = 1;

	return Box;
}

IAvaSceneInterface* FAvaActorUtils::GetSceneInterfaceFromActor(const AActor* InActor)
{
	if (!InActor)
	{
		return nullptr;
	}

	const UWorld* World = InActor->GetWorld();
	if (!World)
	{
		return nullptr;
	}

	const UAvaSceneSubsystem* SceneSubsystem = World->GetSubsystem<UAvaSceneSubsystem>();
	if (!SceneSubsystem)
	{
		return nullptr;
	}

	return SceneSubsystem->GetSceneInterface(InActor->GetLevel());
}