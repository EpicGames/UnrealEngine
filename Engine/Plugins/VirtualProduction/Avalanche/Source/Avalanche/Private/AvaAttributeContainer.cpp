// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaAttributeContainer.h"
#include "AvaNameAttribute.h"
#include "AvaSceneSettings.h"
#include "AvaTagHandle.h"
#include "AvaTagHandleContainer.h"
#include "Tags/AvaTagAttribute.h"
#include "Tags/AvaTagAttributeBase.h"

void UAvaAttributeContainer::Initialize(UAvaSceneSettings* InSceneSettings)
{
	SceneAttributes.Reset();
	if (InSceneSettings)
	{
		SceneAttributes = InSceneSettings->GetSceneAttributes();
	}
}

bool UAvaAttributeContainer::AddTagAttribute(const FAvaTagHandle& InTagHandle)
{
	if (!InTagHandle.IsValid())
	{
		return false;
	}

	// Return true if already existing
	if (ContainsTagAttribute(InTagHandle))
	{
		return true;
	}

	UAvaTagAttribute* TagAttribute = NewObject<UAvaTagAttribute>(this, NAME_None, RF_Transient);
	check(TagAttribute);
	TagAttribute->Tag = InTagHandle;

	SceneAttributes.Add(TagAttribute);
	return true;
}

bool UAvaAttributeContainer::RemoveTagAttribute(const FAvaTagHandle& InTagHandle)
{
	uint32 TagsCleared = 0;

	for (UAvaAttribute* Attribute : SceneAttributes)
	{
		UAvaTagAttributeBase* TagAttribute = Cast<UAvaTagAttributeBase>(Attribute);
		if (!TagAttribute)
		{
			continue;
		}

		// Attempt to clear the given tag handle for the attribute. This will return true if it did remove the entry
		// Do not remove the attribute itself from the list as it could still have valid tags, or later have valid tags
		if (TagAttribute->ClearTagHandle(InTagHandle))
		{
			++TagsCleared;
		}
	}

	return TagsCleared > 0;
}

bool UAvaAttributeContainer::ContainsTagAttribute(const FAvaTagHandle& InTagHandle) const
{
	return SceneAttributes.ContainsByPredicate(
		[&InTagHandle](UAvaAttribute* InAttribute)
		{
			UAvaTagAttributeBase* TagAttribute = Cast<UAvaTagAttributeBase>(InAttribute);
			return TagAttribute && TagAttribute->ContainsTag(InTagHandle);
		});
}

bool UAvaAttributeContainer::AddNameAttribute(FName InName)
{
	if (InName.IsNone())
	{
		return false;
	}

	// Return true if already existing
	if (ContainsNameAttribute(InName))
	{
		return true;
	}

	UAvaNameAttribute* NameAttribute = NewObject<UAvaNameAttribute>(this, NAME_None, RF_Transient);
	check(NameAttribute);
	NameAttribute->Name = InName;

	SceneAttributes.Add(NameAttribute);
	return true;
}

bool UAvaAttributeContainer::RemoveNameAttribute(FName InName)
{
	uint32 NamesCleared = 0;

	for (TArray<TObjectPtr<UAvaAttribute>>::TIterator Iter(SceneAttributes); Iter; ++Iter)
	{
		UAvaNameAttribute* NameAttribute = Cast<UAvaNameAttribute>(*Iter);
		if (!NameAttribute || NameAttribute->Name != InName)
		{
			continue;
		}

		// Attempt to clear the given name for the attribute.
		// If this owns the attribute, remove from the Scene Attributes list
		if (NameAttribute->GetOuter() == this)
		{
			Iter.RemoveCurrent();
		}
		// Do not remove the attribute itself from the list if it's an external attribute as it could still have valid tags,
		// or later have valid tags (due to a dynamic change)
		else
		{
			NameAttribute->Name = NAME_None;
		}

		++NamesCleared;
	}

	return NamesCleared > 0;
}

bool UAvaAttributeContainer::ContainsNameAttribute(FName InName) const
{
	return SceneAttributes.ContainsByPredicate(
		[InName](UAvaAttribute* InAttribute)
		{
			UAvaNameAttribute* NameAttribute = Cast<UAvaNameAttribute>(InAttribute);
			return NameAttribute && NameAttribute->Name == InName;
		});
}
