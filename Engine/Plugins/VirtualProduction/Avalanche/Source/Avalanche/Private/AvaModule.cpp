// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaModule.h"

#include "AvaLog.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogAva);

void FAvaModule::StartupModule()
{
	FCoreDelegates::StatCheckEnabled.AddRaw(this, &FAvaModule::HandleViewportStatCheckEnabled);
	FCoreDelegates::StatEnabled.AddRaw(this, &FAvaModule::HandleViewportStatEnabled);
	FCoreDelegates::StatDisabled.AddRaw(this, &FAvaModule::HandleViewportStatDisabled);
	FCoreDelegates::StatDisableAll.AddRaw(this, &FAvaModule::HandleViewportStatDisableAll);
}

void FAvaModule::ShutdownModule()
{
	FCoreDelegates::StatCheckEnabled.RemoveAll(this);
	FCoreDelegates::StatEnabled.RemoveAll(this);
	FCoreDelegates::StatDisabled.RemoveAll(this);
	FCoreDelegates::StatDisableAll.RemoveAll(this);
}

bool FAvaModule::SetRuntimeStatProcessingEnabled(bool bEnabled)
{
	const bool bPrevious = bRuntimeStatStatProcessingEnabled;
	bRuntimeStatStatProcessingEnabled = bEnabled;
	return bPrevious;
}

bool FAvaModule::IsRuntimeStatEnabled(const FString& InName) const
{
	FString NameLowerCase(InName);
	NameLowerCase.ToLowerInline();
	return EnabledStats.Contains(NameLowerCase);
}

int32 FAvaModule::SetRuntimeStatEnabled(const TCHAR* InName, const bool bInEnabled)
{
	// This seems to be the argument of the command line directly, which is not case sensitive.
	FString NameLowerCase(InName);
	NameLowerCase.ToLowerInline();
	
	if (bInEnabled)
	{
		EnabledStats.Add(MoveTemp(NameLowerCase));
	}
	else
	{
		EnabledStats.Remove(NameLowerCase);
	}
	return EnabledStats.Num();
}

TArray<FString> FAvaModule::GetEnabledRuntimeStats() const
{
	return EnabledStats.Array();
}

void FAvaModule::OverwriteEnabledRuntimeStats(const TArray<FString>& InEnabledStats)
{
	EnabledStats = TSet<FString>(InEnabledStats);
}

void FAvaModule::HandleViewportStatCheckEnabled(const TCHAR* InName, bool& bOutCurrentEnabled, bool& bOutOthersEnabled)
{
	// Check to see which viewports have this enabled (current, non-current)
	const bool bEnabled = IsRuntimeStatEnabled(InName);
	if (bRuntimeStatStatProcessingEnabled)
	{
		bOutCurrentEnabled = bEnabled;
	}
	else
	{
		bOutOthersEnabled |= bEnabled;
	}
}

void FAvaModule::HandleViewportStatEnabled(const TCHAR* InName)
{
	SetRuntimeStatEnabled(InName, true);
}

void FAvaModule::HandleViewportStatDisabled(const TCHAR* InName)
{
	SetRuntimeStatEnabled(InName, false);
}

void FAvaModule::HandleViewportStatDisableAll(const bool bInAnyViewport)
{
	EnabledStats.Empty();
}


IMPLEMENT_MODULE(FAvaModule, Avalanche)
