// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaModule.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"

class FAvaModule : public IAvaModule
{
public:
	//~ Begin IVPMaterialsEditorModule interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IVPMaterialsEditorModule interface

	//~ Begin IAvaModule interface
	virtual bool SetRuntimeStatProcessingEnabled(bool bEnabled) override;
	virtual bool IsRuntimeStatEnabled(const FString& InName) const override;
	virtual int32 SetRuntimeStatEnabled(const TCHAR* InName, const bool bInEnabled) override;
	virtual bool ShouldShowRuntimeStats() const override { return !EnabledStats.IsEmpty(); }
	virtual TArray<FString> GetEnabledRuntimeStats() const override;
	virtual void OverwriteEnabledRuntimeStats(const TArray<FString>& InEnabledStats) override;
	//~ End IAvaModule interface

private:
	/**
	 * Stats for all Motion Design runtime viewports is centralized here.
	 * The state (enabled) of the stats will persist across the lifetime of all
	 * game instances so that an enabled stat remains even if playback is stopped.
	 */
	TSet<FString> EnabledStats;

	/** Track if the module will respond to FCoreDelegates::StatCheckEnabled event. */
	bool bRuntimeStatStatProcessingEnabled = false;

	/** Delegate handler to see if a stat is enabled */
	void HandleViewportStatCheckEnabled(const TCHAR* InName, bool& bOutCurrentEnabled, bool& bOutOthersEnabled);

	/** Delegate handler for when stats are enabled */
	void HandleViewportStatEnabled(const TCHAR* InName);

	/** Delegate handler for when stats are disabled */
	void HandleViewportStatDisabled(const TCHAR* InName);

	/** Delegate handler for when all stats are disabled */
	void HandleViewportStatDisableAll(const bool bInAnyViewport);
};
