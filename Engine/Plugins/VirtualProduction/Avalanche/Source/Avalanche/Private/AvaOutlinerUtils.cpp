// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerUtils.h"

#if WITH_EDITOR
#include "AvaOutlinerSubsystem.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "IAvaOutliner.h"
#include "Item/AvaOutlinerActor.h"
#include "Item/AvaOutlinerLevel.h"

TSharedPtr<IAvaOutliner> FAvaOutlinerUtils::EditorGetOutliner(const UWorld* const InWorld)
{
	if (!IsValid(InWorld))
	{
		return nullptr;
	}

	UAvaOutlinerSubsystem* const OutlinerSubsystem = InWorld->GetSubsystem<UAvaOutlinerSubsystem>();
	if (!IsValid(OutlinerSubsystem))
	{
		return nullptr;
	}

	return OutlinerSubsystem->GetOutliner();
}

bool FAvaOutlinerUtils::EditorOutlinerItemsToActors(const TArray<FAvaOutlinerItemPtr>& InItems, TArray<AActor*>& OutActors)
{
	const int32 ItemCount = InItems.Num();

	OutActors.Empty();

	for (int32 i = 0; i < ItemCount; ++i)
	{
		if (!InItems[i].IsValid())
		{
			// don't allow any invalid items for this function
			OutActors.Empty();
			return false;
		}

		// Root of the tree is level
		const FAvaOutlinerLevel* const LevelItem = InItems[i]->CastTo<FAvaOutlinerLevel>();
		if (LevelItem)
		{
			return EditorOutlinerItemsToActors(LevelItem->GetChildren(), OutActors);
		}

		const FAvaOutlinerActor* const ActorItem = InItems[i]->CastTo<FAvaOutlinerActor>();
		if (ActorItem)
		{
			AActor* const ChildActor = ActorItem->GetActor();
			if (!IsValid(ChildActor))
			{
				OutActors.Empty();
				return false;
			}

			OutActors.Add(ChildActor);
		}
	}

	return true;
}

TArray<AActor*> FAvaOutlinerUtils::EditorOutlinerChildActors(TSharedPtr<IAvaOutliner> InOutliner, AActor* const InParentActor)
{
	if (!InOutliner.IsValid())
	{
		return {};
	}

	// Return all root level actors if no parent is specified.
	if (!IsValid(InParentActor))
	{
		TArray<AActor*> OutActors;
		if (EditorOutlinerItemsToActors(InOutliner->GetTreeRoot()->GetChildrenMutable(), OutActors))
		{
			return OutActors;
		}
		return {};
	}

	FAvaOutlinerItemPtr OutlinerItem = InOutliner->FindItem(InParentActor);
	if (OutlinerItem.IsValid())
	{
		TArray<AActor*> OutActors;
		if (EditorOutlinerItemsToActors(OutlinerItem->GetChildrenMutable(), OutActors))
		{
			return OutActors;
		}
	}

	return {};
}

#endif // WITH_EDITOR
