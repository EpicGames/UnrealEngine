// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaScene.h"
#include "AvaAssetTags.h"
#include "AvaAttributeContainer.h"
#include "AvaCameraSubsystem.h"
#include "AvaRemoteControlUtils.h"
#include "AvaSceneSettings.h"
#include "AvaSceneSubsystem.h"
#include "AvaSequencePlaybackObject.h"
#include "AvaSequenceSubsystem.h"
#include "AvaWorldSubsystemUtils.h"
#include "Containers/Ticker.h"
#include "EngineUtils.h"
#include "RemoteControlPreset.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/UObjectThreadContext.h"

#if WITH_EDITOR
#include "AvaField.h"
#include "EngineAnalytics.h"
#include "Misc/ScopedSlowTask.h"
#include "RemoteControlBinding.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogAvaScene, Log, All);

#define LOCTEXT_NAMESPACE "AvaScene"

#if WITH_EDITOR
void AAvaScene::NotifySceneEvent(ESceneAction InAction)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FString ActionName;
		switch (InAction)
		{
		case ESceneAction::Created:     ActionName = TEXT("Created"); break;
		case ESceneAction::Activated:   ActionName = TEXT("Activated"); break;
		case ESceneAction::Deactivated: ActionName = TEXT("Deactivated"); break;
		}
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MotionDesign.Scene")
			, FAnalyticsEventAttribute(TEXT("Action"), MoveTemp(ActionName)));
	}
}
#endif

AAvaScene* AAvaScene::GetScene(ULevel* InLevel, bool bInCreateSceneIfNotFound)
{
	if (!IsValid(InLevel))
	{
		return nullptr;
	}

	AAvaScene* ExistingScene = nullptr;

	// Return the Existing Scene, or nullptr if not found and not creating a new scene
	if (InLevel->Actors.FindItemByClass<AAvaScene>(&ExistingScene) || !bInCreateSceneIfNotFound)
	{
		return ExistingScene;
	}

	UWorld* const World = InLevel->GetWorld();
	if (!IsValid(World))
	{
		return nullptr;
	}

	FActorSpawnParameters SpawnParameters;
	SpawnParameters.OverrideLevel = InLevel;
#if WITH_EDITOR
	SpawnParameters.bHideFromSceneOutliner = true;
#endif

	AAvaScene* const NewScene = World->SpawnActor<AAvaScene>(SpawnParameters);
#if WITH_EDITOR
	NotifySceneEvent(ESceneAction::Created);
#endif
	return NewScene;
}

AAvaScene::AAvaScene()
{
	SceneSettings = CreateDefaultSubobject<UAvaSceneSettings>(TEXT("SceneSettings"));

	AttributeContainer = CreateDefaultSubobject<UAvaAttributeContainer>(TEXT("SceneState"));

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SceneState = AttributeContainer;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

#if WITH_EDITOR
	if (!HasAnyFlags(EObjectFlags::RF_ClassDefaultObject))
	{
		PreWorldRenameDelegate = FWorldDelegates::OnPostWorldRename.AddUObject(this, &AAvaScene::OnWorldRenamed);
		WorldTagGetterDelegate = UObject::FAssetRegistryTag::OnGetExtraObjectTagsWithContext.AddUObject(this, &AAvaScene::OnGetWorldTags);
	}
#endif
}

IAvaSequencePlaybackObject* AAvaScene::GetScenePlayback() const
{
	// If the saved scene playback is valid use that
	if (IAvaSequencePlaybackObject* const ScenePlayback = PlaybackObject.GetInterface())
	{
		return ScenePlayback;
	}

	UAvaSequenceSubsystem* const SequenceSubsystem = UAvaSequenceSubsystem::Get(GetWorld());
	if (!SequenceSubsystem)
	{
		return nullptr;
	}

	AAvaScene& MutableThis = *const_cast<AAvaScene*>(this);
	if (IAvaSequencePlaybackObject* const ScenePlayback = SequenceSubsystem->FindOrCreatePlaybackObject(GetLevel(), MutableThis))
	{
		MutableThis.PlaybackObject.SetObject(ScenePlayback->ToUObject());
		MutableThis.PlaybackObject.SetInterface(ScenePlayback);
		return ScenePlayback;
	}

	return nullptr;
}

#if WITH_EDITOR
void AAvaScene::SetAutoStartMode(bool bInAutoStartMode)
{
	bAutoStartMode = bInAutoStartMode;
}

void AAvaScene::OnWorldRenamed(UWorld* InWorld)
{
	if (FUObjectThreadContext::Get().IsRoutingPostLoad || InWorld != GetWorld())
	{
		return;
	}

	for (UAvaSequence* Sequence : Animations)
	{
		if (Sequence)
		{
			Sequence->OnOuterWorldRenamed(this);
		}
	}
}

void AAvaScene::OnGetWorldTags(FAssetRegistryTagsContext Context) const
{
	const UObject* InWorld = Context.GetObject();
	if (InWorld != GetTypedOuter<UWorld>())
	{
		return;
	}

	using namespace UE::Ava;
	Context.AddTag(UObject::FAssetRegistryTag(AssetTags::MotionDesignScene, AssetTags::Values::Enabled, UObject::FAssetRegistryTag::TT_Alphabetical));
}
#endif

ULevel* AAvaScene::GetSceneLevel() const
{
	return GetLevel();
}

IAvaSequencePlaybackObject* AAvaScene::GetPlaybackObject() const
{
	return GetScenePlayback();
}

URemoteControlPreset* AAvaScene::GetOrCreateRemoteControlPreset(bool bInCallModify)
{
	if (!RemoteControlPreset)
	{
		if (bInCallModify)
		{
			Modify();
		}
		RemoteControlPreset = NewObject<URemoteControlPreset>(this);
		bRemoteControlPresetPresent = true;
		FAvaRemoteControlUtils::RegisterRemoteControlPreset(RemoteControlPreset, /*bInEnsureUniqueId*/ true);
	}
	return RemoteControlPreset;
}

URemoteControlPreset* AAvaScene::GetRemoteControlPreset() const
{
	return RemoteControlPreset;
}

void AAvaScene::DestroyRemoteControlPreset(bool bInCallModify)
{
	if (RemoteControlPreset)
	{
		if (bInCallModify)
		{
			Modify();
		}
		FAvaRemoteControlUtils::UnregisterRemoteControlPreset(RemoteControlPreset);
		RemoteControlPreset = nullptr;
		bRemoteControlPresetPresent = false;
	}
}

bool AAvaScene::IsRemoteControlPresetExpectedToBePresent() const
{
	return bRemoteControlPresetPresent;
}

UObject* AAvaScene::ToUObject()
{
	return this;
}

UWorld* AAvaScene::GetContextWorld() const
{
	return GetWorld();
}

bool AAvaScene::CreateDirectorInstance(UAvaSequence& InSequence
	, IMovieScenePlayer& InPlayer
	, const FMovieSceneSequenceID& InSequenceID
	, UObject*& OutDirectorInstance)
{
	// Allow ULevelSequence::CreateDirectorInstance to be called
	return false;
}

bool AAvaScene::AddSequence(UAvaSequence* InSequence)
{
	return AddSequences(MakeArrayView(&InSequence, 1)) > 0;
}

uint32 AAvaScene::AddSequences(TConstArrayView<UAvaSequence*> InSequences)
{
	uint32 AddedCount = 0;

	for (UAvaSequence* Sequence : InSequences)
	{
		if (Sequence && !Animations.Contains(Sequence))
		{
			Animations.Add(Sequence);
			++AddedCount;
		}
	}

	if (AddedCount == 0)
	{
		return 0;
	}

	ScheduleRebuildSequenceTree();

#if WITH_EDITOR
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MotionDesign.Sequence"), TEXT("Action"), TEXT("AddedSequence"));
	}
#endif
	return AddedCount;
}

void AAvaScene::RemoveSequence(UAvaSequence* InSequence)
{
	if (InSequence)
	{
		InSequence->Cleanup();
	}

	Animations.Remove(InSequence);
	ScheduleRebuildSequenceTree();
}

void AAvaScene::SetDefaultSequence(UAvaSequence* InSequence)
{
	if (IsValid(InSequence))
	{
		AddSequence(InSequence);
		DefaultSequenceIndex = Animations.Find(InSequence);
	}
}

UAvaSequence* AAvaScene::GetDefaultSequence() const
{
	if (Animations.IsValidIndex(DefaultSequenceIndex))
	{
		return Animations[DefaultSequenceIndex];
	}
	return nullptr;
}

FName AAvaScene::GetSequenceProviderDebugName() const
{
	return GetFName();
}

#if WITH_EDITOR
void AAvaScene::OnEditorSequencerCreated(const TSharedPtr<ISequencer>& InSequencer)
{
	EditorSequencer = InSequencer;
	RebuildSequenceTree();
}
#endif

void AAvaScene::ScheduleRebuildSequenceTree()
{
	// Bail if the Deferred Rebuild is pending and hasn't executed yet
	if (bPendingAnimTreeUpdate)
	{
		return;
	}

	bPendingAnimTreeUpdate = true;

	TWeakObjectPtr<AAvaScene> ThisWeak(this);

	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
		[ThisWeak](float InDeltaTime)->bool
		{
			AAvaScene* const This = ThisWeak.Get();

			// Check if we already have Rebuilt the Animation Tree in between when this was added and when it was executed
			if (This && This->bPendingAnimTreeUpdate)
			{
				This->RebuildSequenceTree();
			}

			// Return false for one time execution
			return false;
		}));
}

void AAvaScene::RebuildSequenceTree()
{
	bPendingAnimTreeUpdate = false;
	IAvaSequenceProvider::RebuildSequenceTree();
}

void AAvaScene::OnValuesApplied_Implementation()
{
	if (UAvaCameraSubsystem* CameraSubsystem = UAvaCameraSubsystem::Get(this))
	{
		CameraSubsystem->ConditionallyUpdateViewTarget(GetLevel());
	}
}

void AAvaScene::PostActorCreated()
{
	Super::PostActorCreated();

	// by default create an RC preset.
	GetOrCreateRemoteControlPreset();

	RegisterObjects();
}

void AAvaScene::PreInitializeComponents()
{
	Super::PreInitializeComponents();

	// Conditionally sort the items for faster lookup.
	SceneTree.SortItems();

	// Fill the object to node map for fast lookup
	SceneTree.ResolveObjects(GetTypedOuter<UWorld>());
}

void AAvaScene::BeginPlay()
{
	Super::BeginPlay();

	if (UAvaCameraSubsystem* CameraSubsystem = UAvaCameraSubsystem::Get(this))
	{
		CameraSubsystem->RegisterScene(GetLevel());
	}
}

void AAvaScene::EndPlay(const EEndPlayReason::Type InEndPlayReason)
{
	Super::EndPlay(InEndPlayReason);

	if (UAvaCameraSubsystem* CameraSubsystem = UAvaCameraSubsystem::Get(this))
	{
		CameraSubsystem->UnregisterScene(GetLevel());
	}
}

void AAvaScene::PostLoad()
{
	Super::PostLoad();

	if (RemoteControlPreset)
	{
		RemoteControlPreset->ClearFlags(RF_DefaultSubObject);
	}

	RegisterObjects();

	if (AttributeContainer)
	{
		AttributeContainer->Initialize(SceneSettings);
	}
}

void AAvaScene::PostInitializeComponents()
{
	Super::PostInitializeComponents();
	if (AttributeContainer)
	{
		AttributeContainer->Initialize(SceneSettings);
	}
}

void AAvaScene::PostDuplicate(EDuplicateMode::Type InDuplicateMode)
{
	Super::PostDuplicate(InDuplicateMode);
	RegisterObjects();
}

void AAvaScene::PostEditImport()
{
	Super::PostEditImport();
	RegisterObjects();
}

#if WITH_EDITOR
void AAvaScene::PostEditUndo()
{
	Super::PostEditUndo();

	// Ensure RC is registered
	FAvaRemoteControlUtils::RegisterRemoteControlPreset(RemoteControlPreset, /*bInEnsureUniqueId*/ true);
}
#endif

void AAvaScene::BeginDestroy()
{
	Super::BeginDestroy();

	UnregisterObjects();

#if WITH_EDITOR
	FWorldDelegates::OnPreWorldRename.Remove(PreWorldRenameDelegate);
	PreWorldRenameDelegate.Reset();

	UObject::FAssetRegistryTag::OnGetExtraObjectTagsWithContext.Remove(WorldTagGetterDelegate);
	WorldTagGetterDelegate.Reset();
#endif
}

void AAvaScene::RegisterObjects()
{
	if (!HasAnyFlags(EObjectFlags::RF_ClassDefaultObject))
	{
		FAvaRemoteControlUtils::RegisterRemoteControlPreset(RemoteControlPreset, /*bInEnsureUniqueId*/ true);
	}

	// Register AAvaScenes created past Subsystem Initialization
	if (UAvaSceneSubsystem* SceneSubsystem = FAvaWorldSubsystemUtils::GetWorldSubsystem<UAvaSceneSubsystem>(this))
	{
		SceneSubsystem->RegisterSceneInterface(GetLevel(), this);
	}

	if (UAvaSequenceSubsystem* SequenceSubsystem = FAvaWorldSubsystemUtils::GetWorldSubsystem<UAvaSequenceSubsystem>(this))
	{
		SequenceSubsystem->RegisterSequenceProvider(GetLevel(), this);
	}
}

void AAvaScene::UnregisterObjects()
{
	if (!HasAnyFlags(EObjectFlags::RF_ClassDefaultObject))
	{
		FAvaRemoteControlUtils::UnregisterRemoteControlPreset(RemoteControlPreset);
	}

	if (UAvaSequenceSubsystem* SequenceSubsystem = FAvaWorldSubsystemUtils::GetWorldSubsystem<UAvaSequenceSubsystem>(this))
	{
		SequenceSubsystem->UnregisterSequenceProvider(GetLevel(), this);
	}
}

#undef LOCTEXT_NAMESPACE
