// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneSubsystem.h"
#include "AvaScene.h"
#include "Containers/Set.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "IAvaSceneInterface.h"
#include "IText3DBuildSystem.h"
#include "Utilities/Text3DUtilities.h"

namespace UE::Ava::Private
{
	static const TSet<EWorldType::Type> GUnsupportedWorlds
		{
			EWorldType::Type::None,
			EWorldType::Type::Inactive,
			EWorldType::Type::EditorPreview,
		};

	const ULevel* GetLevel(const UObject* InContextObject, const UWorld* InWorld)
	{
		if (InContextObject)
		{
			if (const ULevel* ContextLevel = Cast<ULevel>(InContextObject))
			{
				return ContextLevel;
			}
			if (const ULevel* OuterLevel = InContextObject->GetTypedOuter<ULevel>())
			{
				return OuterLevel;
			}
		}
		return InWorld ? InWorld->PersistentLevel : nullptr;
	}
}

void UAvaSceneSubsystem::RegisterSceneInterface(ULevel* InLevel, IAvaSceneInterface* InSceneInterface)
{
	// Replace the existing Interface with new one for the given Level
	SceneInterfaces.Add(InLevel, InSceneInterface);
}

FTSTicker& UAvaSceneSubsystem::GetTicker()
{
	return Ticker;
}

IAvaSceneInterface* UAvaSceneSubsystem::GetSceneInterface() const
{
	if (UWorld* const World = GetWorld())
	{
		return GetSceneInterface(World->PersistentLevel);
	}
	return nullptr;
}

IAvaSceneInterface* UAvaSceneSubsystem::GetSceneInterface(const ULevel* InLevel) const
{
	return SceneInterfaces.FindRef(InLevel).Get();
}

IAvaSceneInterface* UAvaSceneSubsystem::FindSceneInterface(const ULevel* InLevel)
{
	if (!InLevel)
	{
		return nullptr;
	}

	if (const UAvaSceneSubsystem* SceneSubsystem = InLevel->OwningWorld ? InLevel->OwningWorld->GetSubsystem<UAvaSceneSubsystem>() : nullptr)
	{
		if (IAvaSceneInterface* SceneInterface = SceneSubsystem->GetSceneInterface(InLevel))
		{
			return SceneInterface;
		}
	}

	AAvaScene* SceneActor = nullptr;
	InLevel->Actors.FindItemByClass(&SceneActor);
	return SceneActor;
}

bool UAvaSceneSubsystem::IsReadyToPlay(const UObject* InContextObject) const
{
	const IText3DBuildSystemInterface* const TextBuildSystem = UE::Text3D::Utilities::FindBuildSystem(this);
	return !TextBuildSystem || !TextBuildSystem->IsBuildInProgress();
}

void UAvaSceneSubsystem::FlushBuilds()
{
	if (IText3DBuildSystemInterface* const TextBuildSystem = UE::Text3D::Utilities::FindBuildSystem(this))
	{
		TextBuildSystem->FlushBuilds();
	}
}

TArray<AActor*> UAvaSceneSubsystem::GatherSceneTreeActors(const UObject* InContextObject, const AActor* InParentActor, bool bInIncludeDescendants)
{
	UWorld* const World = GEngine ? GEngine->GetWorldFromContextObject(InContextObject, EGetWorldErrorMode::LogAndReturnNull) : nullptr;
	if (!World)
	{
		return {};
	}

	const ULevel* const Level = UE::Ava::Private::GetLevel(InContextObject, World);
	if (!Level)
	{
		return {};
	}

	IAvaSceneInterface* const SceneInterface = UAvaSceneSubsystem::FindSceneInterface(Level);
	if (!SceneInterface)
	{
		return {};
	}

	FAvaSceneTree& SceneTree = SceneInterface->GetSceneTree();

	TArray<AActor*> Actors;
	if (InParentActor)
	{
		InParentActor->GetAttachedActors(Actors, /*bResetArray*/false, bInIncludeDescendants);
	}
	else
	{
		Actors = Level->Actors;	
	}

	if (!SceneTree.IsSorted())
	{
		SceneTree.SortItems();
		SceneTree.ResolveObjects(World);
	}

	Actors.Remove(nullptr);
	Actors.Sort(
		[&SceneTree, World](const AActor& A, const AActor& B)
		{
			const FAvaSceneTreeNode* NodeA = SceneTree.FindObjectTreeNode(&A, World);
			const FAvaSceneTreeNode* NodeB = SceneTree.FindObjectTreeNode(&B, World);
			return FAvaSceneTree::CompareTreeItemOrder(NodeA, NodeB);
		});

	return Actors;
}

void UAvaSceneSubsystem::PostInitialize()
{
	Super::PostInitialize();

	UWorld* const World = GetWorld();
	if (!World)
	{
		return;
	}

	// Register Default Scene Interfaces
	for (FConstLevelIterator LevelIterator = World->GetLevelIterator(); LevelIterator; ++LevelIterator)
	{
		ULevel* const Level = *LevelIterator;

		AAvaScene* SceneActor = nullptr;
		if (Level && Level->Actors.FindItemByClass(&SceneActor))
		{
			SceneActor->RegisterObjects();
		}
	}
}

bool UAvaSceneSubsystem::DoesSupportWorldType(const EWorldType::Type InWorldType) const
{
	const bool bDisallowedWorld = UE::Ava::Private::GUnsupportedWorlds.Contains(InWorldType);
	return !bDisallowedWorld;
}

bool UAvaSceneSubsystem::IsTickableInEditor() const
{
	return true;
}

void UAvaSceneSubsystem::Tick(float InDeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAvaSceneSubsystem::Tick);
	Ticker.Tick(InDeltaTime);
}

TStatId UAvaSceneSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UAvaSceneSubsystem, STATGROUP_Tickables);
}
