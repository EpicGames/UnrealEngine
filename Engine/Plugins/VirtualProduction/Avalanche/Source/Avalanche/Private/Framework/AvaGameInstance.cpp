// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/AvaGameInstance.h"
#include "AudioDevice.h"
#include "AvaLog.h"
#include "AvaRemoteControlRebind.h"
#include "Components/ReflectionCaptureComponent.h"
#include "Components/SkyLightComponent.h"
#include "Engine/Console.h"
#include "Engine/Engine.h"
#include "Engine/LocalPlayer.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/TimeGuard.h"
#include "Slate/SceneViewport.h"
#include "Viewport/AvaViewportQualitySettings.h"

#if WITH_EDITOR
#include "Editor.h"
#include "TickableEditorObject.h"
#endif

#if WITH_EDITOR
/**
 * Editor Ticker Object used to tick the Game Instance World. 
 * Used instead of GEditor->PostEditorTick so that the world ticks before viewport ticks.
 * FTickableGameObject::TickObjects(null, ...) is called in Editor Engine conditionally, so cannot be used as a way to tick the Game Instance World in editor.
 * In addition, other ticking logic could rely on TickObjects(null, ...) to happen after worlds have ticked. 
 */
class FAvaGameInstanceEditorTicker : public FTickableEditorObject
{
public:
	explicit FAvaGameInstanceEditorTicker(TDelegate<void(float)>&& InDelegate)
		: TickDelegate(MoveTemp(InDelegate))
	{
	}

	//~ Begin FTickableObjectBase
	virtual ETickableTickType GetTickableTickType() const override
	{
		return ETickableTickType::Always;
	}

	virtual void Tick(float InDeltaTime) override
	{
		TickDelegate.ExecuteIfBound(InDeltaTime);
	}

	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAvaGameInstanceEditorTicker, STATGROUP_Tickables);
	}
	//~ End FTickableObjectBase

private:
	TDelegate<void(float)> TickDelegate;
};
#endif

UAvaGameInstance::FOnAvaGameInstanceEvent UAvaGameInstance::OnEndPlay;
UAvaGameInstance::FOnAvaGameInstanceEvent UAvaGameInstance::OnRenderTargetReady;

UAvaGameInstance::UAvaGameInstance()
	: FTickableGameObject(ETickableTickType::Never) // Register tickable when it is needed
{
}

UAvaGameInstance* UAvaGameInstance::Create(UObject* InOuter)
{
	checkf(IsInGameThread(), TEXT("Motion Design Game Instance creation can only happen in Game Thread"));

	UAvaGameInstance* GameInstance = NewObject<UAvaGameInstance>(InOuter ? InOuter : GEngine);

	// We need to call Init() (and corresponding Shutdown() later) to get SubsystemCollection to initialize.
	// Not doing it because it might have side effects.
	//GameInstance->Init();

	GameInstance->CreateWorld();
	return GameInstance;
}

bool UAvaGameInstance::CreateWorld()
{
	if (bWorldCreated)
	{
		return false;
	}
	
	TRACE_CPUPROFILER_EVENT_SCOPE(UAvaGameInstance::CreateWorld);

	if (!EnginePreExitHandle.IsValid())
	{
		EnginePreExitHandle = FCoreDelegates::OnEnginePreExit.AddUObject(this, &UAvaGameInstance::OnEnginePreExit);
	}
	
	constexpr EWorldType::Type WorldType = EWorldType::Game;
	
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UAvaGameInstance::LoadInstance::InitWorld);
		
		const FName MotionDesignWorldName = MakeUniqueObjectName(GetOuter(), UWorld::StaticClass(), TEXT("AvaGameInstanceWorld"));

		PlayWorld = NewObject<UWorld>(this, MotionDesignWorldName, RF_Transient);
		check(PlayWorld);
		PlayWorld->WorldType = WorldType;

		PlayWorld->InitializeNewWorld(UWorld::InitializationValues()
			.AllowAudioPlayback(true)
			.CreatePhysicsScene(true)
			.RequiresHitProxies(false)
			.CreateNavigation(true)
			.CreateAISystem(false)	// Disabling AI System until supported in Motion Design.
			.ShouldSimulatePhysics(true)
			.SetTransactional(false)
			//TODO: World Settings
		);
	}

	WorldContext = &GEngine->CreateNewWorldContext(WorldType);
	check(WorldContext);
	WorldContext->OwningGameInstance = this;
	WorldContext->SetCurrentWorld(PlayWorld.Get());

	PlayWorld->SetGameInstance(this);
	PlayWorld->SetGameMode(FURL());

	bWorldCreated = true;
	return true;
}

bool UAvaGameInstance::BeginPlayWorld(const FAvaInstancePlaySettings& InWorldPlaySettings)
{
	// Make sure we don't have pending unload or stop requests left over in the game instance. 
	CancelWorldRequests();
	
	if (bWorldPlaying)
	{
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UAvaGameInstance::BeginPlay);
	TRACE_BOOKMARK(TEXT("UAvaGameInstance::BeginPlay"));

	ViewportClient = NewObject<UAvaGameViewportClient>(GEngine, NAME_None, RF_Transient);
	check(ViewportClient);

	WorldContext->GameViewport = ViewportClient;

	// Note: UGameViewportClient::Init ignores "bCreateNewAudioDevice" parameter and always create a device for the world. 
	ViewportClient->Init(*WorldContext, this);
	ViewportClient->SetRenderTarget(InWorldPlaySettings.RenderTarget);
	ViewportClient->bIsPlayInEditorViewport = false;
	FAvaViewportQualitySettings QualitySettingsMutable = InWorldPlaySettings.QualitySettings; 
	QualitySettingsMutable.Apply(ViewportClient->EngineShowFlags);
	
#if ALLOW_CONSOLE
	// Create the viewport's console.
	ViewportClient->ViewportConsole = NewObject<UConsole>(ViewportClient.Get(), GEngine->ConsoleClass);
	// register console to get all log messages
	GLog->AddOutputDevice(ViewportClient->ViewportConsole);
#endif

	TSharedPtr<SViewport> ViewportWidget;
	Viewport = ViewportClient->CreateViewport(ViewportWidget);
	Viewport->SetInitialSize(InWorldPlaySettings.ViewportSize);
	ViewportClient->Viewport = Viewport.Get();

	// Attempt to initialize a Local Player.
	FString Error;
	LocalPlayer = ViewportClient->SetupInitialLocalPlayer(Error);
	if (!LocalPlayer)
	{
		UE_LOGF(LogAva, Error, "Couldn't create initial local player: %ls", *Error);
	}

	PlayWorld->InitializeActorsForPlay(PlayWorld->URL);

	if (!LocalPlayer->SpawnPlayActor(TEXT(""), Error, PlayWorld))
	{
		UE_LOGF(LogAva, Error, "Couldn't spawn play actor: %ls", *Error);
	}

#if WITH_EDITOR
	if (GEditor)
	{
		WorldTickerEditor = MakePimpl<FAvaGameInstanceEditorTicker>(TDelegate<void(float)>::CreateUObject(this, &UAvaGameInstance::EditorTick));
	}
	else
	{
		WorldTickerEditor.Reset();
	}
#endif // WITH_EDITOR

	// Register game instance to tick
	SetTickableTickType(ETickableTickType::Always);

	PlayWorld->BeginPlay();

	FCoreDelegates::OnEndFrame.AddUObject(this, &UAvaGameInstance::OnEndFrameTick);

	bWorldPlaying = true;
	PlayingChannelName = InWorldPlaySettings.ChannelName;
	return true;
}

void UAvaGameInstance::RequestEndPlayWorld(bool bForceImmediate)
{
	if (bWorldPlaying)
	{
		if (bForceImmediate)
		{
			EndPlayWorld();
		}
		else
		{
			bRequestEndPlayWorld = true;
		}
	}
}

void UAvaGameInstance::RequestUnloadWorld(bool bForceImmediate)
{
	if (bWorldCreated)
	{
		if (bForceImmediate)
		{
			UnloadWorld(/*bShutdown*/true);
		}
		else
		{
			bRequestUnloadWorld = true;
		}
	}
}

void UAvaGameInstance::CancelWorldRequests()
{
	bRequestEndPlayWorld = false;
	bRequestUnloadWorld = false;
}

void UAvaGameInstance::UpdateRenderTarget(UTextureRenderTarget2D* InRenderTarget)
{
	if (ViewportClient)
	{
		ViewportClient->SetRenderTarget(InRenderTarget);
	}
}

UTextureRenderTarget2D* UAvaGameInstance::GetRenderTarget() const
{
	if (ViewportClient)
	{
		return ViewportClient->GetRenderTarget();
	}
	return nullptr;
}

void UAvaGameInstance::UpdateSceneViewportSize(const FIntPoint& InViewportSize)
{
	if (Viewport.IsValid() && Viewport->GetSize() != InViewportSize)
	{
		// Note: calling either SetViewportSize or SetFixedViewportSize doesn't work
		// because ViewportWidget is null in this case. And we can't call ResizeViewport
		// either because it is private. Calling the only function we can call.		
		Viewport->UpdateViewportRHI(false, InViewportSize.X, InViewportSize.Y, EWindowMode::Type::Windowed, PF_Unknown);
	}
}

#if WITH_EDITOR
void UAvaGameInstance::EditorTick(float InDeltaSeconds)
{
	if (!PlayWorld)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UAvaGameInstance::EditorTick);
	PlayWorld->Tick(ELevelTick::LEVELTICK_All, InDeltaSeconds);

	// Only update reflection captures in game once all 'always loaded' levels have been loaded
	// This won't work with actual level streaming though
	if (PlayWorld->AreAlwaysLoadedLevelsLoaded())
	{
		// Update sky light first because it's considered direct lighting, sky diffuse will be visible in reflection capture indirect specular
		USkyLightComponent::UpdateSkyCaptureContents(PlayWorld);
		UReflectionCaptureComponent::UpdateReflectionCaptureContents(PlayWorld);
	}
}
#endif // WITH_EDITOR

void UAvaGameInstance::TickInternal(float InDeltaSeconds)
{
	// Skip if world not playing, or it has already ticked this frame
	if (!bWorldPlaying || TickedFrame == GFrameCounter)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UAvaGameInstance::Tick);
	TickedFrame = GFrameCounter;

	if (!PlayWorld)
	{
		RequestEndPlayWorld(/*bForceImmediate*/false);
		return;
	}

	ViewportClient->Tick(InDeltaSeconds);
	Viewport->Draw();

	if (!bIsRenderTargetReady)
	{
		if (!RenderTargetFence.IsValid())
		{
			RenderTargetFence = MakeUnique<FRenderCommandFence>();
			RenderTargetFence->BeginFence();
		}
		else
		{
			bIsRenderTargetReady = RenderTargetFence->IsFenceComplete();
			OnRenderTargetReady.Broadcast(this, PlayingChannelName);
		}
	}
}

void UAvaGameInstance::UnloadWorld(bool bInShutdown)
{
	if (bWorldPlaying)
	{
		EndPlayWorld();
	}

	bWorldCreated = false;
	bRequestUnloadWorld = false;

	if (PlayWorld)
	{
		if (FAudioDeviceHandle AudioDevice = PlayWorld->GetAudioDevice())
		{
			AudioDevice->Flush(PlayWorld, false);
		}

		// Need to do this before destroying the world context apparently.
		PlayWorld->SetShouldForceUnloadStreamingLevels(true);
		PlayWorld->FlushLevelStreaming();

		GEngine->DestroyWorldContext(PlayWorld.Get());
		PlayWorld->DestroyWorld(true);
	}

	if (bInShutdown)
	{
		Shutdown();
	}

	PlayWorld = nullptr;
	WorldContext = nullptr;
}

void UAvaGameInstance::EndPlayWorld()
{
	bWorldPlaying = false;
	bRequestEndPlayWorld = false;

	SetTickableTickType(ETickableTickType::Never);

#if WITH_EDITOR
	WorldTickerEditor.Reset();
#endif

	FCoreDelegates::OnEndFrame.RemoveAll(this);

	OnEndPlay.Broadcast(this, PlayingChannelName);
	PlayingChannelName = NAME_None;

	if (LocalPlayer)
	{
		RemoveLocalPlayer(LocalPlayer);
		LocalPlayer = nullptr;
	}

	if (PlayWorld)
	{
		PlayWorld->EndPlay(EEndPlayReason::LevelTransition);
	}

	Viewport.Reset();
	RenderTargetFence.Reset();
	bIsRenderTargetReady = false;

#if ALLOW_CONSOLE
	if (ViewportClient)
	{
		GLog->RemoveOutputDevice(ViewportClient->ViewportConsole);
	}
#endif

	ViewportClient = nullptr;
}

void UAvaGameInstance::OnEndFrameTick()
{
	if (bRequestEndPlayWorld)
	{
		EndPlayWorld();
	}
	if (bRequestUnloadWorld)
	{
		UnloadWorld(/*bShutdown*/true);
	}
}

void UAvaGameInstance::OnEnginePreExit()
{
	if (PlayWorld && PlayWorld->GetAudioDevice())
	{
		// Temporarily set the GIsRequestingExit to false so that the DLLs like XAudio2Dll are not unloaded
		// for ref: FMixerPlatformXAudio2::TeardownHardware unloads the dll if Engine Exit is requested
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TGuardValue<bool> RequestExitGuard(GIsRequestingExit, false);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		const FAudioDeviceHandle EmptyHandle;
		PlayWorld->SetAudioDevice(EmptyHandle);
	}
}

void UAvaGameInstance::Tick(float InDeltaSeconds)
{
	TickInternal(InDeltaSeconds);
}

TStatId UAvaGameInstance::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UAvaGameInstance, STATGROUP_Tickables)
}

bool UAvaGameInstance::IsTickableInEditor() const
{
	return true;
}

UWorld* UAvaGameInstance::GetTickableGameObjectWorld() const
{
	// Explicitly set this to null as the game instance should tick outside of the PlayWorld ticking.
	return nullptr;
}

ULocalPlayer* UAvaGameInstance::CreateInitialPlayer(FString& OutError)
{
	if (!GetGameViewportClient())
	{
		if (ensure(IsDedicatedServerInstance()))
		{
			OutError = FString::Printf(TEXT("Dedicated servers cannot have local players"));
			return nullptr;
		}
	}

	const FPlatformUserId UserId = IPlatformInputDeviceMapper::Get().GetPrimaryPlatformUser();

	if (ULocalPlayer* ExistingLocalPlayer = FindLocalPlayerFromPlatformUserId(UserId))
	{
		return ExistingLocalPlayer;
	}

	if (LocalPlayers.IsEmpty())
	{
		ULocalPlayer* NewLocalPlayer = NewObject<ULocalPlayer>(GEngine, GEngine->LocalPlayerClass);
		AddLocalPlayer(NewLocalPlayer, UserId);
		return NewLocalPlayer;
	}

	return LocalPlayers[0];
}

void UAvaGameInstance::BeginDestroy()
{
	FCoreDelegates::OnEnginePreExit.Remove(EnginePreExitHandle);
	EnginePreExitHandle.Reset();

	EndPlayWorld();
	UnloadWorld(/*bShutdown*/false);

	Super::BeginDestroy();
}
