// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/AvaGameViewportClient.h"

#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "Camera/CameraComponent.h"
#include "Camera/CameraPhotography.h"
#include "Components/LineBatchComponent.h"
#include "ContentStreaming.h"
#include "DynamicResolutionState.h"
#include "Engine/Canvas.h"
#include "Engine/CoreSettings.h"
#include "Engine/LocalPlayer.h"
#include "Engine/TextureRenderTarget2D.h"
#include "EngineModule.h"
#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/PlayerController.h"
#include "IAvaModule.h"
#include "IHeadMountedDisplay.h"
#include "IXRCamera.h"
#include "IXRTrackingSystem.h"
#include "LegacyScreenPercentageDriver.h"
#include "Misc/EngineVersionComparison.h"
#include "SceneManagement.h"
#include "SceneViewExtension.h"
#include "TextureResource.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UnrealEngine.h"

/** Util to find named canvas in transient package, and create if not found */
static UCanvas* GetCanvasByName(FName CanvasName)
{
	// Cache to avoid FString/FName conversions/compares
	static TMap<FName, UCanvas*> CanvasMap;
	UCanvas** FoundCanvas = CanvasMap.Find(CanvasName);
	if (!FoundCanvas)
	{
		UCanvas* CanvasObject = FindObject<UCanvas>(GetTransientPackage(), *CanvasName.ToString(), EFindObjectFlags::None);
		if (!CanvasObject)
		{
			CanvasObject = NewObject<UCanvas>(GetTransientPackage(), CanvasName);
			CanvasObject->AddToRoot();
		}

		CanvasMap.Add(CanvasName, CanvasObject);
		return CanvasObject;
	}

	return *FoundCanvas;
}

void UAvaGameViewportClient::Draw(FViewport* InViewport, FCanvas* InCanvas)
{
	if (!GEngine)
	{
		return;
	}

	// Override the Canvas Render Target with ours
	if (RenderTarget.IsValid())
	{
		InCanvas->SetRenderTarget_GameThread(RenderTarget->GameThread_GetRenderTargetResource());
	}

	// Allow HMD to modify the view later, just before rendering
	const bool bStereoRendering = GEngine->IsStereoscopic3D(InViewport);

	// Create a temporary canvas if there isn't already one.
	static FName CanvasObjectName(TEXT("CanvasObject"));
	UCanvas* CanvasObject = GetCanvasByName(CanvasObjectName);
	CanvasObject->Canvas = InCanvas;

	// Create temp debug canvas object
	FCanvas* DebugCanvas = InViewport->GetDebugCanvas();
	FIntPoint DebugCanvasSize = InViewport->GetSizeXY();
	if (bStereoRendering && GEngine->XRSystem.IsValid() && GEngine->XRSystem->GetHMDDevice())
	{
		DebugCanvasSize = GEngine->XRSystem->GetHMDDevice()->GetIdealDebugCanvasRenderTargetSize();
	}

	static FName DebugCanvasObjectName(TEXT("DebugCanvasObject"));
	UCanvas* DebugCanvasObject = GetCanvasByName(DebugCanvasObjectName);
	DebugCanvasObject->Init(DebugCanvasSize.X, DebugCanvasSize.Y, nullptr, DebugCanvas);
	if (DebugCanvas)
	{
		DebugCanvas->SetScaledToRenderTarget(bStereoRendering);
		DebugCanvas->SetStereoRendering(bStereoRendering);
	}
	InCanvas->SetScaledToRenderTarget(bStereoRendering);
	InCanvas->SetStereoRendering(bStereoRendering);

	if (!World)
	{
		return;
	}

	constexpr bool bCaptureNeedsSceneColor = false;
	constexpr ESceneCaptureSource SceneCaptureSource = ESceneCaptureSource::SCS_FinalColorLDR;
	constexpr ESceneCaptureCompositeMode SceneCaptureCompositeMode = ESceneCaptureCompositeMode::SCCM_Overwrite;

	/** When enabled, the scene capture will composite into the render target instead of overwriting its contents. */

	//TODO: World->GetTime() seems to return zeros because we have not Begun Play...
	FGameTime Time = FGameTime::GetTimeSinceAppStart();

	// Setup a FSceneViewFamily/FSceneView for the viewport.
	FSceneViewFamilyContext ViewFamilyContext(FSceneViewFamily::ConstructionValues(InCanvas->GetRenderTarget()
		, World->Scene
		, EngineShowFlags)
		.SetRealtimeUpdate(true)
		.SetResolveScene(!bCaptureNeedsSceneColor)
		.SetTime(Time));

	ViewFamilyContext.SceneCaptureSource = SceneCaptureSource;
	ViewFamilyContext.SceneCaptureCompositeMode = SceneCaptureCompositeMode;
	ViewFamilyContext.DebugDPIScale = GetDPIScale();
	ViewFamilyContext.EngineShowFlags = EngineShowFlags;

#if WITH_EDITOR
	if (GIsEditor)
	{
		// Force enable view family show flag for HighDPI derived's screen percentage.
		ViewFamilyContext.EngineShowFlags.ScreenPercentage = true;
	}
#endif

	if (!bStereoRendering)
	{
		// stereo is enabled, as many HMDs require this for proper visuals
		ViewFamilyContext.EngineShowFlags.SetScreenPercentage(false);
	}

	ViewFamilyContext.ViewExtensions = GEngine->ViewExtensions->GatherActiveExtensions(FSceneViewExtensionContext(InViewport));
	
	for (const TSharedRef<ISceneViewExtension>& ViewExt : ViewFamilyContext.ViewExtensions)
	{
		ViewExt->SetupViewFamily(ViewFamilyContext);
	}

	if (bStereoRendering && GEngine->XRSystem.IsValid() && GEngine->XRSystem->GetHMDDevice())
	{
		// Allow HMD to modify screen settings
		GEngine->XRSystem->GetHMDDevice()->UpdateScreenSettings(InViewport);
	}

	ViewFamilyContext.ViewMode = EViewModeIndex::VMI_Lit;
	EngineShowFlagOverride(ESFIM_Game, ViewFamilyContext.ViewMode, ViewFamilyContext.EngineShowFlags, false);

	bool bFinalScreenPercentageShowFlag;
	bool bUsesDynamicResolution = false;
	// Setup the screen percentage and upscaling method for the view family.
	{
		checkf(ViewFamilyContext.GetScreenPercentageInterface() == nullptr,
			TEXT("Some code has tried to set up an alien screen percentage driver, that could be wrong if not supported very well by the RHI."));

		// Force screen percentage show flag to be turned off if not supported.
		if (!ViewFamilyContext.SupportsScreenPercentage())
		{
			ViewFamilyContext.EngineShowFlags.ScreenPercentage = false;
		}

		// Set up secondary resolution fraction for the view family.
		if (!bStereoRendering && ViewFamilyContext.SupportsScreenPercentage())
		{
			// Automatically compute secondary resolution fraction from DPI.
			ViewFamilyContext.SecondaryViewFraction = GetDPIDerivedResolutionFraction();
		}

		bFinalScreenPercentageShowFlag = ViewFamilyContext.EngineShowFlags.ScreenPercentage;
	}

	TArray<FSceneView*> Views;
	bool bAnyPlayerCameraCut = false;

	for (FLocalPlayerIterator PlayerIterator(GEngine, World); PlayerIterator; ++PlayerIterator)
	{
		ULocalPlayer* LocalPlayer = *PlayerIterator;
		if (!LocalPlayer)
		{
			continue;
		}

		const bool bEnableStereo = GEngine->IsStereoscopic3D(InViewport);
		const int32 NumViews = bStereoRendering ? GEngine->StereoRenderingDevice->GetDesiredNumberOfViews(bStereoRendering) : 1;

		for (int32 ViewIndex = 0; ViewIndex < NumViews; ++ViewIndex)
		{
			// Calculate the player's view information.
			FVector ViewLocation;
			FRotator ViewRotation;

			FSceneView* View = LocalPlayer->CalcSceneView(&ViewFamilyContext
				, ViewLocation
				, ViewRotation
				, InViewport
				, nullptr
				, bStereoRendering ? ViewIndex : INDEX_NONE);

			if (View)
			{
				Views.Add(View);

				// If this is the primary drawing pass, update things that depend on the view location
				if (ViewIndex == 0)
				{
					// Save the location of the view.
					LocalPlayer->LastViewLocation = ViewLocation;
				}

				View->CameraConstrainedViewRect = View->UnscaledViewRect;

				// Add view information for resource streaming. Allow up to 5X boost for small FOV.
				const float StreamingScale = 1.f / FMath::Clamp<float>(View->LODDistanceFactor, .2f, 1.f);

				IStreamingManager::Get().AddViewInformation(View->ViewMatrices.GetViewOrigin()
					, View->UnscaledViewRect.Width()
					, View->UnscaledViewRect.Width() * View->ViewMatrices.GetViewToClip().M[0][0]
					, StreamingScale);

				World->ViewLocationsRenderedLastFrame.Add(View->ViewMatrices.GetViewOrigin());

				FWorldCachedViewInfo& WorldViewInfo = World->CachedViewInfoRenderedLastFrame.AddDefaulted_GetRef();
				WorldViewInfo.ViewMatrix           = View->ViewMatrices.GetWorldToView();
				WorldViewInfo.ProjectionMatrix     = View->ViewMatrices.GetViewToClip();
				WorldViewInfo.ViewProjectionMatrix = View->ViewMatrices.GetWorldToClip();
				WorldViewInfo.ViewToWorld          = View->ViewMatrices.GetViewToWorld();

				World->LastRenderTime = World->GetTimeSeconds();
			}
		}

		if (const APlayerController* const PlayerController = LocalPlayer->PlayerController)
		{
			if (APlayerCameraManager* const CameraManager = PlayerController->PlayerCameraManager)
			{
				bAnyPlayerCameraCut = bAnyPlayerCameraCut || CameraManager->bGameCameraCutThisFrame;
				// Assumes any other systems that call ULocalPlayer::CalcSceneView(...) and care about camera cuts will do so before this happens.
				CameraManager->bGameCameraCutThisFrame = false;
			}
		}
	}

	InViewport->SetCameraCut(bAnyPlayerCameraCut);

	InCanvas->Clear(FLinearColor::Transparent);

	// If a screen percentage interface was not set by one of the view extension, then set the legacy one.
	if (ViewFamilyContext.GetScreenPercentageInterface() == nullptr)
	{
		float GlobalResolutionFraction = 1.0f;

		if (ViewFamilyContext.EngineShowFlags.ScreenPercentage && !bDisableWorldRendering && ViewFamilyContext.Views.Num() > 0)
		{
			// Get global view fraction.
			FStaticResolutionFractionHeuristic StaticHeuristic;
			StaticHeuristic.Settings.PullRunTimeRenderingSettings(GetViewStatusForScreenPercentage());
			StaticHeuristic.PullViewFamilyRenderingSettings(ViewFamilyContext);
			StaticHeuristic.DPIScale = GetDPIScale();

			GlobalResolutionFraction = StaticHeuristic.ResolveResolutionFraction();
		}

		ViewFamilyContext.SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(
			ViewFamilyContext, GlobalResolutionFraction));
	}

	check(ViewFamilyContext.GetScreenPercentageInterface() != nullptr);

	// Make sure the engine show flag for screen percentage is still what it was when setting up the screen percentage interface
	ViewFamilyContext.EngineShowFlags.ScreenPercentage = bFinalScreenPercentageShowFlag;

	if (bStereoRendering && bUsesDynamicResolution)
	{
		// Change screen percentage method to raw output when doing dynamic resolution with VR if not using TAA upsample.
		for (FSceneView* const View : Views)
		{
			if (View->PrimaryScreenPercentageMethod == EPrimaryScreenPercentageMethod::SpatialUpscale)
			{
				View->PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::RawOutput;
			}
		}
	}

	ViewFamilyContext.bIsHDR = InViewport->IsHDRViewport();

	if (!bDisableWorldRendering && FSlateApplication::Get().GetPlatformApplication()->IsAllowedToRender())
	{
		for (const TSharedRef<ISceneViewExtension>& ViewExt : ViewFamilyContext.ViewExtensions)
		{
			for (FSceneView* const View : Views)
			{
				ViewExt->SetupView(ViewFamilyContext, *View);
			}
		}
		GetRendererModule().BeginRenderingViewFamily(InCanvas, &ViewFamilyContext);
	}
	else
	{
		GetRendererModule().PerFrameCleanupIfSkipRenderer();
	}

	// Remove temporary debug lines.
	constexpr const UWorld::ELineBatcherType LineBatchersToFlush[] = { UWorld::ELineBatcherType::World, UWorld::ELineBatcherType::Foreground };
	World->FlushLineBatchers(LineBatchersToFlush);

	// Render Stats HUD in the main canvas so that it gets captured
	// and is displayed in the broadcast channel's outputs.
	if (!Views.IsEmpty() && IAvaModule::Get().ShouldShowRuntimeStats())
	{
		DrawStatsHUD( World, InViewport, InCanvas, nullptr, DebugProperties, Views[0]->ViewLocation, Views[0]->ViewRotation);
	}

	// Ensure canvas has been flushed before rendering UI
	InCanvas->Flush_GameThread();
}

bool UAvaGameViewportClient::IsStatEnabled(const FString& InName) const
{
	// The IAvaModule holds the runtime stats. We want them persistent across all viewports.
	return IAvaModule::Get().IsRuntimeStatEnabled(InName);
}

void UAvaGameViewportClient::AddReferencedObjects(UObject* InThis, FReferenceCollector& InCollector)
{
	UAvaGameViewportClient* This = CastChecked<UAvaGameViewportClient>(InThis);
	for (FSceneViewStateReference& ViewState : This->ViewStates)
	{
		if (ViewState.GetReference())
		{
			ViewState.GetReference()->AddReferencedObjects(InCollector);
		}
	}
	Super::AddReferencedObjects(InThis, InCollector);
}

void UAvaGameViewportClient::SetRenderTarget(UTextureRenderTarget2D* InRenderTarget)
{
	RenderTarget = InRenderTarget;
}
