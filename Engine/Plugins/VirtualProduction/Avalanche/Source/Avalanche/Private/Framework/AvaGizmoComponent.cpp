// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/AvaGizmoComponent.h"

#include "Async/Async.h"
#include "AvaLog.h"
#include "Components/DynamicMeshComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Materials/MaterialInterface.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/Package.h"
#include "UObject/UObjectThreadContext.h"
#include "Viewport/Interaction/IAvaGizmoObject.h"

#if WITH_EDITOR
#include "LevelEditorViewport.h"
#endif

UAvaGizmoComponent::UAvaGizmoComponent()
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = false;

	static constexpr const TCHAR* DefaultMaterialPath = TEXT("Material'/Avalanche/M_SeeThrough.M_SeeThrough'");
	static ConstructorHelpers::FObjectFinderOptional<UMaterialInterface> DefaultMaterial(DefaultMaterialPath);
	if (DefaultMaterial.Succeeded())
	{
		Material = DefaultMaterial.Get();
	}
	else
	{
		UE_LOGF(LogAva, Warning, "DefaultMaterial could not be loaded from path: '%ls'", DefaultMaterialPath);
	}
}

void UAvaGizmoComponent::PostLoad()
{
	Super::PostLoad();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (bIsGizmoEnabled && !bAreGizmoValuesApplied)
		{
			StoreComponentValues();
			ApplyGizmoValues();
		}
	}
}

#if WITH_EDITOR
void UAvaGizmoComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	ApplyGizmoValues();
}
#endif

void UAvaGizmoComponent::OnRegister()
{
	// @note: I haven't seen this used anywhere else
	if (AActor* Owner = GetOwner();
		IsValid(Owner))
	{
		if (UWorld* World = Owner->GetWorld())
		{
			// We don't store or apply values here - they'll be called in the cb below
			
			PostRegisterComponentsHandle = World->AddOnPostRegisterAllActorComponentsHandler(
				FOnPostRegisterAllActorComponents::FDelegate::CreateUObject(this, &UAvaGizmoComponent::OnPostRegisterParentComponents));
		}
	}

	Super::OnRegister();
}

void UAvaGizmoComponent::OnUnregister()
{
	// @note: I haven't seen this used anywhere else
	if (AActor* Owner = GetOwner();
		IsValid(Owner))
	{
		if (UWorld* World = Owner->GetWorld())
		{
			World->RemoveOnPostRegisterAllActorComponentsHandler(PostRegisterComponentsHandle);
			PostRegisterComponentsHandle.Reset();
		}
	}

	Super::OnUnregister();

	RestoreComponentValues();
}

void UAvaGizmoComponent::OnComponentCreated()
{
	Super::OnComponentCreated();

	if (bIsGizmoEnabled)
	{
		if (!bAreGizmoValuesApplied)
		{
			StoreComponentValues();
			ApplyGizmoValues();
		}
	}
}

void UAvaGizmoComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	// Only restore if destroying this single component, not on actor teardown
	if (!bDestroyingHierarchy)
	{
		RestoreComponentValues();
	}

	Super::OnComponentDestroyed(bDestroyingHierarchy);
}

void UAvaGizmoComponent::ApplyGizmoValues()
{
	if (!bIsGizmoEnabled || bIsRestoringValues)
	{
		return;
	}

	if (AActor* Owner = GetOwner();
		IsValid(Owner))
	{
		bAreGizmoValuesApplied = true;

		ForEachPrimitiveComponent(
			[this](UPrimitiveComponent* InComponent)
			{
				InComponent->bVisibleInReflectionCaptures = false;
				InComponent->bVisibleInRealTimeSkyCaptures = false;
				InComponent->bVisibleInRayTracing = false;
				InComponent->bVisibleInReflections = false;
				InComponent->SetVisibleInRayTracing(false);
				InComponent->SetVisibleInSceneCaptureOnly(false);
				
				InComponent->SetAffectDistanceFieldLighting(false);
				InComponent->SetAffectDynamicIndirectLighting(false);
				InComponent->SetAffectIndirectLightingWhileHidden(false);

				InComponent->SetCastShadow(bCastShadow);
				InComponent->SetVisibility(bIsVisibleInEditor, true);
				InComponent->SetHiddenInGame(bIsHiddenInGame);

				InComponent->SetRenderCustomDepth(bSetStencil);
				InComponent->SetCustomDepthStencilValue(StencilId);	
				
				InComponent->SetRenderInMainPass(bRenderInMainPass);
				InComponent->SetRenderInDepthPass(bRenderDepth);
				
				if (Material)
				{
					for (int32 MaterialIdx = 0; MaterialIdx < InComponent->GetNumMaterials(); ++MaterialIdx)
					{
						InComponent->SetMaterial(MaterialIdx, Material);
					}
				}

				
				if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(InComponent))
				{
					DynamicMeshComponent->WireframeColor = WireframeColor;
					DynamicMeshComponent->SetEnableWireframeRenderPass(bDrawWireframe);
				}
			});

		if (!FUObjectThreadContext::Get().IsRoutingPostLoad)
		{
			TArray<UActorComponent*> GizmoObjectComponents = Owner->GetComponentsByInterface(UAvaGizmoObjectInterface::StaticClass());
			for (UActorComponent* Component : GizmoObjectComponents)
			{
				IAvaGizmoObjectInterface::Execute_ToggleGizmo(Component, this, true);	
			}
		}

#if WITH_EDITOR
		if (GEditor)
		{
			// Mark camera cut to remove temporal effects
			for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
			{
				LevelVC->SetIsCameraCut();
			}
		}
#endif

		Owner->MarkComponentsRenderStateDirty();
	}
}

void UAvaGizmoComponent::RestoreComponentValues()
{
	if (!bAreGizmoValuesApplied || bIsRestoringValues)
	{
		return;
	}

	bIsRestoringValues = true;

	ON_SCOPE_EXIT
	{
		bIsRestoringValues = false;
	};

	AActor* Owner = GetOwner();

	if (IsValid(Owner) && !Owner->IsActorBeingDestroyed() && !Owner->IsUnreachable())
	{
		bAreGizmoValuesApplied = false;

		TArray<UPrimitiveComponent*> PrimitiveComponents;
		Owner->GetComponents<UPrimitiveComponent>(PrimitiveComponents, bApplyToChildActors);

		for (UPrimitiveComponent* Component : PrimitiveComponents)
		{
			if (!IsValid(Component) || Component->IsBeingDestroyed() || Component->IsUnreachable())
			{
				continue;
			}
			
			FSoftComponentReference ComponentReference;
			ComponentReference.PathToComponent = Component->GetPathName(Owner);
			
			if (FAvaGizmoComponentPrimitiveValues* StoredComponentValues = ComponentValues.Find(ComponentReference))
			{
				Component->bVisibleInReflectionCaptures = StoredComponentValues->bVisibleInReflectionCaptures;
				Component->bVisibleInRealTimeSkyCaptures = StoredComponentValues->bVisibleInRealTimeSkyCaptures;
				Component->bVisibleInRayTracing = StoredComponentValues->bVisibleInRayTracing;
				Component->bVisibleInReflections = StoredComponentValues->bVisibleInReflections;

				Component->SetAffectDistanceFieldLighting(StoredComponentValues->bAffectDistanceFieldLighting);
				Component->SetAffectDynamicIndirectLighting(StoredComponentValues->bAffectDynamicIndirectLighting);
				Component->SetAffectIndirectLightingWhileHidden(StoredComponentValues->bAffectIndirectLightingWhileHidden);
										
				Component->CastShadow = StoredComponentValues->bCastShadow;
				Component->SetVisibility(StoredComponentValues->bIsVisible);
				Component->bHiddenInGame = StoredComponentValues->bIsHiddenInGame;

				Component->SetRenderCustomDepth(StoredComponentValues->bRendersCustomDepth);
				Component->SetCustomDepthStencilValue(StoredComponentValues->CustomStencilId);

				Component->SetRenderInMainPass(StoredComponentValues->bRendersInMainPass);
				Component->SetRenderInDepthPass(StoredComponentValues->bRendersDepth);

				if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component))
				{
					DynamicMeshComponent->SetEnableWireframeRenderPass(StoredComponentValues->bDrawWireframe);
					DynamicMeshComponent->WireframeColor = StoredComponentValues->WireframeColor;
				}

				const int32 ActualNumMaterials = Component->GetNumMaterials();
				const int32 ExpectedNumMaterials = StoredComponentValues->Materials.Num();

				if (ActualNumMaterials != ExpectedNumMaterials)
				{
					UE_LOGF(LogTemp, Warning, "Material count mismatch when restoring component '%ls' - got %i, expected %i",
						*Component->GetName(),
						ActualNumMaterials,
						ExpectedNumMaterials);
				}

				for (int32 MaterialIdx = 0; MaterialIdx < FMath::Min(ActualNumMaterials, ExpectedNumMaterials); ++MaterialIdx)
				{
					Component->SetMaterial(MaterialIdx, StoredComponentValues->Materials[MaterialIdx]);
				}
			}
		}

		if (!FUObjectThreadContext::Get().IsRoutingPostLoad)
		{
			TArray<UActorComponent*> GizmoObjectComponents = Owner->GetComponentsByInterface(UAvaGizmoObjectInterface::StaticClass());
			for (UActorComponent* Component : GizmoObjectComponents)
			{
				if (IsValid(Component) && !Component->IsBeingDestroyed() && !Component->IsUnreachable())
				{
					IAvaGizmoObjectInterface::Execute_ToggleGizmo(Component, this, false);	
				}
			}
		}

		Owner->MarkComponentsRenderStateDirty();
	}
}

void UAvaGizmoComponent::SetGizmoEnabled(const bool bInIsEnabled)
{
	if (bIsGizmoEnabled != bInIsEnabled)
	{
		bIsGizmoEnabled = bInIsEnabled;
		if (bIsGizmoEnabled)
		{
			ApplyGizmoValues();
		}
		else
		{
			RestoreComponentValues();
		}
	}
}

bool UAvaGizmoComponent::AppliesToChildActors() const
{
	return bApplyToChildActors;
}

void UAvaGizmoComponent::SetApplyToChildActors(bool bInValue)
{
	if (bApplyToChildActors != bInValue)
	{
		bApplyToChildActors = bInValue;
		ApplyGizmoValues();
	}
}

UMaterialInterface* UAvaGizmoComponent::GetMaterial() const
{
	return Material;
}

void UAvaGizmoComponent::SetMaterial(UMaterialInterface* InMaterial)
{
	if (Material != InMaterial)
	{
		Material = InMaterial;
		ApplyToEachPrimitiveComponent(
			[Material = Material](UPrimitiveComponent* InComponent)
			{
				for (int32 MaterialIdx = 0; MaterialIdx < InComponent->GetNumMaterials(); ++MaterialIdx)
				{
					InComponent->SetMaterial(MaterialIdx, Material);
				}
			});
	}
}

void UAvaGizmoComponent::SetMaterialToDefault()
{
	SetMaterial(nullptr);
}

void UAvaGizmoComponent::SetMaterialToSeeThrough()
{
	SetMaterial(nullptr);
}

bool UAvaGizmoComponent::CastsShadow() const
{
	return bCastShadow;
}

void UAvaGizmoComponent::SetCastShadow(bool bInValue)
{
	if (bCastShadow != bInValue)
	{
		bCastShadow = bInValue;
		ApplyToEachPrimitiveComponent(
		[bCastShadow = bCastShadow](UPrimitiveComponent* InComponent)
			{
				InComponent->SetCastShadow(bCastShadow);
			});
	}
}

bool UAvaGizmoComponent::IsVisibleInEditor() const
{
	return bIsVisibleInEditor;
}

void UAvaGizmoComponent::SetVisibleInEditor(bool bInValue)
{
	if (bIsVisibleInEditor != bInValue)
	{
		bIsVisibleInEditor = bInValue;
		ApplyToEachPrimitiveComponent(
		[bIsVisibleInEditor = bIsVisibleInEditor](UPrimitiveComponent* InComponent)
			{
				InComponent->SetVisibility(bIsVisibleInEditor, true);
			});
	}
}

bool UAvaGizmoComponent::IsHiddenInGame() const
{
	return bIsHiddenInGame;
}

void UAvaGizmoComponent::SetHiddenInGame(bool bInValue)
{
	if (bIsHiddenInGame != bInValue)
	{
		bIsHiddenInGame = bInValue;
		ApplyToEachPrimitiveComponent(
		[bIsHiddenInGame = bIsHiddenInGame](UPrimitiveComponent* InComponent)
			{
				InComponent->SetHiddenInGame(bIsHiddenInGame);
			});
	}
}

bool UAvaGizmoComponent::RendersInMainPass() const
{
	return bRenderInMainPass;
}

void UAvaGizmoComponent::SetRenderInMainPass(const bool bInValue)
{
	if (bRenderInMainPass != bInValue)
	{
		bRenderInMainPass = bInValue;
		ApplyToEachPrimitiveComponent(
		[bRenderInMainPass = bRenderInMainPass](UPrimitiveComponent* InComponent)
			{
				InComponent->SetRenderInMainPass(bRenderInMainPass);
			});
	}
}

bool UAvaGizmoComponent::RendersDepth() const
{
	return bRenderDepth;
}

void UAvaGizmoComponent::SetRenderDepth(const bool bInValue)
{
	if (bRenderDepth != bInValue)
	{
		bRenderDepth = bInValue;
		ApplyToEachPrimitiveComponent(
		[bRenderDepth = bRenderDepth](UPrimitiveComponent* InComponent)
			{
				InComponent->SetRenderInDepthPass(bRenderDepth);
			});
	}
}

bool UAvaGizmoComponent::ShowsWireframe() const
{
	return bDrawWireframe;
}

void UAvaGizmoComponent::SetShowWireframe(bool bInValue)
{
	if (bDrawWireframe != bInValue)
	{
		bDrawWireframe = bInValue;
		ApplyToEachPrimitiveComponent(
		[bDrawWireframe = bDrawWireframe](UPrimitiveComponent* InComponent)
			{
				if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(InComponent))
				{
					DynamicMeshComponent->SetEnableWireframeRenderPass(bDrawWireframe);
				}
			});
	}
}

void UAvaGizmoComponent::SetWireframeColor(const FLinearColor& InColor)
{
	if (WireframeColor != InColor)
	{
		WireframeColor = InColor;
		ApplyToEachPrimitiveComponent(
		[WireframeColor = WireframeColor](UPrimitiveComponent* InComponent)
			{
				if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(InComponent))
				{
					DynamicMeshComponent->WireframeColor = WireframeColor;
				}
			});
	}
}

bool UAvaGizmoComponent::SetsStencil() const
{
	return bSetStencil;
}

void UAvaGizmoComponent::SetsStencil(const bool bInSetStencil)
{
	if (bSetStencil != bInSetStencil)
	{
		bSetStencil = bInSetStencil;
		ApplyToEachPrimitiveComponent(
	[bSetStencil = bSetStencil](UPrimitiveComponent* InComponent)
			{
				InComponent->SetRenderCustomDepth(bSetStencil);	
			});
	}
}

const uint8 UAvaGizmoComponent::GetStencilId() const
{
	return StencilId;
}

void UAvaGizmoComponent::SetStencilId(const uint8 InStencilId)
{
	if (StencilId != InStencilId)
	{
		StencilId = InStencilId;
		ApplyToEachPrimitiveComponent(
	[StencilId = StencilId](UPrimitiveComponent* InComponent)
			{
				InComponent->SetCustomDepthStencilValue(StencilId);	
			});
	}
}

void UAvaGizmoComponent::StoreComponentValues()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (const AActor* Owner = GetOwner();
			IsValid(Owner))
		{
			bool bWasOneOrMorePrimitiveComponents = false;
			ForEachPrimitiveComponent(
				[this, Owner, &bWasOneOrMorePrimitiveComponents](UPrimitiveComponent* InComponent)
				{
					bWasOneOrMorePrimitiveComponents = true;

					FSoftComponentReference ComponentReference;
					ComponentReference.PathToComponent = InComponent->GetPathName(Owner);

					FAvaGizmoComponentPrimitiveValues& StoredComponentValues = ComponentValues.FindOrAdd(ComponentReference);

					StoredComponentValues.bVisibleInReflectionCaptures = InComponent->bVisibleInReflectionCaptures;
					StoredComponentValues.bVisibleInRealTimeSkyCaptures = InComponent->bVisibleInRealTimeSkyCaptures;
					StoredComponentValues.bVisibleInRayTracing = InComponent->bVisibleInRayTracing;
					StoredComponentValues.bVisibleInReflections = InComponent->bVisibleInReflections;

					StoredComponentValues.bAffectDistanceFieldLighting = InComponent->bAffectDistanceFieldLighting;
					StoredComponentValues.bAffectDynamicIndirectLighting = InComponent->bAffectDynamicIndirectLighting;
					StoredComponentValues.bAffectIndirectLightingWhileHidden = InComponent->bAffectIndirectLightingWhileHidden;

					StoredComponentValues.bCastShadow = InComponent->CastShadow;
					StoredComponentValues.bIsVisible = InComponent->IsVisible();
					StoredComponentValues.bIsHiddenInGame = InComponent->bHiddenInGame;

					StoredComponentValues.bRendersCustomDepth = InComponent->bRenderCustomDepth > 0;
					StoredComponentValues.CustomStencilId = InComponent->CustomDepthStencilValue;

					StoredComponentValues.bRendersInMainPass = InComponent->bRenderInMainPass;
					StoredComponentValues.bRendersDepth = InComponent->bRenderInDepthPass;

					// @todo: move this impl specific stuff to a Handler/Provider subsystem
					if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(InComponent))
					{
						StoredComponentValues.bDrawWireframe = DynamicMeshComponent->GetEnableWireframeRenderPass();
						StoredComponentValues.WireframeColor = DynamicMeshComponent->WireframeColor;
					}

					TArray<UMaterialInterface*> UsedMaterials;
					InComponent->GetUsedMaterials(UsedMaterials);

					StoredComponentValues.Materials.Reset(UsedMaterials.Num());
					Algo::Transform(UsedMaterials, StoredComponentValues.Materials, [](UMaterialInterface* InMaterial)
					{
						return InMaterial;
					});
				});

			if (!bWasOneOrMorePrimitiveComponents)
			{
				UE_LOGF(LogAva, Warning, "Actor '%ls' doesn't contain any components applicable to UAvaGizmoComponent", *Owner->GetName());
			}
		}
	}
}

void UAvaGizmoComponent::OnPostRegisterParentComponents(AActor* InActor)
{
	if (AActor* Owner = GetOwner();
		IsValid(Owner) && Owner == InActor)
	{
		if (bIsGizmoEnabled)
		{
			if (!bAreGizmoValuesApplied)
			{
				StoreComponentValues();

				// Allows for all other components to do stuff first
				AsyncTask(ENamedThreads::GameThread, [WeakThis = TWeakObjectPtr<UAvaGizmoComponent>(this)]()
				{
					if (UAvaGizmoComponent* StrongThis = WeakThis.Get();
						UObjectInitialized() && StrongThis)
					{
						StrongThis->ApplyGizmoValues();
					}
				});
			}
		}
	}
}
	
void UAvaGizmoComponent::ForEachPrimitiveComponent(TFunctionRef<void(UPrimitiveComponent*)> InFunc)
{
	if (AActor* Owner = GetOwner();
		IsValid(Owner))
	{
		TArray<UPrimitiveComponent*> PrimitiveComponents;
		Owner->GetComponents<UPrimitiveComponent>(PrimitiveComponents, bApplyToChildActors);
	
		for (UPrimitiveComponent* Component : PrimitiveComponents)
		{
			if (!IsValid(Component))
			{
				continue;
			}
	
			InFunc(Component);
		}
	}
}

void UAvaGizmoComponent::ApplyToEachPrimitiveComponent(TFunctionRef<void(UPrimitiveComponent*)> InFunc)
{
	if (!bIsGizmoEnabled || bIsRestoringValues)
	{
		return;
	}
	
	if (AActor* Owner = GetOwner();
		IsValid(Owner))
	{
		bAreGizmoValuesApplied = true;
	
		ForEachPrimitiveComponent(MoveTemp(InFunc));
	}
}
