// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/AvaNullActor.h"
#include "Framework/AvaNullComponent.h"
#include "UObject/ConstructorHelpers.h"

const FString AAvaNullActor::DefaultLabel = TEXT("Null");

AAvaNullActor::AAvaNullActor()
{
	PrimaryActorTick.bCanEverTick = false;

	NullComponent = CreateDefaultSubobject<UAvaNullComponent>(TEXT("NullComponent"));
	SetRootComponent(NullComponent);
}

#if WITH_EDITOR
FString AAvaNullActor::GetDefaultActorLabel() const
{
	return TEXT("Null Actor");
}
#endif
