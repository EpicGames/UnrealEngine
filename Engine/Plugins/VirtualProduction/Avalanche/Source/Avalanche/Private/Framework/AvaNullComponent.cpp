// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/AvaNullComponent.h"

UAvaNullComponent::UAvaNullComponent()
{
	PrimaryComponentTick.bCanEverTick = false;
}
