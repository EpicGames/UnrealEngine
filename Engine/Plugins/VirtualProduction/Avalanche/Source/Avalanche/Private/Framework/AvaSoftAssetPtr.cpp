// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/AvaSoftAssetPtr.h"
#include "Engine/World.h"

EMotionDesignAssetType FAvaSoftAssetPath::GetAssetTypeFromClass(const FSoftClassPath& InAssetClassPath, bool bInLoadIfUnknown)
{
	static const FSoftClassPath WorldClassPath(UWorld::StaticClass());

	if (InAssetClassPath == WorldClassPath)
	{
		return EMotionDesignAssetType::World;
	}

	// Try to load the asset class and compare possible derived types.
	if (InAssetClassPath.IsValid())
	{
		const UClass* AssetClass = InAssetClassPath.ResolveClass();
		if (AssetClass->IsChildOf(UWorld::StaticClass()))
		{
			return EMotionDesignAssetType::World;
		}
	}
	return EMotionDesignAssetType::Unknown;
}

EMotionDesignAssetType FAvaSoftAssetPtr::GetAssetType(bool bInLoadIfUnknown) const
{
	const EMotionDesignAssetType AssetTypeFromClass = FAvaSoftAssetPath::GetAssetTypeFromClass(AssetClassPath, bInLoadIfUnknown);
	if (AssetTypeFromClass != EMotionDesignAssetType::Unknown)
	{
		return AssetTypeFromClass;
	}

	if (bInLoadIfUnknown)
	{
		// Todo: problem with loading the asset sync here.
		// Todo: Investigate LoadPackageAsync.
		// This will cause a hitch. For level asset, we want to avoid that.
		UObject* LoadedSourceAsset;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FAvaSoftAssetPtr::LoadSourceAsset);
			const FSoftObjectPtr SourceAsset(AssetPtr.ToSoftObjectPath());
			LoadedSourceAsset = SourceAsset.LoadSynchronous();
		}

		if (Cast<UWorld>(LoadedSourceAsset))
		{
			return EMotionDesignAssetType::World;
		}
	}

	return EMotionDesignAssetType::Unknown;
}
