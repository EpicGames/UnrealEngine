// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/AvaSplineActor.h"
#include "Components/SplineComponent.h"

AAvaSplineActor::AAvaSplineActor()
{
	PrimaryActorTick.bCanEverTick = false;

	SplineComponent = CreateDefaultSubobject<USplineComponent>(TEXT("SplineComponent"));
	SetRootComponent(SplineComponent);

	// Update default points to appear in front of the camera in Y and Z
	SplineComponent->SetLocationAtSplinePoint(0, FVector(0, -100, 0), ESplineCoordinateSpace::Local, false);
	SplineComponent->SetLocationAtSplinePoint(1, FVector(0, 100, 0), ESplineCoordinateSpace::Local, true);
}