// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Ticker/AvaTickerActor.h"
#include "Framework/Ticker/AvaTickerComponent.h"

AAvaTickerActor::AAvaTickerActor()
{
	TickerComponent = CreateDefaultSubobject<UAvaTickerComponent>(TEXT("TickerComponent"));
}

#if WITH_EDITOR
FString AAvaTickerActor::GetDefaultActorLabel() const
{
	return TEXT("Motion Design Ticker Actor");
}
#endif
