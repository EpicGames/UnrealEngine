// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Ticker/AvaTickerComponent.h"
#include "AvaTickerSceneProxy.h"
#include "Misc/EnumerateRange.h"

namespace UE::Avalanche::Private
{
	/** For each vector component returns 1 if positive, -1 if negative and 0 if 0. */
	FVector GetSignVector(const FVector& InVector)
	{
		// Not using FVector::GetSignVector as it only returns +1 or -1
		return FVector(FMath::Sign(InVector.X), FMath::Sign(InVector.Y), FMath::Sign(InVector.Z));
	}
}

FAvaTickerElement::FAvaTickerElement(AActor* InActor)
	: Actor(InActor)
{
}

bool FAvaTickerElement::operator==(const FAvaTickerElement& InOther) const
{
	return Actor == InOther.Actor;
}

bool FAvaTickerElement::IsValid() const
{
	return !!Actor;
}

void FAvaTickerElement::SetVisibility(bool bInVisible)
{
	if (Actor)
	{
		Actor->SetActorHiddenInGame(!bInVisible);
#if WITH_EDITOR
    	Actor->SetIsTemporarilyHiddenInEditor(!bInVisible);
#endif
	}
}

FVector FAvaTickerElement::GetElementLocation() const
{
	if (Actor)
	{
		return Actor->GetActorLocation();
	}
	return FVector::ZeroVector;
}

void FAvaTickerElement::SetLocation(const FVector& InLocation)
{
	if (Actor)
	{
		Actor->SetActorLocation(InLocation, /*bSweep*/false, /*OutSweepHitResult*/nullptr, ETeleportType::ResetPhysics);
	}
}

void FAvaTickerElement::Displace(const FVector& InDisplacement)
{
	if (Actor)
	{
		Actor->AddActorWorldOffset(InDisplacement);
	}
}

void FAvaTickerElement::Destroy()
{
	if (Actor)
	{
		Actor->Destroy();
	}
}

void FAvaTickerElement::GetBounds(FVector& OutOrigin, FVector& OutExtents) const
{
	if (Actor)
	{
		constexpr bool bOnlyCollidingComponents = false;
    	constexpr bool bIncludeFromChildActors = true;
    	Actor->GetActorBounds(bOnlyCollidingComponents, OutOrigin, OutExtents, bIncludeFromChildActors);
	}
	else
	{
		OutOrigin = FVector::ZeroVector;
		OutExtents = FVector::ZeroVector;
	}
}

UAvaTickerComponent::UAvaTickerComponent()
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	bTickInEditor = true;
	bHiddenInGame = true;
}

bool UAvaTickerComponent::CanQueueElements() const
{
	if (IsQueueFull())
	{
		return QueueLimitType != EAvaTickerQueueLimitType::DisableQueueing;
	}

	if (GetWorldVelocity().IsNearlyZero())
	{
		return false;
	}

	return true;
}

bool UAvaTickerComponent::QueueActor(AActor* InActor, bool bInDestroyOnFailure)
{
	return InActor && QueueElement(FAvaTickerElement(InActor), bInDestroyOnFailure);
}

void UAvaTickerComponent::SetStartLocation(const FVector& InStartLocation)
{
	StartLocation = InStartLocation;
}

void UAvaTickerComponent::SetDestroyDistance(double InDestroyDistance)
{
	DestroyDistance = InDestroyDistance;
}

void UAvaTickerComponent::SetVelocity(const FVector& InVelocity)
{
	Velocity = InVelocity;
}

void UAvaTickerComponent::SetPadding(double InPadding)
{
	Padding = InPadding;
}

void UAvaTickerComponent::SetLimitQueue(bool bInLimitQueue)
{
	bLimitQueue = bInLimitQueue;
}

void UAvaTickerComponent::SetQueueLimitCount(int32 InQueueLimitCount)
{
	QueueLimitCount = InQueueLimitCount;
}

void UAvaTickerComponent::SetQueueLimitType(EAvaTickerQueueLimitType InQueueLimitType)
{
	QueueLimitType = InQueueLimitType;
}

FPrimitiveSceneProxy* UAvaTickerComponent::CreateSceneProxy()
{
	return new UE::Avalanche::FTickerSceneProxy(this);
}

FBoxSphereBounds UAvaTickerComponent::CalcBounds(const FTransform& InLocalToWorld) const
{
	const FVector HalfDestroyDisplacement = 0.5 * Velocity.GetSafeNormal() * DestroyDistance;
	return FBoxSphereBounds(FBox::BuildAABB(StartLocation + HalfDestroyDisplacement, HalfDestroyDisplacement)).TransformBy(InLocalToWorld);
}

void UAvaTickerComponent::TickComponent(float InDeltaTime, ELevelTick InTickType, FActorComponentTickFunction* InThisTickFunction)
{
	Super::TickComponent(InDeltaTime, InTickType, InThisTickFunction);
	TryDequeueElement();
	UpdateActiveElements(InDeltaTime);
}

bool UAvaTickerComponent::IsQueueFull() const
{
	return bLimitQueue && QueuedElements.Num() >= QueueLimitCount;
}

bool UAvaTickerComponent::QueueElement(FAvaTickerElement&& InElement, bool bInDestroyOnFailure)
{
	// Can't queue if velocity is zero
	if (GetWorldVelocity().IsNearlyZero())
	{
		if (bInDestroyOnFailure)
		{
			InElement.Destroy();
		}
		return false;
	}

	if (IsQueueFull())
	{
		switch (QueueLimitType)
		{
		case EAvaTickerQueueLimitType::DisableQueueing:
			if (bInDestroyOnFailure)
			{
				InElement.Destroy();
			}
			// Queue is disabled on full, return
			return false;

		case EAvaTickerQueueLimitType::DiscardOldest:
			if (QueuedElements.Num() > 0)
			{
				QueuedElements[0].Destroy();
				QueuedElements.RemoveAt(0);
			}
			break;

		default:
			break;
		}
	}

	InElement.SetVisibility(/*bVisible*/false);
	QueuedElements.AddUnique(MoveTemp(InElement));
	return true;
}

FVector UAvaTickerComponent::GetWorldStartLocation() const
{
	return GetComponentTransform().TransformPosition(StartLocation);
}

FVector UAvaTickerComponent::GetWorldDestroyDisplacement() const
{
	return GetComponentTransform().TransformVector(Velocity.GetSafeNormal() * DestroyDistance);
}

FVector UAvaTickerComponent::GetWorldVelocity() const
{
	// Todo: Consider giving an option whether to scale the velocity or not?
	return GetComponentTransform().TransformVectorNoScale(Velocity);
}

void UAvaTickerComponent::TryDequeueElement()
{
	using namespace UE::Avalanche;

	if (QueuedElements.IsEmpty())
	{
		return;
	}

	const FVector NormalizedVelocity = GetWorldVelocity().GetSafeNormal();
	if (NormalizedVelocity.IsNearlyZero())
	{
		return;
	}

	const FAvaTickerElement& Element = QueuedElements[0];

	FVector ElementOrigin;
	FVector ElementExtents;
	Element.GetBounds(ElementOrigin, ElementExtents);

	const FVector WorldStartLocation = GetWorldStartLocation();

	// Switch the sign extents so that it can be projected onto the velocity direction
	const FVector SignedExtents = ElementExtents * Private::GetSignVector(NormalizedVelocity);
	const FVector StartExtentsToOrigin = -1.0 * NormalizedVelocity * SignedExtents.ProjectOnTo(NormalizedVelocity).Length();
	const FVector OriginToLocation = Element.GetElementLocation() - ElementOrigin;

	// Check if the last active element is in the way
	if (!ActiveElements.IsEmpty())
	{
		FVector TargetOrigin;
		FVector TargetExtents;
		ActiveElements.Last().GetBounds(TargetOrigin, TargetExtents);

		// Check if the queued element bounds is clear from any overlap with the last active element's bounds
		if (FBox::BuildAABB(WorldStartLocation + StartExtentsToOrigin, ElementExtents).Overlap(FBox::BuildAABB(TargetOrigin, TargetExtents)).IsValid)
		{
			return;
		}
	}

	FVector ElementStartLocation = WorldStartLocation + StartExtentsToOrigin + OriginToLocation;

	// Add padding in the opposite direction of world velocity
	ElementStartLocation += -1.0 * NormalizedVelocity * Padding;

	FAvaTickerElement DequeuedElement = MoveTemp(QueuedElements[0]);
	QueuedElements.RemoveAt(0);

	DequeuedElement.SetLocation(ElementStartLocation);
	DequeuedElement.SetVisibility(/*bVisible*/true);

	ActiveElements.Add(MoveTemp(DequeuedElement));
}

void UAvaTickerComponent::UpdateActiveElements(float InDeltaTime)
{
	using namespace UE::Avalanche;

	if (ActiveElements.IsEmpty())
	{
		return;
	}

	const FVector WorldVelocity = GetWorldVelocity();
	const FVector Displacement = WorldVelocity * InDeltaTime;

	for (FAvaTickerElement& ActiveElement : ActiveElements)
	{
		ActiveElement.Displace(Displacement);
	}

	const FVector WorldStartLocation = GetWorldStartLocation();
	const FVector NormalizedVelocity = WorldVelocity.GetSafeNormal();
	const double DistanceSquared = GetWorldDestroyDisplacement().SquaredLength();

	int32 IndexRangeToRemove = INDEX_NONE;

	// Find the last active element that is more than the given distance away from 
	for (TEnumerateRef<FAvaTickerElement> ActiveElement : EnumerateRange(ActiveElements))
	{
		if (!ActiveElement->IsValid())
		{
			IndexRangeToRemove = ActiveElement.GetIndex();
			continue;
		}

		FVector ElementOrigin;
		FVector ElementExtents;
		ActiveElement->GetBounds(ElementOrigin, ElementExtents);

		// Switch the sign extents so that it can be projected onto the velocity direction
		const FVector SignedExtents = ElementExtents * Private::GetSignVector(NormalizedVelocity);
		const FVector EndExtents = ElementOrigin + (-1.0 * NormalizedVelocity * SignedExtents.ProjectOnTo(NormalizedVelocity).Length());

		// Ensure that the end extents has passed the destroy distance by checking distance
		// while also preventing cases where an actors extents are large so that its end extents are at a distance larger than the destroy distance
		if (FVector::DistSquared(WorldStartLocation, EndExtents) >= DistanceSquared
			&& FVector::DotProduct(EndExtents - WorldStartLocation, NormalizedVelocity) >= 0)
		{
			IndexRangeToRemove = ActiveElement.GetIndex();
		}
		else
		{
			// The elements coming after will all still be within range
			break;
		}
	}

	while (IndexRangeToRemove-- >= 0)
	{
		ActiveElements[0].Destroy();
		ActiveElements.RemoveAt(0);
	}
}
