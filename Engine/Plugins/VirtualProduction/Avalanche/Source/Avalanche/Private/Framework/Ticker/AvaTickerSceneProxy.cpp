// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTickerSceneProxy.h"
#include "Engine/Engine.h"
#include "Framework/Ticker/AvaTickerComponent.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "SceneInterface.h"
#include "SceneView.h"

namespace UE::Avalanche
{

FTickerSceneProxy::FTickerSceneProxy(UAvaTickerComponent* InComponent)
	: FPrimitiveSceneProxy(InComponent)
	, ArrowVertexFactory(GetScene().GetFeatureLevel(), "FAvaTickerSceneProxyArrow")
	, EndPlaneVertexFactory(GetScene().GetFeatureLevel(), "FAvaTickerSceneProxyEndPlane")
	, StartLocation(InComponent->GetStartLocation())
	, Velocity(InComponent->GetVelocity())
	, DestroyDistance(InComponent->GetDestroyDistance())
	, DrawColor(GEngine ? GEngine->C_AddWire : FColor(127, 127, 255, 255))
{
	bWillEverBeLit = false;

	LocalArrowMatrix = FRotationMatrix::MakeFromX(Velocity);
	LocalArrowMatrix.SetOrigin(StartLocation);

	LocalEndPlaneMatrix = FRotationMatrix::MakeFromX(-Velocity);
	LocalEndPlaneMatrix.SetOrigin(StartLocation + Velocity.GetSafeNormal() * DestroyDistance);

	BuildArrowMesh();
	BuildEndPlaneMesh();
}

FTickerSceneProxy::~FTickerSceneProxy()
{
	ArrowVertexBuffers.PositionVertexBuffer.ReleaseResource();
	ArrowVertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
	ArrowVertexBuffers.ColorVertexBuffer.ReleaseResource();
	ArrowIndexBuffer.ReleaseResource();
	ArrowVertexFactory.ReleaseResource();

	EndPlaneVertexBuffers.PositionVertexBuffer.ReleaseResource();
	EndPlaneVertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
	EndPlaneVertexBuffers.ColorVertexBuffer.ReleaseResource();
	EndPlaneIndexBuffer.ReleaseResource();
	EndPlaneVertexFactory.ReleaseResource();
}

SIZE_T FTickerSceneProxy::GetTypeHash() const
{
	static SIZE_T UniquePointer;
	return reinterpret_cast<SIZE_T>(&UniquePointer);
}

void FTickerSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& InViews, const FSceneViewFamily& InViewFamily, uint32 InVisibilityMap, FMeshElementCollector& InCollector) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_AvaTickerSceneProxy_DrawDynamicElements);

	const FMatrix ArrowWorldMatrix = LocalArrowMatrix * GetLocalToWorld();
	const FMatrix EndPlaneWorldMatrix = LocalEndPlaneMatrix * GetLocalToWorld();

	FColoredMaterialRenderProxy* MaterialRenderProxy = new FColoredMaterialRenderProxy(GEngine->ArrowMaterial->GetRenderProxy()
		, DrawColor
		, "GizmoColor");

	InCollector.RegisterOneFrameMaterialProxy(MaterialRenderProxy);

	for (int32 ViewIndex = 0; ViewIndex < InViews.Num(); ViewIndex++)
	{
		if (InVisibilityMap & (1 << ViewIndex))
		{
			DrawArrowElement(InCollector, ViewIndex, ArrowWorldMatrix, MaterialRenderProxy);
			DrawEndPlaneElement(InCollector, ViewIndex, EndPlaneWorldMatrix, MaterialRenderProxy);
		}
	}
}

FPrimitiveViewRelevance FTickerSceneProxy::GetViewRelevance(const FSceneView* InView) const
{
	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = IsShown(InView) && InView->Family->EngineShowFlags.BillboardSprites;
	Result.bDynamicRelevance = true;
	Result.bShadowRelevance = IsShadowCast(InView);
	Result.bEditorPrimitiveRelevance = UseEditorCompositing(InView);
	Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;
	return Result;
}

uint32 FTickerSceneProxy::GetMemoryFootprint() const
{
	return sizeof(*this) + GetAllocatedSize();
}

uint32 FTickerSceneProxy::GetAllocatedSize() const
{
	return FPrimitiveSceneProxy::GetAllocatedSize();
}

void FTickerSceneProxy::BuildArrowMesh()
{
	TArray<FDynamicMeshVertex> Vertices;

	constexpr float HeadAngle   = FMath::DegreesToRadians(10.f);
	constexpr float HeadLength  = 16.f;
	constexpr float ShaftRadius = 1.f;
	constexpr float TotalLength = 50.f;
	constexpr float ShaftLength = TotalLength - HeadLength * 0.5; // 10% overlap between shaft and head

	const FVector ShaftCenter = FVector(-0.5 * ShaftLength - HeadLength, 0.0, 0.0);

	BuildConeVerts(HeadAngle, HeadAngle
		, -HeadLength
		, /*XOffset*/0.0
		, /*Sides*/32
		, Vertices
		, ArrowIndexBuffer.Indices);

	BuildCylinderVerts(ShaftCenter
		, /*X*/FVector(0, 0, 1)
		, /*Y*/FVector(0, 1, 0)
		, /*Z*/FVector(1, 0, 0)
		, ShaftRadius
		, 0.5f * ShaftLength
		, /*Sides*/16
		, Vertices
		, ArrowIndexBuffer.Indices);

	ArrowVertexBuffers.InitFromDynamicVertex(&ArrowVertexFactory, Vertices);

	// Enqueue initialization of render resource
	BeginInitResource(&ArrowIndexBuffer);
}

void FTickerSceneProxy::BuildEndPlaneMesh()
{
	constexpr float HalfWidth = 25.f;
	constexpr float HalfHeight = 25.f;

	const FVector XOffset = FVector::RightVector * HalfWidth;
	const FVector YOffset = FVector::UpVector * HalfHeight;

	// Calculate verts for a face lying in plane defined by the X and Y vectors
	TArray<FDynamicMeshVertex> Vertices;
	Vertices.SetNum(4);

	Vertices[0].Position = static_cast<FVector3f>(-XOffset - YOffset);
	Vertices[1].Position = static_cast<FVector3f>(XOffset - YOffset);
	Vertices[2].Position = static_cast<FVector3f>(XOffset + YOffset);
	Vertices[3].Position = static_cast<FVector3f>(-XOffset + YOffset);

	EndPlaneIndexBuffer.Indices.Reserve(12);
	EndPlaneIndexBuffer.Indices.Append({0, 1, 2});
	EndPlaneIndexBuffer.Indices.Append({0, 2, 3});
	EndPlaneIndexBuffer.Indices.Append({0, 2, 1});
	EndPlaneIndexBuffer.Indices.Append({0, 3, 2});

	EndPlaneVertexBuffers.InitFromDynamicVertex(&EndPlaneVertexFactory, Vertices);

	// Enqueue initialization of render resource
	BeginInitResource(&EndPlaneIndexBuffer);
}

void FTickerSceneProxy::DrawArrowElement(FMeshElementCollector& InCollector, int32 InViewIndex, const FMatrix& InLocalToWorld, FMaterialRenderProxy* InMaterialRenderProxy) const
{
	FMeshBatch& Mesh = InCollector.AllocateMesh();
	Mesh.bWireframe = false;
	Mesh.VertexFactory = &ArrowVertexFactory;
	Mesh.MaterialRenderProxy = InMaterialRenderProxy;
	Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
	Mesh.Type = PT_TriangleList;
	Mesh.DepthPriorityGroup = SDPG_World;
	Mesh.bCanApplyViewModeOverrides = false;

	constexpr bool bShouldReceiveDecals = false;
	constexpr bool bHasPrecomputedVolumetricLightmap = false;

	FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = InCollector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
	DynamicPrimitiveUniformBuffer.Set(InCollector.GetRHICommandList()
		, InLocalToWorld
		, InLocalToWorld
		, GetBounds()
		, GetLocalBounds()
		, bShouldReceiveDecals
		, bHasPrecomputedVolumetricLightmap
		, AlwaysHasVelocity());

	FMeshBatchElement& BatchElement = Mesh.Elements[0];
	BatchElement.IndexBuffer = &ArrowIndexBuffer;
	BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;
	BatchElement.FirstIndex = 0;
	BatchElement.NumPrimitives = ArrowIndexBuffer.Indices.Num() / 3;
	BatchElement.MinVertexIndex = 0;
	BatchElement.MaxVertexIndex = ArrowVertexBuffers.PositionVertexBuffer.GetNumVertices() - 1;

	InCollector.AddMesh(InViewIndex, Mesh);
}

void FTickerSceneProxy::DrawEndPlaneElement(FMeshElementCollector& InCollector, int32 InViewIndex, const FMatrix& InLocalToWorld, FMaterialRenderProxy* InMaterialRenderProxy) const
{
	FMeshBatch& Mesh = InCollector.AllocateMesh();
	Mesh.bWireframe = true;
	Mesh.VertexFactory = &EndPlaneVertexFactory;
	Mesh.MaterialRenderProxy = InMaterialRenderProxy;
	Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
	Mesh.Type = PT_TriangleList;
	Mesh.DepthPriorityGroup = SDPG_World;
	Mesh.bCanApplyViewModeOverrides = false;

	constexpr bool bShouldReceiveDecals = false;
	constexpr bool bHasPrecomputedVolumetricLightmap = false;

	FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = InCollector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
	DynamicPrimitiveUniformBuffer.Set(InCollector.GetRHICommandList()
		, InLocalToWorld
		, InLocalToWorld
		, GetBounds()
		, GetLocalBounds()
		, bShouldReceiveDecals
		, bHasPrecomputedVolumetricLightmap
		, AlwaysHasVelocity());

	FMeshBatchElement& BatchElement = Mesh.Elements[0];
	BatchElement.IndexBuffer = &EndPlaneIndexBuffer;
	BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;
	BatchElement.FirstIndex = 0;
	BatchElement.NumPrimitives = EndPlaneIndexBuffer.Indices.Num() / 3;
	BatchElement.MinVertexIndex = 0;
	BatchElement.MaxVertexIndex = EndPlaneVertexBuffers.PositionVertexBuffer.GetNumVertices() - 1;

	InCollector.AddMesh(InViewIndex, Mesh);
}

} // UE::Avalanche
