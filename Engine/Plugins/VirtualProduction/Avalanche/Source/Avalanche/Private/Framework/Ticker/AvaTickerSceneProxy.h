// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMeshBuilder.h"
#include "PrimitiveSceneProxy.h"
#include "StaticMeshResources.h"

class UAvaTickerComponent;

namespace UE::Avalanche
{

/** Represents a UAvaTickerComponent to the scene manager. */
class FTickerSceneProxy final : public FPrimitiveSceneProxy
{
public:
	explicit FTickerSceneProxy(UAvaTickerComponent* InComponent);

	virtual ~FTickerSceneProxy() override;

	//~ Begin FPrimitiveSceneProxy
	virtual SIZE_T GetTypeHash() const override;
	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& InViews, const FSceneViewFamily& InViewFamily, uint32 InVisibilityMap, FMeshElementCollector& InCollector) const override;
	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* InView) const override;
	virtual uint32 GetMemoryFootprint() const override;
	uint32 GetAllocatedSize() const;
	//~ End FPrimitiveSceneProxy

private:
	void BuildArrowMesh();

	void BuildEndPlaneMesh();

	/** Gets the dynamic mesh element for the start arrow */
	void DrawArrowElement(FMeshElementCollector& InCollector, int32 InViewIndex, const FMatrix& InLocalToWorld, FMaterialRenderProxy* InMaterialRenderProxy) const;

	/** Gets the dynamic mesh element for the ending plane */
	void DrawEndPlaneElement(FMeshElementCollector& InCollector, int32 InViewIndex, const FMatrix& InLocalToWorld, FMaterialRenderProxy* InMaterialRenderProxy) const;

	FStaticMeshVertexBuffers ArrowVertexBuffers;
	FDynamicMeshIndexBuffer32 ArrowIndexBuffer;
	FLocalVertexFactory ArrowVertexFactory;

	FStaticMeshVertexBuffers EndPlaneVertexBuffers;
	FDynamicMeshIndexBuffer32 EndPlaneIndexBuffer;
	FLocalVertexFactory EndPlaneVertexFactory;

	/** The start location (in local space) where active elements start at */
	FVector StartLocation;

	/** The velocity (in local space) to move the active elements at */
	FVector Velocity;

	/** The distance the last part of the active element must cross before it gets destroyed */
	double DestroyDistance;

	/** Draw color for the arrow */
	FColor DrawColor;

	/** Transformation of StartLocation and Velocity in Local Space */
	FMatrix LocalArrowMatrix;

	/** Transformation of End Location and Inverse Velocity in Local Space */
	FMatrix LocalEndPlaneMatrix;
};

} // UE::Avalanche
