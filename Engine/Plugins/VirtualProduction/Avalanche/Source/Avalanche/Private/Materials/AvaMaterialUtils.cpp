// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/AvaMaterialUtils.h"

#include "Containers/ArrayView.h"
#include "Materials/MaterialInterface.h"
#include "Misc/Guid.h"
#include "Templates/Tuple.h"

bool UE::Ava::MaterialHasParameter(const UMaterialInterface& InMaterial, const FName InParameterName, const EMaterialParameterType InParameterType)
{
	FMaterialParameterMetadata Unused;
		
	TArray<FMaterialParameterInfo> ParameterInfos;
	TArray<FGuid> ParameterIds;
	InMaterial.GetAllParameterInfoOfType(InParameterType, ParameterInfos, ParameterIds);

	if (ParameterInfos.IsEmpty())
	{
		return false;
	}

	const int32 FoundParameterIdx = ParameterInfos.IndexOfByPredicate([InParameterName](const FMaterialParameterInfo& InParameterInfo)
	{
		return InParameterInfo.Name == InParameterName;
	});

	if (ParameterIds.IsValidIndex(FoundParameterIdx))
	{
		return ParameterIds[FoundParameterIdx].IsValid();
	}

	return false;
}

bool UE::Ava::MaterialHasParameters(const UMaterialInterface& InMaterial, const TConstArrayView<TPair<FName, EMaterialParameterType>> InParameters, TArray<FString>& OutMissingParameters)
{
	OutMissingParameters.Reserve(InParameters.Num());

	bool bHasAllRequiredParameters = true;
	for (const TPair<FName, EMaterialParameterType>& Parameter : InParameters)
	{
		if (!MaterialHasParameter(InMaterial, Parameter.Key, Parameter.Value))
		{
			bHasAllRequiredParameters = false;
			OutMissingParameters.Add(Parameter.Key.ToString());
		}
	}

	return bHasAllRequiredParameters;
}
