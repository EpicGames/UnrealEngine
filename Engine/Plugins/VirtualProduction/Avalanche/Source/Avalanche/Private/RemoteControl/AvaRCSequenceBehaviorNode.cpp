// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControl/AvaRCSequenceBehaviorNode.h"
#include "AvaLog.h"
#include "AvaSequencePlaybackObject.h"
#include "AvaSequenceSubsystem.h"
#include "Behaviour/RCBehaviour.h"
#include "Engine/Level.h"
#include "RemoteControl/AvaRCSequenceBehavior.h"

#define LOCTEXT_NAMESPACE "AvaRCPlaySequenceBehaviorNode"

UAvaRCSequenceBehaviorNode::UAvaRCSequenceBehaviorNode()
{
	DisplayName = LOCTEXT("DisplayName", "Motion Design Sequence");
	BehaviorDescription = LOCTEXT("Description", "Performs a specified action to Motion Design Sequences (Play, Stop, etc)");
}

bool UAvaRCSequenceBehaviorNode::Execute(URCBehaviour* InBehavior) const
{
	return true;
}

bool UAvaRCSequenceBehaviorNode::IsSupported(URCBehaviour* InBehavior) const
{
	// Only allow behaviors that are embedded in levels (i.e. in embedded presets)
	return InBehavior
		&& InBehavior->IsA<UAvaRCSequenceBehavior>()
		&& InBehavior->GetTypedOuter<ULevel>();
}

void UAvaRCSequenceBehaviorNode::OnPassed(URCBehaviour* InBehavior) const
{
	ULevel* const Level = InBehavior ? InBehavior->GetTypedOuter<ULevel>() : nullptr;
	if (!Level)
	{
		UE_LOGF(LogAva, Error, "Sequence behavior failed. Behavior node %ls could not find a valid outer level.", *GetFullName())
		return;
	}

	const UAvaSequenceSubsystem* const SequenceSubsystem = UAvaSequenceSubsystem::Get(Level);
	if (!SequenceSubsystem)
	{
		UE_LOGF(LogAva, Error, "Sequence behavior failed. Behavior node %ls could not find a valid sequence subsystem.", *GetFullName())
		return;
	}

	IAvaSequencePlaybackObject* const PlaybackObject = SequenceSubsystem->FindPlaybackObject(Level);
	if (!PlaybackObject)
	{
		UE_LOGF(LogAva, Error, "Sequence behavior failed. Behavior node %ls could not find a valid playback object.", *GetFullName());
		return;
	}

	switch (SequenceAction)
	{
	case EAvaSequenceActionType::Play:
		PlaybackObject->PlaySequencesByLabel(SequenceName.Name, PlaySettings);
		break;

	case EAvaSequenceActionType::Continue:
		PlaybackObject->ContinueSequencesByLabel(SequenceName.Name);
		break;

	case EAvaSequenceActionType::Pause:
		PlaybackObject->PauseSequencesByLabel(SequenceName.Name);
		break;

	case EAvaSequenceActionType::Stop:
		PlaybackObject->StopSequencesByLabel(SequenceName.Name);
		break;

	default:
		UE_LOGF(LogAva, Error, "Sequence behavior failed. Behavior node %ls does not have a valid sequence action.", *GetFullName())
		break;
	}
}

UClass* UAvaRCSequenceBehaviorNode::GetBehaviourClass() const
{
	return UAvaRCSequenceBehavior::StaticClass();
}

#undef LOCTEXT_NAMESPACE
