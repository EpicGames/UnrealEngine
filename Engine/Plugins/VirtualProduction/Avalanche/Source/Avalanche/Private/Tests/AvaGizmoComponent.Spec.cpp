// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMeshActor.h"
#include "Framework/AvaGizmoComponent.h"
#include "Misc/AutomationTest.h"
#include "Tests/Framework/AvaTestUtils.h"

BEGIN_DEFINE_SPEC(FAvalancheGizmoComponentSpec, "Avalanche.GizmoComponent",
                  EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	TSharedPtr<FAvaTestUtils> TestUtils = MakeShared<FAvaTestUtils>();

	UAvaGizmoComponent* FindOrAddGizmoComponent(AActor* InActor)
	{
		UAvaGizmoComponent* Component = InActor->GetComponentByClass<UAvaGizmoComponent>();
		if (!Component)
		{
			Component = NewObject<UAvaGizmoComponent>(
				InActor
				, UAvaGizmoComponent::StaticClass()
				, MakeUniqueObjectName(InActor, UAvaGizmoComponent::StaticClass())
				, RF_Transactional);

			InActor->AddInstanceComponent(Component);
			Component->OnComponentCreated();
			Component->RegisterComponent();	
		}

		return Component;
	}

END_DEFINE_SPEC(FAvalancheGizmoComponentSpec);

void FAvalancheGizmoComponentSpec::Define()
{
	BeforeEach([this]
	{
		TestUtils->Init();

		// Spawn actors
	});

	AfterEach([this]
	{
		TestUtils->Destroy();
	});

	Describe("When a non-mesh Actor has an AvaGizmoComponent added to it", [this]
	{
		It("Logs a warning that the owning actor has no supported components", [this]
		{
			AActor* EmptyActor = TestUtils->SpawnActor<AActor>();

			AddExpectedMessage(TEXT("doesn't contain any components applicable"), ELogVerbosity::Warning);

			FindOrAddGizmoComponent(EmptyActor);
		});
	});

	Describe("When a mesh Actor has an AvaGizmoComponent added to it", [this]
	{
		Describe("When HiddenInGame == true, bIsGizmoEnabled == true", [this]
		{
			It("SMC HiddenInGame is true", [this]
			{
				AStaticMeshActor* StaticMeshActor = TestUtils->SpawnActor<AStaticMeshActor>();
				const UStaticMeshComponent* StaticMeshComponent = StaticMeshActor->GetStaticMeshComponent();

				UAvaGizmoComponent* GizmoComponent = FindOrAddGizmoComponent(StaticMeshActor);

				GizmoComponent->SetHiddenInGame(true);
				
				TestTrue(TEXT("Hidden in game"), StaticMeshComponent->bHiddenInGame);
			});
		});

		Describe("When HiddenInGame == true, bIsGizmoEnabled == false", [this]
		{
			It("SMC HiddenInGame is false", [this]
			{
				AStaticMeshActor* StaticMeshActor = TestUtils->SpawnActor<AStaticMeshActor>();
				const UStaticMeshComponent* StaticMeshComponent = StaticMeshActor->GetStaticMeshComponent();

				UAvaGizmoComponent* GizmoComponent = FindOrAddGizmoComponent(StaticMeshActor);

				GizmoComponent->SetHiddenInGame(true);
				GizmoComponent->SetGizmoEnabled(false);
				
				TestFalse(TEXT("Hidden in game"), StaticMeshComponent->bHiddenInGame);			
			});
		});
	});

	Describe("When a mesh Actor has an AvaGizmoComponent removed from it", [this]
	{
		Describe("When HiddenInGame == true, bIsGizmoEnabled == true", [this]
		{
			It("SMC HiddenInGame is false", [this]
			{
				AStaticMeshActor* StaticMeshActor = TestUtils->SpawnActor<AStaticMeshActor>();
				const UStaticMeshComponent* StaticMeshComponent = StaticMeshActor->GetStaticMeshComponent();

				UAvaGizmoComponent* GizmoComponent = FindOrAddGizmoComponent(StaticMeshActor);

				GizmoComponent->SetHiddenInGame(true);
				
				TestTrue(TEXT("Hidden in game"), StaticMeshComponent->bHiddenInGame);

				StaticMeshActor->RemoveInstanceComponent(GizmoComponent);
				GizmoComponent->DestroyComponent();

				TestFalse(TEXT("Hidden in game"), StaticMeshComponent->bHiddenInGame);
			});
		});
	});

	Describe("When a mesh Actor is destroyed that had an AvaGizmoComponent added to it", [this]
	{
		Describe("When bIsGizmoEnabled == true", [this]
		{
			It("Nothing goes wrong", [this]
			{
				AStaticMeshActor* StaticMeshActor = TestUtils->SpawnActor<AStaticMeshActor>();

				FindOrAddGizmoComponent(StaticMeshActor);

				StaticMeshActor->Destroy();
			});
		});

		Describe("When bIsGizmoEnabled == false", [this]
		{
			It("Nothing goes wrong", [this]
			{
				AStaticMeshActor* StaticMeshActor = TestUtils->SpawnActor<AStaticMeshActor>();

				UAvaGizmoComponent* GizmoComponent = FindOrAddGizmoComponent(StaticMeshActor);
				GizmoComponent->SetGizmoEnabled(false);

				StaticMeshActor->Destroy();					
			});
		});
	});
}
