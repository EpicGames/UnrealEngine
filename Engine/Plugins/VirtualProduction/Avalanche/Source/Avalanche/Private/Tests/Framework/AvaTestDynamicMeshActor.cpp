// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Framework/AvaTestDynamicMeshActor.h"

#include "Components/DynamicMeshComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/Object.h"

AAvaTestDynamicMeshActor::AAvaTestDynamicMeshActor()
{
	ShapeDynamicMeshComponent = CreateDefaultSubobject<UDynamicMeshComponent>(TEXT("DynamicMeshComponent"));
	SetRootComponent(ShapeDynamicMeshComponent);
}
