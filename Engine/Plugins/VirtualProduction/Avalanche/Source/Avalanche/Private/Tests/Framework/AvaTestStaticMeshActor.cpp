// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Framework/AvaTestStaticMeshActor.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/Object.h"

AAvaTestStaticMeshActor::AAvaTestStaticMeshActor()
{
	StaticMeshComponent = CreateDefaultSubobject<UStaticMeshComponent>(TEXT("StaticMeshComponent"));
	SetRootComponent(StaticMeshComponent);

	static ConstructorHelpers::FObjectFinder<UStaticMesh> StaticMeshCube(
		TEXT("/Script/Engine.StaticMesh'/Engine/BasicShapes/Cube.Cube'"));
	StaticMeshComponent->SetStaticMesh(StaticMeshCube.Object);
}
