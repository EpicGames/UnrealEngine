// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Framework/AvaTestUtils.h"

#include "Components/DynamicMeshComponent.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Logging/LogVerbosity.h"
#include "Misc/AutomationTest.h"
#include "Tests/Framework/AvaTestDynamicMeshActor.h"
#include "Tests/Framework/AvaTestStaticMeshActor.h"
#include "UDynamicMesh.h"

// General Motion Design unit test log
DEFINE_LOG_CATEGORY(LogAvaTest);

void FAvaTestUtils::Init()
{
	// ensure World is only initialized once
	ensureAlways(!WorldWeak.IsValid());
	WorldWeak = CreateWorld();
}

void FAvaTestUtils::Destroy()
{
	if (UWorld* World = WorldWeak.Get())
	{
		GEngine->DestroyWorldContext(World);
		World->DestroyWorld(true);
		World->MarkAsGarbage();
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		WorldWeak.Reset();
	}
}

UWorld* FAvaTestUtils::CreateWorld()
{
	constexpr EWorldType::Type WorldType = EWorldType::Editor;
	UWorld* const World = UWorld::CreateWorld(WorldType, false, FName(TEXT("AvaTestWorld")));
	FWorldContext& WorldContext = GEngine->CreateNewWorldContext(WorldType);
	WorldContext.SetCurrentWorld(World);
	return World;
}

AAvaTestDynamicMeshActor* FAvaTestUtils::SpawnTestDynamicMeshActor(const FTransform& InTransform)
{
	return SpawnActor<AAvaTestDynamicMeshActor>(InTransform);
}

AAvaTestStaticMeshActor* FAvaTestUtils::SpawnTestStaticMeshActor()
{
	return SpawnActor<AAvaTestStaticMeshActor>();
}

TArray<AAvaTestDynamicMeshActor*> FAvaTestUtils::SpawnTestDynamicMeshActors(
	int32 InNumberOfActors
	, AAvaTestDynamicMeshActor* InParentActor)
{
	TArray<AAvaTestDynamicMeshActor*> GeneratedActors;
	for (int32 ActorIdx = 0; ActorIdx < InNumberOfActors; ++ActorIdx)
	{
		AAvaTestDynamicMeshActor* NewDynamicMeshActor = SpawnTestDynamicMeshActor();
		if (InParentActor)
		{
			NewDynamicMeshActor->AttachToActor(InParentActor, FAttachmentTransformRules::KeepRelativeTransform);
		}

		GeneratedActors.Add(NewDynamicMeshActor);
	}

	return GeneratedActors;
}

void FAvaTestUtils::GenerateRectangleForDynamicMesh(
	AAvaTestDynamicMeshActor* InDynamicMeshActor
	, double InHeight
	, double InWidth)
{
	UDynamicMeshComponent* DynamicMeshComponent = InDynamicMeshActor->FindComponentByClass<UDynamicMeshComponent>();
	if (!DynamicMeshComponent)
	{
		return;
	}

	UDynamicMesh* DynamicMesh = DynamicMeshComponent->GetDynamicMesh();
	if (!DynamicMesh)
	{
		return;
	}

	DynamicMesh->EditMesh([InHeight, InWidth](FDynamicMesh3& InMesh)
	{
		InMesh.Clear();
		InMesh.EnableTriangleGroups();
		InMesh.EnableAttributes();
		InMesh.Attributes()->EnableMaterialID();
		InMesh.Attributes()->EnableTangents();
		InMesh.Attributes()->SetNumPolygroupLayers(1);
		InMesh.Attributes()->SetNumUVLayers(1);
		InMesh.Attributes()->EnablePrimaryColors();

		// 1 - 3
		// 0 - 2
		const int32 LowerRight = InMesh.AppendVertex(FVector3d(0, 0, 0)); // 0
		const int32 UpperRight = InMesh.AppendVertex(FVector3d(0, 0, InHeight)); // 1
		const int32 LowerLeft = InMesh.AppendVertex(FVector3d(0, InWidth, 0)); // 2
		const int32 UpperLeft = InMesh.AppendVertex(FVector3d(0, InWidth, InHeight)); // 3

		InMesh.AppendTriangle(LowerRight, UpperRight, LowerLeft);
		InMesh.AppendTriangle(UpperLeft, LowerLeft, UpperRight);
	});
}

void FAvaTestUtils::LogActorLocation(AActor* InActor)
{
	UE_LOGF(LogAvaTest, Error, "CHILD %ls LOCATION : %ls", *InActor->GetActorNameOrLabel(),
	       *InActor->GetActorTransform().GetLocation().ToString());
}
