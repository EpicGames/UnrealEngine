// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionAttributeLibrary.h"
#include "AvaAttributeContainer.h"
#include "AvaSceneSubsystem.h"
#include "AvaTransitionContext.h"
#include "AvaTransitionLayer.h"
#include "AvaTransitionLayerUtils.h"
#include "AvaTransitionSubsystem.h"
#include "Behavior/AvaTransitionBehaviorInstance.h"
#include "Behavior/AvaTransitionBehaviorInstanceCache.h"
#include "IAvaSceneInterface.h"
#include "IAvaTransitionNodeInterface.h"

namespace UE::Ava::Private
{
	UAvaAttributeContainer* GetAttributeContainer(UObject* InTransitionNode)
	{
		const IAvaTransitionNodeInterface* NodeInterface = Cast<IAvaTransitionNodeInterface>(InTransitionNode);
		if (!NodeInterface)
		{
			return nullptr;
		}

		const FAvaTransitionContext* TransitionContext = NodeInterface->GetBehaviorInstanceCache().GetTransitionContext();
		if (!TransitionContext)
		{
			return nullptr;
		}

		const FAvaTransitionScene* TransitionScene = TransitionContext->GetTransitionScene();
		if (!TransitionScene)
		{
			return nullptr;
		}

		if (IAvaSceneInterface* SceneInterface = UAvaSceneSubsystem::FindSceneInterface(TransitionScene->GetLevel()))
		{
			return SceneInterface->GetAttributeContainer();
		}
		return nullptr;
	}
}

bool UAvaTransitionAttributeLibrary::AddTagAttribute(UObject* InTransitionNode, const FAvaTagHandle& InTagHandle)
{
	if (UAvaAttributeContainer* AttributeContainer = UE::Ava::Private::GetAttributeContainer(InTransitionNode))
	{
		return AttributeContainer->AddTagAttribute(InTagHandle);
	}
	return false;
}

bool UAvaTransitionAttributeLibrary::RemoveTagAttribute(UObject* InTransitionNode, const FAvaTagHandle& InTagHandle)
{
	if (UAvaAttributeContainer* AttributeContainer = UE::Ava::Private::GetAttributeContainer(InTransitionNode))
	{
		return AttributeContainer->RemoveTagAttribute(InTagHandle);
	}
	return false;
}

bool UAvaTransitionAttributeLibrary::ContainsTagAttribute(UObject* InTransitionNode, const FAvaTagHandle& InTagHandle)
{
	if (UAvaAttributeContainer* AttributeContainer = UE::Ava::Private::GetAttributeContainer(InTransitionNode))
	{
		return AttributeContainer->ContainsTagAttribute(InTagHandle);
	}
	return false;
}

bool UAvaTransitionAttributeLibrary::AddNameAttribute(UObject* InTransitionNode, FName InName)
{
	if (UAvaAttributeContainer* AttributeContainer = UE::Ava::Private::GetAttributeContainer(InTransitionNode))
	{
		return AttributeContainer->AddNameAttribute(InName);
	}
	return false;
}

bool UAvaTransitionAttributeLibrary::RemoveNameAttribute(UObject* InTransitionNode, FName InName)
{
	if (UAvaAttributeContainer* AttributeContainer = UE::Ava::Private::GetAttributeContainer(InTransitionNode))
	{
		return AttributeContainer->RemoveNameAttribute(InName);
	}
	return false;
}

bool UAvaTransitionAttributeLibrary::ContainsNameAttribute(UObject* InTransitionNode, FName InName)
{
	if (UAvaAttributeContainer* AttributeContainer = UE::Ava::Private::GetAttributeContainer(InTransitionNode))
	{
		return AttributeContainer->ContainsNameAttribute(InName);
	}
	return false;
}
