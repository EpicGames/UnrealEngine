// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "AvaTransitionAttributeLibrary.generated.h"

enum class EAvaTransitionLayerCompareType : uint8;
enum class EAvaTransitionSceneType : uint8;
struct FAvaTagHandle;
struct FAvaTagHandleContainer;

UCLASS()
class UAvaTransitionAttributeLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Scene Attributes", meta=(DefaultToSelf="InTransitionNode"))
	static bool AddTagAttribute(UObject* InTransitionNode, const FAvaTagHandle& InTagHandle);

	UFUNCTION(BlueprintCallable, Category="Scene Attributes", meta=(DefaultToSelf="InTransitionNode"))
	static bool RemoveTagAttribute(UObject* InTransitionNode, const FAvaTagHandle& InTagHandle);

	UFUNCTION(BlueprintCallable, Category="Scene Attributes", meta=(DefaultToSelf="InTransitionNode"))
	static bool ContainsTagAttribute(UObject* InTransitionNode, const FAvaTagHandle& InTagHandle);

	UFUNCTION(BlueprintCallable, Category="Scene Attributes", meta=(DefaultToSelf="InTransitionNode"))
	static bool AddNameAttribute(UObject* InTransitionNode, FName InName);

	UFUNCTION(BlueprintCallable, Category="Scene Attributes", meta=(DefaultToSelf="InTransitionNode"))
	static bool RemoveNameAttribute(UObject* InTransitionNode, FName InName);

	UFUNCTION(BlueprintCallable, Category="Scene Attributes", meta=(DefaultToSelf="InTransitionNode"))
	static bool ContainsNameAttribute(UObject* InTransitionNode, FName InName);
};
