// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transition/AvaTransitionRCLibrary.h"
#include "AvaRCControllerId.h"
#include "AvaSceneSubsystem.h"
#include "AvaTransitionContext.h"
#include "AvaTransitionLayer.h"
#include "AvaTransitionLayerUtils.h"
#include "AvaTransitionSubsystem.h"
#include "Behavior/AvaTransitionBehaviorInstance.h"
#include "Behavior/AvaTransitionBehaviorInstanceCache.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "IAvaSceneInterface.h"
#include "IAvaTransitionNodeInterface.h"
#include "RCVirtualProperty.h"
#include "RemoteControlPreset.h"
#include "Transition/Extensions/IAvaTransitionRCExtension.h"

namespace UE::Ava::Private
{
	URCVirtualPropertyBase* GetController(const FAvaRCControllerId& InControllerId, const UAvaSceneSubsystem& InSceneSubsystem, ULevel* InLevel)
	{
		IAvaSceneInterface* SceneInterface = InSceneSubsystem.GetSceneInterface(InLevel);
		if (!SceneInterface)
		{
			return nullptr;
		}

		if (URemoteControlPreset* RemoteControlPreset = SceneInterface->GetRemoteControlPreset())
		{
			return InControllerId.FindController(RemoteControlPreset);
		}

		return nullptr;
	}
}

bool UAvaTransitionRCLibrary::CompareRCControllerValues(const FAvaTransitionContext& InTransitionContext, const FAvaRCControllerId& InControllerId, EAvaTransitionComparisonResult InValueComparisonType)
{
	const FAvaTransitionScene* TransitionScene = InTransitionContext.GetTransitionScene();
	if (!TransitionScene)
	{
		return false;
	}

	ULevel* Level = TransitionScene->GetLevel();
	if (!Level || !Level->OwningWorld)
	{
		return false;
	}

	UAvaTransitionSubsystem* TransitionSubsystem = Level->OwningWorld->GetSubsystem<UAvaTransitionSubsystem>();
	UAvaSceneSubsystem* SceneSubsystem = Level->OwningWorld->GetSubsystem<UAvaSceneSubsystem>();
	if (!TransitionSubsystem || !SceneSubsystem)
	{
		return false;
	}

	URCVirtualPropertyBase* Controller = UE::Ava::Private::GetController(InControllerId, *SceneSubsystem, Level);
	if (!Controller)
	{
		return false;
	}

	// Get all the Behavior Instances in the same Layer
	TArray<const FAvaTransitionBehaviorInstance*> BehaviorInstances;
	{
		FAvaTransitionLayerComparator Comparator = FAvaTransitionLayerUtils::BuildComparator(InTransitionContext, EAvaTransitionLayerCompareType::Same, FAvaTagHandle());
		BehaviorInstances = FAvaTransitionLayerUtils::QueryBehaviorInstances(*TransitionSubsystem, Comparator);
	}

	if (BehaviorInstances.IsEmpty())
	{
		return false;
	}

	// Optional Extension to override Controller Comparison
	IAvaRCTransitionExtension* const RCTransitionExtension = TransitionScene->FindExtension<IAvaRCTransitionExtension>();

	for (const FAvaTransitionBehaviorInstance* BehaviorInstance : BehaviorInstances)
	{
		check(BehaviorInstance);

		const FAvaTransitionContext& OtherTransitionContext = BehaviorInstance->GetTransitionContext();

		const FAvaTransitionScene* OtherTransitionScene = OtherTransitionContext.GetTransitionScene();
		if (!OtherTransitionScene)
		{
			continue;
		}

		EAvaTransitionComparisonResult Result;
		if (RCTransitionExtension)
		{
			Result = RCTransitionExtension->CompareControllers(Controller->Id
				, InTransitionContext
				, OtherTransitionContext);
		}
		else if (URCVirtualPropertyBase* OtherController = UE::Ava::Private::GetController(InControllerId, *SceneSubsystem, OtherTransitionScene->GetLevel()))
		{
			Result = Controller->IsValueEqual(OtherController)
				? EAvaTransitionComparisonResult::Same
				: EAvaTransitionComparisonResult::Different;
		}
		else
		{
			Result = EAvaTransitionComparisonResult::None;
		}

		if (InValueComparisonType == Result)
		{
			return true;
		}
	}

	return false;
}

bool UAvaTransitionRCLibrary::CompareRCControllerValues(UObject* InTransitionNode, const FAvaRCControllerId& InControllerId, EAvaTransitionComparisonResult InValueComparisonType)
{
	IAvaTransitionNodeInterface* NodeInterface = Cast<IAvaTransitionNodeInterface>(InTransitionNode);
	if (!NodeInterface)
	{
		return false;
	}

	const FAvaTransitionContext* TransitionContext = NodeInterface->GetBehaviorInstanceCache().GetTransitionContext();
	if (!TransitionContext)
	{
		return false;
	}

	return CompareRCControllerValues(*TransitionContext, InControllerId, InValueComparisonType);
}

TArray<URCVirtualPropertyBase*> UAvaTransitionRCLibrary::GetChangedRCControllers(UObject* InTransitionNode)
{
	IAvaTransitionNodeInterface* NodeInterface = Cast<IAvaTransitionNodeInterface>(InTransitionNode);
	if (!NodeInterface)
	{
		return {};
	}

	const FAvaTransitionContext* TransitionContext = NodeInterface->GetBehaviorInstanceCache().GetTransitionContext();
	if (!TransitionContext)
	{
		return {};
	}

	const FAvaTransitionScene* TransitionScene = TransitionContext->GetTransitionScene();
	if (!TransitionScene)
	{
		return {};
	}

	ULevel* Level = TransitionScene->GetLevel();
	if (!Level || !Level->OwningWorld)
	{
		return {};
	}

	UAvaSceneSubsystem* SceneSubsystem = Level->OwningWorld->GetSubsystem<UAvaSceneSubsystem>();
	if (!SceneSubsystem)
	{
		return {};
	}

	IAvaSceneInterface* SceneInterface = SceneSubsystem->GetSceneInterface(Level);
	if (!SceneInterface)
	{
		return {};
	}

	URemoteControlPreset* RemoteControlPreset = SceneInterface->GetRemoteControlPreset();
	if (!RemoteControlPreset)
	{
		return {};
	}

	TArray<URCVirtualPropertyBase*> Controllers = RemoteControlPreset->GetControllers();

	// Remove Invalid Controllers and Controllers that are not different/changed between scenes
	Controllers.RemoveAll(
		[TransitionContext](URCVirtualPropertyBase* InController)
		{
			if (!InController)
			{
				return true;
			}
			return !CompareRCControllerValues(*TransitionContext, FAvaRCControllerId(InController), EAvaTransitionComparisonResult::Different);
		});

	Controllers.StableSort(
		[](const URCVirtualPropertyBase& A, const URCVirtualPropertyBase& B)
		{
			return A.DisplayIndex < B.DisplayIndex;
		});

	return Controllers;
}
