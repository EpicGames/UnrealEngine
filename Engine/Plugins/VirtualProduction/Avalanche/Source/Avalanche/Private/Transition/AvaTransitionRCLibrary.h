// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionEnums.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "AvaTransitionRCLibrary.generated.h"

class IAvaTransitionNodeInterface;
class UAvaSceneSubsystem;
class URCVirtualPropertyBase;
struct FAvaRCControllerId;
struct FAvaTransitionContext;
struct FAvaTransitionScene;

UCLASS()
class UAvaTransitionRCLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	static bool CompareRCControllerValues(const FAvaTransitionContext& InTransitionContext, const FAvaRCControllerId& InControllerId, EAvaTransitionComparisonResult InValueComparisonType);

	UFUNCTION(BlueprintPure, DisplayName="Compare RC Controller Values", Category="Transition Logic|Remote Control", meta=(DefaultToSelf="InTransitionNode"))
	static bool CompareRCControllerValues(UObject* InTransitionNode, const FAvaRCControllerId& InControllerId, EAvaTransitionComparisonResult InValueComparisonType = EAvaTransitionComparisonResult::Different);

	UFUNCTION(BlueprintCallable, DisplayName="Get Changed RC Controllers", Category="Transition Logic|Remote Control", meta=(DefaultToSelf="InTransitionNode"))
	static TArray<URCVirtualPropertyBase*> GetChangedRCControllers(UObject* InTransitionNode);
};
