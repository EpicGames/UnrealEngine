// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transition/Conditions/AvaSceneContainsTagAttributeCondition.h"
#include "AvaAttributeContainer.h"
#include "AvaSceneSubsystem.h"
#include "AvaTransitionLayer.h"
#include "AvaTransitionLayerUtils.h"
#include "AvaTransitionSubsystem.h"
#include "AvaTransitionUtils.h"
#include "Behavior/AvaTransitionBehaviorInstance.h"
#include "IAvaSceneInterface.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeLinker.h"

#define LOCTEXT_NAMESPACE "AvaSceneContainsTagAttributeConditionBase"

#if WITH_EDITOR
FText FAvaSceneContainsTagAttributeConditionBase::GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const
{
	const FInstanceDataType& InstanceData = InInstanceDataView.Get<FInstanceDataType>();

	FFormatNamedArguments Arguments;

	switch (InstanceData.SceneType)
	{
	case EAvaTransitionSceneType::This:
		{
			if (InFormatting == EStateTreeNodeFormatting::RichText)
			{
				Arguments.Add(TEXT("IndefinitePronoun"), FText::GetEmpty());
				Arguments.Add(TEXT("Scene"), LOCTEXT("ThisSceneRich", "<b>this</> <s>scene</>"));
				Arguments.Add(TEXT("Contains"), bInvertCondition ? LOCTEXT("ThisDoesntContainRich", "<s>does</> <b>not</> <s>contain</>") : LOCTEXT("ThisContainsRich", "<s>contains</>"));
			}
			else
			{
				Arguments.Add(TEXT("IndefinitePronoun"), FText::GetEmpty());
				Arguments.Add(TEXT("Scene"), LOCTEXT("ThisScene", "this scene"));
				Arguments.Add(TEXT("Contains"), bInvertCondition ? LOCTEXT("ThisDoesntContain", "does not contain") : LOCTEXT("ThisContains", "contains"));
			}
		}
		break;

	case EAvaTransitionSceneType::Other:
		{
			FText LayerDesc;
			{
				FAvaTransitionLayerUtils::FLayerQueryTextParams Params;
				Params.LayerType = InstanceData.LayerType;
				Params.SpecificLayerName = *InstanceData.SpecificLayers.ToString();
				Params.LayerTypePropertyName = GET_MEMBER_NAME_CHECKED(FInstanceDataType, LayerType);
				Params.SpecificLayerPropertyName = GET_MEMBER_NAME_CHECKED(FInstanceDataType, SpecificLayers);

				LayerDesc = FAvaTransitionLayerUtils::GetLayerQueryText(MoveTemp(Params), InId, InBindingLookup, InFormatting);
			}

			if (InFormatting == EStateTreeNodeFormatting::RichText)
			{
				Arguments.Add(TEXT("IndefinitePronoun"), bInvertCondition ? LOCTEXT("NoSceneRich", "<b>no</> ") : LOCTEXT("AnySceneRich", "<s>a</> "));
				Arguments.Add(TEXT("Scene"), FText::Format(LOCTEXT("OtherSceneRich", "<s>scene in</> {0}"), LayerDesc));
				Arguments.Add(TEXT("Contains"), LOCTEXT("OtherSceneContainsRich", "<s>contains</>"));
			}
			else
			{
				Arguments.Add(TEXT("IndefinitePronoun"), bInvertCondition ? LOCTEXT("NoScene", "no ") : LOCTEXT("AnyScene", "a "));
				Arguments.Add(TEXT("Scene"), FText::Format(LOCTEXT("OtherScene", "scene in {0}"), LayerDesc));
				Arguments.Add(TEXT("Contains"), LOCTEXT("OtherSceneContains", "contains"));
			}
		}
		break;
	}

	Arguments.Add(TEXT("TagAttribute"), FText::FromName(InstanceData.TagAttribute.ToName()));

	return InFormatting == EStateTreeNodeFormatting::RichText
		? FText::Format(LOCTEXT("DescRich", "{IndefinitePronoun}{Scene} {Contains} <s>tag attribute</> <b>'{TagAttribute}'</>"), Arguments)
		: FText::Format(LOCTEXT("Desc", "{IndefinitePronoun}{Scene} {Contains} tag attribute '{TagAttribute}'"), Arguments);
}
#endif

bool FAvaSceneContainsTagAttributeConditionBase::Link(FStateTreeLinker& InLinker)
{
	FAvaTransitionCondition::Link(InLinker);
	InLinker.LinkExternalData(SceneSubsystemHandle);
	return true;
}

bool FAvaSceneContainsTagAttributeConditionBase::TestCondition(FStateTreeExecutionContext& InContext) const
{
	return ContainsTagAttribute(InContext) ^ bInvertCondition;
}

bool FAvaSceneContainsTagAttributeConditionBase::ContainsTagAttribute(FStateTreeExecutionContext& InContext) const
{
	TArray<const FAvaTransitionScene*> TransitionScenes = GetTransitionScenes(InContext);
	if (TransitionScenes.IsEmpty())
	{
		return false;
	}

	const FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);
	const UAvaSceneSubsystem& SceneSubsystem = InContext.GetExternalData(SceneSubsystemHandle);

	for (const FAvaTransitionScene* TransitionScene : TransitionScenes)
	{
		if (!TransitionScene)
		{
			continue;
		}

		const IAvaSceneInterface* Scene = SceneSubsystem.GetSceneInterface(TransitionScene->GetLevel());
		if (!Scene)
		{
			continue;
		}

		const UAvaAttributeContainer* AttributeContainer = Scene->GetAttributeContainer();
		if (AttributeContainer && AttributeContainer->ContainsTagAttribute(InstanceData.TagAttribute))
		{
			return true;
		}
	}

	return false;
}

TArray<const FAvaTransitionScene*> FAvaSceneContainsTagAttributeConditionBase::GetTransitionScenes(FStateTreeExecutionContext& InContext) const
{
	const FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);
	const FAvaTransitionContext& TransitionContext = InContext.GetExternalData(TransitionContextHandle);

	if (InstanceData.SceneType == EAvaTransitionSceneType::This)
	{
		return { TransitionContext.GetTransitionScene() };
	}

	ensureMsgf(InstanceData.SceneType == EAvaTransitionSceneType::Other
		, TEXT("FAvaSceneContainsAttributeCondition::GetTargetSceneContexts did not recognize the provided transition scene type: %s")
		, *UEnum::GetValueAsString(InstanceData.SceneType));

	// Get all the Behavior Instances from Query 
	TArray<const FAvaTransitionBehaviorInstance*> BehaviorInstances;
	{
		UAvaTransitionSubsystem& TransitionSubsystem = InContext.GetExternalData(TransitionSubsystemHandle);

		FAvaTransitionLayerComparator Comparator = FAvaTransitionLayerUtils::BuildComparator(TransitionContext, InstanceData.LayerType, InstanceData.SpecificLayers);

		BehaviorInstances = FAvaTransitionLayerUtils::QueryBehaviorInstances(TransitionSubsystem, Comparator);
	}

	TArray<const FAvaTransitionScene*> TransitionScenes;
	TransitionScenes.Reserve(BehaviorInstances.Num());

	for (const FAvaTransitionBehaviorInstance* BehaviorInstance : BehaviorInstances)
	{
		const FAvaTransitionContext& OtherTransitionContext = BehaviorInstance->GetTransitionContext();
		const FAvaTransitionScene* TransitionScene = OtherTransitionContext.GetTransitionScene();

		// do not add if scene is marked as needing discard
		if (TransitionScene && !TransitionScene->HasAllFlags(EAvaTransitionSceneFlags::NeedsDiscard))
		{
			TransitionScenes.AddUnique(TransitionScene);
		}
	}

	return TransitionScenes;
}

#undef LOCTEXT_NAMESPACE
