// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transition/Conditions/AvaTransitionRCControllerMatchCondition.h"
#include "AvaSceneSubsystem.h"
#include "AvaTransitionContext.h"
#include "AvaTransitionLog.h"
#include "AvaTransitionUtils.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeLinker.h"
#include "Transition/AvaTransitionRCLibrary.h"

#define LOCTEXT_NAMESPACE "AvaTransitionRCControllerMatchCondition"

#if WITH_EDITOR
FText FAvaTransitionRCControllerMatchCondition::GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const
{
	const FInstanceDataType& InstanceData = InInstanceDataView.Get<FInstanceDataType>();

	const FText ControllerIdText = InstanceData.ControllerId.ToText();
	const FText ComparisonText = UEnum::GetDisplayValueAsText(InstanceData.ValueComparisonType).ToLower();

	return InFormatting == EStateTreeNodeFormatting::RichText
		? FText::Format(LOCTEXT("DescRich", "'<b>{0}</>' <s>is</> <b>{1}</>"), ControllerIdText, ComparisonText)
		: FText::Format(LOCTEXT("Desc", "'{0}' is {1}"), ControllerIdText, ComparisonText);
}
#endif

bool FAvaTransitionRCControllerMatchCondition::Link(FStateTreeLinker& InLinker)
{
	FAvaTransitionCondition::Link(InLinker);
	InLinker.LinkExternalData(SceneSubsystemHandle);
	return true;
}

bool FAvaTransitionRCControllerMatchCondition::TestCondition(FStateTreeExecutionContext& InContext) const
{
	const FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);

	return UAvaTransitionRCLibrary::CompareRCControllerValues(InContext.GetExternalData(TransitionContextHandle)
		, InstanceData.ControllerId
		, InstanceData.ValueComparisonType);
}

#undef LOCTEXT_NAMESPACE
