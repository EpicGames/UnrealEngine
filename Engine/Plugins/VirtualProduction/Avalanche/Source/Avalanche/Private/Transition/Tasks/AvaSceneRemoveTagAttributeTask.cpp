// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transition/Tasks/AvaSceneRemoveTagAttributeTask.h"
#include "AvaAttributeContainer.h"
#include "IAvaSceneInterface.h"
#include "StateTreeExecutionContext.h"

#define LOCTEXT_NAMESPACE "AvaSceneRemoveTagAttributeTask"

#if WITH_EDITOR
FText FAvaSceneRemoveTagAttributeTask::GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const
{
	const FInstanceDataType& InstanceData = InInstanceDataView.Get<FInstanceDataType>();

	const FText TagAttributeDesc = FText::FromName(InstanceData.TagAttribute.ToName());

	return InFormatting == EStateTreeNodeFormatting::RichText
		? FText::Format(LOCTEXT("DescRich", " <b>Remove'{0}'</> <s>tag attribute from this scene</>"), TagAttributeDesc)
		: FText::Format(LOCTEXT("Desc", "Remove '{0}' tag attribute from this scene"), TagAttributeDesc);
}
#endif

EStateTreeRunStatus FAvaSceneRemoveTagAttributeTask::EnterState(FStateTreeExecutionContext& InContext, const FStateTreeTransitionResult& InTransition) const
{
	IAvaSceneInterface* Scene = GetScene(InContext);
	if (!Scene)
	{
		return EStateTreeRunStatus::Failed;
	}

	UAvaAttributeContainer* AttributeContainer = Scene->GetAttributeContainer();
	if (!AttributeContainer)
	{
		return EStateTreeRunStatus::Failed;
	}

	const FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);
	if (AttributeContainer->RemoveTagAttribute(InstanceData.TagAttribute))
	{
		return EStateTreeRunStatus::Succeeded;
	}

	return EStateTreeRunStatus::Failed;
}

#undef LOCTEXT_NAMESPACE
