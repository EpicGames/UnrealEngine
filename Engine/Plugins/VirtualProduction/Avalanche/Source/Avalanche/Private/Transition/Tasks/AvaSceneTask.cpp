// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transition/Tasks/AvaSceneTask.h"
#include "AvaSceneSubsystem.h"
#include "AvaTransitionContext.h"
#include "AvaTransitionUtils.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeLinker.h"

bool FAvaSceneTask::Link(FStateTreeLinker& InLinker)
{
	Super::Link(InLinker);
	InLinker.LinkExternalData(SceneSubsystemHandle);
	return true;
}

IAvaSceneInterface* FAvaSceneTask::GetScene(FStateTreeExecutionContext& InContext) const
{
	const UAvaSceneSubsystem& SceneSubsystem       = InContext.GetExternalData(SceneSubsystemHandle);
	const FAvaTransitionContext& TransitionContext = InContext.GetExternalData(TransitionContextHandle);
	const FAvaTransitionScene* TransitionScene     = TransitionContext.GetTransitionScene();

	return TransitionScene ? SceneSubsystem.GetSceneInterface(TransitionScene->GetLevel()) : nullptr;
}
