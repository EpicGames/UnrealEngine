// Copyright Epic Games, Inc. All Rights Reserved.

#include "Viewport/AvaCineCameraActor.h"

#include "CineCameraComponent.h"

AAvaCineCameraActor::AAvaCineCameraActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, DefaultCameraDistance(500.0f) // Same value as UAvaEditorSettings::CameraDistance
{
	/* todo: when the reset to defaults issue will be solved, Configure(GetDefaultCameraDistance()) could be called here to to initialize defaults */
}

void AAvaCineCameraActor::Configure(float InCameraDistance)
{
	// initialize default camera distance - todo: we might use this value later for a custom reset?
	DefaultCameraDistance = InCameraDistance;
		
	if (UCineCameraComponent* const CineCameraComp = GetCineCameraComponent())
	{
		CineCameraComp->SetFieldOfView(90.0f);

		// smaller camera mesh, todo: this will be substituted with a custom visualizer, or other solution, to visualize frustum + direction
		CineCameraComp->SetWorldScale3D(FVector(0.5f));

		// using default ManualFocusDistance value from Default CineCameraComponent, since that is handled by UAvaEditorSettings
		CineCameraComp->FocusSettings.ManualFocusDistance = DefaultCameraDistance;

		FVector CameraPosition = FVector::ZeroVector;
		CameraPosition.X = -DefaultCameraDistance;
		SetActorLocation(CameraPosition);
	}
}
