// Copyright Epic Games, Inc. All Rights Reserved.

#include "Viewport/AvaPostProcessVolume.h"

AAvaPostProcessVolume::AAvaPostProcessVolume()
{
	BlendWeight = 1.f;
	bUnbound = true;

	Settings.bOverride_AutoExposureBias = 1;
	Settings.AutoExposureBias = 1.f;

	Settings.bOverride_AutoExposureMinBrightness = 1;
	Settings.AutoExposureMinBrightness = 1.f;

	Settings.bOverride_AutoExposureMaxBrightness = 1;
	Settings.AutoExposureMaxBrightness = 1.f;

	Settings.bOverride_ExpandGamut = 1;
	Settings.ExpandGamut = 0.f;

	Settings.bOverride_ToneCurveAmount = 1;
	Settings.ToneCurveAmount = 0.f;

	Settings.bOverride_BlueCorrection = 1;
	Settings.BlueCorrection = 0.f;

	Settings.bOverride_VignetteIntensity = 1;
	Settings.VignetteIntensity = 0.f;
}
