// Copyright Epic Games, Inc. All Rights Reserved.

#include "Viewport/AvaViewportQualitySettings.h"
#include "ShowFlags.h"

#define LOCTEXT_NAMESPACE "AvaViewportQualitySettings"

const FText FAvaViewportQualitySettingsPreset::NoLumen = LOCTEXT("PresetNoLumen", "No Lumen");
const FText FAvaViewportQualitySettingsPreset::Reduced = LOCTEXT("PresetReduced", "Reduced");

FAvaViewportQualitySettings::FAvaViewportQualitySettings()
{
	Features = DefaultFeatures();
}

FAvaViewportQualitySettings::FAvaViewportQualitySettings(ENoInit NoInit)
{
}

FAvaViewportQualitySettings::FAvaViewportQualitySettings(const bool bInUseAllFeatures)
{
	Features = AllFeatures(bInUseAllFeatures);
}

FAvaViewportQualitySettings::FAvaViewportQualitySettings(const FEngineShowFlags& InShowFlags)
{
	Features = DefaultFeatures();

	TSet<FString> InvalidFeatureNames;

	// Set the enabled state of the feature based on value in FEngineShowFlags.
	for (FAvaViewportQualitySettingsFeature& Feature : Features)
	{
		const int32 FeatureIndex = InShowFlags.FindIndexByName(*Feature.Name);
		if (FeatureIndex == INDEX_NONE)
		{
			InvalidFeatureNames.Add(Feature.Name);
			continue;
		}

		Feature.bEnabled = InShowFlags.GetSingleFlag(FeatureIndex);
	}

	SortFeaturesByDisplayText();

	check(InvalidFeatureNames.IsEmpty());
}

FAvaViewportQualitySettings::FAvaViewportQualitySettings(const TArray<FAvaViewportQualitySettingsFeature>& InFeatures)
{
	Features = InFeatures;

	SortFeaturesByDisplayText();
}

bool FAvaViewportQualitySettings::operator==(const FAvaViewportQualitySettings& InOther) const
{
	// All features must exist and match the same value in the other struct to be equal
	for (const FAvaViewportQualitySettingsFeature& Feature : Features)
	{
		const FAvaViewportQualitySettingsFeature* const FoundFeaturePtr = FindFeatureByName(InOther.Features, Feature.Name);
		if (!FoundFeaturePtr || FoundFeaturePtr->bEnabled != Feature.bEnabled)
		{
			return false;
		}
	}

	return true;
}

bool FAvaViewportQualitySettings::operator!=(const FAvaViewportQualitySettings& InOther) const
{
	return !(*this == InOther);
}

TArray<FAvaViewportQualitySettingsFeature> FAvaViewportQualitySettings::DefaultFeatures()
{
	// Add all the engine features we will support for editing by the designer and their Motion Design defaults.
	TArray<FAvaViewportQualitySettingsFeature> AllDefaultFeatures =
		{
			{ "AntiAliasing", true },
			{ "TemporalAA", true },
			{ "AmbientCubemap", true },
			{ "EyeAdaptation", true },
			{ "LensFlares", true },
			{ "GlobalIllumination", true },
			{ "AmbientOcclusion", true },
			{ "DirectLighting", true },
			{ "DepthOfField", true },
			{ "MotionBlur", true },
			{ "SeparateTranslucency", false },
			{ "ReflectionEnvironment", true },
			{ "ScreenSpaceReflections", true },
			{ "LumenReflections", false },
			{ "ContactShadows", true },
			{ "RayTracedDistanceFieldShadows", true },
			{ "CapsuleShadows", true },
			{ "SubsurfaceScattering", true },
			{ "VolumetricLightmap", true },
			{ "IndirectLightingCache", true },
			{ "TexturedLightProfiles", true },
			{ "DynamicShadows", true },
			{ "Translucency", true },
			{ "LightShafts", true },
			{ "PrecomputedVisibility", true },
			{ "ScreenSpaceAO", true },
			{ "DistanceFieldAO", true },
			{ "LumenGlobalIllumination", false },
			{ "VolumetricFog", true },
			{ "DisableOcclusionQueries", true },
			{ "LumenDetailTraces", false },
			{ "LumenGlobalTraces", false },
			{ "LumenFarFieldTraces", false },
			{ "LumenSecondaryBounces", false },
			{ "LumenScreenSpaceDirectionalOcclusion", false },
			{ "LumenReuseShadowMaps", false }
		};

	SortFeaturesByDisplayText(AllDefaultFeatures);

	return AllDefaultFeatures;
}

TArray<FAvaViewportQualitySettingsFeature> FAvaViewportQualitySettings::AllFeatures(const bool bUseAllFeatures)
{
	TArray<FAvaViewportQualitySettingsFeature> AllDefaultFeatures = DefaultFeatures();

	for (FAvaViewportQualitySettingsFeature& Feature : AllDefaultFeatures)
	{
		Feature.bEnabled = bUseAllFeatures;
	}

	SortFeaturesByDisplayText(AllDefaultFeatures);

	return AllDefaultFeatures;
}

FAvaViewportQualitySettings FAvaViewportQualitySettings::Default()
{
	return FAvaViewportQualitySettings();
}

FAvaViewportQualitySettings FAvaViewportQualitySettings::Preset(const FText& InPresetName)
{
	FAvaViewportQualitySettings PresetSettings(true);

	if (InPresetName.EqualTo(FAvaViewportQualitySettingsPreset::NoLumen))
	{
		PresetSettings.EnableFeaturesByName(false, 
			{
				"LumenReflections",
				"LumenGlobalIllumination",
				"LumenScreenTraces",
				"LumenDetailTraces",
				"LumenGlobalTraces",
				"LumenFarFieldTraces",
				"LumenSecondaryBounces",
				"LumenScreenSpaceDirectionalOcclusion",
				"LumenReuseShadowMaps"
			});
	}
	else if (InPresetName.EqualTo(FAvaViewportQualitySettingsPreset::Reduced))
	{
		PresetSettings.EnableFeaturesByName(false,
			{
				"AmbientOcclusion",
				"AntiAliasing",
				"TemporalAA",
				"LumenReflections",
				"LumenGlobalIllumination",
				"LumenScreenTraces",
				"LumenDetailTraces",
				"LumenGlobalTraces",
				"LumenFarFieldTraces",
				"LumenSecondaryBounces",
				"LumenScreenSpaceDirectionalOcclusion",
				"LumenReuseShadowMaps"
			});
	}

	return PresetSettings;
}

FAvaViewportQualitySettings FAvaViewportQualitySettings::All(const bool bUseAllFeatures)
{
	FAvaViewportQualitySettings DefaultSettings = Default();
	for (FAvaViewportQualitySettingsFeature& Feature : DefaultSettings.Features)
	{
		Feature.bEnabled = bUseAllFeatures;
	}
	return DefaultSettings;
}

void FAvaViewportQualitySettings::Apply(FEngineShowFlags& InFlags)
{
	VerifyIntegrity(Features);

	for (const FAvaViewportQualitySettingsFeature& Feature : Features)
	{
		const int32 FeatureIndex = InFlags.FindIndexByName(*Feature.Name);
		if (FeatureIndex != INDEX_NONE)
		{
			InFlags.SetSingleFlag(FeatureIndex, Feature.bEnabled);
		}
	}
}

void FAvaViewportQualitySettings::VerifyIntegrity(TArray<FAvaViewportQualitySettingsFeature>& InFeatures)
{
	TArray<FAvaViewportQualitySettingsFeature> AllDefaultFeatures = DefaultFeatures();

	// Remove any old feature names that are no longer used.
	for (TArray<FAvaViewportQualitySettingsFeature>::TIterator FeatureIt(InFeatures); FeatureIt; ++FeatureIt)
	{
		if (!AllDefaultFeatures.Contains(*FeatureIt))
		{
			FeatureIt.RemoveCurrent();
		}
	}

	// Add any new feature names that have been added to the default feature map.
	for (FAvaViewportQualitySettingsFeature& Feature : AllDefaultFeatures)
	{
		if (!InFeatures.Contains(Feature))
		{
			InFeatures.AddUnique(Feature);
		}
	}
}

void FAvaViewportQualitySettings::VerifyIntegrity()
{
	VerifyIntegrity(Features);
}

void FAvaViewportQualitySettings::SortFeaturesByDisplayText(TArray<FAvaViewportQualitySettingsFeature>& InFeatures)
{
	InFeatures.Sort([](const FAvaViewportQualitySettingsFeature& FeatureA, const FAvaViewportQualitySettingsFeature& FeatureB)
		{
			FText NameTextA;
			FText TooltipTextA;
			FeatureNameAndTooltipText(FeatureA.Name, NameTextA, TooltipTextA);

			FText NameTextB;
			FText TooltipTextB;
			FeatureNameAndTooltipText(FeatureB.Name, NameTextB, TooltipTextB);

			return NameTextA.CompareToCaseIgnored(NameTextB) < 0;
		});
}
void FAvaViewportQualitySettings::SortFeaturesByDisplayText()
{
	SortFeaturesByDisplayText(Features);
}

void FAvaViewportQualitySettings::EnableFeaturesByName(const bool bInEnabled, const TArray<FString>& InFeatureNames)
{
	TArray<FAvaViewportQualitySettingsFeature> AllDefaultFeatures = DefaultFeatures();

	for (const FString& FeatureName : InFeatureNames)
	{
		const FAvaViewportQualitySettingsFeature* DefaultFeature = FindFeatureByName(AllDefaultFeatures, FeatureName);
		if (DefaultFeature)
		{
			if (FAvaViewportQualitySettingsFeature* Feature = FindFeatureByName(Features, FeatureName))
			{
				Feature->bEnabled = bInEnabled;
			}
		}
	}
}

FAvaViewportQualitySettingsFeature* FAvaViewportQualitySettings::FindFeatureByName(TArray<FAvaViewportQualitySettingsFeature>& InFeatures, const FString& InFeatureName)
{
	return InFeatures.FindByPredicate([&InFeatureName](FAvaViewportQualitySettingsFeature& InFeature)
		{
			return InFeature.Name.Equals(InFeatureName);
		});
}

const FAvaViewportQualitySettingsFeature* FAvaViewportQualitySettings::FindFeatureByName(const TArray<FAvaViewportQualitySettingsFeature>& InFeatures, const FString& InFeatureName)
{
	return InFeatures.FindByPredicate([&InFeatureName](FAvaViewportQualitySettingsFeature& InFeature)
		{
			return InFeature.Name.Equals(InFeatureName);
		});
}

void FAvaViewportQualitySettings::FeatureNameAndTooltipText(const FString& InFeatureName, FText& OutNameText, FText& OutTooltipText)
{
	//FEngineShowFlags::FindShowFlagDisplayName(InFeatureName, OutNameText);

	if (InFeatureName.Equals("AntiAliasing", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("AntiAliasing_Name", "Anti Aliasing");
		OutTooltipText = LOCTEXT("AntiAliasing_Tooltip", "Any Anti-aliasing e.g. FXAA, Temporal AA.");
	}
	else if (InFeatureName.Equals("TemporalAA", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("TemporalAA_Name", "Temporal AA");
		OutTooltipText = LOCTEXT("TemporalAA_Tooltip", "Only used if AntiAliasing is on, true:uses Temporal AA, otherwise FXAA.");
	}
	else if (InFeatureName.Equals("AmbientCubemap", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("AmbientCubemap_Name", "Ambient Cubemap");
		OutTooltipText = LOCTEXT("AmbientCubemap_Tooltip", "Ambient Cube Map");
	}
	else if (InFeatureName.Equals("EyeAdaptation", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("EyeAdaptation_Name", "Auto Exposure");
		OutTooltipText = LOCTEXT("EyeAdaptation_Tooltip", "Human like eye simulation to adapt to the brightness of the view.");
	}
	else if (InFeatureName.Equals("LensFlares", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("LensFlares_Name", "Lens Flares");
		OutTooltipText = LOCTEXT("LensFlares_Tooltip", "Image based lens flares (Simulate artifact of reflections within a camera system).");
	}
	else if (InFeatureName.Equals("GlobalIllumination", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("GlobalIllumination_Name", "Global Illumination");
		OutTooltipText = LOCTEXT("GlobalIllumination_Tooltip", "Show indirect lighting component.");
	}
	else if (InFeatureName.Equals("AmbientOcclusion", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("AmbientOcclusion_Name", "Ambient Occlusion");
		OutTooltipText = LOCTEXT("AmbientOcclusion_Tooltip", "Screen Space Ambient Occlusion");
	}
	else if (InFeatureName.Equals("DirectLighting", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("DirectLighting_Name", "Direct Lighting");
		OutTooltipText = LOCTEXT("DirectLighting_Tooltip", "Allows to disable all direct lighting (does not affect indirect light).");
	}
	else if (InFeatureName.Equals("DepthOfField", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("DepthOfField_Name", "Depth Of Field");
		OutTooltipText = LOCTEXT("DepthOfField_Tooltip", "Depth of Field");
	}
	else if (InFeatureName.Equals("MotionBlur", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("MotionBlur_Name", "Motion Blur");
		OutTooltipText = LOCTEXT("MotionBlur_Tooltip", "MotionBlur, for now only camera motion blur.");
	}
	else if (InFeatureName.Equals("SeparateTranslucency", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("SeparateTranslucency_Name", "Separate Translucency");
		OutTooltipText = LOCTEXT("SeparateTranslucency_Tooltip", "If Translucency should be rendered into a separate RT and composited without DepthOfField.\n"
			"If this is false, it allows all translucent materials to be rendered in the same pass and produce a correct alpha channel for compositing (used as key).");
	}
	else if (InFeatureName.Equals("ReflectionEnvironment", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("ReflectionEnvironment_Name", "Reflection Environment");
		OutTooltipText = LOCTEXT("ReflectionEnvironment_Tooltip", "Whether to display the Reflection Environment feature, which has local reflections from Reflection Capture actors.");
	}
	else if (InFeatureName.Equals("ScreenSpaceReflections", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("ScreenSpaceReflections_Name", "Screen Space Reflections");
		OutTooltipText = LOCTEXT("ScreenSpaceReflections_Tooltip", "If screen space reflections are enabled.");
	}
	else if (InFeatureName.Equals("LumenReflections", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("LumenReflections_Name", "Lumen Reflections");
		OutTooltipText = LOCTEXT("LumenReflections_Tooltip", "Lumen Reflections");
	}
	else if (InFeatureName.Equals("ContactShadows", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("ContactShadows_Name", "Contact Shadows");
		OutTooltipText = LOCTEXT("ContactShadows_Tooltip", "If Screen space contact shadows are enabled.");
	}
	else if (InFeatureName.Equals("RayTracedDistanceFieldShadows", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("RayTracedDistanceFieldShadows_Name", "Ray Traced Distance Field Shadows");
		OutTooltipText = LOCTEXT("RayTracedDistanceFieldShadows_Tooltip", "If RTDF shadows are enabled.");
	}
	else if (InFeatureName.Equals("CapsuleShadows", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("CapsuleShadows_Name", "Capsule Shadows");
		OutTooltipText = LOCTEXT("CapsuleShadows_Tooltip", "If Capsule shadows are enabled.");
	}
	else if (InFeatureName.Equals("SubsurfaceScattering", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("SubsurfaceScattering_Name", "Subsurface Scattering");
		OutTooltipText = LOCTEXT("SubsurfaceScattering_Tooltip", "If Screen Space Subsurface Scattering enabled.");
	}
	else if (InFeatureName.Equals("VolumetricLightmap", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("VolumetricLightmap_Name", "Volumetric Lightmap");
		OutTooltipText = LOCTEXT("VolumetricLightmap_Tooltip", "Whether to apply volumetric lightmap lighting, when present.");
	}
	else if (InFeatureName.Equals("IndirectLightingCache", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("IndirectLightingCache_Name", "Indirect Lighting Cache");
		OutTooltipText = LOCTEXT("IndirectLightingCache_Tooltip", "If the indirect lighting cache is enabled.");
	}
	else if (InFeatureName.Equals("TexturedLightProfiles", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("TexturedLightProfiles_Name", "Textured Light Profiles");
		OutTooltipText = LOCTEXT("TexturedLightProfiles_Tooltip", "LightProfiles, usually 1d textures to have a light (IES).");
	}
	else if (InFeatureName.Equals("DynamicShadows", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("DynamicShadows_Name", "Dynamic Shadows");
		OutTooltipText = LOCTEXT("DynamicShadows_Tooltip", "Non baked shadows.");
	}
	else if (InFeatureName.Equals("Translucency", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("Translucency_Name", "Translucency");
		OutTooltipText = LOCTEXT("Translucency_Tooltip", "Render translucency.");
	}
	else if (InFeatureName.Equals("LightShafts", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("LightShafts_Name", "Light Shafts");
		OutTooltipText = LOCTEXT("LightShafts_Tooltip", "Render LightShafts");
	}
	else if (InFeatureName.Equals("PrecomputedVisibility", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("PrecomputedVisibility_Name", "Precomputed Visibility");
		OutTooltipText = LOCTEXT("PrecomputedVisibility_Tooltip", "To disable precomputed visibility.");
	}
	else if (InFeatureName.Equals("ScreenSpaceAO", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("ScreenSpaceAO_Name", "Screen Space AO");
		OutTooltipText = LOCTEXT("ScreenSpaceAO_Tooltip", "Screen space AO");
	}
	else if (InFeatureName.Equals("DistanceFieldAO", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("DistanceFieldAO_Name", "Distance Field AO");
		OutTooltipText = LOCTEXT("DistanceFieldAO_Tooltip", "Distance field AO");
	}
	else if (InFeatureName.Equals("LumenGlobalIllumination", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("LumenGlobalIllumination_Name", "Lumen Global Illumination");
		OutTooltipText = LOCTEXT("LumenGlobalIllumination_Tooltip", "Lumen Global Illumination");
	}
	else if (InFeatureName.Equals("VolumetricFog", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("VolumetricFog_Name", "Volumetric Fog");
		OutTooltipText = LOCTEXT("VolumetricFog_Tooltip", "Volumetric Fog");
	}
	else if (InFeatureName.Equals("DisableOcclusionQueries", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("DisableOcclusionQueries_Name", "Disable Occlusion Queries");
		OutTooltipText = LOCTEXT("DisableOcclusionQueries_Tooltip", "Disable hardware occlusion queries, similar to setting r.AllowOcclusionQueries=0, but just for this scene.");
	}
	else if (InFeatureName.Equals("LumenScreenTraces", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("LumenScreenTraces_Name", "Lumen Screen Traces");
		OutTooltipText = LOCTEXT("LumenScreenTraces_Tooltip", "Use screen space tracing in Lumen.");
	}
	else if (InFeatureName.Equals("LumenDetailTraces", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("LumenDetailTraces_Name", "Lumen Detail Traces");
		OutTooltipText = LOCTEXT("LumenDetailTraces_Tooltip", "Use detail tracing in Lumen.");
	}
	else if (InFeatureName.Equals("LumenGlobalTraces", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("LumenGlobalTraces_Name", "Lumen Global Traces");
		OutTooltipText = LOCTEXT("LumenGlobalTraces_Tooltip", "Use global traces in Lumen.");
	}
	else if (InFeatureName.Equals("LumenFarFieldTraces", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("LumenFarFieldTraces_Name", "Lumen Far Field Traces");
		OutTooltipText = LOCTEXT("LumenFarFieldTraces_Tooltip", "Use far field traces in Lumen.");
	}
	else if (InFeatureName.Equals("LumenSecondaryBounces", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("LumenSecondaryBounces_Name", "Lumen Secondary Bounces");
		OutTooltipText = LOCTEXT("LumenSecondaryBounces_Tooltip", "Compute secondary bounces in Lumen.");
	}
	else if (InFeatureName.Equals("LumenScreenSpaceDirectionalOcclusion", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("LumenScreenSpaceDirectionalOcclusion_Name", "Lumen Screen Space Directional Occlusion");
		OutTooltipText = LOCTEXT("LumenScreenSpaceDirectionalOcclusion_Tooltip", "Compute screen space directional occlusion in Lumen.");
	}
	else if (InFeatureName.Equals("LumenReuseShadowMaps", ESearchCase::IgnoreCase))
	{
		OutNameText = LOCTEXT("LumenReuseShadowMaps_Name", "Lumen Reuse Shadow Maps");
		OutTooltipText = LOCTEXT("LumenReuseShadowMaps_Tooltip", "Whether to reuse shadowmaps when calculating shadowing. Can be disabled to debug view dependent lighting from shadowing technique mismatches.");
	}
}

#undef LOCTEXT_NAMESPACE
