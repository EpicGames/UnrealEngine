// Copyright Epic Games, Inc. All Rights Reserved.

#include "Viewport/Interaction/AvaSnapPoint.h"
#include "GameFramework/Actor.h"

FAvaSnapPoint FAvaSnapPoint::CreateScreenSnapPoint(EAvaAnchors InAnchor, const FVector2f& InLocation)
{
	check(InAnchor != EAvaAnchors::Custom && InAnchor != EAvaAnchors::None);

	return FAvaSnapPoint(nullptr, InAnchor, FVector(0.f, InLocation.X, InLocation.Y), INDEX_NONE);
}

FAvaSnapPoint FAvaSnapPoint::CreateGuideSnapPoint(float Offset, EOrientation Orientation)
{
	FVector Location = FVector::ZeroVector;

	if (Orientation == EOrientation::Orient_Horizontal)
	{
		Location.Y = 0;
		Location.Z = Offset;
	}
	else
	{
		Location.Y = Offset;
		Location.Z = 0;
	}

	return FAvaSnapPoint(nullptr, EAvaAnchors::Custom, Location, INDEX_NONE);
}

FAvaSnapPoint FAvaSnapPoint::CreateSelectionSnapPoint(EAvaAnchors InAnchor, const FVector& InLocation, EAvaDepthAlignment InDepth)
{
	check(InAnchor != EAvaAnchors::Custom && InAnchor != EAvaAnchors::None);

	return FAvaSnapPoint(nullptr, InAnchor, InLocation, static_cast<int32>(InDepth));
}

FAvaSnapPoint FAvaSnapPoint::CreateActorSnapPoint(const AActor* InActor, EAvaAnchors InAnchor, const FVector& InLocation, EAvaDepthAlignment InDepth)
{
	check(InActor); 	
	check(InAnchor != EAvaAnchors::None && InAnchor != EAvaAnchors::Custom);

	return FAvaSnapPoint(InActor, InAnchor, InLocation, static_cast<int32>(InDepth));
}

FAvaSnapPoint FAvaSnapPoint::CreateActorCustomSnapPoint(const AActor* InActor, const FVector& InLocation, int32 PointIndex)
{
	check(InActor);

	return FAvaSnapPoint(InActor, EAvaAnchors::Custom, InLocation, PointIndex);
}

FAvaSnapPoint FAvaSnapPoint::CreateActorIndexedSnapPoint(const AActor* InActor, EAvaAnchors InAnchor, const FVector& InLocation, int32 PointIndex)
{
	check(InActor);
	check(InAnchor != EAvaAnchors::Custom);

	return FAvaSnapPoint(InActor, InAnchor, InLocation, PointIndex);

}

FAvaSnapPoint FAvaSnapPoint::CreateLocalActorSnapPoint(EAvaAnchors InAnchor, const FVector& InLocation, EAvaDepthAlignment InDepth)
{
	check(InAnchor != EAvaAnchors::None && InAnchor != EAvaAnchors::Custom);

	return FAvaSnapPoint(nullptr, InAnchor, InLocation, static_cast<int32>(InDepth));
}

FAvaSnapPoint FAvaSnapPoint::CreateLocalActorCustomSnapPoint(const FVector& InLocation, int32 PointIndex)
{
	return FAvaSnapPoint(nullptr, EAvaAnchors::Custom, InLocation, PointIndex);

}

FAvaSnapPoint FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors InAnchor, const FVector& InLocation, int32 PointIndex)
{
	check(InAnchor != EAvaAnchors::Custom);
	return FAvaSnapPoint(nullptr, InAnchor, InLocation, PointIndex);
}

FAvaSnapPoint FAvaSnapPoint::CreateComponentSnapPoint(const USceneComponent* InComponent, EAvaAnchors InAnchor, const FVector& InLocation, EAvaDepthAlignment InDepth)
{
	check(InComponent);
	check(InAnchor != EAvaAnchors::None && InAnchor != EAvaAnchors::Custom);

	return FAvaSnapPoint(InComponent, InAnchor, InLocation, static_cast<int32>(InDepth));
}

FAvaSnapPoint FAvaSnapPoint::CreateComponentCustomSnapPoint(const USceneComponent* InComponent, const FVector& InLocation, int32 PointIndex)
{
	check(InComponent);

	return FAvaSnapPoint(InComponent, EAvaAnchors::Custom, InLocation, PointIndex);
}

FAvaSnapPoint FAvaSnapPoint::CreateComponentIndexedSnapPoint(const USceneComponent* InComponent, EAvaAnchors InAnchor, const FVector& InLocation, int32 PointIndex)
{
	check(InComponent);
	check(InAnchor != EAvaAnchors::Custom);

	return FAvaSnapPoint(InComponent, InAnchor, InLocation, PointIndex);

}

FAvaSnapPoint FAvaSnapPoint::CreateLocalComponentSnapPoint(EAvaAnchors InAnchor, const FVector& InLocation, EAvaDepthAlignment InDepth)
{
	check(InAnchor != EAvaAnchors::None && InAnchor != EAvaAnchors::Custom);

	return FAvaSnapPoint(nullptr, InAnchor, InLocation, static_cast<int32>(InDepth));
}

FAvaSnapPoint FAvaSnapPoint::CreateLocalComponentCustomSnapPoint(const FVector& InLocation, int32 PointIndex)
{
	return FAvaSnapPoint(nullptr, EAvaAnchors::Custom, InLocation, PointIndex);

}

FAvaSnapPoint FAvaSnapPoint::CreateLocalComponentIndexedSnapPoint(EAvaAnchors InAnchor, const FVector& InLocation, int32 PointIndex)
{
	check(InAnchor != EAvaAnchors::Custom);
	return FAvaSnapPoint(nullptr, InAnchor, InLocation, PointIndex);
}

FAvaSnapPoint FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors InAnchor, const FVector& InLocation, EAvaDepthAlignment InDepth)
{
	check(InAnchor != EAvaAnchors::None && InAnchor != EAvaAnchors::Custom);

	return FAvaSnapPoint(nullptr, InAnchor, InLocation, static_cast<int32>(InDepth));
}

FAvaSnapPoint FAvaSnapPoint::CreateNullSnapPoint()
{
	return FAvaSnapPoint(nullptr, EAvaAnchors::None, FVector::ZeroVector, INDEX_NONE);
}

FAvaSnapPoint::FAvaSnapPoint(const UObject* InOuter, EAvaAnchors InAnchor, const FVector& InLocation, int32 InPointIndex)
	: Outer(InOuter)
	, Anchor(InAnchor)
	, Location(InLocation)
	, PointIndex(InPointIndex)
{ 
}

FString FAvaSnapPoint::ToString() const
{
	return FString::Printf(TEXT("Snap Point (%s) (%d) (%d) (%f,%f,%f)"), Outer != nullptr ? *Outer->GetName() : TEXT(""), (uint8)Anchor, PointIndex, Location.X, Location.Y, Location.Z);
}

bool FAvaSnapPoint::IsScreenSnapPoint() const
{
	return Outer.IsExplicitlyNull() && Anchor != EAvaAnchors::Custom && Anchor != EAvaAnchors::None && PointIndex == INDEX_NONE;
}

bool FAvaSnapPoint::IsGuideSnapPoint() const
{
	return Outer.IsExplicitlyNull() && Anchor == EAvaAnchors::Custom;
}

bool FAvaSnapPoint::IsSelectionSnapPoint() const
{
	return Outer.IsExplicitlyNull() && Anchor != EAvaAnchors::Custom && Anchor != EAvaAnchors::None && PointIndex != INDEX_NONE;
}

bool FAvaSnapPoint::IsActorSnapPoint() const
{
	return !Outer.IsExplicitlyNull();
}
