// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Math/MathFwd.h"

class AActor;
class IAvaSceneInterface;
class USceneComponent;
enum class EActorModifierReferenceContainer : uint8;
struct FOrientedBox;

struct AVALANCHE_API FAvaActorUtils
{
	static FOrientedBox MakeOrientedBox(const FBox& InLocalBox, const FTransform& InWorldTransform);

	static FBox GetActorLocalBoundingBox(const AActor* InActor, bool bIncludeFromChildActors = false, bool bMustBeRegistered = true);

	static FBox GetComponentLocalBoundingBox(const USceneComponent* InComponent);

	static IAvaSceneInterface* GetSceneInterfaceFromActor(const AActor* InActor);
};
