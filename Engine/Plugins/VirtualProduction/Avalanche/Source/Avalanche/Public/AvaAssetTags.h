// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::Ava::AssetTags
{
	constexpr const TCHAR* MotionDesignScene = TEXT("MotionDesignScene");

	namespace Values
	{
		constexpr const TCHAR* Enabled = TEXT("Enabled");
	}
}
