// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTag.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "UObject/Object.h"
#include "AvaAttributeContainer.generated.h"

#define UE_API AVALANCHE_API

class UAvaAttribute;
class UAvaSceneSettings;
struct FAvaTagHandle;

/** Object providing attribute information of the Scene */
UCLASS(MinimalAPI)
class UAvaAttributeContainer : public UObject
{
	GENERATED_BODY()

public:
	void Initialize(UAvaSceneSettings* InSceneSettings);

	UE_API bool AddTagAttribute(const FAvaTagHandle& InTagHandle);
	UE_API bool RemoveTagAttribute(const FAvaTagHandle& InTagHandle);
	UE_API bool ContainsTagAttribute(const FAvaTagHandle& InTagHandle) const;

	UE_API bool AddNameAttribute(FName InName);
	UE_API bool RemoveNameAttribute(FName InName);
	UE_API bool ContainsNameAttribute(FName InName) const;

private:
	/** In-play Scene Attributes. Can be added to / removed from while in-play */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UAvaAttribute>> SceneAttributes;
};

#undef UE_API
