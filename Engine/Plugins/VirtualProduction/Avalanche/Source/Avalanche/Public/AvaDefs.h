// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "UObject/ObjectMacros.h"

#include "AvaDefs.generated.h"

// Single-axis exclusive bitflag map
namespace UE::Ava::AnchorPoints
{
	constexpr int32 None       = 0;
	constexpr int32 Left       = 1 << 0; // Y- = 1
	constexpr int32 HMiddle    = 1 << 1; // Y/2 = 2
	constexpr int32 Right      = 1 << 2; // Y+ = 4
	constexpr int32 Top        = 1 << 3; // Z+ = 8
	constexpr int32 VMiddle    = 1 << 4; // Z/2 = 16
	constexpr int32 Bottom     = 1 << 5; // Z- = 32
	constexpr int32 Horizontal = Left | HMiddle | Right; // Y- to Y+ = 7
	constexpr int32 Vertical   = Top | VMiddle | Bottom; // Z+ to Z- = 56
	constexpr int32 Custom     = 1 << 6; // Custom = 64
}

// UType 2D space point map. None should not be selectable in the UI. 
// Z: Top, Bottom, VMiddle & Y: Left, Right, HMiddle
// Expressed as ZY eg TopLeft
UENUM(BlueprintType)
enum class EAvaAnchors : uint8
{
	None = UE::Ava::AnchorPoints::None UMETA(Hidden),
	TopLeft = UE::Ava::AnchorPoints::Top + UE::Ava::AnchorPoints::Left,
	Top = UE::Ava::AnchorPoints::Top + UE::Ava::AnchorPoints::HMiddle,
	TopRight = UE::Ava::AnchorPoints::Top + UE::Ava::AnchorPoints::Right,
	Left = UE::Ava::AnchorPoints::VMiddle + UE::Ava::AnchorPoints::Left,
	Center = UE::Ava::AnchorPoints::VMiddle + UE::Ava::AnchorPoints::HMiddle,
	Right = UE::Ava::AnchorPoints::VMiddle + UE::Ava::AnchorPoints::Right,
	BottomLeft = UE::Ava::AnchorPoints::Bottom + UE::Ava::AnchorPoints::Left,
	Bottom = UE::Ava::AnchorPoints::Bottom + UE::Ava::AnchorPoints::HMiddle,
	BottomRight = UE::Ava::AnchorPoints::Bottom + UE::Ava::AnchorPoints::Right,
	Custom = UE::Ava::AnchorPoints::Custom
};

UENUM(BlueprintType)
enum class EAvaVerticalAlignment : uint8
{
	Top,
	Center,
	Bottom,
};

UENUM(BlueprintType)
enum class EAvaHorizontalAlignment : uint8
{
	Left,
	Center,
	Right,
};

UENUM(BlueprintType)
enum class EAvaDepthAlignment : uint8
{
	Front,
	Center,
	Back
};

template<typename Enum>
constexpr auto ToUnderlyingType(Enum Val)
{
	return static_cast<std::underlying_type_t<Enum>>(Val);
}

using AvaAlignment = int32; // xyz

// alignment for X (Depth)
template<typename AvaAlignment>
constexpr EAvaDepthAlignment GetDAlignment(AvaAlignment InAlignment)
{
	return static_cast<EAvaDepthAlignment>((0xFF & InAlignment) >> 0);
}

// alignment for Y (Horizontal)
template<typename AvaAlignment>
constexpr EAvaHorizontalAlignment GetHAlignment(AvaAlignment InAlignment)
{
	return static_cast<EAvaHorizontalAlignment>((0xFF00 & InAlignment) >> 8);
}

// alignment for Z (Vertical)
template<typename AvaAlignment>
constexpr EAvaVerticalAlignment GetVAlignment(AvaAlignment InAlignment)
{
	return static_cast<EAvaVerticalAlignment>((0xFF0000 & InAlignment) >> 16);
}

// gets an anchor location from an alignment and a size
template<typename AvaAlignment>
constexpr FVector GetLocationFromAlignment(AvaAlignment InAlignment, FVector Size3D)
{
	FVector AnchorLocation = FVector::ZeroVector;
	
	// get alignment enums from value
	const EAvaDepthAlignment Depth = GetDAlignment(InAlignment);
	// X
	switch (Depth)
	{
	case EAvaDepthAlignment::Back:
		AnchorLocation.X = -Size3D.X / 2.f;
		break;
	case EAvaDepthAlignment::Center:
		AnchorLocation.X = 0.f;
		break;
	case EAvaDepthAlignment::Front:
		AnchorLocation.X = Size3D.X / 2.f;
		break;
	default:
		return FVector::ZeroVector;
	}

	const EAvaHorizontalAlignment Horizontal = GetHAlignment(InAlignment);
	// Y
	switch (Horizontal)
	{
	case EAvaHorizontalAlignment::Left:
		AnchorLocation.Y = -Size3D.Y / 2.f;
		break;
	case EAvaHorizontalAlignment::Center:
		AnchorLocation.Y = 0.f;
		break;
	case EAvaHorizontalAlignment::Right:
		AnchorLocation.Y = Size3D.Y / 2.f;
		break;
	default:
		return FVector::ZeroVector;
	}

	const EAvaVerticalAlignment Vertical = GetVAlignment(InAlignment);
	// Z
	switch (Vertical)
	{
	case EAvaVerticalAlignment::Bottom:
		AnchorLocation.Z = -Size3D.Z / 2.f;
		break;
	case EAvaVerticalAlignment::Center:
		AnchorLocation.Z = 0.f;
		break;
	case EAvaVerticalAlignment::Top:
		AnchorLocation.Z = Size3D.Z / 2.f;
		break;
	default:
		return FVector::ZeroVector;
	}

	return AnchorLocation;
}

// create alignment value from enums
constexpr AvaAlignment MakeAlignment(EAvaDepthAlignment Depth, EAvaHorizontalAlignment Horizontal, EAvaVerticalAlignment Vertical)
{
	return AvaAlignment(static_cast<uint8>(Vertical) << 16 | static_cast<uint8>(Horizontal) << 8 | static_cast<uint8>(Depth) << 0);
}

UENUM(BlueprintType)
enum class EAvaColorStyle : uint8
{
	None,
	Solid,
	LinearGradient
};

USTRUCT(BlueprintType)
struct FAvaColorChangeData
{
	GENERATED_BODY()

	UPROPERTY(Category = "Motion Design", EditAnywhere, BlueprintReadWrite)
	EAvaColorStyle ColorStyle;

	UPROPERTY(Category = "Motion Design", EditAnywhere, BlueprintReadWrite)
	FLinearColor PrimaryColor;
	
	UPROPERTY(Category = "Motion Design", EditAnywhere, BlueprintReadWrite)
	FLinearColor SecondaryColor;

	UPROPERTY(Category = "Motion Design", EditAnywhere, BlueprintReadWrite)
	bool bIsUnlit;

	FAvaColorChangeData()
		: ColorStyle(EAvaColorStyle::None), PrimaryColor(FLinearColor::White), SecondaryColor(FLinearColor::White), bIsUnlit(true)
	{
	}

	FAvaColorChangeData(EAvaColorStyle InColorStyle, const FLinearColor& InPrimaryColor, const FLinearColor& InSecondaryColor, bool bInIsUnlit)
		: ColorStyle(InColorStyle), PrimaryColor(InPrimaryColor), SecondaryColor(InSecondaryColor), bIsUnlit(bInIsUnlit)
	{
	}

	bool operator==(const FAvaColorChangeData& InOtherColorData) const
	{
		return
			ColorStyle     == InOtherColorData.ColorStyle &&
			PrimaryColor   == InOtherColorData.PrimaryColor &&
			SecondaryColor == InOtherColorData.SecondaryColor &&
			bIsUnlit       == InOtherColorData.bIsUnlit;
	};
	
	bool operator!=(const FAvaColorChangeData& InOtherColorData) const
	{
		return !operator==(InOtherColorData);
	};
};
