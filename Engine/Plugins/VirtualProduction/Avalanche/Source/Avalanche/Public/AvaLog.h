// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

AVALANCHE_API DECLARE_LOG_CATEGORY_EXTERN(LogAva, Log, All);
