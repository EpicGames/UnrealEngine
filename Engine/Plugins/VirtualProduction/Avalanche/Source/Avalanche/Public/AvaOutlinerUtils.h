// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "AvaOutlinerDefines.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class AActor;
class IAvaOutliner;

/** Editor Only Outliner Access Functions */
struct AVALANCHE_API FAvaOutlinerUtils
{
	/** Returns the Motion Design outliner from the outliner module at runtime in the editor. */
	static TSharedPtr<IAvaOutliner> EditorGetOutliner(const UWorld* const InWorld);

	/** Returns an array of direct child actors attached to a parent actor or all root level actors if the parent actor is null. */
	static TArray<AActor*> EditorOutlinerChildActors(TSharedPtr<IAvaOutliner> InOutliner, AActor* const InParentActor = nullptr);

	/**
	 * Gets an array of actors from an array of outliner items while maintaining index ordering (for actor outliner items only).
	 * Returns false and an empty OutActors if any outliner items are invalid or any actor objects they represent are invalid.
	 */
	static bool EditorOutlinerItemsToActors(const TArray<FAvaOutlinerItemPtr>& InItems, TArray<AActor*>& OutActors);
};

#endif
