// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSceneTree.h"
#include "Delegates/Delegate.h"
#include "EditorSequenceNavigationDefs.h"
#include "GameFramework/Actor.h"
#include "IAvaRemoteControlInterface.h"
#include "IAvaSceneInterface.h"
#include "IAvaSequenceProvider.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "AvaScene.generated.h"

class IAvaSequencePlaybackObject;
class UAvaAttributeContainer;
class UAvaSceneSettings;
class UAvaSequence;
class URemoteControlPreset;

#if WITH_EDITOR
class ISequencer;
#endif

UCLASS(MinimalAPI, NotPlaceable, Hidden, NotBlueprintable, NotBlueprintType, DisplayName = "Motion Design Scene")
class AAvaScene : public AActor, public IAvaSequenceProvider, public IAvaSceneInterface, public IAvaRemoteControlInterface
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	enum class ESceneAction : uint8
	{
		Created,
		Activated,
		Deactivated,
	};
	AVALANCHE_API static void NotifySceneEvent(ESceneAction InAction);
#endif

	AVALANCHE_API static AAvaScene* GetScene(ULevel* InLevel, bool bInCreateSceneIfNotFound);

	AAvaScene();

	IAvaSequencePlaybackObject* GetScenePlayback() const;

#if WITH_EDITOR
	bool ShouldAutoStartMode() const
	{
		return bAutoStartMode;
	}

	AVALANCHE_API void SetAutoStartMode(bool bInAutoStartMode);

	TArray<uint8>& GetOutlinerData() { return OutlinerData; }

	void OnWorldRenamed(UWorld* InWorld);

	void OnGetWorldTags(FAssetRegistryTagsContext Context) const;
#endif

	//~ Begin IAvaSceneInterface
	virtual ULevel* GetSceneLevel() const override;
	virtual UAvaSceneSettings* GetSceneSettings() const override { return SceneSettings; }
	virtual UAvaAttributeContainer* GetAttributeContainer() const override { return AttributeContainer; }
	virtual FAvaSceneTree& GetSceneTree() override { return SceneTree; }
	virtual const FAvaSceneTree& GetSceneTree() const override { return SceneTree; }
	virtual IAvaSequencePlaybackObject* GetPlaybackObject() const override;
	virtual IAvaSequenceProvider* GetSequenceProvider() override { return this; }
	virtual const IAvaSequenceProvider* GetSequenceProvider() const override { return this; }
	virtual URemoteControlPreset* GetOrCreateRemoteControlPreset(bool bInCallModify=true) override;
	virtual URemoteControlPreset* GetRemoteControlPreset() const override;
	virtual void DestroyRemoteControlPreset(bool bInCallModify=true) override;
	virtual bool IsRemoteControlPresetExpectedToBePresent() const  override;
#if WITH_EDITOR
	virtual FNavigationToolSaveState& GetNavigationToolSaveState() override { return NavigationToolState; }
#endif
	//~ End IAvaSceneInterface

	//~ Begin IAvaSequenceProvider
	virtual UObject* ToUObject() override;
	virtual UWorld* GetContextWorld() const override;
	virtual bool CreateDirectorInstance(UAvaSequence& InSequence, IMovieScenePlayer& InPlayer, const FMovieSceneSequenceID& InSequenceID, UObject*& OutDirectorInstance) override;
	virtual bool AddSequence(UAvaSequence* InSequence) override;
	virtual uint32 AddSequences(TConstArrayView<UAvaSequence*> InSequences) override;
	virtual void RemoveSequence(UAvaSequence* InSequence) override;
	virtual void SetDefaultSequence(UAvaSequence* InSequence) override;
	virtual UAvaSequence* GetDefaultSequence() const override;
	virtual const TArray<TObjectPtr<UAvaSequence>>& GetSequences() const override { return Animations; }
	virtual const TArray<TWeakObjectPtr<UAvaSequence>>& GetRootSequences() const override { return RootAnimations; }
	virtual TArray<TWeakObjectPtr<UAvaSequence>>& GetRootSequencesMutable() override { return RootAnimations; }
	virtual FName GetSequenceProviderDebugName() const override;
#if WITH_EDITOR
	virtual TSharedPtr<ISequencer> GetEditorSequencer() const override { return EditorSequencer.Pin(); }
	virtual void OnEditorSequencerCreated(const TSharedPtr<ISequencer>& InSequencer) override;
	virtual bool GetDirectorBlueprint(UAvaSequence& InSequence, UBlueprint*& OutBlueprint) override { return false; }
#endif
	virtual FSimpleMulticastDelegate& GetOnSequenceTreeRebuilt() override { return OnTreeAnimationRebuilt; }
	virtual void ScheduleRebuildSequenceTree() override;
	virtual void RebuildSequenceTree() override;
	//~ End IAvaSequenceProvider

	//~ Begin IAvaRemoteControlInterface
	virtual void OnValuesApplied_Implementation() override;
	//~ End IAvaRemoteControlInterface

	//~ Begin AActor
	virtual void PostActorCreated() override;
	virtual void PreInitializeComponents() override;
	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type InEndPlayReason) override;
#if WITH_EDITOR
	virtual bool IsSelectable() const override { return false; }
	virtual bool SupportsExternalPackaging() const override { return false; }
#endif
	//~ End AActor

	//~ Begin UObject
	virtual void PostLoad() override;
	virtual void PostInitializeComponents() override;
	virtual void PostDuplicate(EDuplicateMode::Type InDuplicateMode) override;
	virtual void PostEditImport() override;
#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif
	virtual void BeginDestroy() override;
	//~ End UObject

	void RegisterObjects();
	void UnregisterObjects();

protected:
	UPROPERTY()
	FAvaSceneTree SceneTree;

	UPROPERTY()
	TObjectPtr<UAvaSceneSettings> SceneSettings;

	UPROPERTY()
	TObjectPtr<UAvaAttributeContainer> AttributeContainer;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.7, "This property is deprecated. Please use AttributeContainer")
	TObjectPtr<UAvaAttributeContainer> SceneState;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(Instanced)
	TObjectPtr<URemoteControlPreset> RemoteControlPreset;

	/** A List of All Animations, including those that are nested in other Animations */
	UPROPERTY()
	TArray<TObjectPtr<UAvaSequence>> Animations;

	/** The Base Playback Scene that is always present to Play Animations */
	UPROPERTY()
	TScriptInterface<IAvaSequencePlaybackObject> PlaybackObject;

	/** A list of only the Root Animations (those without Parent Animations) */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TArray<TWeakObjectPtr<UAvaSequence>> RootAnimations;

#if WITH_EDITORONLY_DATA
	/** Todo: Outliner Editor Object to store the data like filters, options, etc */
	UPROPERTY()
	TArray<uint8> OutlinerData;
#endif

#if WITH_EDITOR
	TWeakPtr<ISequencer> EditorSequencer;

	FDelegateHandle PreWorldRenameDelegate;

	FDelegateHandle WorldTagGetterDelegate;
#endif

	FSimpleMulticastDelegate OnTreeAnimationRebuilt;

	/** The index to the animation to use as the default animation */
	UPROPERTY()
	int32 DefaultSequenceIndex = 0;

	bool bPendingAnimTreeUpdate = false;

	/** Determines whether the remote control preset should be present in this scene */
	UPROPERTY()
	bool bRemoteControlPresetPresent = true;

#if WITH_EDITORONLY_DATA
	/** Whether the motion design mode should auto start */
	UPROPERTY()
	bool bAutoStartMode = true;

	/** Saved Sequence Navigator state to restore */
	UPROPERTY()
	FNavigationToolSaveState NavigationToolState;
#endif

};
