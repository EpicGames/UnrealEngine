// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaAttribute.h"
#include "Containers/Array.h"
#include "UObject/NoExportTypes.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "AvaSceneSettings.generated.h"

/** Object containing information about its Scene */
UCLASS(MinimalAPI)
class UAvaSceneSettings : public UObject
{
	GENERATED_BODY()

public:
	static FName GetSceneAttributesPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(UAvaSceneSettings, SceneAttributes);
	}

	static FName GetSceneRigPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(UAvaSceneSettings, SceneRig);
	}

	TConstArrayView<TObjectPtr<UAvaAttribute>> GetSceneAttributes() const
	{
		return SceneAttributes;
	}

	FSoftObjectPath GetSceneRig() const
	{
		return SceneRig;
	}

	void SetSceneRig(const FSoftObjectPath& InSceneRig)
	{
		SceneRig = InSceneRig;
	}

private:
	UPROPERTY(EditAnywhere, Instanced, Category="Scene Attributes")
	TArray<TObjectPtr<UAvaAttribute>> SceneAttributes;

	UPROPERTY()
	FSoftObjectPath SceneRig;
};
