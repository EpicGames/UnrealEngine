// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

/** Renamed AvaSceneState to AvaAttributeContainer to avoid confusion with the 'Scene State' system */
UE_DEPRECATED_HEADER(5.7, "Use AvaAttributeContainer.h instead")
#include "AvaAttributeContainer.h"

UE_DEPRECATED(5.7, "UAvaSceneState has been renamed to UAvaAttributeContainer. Use UAvaAttributeContainer instead.")
typedef UAvaAttributeContainer UAvaSceneState;
