// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/Ticker.h"
#include "Subsystems/WorldSubsystem.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakInterfacePtr.h"
#include "AvaSceneSubsystem.generated.h"

#define UE_API AVALANCHE_API

class IAvaSceneInterface;
class ULevel;

UCLASS(MinimalAPI, BlueprintType, DisplayName = "Motion Design Scene Subsystem")
class UAvaSceneSubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()

public:
	void RegisterSceneInterface(ULevel* InLevel, IAvaSceneInterface* InSceneInterface);

	UE_API FTSTicker& GetTicker();

	/** Gets the Scene Interface for the World's Persistent Level */
	UE_API IAvaSceneInterface* GetSceneInterface() const;

	/** Gets the Scene Interface for the provided Level */
	UE_API IAvaSceneInterface* GetSceneInterface(const ULevel* InLevel) const;

	/** Returns the first interface found in the given Level, if any */
	UE_API static IAvaSceneInterface* FindSceneInterface(const ULevel* InLevel);

	/** Determines whether the motion design is ready for play */
	UFUNCTION(BlueprintPure, Category="Motion Design", meta=(DefaultToSelf="InContextObject"))
	UE_API bool IsReadyToPlay(const UObject* InContextObject) const;

	/** Completes all pending work from Motion Design build systems (e.g. Text3D) */
	UFUNCTION(BlueprintCallable, Category="Motion Design", meta=(DefaultToSelf="InContextObject"))
	UE_API void FlushBuilds();

	/**
	 * Gathers actors in order of their appearance in the tree 
	 * @param InContextObject the context object to get the level from
	 * @param InParentActor if specified, only the children (and optionally including descendants) of this actor will be considered. If not specified, all the actors in the context level will be used.
	 * @param bInIncludeDescendants if specified, descendants of the given parent actor will also be considered.
	 */
	UFUNCTION(BlueprintCallable, Category="Motion Design", meta=(DefaultToSelf="InContextObject"))
	static UE_API TArray<AActor*> GatherSceneTreeActors(const UObject* InContextObject, const AActor* InParentActor=nullptr, bool bInIncludeDescendants=true);

protected:
	//~ Begin UWorldSubsystem
	UE_API virtual void PostInitialize() override;
	UE_API virtual bool DoesSupportWorldType(const EWorldType::Type InWorldType) const override;
	//~ End UWorldSubsystem

	//~ Begin FTickableObjectBase
	UE_API virtual bool IsTickableInEditor() const override;
	UE_API virtual void Tick(float InDeltaTime) override;
	UE_API virtual TStatId GetStatId() const override;
	//~ End FTickableObjectBase

private:
	/** Ticker for motion design elements */
	FTSTicker Ticker;

	TMap<TWeakObjectPtr<ULevel>, TWeakInterfacePtr<IAvaSceneInterface>> SceneInterfaces;
};

#undef UE_API
