// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaGameViewportClient.h"
#include "Engine/GameInstance.h"
#include "Engine/World.h"
#include "AvaGameInstance.generated.h"

#define UE_API AVALANCHE_API

class FRenderCommandFence;
class FSceneViewport;
class UTextureRenderTarget2D;
struct FAvaViewportQualitySettings;
struct FAvaInstanceSettings;

struct FAvaInstancePlaySettings
{
	const FAvaInstanceSettings& Settings;
	FName ChannelName;
	UTextureRenderTarget2D* RenderTarget;
	FIntPoint ViewportSize;
	const FAvaViewportQualitySettings& QualitySettings;
};

UCLASS(MinimalAPI, DisplayName = "Motion Design Game Instance")
class UAvaGameInstance : public UGameInstance, public FTickableGameObject
{
	GENERATED_BODY()

public:
	UE_API UAvaGameInstance();

	/**
	 * Creates a new Motion Design Game instance from the given Motion Design Template Asset.
	 * Supported asset types: UWorld (Level).
	 */
	UE_API static UAvaGameInstance* Create(UObject* InOuter);

	/** Create the game world */
	UE_INTERNAL UE_API bool CreateWorld();

	bool IsWorldCreated() const { return bWorldCreated; }

	UWorld* GetPlayWorld() const { return PlayWorld.Get(); }

	UE_API bool BeginPlayWorld(const FAvaInstancePlaySettings& InWorldPlaySettings);

	/**
	 * The end play is normally requested to be done on the next Tick(). This is to
	 * avoid having the world destroyed while within the Tick() of that world.
	 * However, when shutting down, we need to force the end play to be done immediately because
	 * there will not a be a next Tick().
	 */
	UE_API void RequestEndPlayWorld(bool bForceImmediate);
	
	UE_API void RequestUnloadWorld(bool bForceImmediate);

	bool IsWorldPlaying() const { return bWorldPlaying; }

	/**
	 * Cancel any pending EndPlayWorld or UnloadWorld requests. 
	 */
	UE_API void CancelWorldRequests();

	/**
	* The output channels can be resized during playback, if this happens, the
	* scene viewport needs to be updated to reflect the change.
	*/
	UE_API void UpdateSceneViewportSize(const FIntPoint& InViewportSize);

	UE_API void UpdateRenderTarget(UTextureRenderTarget2D* InRenderTarget);

	UE_API UTextureRenderTarget2D* GetRenderTarget() const;

	/**
	 * Returns true if the render target has been rendered and is ready to be used for output.
	 */
	bool IsRenderTargetReady() const { return bIsRenderTargetReady;}

	/**
	 * Reset the flag so the OnRenderTargetReady event can be called again on the next render.
	 */
	void ResetRenderTargetReady() { bIsRenderTargetReady = false; }

	/**
	 * @brief Get the currently playing scene viewport.
	 * @remark Only valid within a BeginPlay()/EndPlay() scope. Null otherwise.
	 */
	TSharedPtr<FSceneViewport> GetSceneViewport() const { return Viewport; }

	/**
	 * @brief Get the currently playing game viewport client.
	 * @remark Only valid within a BeginPlay()/EndPlay() scope. Null otherwise.
	 */
	UAvaGameViewportClient* GetAvaGameViewportClient() const { return ViewportClient; }

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAvaGameInstanceEvent, UAvaGameInstance* /*InGameInstance*/, FName /*InChannelName*/);

	/** Event called on EndPlayWorld(). */
	static FOnAvaGameInstanceEvent& GetOnEndPlay() { return OnEndPlay; }

	/** Event called when the render target becomes ready. */
	static FOnAvaGameInstanceEvent& GetOnRenderTargetReady() { return OnRenderTargetReady; }

protected:
#if WITH_EDITOR
	void EditorTick(float InDeltaSeconds);
#endif // WITH_EDITOR

	void TickInternal(float InDeltaSeconds);

	UE_API void UnloadWorld(bool bInShutdown);
	UE_API void EndPlayWorld();
	UE_API void OnEndFrameTick();
	UE_API void OnEnginePreExit();

	//~ Begin FTickableObjectBase
	UE_API virtual void Tick(float InDeltaSeconds) override;
	UE_API virtual TStatId GetStatId() const override;
	//~ End FTickableObjectBase

	//~ Begin FTickableGameObject
	UE_API virtual bool IsTickableInEditor() const override;
	UE_API virtual UWorld* GetTickableGameObjectWorld() const override;
	//~ End FTickableGameObject

	//~ Begin UGameInstance
	UE_API virtual ULocalPlayer* CreateInitialPlayer(FString& OutError) override;
	//~ End UGameInstance
	
	//~ Begin UObject
	UE_API virtual void BeginDestroy() override;
	//~ End UObject

	UPROPERTY(Transient)
	TObjectPtr<UWorld> PlayWorld;

	UPROPERTY(Transient)
	TObjectPtr<ULocalPlayer> LocalPlayer;

	UPROPERTY(Transient)
	TObjectPtr<UAvaGameViewportClient> ViewportClient;

	TSharedPtr<FSceneViewport> Viewport;

	/** Channel this game instance is playing on. */
	FName PlayingChannelName;

	bool bWorldCreated = false;
	bool bRequestUnloadWorld = false;
	bool bWorldPlaying = false;
	bool bRequestEndPlayWorld = false;
	bool bIsRenderTargetReady = false;

	/** Render command fence to ensure the render target has been rendered. */
	TUniquePtr<FRenderCommandFence> RenderTargetFence;

#if WITH_EDITOR
	/** Ticker implementation for the game instance to tick the world in editor engine */
	TPimplPtr<class FAvaGameInstanceEditorTicker> WorldTickerEditor; 
#endif

	/** Last frame that was ticked */
	uint64 TickedFrame = 0;

	/** Handle to delegate for EnginePreExit event. */
	FDelegateHandle EnginePreExitHandle;

	UE_API static FOnAvaGameInstanceEvent OnEndPlay;
	UE_API static FOnAvaGameInstanceEvent OnRenderTargetReady;
};

#undef UE_API
