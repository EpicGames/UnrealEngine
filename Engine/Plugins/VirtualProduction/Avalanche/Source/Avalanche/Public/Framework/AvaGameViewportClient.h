// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/GameViewportClient.h"
#include "SceneTypes.h"
#include "AvaGameViewportClient.generated.h"

class FViewElementDrawer;
class UTextureRenderTarget2D;
struct FSceneViewInitOptions;
struct FSceneViewProjectionData;

UCLASS(MinimalAPI, DisplayName = "Motion Design Game Viewport Client")
class UAvaGameViewportClient : public UGameViewportClient
{
	GENERATED_BODY()

public:
	//~ Begin FViewportClient
	virtual void Draw(FViewport* InViewport,FCanvas* InCanvas) override;
	virtual bool IsStatEnabled(const FString& InName) const override;
	//~ End FViewportClient

	AVALANCHE_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& InCollector);

	AVALANCHE_API void SetRenderTarget(UTextureRenderTarget2D* InRenderTarget);

	UTextureRenderTarget2D* GetRenderTarget() const { return RenderTarget.Get(); }

private:
	TArray<FSceneViewStateReference> ViewStates;

	TWeakObjectPtr<UTextureRenderTarget2D> RenderTarget;
};
