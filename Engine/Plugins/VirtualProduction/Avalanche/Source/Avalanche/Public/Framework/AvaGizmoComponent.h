// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"

#include "AvaGizmoComponent.generated.h"

/** Stores original per-primitive visibility properties. Used for restoration when component removed. */
USTRUCT()
struct FAvaGizmoComponentPrimitiveValues
{
	GENERATED_BODY()

	/** Controls whether the primitive component should cast a shadow or not. */
	UPROPERTY()
	bool bCastShadow = true;

	/** Whether to completely draw the primitive; if false, the primitive is not drawn, does not cast a shadow. */
	UPROPERTY()
	bool bIsVisible = true;

	/** Whether to hide the primitive in game, if the primitive is Visible. */
	UPROPERTY()
	bool bIsHiddenInGame = false;

	/** If true, this component will be visible in reflection captures. */
	UPROPERTY()
	bool bVisibleInReflectionCaptures = true;

	/** If true, this component will be visible in real-time sky light reflection captures. */
	UPROPERTY()
	bool bVisibleInRealTimeSkyCaptures = true;

	/** If true, this component will be visible in ray tracing effects. Turning this off will remove it from ray traced reflections, shadows, etc. */
	UPROPERTY()
	bool bVisibleInRayTracing = true;

	/** If true, this component will be visible in ray traced reflections. Screen space traces will ignore this flag, as they directly sample SceneColor. */
	UPROPERTY()
	bool bVisibleInReflections = true;

	/** Controls whether the primitive should affect dynamic distance field lighting methods.  This flag is only used if CastShadow is true. **/
	UPROPERTY()
	bool bAffectDistanceFieldLighting = true;

	/** Controls whether the primitive should influence indirect lighting. */
	UPROPERTY()
	bool bAffectDynamicIndirectLighting = true;

	/** Controls whether the primitive should affect indirect lighting when hidden. This flag is only used if bAffectDynamicIndirectLighting is true. */
	UPROPERTY()
	bool bAffectIndirectLightingWhileHidden = true;

	/** Currently only applies to dynamic meshes.  */
	UPROPERTY()
	bool bDrawWireframe = false;

	/** The color of the wireframe when drawn. */
	UPROPERTY()
	FLinearColor WireframeColor = FLinearColor(0.0, 0.5, 1.0, 1.0);

	/** Whether to render to the custom depth buffer. */
	UPROPERTY()
	bool bRendersCustomDepth = false;

	/** The custom stencil id, if applicable. */
	UPROPERTY()
	uint8 CustomStencilId = 0;

	/** Whether to render in the main pass. */
	UPROPERTY()
	bool bRendersInMainPass = true;

	/** Whether to render to the depth buffer. */
	UPROPERTY()
	bool bRendersDepth = true;

	UPROPERTY()
	TArray<TObjectPtr<UMaterialInterface>> Materials;
};

/** Add to an actor to indicate it's used as a Gizmo. */
UCLASS(DisplayName = "Motion Design Gizmo Component", meta = (BlueprintSpawnableComponent))
class AVALANCHE_API UAvaGizmoComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	UAvaGizmoComponent();

	virtual void PostLoad() override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	virtual void OnRegister() override;
	virtual void OnUnregister() override;

	virtual void OnComponentCreated() override;
	virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;

	/** Applies the gizmo values to all primitive components in the parent actor. */
	void ApplyGizmoValues();

	/** Applies the original (visibility etc) property values to the Actor. Use prior to component removal. */
	void RestoreComponentValues();

public:
	/** Whether "gizmo mode" is enabled. */
	const bool IsGizmoEnabled() const { return bIsGizmoEnabled; }

	/** Sets whether "gizmo mode" is enabled. */
	void SetGizmoEnabled(const bool bInIsEnabled);
	
	/** Whether the settings apply to child actors or not. */
	bool AppliesToChildActors() const;

	/** Sets whether the settings apply to child actors or not. */
	void SetApplyToChildActors(bool bInValue);

	/** The (optional) material to use. */
	UMaterialInterface* GetMaterial() const;

	/** Set the material to use. */
	void SetMaterial(UMaterialInterface* InMaterial);

	/** Sets the material to engine default. */
	void SetMaterialToDefault();

	/** Set the material to a see through one. */
	UE_DEPRECATED(5.8, "SetMaterialToSeeThrough deprecated. Unused.")
	void SetMaterialToSeeThrough();

	/** Whether the primitive component should cast a shadow or not. */
	bool CastsShadow() const;

	/** Controls whether the primitive component should cast a shadow or not. */
	void SetCastShadow(bool bInValue);

	/** Whether to completely draw the primitive; if false, the primitive is not drawn, does not cast a shadow. */
	bool IsVisibleInEditor() const;

	/** Whether to completely draw the primitive; if false, the primitive is not drawn, does not cast a shadow. */
	void SetVisibleInEditor(bool bInValue);

	/** Whether to hide the primitive in game, if the primitive is Visible. */
	bool IsHiddenInGame() const;

	/** Whether to hide the primitive in game, if the primitive is Visible. */
	void SetHiddenInGame(bool bInValue);

	/** Whether to render in the main pass. */
	bool RendersInMainPass() const;

	/** Sets whether to render in the main pass. */
	void SetRenderInMainPass(const bool bInValue);

	/** Whether to render to the depth buffer. */
	bool RendersDepth() const;

	/** Sets whether to render to the depth buffer. */
	void SetRenderDepth(const bool bInValue);

	/** Whether to draw the mesh wireframe, currently only applicable to Dynamic Meshes. */
	bool ShowsWireframe() const;

	/** Whether to draw the mesh wireframe, currently only applicable to Dynamic Meshes. */
	void SetShowWireframe(bool bInValue);

	/** The color of the wireframe when drawn. */
	void SetWireframeColor(const FLinearColor& InColor);

	/** Whether to render to the custom depth/stencil buffer. */
	bool SetsStencil() const;

	/** Sets whether to render to the custom depth/stencil buffer. */
	void SetsStencil(const bool bInSetStencil);

	/** The custom stencil id, if applicable. */
	const uint8 GetStencilId() const;

	/** Sets the custom stencil id, if applicable. */
	void SetStencilId(const uint8 InStencilId);

private:
	/** Stores all primitive component values (that this overrides). Call before applying gizmo values. */
	void StoreComponentValues();

	void OnPostRegisterParentComponents(AActor* InActor);

	void ForEachPrimitiveComponent(TFunctionRef<void(UPrimitiveComponent*)> InFunc);
	
	/** Variation of ForEachPrimitiveComponent that checks various flags. */
	void ApplyToEachPrimitiveComponent(TFunctionRef<void(UPrimitiveComponent*)> InFunc);

protected:
	FDelegateHandle PostRegisterComponentsHandle;

	/** Whether "gizmo mode" is enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsGizmoEnabled", Setter = "SetGizmoEnabled", Category = "Rendering", meta = (DisplayPriority = -99, AllowPrivateAccess = "true"))
	bool bIsGizmoEnabled = true;

	/** Whether the gizmo values are actually applied. */
	UPROPERTY()
	bool bAreGizmoValuesApplied = false;

	/** Whether the settings apply to child actors or not. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "AppliesToChildActors", Setter = "SetApplyToChildActors", Category = "Rendering", AdvancedDisplay, meta = (AllowPrivateAccess = "true"))
	bool bApplyToChildActors = false;

	/** Material that overrides all others attached to the parent actor, if specified. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Rendering", meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UMaterialInterface> Material;

	/** Controls whether the primitive component should cast a shadow or not. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "CastsShadow", Setter = "SetCastShadow", Category = "Rendering", meta = (AllowPrivateAccess = "true"))
	bool bCastShadow = false;

	/** Whether to completely draw the primitive; if false, the primitive is not drawn, does not cast a shadow. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsVisibleInEditor", Setter = "SetVisibleInEditor", Category = "Rendering", meta = (AllowPrivateAccess = "true"))
	bool bIsVisibleInEditor = true;

	/** Whether to hide the primitive in game, if the primitive is Visible. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsHiddenInGame", Setter = "SetHiddenInGame", Category = "Rendering", meta = (AllowPrivateAccess = "true"))
	bool bIsHiddenInGame = true;

	/** Whether to render in the main pass. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "RendersInMainPass", Setter = "SetRenderInMainPass", Category = "Rendering", meta = (AllowPrivateAccess = "true"))
	bool bRenderInMainPass = true;

	/** Whether to render to the depth buffer. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "RendersDepth", Setter = "SetRenderDepth", Category = "Rendering", meta = (AllowPrivateAccess = "true"))
	bool bRenderDepth = true;

	/** Currently only applies to dynamic meshes.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "ShowsWireframe", Setter = "SetShowWireframe", Category = "Rendering", meta = (InlineEditConditionToggle, AllowPrivateAccess = "true"))
	bool bDrawWireframe = true;

	/** The color of the wireframe when drawn. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "Auto", Setter, Category = "Rendering", meta = (EditCondition = "bDrawWireframe", AllowPrivateAccess = "true"))
	FLinearColor WireframeColor = FLinearColor(0.0, 0.5, 1.0, 1.0);

	/** Whether to render to the custom depth/stencil buffer. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "SetsStencil", Setter = "SetsStencil", Category = "Rendering", meta = (InlineEditConditionToggle, AllowPrivateAccess = "true"))
	bool bSetStencil = false;

	/** The custom stencil id, if applicable. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter = "SetStencilId", Category = "Rendering", meta = (EditCondition = "bSetStencil", AllowPrivateAccess = "true"))
	uint8 StencilId = 150;
	
	/** Original values to restore upon component removal. */
	UPROPERTY()
	TMap<FSoftComponentReference, FAvaGizmoComponentPrimitiveValues> ComponentValues;

	std::atomic<bool> bIsRestoringValues = false;
};
