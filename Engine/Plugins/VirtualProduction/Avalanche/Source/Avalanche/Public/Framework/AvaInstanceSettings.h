// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaInstanceSettings.generated.h"

/**
 * Settings applied when instancing a Motion Design Asset for playback.
 */
USTRUCT()
struct FAvaInstanceSettings
{
	GENERATED_BODY()

	/** Enable loading dependent levels as sub-playables. */
	UPROPERTY(config, EditAnywhere, Category = Settings)
	bool bEnableLoadSubPlayables = true;

	/**
	 * For default playable transitions (when there is no transition tree),
	 * wait for the sequences to finish before ending the transition.
	 */
	UPROPERTY(config, EditAnywhere, Category = Settings)
	bool bDefaultPlayableTransitionWaitForSequences = false;
};
