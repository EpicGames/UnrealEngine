// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "GameFramework/Actor.h"
#include "AvaNullActor.generated.h"

class UAvaNullComponent;

/**
 * Null Actor (Empty Group)
 */
UCLASS(MinimalAPI, DisplayName = "Null Actor")
class AAvaNullActor : public AActor
{
	GENERATED_BODY()

public:
	static const FString DefaultLabel;

	AAvaNullActor();

#if WITH_EDITOR
	//~ Begin AActor
	virtual FString GetDefaultActorLabel() const override;
	//~ End AActor
#endif

protected:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Components")
	TObjectPtr<UAvaNullComponent> NullComponent;
};
