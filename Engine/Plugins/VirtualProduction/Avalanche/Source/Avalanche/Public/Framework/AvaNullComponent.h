// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"
#include "AvaNullComponent.generated.h"

UCLASS(MinimalAPI, DisplayName = "Null Component")
class UAvaNullComponent : public USceneComponent
{
	GENERATED_BODY()

public:
	UAvaNullComponent();
};
