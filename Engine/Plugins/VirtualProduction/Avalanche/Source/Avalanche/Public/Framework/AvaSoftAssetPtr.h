// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/SoftObjectPtr.h"

enum class EMotionDesignAssetType
{
	Unknown,
	World
};

/**
 * Motion Design Extension to FSoftObjectPath.
 * Adding the class information along to resolve about a Motion Design Asset without loading anything.
 */
struct FAvaSoftAssetPath
{
	FSoftClassPath AssetClassPath;
	FSoftObjectPath AssetPath;

	bool IsNull() const { return AssetPath.IsNull(); }
	const FSoftObjectPath& ToSoftObjectPath() const { return AssetPath; }
	EMotionDesignAssetType GetAssetType(bool bInLoadIfUnknown = true) const { return GetAssetTypeFromClass(AssetClassPath, bInLoadIfUnknown);}

	AVALANCHE_API static EMotionDesignAssetType GetAssetTypeFromClass(const FSoftClassPath& InAssetClassPath, bool bInLoadIfUnknown = true);
};

/**
 * Motion Design Extension to FSoftObjectPtr.
 * Adding the class information along to resolve a Motion Design Asset without loading anything.
 */

struct FAvaSoftAssetPtr
{
	FSoftClassPath AssetClassPath;

	// We need to use the template, it is the only way we can assign it from another SoftObjetPtr.
	TSoftObjectPtr<UObject> AssetPtr;

	// Warning: IsValid() for an AssetPtr will cause the asset to be loaded.
	// Using IsNull() is the only option we have.	
	bool IsNull() const { return AssetPtr.IsNull(); }
	
	const FSoftObjectPath& ToSoftObjectPath() const { return AssetPtr.ToSoftObjectPath(); }

	/**
	 * @brief Determines the asset type from the given class path or by loading the object.
	 * @param bInLoadIfUnknown If the type can't be determined by the class path, the object is loaded to check it's type.
	 * @return One of the recognized asset types for Motion Design playables.
	 */
	AVALANCHE_API EMotionDesignAssetType GetAssetType(bool bInLoadIfUnknown = true) const;

	FORCEINLINE bool operator==(const FAvaSoftAssetPtr& Rhs) const
	{
		return AssetClassPath == Rhs.AssetClassPath && AssetPtr == Rhs.AssetPtr;
	}
};