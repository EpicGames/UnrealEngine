// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "AvaTickerActor.generated.h"

class UAvaTickerComponent;

UCLASS(MinimalAPI, DisplayName="Motion Design Ticker Actor")
class AAvaTickerActor : public AActor
{
	GENERATED_BODY()

public:
	AAvaTickerActor();

	UAvaTickerComponent* GetTickerComponent() const
	{
		return TickerComponent;
	}

	//~ Begin AActor
#if WITH_EDITOR
	virtual FString GetDefaultActorLabel() const override;
#endif
	//~ End AActor

protected:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Ticker")
	TObjectPtr<UAvaTickerComponent> TickerComponent;
};
