// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PrimitiveComponent.h"
#include "AvaTickerComponent.generated.h"

#define UE_API AVALANCHE_API

UENUM()
enum class EAvaTickerQueueLimitType : uint8
{
	None UMETA(Hidden),

	/** No new elements will be queued if queue is full*/
	DisableQueueing,

	/** New elements will be queued by discarding the oldest (front of queue) */
	DiscardOldest,
};

USTRUCT()
struct FAvaTickerElement
{
	GENERATED_BODY()

	FAvaTickerElement() = default;

	UE_API explicit FAvaTickerElement(AActor* InActor);

	UE_API bool operator==(const FAvaTickerElement& InOther) const;

	/** Whether the element is a valid element */
	bool IsValid() const;

	/** Sets the game and editor visibility of the element */
	void SetVisibility(bool bInVisible);

	/** Retrieves the world space location of the element */
	FVector GetElementLocation() const;

	/** Moves the given element to the given world space location */
	void SetLocation(const FVector& InLocation);

	/** Moves the given element by the given displacement vector */
	void Displace(const FVector& InDisplacement);

	/** Destroys the element */
	void Destroy();

	/** Gets the bounds for the element */
	void GetBounds(FVector& OutOrigin, FVector& OutExtents) const;

	/** Actor mapped to this ticker element */
	UPROPERTY()
	TObjectPtr<AActor> Actor;
};

UCLASS(MinimalAPI, Blueprintable, DisplayName="Motion Design Ticker Component")
class UAvaTickerComponent : public UPrimitiveComponent
{
	GENERATED_BODY()

public:
	UE_API UAvaTickerComponent();

	/** Returns whether the queue can be added to */
	UFUNCTION(BlueprintPure, Category="Ticker")
	bool CanQueueElements() const;

	/**
	 * Adds the given actor to the queue
	 * @param InActor the actor to queue
	 * @param bInDestroyOnFailure whether to destroy the actor if it failed to be queued
	 * @return true if the actor was successfully queued
	 */
	UFUNCTION(BlueprintCallable, Category="Ticker")
	UE_API bool QueueActor(AActor* InActor, bool bInDestroyOnFailure = true);

	UFUNCTION(BlueprintPure, Category="Ticker")
	FVector GetStartLocation() const
	{
		return StartLocation;
	}

	UFUNCTION(BlueprintCallable, Category="Ticker")
	UE_API void SetStartLocation(const FVector& InStartLocation);

	UFUNCTION(BlueprintPure, Category="Ticker")
	double GetDestroyDistance() const
	{
		return DestroyDistance;
	}

	UFUNCTION(BlueprintCallable, Category="Ticker")
	UE_API void SetDestroyDistance(double InDestroyDistance);

	UFUNCTION(BlueprintPure, Category="Ticker")
	FVector GetVelocity() const
	{
		return Velocity;
	}

	UFUNCTION(BlueprintCallable, Category="Ticker")
	UE_API void SetVelocity(const FVector& InVelocity);

	UFUNCTION(BlueprintPure, Category="Ticker")
	double GetPadding() const
	{
		return Padding;
	}

	UFUNCTION(BlueprintCallable, Category="Ticker")
	UE_API void SetPadding(double InPadding);

	UFUNCTION(BlueprintPure, Category="Ticker")
	bool ShouldLimitQueue() const
	{
		return bLimitQueue;
	}

	UFUNCTION(BlueprintCallable, Category="Ticker")
	UE_API void SetLimitQueue(bool bInLimitQueue);

	UFUNCTION(BlueprintPure, Category="Ticker")
	int32 GetQueueLimitCount() const
	{
		return QueueLimitCount;
	}

	UFUNCTION(BlueprintCallable, Category="Ticker")
	UE_API void SetQueueLimitCount(int32 InQueueLimitCount);

	UFUNCTION(BlueprintPure, Category="Ticker")
	EAvaTickerQueueLimitType GetQueueLimitType() const
	{
		return QueueLimitType;
	}

	UFUNCTION(BlueprintCallable, Category="Ticker")
	UE_API void SetQueueLimitType(EAvaTickerQueueLimitType InQueueLimitType);

	//~ Begin UPrimitiveComponent
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	//~ End UPrimitiveComponent

	//~ Begin USceneComponent
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& InLocalToWorld) const override;
	//~ End USceneComponent

	//~ Begin UActorComponent
	UE_API virtual void TickComponent(float InDeltaTime, ELevelTick InTickType, FActorComponentTickFunction* InThisTickFunction) override;
	//~ End UActorComponent

protected:
	bool IsQueueFull() const;

	/** Adds the given element to the queue */
	bool QueueElement(FAvaTickerElement&& InElement, bool bInDestroyOnFailure);

	/** Retrieves the start location transformed to World Space */
	FVector GetWorldStartLocation() const;

	/** Retrieves the destroy displacement transformed to World Space */
	FVector GetWorldDestroyDisplacement() const;

	/** Retrieves the velocity transformed to World Space */
	FVector GetWorldVelocity() const;

	/** Removes the first element from the queue and activates it if the last active element isn't on the way */
	void TryDequeueElement();

	/** Updates the active elements to move with the given velocity */
	void UpdateActiveElements(float InDeltaTime);

	/** Elements being moved by this ticker */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TArray<FAvaTickerElement> ActiveElements;

	/** Elements currently hidden, awaiting to become available */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TArray<FAvaTickerElement> QueuedElements;

	/** The start location (in local space) where active elements start at */
	UPROPERTY(EditAnywhere, Category="Ticker", meta=(AllowPrivateAccess="true"))
	FVector StartLocation = FVector(0, 500, 0);

	/** The distance the last part of the active element must cross before it gets destroyed */
	UPROPERTY(EditAnywhere, Category="Ticker", meta=(AllowPrivateAccess="true"))
	double DestroyDistance = 1000.0;

	/** The velocity (in local space) to move the active elements at */
	UPROPERTY(EditAnywhere, Category="Ticker", meta=(AllowPrivateAccess="true"))
	FVector Velocity = FVector(0, -200, 0);

	/** Padding between the active ticker elements */
	UPROPERTY(EditAnywhere, Category="Ticker", meta=(AllowPrivateAccess="true"))
	double Padding = 30.0;

	/** Indicates whether to limit the queue when it gets past a certain number of elements */
	UPROPERTY(EditAnywhere, Category="Ticker|Queue", meta=(AllowPrivateAccess="true"))
	bool bLimitQueue = true;

	/** The number of items allowed in the queue */
	UPROPERTY(EditAnywhere, Category="Ticker|Queue", meta=(EditCondition="bLimitQueue", EditConditionHides, ClampMin=0, AllowPrivateAccess="true"))
	int32 QueueLimitCount = 2;

	/** The handling method when the queue is full */
	UPROPERTY(EditAnywhere, Category="Ticker|Queue", meta=(EditCondition="bLimitQueue", EditConditionHides, AllowPrivateAccess="true"))
	EAvaTickerQueueLimitType QueueLimitType = EAvaTickerQueueLimitType::DisableQueueing;
};

#undef UE_API
