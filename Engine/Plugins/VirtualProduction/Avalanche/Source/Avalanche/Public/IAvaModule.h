// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IAvaModule : public IModuleInterface
{
public:
	static IAvaModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IAvaModule>(TEXT("Avalanche"));
	}

	/**
	 * Allow the AvaModule to process the FCoreDelegates::StatCheckEnabled event.
	 * This is an equivalent logic as using GStatProcessingViewportClient.
	 */
	virtual bool SetRuntimeStatProcessingEnabled(bool bEnabled) = 0;

	/**
	* Returns true if the specified stat is enabled.
	*/
	virtual bool IsRuntimeStatEnabled(const FString& InName) const = 0;

	/**
	* Set a specific stat to either enabled or disabled (returns the number of remaining enabled stats)
	* This applies to all the Motion Design instance viewports.
	*/
	virtual int32 SetRuntimeStatEnabled(const TCHAR* InName, const bool bInEnabled) = 0;

	/**
	 * Returns true if there are runtime stats enabled.
	 */
	virtual bool ShouldShowRuntimeStats() const = 0;

	/**
	 * Returns the list of enabled runtime stats.
	 */
	virtual TArray<FString> GetEnabledRuntimeStats() const = 0;

	/**
	 * Overwrite the enabled runtime stats with provided ones.
	 */
	virtual void OverwriteEnabledRuntimeStats(const TArray<FString>& InEnabledStats) = 0;
};
