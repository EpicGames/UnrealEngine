// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IAvaSceneInterface.generated.h"

class IAvaSequencePlaybackObject;
class IAvaSequenceProvider;
class UAvaAttributeContainer;
class UAvaSceneSettings;
class ULevel;
class URemoteControlPreset;
struct FAvaSceneTree;

#if WITH_EDITOR
struct FNavigationToolSaveState;
#endif

/**
 * Interface for the driver class of an Motion Design Scene
 * This interface's purpose is to provide access to scene data and to core interfaces in the Scene
 */
UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UAvaSceneInterface : public UInterface
{
	GENERATED_BODY()
};

class IAvaSceneInterface
{
	GENERATED_BODY()

public:
	virtual ULevel* GetSceneLevel() const = 0;

	virtual UAvaSceneSettings* GetSceneSettings() const = 0;

	virtual UAvaAttributeContainer* GetAttributeContainer() const = 0;

	UE_DEPRECATED(5.7, "GetSceneState deprecated. Use GetAttributeContainer instead")
	virtual UAvaAttributeContainer* GetSceneState() const
	{
		return GetAttributeContainer();
	}

	virtual const FAvaSceneTree& GetSceneTree() const = 0;

	virtual FAvaSceneTree& GetSceneTree() = 0;

	virtual IAvaSequenceProvider* GetSequenceProvider() = 0;

	virtual const IAvaSequenceProvider* GetSequenceProvider() const = 0;

	virtual IAvaSequencePlaybackObject* GetPlaybackObject() const = 0;

	/** Retrieve the remote control preset, or create one if there's no preset present */
	virtual URemoteControlPreset* GetOrCreateRemoteControlPreset(bool bInCallModify=true) = 0;

	/** Gets the remote control preset of the scene */
	virtual URemoteControlPreset* GetRemoteControlPreset() const = 0;

	/** Destroys the remote control preset of the scene */
	UE_EXPERIMENTAL(5.8, "Destroying the Remote Control Preset in Motion Design Scenes is experimental.")
	virtual void DestroyRemoteControlPreset(bool bInCallModify=true) = 0;

	/** Whether the remote control preset is expected to be present in this scene */
	virtual bool IsRemoteControlPresetExpectedToBePresent() const = 0;

#if WITH_EDITOR
	virtual FNavigationToolSaveState& GetNavigationToolSaveState() = 0;
#endif
};
