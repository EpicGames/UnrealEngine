// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Materials/MaterialParameters.h"
#include "UObject/NameTypes.h"

class UMaterialInterface;
struct FMaterialCachedParameterEntry;

namespace UE::Ava
{
	/** Checks for the presence of a named parameter in the given material. */
	AVALANCHE_API bool MaterialHasParameter(const UMaterialInterface& InMaterial, const FName InParameterName, const EMaterialParameterType InParameterType);

	/** Checks for the presence of a series of named parameters in the given material. */
	AVALANCHE_API bool MaterialHasParameters(const UMaterialInterface& InMaterial, const TConstArrayView<TPair<FName, EMaterialParameterType>> InParameters, TArray<FString>& OutMissingParameters);
}
