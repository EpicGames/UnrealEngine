// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviour/RCBehaviour.h"
#include "AvaRCSequenceBehavior.generated.h"

UCLASS(MinimalAPI)
class UAvaRCSequenceBehavior : public URCBehaviour
{
	GENERATED_BODY()
};
