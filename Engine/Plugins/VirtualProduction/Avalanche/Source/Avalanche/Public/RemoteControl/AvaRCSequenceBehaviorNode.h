// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequenceName.h"
#include "AvaSequenceShared.h"
#include "Behaviour/RCBehaviourNode.h"
#include "AvaRCSequenceBehaviorNode.generated.h"

/** Performs an action (e.g. Play, Stop) for a defined Motion Design sequence */
UCLASS(MinimalAPI)
class UAvaRCSequenceBehaviorNode : public URCBehaviourNode
{
	GENERATED_BODY()

public:
	UAvaRCSequenceBehaviorNode();

protected:
	//~ Begin URCBehaviourNode
	bool Execute(URCBehaviour* InBehavior) const override;
	bool IsSupported(URCBehaviour* InBehavior) const override;
	void OnPassed(URCBehaviour* InBehavior) const;
	UClass* GetBehaviourClass() const override;
	//~ End URCBehaviourNode

private:
	/** Sequence name to play */
	UPROPERTY(EditAnywhere, Category="Sequence")
	FAvaSequenceName SequenceName;

	/** Action to perform for the given sequence */
	UPROPERTY(EditAnywhere, Category="Sequence")
	EAvaSequenceActionType SequenceAction = EAvaSequenceActionType::Play;

	/** Sequence Play Settings */
	UPROPERTY(EditAnywhere, Category="Sequence", meta=(EditCondition="SequenceAction==EAvaSequenceActionType::Play", EditConditionHides))
	FAvaSequencePlayParams PlaySettings;
};
