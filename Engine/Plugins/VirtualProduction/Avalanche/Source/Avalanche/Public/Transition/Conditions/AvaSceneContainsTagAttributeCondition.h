// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagHandle.h"
#include "AvaTagHandleContainer.h"
#include "AvaTransitionEnums.h"
#include "Conditions/AvaTransitionCondition.h"
#include "AvaSceneContainsTagAttributeCondition.generated.h"

class UAvaSceneSubsystem;
struct FAvaTransitionScene;

USTRUCT()
struct FAvaSceneContainsTagAttributeConditionInstanceData
{
	GENERATED_BODY()

	/** Whether the scene to check should be this scene or other scene */
	UPROPERTY(EditAnywhere, Category="Parameter")
	EAvaTransitionSceneType SceneType = EAvaTransitionSceneType::This;

	/** Which Layer should be queried for the Scene Attributes */
	UPROPERTY(EditAnywhere, Category="Parameter", meta=(EditCondition="SceneType==EAvaTransitionSceneType::Other", EditConditionHides))
	EAvaTransitionLayerCompareType LayerType = EAvaTransitionLayerCompareType::Same;

	/** Specific layer tags to check */
	UPROPERTY(EditAnywhere, Category="Parameter", meta=(EditCondition="SceneType==EAvaTransitionSceneType::Other && LayerType==EAvaTransitionLayerCompareType::MatchingTag", EditConditionHides))
	FAvaTagHandleContainer SpecificLayers;

	/** The Tag Attribute to check if it's contained in the scene(s) or not */
	UPROPERTY(EditAnywhere, Category="Parameter")
	FAvaTagHandle TagAttribute;
};

USTRUCT(meta=(Hidden))
struct AVALANCHE_API FAvaSceneContainsTagAttributeConditionBase : public FAvaTransitionCondition
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaSceneContainsTagAttributeConditionInstanceData;

	FAvaSceneContainsTagAttributeConditionBase() = default;

	FAvaSceneContainsTagAttributeConditionBase(bool bInInvertCondition)
		: bInvertCondition(bInInvertCondition)
	{
	}

	//~ Begin FStateTreeNodeBase
#if WITH_EDITOR
	virtual FText GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const override;
#endif
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	virtual bool Link(FStateTreeLinker& InLinker) override;
	//~ End FStateTreeNodeBase

	//~ Begin FStateTreeConditionBase
	virtual bool TestCondition(FStateTreeExecutionContext& InContext) const override;
	//~ End FStateTreeConditionBase

	bool ContainsTagAttribute(FStateTreeExecutionContext& InContext) const;

	TArray<const FAvaTransitionScene*> GetTransitionScenes(FStateTreeExecutionContext& InContext) const;

protected:
	bool bInvertCondition = false;

	TStateTreeExternalDataHandle<UAvaSceneSubsystem> SceneSubsystemHandle;
};

USTRUCT(DisplayName="A scene contains tag attribute", Category="Scene Attributes")
struct FAvaSceneContainsTagAttributeCondition : public FAvaSceneContainsTagAttributeConditionBase
{
	GENERATED_BODY()

	FAvaSceneContainsTagAttributeCondition()
		: FAvaSceneContainsTagAttributeConditionBase(/*bInvertCondition*/false)
	{
	}
};

USTRUCT(DisplayName="No scene contains tag attribute", Category="Scene Attributes")
struct FAvaNoSceneContainsTagAttributeCondition : public FAvaSceneContainsTagAttributeConditionBase
{
	GENERATED_BODY()

	FAvaNoSceneContainsTagAttributeCondition()
		: FAvaSceneContainsTagAttributeConditionBase(/*bInvertCondition*/true)
	{
	}
};
