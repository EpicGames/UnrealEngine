// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaRCControllerId.h"
#include "AvaTransitionEnums.h"
#include "Conditions/AvaTransitionCondition.h"
#include "AvaTransitionRCControllerMatchCondition.generated.h"

class UAvaSceneSubsystem;
class URCVirtualPropertyBase;
struct FAvaTransitionScene;

USTRUCT()
struct FAvaTransitionRCControllerMatchConditionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Parameter")
	FAvaRCControllerId ControllerId;

	UPROPERTY(EditAnywhere, Category="Parameter")
	EAvaTransitionComparisonResult ValueComparisonType = EAvaTransitionComparisonResult::None;
};

USTRUCT(DisplayName="Compare RC Controller Values", Category="Remote Control")
struct AVALANCHE_API FAvaTransitionRCControllerMatchCondition : public FAvaTransitionCondition
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaTransitionRCControllerMatchConditionInstanceData;

	//~ Begin FStateTreeNodeBase
#if WITH_EDITOR
	virtual FText GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const override;
#endif
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	virtual bool Link(FStateTreeLinker& InLinker) override;
	//~ End FStateTreeNodeBase

	//~ Begin FStateTreeConditionBase
	virtual bool TestCondition(FStateTreeExecutionContext& InContext) const override;
	//~ End FStateTreeConditionBase

	TStateTreeExternalDataHandle<UAvaSceneSubsystem> SceneSubsystemHandle;
};
