// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionEnums.h"
#include "HAL/Platform.h"
#include "IAvaTransitionExtension.h"

struct FAvaTransitionContext;
struct FAvaTransitionScene;
struct FGuid;

class IAvaRCTransitionExtension : public IAvaTransitionExtension
{
public:
	static constexpr const TCHAR* ExtensionIdentifier = TEXT("IAvaRCTransitionExtension");

	/**
	 * Compares the values of an RC controller with the given ControllerId between two transition contexts
	 * @param InControllerId the Id of the controller to compare
	 * @param InMyContext the first transition context to use
	 * @param InOtherContext the second transition context to use
	 * @return the comparison result whether same, different, or none (none meaning that the values could not be properly compared)
	 */
	virtual EAvaTransitionComparisonResult CompareControllers(const FGuid& InControllerId
		, const FAvaTransitionContext& InMyContext
		, const FAvaTransitionContext& InOtherContext) const = 0;
};
