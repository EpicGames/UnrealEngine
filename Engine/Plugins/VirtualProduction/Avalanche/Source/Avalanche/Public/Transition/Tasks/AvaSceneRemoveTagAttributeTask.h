// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSceneTask.h"
#include "AvaSceneRemoveTagAttributeTask.generated.h"

USTRUCT(DisplayName="Remove tag attribute from this scene", Category="Scene Attributes")
struct AVALANCHE_API FAvaSceneRemoveTagAttributeTask : public FAvaSceneTask
{
	GENERATED_BODY()

	//~ Begin FStateTreeNodeBase
#if WITH_EDITOR
	virtual FText GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const override;
#endif
	//~ End FStateTreeNodeBase

	//~ Begin FStateTreeTaskBase
	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& InContext, const FStateTreeTransitionResult& InTransition) const override;
	//~ End FStateTreeTaskBase
};
