// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagHandle.h"
#include "Tasks/AvaTransitionTask.h"
#include "AvaSceneTask.generated.h"

class IAvaSceneInterface;
class UAvaSceneSubsystem;

USTRUCT()
struct FAvaSceneTaskInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Parameter")
	FAvaTagHandle TagAttribute;
};

USTRUCT(meta=(Hidden))
struct AVALANCHE_API FAvaSceneTask : public FAvaTransitionTask
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaSceneTaskInstanceData;

	//~ Begin FStateTreeNodeBase
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	virtual bool Link(FStateTreeLinker& InLinker) override;
	//~ End FStateTreeNodeBase

	IAvaSceneInterface* GetScene(FStateTreeExecutionContext& InContext) const;

	TStateTreeExternalDataHandle<UAvaSceneSubsystem> SceneSubsystemHandle;
};
