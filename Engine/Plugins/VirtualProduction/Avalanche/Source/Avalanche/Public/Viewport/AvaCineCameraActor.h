// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CineCameraActor.h"
#include "AvaCineCameraActor.generated.h"

/** 
 * Motion Design Cine Camera Actor is derived from Cine Camera Actor.
 * Its function is to provide a Cine Camera which can be used right away inside Motion Design.
 * This is done by customizing some of its default values.
 * In particular, Motion Design Editor configuration property "Camera Distance" is used to initialize camera position and manual focus.
 * See: Editor Preferences --> Motion Design --> Editor Settings --> Camera Distance
 */
UCLASS(MinimalAPI, DisplayName = "Motion Design Cine Camera Actor")
class AAvaCineCameraActor : public ACineCameraActor
{
	friend class UAvaEditorSettings;
	
	GENERATED_BODY()

public:
	AAvaCineCameraActor(const FObjectInitializer& ObjectInitializer);

	/**
	 * Initialize the camera with Motion Design scene default values: field of view, camera position, focus distance.
	 * @param InCameraDistance: camera distance value, used to initialize camera position and manual focus distance
	 */
	AVALANCHE_API void Configure(float InCameraDistance);

private:
	/** The CameraDistance value used when configuring this AvaCineCamera*/
	UPROPERTY()
	float DefaultCameraDistance;
};
