// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/PostProcessVolume.h"
#include "AvaPostProcessVolume.generated.h"

/** 
 * Motion Design Post Process Volume is derived from Post Process Volume.
 * Its function is to provide a Post Process which can be used right away inside Motion Design.
 * This is done by customizing some of the default values.
 */
UCLASS(MinimalAPI, DisplayName = "Motion Design Post Process Volume")
class AAvaPostProcessVolume : public APostProcessVolume
{
	GENERATED_BODY()

public:
	AAvaPostProcessVolume();
};
