// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaViewportQualitySettings.generated.h"

class FString;
class FText;
struct FEngineShowFlags;

USTRUCT(BlueprintType)
struct AVALANCHE_API FAvaViewportQualitySettingsFeature
{
	GENERATED_BODY()

	FAvaViewportQualitySettingsFeature() {}
	FAvaViewportQualitySettingsFeature(const FString& InName, const bool bInEnabled)
		: Name(InName), bEnabled(bInEnabled)
	{}

	bool operator==(const FAvaViewportQualitySettingsFeature& InOther) const
	{
		return Name.Equals(InOther.Name);
	}

	/** The name of the feature in the engine show flags. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Quality")
	FString Name;

	/** True if this engine feature show flag should be enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Quality")
	bool bEnabled = false;
};

/** 
 * Motion Design Viewport Quality Settings
 * 
 * Advanced render and quality viewport settings to control performance for a given Viewport.
 * Human-readable and blueprint-able structure that holds flags for the FShowEngineFlags structure.
 * Can convert FShowEngineFlags to FAvaViewportQualitySettings and apply FAvaViewportQualitySettings to a FShowEngineFlags structure.
 */
USTRUCT(BlueprintType)
struct AVALANCHE_API FAvaViewportQualitySettings
{
	GENERATED_BODY()

	static TArray<FAvaViewportQualitySettingsFeature> DefaultFeatures();
	static TArray<FAvaViewportQualitySettingsFeature> AllFeatures(const bool bUseAllFeatures);

	static FAvaViewportQualitySettings Default();
	static FAvaViewportQualitySettings Preset(const FText& InPresetName);
	static FAvaViewportQualitySettings All(const bool bUseAllFeatures);

	static void FeatureNameAndTooltipText(const FString& InFeatureName, FText& OutNameText, FText& OutTooltipText);

	static FAvaViewportQualitySettingsFeature* FindFeatureByName(TArray<FAvaViewportQualitySettingsFeature>& InFeatures, const FString& InFeatureName);
	static const FAvaViewportQualitySettingsFeature* FindFeatureByName(const TArray<FAvaViewportQualitySettingsFeature>& InFeatures, const FString& InFeatureName);

	static void VerifyIntegrity(TArray<FAvaViewportQualitySettingsFeature>& InFeatures);

	static void SortFeaturesByDisplayText(TArray<FAvaViewportQualitySettingsFeature>& InFeatures);

	FAvaViewportQualitySettings();
	FAvaViewportQualitySettings(ENoInit NoInit);
	FAvaViewportQualitySettings(const bool bInUseAllFeatures);
	FAvaViewportQualitySettings(const FEngineShowFlags& InShowFlags);
	FAvaViewportQualitySettings(const TArray<FAvaViewportQualitySettingsFeature>& InFeatures);

	bool operator==(const FAvaViewportQualitySettings& InOther) const;
	bool operator!=(const FAvaViewportQualitySettings& InOther) const;

	/** Applies the settings to the FEngineShowFlags structure provided. */
	void Apply(FEngineShowFlags& InFlags);

	void EnableFeaturesByName(const bool bInEnabled, const TArray<FString>& InFeatureNames);

	void VerifyIntegrity();

	void SortFeaturesByDisplayText();

	/** Advanced viewport client engine features indexed by FEngineShowFlags names. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, EditFixedSize, Category = "Quality", meta = (EditFixedOrder))
	TArray<FAvaViewportQualitySettingsFeature> Features;
};

USTRUCT(BlueprintType)
struct AVALANCHE_API FAvaViewportQualitySettingsPreset
{
	GENERATED_BODY()

	static const FText NoLumen;
	static const FText Reduced;

	FAvaViewportQualitySettingsPreset() {}
	FAvaViewportQualitySettingsPreset(const FText& InPresetName, const FAvaViewportQualitySettings& InQualitySettings)
		: PresetName(InPresetName), QualitySettings(InQualitySettings)
	{}

	bool operator==(const FAvaViewportQualitySettingsPreset& InOther) const
	{
		return PresetName.EqualTo(InOther.PresetName);
	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preset")
	FText PresetName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Preset")
	FAvaViewportQualitySettings QualitySettings;
};
