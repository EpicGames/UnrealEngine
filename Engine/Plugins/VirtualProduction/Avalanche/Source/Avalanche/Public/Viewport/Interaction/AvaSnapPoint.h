// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaDefs.h"
#include "Types/SlateEnums.h"

struct AVALANCHE_API FAvaSnapPoint
{
protected:
	FAvaSnapPoint(const UObject* InOuter, EAvaAnchors InAnchor, const FVector& InLocation, int32 InPointIndex);

public:
	static FAvaSnapPoint CreateScreenSnapPoint(EAvaAnchors InAnchor, const FVector2f& InLocation);
	static FAvaSnapPoint CreateGuideSnapPoint(float Offset, EOrientation Orientation);
	static FAvaSnapPoint CreateSelectionSnapPoint(EAvaAnchors InAnchor, const FVector& InLocation, EAvaDepthAlignment InDepth);
	static FAvaSnapPoint CreateActorSnapPoint(const AActor* InActor, EAvaAnchors InAnchor, const FVector& InLocation, EAvaDepthAlignment InDepth);
	static FAvaSnapPoint CreateActorCustomSnapPoint(const AActor* InActor, const FVector& InLocation, int32 PointIndex);
	static FAvaSnapPoint CreateActorIndexedSnapPoint(const AActor* InActor, EAvaAnchors InAnchor, const FVector& InLocation, int32 PointIndex);
	static FAvaSnapPoint CreateLocalActorSnapPoint(EAvaAnchors InAnchor, const FVector& InLocation, EAvaDepthAlignment InDepth);
	static FAvaSnapPoint CreateLocalActorCustomSnapPoint(const FVector& InLocation, int32 PointIndex);
	static FAvaSnapPoint CreateLocalActorIndexedSnapPoint(EAvaAnchors InAnchor, const FVector& InLocation, int32 PointIndex);
	static FAvaSnapPoint CreateComponentSnapPoint(const USceneComponent* InComponent, EAvaAnchors InAnchor, const FVector& InLocation, EAvaDepthAlignment InDepth);
	static FAvaSnapPoint CreateComponentCustomSnapPoint(const USceneComponent* InComponent, const FVector& InLocation, int32 PointIndex);
	static FAvaSnapPoint CreateComponentIndexedSnapPoint(const USceneComponent* InComponent, EAvaAnchors InAnchor, const FVector& InLocation, int32 PointIndex);
	static FAvaSnapPoint CreateLocalComponentSnapPoint(EAvaAnchors InAnchor, const FVector& InLocation, EAvaDepthAlignment InDepth);
	static FAvaSnapPoint CreateLocalComponentCustomSnapPoint(const FVector& InLocation, int32 PointIndex);
	static FAvaSnapPoint CreateLocalComponentIndexedSnapPoint(EAvaAnchors InAnchor, const FVector& InLocation, int32 PointIndex);
	static FAvaSnapPoint CreateBoundsSnapPoint(EAvaAnchors InAnchor, const FVector& InLocation, EAvaDepthAlignment InDepth);
	static FAvaSnapPoint CreateNullSnapPoint();

	static FAvaSnapPoint CreateActorSnapPoint(const AActor* InActor, EAvaAnchors InAnchor, const FVector2D& InLocation)
	{
		return CreateActorSnapPoint(InActor, InAnchor, FVector(0.f, InLocation.X, InLocation.Y), EAvaDepthAlignment::Front);
	}

	static FAvaSnapPoint CreateActorCustomSnapPoint(const AActor* InActor, const FVector2D& InLocation, int32 PointIndex)
	{
		return CreateActorCustomSnapPoint(InActor, FVector(0.f, InLocation.X, InLocation.Y), PointIndex);
	}

	static FAvaSnapPoint CreateActorIndexedSnapPoint(const AActor* InActor, EAvaAnchors InAnchor, const FVector2D& InLocation, int32 PointIndex)
	{
		return CreateActorIndexedSnapPoint(InActor, InAnchor, FVector(0.f, InLocation.X, InLocation.Y), PointIndex);
	}

	static FAvaSnapPoint CreateLocalActorSnapPoint(EAvaAnchors InAnchor, const FVector2D& InLocation)
	{
		return CreateLocalActorSnapPoint(InAnchor, FVector(0.f, InLocation.X, InLocation.Y), EAvaDepthAlignment::Front);
	}

	static FAvaSnapPoint CreateLocalActorCustomSnapPoint(const FVector2D& InLocation, int32 PointIndex)
	{
		return CreateLocalActorCustomSnapPoint(FVector(0.f, InLocation.X, InLocation.Y), PointIndex);
	}

	static FAvaSnapPoint CreateLocalActorIndexedSnapPoint(EAvaAnchors InAnchor, const FVector2D& InLocation, int32 PointIndex)
	{
		return CreateLocalActorIndexedSnapPoint(InAnchor, FVector(0.f, InLocation.X, InLocation.Y), PointIndex);
	}

	static FAvaSnapPoint CreateComponentSnapPoint(const USceneComponent* InComponent, EAvaAnchors InAnchor, const FVector2D& InLocation)
	{
		return CreateComponentSnapPoint(InComponent, InAnchor, FVector(0.f, InLocation.X, InLocation.Y), EAvaDepthAlignment::Front);
	}

	static FAvaSnapPoint CreateComponentCustomSnapPoint(const USceneComponent* InComponent, const FVector2D& InLocation, int32 PointIndex)
	{
		return CreateComponentCustomSnapPoint(InComponent, FVector(0.f, InLocation.X, InLocation.Y), PointIndex);
	}

	static FAvaSnapPoint CreateComponentIndexedSnapPoint(const USceneComponent* InComponent, EAvaAnchors InAnchor, const FVector2D& InLocation, int32 PointIndex)
	{
		return CreateComponentIndexedSnapPoint(InComponent, InAnchor, FVector(0.f, InLocation.X, InLocation.Y), PointIndex);
	}

	static FAvaSnapPoint CreateLocalComponentSnapPoint(EAvaAnchors InAnchor, const FVector2D& InLocation)
	{
		return CreateLocalComponentSnapPoint(InAnchor, FVector(0.f, InLocation.X, InLocation.Y), EAvaDepthAlignment::Front);
	}

	static FAvaSnapPoint CreateLocalComponentCustomSnapPoint(const FVector2D& InLocation, int32 PointIndex)
	{
		return CreateLocalComponentCustomSnapPoint(FVector(0.f, InLocation.X, InLocation.Y), PointIndex);
	}

	static FAvaSnapPoint CreateLocalComponentIndexedSnapPoint(EAvaAnchors InAnchor, const FVector2D& InLocation, int32 PointIndex)
	{
		return CreateLocalComponentIndexedSnapPoint(InAnchor, FVector(0.f, InLocation.X, InLocation.Y), PointIndex);
	}

	TWeakObjectPtr<const UObject> Outer;
	EAvaAnchors Anchor;
	FVector Location;
	int32 PointIndex;

	FString ToString() const;

	bool IsScreenSnapPoint() const;

	bool IsGuideSnapPoint() const;

	bool IsSelectionSnapPoint() const;

	bool IsActorSnapPoint() const;
};
