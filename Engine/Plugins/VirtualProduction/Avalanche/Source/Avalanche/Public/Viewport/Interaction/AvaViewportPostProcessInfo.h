// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/SoftObjectPtr.h"
#include "AvaViewportPostProcessInfo.generated.h"

class UTexture;

UENUM()
enum class EAvaViewportPostProcessType_Deprecated : uint8
{
	None,
	Background,
	RedChannel,
	GreenChannel,
	BlueChannel,
	AlphaChannel
};

USTRUCT()
struct FAvaViewportPostProcessInfo_Deprecated
{
	GENERATED_BODY()

	UPROPERTY()
	EAvaViewportPostProcessType_Deprecated Type = EAvaViewportPostProcessType_Deprecated::None;

	UPROPERTY()
	TSoftObjectPtr<UTexture> Texture = nullptr;

	UPROPERTY()
	float Opacity = 1.0f;
};
