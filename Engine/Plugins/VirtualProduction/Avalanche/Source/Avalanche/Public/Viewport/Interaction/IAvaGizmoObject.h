// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IAvaGizmoObject.generated.h"

class UAvaGizmoComponent;
class UObject;

/**
 * Interface for objects that can be (optionally) represented as gizmos.
 * Toggling only occurs in-editor, but objects can be gizmos at runtime (ie. used for booleans only)
 */
UINTERFACE(Blueprintable)
class AVALANCHE_API UAvaGizmoObjectInterface : public UInterface
{
	GENERATED_BODY()
};

class AVALANCHE_API IAvaGizmoObjectInterface
{
	GENERATED_BODY()
	
public:
	/** Whether to show the callee as a gizmo or not */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Editor")
	void ToggleGizmo(const UAvaGizmoComponent* InGizmoComponent, const bool bShowAsGizmo = true);
};
