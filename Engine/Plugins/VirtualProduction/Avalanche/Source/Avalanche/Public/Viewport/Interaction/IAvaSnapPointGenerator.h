// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSnapPoint.h"
#include "UObject/Interface.h"
#include "IAvaSnapPointGenerator.generated.h"

UINTERFACE(MinimalAPI, Blueprintable)
class UAvaSnapPointGenerator : public UInterface
{
	GENERATED_BODY()
};

class IAvaSnapPointGenerator
{
	GENERATED_BODY()

public:
	virtual TArray<FAvaSnapPoint> GetLocalSnapPoints() const = 0;
};
