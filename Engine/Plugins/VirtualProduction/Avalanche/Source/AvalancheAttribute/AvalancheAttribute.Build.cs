// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheAttribute : ModuleRules
{
    public AvalancheAttribute(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "AvalancheTag",
                "Core",
                "CoreUObject",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "Engine",
                "Slate",
                "SlateCore",
            }
        );
    }
}
