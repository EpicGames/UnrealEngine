// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaNameAttribute.h"

#define LOCTEXT_NAMESPACE "AvaNameAttribute"

FText UAvaNameAttribute::GetDisplayName() const
{
	return FText::Format(LOCTEXT("DisplayName", "Name: {0}"), FText::FromName(Name));
}

void UAvaNameAttribute::SetName(FName InName)
{
	Name = InName;
}

#undef LOCTEXT_NAMESPACE
