// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tags/AvaTagAttribute.h"

#define LOCTEXT_NAMESPACE "AvaTagAttribute"

FText UAvaTagAttribute::GetDisplayName() const
{
	return FText::Format(LOCTEXT("DisplayName", "Tag Attribute: {0}"), FText::FromName(Tag.ToName()));
}

bool UAvaTagAttribute::SetTagHandle(const FAvaTagHandle& InTagHandle)
{
	if (Tag.MatchesExact(InTagHandle))
	{
		return false;
	}

	Tag = InTagHandle;
	return true;
}

bool UAvaTagAttribute::ClearTagHandle(const FAvaTagHandle& InTagHandle)
{
	if (Tag.MatchesExact(InTagHandle))
	{
		Tag = FAvaTagHandle();
		return true;
	}
	return false;
}

bool UAvaTagAttribute::ContainsTag(const FAvaTagHandle& InTagHandle) const
{
	return Tag.Overlaps(InTagHandle);
}

bool UAvaTagAttribute::HasValidTagHandle() const
{
	return Tag.IsValid();
}

#undef LOCTEXT_NAMESPACE
