// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tags/AvaTagContainerAttribute.h"

#define LOCTEXT_NAMESPACE "AvaTagContainerAttribute"

FText UAvaTagContainerAttribute::GetDisplayName() const
{
	return FText::Format(LOCTEXT("DisplayName", "Tag Container Attribute: {0}"), FText::FromString(TagContainer.ToString()));
}

bool UAvaTagContainerAttribute::SetTagHandle(const FAvaTagHandle& InTagHandle)
{
	return TagContainer.AddTagHandle(InTagHandle);
}

bool UAvaTagContainerAttribute::ClearTagHandle(const FAvaTagHandle& InTagHandle)
{
	return TagContainer.RemoveTagHandle(InTagHandle);
}

bool UAvaTagContainerAttribute::ContainsTag(const FAvaTagHandle& InTagHandle) const
{
	return TagContainer.ContainsTag(InTagHandle);
}

bool UAvaTagContainerAttribute::HasValidTagHandle() const
{
	return TagContainer.GetTagIds().ContainsByPredicate(
		[](const FAvaTagId& InTagId)
		{
			return InTagId.IsValid();
		});
}

void UAvaTagContainerAttribute::SetTagContainer(const FAvaTagHandleContainer& InTagContainer)
{
	TagContainer = InTagContainer;
}

#undef LOCTEXT_NAMESPACE
