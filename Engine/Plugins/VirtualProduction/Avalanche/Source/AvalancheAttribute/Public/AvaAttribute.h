// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "AvaAttribute.generated.h"

/** Attributes are objects that are added to other objects to describe said object */
UCLASS(MinimalAPI, Abstract, EditInlineNew)
class UAvaAttribute : public UObject
{
	GENERATED_BODY()

public:
	/** Gets the display name to use for the attribute */
	virtual FText GetDisplayName() const
	{
		return FText::GetEmpty();
	}
};
