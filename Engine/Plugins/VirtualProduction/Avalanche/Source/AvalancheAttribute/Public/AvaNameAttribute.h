// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaAttribute.h"
#include "AvaNameAttribute.generated.h"

/** Attribute that holds a name */
UCLASS(MinimalAPI, DisplayName="Name Attribute")
class UAvaNameAttribute : public UAvaAttribute
{
	GENERATED_BODY()

public:
	//~ Begin UAvaAttribute
	AVALANCHEATTRIBUTE_API virtual FText GetDisplayName() const override;
	//~ End UAvaAttribute

	UFUNCTION()
	AVALANCHEATTRIBUTE_API void SetName(FName InName);

	UPROPERTY(EditAnywhere, Setter, Category="Attributes")
	FName Name;
};
