// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagAttributeBase.h"
#include "AvaTagHandle.h"
#include "AvaTagAttribute.generated.h"

/** Attribute that holds a tag handle, for single tag referencing */
UCLASS(MinimalAPI, DisplayName="Tag Attribute")
class UAvaTagAttribute : public UAvaTagAttributeBase
{
	GENERATED_BODY()

public:
	//~ Begin UAvaAttribute
	AVALANCHEATTRIBUTE_API virtual FText GetDisplayName() const override;
	//~ End UAvaAttribute

	//~ Begin UAvaTagAttributeBase
	AVALANCHEATTRIBUTE_API virtual bool SetTagHandle(const FAvaTagHandle& InTagHandle) override;
	AVALANCHEATTRIBUTE_API virtual bool ClearTagHandle(const FAvaTagHandle& InTagHandle) override;
	AVALANCHEATTRIBUTE_API virtual bool ContainsTag(const FAvaTagHandle& InTagHandle) const override;
	AVALANCHEATTRIBUTE_API virtual bool HasValidTagHandle() const override;
	//~ End UAvaTagAttributeBase

	UFUNCTION()
	void SetTag(const FAvaTagHandle& InTag)
	{
		Tag = InTag;
	}

	UPROPERTY(EditAnywhere, Setter, Category="Attributes")
	FAvaTagHandle Tag;
};
