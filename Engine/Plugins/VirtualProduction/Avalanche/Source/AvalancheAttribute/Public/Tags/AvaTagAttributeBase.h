// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaAttribute.h"
#include "AvaTagAttributeBase.generated.h"

struct FAvaTagHandle;

/** Base implementation of an attribute that represent one or more tags in some form */
UCLASS(MinimalAPI, Abstract)
class UAvaTagAttributeBase : public UAvaAttribute
{
	GENERATED_BODY()

public:
	/**
	 * Sets/Adds the given tag handle to the tag attribute
	 * @param InTagHandle the tag handle to add
	 * @return true if the tag handle was added, false if this tag handle already existed in the tag attribute
	 */
	virtual bool SetTagHandle(const FAvaTagHandle& InTagHandle)
	{
		return false;
	}

	/**
	 * Clears/Removes the given tag handle from the tag attribute
	 * @param InTagHandle the tag handle to remove
	 * @return true if the tag handle was removed, false if this tag handle was not present in the tag attribute and nothing was removed
	 */
	virtual bool ClearTagHandle(const FAvaTagHandle& InTagHandle)
	{
		return false;
	}

	/**
	 * Checks whether the resolved tag from the given tag handle exists in the tag attribute
	 * @param InTagHandle the tag handle to resolve into a tag and check if the tag attribute contains it
	 * @return true if the resolved tag exists in the tag attribute, false otherwise.
	 */
	virtual bool ContainsTag(const FAvaTagHandle& InTagHandle) const
	{
		return false;
	}

	/**
	 * Checks whether the tag attribute has any tag handle that could resolve to a valid tag
	 * @return true if the tag attribute has a valid tag handle that could resolve to a tag
	 */
	virtual bool HasValidTagHandle() const
	{
		return false;
	}
};
