// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagAttributeBase.h"
#include "AvaTagHandleContainer.h"
#include "AvaTagContainerAttribute.generated.h"

/** Attribute that holds a tag container handle, for multi-tag referencing */
UCLASS(MinimalAPI, DisplayName="Tag Container Attribute")
class UAvaTagContainerAttribute : public UAvaTagAttributeBase
{
	GENERATED_BODY()

public:
	//~ Begin UAvaAttribute
	AVALANCHEATTRIBUTE_API virtual FText GetDisplayName() const override;
	//~ End UAvaAttribute

	//~ Begin UAvaTagAttributeBase
	AVALANCHEATTRIBUTE_API virtual bool SetTagHandle(const FAvaTagHandle& InTagHandle) override;
	AVALANCHEATTRIBUTE_API virtual bool ClearTagHandle(const FAvaTagHandle& InTagHandle) override;
	AVALANCHEATTRIBUTE_API virtual bool ContainsTag(const FAvaTagHandle& InTagHandle) const override;
	AVALANCHEATTRIBUTE_API virtual bool HasValidTagHandle() const;
	//~ End UAvaTagAttributeBase

	UFUNCTION()
	AVALANCHEATTRIBUTE_API void SetTagContainer(const FAvaTagHandleContainer& InTagContainer);

	UPROPERTY(EditAnywhere, Setter, Category="Attributes")
	FAvaTagHandleContainer TagContainer;
};
