// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheAttributeEditor : ModuleRules
{
    public AvalancheAttributeEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "AvalancheAttribute",
                "CoreUObject",
                "Engine",
                "PropertyEditor",
                "Slate",
                "SlateCore", 
                "UnrealEd",
            }
        );
    }
}
