// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaAttributeEditorModule.h"
#include "Customizations/AvaAttributeNodeBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Text/STextBlock.h"

IMPLEMENT_MODULE(FAvaAttributeEditorModule, AvalancheAttributeEditor)

void FAvaAttributeEditorModule::CustomizeAttributes(const TSharedRef<IPropertyHandle>& InAttributesHandle, IDetailLayoutBuilder& InDetailBuilder)
{
	InAttributesHandle->MarkHiddenByCustomization();

	IDetailCategoryBuilder& AttributesCategory = InDetailBuilder.EditCategory(InAttributesHandle->GetDefaultCategoryName());

	TSharedRef<SWidget> HeaderContentWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(AttributesCategory.GetDisplayName())
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			InAttributesHandle->CreateDefaultPropertyButtonWidgets()
		];

	AttributesCategory.HeaderContent(HeaderContentWidget, /*bWholeRowContent*/true);

	TSharedRef<FDetailArrayBuilder> AttributeArrayBuilder = MakeShared<FDetailArrayBuilder>(InAttributesHandle
		, /*GenerateHeader*/false
		, /*DisplayResetToDefault*/false
		, /*bDisplayElementNum*/false);

	TWeakPtr<IPropertyUtilities> PropertyUtilitiesWeak = InDetailBuilder.GetPropertyUtilities();

	AttributeArrayBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateLambda(
		[PropertyUtilitiesWeak](TSharedRef<IPropertyHandle> InAttributeHandle, int32 InArrayIndex, IDetailChildrenBuilder& InChildrenBuilder)
		{
			InChildrenBuilder.AddCustomBuilder(MakeShared<FAvaAttributeNodeBuilder>(InAttributeHandle, PropertyUtilitiesWeak));
		}));

	AttributesCategory.AddCustomBuilder(AttributeArrayBuilder);
}
