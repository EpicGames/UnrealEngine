// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaAttributeEditorModule.h"

class FAvaAttributeEditorModule : public IAvaAttributeEditorModule
{
public:
	//~ Begin IAvaAttributeEditorModule
	virtual void CustomizeAttributes(const TSharedRef<IPropertyHandle>& InAttributesHandle, IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IAvaAttributeEditorModule
};
