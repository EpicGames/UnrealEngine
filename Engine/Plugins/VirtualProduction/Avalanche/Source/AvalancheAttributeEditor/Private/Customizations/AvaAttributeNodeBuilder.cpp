// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaAttributeNodeBuilder.h"
#include "DetailWidgetRow.h"
#include "DragDrop/AvaArrayItemDragDropHandler.h"
#include "IDetailChildrenBuilder.h"
#include "ObjectEditorUtils.h"
#include "Widgets/SAvaAttributePicker.h"

FName FAvaAttributeNodeBuilder::GetName() const
{
	return TEXT("FAvaAttributeNodeBuilder");
}

void FAvaAttributeNodeBuilder::GenerateHeaderRowContent(FDetailWidgetRow& InNodeRow)
{
	TSharedRef<SWidget> WholeRowContent = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.HAlign(HAlign_Fill)
		[
			SNew(SAvaAttributePicker, AttributeHandle)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			AttributeHandle->CreateDefaultPropertyButtonWidgets()
		];

	InNodeRow.WholeRowContent()
		.HAlign(HAlign_Left)
		[
			SNew(SBox)
			.MinDesiredWidth(250.f)
			[
				WholeRowContent
			]
		];

	InNodeRow.DragDropHandler(MakeShared<FAvaArrayItemDragDropHandler>(AttributeHandle, WholeRowContent, PropertyUtilitiesWeak));
}

void FAvaAttributeNodeBuilder::GenerateChildContent(IDetailChildrenBuilder& InChildrenBuilder)
{
	GenerateChildContentRecursive(AttributeHandle, InChildrenBuilder, TEXT("Attributes"));
}

TSharedPtr<IPropertyHandle> FAvaAttributeNodeBuilder::GetPropertyHandle() const
{
	return AttributeHandle;
}

void FAvaAttributeNodeBuilder::GenerateChildContentRecursive(const TSharedRef<IPropertyHandle>& InParentHandle, IDetailChildrenBuilder& InChildrenBuilder, FName InDefaultCategoryName)
{
	uint32 NumChildren = 0;
	InParentHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = InParentHandle->GetChildHandle(ChildIndex);
		if (!ChildHandle.IsValid())
		{
			return;
		}

		if (ChildHandle->GetProperty())
		{
			InChildrenBuilder.AddProperty(ChildHandle.ToSharedRef());
			continue;
		}

		// If handle is a category handle, find its category name through its children (currently, no way to get the category name directly)
		if (ChildHandle->IsCategoryHandle())
		{
			TOptional<FName> CategoryName;

			uint32 NumGrandChildren = 0;
			ChildHandle->GetNumChildren(NumGrandChildren);

			for (uint32 GrandChildIndex = 0; GrandChildIndex < NumGrandChildren; ++GrandChildIndex)
			{
				TSharedPtr<IPropertyHandle> GrandChildHandle = ChildHandle->GetChildHandle(GrandChildIndex);
				if (FProperty* Property = GrandChildHandle->GetProperty())
				{
					CategoryName = FObjectEditorUtils::GetCategoryFName(Property);
					break;
				}
			}

			// If a category was found, but it does not match the default category, add it to the builder
			if (CategoryName.IsSet() && *CategoryName != InDefaultCategoryName)
			{
				InChildrenBuilder.AddProperty(ChildHandle.ToSharedRef());
				continue;
			}
		}

		// If the handle was a handle to the default category, or it's a whole other type of handle, recurse its children 
		GenerateChildContentRecursive(ChildHandle.ToSharedRef(), InChildrenBuilder, InDefaultCategoryName);
	}
}
