// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomNodeBuilder.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;
class IPropertyUtilities;

class FAvaAttributeNodeBuilder : public IDetailCustomNodeBuilder
{
public:
	explicit FAvaAttributeNodeBuilder(const TSharedRef<IPropertyHandle>& InAttributeHandle, const TWeakPtr<IPropertyUtilities>& InPropertyUtilitiesWeak)
		: AttributeHandle(InAttributeHandle)
		, PropertyUtilitiesWeak(InPropertyUtilitiesWeak)
	{
	}

private:
	//~ Begin IDetailCustomNodeBuilder
	virtual FName GetName() const override;
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& InNodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& InChildrenBuilder) override;
	virtual TSharedPtr<IPropertyHandle> GetPropertyHandle() const override;
	//~ End IDetailCustomNodeBuilder

	void GenerateChildContentRecursive(const TSharedRef<IPropertyHandle>& InParentHandle, IDetailChildrenBuilder& InChildrenBuilder, FName InDefaultCategoryName);

	TSharedRef<IPropertyHandle> AttributeHandle;

	TWeakPtr<IPropertyUtilities> PropertyUtilitiesWeak;
};
