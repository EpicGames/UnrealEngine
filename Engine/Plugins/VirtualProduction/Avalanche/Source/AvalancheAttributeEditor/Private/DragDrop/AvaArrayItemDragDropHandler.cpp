// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaArrayItemDragDropHandler.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "AvaArrayItemDragDropHandler"

namespace UE::AvaEditor::Private
{
	int32 CalculateTargetIndex(int32 InSourceIndex, int32 InTargetIndex, EItemDropZone DropZone)
	{
		if (DropZone == EItemDropZone::BelowItem)
		{
			// If the drop zone is below, then move it to the next item's index
			InTargetIndex++;
		}

		if (InSourceIndex < InTargetIndex)
		{
			// If the item is moved down the list, then all the other elements below it are shifted up one
			InTargetIndex--;
		}

		return ensure(InTargetIndex >= 0) ? InTargetIndex : 0;
	}

	bool IsValidDrop(const TSharedPtr<IPropertyHandle>& InTargetItemHandle, const TSharedPtr<IPropertyHandle>& InDraggedItemHandle, EItemDropZone InDropZone)
	{
		// Can't drop onto another array item; need to drop above or below. Likewise, cannot drop above/below itself (redundant operation)
		if (InDropZone == EItemDropZone::OntoItem || InTargetItemHandle == InDraggedItemHandle)
		{
			return false;
		}

		const int32 SourceIndex = InDraggedItemHandle->GetIndexInArray();
		const int32 TargetIndex = CalculateTargetIndex(SourceIndex, InTargetItemHandle->GetArrayIndex(), InDropZone);

		if (SourceIndex != TargetIndex)
		{
			TSharedPtr<IPropertyHandle> TargetParentHandle  = InTargetItemHandle->GetParentHandle();
			TSharedPtr<IPropertyHandle> DraggedParentHandle = InDraggedItemHandle->GetParentHandle();

			// Ensure that these two property handles belong to the same array parent property
			return TargetParentHandle.IsValid() && DraggedParentHandle.IsValid()
				&& TargetParentHandle->IsSamePropertyNode(DraggedParentHandle)
				&& TargetParentHandle->AsArray().IsValid();
		}

		return false;
	}
}

class FAvaArrayItemDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FAvaAttributeElementDragDropOp, FDecoratedDragDropOp)

	FAvaArrayItemDragDropOp(const TWeakPtr<IPropertyHandle>& InArrayItemHandleWeak)
	{
		ArrayItemHandleWeak = InArrayItemHandleWeak;
		MouseCursor = EMouseCursor::GrabHandClosed;
	}

	void Init()
	{
		SetValidDrop(false);
		SetupDefaults();
		Construct();
	}

	void SetValidDrop(bool bIsValidDrop)
	{
		if (bIsValidDrop)
		{
			CurrentHoverText = LOCTEXT("PlaceRowHere", "Place Row Here");
			CurrentIconBrush = FAppStyle::GetBrush("Graph.ConnectorFeedback.OK");
		}
		else
		{
			CurrentHoverText = LOCTEXT("CannotPlaceRowHere", "Cannot Place Row Here");
			CurrentIconBrush = FAppStyle::GetBrush("Graph.ConnectorFeedback.Error");
		}
	}

	TWeakPtr<IPropertyHandle> ArrayItemHandleWeak;
};

FAvaArrayItemDragDropHandler::FAvaArrayItemDragDropHandler(const TSharedRef<IPropertyHandle>& InArrayItemHandle, const TSharedRef<SWidget>& InRowWidget, const TWeakPtr<IPropertyUtilities>& InPropertyUtilities)
	: ArrayItemHandleWeak(InArrayItemHandle)
	, RowWidgetWeak(InRowWidget)
	, PropertyUtilitiesWeak(InPropertyUtilities)
{
}

TSharedPtr<FDragDropOperation> FAvaArrayItemDragDropHandler::CreateDragDropOperation() const
{
	return MakeShared<FAvaArrayItemDragDropOp>(ArrayItemHandleWeak);
}

TOptional<EItemDropZone> FAvaArrayItemDragDropHandler::CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) const
{
	TSharedPtr<FAvaArrayItemDragDropOp> ArrayItemDragDropOp = InDragDropEvent.GetOperationAs<FAvaArrayItemDragDropOp>();
	TSharedPtr<IPropertyHandle> ArrayItemHandle = ArrayItemHandleWeak.Pin();
	TSharedPtr<SWidget> RowWidget = RowWidgetWeak.Pin();

	if (!RowWidget.IsValid() || !ArrayItemHandle.IsValid() || !ArrayItemDragDropOp.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	TSharedPtr<IPropertyHandle> DraggedArrayItemHandle = ArrayItemDragDropOp->ArrayItemHandleWeak.Pin();
	if (!DraggedArrayItemHandle.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	const FGeometry& Geometry = RowWidget->GetTickSpaceGeometry();
	const float LocalPointerY = Geometry.AbsoluteToLocal(InDragDropEvent.GetScreenSpacePosition()).Y;

	const EItemDropZone OverrideDropZone = LocalPointerY < Geometry.GetLocalSize().Y * 0.5f
		? EItemDropZone::AboveItem
		: EItemDropZone::BelowItem;

	if (!UE::AvaEditor::Private::IsValidDrop(ArrayItemHandle, DraggedArrayItemHandle, OverrideDropZone))
	{
		ArrayItemDragDropOp->SetValidDrop(false);
		return TOptional<EItemDropZone>();
	}

	ArrayItemDragDropOp->SetValidDrop(true);
	return OverrideDropZone;
}

bool FAvaArrayItemDragDropHandler::AcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) const
{
	TSharedPtr<FAvaArrayItemDragDropOp> ArrayItemDragDropOp = InDragDropEvent.GetOperationAs<FAvaArrayItemDragDropOp>();
	TSharedPtr<IPropertyHandle> ArrayItemHandle = ArrayItemHandleWeak.Pin();
	TSharedPtr<SWidget> RowWidget = RowWidgetWeak.Pin();

	if (!RowWidget.IsValid() || !ArrayItemHandle.IsValid() || !ArrayItemDragDropOp.IsValid())
	{
		return false;
	}

	TSharedPtr<IPropertyHandle> DraggedArrayItemHandle = ArrayItemDragDropOp->ArrayItemHandleWeak.Pin();
	TSharedPtr<IPropertyHandle> ParentHandle = ArrayItemHandle->GetParentHandle();

	if (!DraggedArrayItemHandle.IsValid() || !ParentHandle.IsValid())
	{
		return false;
	}

	TSharedPtr<IPropertyHandleArray> ParentArrayHandle = ParentHandle->AsArray();

	if (!ParentArrayHandle.IsValid())
	{
		return false;
	}

	const int32 SourceIndex = DraggedArrayItemHandle->GetArrayIndex();
	const int32 TargetIndex = UE::AvaEditor::Private::CalculateTargetIndex(SourceIndex, ArrayItemHandle->GetArrayIndex(), InDropZone);

	FScopedTransaction Transaction(LOCTEXT("MoveRow", "Move Row"));

	ParentHandle->NotifyPreChange();
	ParentArrayHandle->MoveElementTo(SourceIndex, TargetIndex);
	ParentHandle->NotifyPostChange(EPropertyChangeType::ArrayMove);

	// IPropertyHandle::NotifyFinishedChangingProperties is not called as it hard codes the change type to ValueSet, when this is an Array Move
	// Instead, PropertyUtilities function is called directly with the correct change type
	if (TSharedPtr<IPropertyUtilities> PropertyUtilities = PropertyUtilitiesWeak.Pin())
	{
		FPropertyChangedEvent MoveEvent(ParentHandle->GetProperty(), EPropertyChangeType::ArrayMove);
		PropertyUtilities->NotifyFinishedChangingProperties(MoveEvent);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
