// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailDragDropHandler.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;
class IPropertyUtilities;
class SWidget;

/**
 * Detail Drag Drop Handler for Custom Builders to re-order the array items
 *
 * Its existence is due to the following,
 * 1) The Property Editor API only adds a reorder handle widget if the FProperty of GetPropertyNode is able to re-order (owner is an array property that can re-order)
 * 2) The Array item property handle in question is being presented through a Custom Builder instead of directly as a property
 * 3) Custom Builder customizations don't have a Property Node set (null), so the Property Editor API does not pick up the property to check with (1)
 * 4) The only other way for the Property Editor API to add a reorder handle widget is by having a custom drag drop handler
 * 5) There's no Array Detail Drag Drop Handler built-in; the default array reordering is handled differently (and is private) rather than through a IDetailDragDropHandler
 */
class FAvaArrayItemDragDropHandler : public IDetailDragDropHandler
{
public:
	explicit FAvaArrayItemDragDropHandler(const TSharedRef<IPropertyHandle>& InArrayItemHandle, const TSharedRef<SWidget>& InRowWidget, const TWeakPtr<IPropertyUtilities>& InPropertyUtilities);

	//~ Begin IDetailDragDropHandler
	virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation() const override;
	virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) const override;
	virtual bool AcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) const override;
	//~ End IDetailDragDropHandler

private:
	TWeakPtr<IPropertyHandle> ArrayItemHandleWeak;

	TWeakPtr<SWidget> RowWidgetWeak;

	TWeakPtr<IPropertyUtilities> PropertyUtilitiesWeak;
};
