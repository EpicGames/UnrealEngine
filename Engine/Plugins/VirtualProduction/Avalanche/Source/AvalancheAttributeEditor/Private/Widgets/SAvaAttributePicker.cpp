// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaAttributePicker.h"
#include "AvaAttribute.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "DetailLayoutBuilder.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "Styling/SlateIconFinder.h"
#include "UObject/ConstructorHelpers.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaAttributePicker"

class FAvaAttributeClassFilter : public IClassViewerFilter
{
public:
	static constexpr EClassFlags DisallowedClassFlags = CLASS_Deprecated | CLASS_Abstract;

	//~ Begin IClassViewerFilter
	virtual bool IsClassAllowed(const FClassViewerInitializationOptions&, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs>) override
	{
		return InClass
			&& InClass->IsChildOf<UAvaAttribute>()
			&& !InClass->HasAnyClassFlags(DisallowedClassFlags);
	}
	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions&, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs>) override
	{
		return InUnloadedClassData->IsChildOf(UAvaAttribute::StaticClass())
			&& !InUnloadedClassData->HasAnyClassFlags(DisallowedClassFlags);
	}
	//~ End IClassViewerFilter
};

void SAvaAttributePicker::Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InAttributeHandle)
{
	AttributeHandle = InAttributeHandle;

	int32 ArrayIndex = InAttributeHandle->GetArrayIndex();

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(INVTEXT("["))
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		]
		+ SHorizontalBox::Slot()
		.Padding(3.f, 0.f, 3.f, 0.f)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(FText::AsNumber(ArrayIndex))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(INVTEXT("]"))
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		]
		+ SHorizontalBox::Slot()
		.Padding(5.f, 0.f, 0.f, 0.f)
		.FillWidth(1.f)
		.VAlign(VAlign_Center)
		[
			SAssignNew(PickerComboButton, SComboButton)
			.IsEnabled(this, &SAvaAttributePicker::IsPropertyEnabled)
			.ContentPadding(1)
			.OnGetMenuContent(this, &SAvaAttributePicker::GenerateClassPicker)
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(this, &SAvaAttributePicker::GetIcon)
				]
				+ SHorizontalBox::Slot()
				.Padding(4.f, 0.f, 0.f, 0.f)
				.FillWidth(1.f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &SAvaAttributePicker::GetDisplayName)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
			]
		]
	];
}

bool SAvaAttributePicker::IsPropertyEnabled() const
{
	return !AttributeHandle->IsEditConst();
}

UAvaAttribute* SAvaAttributePicker::GetAttribute() const
{
	UObject* Object = nullptr;
	if (AttributeHandle->GetValue(Object) == FPropertyAccess::Success)
	{
		return Cast<UAvaAttribute>(Object);
	}
	return nullptr;
}

FText SAvaAttributePicker::GetDisplayName() const
{
	if (UAvaAttribute* Attribute = GetAttribute())
	{
		FText DisplayName = Attribute->GetDisplayName();
		if (!DisplayName.IsEmpty())
		{
			return DisplayName;
		}
		return Attribute->GetClass()->GetDisplayNameText();
	}
	return FText::GetEmpty();
}

const FSlateBrush* SAvaAttributePicker::GetIcon() const
{
	if (UAvaAttribute* Attribute = GetAttribute())
	{
		return FSlateIconFinder::FindIconBrushForClass(Attribute->GetClass());
	}
	return nullptr;
}

TSharedRef<SWidget> SAvaAttributePicker::GenerateClassPicker()
{
	FClassViewerInitializationOptions Options;
	Options.PropertyHandle = AttributeHandle;
	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
	Options.bShowBackgroundBorder = false;
	Options.bShowUnloadedBlueprints = true;
	Options.ClassFilters.Add(MakeShared<FAvaAttributeClassFilter>());
	Options.bShowNoneOption = true;

	FOnClassPicked OnClassPicked(FOnClassPicked::CreateSP(this, &SAvaAttributePicker::OnClassPicked));
	return FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, OnClassPicked);
}

void SAvaAttributePicker::OnClassPicked(UClass* InClassPicked)
{
	// Class can be null, but never a class that isn't an attribute
	if (!ensure(!InClassPicked || InClassPicked->IsChildOf<UAvaAttribute>()))
	{
		return;
	}

	// Gather current values to cleanup later (named previous to clearly differentiate with the new values after they've been set)
	TArray<FString> PreviousValues;
	if (FObjectProperty* ObjectProperty = CastFieldChecked<FObjectProperty>(AttributeHandle->GetProperty()))
	{
		if (ObjectProperty->HasAnyPropertyFlags(CPF_InstancedReference))
		{
			AttributeHandle->GetPerObjectValues(PreviousValues);
		}
	}

	TArray<FString> NewValues = GenerateNewValues(PreviousValues, InClassPicked);
	ensure(PreviousValues.IsEmpty() || PreviousValues.Num() == NewValues.Num());

	// SetPerObjectValues already transacts, but old subobject renames must also happen within the same transaction,
	// so create one covering this entire scope
	FScopedTransaction Transaction(LOCTEXT("SetAttributeClass", "Set Attribute Class"));

	AttributeHandle->SetPerObjectValues(NewValues);

	for (int32 ValueIndex = 0; ValueIndex < PreviousValues.Num(); ++ValueIndex)
	{
		FString& PreviousValue = PreviousValues[ValueIndex];
		if (!PreviousValue.IsEmpty() && PreviousValue != NewValues[ValueIndex])
		{
			// Move the old subobject to the transient package so GetObjectsWithOuter will not return it
			ConstructorHelpers::StripObjectClass(PreviousValue);
			if (UObject* SubObject = StaticFindObject(UObject::StaticClass(), nullptr, *PreviousValue))
			{
				SubObject->Rename(nullptr, GetTransientOuterForRename(SubObject->GetClass()), REN_DontCreateRedirectors);
			}
		}
	}

	AttributeHandle->RequestRebuildChildren();
	PickerComboButton->SetIsOpen(false);
}

TArray<FString> SAvaAttributePicker::GenerateNewValues(TConstArrayView<FString> InCurrentValues, UClass* InClassPicked) const
{
	const FString NoneString = FName(NAME_None).ToString();

	TArray<FString> NewValues;

	if (!InClassPicked)
	{
		int32 PerObjectValueCount = AttributeHandle->GetNumPerObjectValues();
		NewValues.Init(NoneString, PerObjectValueCount);
		return NewValues;
	}

	const FString ClassPath = InClassPicked->GetClassPathName().ToString();

	TArray<UObject*> Outers;
	AttributeHandle->GetOuterObjects(Outers);

	NewValues.Reserve(Outers.Num());

	for (int32 ObjectIndex = 0; ObjectIndex < Outers.Num(); ++ObjectIndex)
	{
		FString& NewValue = NewValues.Emplace_GetRef();
		const FString* CurrentValue = nullptr;

		if (InCurrentValues.IsValidIndex(ObjectIndex))
		{
			CurrentValue = &InCurrentValues[ObjectIndex];
		}

		FStringView CurrentClassName;
		int32 ClassEndIndex;
		if (CurrentValue && *CurrentValue != NoneString && CurrentValue->FindChar(TEXT('\''), ClassEndIndex))
		{
			CurrentClassName = FStringView(*CurrentValue).Left(ClassEndIndex);
		}

		// Re-use old object if the same class was picked
		if (CurrentValue &&  ClassPath == CurrentClassName)
		{
			NewValue = *CurrentValue;
		}
		else
		{
			UObject* Outer = Outers[ObjectIndex];
			EObjectFlags MaskedOuterFlags = Outer->GetMaskedFlags(RF_PropagateToSubObjects);

			UObject* InstancedObject = NewObject<UObject>(Outer, InClassPicked, NAME_None, MaskedOuterFlags);

			// Wrap the value in quotes before setting - in some cases for editinline-instanced values, the outer object path
			// can potentially contain a space token (e.g. if the outer object was instanced as a Blueprint template object
			// based on a user-facing variable name). While technically such characters should not be allowed, historically there
			// has been no issue with most tokens in the INVALID_OBJECTNAME_CHARACTERS set being present in in-memory object names,
			// other than some systems failing to resolve the object's path if the string representation of the value is not quoted.
			NewValue = FString::Printf(TEXT("\"%s\""), *InstancedObject->GetPathName().ReplaceQuotesWithEscapedQuotes());
		}
	}

	return NewValues;
}

#undef LOCTEXT_NAMESPACE
