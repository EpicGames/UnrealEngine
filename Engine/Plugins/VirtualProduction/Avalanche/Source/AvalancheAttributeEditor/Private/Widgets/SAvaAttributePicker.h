// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FText;
class IPropertyHandle;
class SComboButton;
class UAvaAttribute;
class UClass;
struct FSlateBrush;

class SAvaAttributePicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaAttributePicker) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InAttributeHandle);

private:
	bool IsPropertyEnabled() const;

	UAvaAttribute* GetAttribute() const;

	FText GetDisplayName() const;

	const FSlateBrush* GetIcon() const;

	TSharedRef<SWidget> GenerateClassPicker();

	void OnClassPicked(UClass* InClassPicked);

	TArray<FString> GenerateNewValues(TConstArrayView<FString> InCurrentValues, UClass* InClassPicked) const;

	TSharedPtr<IPropertyHandle> AttributeHandle;

	TSharedPtr<SComboButton> PickerComboButton;
};
