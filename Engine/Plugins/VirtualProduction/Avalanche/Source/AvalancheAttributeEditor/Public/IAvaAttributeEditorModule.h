// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Templates/SharedPointerFwd.h"

class IDetailLayoutBuilder;
class IPropertyHandle;

class IAvaAttributeEditorModule : public IModuleInterface
{
	static constexpr const TCHAR* ModuleName = TEXT("AvalancheAttributeEditor");

public:
	static bool IsLoaded()
	{
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	static IAvaAttributeEditorModule& Get()
	{
		return FModuleManager::Get().LoadModuleChecked<IAvaAttributeEditorModule>(ModuleName);
	}

	virtual void CustomizeAttributes(const TSharedRef<IPropertyHandle>& InAttributesHandle, IDetailLayoutBuilder& InDetailBuilder) = 0;
};
