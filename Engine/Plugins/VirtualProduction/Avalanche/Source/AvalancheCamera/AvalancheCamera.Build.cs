// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheCamera : ModuleRules
{
    public AvalancheCamera(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "ActorModifierCore",
                "AvalancheTransition",
                "Core",
                "StateTreeModule",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "CoreUObject",
                "Engine",
            }
        );

        if (Target.Type == TargetRules.TargetType.Editor)
        {
            PrivateDependencyModuleNames.AddRange(new string[]
            {
                "SlateCore",
            });
        }
    }
}