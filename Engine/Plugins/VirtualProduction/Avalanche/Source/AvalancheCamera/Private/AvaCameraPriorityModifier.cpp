// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaCameraPriorityModifier.h"
#include "Camera/CameraComponent.h"

#if WITH_EDITOR
#include "Styling/SlateIconFinder.h"
#endif

#define LOCTEXT_NAMESPACE "AvaCameraModifier"

UAvaCameraPriorityModifier::UAvaCameraPriorityModifier()
{
	TransitionParams.bLockOutgoing = true;
}

void UAvaCameraPriorityModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetCompatibilityRule([](const AActor* InActor)->bool
	{
		return InActor && InActor->FindComponentByClass<UCameraComponent>();
	});

	InMetadata.SetName(TEXT("CameraPriority"));
	InMetadata.SetCategory(TEXT("Camera"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Camera Priority"));
	InMetadata.SetIcon(FSlateIconFinder::FindIconForClass(UCameraComponent::StaticClass()));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Camera Priority information for Motion Design Scenes"));
#endif
}

void UAvaCameraPriorityModifier::Apply()
{
	// Nothing to apply. Continue onto next
	Next();
}

#undef LOCTEXT_NAMESPACE
