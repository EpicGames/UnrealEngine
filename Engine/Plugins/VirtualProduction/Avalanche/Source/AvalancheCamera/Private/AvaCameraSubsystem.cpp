// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaCameraSubsystem.h"
#include "AvaCameraPriorityModifier.h"
#include "AvaTransitionSubsystem.h"
#include "AvaTransitionTree.h"
#include "Behavior/AvaTransitionBehaviorInstance.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "Camera/CameraComponent.h"
#include "Camera/PlayerCameraManager.h"
#include "Engine/Level.h"
#include "Execution/IAvaTransitionExecutor.h"
#include "GameFramework/PlayerController.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "Transition/Tasks/AvaCameraBlendTask.h"

static TAutoConsoleVariable<bool> CVarEnableMotionDesignCameraSubsystem(
	TEXT("MotionDesignCamera.EnableCameraSubsystem"),
	true,	
	TEXT("Enable/Disable Motion Design's camera subsystem."),
	ECVF_Default);

bool FAvaViewTarget::IsValid() const
{
	return ::IsValid(Actor);
}

int32 FAvaViewTarget::GetPriority() const
{
	if (::IsValid(CameraPriorityModifier))
	{
		return CameraPriorityModifier->GetPriority();
	}
	return 0;
}

const FViewTargetTransitionParams& FAvaViewTarget::GetTransitionParams() const
{
	if (::IsValid(CameraPriorityModifier))
	{
		return CameraPriorityModifier->GetTransitionParams();
	}

	static const FViewTargetTransitionParams DefaultTransitionParams;
	return DefaultTransitionParams;
}

UAvaCameraSubsystem* UAvaCameraSubsystem::Get(const UObject* InObject)
{
	if (!InObject)
	{
		return nullptr;
	}

	const UWorld* World = InObject->GetWorld();
	if (!World)
	{
		return nullptr;
	}

	return World->GetSubsystem<UAvaCameraSubsystem>();
}

void UAvaCameraSubsystem::RegisterScene(const ULevel* InSceneLevel)
{
	if (!InSceneLevel)
	{
		return;
	}

	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	if (!ModifierSubsystem)
	{
		return;
	}

	for (AActor* Actor : InSceneLevel->Actors)
	{
		if (!Actor)
		{
			continue;
		}

		const UActorModifierCoreStack* ModifierStack = ModifierSubsystem->GetActorModifierStack(Actor);

		const UAvaCameraPriorityModifier* CameraPriorityModifier = ModifierStack
			? ModifierStack->GetClassModifier<UAvaCameraPriorityModifier>()
			: nullptr;

		if (CameraPriorityModifier)
		{
			FAvaViewTarget& Entry = ViewTargets.AddDefaulted_GetRef();
			Entry.Actor = Actor;
			Entry.CameraPriorityModifier = CameraPriorityModifier;
		}
		// Fallback case where the Actor has a Camera Component but does not have a Priority Modifier
		else if (Actor->FindComponentByClass<UCameraComponent>())
		{
			FAvaViewTarget& Entry = ViewTargets.AddDefaulted_GetRef();
			Entry.Actor = Actor;
			Entry.CameraPriorityModifier = nullptr;
		}
	}

	ConditionallyUpdateViewTarget(InSceneLevel);
}

void UAvaCameraSubsystem::UnregisterScene(const ULevel* InSceneLevel)
{
	if (!InSceneLevel)
	{
		return;
	}

	// Remove entries with invalid view targets or targets that match the level that is being unregistered
	ViewTargets.RemoveAll(
		[InSceneLevel](const FAvaViewTarget& InViewTarget)
		{
			return !InViewTarget.IsValid() || InViewTarget.Actor->GetLevel() == InSceneLevel;
		});

	ConditionallyUpdateViewTarget(InSceneLevel);
}

bool UAvaCameraSubsystem::IsBlendingToViewTarget(const ULevel* InSceneLevel) const
{
	if (InSceneLevel && PlayerController && PlayerController->PlayerCameraManager)
	{
		return PlayerController->PlayerCameraManager->BlendTimeToGo > 0.f
			&& PlayerController->PlayerCameraManager->ViewTarget.Target
			&& PlayerController->PlayerCameraManager->ViewTarget.Target->GetLevel() == InSceneLevel;
	}

	return false;
}

void UAvaCameraSubsystem::UpdatePlayerControllerViewTarget(const FViewTargetTransitionParams* InOverrideTransitionParams)
{
	if (!IsValid(PlayerController) || ViewTargets.IsEmpty() || !CVarEnableMotionDesignCameraSubsystem.GetValueOnGameThread())
	{
		return;
	}

	// Remove invalid entries
	ViewTargets.RemoveAll([](const FAvaViewTarget& InViewTarget)
		{
			return !InViewTarget.IsValid();
		});

	// Sort so that the higher priorities are at the end of the list (end = current)
	// Stable sorting so that the more recently registered scenes are preferred when the priorities match
	ViewTargets.StableSort(
		[](const FAvaViewTarget& A, const FAvaViewTarget& B)
		{
			return A.GetPriority() < B.GetPriority();
		});

	const FAvaViewTarget& DesiredViewTarget = ViewTargets.Last();

	// These should've been removed prior to calling this private function
	check(DesiredViewTarget.Actor);

	// Current View Target matches the desired. Nothing to do
	if (DesiredViewTarget.Actor == PlayerController->GetViewTarget())
	{
		return;
	}

	const FViewTargetTransitionParams* ViewTargetTransitionParams = InOverrideTransitionParams
		? InOverrideTransitionParams
		: &DesiredViewTarget.GetTransitionParams();

	PlayerController->SetViewTarget(DesiredViewTarget.Actor, *ViewTargetTransitionParams);
}

bool UAvaCameraSubsystem::ConditionallyUpdateViewTarget(const ULevel* InSceneLevel)
{
	if (!HasCustomViewTargetting(InSceneLevel))
	{
		UpdatePlayerControllerViewTarget();
		return true;
	}
	return false;
}

bool UAvaCameraSubsystem::DoesSupportWorldType(const EWorldType::Type InWorldType) const
{
	return InWorldType == EWorldType::GamePreview
		|| InWorldType == EWorldType::Game
		|| InWorldType == EWorldType::PIE;
}

void UAvaCameraSubsystem::OnWorldBeginPlay(UWorld& InWorld)
{
	Super::OnWorldBeginPlay(InWorld);
	PlayerController = InWorld.GetFirstPlayerController();
}

void UAvaCameraSubsystem::Initialize(FSubsystemCollectionBase& InCollection)
{
	Super::Initialize(InCollection);
	IAvaTransitionExecutor::GetOnTransitionStart().AddUObject(this, &UAvaCameraSubsystem::OnTransitionStart);
}

void UAvaCameraSubsystem::Deinitialize()
{
	Super::Deinitialize();
	IAvaTransitionExecutor::GetOnTransitionStart().RemoveAll(this);
}

void UAvaCameraSubsystem::OnTransitionStart(const IAvaTransitionExecutor& InExecutor)
{
	// Unregister scenes that are marked as Needing Discard from the Start
	InExecutor.ForEachBehaviorInstance(
		[this](const FAvaTransitionBehaviorInstance& InInstance)
		{
			const FAvaTransitionScene* TransitionScene = InInstance.GetTransitionContext().GetTransitionScene();
			if (!TransitionScene || !TransitionScene->HasAnyFlags(EAvaTransitionSceneFlags::NeedsDiscard))
			{
				return;
			}

			const ULevel* Level = TransitionScene->GetLevel();
			if (!Level)
			{
				return;
			}

			UnregisterScene(Level);
		});
}

bool UAvaCameraSubsystem::HasCustomViewTargetting(const ULevel* InSceneLevel) const
{
	if (!InSceneLevel)
	{
		return false;
	}

	UAvaTransitionSubsystem* TransitionSubsystem = InSceneLevel->OwningWorld->GetSubsystem<UAvaTransitionSubsystem>();
	if (!TransitionSubsystem)
	{
		return false;
	}

	IAvaTransitionBehavior* TransitionBehavior = TransitionSubsystem->GetTransitionBehavior(InSceneLevel);
	if (!TransitionBehavior)
	{
		return false;
	}

	UAvaTransitionTree* TransitionTree = TransitionBehavior->GetTransitionTree();
	if (!TransitionTree)
	{
		return false;
	}

	return TransitionTree->ContainsTask<FAvaCameraBlendTask>();
}
