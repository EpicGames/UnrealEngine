// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionCameraLibrary.h"
#include "AvaCameraSubsystem.h"
#include "AvaTransitionContext.h"
#include "Behavior/AvaTransitionBehaviorInstanceCache.h"
#include "Engine/Level.h"
#include "IAvaTransitionNodeInterface.h"

bool UAvaTransitionCameraLibrary::ConditionallyUpdateViewTarget(UObject* InTransitionNode)
{
	IAvaTransitionNodeInterface* NodeInterface = Cast<IAvaTransitionNodeInterface>(InTransitionNode);
	if (!NodeInterface)
	{
		return false;
	}

	const FAvaTransitionContext* TransitionContext = NodeInterface->GetBehaviorInstanceCache().GetTransitionContext();
	if (!TransitionContext)
	{
		return false;
	}

	const FAvaTransitionScene* TransitionScene = TransitionContext->GetTransitionScene();
	if (!TransitionScene)
	{
		return false;
	}

	const ULevel* SceneLevel = TransitionScene->GetLevel();

	UAvaCameraSubsystem* CameraSubsystem = UAvaCameraSubsystem::Get(SceneLevel);
	if (!CameraSubsystem)
	{
		return false;
	}

	return CameraSubsystem->ConditionallyUpdateViewTarget(SceneLevel);
}
