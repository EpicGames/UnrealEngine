// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "AvaTransitionCameraLibrary.generated.h"

UCLASS()
class UAvaTransitionCameraLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Transition Logic|Camera", meta=(DefaultToSelf="InTransitionNode"))
	static bool ConditionallyUpdateViewTarget(UObject* InTransitionNode);
};
