// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transition/Tasks/AvaCameraBlendTask.h"
#include "AvaCameraSubsystem.h"
#include "AvaTransitionContext.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeLinker.h"

FAvaCameraBlendInstanceData::FAvaCameraBlendInstanceData()
{
	TransitionParams.bLockOutgoing = true;
}

bool FAvaCameraBlendTask::Link(FStateTreeLinker& InLinker)
{
	Super::Link(InLinker);
	InLinker.LinkExternalData(CameraSubsystemHandle);
	return true;
}

EStateTreeRunStatus FAvaCameraBlendTask::EnterState(FStateTreeExecutionContext& InContext, const FStateTreeTransitionResult& InTransition) const
{
	const ULevel* TransitionLevel = GetTransitionLevel(InContext);
	if (!TransitionLevel)
	{
		return EStateTreeRunStatus::Failed;
	}

	const FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);

	UAvaCameraSubsystem& CameraSubsystem = InContext.GetExternalData(CameraSubsystemHandle);

	const FViewTargetTransitionParams* TransitionParams = nullptr;
	if (InstanceData.bOverrideTransitionParams)
	{
		TransitionParams = &InstanceData.TransitionParams;
	}

	CameraSubsystem.UpdatePlayerControllerViewTarget(TransitionParams);

	return CameraSubsystem.IsBlendingToViewTarget(TransitionLevel)
		? EStateTreeRunStatus::Running
		: EStateTreeRunStatus::Succeeded;
}

EStateTreeRunStatus FAvaCameraBlendTask::Tick(FStateTreeExecutionContext& InContext, const float InDeltaTime) const
{
	const ULevel* TransitionLevel = GetTransitionLevel(InContext);
	if (!TransitionLevel)
	{
		return EStateTreeRunStatus::Failed;
	}

	const UAvaCameraSubsystem& CameraSubsystem = InContext.GetExternalData(CameraSubsystemHandle);

	return CameraSubsystem.IsBlendingToViewTarget(TransitionLevel)
		? EStateTreeRunStatus::Succeeded
		: EStateTreeRunStatus::Running;
}

const ULevel* FAvaCameraBlendTask::GetTransitionLevel(FStateTreeExecutionContext& InContext) const
{
	const FAvaTransitionContext& TransitionContext = InContext.GetExternalData(TransitionContextHandle);

	if (const FAvaTransitionScene* TransitionScene = TransitionContext.GetTransitionScene())
	{
		return TransitionScene->GetLevel();
	}

	return nullptr;
}
