// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/PlayerCameraManager.h"
#include "Modifiers/ActorModifierCoreBase.h"
#include "AvaCameraPriorityModifier.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UAvaCameraPriorityModifier : public UActorModifierCoreBase
{
	GENERATED_BODY()

public:
	UAvaCameraPriorityModifier();

	int32 GetPriority() const
	{
		return Priority;
	}

	const FViewTargetTransitionParams& GetTransitionParams() const
	{
		return TransitionParams;
	}

protected:
	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	UPROPERTY(EditInstanceOnly, Category="Camera", meta=(ClampMin="0", UIMin="0", AllowPrivateAccess="true"))
	int32 Priority = 0;

	/** The parameters to blend to the view target */
	UPROPERTY(EditInstanceOnly, Category="Camera", meta=(AllowPrivateAccess="true"))
	FViewTargetTransitionParams TransitionParams;
};
