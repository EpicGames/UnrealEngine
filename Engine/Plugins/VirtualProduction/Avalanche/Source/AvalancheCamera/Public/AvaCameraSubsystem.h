// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "AvaCameraSubsystem.generated.h"

class AActor;
class APlayerController;
class IAvaTransitionExecutor;
class UAvaCameraPriorityModifier;
struct FViewTargetTransitionParams;
class ULevel;

USTRUCT()
struct FAvaViewTarget
{
	GENERATED_BODY()

	/** returns true if the View Target Actor is valid */
	bool IsValid() const;

	/** Returns the priority of the camera priority modifier of the actor, or 0 if the priority modifier doesn't exist */
	int32 GetPriority() const;

	/** Returns the Transition Params of the camera priority modifier of the actor, or default transition params if the modifier doesn't exist */
	const FViewTargetTransitionParams& GetTransitionParams() const;

	/** The View Target Actor */
	UPROPERTY()
	TObjectPtr<AActor> Actor;

	/** The Camera Modifier that the View Target Actor has, if any */
	UPROPERTY()
	TObjectPtr<const UAvaCameraPriorityModifier> CameraPriorityModifier;
};

UCLASS(MinimalAPI, DisplayName="Motion Design Camera Subsystem")
class UAvaCameraSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	AVALANCHECAMERA_API static UAvaCameraSubsystem* Get(const UObject* InObject);

	AVALANCHECAMERA_API void RegisterScene(const ULevel* InSceneLevel);
	AVALANCHECAMERA_API void UnregisterScene(const ULevel* InSceneLevel);

	bool IsBlendingToViewTarget(const ULevel* InSceneLevel) const;

	void UpdatePlayerControllerViewTarget(const FViewTargetTransitionParams* InOverrideTransitionParams = nullptr);

	AVALANCHECAMERA_API bool ConditionallyUpdateViewTarget(const ULevel* InSceneLevel);

protected:
	//~ Begin UWorldSubsystem
	virtual bool DoesSupportWorldType(const EWorldType::Type InWorldType) const override;
	virtual void OnWorldBeginPlay(UWorld& InWorld) override;
	//~ End UWorldSubsystem

	//~ Begin USubsystem
	virtual void Initialize(FSubsystemCollectionBase& InCollection) override;
	virtual void Deinitialize() override;
	//~ End USubsystem

private:
	void OnTransitionStart(const IAvaTransitionExecutor& InExecutor);

	bool HasCustomViewTargetting(const ULevel* InSceneLevel) const;

	UPROPERTY()
	TObjectPtr<APlayerController> PlayerController;

	UPROPERTY()
	TArray<FAvaViewTarget> ViewTargets;
};
