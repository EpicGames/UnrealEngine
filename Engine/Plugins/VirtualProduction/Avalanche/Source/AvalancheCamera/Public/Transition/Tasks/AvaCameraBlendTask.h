// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/PlayerCameraManager.h"
#include "Tasks/AvaTransitionTask.h"
#include "AvaCameraBlendTask.generated.h"

class UAvaCameraSubsystem;

USTRUCT()
struct FAvaCameraBlendInstanceData
{
	GENERATED_BODY()

	FAvaCameraBlendInstanceData();

	UPROPERTY(EditAnywhere, Category="Camera")
	bool bOverrideTransitionParams = false;

	UPROPERTY(EditAnywhere, Category="Camera", meta=(EditCondition="bOverrideTransitionParams"))
	FViewTargetTransitionParams TransitionParams;
};

USTRUCT(DisplayName="Blend Scene Camera", Category="Scene Camera")
struct FAvaCameraBlendTask : public FAvaTransitionTask
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaCameraBlendInstanceData;

	//~ Begin FStateTreeNodeBase
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	virtual bool Link(FStateTreeLinker& InLinker) override;
	//~ End FStateTreeNodeBase

	//~ Begin FStateTreeTaskBase
	AVALANCHECAMERA_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& InContext, const FStateTreeTransitionResult& InTransition) const override;
	AVALANCHECAMERA_API virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& InContext, const float InDeltaTime) const override;
	//~ End FStateTreeTaskBase

protected:
	const ULevel* GetTransitionLevel(FStateTreeExecutionContext& InContext) const;

	TStateTreeExternalDataHandle<UAvaCameraSubsystem> CameraSubsystemHandle;
};
