// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheComponentVisualizers : ModuleRules
{
	public AvalancheComponentVisualizers(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"ComponentVisualizers",
				"Core",
				"CoreUObject",
				"Engine",
				"SlateCore"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Avalanche",
				"AvalancheViewport",
				"CustomDetailsView",
				"GeometryFramework",
				"DeveloperSettings",
				"EditorFramework",
				"Slate",
				"UnrealEd"
			}
		);

		ShortName = "AvCompViz";
	}
}
