// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaComponentVisualizersEdMode.h"
#include "AvaViewportUtils.h"
#include "AvaVisBase.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "LevelEditorViewport.h"

#define LOCTEXT_NAMESPACE "AvaComponentVisualizersEdMode"

namespace UE::AvaComponentVisualizers::Private
{
	struct FAvaViewportClientAndVisualizer
	{
		FAvaViewportClientAndVisualizer(const UAvaComponentVisualizersEdMode* InEdMode)
		{
			ActiveVisualizer = UAvaComponentVisualizersEdMode::GetActiveVisualizer();

			if (const FEditorModeTools* ModeManager = InEdMode->GetModeManager())
			{
				EditorViewportClient = ModeManager->GetFocusedViewportClient();
			}
		}

		FEditorViewportClient* EditorViewportClient = nullptr;
		TSharedPtr<FAvaVisualizerBase> ActiveVisualizer = nullptr;
	};
}

TWeakPtr<FAvaVisualizerBase> UAvaComponentVisualizersEdMode::ActiveVisualizerWeak;

const FEditorModeID UAvaComponentVisualizersEdMode::EM_AvaComponentVisualizersEdModeId = TEXT("EM_AvaComponentVisualizersEdModeId");

TSharedPtr<FAvaVisualizerBase> UAvaComponentVisualizersEdMode::GetActiveVisualizer()
{
	return ActiveVisualizerWeak.Pin();
}

void UAvaComponentVisualizersEdMode::OnVisualizerActivated(TSharedPtr<FAvaVisualizerBase> InVisualizer)
{
	ActiveVisualizerWeak = InVisualizer;

	if (InVisualizer.IsValid())
	{
		if (FEditorViewportClient* ViewportClient = InVisualizer->GetLastUsedViewportClient())
		{
			if (FEditorModeTools* ModeTools = ViewportClient->GetModeTools())
			{
				ModeTools->ActivateMode(UAvaComponentVisualizersEdMode::EM_AvaComponentVisualizersEdModeId);
			}
		}
	}
}

void UAvaComponentVisualizersEdMode::OnVisualizerDeactivated(TSharedPtr<FAvaVisualizerBase> InVisualizer)
{
	ActiveVisualizerWeak = nullptr;

	if (InVisualizer.IsValid())
	{
		if (FEditorViewportClient* ViewportClient = InVisualizer->GetLastUsedViewportClient())
		{
			// The viewport client can become invalid.
			if (FAvaViewportUtils::GetAsEditorViewportClient(ViewportClient->Viewport))
			{
				if (FEditorModeTools* ModeTools = ViewportClient->GetModeTools())
				{
					ModeTools->DeactivateMode(UAvaComponentVisualizersEdMode::EM_AvaComponentVisualizersEdModeId);
				}
			}
		}
	}
}

UAvaComponentVisualizersEdMode::UAvaComponentVisualizersEdMode()
{
	Info = FEditorModeInfo(UAvaComponentVisualizersEdMode::EM_AvaComponentVisualizersEdModeId,
		LOCTEXT("AvaComponentVisualizersEdModeName", "Motion Design Visualizer"),
		FSlateIcon(),
		false);
}

namespace UE::AvaComponentVisualizers::Private
{
	const TArray<UE::Widget::EWidgetMode> WidgetModes = {
			UE::Widget::WM_Translate,
			UE::Widget::WM_Rotate,
			UE::Widget::WM_Scale
	};
}

void UAvaComponentVisualizersEdMode::ModeTick(float DeltaTime)
{
	Super::ModeTick(DeltaTime);

	UE::AvaComponentVisualizers::Private::FAvaViewportClientAndVisualizer VisualizerAndClient(this);

	if (VisualizerAndClient.ActiveVisualizer.IsValid())
	{
		if (VisualizerAndClient.EditorViewportClient && !VisualizerAndClient.EditorViewportClient->IsTracking()
			&& VisualizerAndClient.ActiveVisualizer->IsTracking())
		{
			VisualizerAndClient.ActiveVisualizer->TrackingStopped(VisualizerAndClient.EditorViewportClient, /* bDidMove */ true);
		}

		FEditorModeTools* ModeManager = GetModeManager();
		const UE::Widget::EWidgetMode CurrentModeStart = ModeManager->GetWidgetMode();
		UE::Widget::EWidgetMode CurrentModeModified = ModeManager->GetWidgetMode();

		if (VisualizerAndClient.ActiveVisualizer->GetWidgetMode(VisualizerAndClient.EditorViewportClient, CurrentModeModified))
		{
			if (CurrentModeStart == CurrentModeModified)
			{
				return;
			}
		}
			
		for (UE::Widget::EWidgetMode WidgetMode : UE::AvaComponentVisualizers::Private::WidgetModes)
		{
			if (WidgetMode == CurrentModeStart)
			{
				continue;
			}

			CurrentModeModified = WidgetMode;

			if (VisualizerAndClient.ActiveVisualizer->GetWidgetMode(VisualizerAndClient.EditorViewportClient, CurrentModeModified))
			{
				if (WidgetMode == CurrentModeModified)
				{
					ModeManager->SetWidgetMode(WidgetMode);
					return;
				}
			}
		}
	}
}

void UAvaComponentVisualizersEdMode::Exit()
{
	Super::Exit();

	ActiveVisualizerWeak.Reset();
}

EAxisList::Type UAvaComponentVisualizersEdMode::GetWidgetAxisToDraw(UE::Widget::EWidgetMode InWidgetMode) const
{
	UE::AvaComponentVisualizers::Private::FAvaViewportClientAndVisualizer VCnV(this);

	EAxisList::Type AxisList = EAxisList::XYZ;

	if (VCnV.EditorViewportClient && VCnV.ActiveVisualizer.IsValid())
	{
		VCnV.ActiveVisualizer->GetWidgetAxisList(VCnV.EditorViewportClient, InWidgetMode, AxisList);
	}

	return AxisList;
}

FVector UAvaComponentVisualizersEdMode::GetWidgetLocation() const
{
	UE::AvaComponentVisualizers::Private::FAvaViewportClientAndVisualizer VCnV(this);

	FVector WidgetLocation = FVector::ZeroVector;

	if (VCnV.ActiveVisualizer)
	{
		VCnV.ActiveVisualizer->GetWidgetLocation(VCnV.EditorViewportClient, WidgetLocation);
	}

	return WidgetLocation;
}

bool UAvaComponentVisualizersEdMode::UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const
{
	UE::AvaComponentVisualizers::Private::FAvaViewportClientAndVisualizer VCnV(this);

	if (VCnV.ActiveVisualizer)
	{
		UE::Widget::EWidgetMode ToCheck = CheckMode;

		if (VCnV.ActiveVisualizer->GetWidgetMode(VCnV.EditorViewportClient, CheckMode))
		{
			// Will mutate if it isn't available
			return CheckMode == ToCheck;
		}
	}

	return true;
}

FVector UAvaComponentVisualizersEdMode::GetWidgetNormalFromCurrentAxis(void* InData)
{
	UE::AvaComponentVisualizers::Private::FAvaViewportClientAndVisualizer VCnV(this);

	FVector WidgetNormal = FVector::ZeroVector;
	UE::Widget::EWidgetMode WidgetMode;

	if (VCnV.ActiveVisualizer && VCnV.ActiveVisualizer->GetWidgetMode(VCnV.EditorViewportClient, WidgetMode))
	{
		EAxisList::Type AxisList = EAxisList::XYZ;
		bool bValidAxisList = false;

		if (VCnV.EditorViewportClient && VCnV.EditorViewportClient->bWidgetAxisControlledByDrag)
		{
			bValidAxisList = VCnV.ActiveVisualizer->GetWidgetAxisListDragOverride(VCnV.EditorViewportClient, WidgetMode, AxisList);
		}

		if (!bValidAxisList)
		{
			bValidAxisList = VCnV.ActiveVisualizer->GetWidgetAxisList(VCnV.EditorViewportClient, WidgetMode, AxisList);
		}

		if (bValidAxisList)
		{
			FMatrix WidgetMatrix;

			if (VCnV.ActiveVisualizer->GetCustomInputCoordinateSystem(VCnV.EditorViewportClient, WidgetMatrix))
			{
				FVector BaseNormal(1, 0, 0);		// Default to X axis

				switch (AxisList)
				{
					case EAxisList::Y:   BaseNormal = FVector(0, 1, 0); break;
					case EAxisList::Z:   BaseNormal = FVector(0, 0, 1); break;
					case EAxisList::XY:  BaseNormal = FVector(1, 1, 0); break;
					case EAxisList::XZ:  BaseNormal = FVector(1, 0, 1); break;
					case EAxisList::YZ:  BaseNormal = FVector(0, 1, 1); break;
					case EAxisList::XYZ: BaseNormal = FVector(1, 1, 1); break;
				}

				// Transform the base normal into the proper coordinate space.
				return WidgetMatrix.TransformVector(BaseNormal);
			}
		}
	}

	return WidgetNormal;
}

void UAvaComponentVisualizersEdMode::SetCurrentWidgetAxis(EAxisList::Type InAxis)
{
	// Does nothing because it's all specified by the visualizer.
}

EAxisList::Type UAvaComponentVisualizersEdMode::GetCurrentWidgetAxis() const
{
	UE::AvaComponentVisualizers::Private::FAvaViewportClientAndVisualizer VCnV(this);

	EAxisList::Type AxisList = EAxisList::XYZ;

	if (VCnV.ActiveVisualizer)
	{
		UE::Widget::EWidgetMode WidgetMode;

		if (VCnV.ActiveVisualizer->GetWidgetMode(VCnV.EditorViewportClient, WidgetMode))
		{
			bool bValidAxisList = false;

			if (VCnV.EditorViewportClient && VCnV.EditorViewportClient->bWidgetAxisControlledByDrag)
			{
				bValidAxisList = VCnV.ActiveVisualizer->GetWidgetAxisListDragOverride(VCnV.EditorViewportClient, WidgetMode, AxisList);
			}

			if (!bValidAxisList)
			{
				bValidAxisList = VCnV.ActiveVisualizer->GetWidgetAxisList(VCnV.EditorViewportClient, WidgetMode, AxisList);
			}
		}
	}

	return AxisList;
}

bool UAvaComponentVisualizersEdMode::UsesPropertyWidgets() const
{
	return true;
}

bool UAvaComponentVisualizersEdMode::GetCustomDrawingCoordinateSystem(FMatrix& OutMatrix, void* InData)
{
	UE::AvaComponentVisualizers::Private::FAvaViewportClientAndVisualizer VCnV(this);

	return VCnV.ActiveVisualizer && VCnV.ActiveVisualizer->GetCustomInputCoordinateSystem(VCnV.EditorViewportClient, OutMatrix);
}

bool UAvaComponentVisualizersEdMode::GetCustomInputCoordinateSystem(FMatrix& OutMatrix, void* InData)
{
	UE::AvaComponentVisualizers::Private::FAvaViewportClientAndVisualizer VCnV(this);

	return VCnV.ActiveVisualizer && VCnV.ActiveVisualizer->GetCustomInputCoordinateSystem(VCnV.EditorViewportClient, OutMatrix);
}

#undef LOCTEXT_NAMESPACE
