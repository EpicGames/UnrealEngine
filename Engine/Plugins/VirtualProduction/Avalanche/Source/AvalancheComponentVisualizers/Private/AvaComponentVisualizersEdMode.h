// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/LegacyEdModeInterfaces.h"
#include "Tools/UEdMode.h"
#include "AvaComponentVisualizersEdMode.generated.h"

class FAvaVisualizerBase;

UCLASS()
class UAvaComponentVisualizersEdMode : public UEdMode, public ILegacyEdModeWidgetInterface
{
	GENERATED_BODY()

public:
	const static FEditorModeID EM_AvaComponentVisualizersEdModeId;

	static TSharedPtr<FAvaVisualizerBase> GetActiveVisualizer();
	static void OnVisualizerActivated(TSharedPtr<FAvaVisualizerBase> InVisualizer);
	static void OnVisualizerDeactivated(TSharedPtr<FAvaVisualizerBase> InVisualizer);

	UAvaComponentVisualizersEdMode();
	virtual ~UAvaComponentVisualizersEdMode() override = default;

	//~ Begin UEdMode
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override { return true; }
	virtual bool UsesToolkits() const { return false; }
	virtual void ModeTick(float DeltaTime) override;
	virtual void Exit() override;
	//~ End UEdMode

	//~ Begin ILegacyEdModeWidgetInterface
	virtual bool AllowWidgetMove() override { return true; }
	virtual bool CanCycleWidgetMode() const override { return true; }
	virtual bool ShowModeWidgets() const override { return false; }
	virtual EAxisList::Type GetWidgetAxisToDraw(UE::Widget::EWidgetMode InWidgetMode) const override;
	virtual FVector GetWidgetLocation() const override;
	virtual bool ShouldDrawWidget() const override { return true; }
	virtual bool UsesTransformWidget() const override { return true; }
	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const override;
	virtual FVector GetWidgetNormalFromCurrentAxis(void* InData) override;
	virtual void SetCurrentWidgetAxis(EAxisList::Type InAxis) override;
	virtual EAxisList::Type GetCurrentWidgetAxis() const override;
	virtual bool UsesPropertyWidgets() const override;
	virtual bool GetCustomDrawingCoordinateSystem(FMatrix& OutMatrix, void* InData) override;
	virtual bool GetCustomInputCoordinateSystem(FMatrix& OutMatrix, void* InData) override;
	//~ End ILegacyEdModeWidgetInterface

protected:
	static TWeakPtr<FAvaVisualizerBase> ActiveVisualizerWeak;
};
