// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaComponentVisualizersSettings.h"
#include "Engine/Texture2D.h"
#include "UObject/UObjectGlobals.h"

UAvaComponentVisualizersSettings* UAvaComponentVisualizersSettings::Get()
{
	return GetMutableDefault<UAvaComponentVisualizersSettings>();
}

UAvaComponentVisualizersSettings::UAvaComponentVisualizersSettings()
{
	CategoryName = TEXT("Motion Design");
	SectionName = TEXT("Visualizers");

	SpriteSize = 10.f;
}

float UAvaComponentVisualizersSettings::GetSpriteSize() const
{
	return SpriteSize;
}

void UAvaComponentVisualizersSettings::SetSpriteSize(float InSpriteSize)
{
	SpriteSize = InSpriteSize;
}

UTexture2D* UAvaComponentVisualizersSettings::GetVisualizerSprite(FName InName) const
{
	if (const TSoftObjectPtr<UTexture2D>* SpritePtr = VisualizerSprites.Find(InName))
	{
		return SpritePtr->LoadSynchronous();
	}

	return nullptr;
}

void UAvaComponentVisualizersSettings::SetVisualizerSprite(FName InName, UTexture2D* InTexture)
{
	if (InName.IsNone() || !IsValid(InTexture))
	{
		return;
	}

	VisualizerSprites.FindOrAdd(InName) = TSoftObjectPtr<UTexture2D>(InTexture);
}

void UAvaComponentVisualizersSettings::SetDefaultVisualizerSprite(FName InName, UTexture2D* InTexture)
{
	if (VisualizerSprites.Contains(InName))
	{
		return;
	}

	SetVisualizerSprite(InName, InTexture);
}

void UAvaComponentVisualizersSettings::SaveSettings()
{
	SaveConfig();
}
