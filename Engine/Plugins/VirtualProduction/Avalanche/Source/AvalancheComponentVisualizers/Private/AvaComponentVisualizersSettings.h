// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "IAvaComponentVisualizersSettings.h"
#include "AvaComponentVisualizersSettings.generated.h"

UCLASS(Config = EditorPerProjectUserSettings, meta = (DisplayName = "Visualizers"))
class UAvaComponentVisualizersSettings : public UDeveloperSettings, public IAvaComponentVisualizersSettings
{
	GENERATED_BODY()

public:
	static UAvaComponentVisualizersSettings* Get();

	UAvaComponentVisualizersSettings();

	/**
	 * The ideal size for sprites when at the default camera range. Beyond which they will start to become
	 * smaller. This is a guide size and icons may be intentionally bigger or smaller based on their role.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Motion Design")
	float SpriteSize;

	UPROPERTY(Config, EditAnywhere, Category = "Motion Design")
	TMap<FName, TSoftObjectPtr<UTexture2D>> VisualizerSprites;

	//~ Begin IAvaComponentVisualizersSettings
	virtual float GetSpriteSize() const override;
	virtual void SetSpriteSize(float InSpriteSize) override;
	virtual UTexture2D* GetVisualizerSprite(FName InName) const override;
	virtual void SetVisualizerSprite(FName InName, UTexture2D* InTexture) override;
	virtual void SetDefaultVisualizerSprite(FName InName, UTexture2D* InTexture) override;
	virtual void SaveSettings() override;
	//~ End IAvaComponentVisualizersSettings
};
