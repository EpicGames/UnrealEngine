// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaComponentVisualizersViewportOverlay.h"
#include "AvaViewportSettings.h"
#include "AvaVisBase.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "ClassIconFinder.h"
#include "CustomDetailsViewArgs.h"
#include "CustomDetailsViewModule.h"
#include "Editor/UnrealEdEngine.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "ICustomDetailsView.h"
#include "SLevelViewport.h"
#include "Styling/StyleColors.h"
#include "UnrealEdGlobals.h"
#include "ViewportClient/IAvaViewportClient.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SAvaDraggableBoxOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AvaComponentVisualizersViewportOverlay"

namespace UE::AvaComponentVisualizers::Private
{
	TSharedRef<SWidget> CreateDraggableDetailsWidget(const FSlateBrush* InActorBrush, const FText& InActorName, const TSharedRef<SWidget>& InDetailsView)
	{
		static const FSlateColor BackgroundColor = FSlateColor(EStyleColor::Background).GetSpecifiedColor() * FLinearColor(1.f, 1.f, 1.f, 0.5f);
		static const FSlateColor BorderColor = FSlateColor(EStyleColor::Background);
		static const FSlateRoundedBoxBrush Background(BackgroundColor, 8.f, BorderColor, 1.f);
		static const FSlateFontInfo TitleFont = FCoreStyle::GetDefaultFontStyle("Bold", 12);

		return SNew(SAvaDraggableBoxOverlay)
			.Content()
			[
				SNew(SBorder)
				.BorderImage(&Background)
				.Padding(2.f, 4.f, 2.f, 2.f)
				.HAlign(EHorizontalAlignment::HAlign_Left)
				.VAlign(EVerticalAlignment::VAlign_Bottom)
				[
					SNew(SBox)
					.WidthOverride(250.f)
					.MaxDesiredHeight(150.f)
					[
						SNew(SScrollBox)
						.Orientation(Orient_Vertical)
						+ SScrollBox::Slot()
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.f, 0.f, 0.f, 2.f)
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(8.f, 0.f, 0.f, 4.f)
								.VAlign(EVerticalAlignment::VAlign_Bottom)
								[
									SNew(SImage)
									.Image(InActorBrush)
									.DesiredSizeOverride(FVector2D(16.f))
								]
								+ SHorizontalBox::Slot()
								.FillWidth(1.0f)
								.Padding(8.f, 0.f, 0.f, 0.f)
								.VAlign(EVerticalAlignment::VAlign_Bottom)
								.HAlign(EHorizontalAlignment::HAlign_Left)
								[
									SNew(STextBlock)
									.Font(TitleFont)
									.Text(InActorName)
								]
							]
							+ SVerticalBox::Slot()
							.AutoHeight()
							[
								InDetailsView
							]
						]
					]
				]
			];
	}

	TSharedPtr<SWidget> CreateFullDetailsWidget(const TArray<UObject*>& InSelectedObjects)
	{
		FCustomDetailsViewArgs CustomDetailsViewArgs;
		CustomDetailsViewArgs.bShowCategories = false;
		CustomDetailsViewArgs.CategoryAllowList.Allow("Shape");
		CustomDetailsViewArgs.CategoryAllowList.Allow("Cloner");
		CustomDetailsViewArgs.CategoryAllowList.Allow("Effector");
		CustomDetailsViewArgs.CategoryAllowList.Allow("Text");
		CustomDetailsViewArgs.CategoryAllowList.Allow("Style");
		CustomDetailsViewArgs.CategoryAllowList.Allow("Layout");
		CustomDetailsViewArgs.TableBackgroundOpacity = 0.0f;
		CustomDetailsViewArgs.RowBackgroundOpacity = 0.0f;

		TMap<UClass*, TArray<UObject*>> ClassMap;
		TSet<UClass*> ActorClasses;

		TSharedRef<SVerticalBox> DetailsViews = SNew(SVerticalBox);

		for (UObject* Object : InSelectedObjects)
		{
			if (UActorComponent* Component = Cast<UActorComponent>(Object))
			{
				if (AActor* Parent = Component->GetOwner())
				{
					if (InSelectedObjects.Contains(Parent))
					{
						continue;
					}
				}
			}

			UClass* CurrentClass = Object->GetClass();

			if (CurrentClass->IsChildOf<AActor>())
			{
				ActorClasses.Add(CurrentClass);
			}

			if (ClassMap.Contains(CurrentClass))
			{
				continue;
			}

			TArray<UObject*> ObjectsOfClass;
			ObjectsOfClass.Reserve(InSelectedObjects.Num());

			for (UObject* ObjectIter : InSelectedObjects)
			{
				if (ObjectIter->GetClass() == CurrentClass)
				{
					ObjectsOfClass.Add(ObjectIter);
				}
			}

			ClassMap.Add(CurrentClass, ObjectsOfClass);

			TSharedRef<ICustomDetailsView> DetailsView = ICustomDetailsViewModule::Get().CreateCustomDetailsView(CustomDetailsViewArgs);
			DetailsView->SetObjects(ObjectsOfClass);

			DetailsViews->AddSlot()
				.AutoHeight()
				[
					DetailsView
				];
		}

		const FSlateBrush* ActorBrush = FClassIconFinder::FindIconForActor(Cast<AActor>(AActor::StaticClass()->GetDefaultObject()));
		FText ActorName = LOCTEXT("ActorName", "Multiple Actors");

		if (ActorClasses.Num() == 1)
		{
			UClass* ActorClass = ActorClasses.Array()[0];
			const TArray<UObject*>& ActorList = ClassMap[ActorClass];
			AActor* Actor = Cast<AActor>(ActorList[0]);

			if (const FSlateBrush* SpecificBrush = FClassIconFinder::FindIconForActor(Actor))
			{
				ActorBrush = SpecificBrush;
			}

			if (ActorList.Num() == 1)
			{
				ActorName = FText::FromString(Actor->GetActorNameOrLabel());
			}
		}

		return CreateDraggableDetailsWidget(ActorBrush, ActorName, DetailsViews);
	}

	TSharedPtr<SWidget> CreateComponentVisualizerWidget(const TArray<UObject*>& InSelectedOjects)
	{
		// Visualizers are not recognised by the system until they are activated, rather than just displayed inactively.
		// This aims to provide settings for all visualizers for the selected actor(s) even if they are not active.

		if (!GUnrealEd)
		{
			return nullptr;
		}

		IAvalancheComponentVisualizersModule* VisualizerModule = IAvalancheComponentVisualizersModule::GetIfLoaded();

		if (!VisualizerModule)
		{
			return nullptr;
		}

		TMap<UClass*, TArray<UObject*>> ClassMap;
		TSet<UClass*> ActorClasses;

		for (UObject* Object : InSelectedOjects)
		{
			if (IsValid(Object))
			{
				UClass* ObjectClass = Object->GetClass();
				ClassMap.FindOrAdd(ObjectClass).Add(Object);

				if (ObjectClass->IsChildOf<AActor>())
				{
					ActorClasses.Add(ObjectClass);
				}
			}
		}

		if (ClassMap.IsEmpty())
		{
			return nullptr;
		}

		TMap<UClass*, TSet<UObject*>> ClassObjectMap;
		TMap<UClass*, TSet<FProperty*>> ClassPropertyMap;

		for (const TPair<UClass*, TArray<UObject*>>& ObjectPair : ClassMap)
		{
			if (TSharedPtr<FComponentVisualizer> Visualizer = GUnrealEd->FindComponentVisualizer(ObjectPair.Key))
			{
				if (VisualizerModule->IsAvalancheVisualizer(Visualizer.ToSharedRef()))
				{
					TSharedRef<FAvaVisualizerBase> AvaVis = StaticCastSharedRef<FAvaVisualizerBase>(Visualizer.ToSharedRef());

					for (UObject* Object : ObjectPair.Value)
					{
						const TMap<UObject*, TArray<FProperty*>> PropertyMap = AvaVis->GatherEditableProperties(Object);

						for (const TPair<UObject*, TArray<FProperty*>>& PropertyPair : PropertyMap)
						{
							if (IsValid(PropertyPair.Key))
							{
								UClass* Class = PropertyPair.Key->GetClass();

								ClassObjectMap.FindOrAdd(Class).Add(PropertyPair.Key);
								ClassPropertyMap.FindOrAdd(Class).Append(PropertyPair.Value);
							}
						}
					}
				}
			}
		}

		if (ClassObjectMap.IsEmpty())
		{
			return nullptr;
		}

		TSharedRef<SVerticalBox> DetailsViews = SNew(SVerticalBox);

		for (const TPair<UClass*, TSet<UObject*>>& ObjectPair : ClassObjectMap)
		{
			const TSet<FProperty*>& Properties = ClassPropertyMap[ObjectPair.Key];

			if (Properties.IsEmpty())
			{
				continue;
			}

			FCustomDetailsViewArgs CustomDetailsViewArgs;
			CustomDetailsViewArgs.bShowCategories = false;
			CustomDetailsViewArgs.TableBackgroundOpacity = 0.0f;
			CustomDetailsViewArgs.RowBackgroundOpacity = 0.0f;
			CustomDetailsViewArgs.bAllowResetToDefault = false;
			CustomDetailsViewArgs.bAllowGlobalExtensions = false;

			for (FProperty* Property : Properties)
			{
				CustomDetailsViewArgs.ItemAllowList.Allow(FCustomDetailsViewItemId::MakePropertyId(Property));
			}

			TSharedRef<ICustomDetailsView> DetailsView = ICustomDetailsViewModule::Get().CreateCustomDetailsView(CustomDetailsViewArgs);
			DetailsView->SetObjects(ObjectPair.Value.Array());

			DetailsViews->AddSlot()
				.AutoHeight()
				[
					DetailsView
				];
		}

		const FSlateBrush* ActorBrush = FClassIconFinder::FindIconForActor(Cast<AActor>(AActor::StaticClass()->GetDefaultObject()));
		FText ActorName = LOCTEXT("ActorName", "Multiple Actors");

		if (ActorClasses.Num() == 1)
		{
			UClass* ActorClass = ActorClasses.Array()[0];
			const TArray<UObject*>& ActorList = ClassMap[ActorClass];
			AActor* Actor = Cast<AActor>(ActorList[0]);

			if (const FSlateBrush* SpecificBrush = FClassIconFinder::FindIconForActor(Actor))
			{
				ActorBrush = SpecificBrush;
			}

			if (ActorList.Num() == 1)
			{
				ActorName = FText::FromString(Actor->GetActorNameOrLabel());
			}
		}

		return CreateDraggableDetailsWidget(ActorBrush, ActorName, DetailsViews);
	}
}

void FAvaComponentVisualizersViewportOverlay::AddWidget(const TArray<TSharedPtr<IAvaViewportClient>>& InAvaViewportClients, 
	const TArray<UObject*>& InSelectedObjects)
{
	if (OverlayWidget.IsValid())
	{
		RemoveWidget(InAvaViewportClients);
	}

	const UAvaViewportSettings* ViewportSettings = GetDefault<UAvaViewportSettings>();

	if (!ViewportSettings)
	{
		return;
	}

	using namespace UE::AvaComponentVisualizers::Private;

	switch (ViewportSettings->ShapeEditorOverlayType)
	{
		default:
			return;

		case EAvaShapeEditorOverlayType::ComponentVisualizerOnly:
			OverlayWidget = CreateComponentVisualizerWidget(InSelectedObjects);
			break;

		case EAvaShapeEditorOverlayType::FullDetails:
			OverlayWidget = CreateFullDetailsWidget(InSelectedObjects);
			break;
	}

	if (!OverlayWidget.IsValid())
	{
		return;
	}

	for (TSharedPtr<IAvaViewportClient> AvaViewportClient : InAvaViewportClients)
	{
		if (const FEditorViewportClient* EditorViewportClient = AvaViewportClient->AsEditorViewportClient())
		{
			if (EditorViewportClient->IsLevelEditorClient())
			{
				if (TSharedPtr<SEditorViewport> ViewportWidget = EditorViewportClient->GetEditorViewportWidget())
				{
					StaticCastSharedPtr<SLevelViewport>(ViewportWidget)->AddOverlayWidget(OverlayWidget.ToSharedRef());
				}
			}
		}
	}
}

void FAvaComponentVisualizersViewportOverlay::RemoveWidget(const TArray<TSharedPtr<IAvaViewportClient>>& InAvaViewportClients)
{
	if (!OverlayWidget.IsValid())
	{
		return;
	}

	for (TSharedPtr<IAvaViewportClient> AvaViewportClient : InAvaViewportClients)
	{
		if (const FEditorViewportClient* EditorViewportClient = AvaViewportClient->AsEditorViewportClient())
		{
			if (EditorViewportClient->IsLevelEditorClient())
			{
				if (TSharedPtr<SEditorViewport> ViewportWidget = EditorViewportClient->GetEditorViewportWidget())
				{
					StaticCastSharedPtr<SLevelViewport>(ViewportWidget)->RemoveOverlayWidget(OverlayWidget.ToSharedRef());
				}
			}
		}
	}

	OverlayWidget.Reset();
}

bool FAvaComponentVisualizersViewportOverlay::IsWidgetActive() const
{
	return OverlayWidget.IsValid();
}

#undef LOCTEXT_NAMESPACE
