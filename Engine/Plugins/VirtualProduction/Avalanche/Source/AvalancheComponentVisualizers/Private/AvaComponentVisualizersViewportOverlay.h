// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaComponentVisualizersViewportOverlay.h"
#include "Templates/SharedPointer.h"

class SWidget;

class FAvaComponentVisualizersViewportOverlay : public IAvaComponentVisualizersViewportOverlay
{
public:
	virtual ~FAvaComponentVisualizersViewportOverlay() override = default;

	//~ Begin IAvaComponentVisualizersViewportOverlay
	virtual void AddWidget(const TArray<TSharedPtr<IAvaViewportClient>>& InAvaViewportClients, const TArray<UObject*>& InSelectedObjects) override;
	virtual void RemoveWidget(const TArray<TSharedPtr<IAvaViewportClient>>& InAvaViewportClients) override;
	virtual bool IsWidgetActive() const override;
	//~ End IAvaComponentVisualizersViewportOverlay

protected:
	TSharedPtr<SWidget> OverlayWidget;
};
