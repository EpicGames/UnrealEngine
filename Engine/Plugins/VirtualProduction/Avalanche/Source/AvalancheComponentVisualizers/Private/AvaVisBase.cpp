// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaVisBase.h"
#include "AvaActorUtils.h"
#include "AvaComponentVisualizersEdMode.h"
#include "AvaComponentVisualizersSettings.h"
#include "AvaViewportUtils.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "Interaction/AvaSnapOperation.h"
#include "Math/Box.h"
#include "Misc/CoreMiscDefines.h"
#include "SceneView.h"
#include "Styling/StyleColors.h"
#include "Templates/SharedPointer.h"
#include "UnrealClient.h"
#include "ViewportClient/IAvaViewportClient.h"

#define LOCTEXT_NAMESPACE "AvaVisualizerBase"

IMPLEMENT_HIT_PROXY(HAvaHitProxy, HComponentVisProxy);
IMPLEMENT_HIT_PROXY(HAvaDirectHitProxy, HAvaHitProxy);

FLinearColor FAvaVisualizerBase::Inactive      = FStyleColors::AccentGray.GetSpecifiedColor();
FLinearColor FAvaVisualizerBase::Active        = FStyleColors::AccentBlue.GetSpecifiedColor();
FLinearColor FAvaVisualizerBase::ActiveAltMode = FStyleColors::AccentRed.GetSpecifiedColor();
FLinearColor FAvaVisualizerBase::Enabled       = FStyleColors::AccentBlue.GetSpecifiedColor();
FLinearColor FAvaVisualizerBase::Disabled      = FStyleColors::AccentBlack.GetSpecifiedColor();

const USceneComponent* FAvaVisualizerBase::GetEditedSceneComponent(const UActorComponent* InComponent) const
{
	return Cast<USceneComponent>(InComponent);
}

bool FAvaVisualizerBase::GetCustomInputCoordinateSystem(const FEditorViewportClient* InViewportClient, FMatrix& OutMatrix) const
{
	if (!GetEditedComponent() || !GetEditedComponent()->GetOwner())
	{
		return Super::GetCustomInputCoordinateSystem(InViewportClient, OutMatrix);
	}

	OutMatrix = FRotationMatrix::Make(GetEditedComponent()->GetOwner()->GetActorRotation());
	return true;
}

bool FAvaVisualizerBase::HandleInputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport,
	FVector& InDeltaTranslate, FRotator& InDeltaRotate, FVector& InDeltaScale)
{
	if (!GetEditedComponent() || !GetEditedComponent()->GetOwner())
	{
		EndEditing();
		return false;
	}

	const USceneComponent* RootComponent = GetEditedComponent()->GetOwner()->GetRootComponent();

	if (!RootComponent)
	{
		EndEditing();
		return false;
	}

	if (!bTracking)
	{
		TrackingStartedInternal(InViewportClient);
	}

	FVector RootComponentScale = RootComponent->GetComponentScale();

	if (RootComponentScale != FVector::OneVector)
	{
		const FRotator MeshComponentRotation = RootComponent->GetComponentRotation();
		RootComponentScale = MeshComponentRotation.RotateVector(RootComponentScale);
	}

	FVector WidgetLocation;
	GetWidgetLocation(InViewportClient, WidgetLocation);

	if (!bHasInitialWidgetLocation)
	{
		InitialWidgetLocation = WidgetLocation;
		bHasInitialWidgetLocation = true;
	}

	WidgetLocation -= InitialWidgetLocation;
	WidgetLocation += InDeltaTranslate; // Delta translate is the distance the mouse has moved away from the widget.

	AccumulatedTranslation = GetLocalVector(InViewportClient, WidgetLocation / RootComponentScale);
	AccumulatedRotation += InDeltaRotate;
	AccumulatedScale += InDeltaScale;

	if (HandleInputDeltaInternal(InViewportClient, InViewport, AccumulatedTranslation, AccumulatedRotation, AccumulatedScale))
	{
		return true;
	}

	return Super::HandleInputDelta(InViewportClient, InViewport, InDeltaTranslate, InDeltaRotate, InDeltaScale);
}

bool FAvaVisualizerBase::HandleModifiedClick(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy, const FViewportClick& InClick)
{
	if (InClick.IsAltDown() && !InClick.IsControlDown() && !InClick.IsShiftDown())
	{
		if (ResetValue(InViewportClient, InHitProxy))
		{
			return true;
		}
	}

	return Super::HandleModifiedClick(InViewportClient, InHitProxy, InClick);
}

void FAvaVisualizerBase::StartTransaction()
{
	UActorComponent* EditedComponent = GetEditedComponent();

	if (!EditedComponent)
	{
		return;
	}

	EditedComponent->SetFlags(RF_Transactional);

	const FString OwnerLabel = EditedComponent->GetOwner() ? EditedComponent->GetOwner()->GetActorNameOrLabel() : TEXT("");
	TransactionIdx = GEditor->BeginTransaction(FText::Format(LOCTEXT("VisualizerChange", "Edit {0} Visualizer Change"), FText::FromString(OwnerLabel)));

	EditedComponent->Modify();

	bHasBeenModified = false;
}

void FAvaVisualizerBase::EndTransaction()
{
	if (TransactionIdx != INDEX_NONE)
	{
		if (bHasBeenModified)
		{
			GEditor->EndTransaction();
		}
		else
		{
			GEditor->CancelTransaction(TransactionIdx);
		}
		TransactionIdx = INDEX_NONE;
	}

	bHasBeenModified = false;
}

void FAvaVisualizerBase::StoreInitialValues()
{
	AccumulatedTranslation = FVector::ZeroVector;
	AccumulatedRotation = FRotator::ZeroRotator;
	AccumulatedScale = FVector::ZeroVector;
	bHasInitialWidgetLocation = false;
}

void FAvaVisualizerBase::StartEditing(FEditorViewportClient* InViewportClient, UActorComponent* InEditedComponent)
{
	LastUsedViewportClient = InViewportClient;

	// Disable AutoSelectComponent to keep active actor selected
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Editor.ComponentVisualizer.AutoSelectComponent")))
	{
		bCVarAutoSelectComponent = CVar->GetBool();
		CVar->Set(false);
	}

	UAvaComponentVisualizersEdMode::OnVisualizerActivated(SharedThis(this));
}

void FAvaVisualizerBase::EndEditing()
{
	Super::EndEditing();

	EndTransaction();
	UAvaComponentVisualizersEdMode::OnVisualizerDeactivated(SharedThis(this));

	LastUsedViewportClient = nullptr;

	// Restore AutoSelectComponent state
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Editor.ComponentVisualizer.AutoSelectComponent")))
	{
		CVar->Set(bCVarAutoSelectComponent);
	}
}

void FAvaVisualizerBase::TrackingStarted(FEditorViewportClient* InViewportClient)
{
	TrackingStartedInternal(InViewportClient);
}

void FAvaVisualizerBase::TrackingStopped(FEditorViewportClient* InViewportClient, bool bInDidMove)
{
	TrackingStoppedInternal(InViewportClient);
}

void FAvaVisualizerBase::DrawVisualization(const UActorComponent* InComponent, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI)
{
	Super::DrawVisualization(InComponent, InView, InPDI);

	if (!InComponent)
	{
		return;
	}

	const USceneComponent* SceneComponent = GetEditedSceneComponent(InComponent);

	if (!SceneComponent)
	{
		return;
	}

	if (!CalcIconLocation(SceneComponent, InView))
	{
		return;
	}

	int32 IconIndex = 0;

	if (IsEditing())
	{
		DrawVisualizationEditing(InComponent, InView, InPDI, IconIndex);
	}
	else
	{
		DrawVisualizationNotEditing(InComponent, InView, InPDI, IconIndex);
	}
}

FBox FAvaVisualizerBase::GetComponentBounds(const UActorComponent* InComponent) const
{
	const USceneComponent* SceneComp = Cast<USceneComponent>(InComponent);

	if (!SceneComp || !SceneComp->GetOwner())
	{
		return FBox(ForceInit);
	}

	return FAvaActorUtils::GetActorLocalBoundingBox(SceneComp->GetOwner());
}

FTransform FAvaVisualizerBase::GetComponentTransform(const UActorComponent* InComponent) const
{
	const USceneComponent* SceneComp = Cast<USceneComponent>(InComponent);

	if (!SceneComp || !SceneComp->GetOwner())
	{
		return FTransform::Identity;
	}

	return SceneComp->GetComponentTransform();
}

bool FAvaVisualizerBase::CalcIconLocation(const USceneComponent* InComponent, const FSceneView* InView)
{
	if (!InComponent)
	{
		return false;
	}

	if (InView->UnconstrainedViewRect.Height() == 0 || InView->UnconstrainedViewRect.Width() == 0)
	{
		return false;
	}

	FBox Bounds                     = GetComponentBounds(InComponent);
	const FTransform ShapeTransform = GetComponentTransform(InComponent);
	const FVector2f ScreenSize      = FVector2f(InView->UnconstrainedViewRect.Width(), InView->UnconstrainedViewRect.Height());
	const FVector2f ScreenMiddle    = ScreenSize * 0.5f;

	auto LocalToWorld = [&](const FVector& Point)->FVector
	{
		return ShapeTransform.TransformPositionNoScale(Point);
	};

	auto WorldToScreen = [&](const FVector& Point)->FVector2f
	{
		const FPlane ScreenPlane = InView->Project(Point);
		FVector2f ScreenPosition;
		ScreenPosition.X = ScreenMiddle.X + ScreenMiddle.X * ScreenPlane.X;
		ScreenPosition.Y = ScreenMiddle.Y - ScreenMiddle.Y * ScreenPlane.Y;
		ScreenPosition.X += InView->UnconstrainedViewRect.Min.X;
		ScreenPosition.Y += InView->UnconstrainedViewRect.Min.Y;
		return ScreenPosition;
	};

	// Top left
	const FVector World      = LocalToWorld({Bounds.Min.X, Bounds.Min.Y, Bounds.Max.Z});
	FVector2f ScreenPosition = WorldToScreen(World);

	auto CheckPoint = [&](const FVector& Point)->void
	{
		const FVector2f Camera = WorldToScreen(LocalToWorld(Point));

		if (Camera.X < ScreenPosition.X)
		{
			ScreenPosition.X = Camera.X;
		}

		if (Camera.Y < ScreenPosition.Y)
		{
			ScreenPosition.Y = Camera.Y;
		}
	};

	// Top Right
	CheckPoint({Bounds.Min.X, Bounds.Max.Y, Bounds.Max.Z});

	// Bottom Left
	CheckPoint({Bounds.Min.X, Bounds.Min.Y, Bounds.Min.Z});

	// Bottom Right
	CheckPoint({Bounds.Min.X, Bounds.Max.Y, Bounds.Min.Z});

	if (Bounds.Min.X != Bounds.Max.X)
	{
		CheckPoint({Bounds.Max.X, Bounds.Min.Y, Bounds.Max.Z});
		CheckPoint({Bounds.Max.X, Bounds.Max.Y, Bounds.Max.Z});
		CheckPoint({Bounds.Max.X, Bounds.Min.Y, Bounds.Min.Z});
		CheckPoint({Bounds.Max.X, Bounds.Max.Y, Bounds.Min.Z});
	}

	FVector WorldPosition;
	FVector WorldDirection;
	InView->DeprojectFVector2D(static_cast<FVector2D>(ScreenPosition), WorldPosition, WorldDirection);
	IconStartPosition = WorldPosition + WorldDirection * (InView->ViewLocation - ShapeTransform.GetLocation()).Size();

	return true;
}

float FAvaVisualizerBase::GetIconSizeScale(const FSceneView* InView, const FVector& InWorldPosition)
{
	// This value accounts for perspective.
	static constexpr float SizeMultiplier = 4.f / 9.f;
	static constexpr float Inflection = 500.f;
	static constexpr float InverseInflection = 1.f / Inflection;
	float Distance = Inflection;

	if (InView)
	{
		Distance = (InView->ViewLocation - InWorldPosition).Size();
	}

	static constexpr float DefaultSize = 12.f;
	float Size = DefaultSize;

	if (const UAvaComponentVisualizersSettings* Settings = UAvaComponentVisualizersSettings::Get())
	{
		if (Settings->SpriteSize > 0)
		{
			Size = Settings->SpriteSize;
		}
	}

	return Size * FMath::Min(Distance, Inflection) * InverseInflection * SizeMultiplier;
}

void FAvaVisualizerBase::GetIconMetrics(const FSceneView* InView, int32 InIconIndex, FVector& InOutPosition, float& InOutSize) const
{
	static constexpr float Padding = 1.f;

	InOutPosition = IconStartPosition;
	InOutSize = GetIconSizeScale(InView, InOutPosition);

	FVector ScreenOffset = FVector::ZeroVector;
	ScreenOffset.Z += InOutSize * 4.f;
	ScreenOffset.Z += Padding;
	ScreenOffset.Y += InOutSize;
	ScreenOffset.Y += static_cast<float>(InIconIndex) * InOutSize * 2.f;
	ScreenOffset.Y += static_cast<float>(InIconIndex) * Padding;
	ScreenOffset = InView->ViewRotation.RotateVector(ScreenOffset);
	InOutPosition += ScreenOffset;
}

const FLinearColor& FAvaVisualizerBase::GetIconColor(bool bInActive, bool bInAltMode)
{
	if (!bInActive)
	{
		return Inactive;
	}

	if (!bInAltMode)
	{
		return Active;
	}

	return ActiveAltMode;
}

UE::Widget::EWidgetMode FAvaVisualizerBase::GetViewportWidgetMode(FEditorViewportClient* InViewportClient) const
{
	return InViewportClient->GetWidgetMode();
}

EAxisList::Type FAvaVisualizerBase::GetViewportWidgetAxisList(FEditorViewportClient* InViewportClient) const
{
	EAxisList::Type AxisList = InViewportClient->GetCurrentWidgetAxis();

	GetWidgetAxisListDragOverride(InViewportClient, GetViewportWidgetMode(InViewportClient), AxisList);

	return AxisList;
}

FVector FAvaVisualizerBase::GetLocalVector(FEditorViewportClient* InViewport, const FVector& InVector) const
{
	FVector OutVector = FVector::ZeroVector;
	FMatrix RotMatrix;

	if (GetCustomInputCoordinateSystem(InViewport, RotMatrix))
	{
		OutVector = RotMatrix.Inverse().TransformVector(InVector);
	}
	else if (GetEditedComponent() && GetEditedComponent()->GetOwner())
	{
		OutVector = GetEditedComponent()->GetOwner()->GetActorForwardVector().Rotation().RotateVector(InVector);
	}

	UE::Widget::EWidgetMode WidgetMode = UE::Widget::WM_None;

	GetWidgetMode(InViewport, WidgetMode);

	if (WidgetMode == UE::Widget::WM_None)
	{
		return OutVector;
	}

	EAxisList::Type AxisList = InViewport->GetCurrentWidgetAxis(); // Fallback value

	if (!GetWidgetAxisListDragOverride(InViewport, WidgetMode, AxisList))
	{
		GetWidgetAxisList(InViewport, WidgetMode, AxisList);
	}

	if (!(AxisList & EAxisList::X))
	{
		OutVector.X = 0.f;
	}

	if (!(AxisList & EAxisList::Y))
	{
		OutVector.Y = 0.f;
	}

	if (!(AxisList & EAxisList::Z))
	{
		OutVector.Z = 0.f;
	}

	return OutVector;
}

bool FAvaVisualizerBase::IsMouseOverComponent(const UActorComponent* Component, const FSceneView* InView) const
{
	if (!Component)
	{
		return false;
	}

	if (!InView || InView->CursorPos.X == -1 || InView->CursorPos.Y == -1)
	{
		return false;
	}

	const FBox ActorSizeBox = GetComponentBounds(Component);

	if (!ActorSizeBox.IsValid)
	{
		return false;
	}

	// Cancel scale out since it's not taken into account in ActorSizeBox
	FTransform ComponentTransform = GetComponentTransform(Component);
	ComponentTransform.SetScale3D(FVector::OneVector);
	const FOrientedBox ActorOrientedBox = FAvaActorUtils::MakeOrientedBox(ActorSizeBox, ComponentTransform);

	// Calculate the rotation of the oriented box and revert it to axis aligned, moving the camera position and look direction along with it
	// so that we can use the line/box intersection method.
	const FMatrix InvRotMatrix = FMatrix(
		ActorOrientedBox.AxisX, 
		ActorOrientedBox.AxisY, 
		ActorOrientedBox.AxisZ, 
		FVector::ZeroVector
	).Inverse();

	const FVector2D MouseLocation = {
		static_cast<double>(InView->CursorPos.X),
		static_cast<double>(InView->CursorPos.Y)
	};
	FVector WorldOrigin;
	FVector WorldDirection;

	InView->DeprojectFVector2D(MouseLocation, WorldOrigin, WorldDirection);

	const FVector MouseWorldPosition   = InvRotMatrix.TransformVector(WorldOrigin - ActorOrientedBox.Center);
	const FVector CameraRotationVector = InvRotMatrix.TransformVector((WorldOrigin - InView->ViewLocation).GetUnsafeNormal());
	const FVector Extent               = {ActorOrientedBox.ExtentX, ActorOrientedBox.ExtentY, ActorOrientedBox.ExtentZ};
	const FBox    OriginBounds         = FBox(-Extent, Extent);

	const FVector RayEnd = MouseWorldPosition + CameraRotationVector * 10000.f;

	return FMath::LineBoxIntersection(OriginBounds, MouseWorldPosition, RayEnd, RayEnd - MouseWorldPosition);
}

bool FAvaVisualizerBase::ShouldDrawExtraHandles(const UActorComponent* InComponent, const FSceneView* InView) const
{
	return IsMouseOverComponent(InComponent, InView);
}

void FAvaVisualizerBase::TrackingStartedInternal(FEditorViewportClient* InViewportClient)
{
	if (!GetEditedComponent())
	{
		return;
	}

	if (bTracking)
	{
		return;
	}

	StoreInitialValues();
	StartTransaction();
	bTracking = true;

	if (TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(InViewportClient))
	{
		SnapOperation = AvaViewportClient->StartSnapOperation();

		if (SnapOperation.IsValid())
		{
			SnapOperation->GenerateComponentSnapPoints(GetEditedComponent());
			GenerateContextSensitiveSnapPoints();
			SnapOperation->FinaliseSnapPoints();
		}
	}
}

void FAvaVisualizerBase::TrackingStoppedInternal(FEditorViewportClient* InViewportClient)
{
	EndTransaction();
	AddSnapDataBinding();
	bTracking = false;
	SnapOperation.Reset();
}

void FAvaVisualizerBase::ModifyProperty(UObject* InObject, FProperty* InProperty, EPropertyChangeType::Type InPropertyChangeType, TFunctionRef<void()> InFunction)
{
	if (!InObject)
	{
		return;
	}

	InObject->SetFlags(RF_Transactional);

	bHasBeenModified = InObject->Modify();

	InFunction();

	if (InPropertyChangeType != EPropertyChangeType::Interactive)
	{
		NotifyPropertyModified(InObject, InProperty, InPropertyChangeType);
	}
}

void FAvaVisualizerBase::ModifyProperty(UObject* InObject, FName InPropertyName, EPropertyChangeType::Type InPropertyChangeType, TFunctionRef<void()> InFunction)
{
	if (!InObject)
	{
		return;
	}

	ModifyProperty(InObject, FindFProperty<FProperty>(InObject->GetClass(), InPropertyName), InPropertyChangeType, InFunction);
}

void FAvaVisualizerBase::NotifyPropertyModified(UObject* InObject, FProperty* InProperty,
	EPropertyChangeType::Type InPropertyChangeType, FProperty* InMemberProperty)
{
	TArray<FProperty*> Properties;
	Properties.Add(InProperty);
	NotifyPropertiesModified(InObject, Properties, InPropertyChangeType, InMemberProperty);
}

void FAvaVisualizerBase::NotifyPropertiesModified(UObject* InObject, const TArray<FProperty*>& InProperties,
	EPropertyChangeType::Type InPropertyChangeType, FProperty* InMemberProperty)
{
	if (InObject == nullptr)
	{
		return;
	}

	for (FProperty* Property : InProperties)
	{
		FPropertyChangedEvent PropertyChangedEvent(Property);

		if (InMemberProperty)
		{
			PropertyChangedEvent.SetActiveMemberProperty(InMemberProperty);
		}

		InObject->PostEditChangeProperty(PropertyChangedEvent);
	}

	// Rerun construction script on preview actor
	if (AActor* Actor = Cast<AActor>(InObject))
	{
		Actor->PostEditMove(InPropertyChangeType == EPropertyChangeType::ValueSet);
	}
	else if (const UActorComponent* ActorComponent = Cast<UActorComponent>(InObject))
	{
		if (AActor* Owner = ActorComponent->GetOwner())
		{
			Owner->PostEditMove(InPropertyChangeType == EPropertyChangeType::ValueSet);
		}
	}
}

void FAvaVisualizerBase::NotifyPropertyChainModified(UObject* InObject, FProperty* InProperty,
	EPropertyChangeType::Type InPropertyChangeType, int32 InContainerIdx, TArray<FProperty*> InMemberChainProperties)
{
	if (InObject == nullptr || InMemberChainProperties.Num() == 0)
	{
		return;
	}

	FPropertyChangedEvent PropertyChangedEvent(InProperty);
	FEditPropertyChain PropertyChain;
	FPropertyChangedChainEvent PropertyChangedChainEvent(PropertyChain, PropertyChangedEvent);
	PropertyChangedChainEvent.PropertyChain.AddHead(InProperty);
	for (FProperty*& ChainProperty : InMemberChainProperties)
	{
		PropertyChangedChainEvent.PropertyChain.AddHead(ChainProperty);
	}

	PropertyChangedChainEvent.SetActiveMemberProperty(InMemberChainProperties.Last());
	PropertyChangedChainEvent.ObjectIteratorIndex = 0;

	TArray<TMap<FString, int32>> ArrayIndices;
	ArrayIndices.SetNum(1);
	ArrayIndices[0].Add(InMemberChainProperties.Last()->GetFName().ToString(), InContainerIdx);
	PropertyChangedChainEvent.SetArrayIndexPerObject(ArrayIndices);

	InObject->PostEditChangeChainProperty(PropertyChangedChainEvent);

	// Rerun construction script on preview actor
	if (AActor* Actor = Cast<AActor>(InObject))
	{
		Actor->PostEditMove(InPropertyChangeType == EPropertyChangeType::ValueSet);
	}
	else if (const UActorComponent* ActorComponent = Cast<UActorComponent>(InObject))
	{
		if (AActor* Owner = ActorComponent->GetOwner())
		{
			Owner->PostEditMove(InPropertyChangeType == EPropertyChangeType::ValueSet);
		}
	}
}

#undef LOCTEXT_NAMESPACE