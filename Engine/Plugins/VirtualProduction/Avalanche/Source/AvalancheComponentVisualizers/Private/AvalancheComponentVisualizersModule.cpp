// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvalancheComponentVisualizersModule.h"
#include "AvaComponentVisualizersSettings.h"
#include "AvaComponentVisualizersViewportOverlay.h"
#include "ComponentVisualizers.h"
#include "Modules/ModuleManager.h"

namespace UE::AvaComponentVisualizers::Private
{
	TSet<TWeakPtr<FComponentVisualizer>> AvalancheVisualizers;
	static FAvaComponentVisualizersViewportOverlay ViewportOverlay;
}

IAvaComponentVisualizersSettings* FAvalancheComponentVisualizersModule::GetSettings() const
{
	return GetMutableDefault<UAvaComponentVisualizersSettings>();
}

void FAvalancheComponentVisualizersModule::RegisterComponentVisualizer(FName InComponentClassName,
	TSharedRef<FComponentVisualizer> InVisualizer) const
{
	FComponentVisualizersModule& VisualisersModule = FModuleManager::LoadModuleChecked<FComponentVisualizersModule>("ComponentVisualizers");
	VisualisersModule.RegisterComponentVisualizer(InComponentClassName, InVisualizer);
	UE::AvaComponentVisualizers::Private::AvalancheVisualizers.Add(InVisualizer);
}

bool FAvalancheComponentVisualizersModule::IsAvalancheVisualizer(TSharedRef<FComponentVisualizer> InVisualizer) const
{
	return UE::AvaComponentVisualizers::Private::AvalancheVisualizers.Contains(InVisualizer);
}

IAvaComponentVisualizersViewportOverlay& FAvalancheComponentVisualizersModule::GetViewportOverlay() const
{
	return UE::AvaComponentVisualizers::Private::ViewportOverlay;
}

IMPLEMENT_MODULE(FAvalancheComponentVisualizersModule, AvalancheComponentVisualizers)
