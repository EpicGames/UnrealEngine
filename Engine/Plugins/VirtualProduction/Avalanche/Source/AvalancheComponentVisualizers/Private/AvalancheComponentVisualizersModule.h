// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvalancheComponentVisualizersModule.h"

class FAvalancheComponentVisualizersModule : public IAvalancheComponentVisualizersModule
{
public:
	//~ Begin IAvalancheComponentVisualizersModule
	virtual IAvaComponentVisualizersSettings* GetSettings() const override;
	virtual void RegisterComponentVisualizer(FName InComponentClassName, TSharedRef<FComponentVisualizer> InVisualizer) const override;
	virtual bool IsAvalancheVisualizer(TSharedRef<FComponentVisualizer> InVisualizer) const override;
	virtual IAvaComponentVisualizersViewportOverlay& GetViewportOverlay() const override;
	//~ End IAvalancheComponentVisualizersModule
};
