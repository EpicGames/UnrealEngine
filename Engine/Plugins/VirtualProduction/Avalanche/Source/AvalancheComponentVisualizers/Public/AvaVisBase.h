// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentVisualizer.h"
#include "UnrealWidgetFwd.h"

struct AVALANCHECOMPONENTVISUALIZERS_API HAvaHitProxy : HComponentVisProxy
{
	DECLARE_HIT_PROXY();

	HAvaHitProxy(const UActorComponent* InComponent)
		: HComponentVisProxy(InComponent, HPP_Wireframe)
	{ }
};

struct AVALANCHECOMPONENTVISUALIZERS_API HAvaDirectHitProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaDirectHitProxy(const UActorComponent* InComponent)
		: HAvaHitProxy(InComponent)
	{ }
};

class FAvaEditorViewportClient;
class FAvaSnapOperation;
class FProperty;

/*
 * Select shape:
 * - DrawVisualization
 *
 * Select HitProxy:
 * - VisProxyHandleClick
 *    |- EndEditing
 *    |   |- EndTransaction
 *    |   \- Update ST that no Vis is active.
 *    \- StartEditing
 *        |- Store component & HP settings
 *        \- Update ST that we are active.
 *
 * Mouse down on widget:
 * - Viewport: TrackingStarted (not used)
 *
 * Drag mouse
 * - HandleInputDelta
 *    |- (Not tracking)
 *    |   \- StartTracking
 *    |       |- TrackingStarted
 *    |       \- StoreInitialValues
 *    \- HandleInputDeltaInternal
 *
 * Release mouse
 * - Viewport: TrackingStopped
 *    \- TrackingStopped
 *        \- EndTransaction (see above)
 *
 */
class AVALANCHECOMPONENTVISUALIZERS_API FAvaVisualizerBase : public FComponentVisualizer
{
	using FComponentVisualizer::TrackingStopped;

public:
	using Super = FComponentVisualizer;

	FEditorViewportClient* GetLastUsedViewportClient() const { return LastUsedViewportClient; }

	/** Returns a list of objects and the properties being edited on that object. */
	virtual TMap<UObject*, TArray<FProperty*>> GatherEditableProperties(UObject* InObject) const = 0;

	virtual const USceneComponent* GetEditedSceneComponent() const { return GetEditedSceneComponent(GetEditedComponent()); }
	virtual const USceneComponent* GetEditedSceneComponent(const UActorComponent* InComponent) const;

	virtual bool GetWidgetMode(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode& OutMode) const { return false; }

	virtual bool GetWidgetAxisList(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode,
		EAxisList::Type& OutAxisList) const { return false; }

	virtual void StartEditing(FEditorViewportClient* InViewportClient, UActorComponent* InEditedComponent);

	virtual bool IsEditing() const { return false; }

	// Used to override the axis used to actually calculate drag.
	virtual bool GetWidgetAxisListDragOverride(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode,
		EAxisList::Type& OutAxisList) const { return false; }

	virtual bool ResetValue(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy) { return false; }

	bool IsTracking() const { return bTracking; }

	TSharedPtr<FAvaSnapOperation> GetSnapOperation() const { return SnapOperation; }

	//~ Begin FComponentVisualizer
	virtual void DrawVisualization(const UActorComponent* InComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI) override;
	virtual void DrawVisualizationHUD(const UActorComponent* InComponent, const FViewport* InViewport, const FSceneView* InView, FCanvas* InCanvas) override {}
	virtual bool GetCustomInputCoordinateSystem(const FEditorViewportClient* InViewportClient, FMatrix& OutMatrix) const override;
	virtual bool HandleInputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDeltaTranslate,
		FRotator& InDeltaRotate, FVector& InDeltaScale) override;
	virtual bool HandleModifiedClick(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy, const FViewportClick& InClick) override;
	// Editing -> We have a hitproxy selected
	virtual void EndEditing() override;
	// Tracking -> We are actively dragging a widget
	virtual void TrackingStarted(FEditorViewportClient* InViewportClient) override;
	virtual void TrackingStopped(FEditorViewportClient* InViewportClient, bool bInDidMove) override;
	//~ End FComponentVisualizer

protected:
	/** Notify that a component property has been modified */
	static void NotifyPropertyModified(UObject* InObject, FProperty* InProperty,
		EPropertyChangeType::Type InPropertyChangeType = EPropertyChangeType::Unspecified, FProperty* InMemberProperty = nullptr);

	/** Notify that many component properties have been modified */
	static void NotifyPropertiesModified(UObject* InObject, const TArray<FProperty*>& InProperties,
		EPropertyChangeType::Type InPropertyChangeType = EPropertyChangeType::Unspecified, FProperty* InMemberProperty = nullptr);

	/** Notify that a component property has been modified inside a container */
	static void NotifyPropertyChainModified(UObject* InObject, FProperty* InProperty,
		EPropertyChangeType::Type InPropertyChangeType, int32 InContainerIdx, TArray<FProperty*> InMemberChainProperties);

	static FLinearColor Inactive;
	static FLinearColor Active;
	static FLinearColor ActiveAltMode;
	static FLinearColor Enabled;
	static FLinearColor Disabled;

	static float GetIconSizeScale(const FSceneView* InView, const FVector& InWorldPosition);
	static const FLinearColor& GetIconColor(bool bInActive, bool bInAltMode = false);

	// Last used viewport client
	FEditorViewportClient* LastUsedViewportClient = nullptr;
	bool bCVarAutoSelectComponent = true;

	// Delta sums
	bool bHasInitialWidgetLocation;
	FVector InitialWidgetLocation;
	FVector AccumulatedTranslation;
	FRotator AccumulatedRotation;
	FVector AccumulatedScale;

	// Top icons
	FVector IconStartPosition;

	// Tracking info
	int32 TransactionIdx;
	bool bTracking;
	bool bHasBeenModified;

	// Snapping stuff
	TSharedPtr<FAvaSnapOperation> SnapOperation;

	FAvaVisualizerBase() = default;
	virtual ~FAvaVisualizerBase() override = default;

	virtual void DrawVisualizationNotEditing(const UActorComponent* InComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) { }

	virtual void DrawVisualizationEditing(const UActorComponent* InComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) { }

	void StartTransaction();
	void EndTransaction();
	virtual void StoreInitialValues();

	virtual FBox GetComponentBounds(const UActorComponent* InComponent) const;
	virtual FTransform GetComponentTransform(const UActorComponent* InComponent) const;

	// Calculates the location of the first top-of-shape icon in world space
	// Uses the shape's bounding box to do this.
	bool CalcIconLocation(const USceneComponent* InComponent, const FSceneView* InView);

	// Gets the position and size of icons based on the above ^
	void GetIconMetrics(const FSceneView* InView, int32 InIconIndex, FVector& InOutPosition, float& InOutSize) const;

	UE::Widget::EWidgetMode GetViewportWidgetMode(FEditorViewportClient* InViewportClient) const;
	EAxisList::Type GetViewportWidgetAxisList(FEditorViewportClient* InViewportClient) const;

	// Converts vectors from world space to local space
	FVector GetLocalVector(FEditorViewportClient* InViewport, const FVector& InVector) const;

	virtual bool HandleInputDeltaInternal(FEditorViewportClient* InViewportClient, FViewport* InViewport, const FVector& InAccumulatedTranslation,
		const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale) { return false; }

	bool IsMouseOverComponent(const UActorComponent* InComponent, const FSceneView* InView) const;
	bool ShouldDrawExtraHandles(const UActorComponent* InComponent, const FSceneView* InView) const;

	virtual void AddSnapDataBinding() {}

	virtual void GenerateContextSensitiveSnapPoints() {}

	virtual void TrackingStartedInternal(FEditorViewportClient* InViewportClient);
	virtual void TrackingStoppedInternal(FEditorViewportClient* InViewportClient);

	void ModifyProperty(UObject* InObject, FProperty* InProperty, EPropertyChangeType::Type InPropertyChangeType, TFunctionRef<void()> InFunction);
	void ModifyProperty(UObject* InObject, FName InPropertyName, EPropertyChangeType::Type InPropertyChangeType, TFunctionRef<void()> InFunction);
};
