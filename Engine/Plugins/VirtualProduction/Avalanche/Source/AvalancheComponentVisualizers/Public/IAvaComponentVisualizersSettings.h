// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

class UTexture2D;

class IAvaComponentVisualizersSettings
{
public:
	virtual float GetSpriteSize() const = 0;

	virtual void SetSpriteSize(float InSpriteSize) = 0;

	virtual UTexture2D* GetVisualizerSprite(FName InName) const = 0;

	virtual void SetVisualizerSprite(FName InName, UTexture2D* InTexture) = 0;

	/** Same as above, but doesn't replace an existing value */
	virtual void SetDefaultVisualizerSprite(FName InName, UTexture2D* InTexture) = 0;

	virtual void SaveSettings() = 0;
};
