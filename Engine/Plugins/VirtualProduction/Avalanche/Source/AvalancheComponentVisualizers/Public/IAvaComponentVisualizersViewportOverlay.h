// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointerFwd.h"

class IAvaViewportClient;
class UObject;

class IAvaComponentVisualizersViewportOverlay
{
public:
	virtual ~IAvaComponentVisualizersViewportOverlay() = default;

	virtual void AddWidget(const TArray<TSharedPtr<IAvaViewportClient>>& InAvaViewportClients, const TArray<UObject*>& InSelectedObjects) = 0;

	virtual void RemoveWidget(const TArray<TSharedPtr<IAvaViewportClient>>& InAvaViewportClients) = 0;

	virtual bool IsWidgetActive() const = 0;
};
