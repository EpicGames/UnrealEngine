// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentVisualizer.h"
#include "IAvaComponentVisualizersSettings.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class IAvaComponentVisualizersViewportOverlay;

namespace UE::AvaComponentVisualizers
{
	const FName ModuleName = TEXT("AvalancheComponentVisualizers");
}

class IAvalancheComponentVisualizersModule : public IModuleInterface
{
public:
	static IAvalancheComponentVisualizersModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IAvalancheComponentVisualizersModule>(UE::AvaComponentVisualizers::ModuleName);
	}

	static IAvalancheComponentVisualizersModule* GetIfLoaded()
	{
		return FModuleManager::LoadModulePtr<IAvalancheComponentVisualizersModule>(UE::AvaComponentVisualizers::ModuleName);
	}

	/** Could also use a TSet or custom storage class. */
	template<typename InComponentClass, typename InVisualizerClass, typename InStorageType = TArray<TSharedPtr<FComponentVisualizer>>>
	static TSharedPtr<FComponentVisualizer> RegisterComponentVisualizer(InStorageType* Storage = nullptr)
	{
		IAvalancheComponentVisualizersModule* VisualizerModule = GetIfLoaded();

		if (!VisualizerModule)
		{
			return nullptr;
		}

		TSharedRef<FComponentVisualizer> NewVis = MakeShared<InVisualizerClass>();

		if (Storage)
		{
			Storage->Add(NewVis);
		}

		VisualizerModule->RegisterComponentVisualizer(InComponentClass::StaticClass()->GetFName(), NewVis);

		return NewVis;
	}

	virtual IAvaComponentVisualizersSettings* GetSettings() const = 0;

	virtual void RegisterComponentVisualizer(FName InComponentClassName, TSharedRef<FComponentVisualizer> InVisualizer) const = 0;

	virtual bool IsAvalancheVisualizer(TSharedRef<FComponentVisualizer> InVisualizer) const = 0;

	virtual IAvaComponentVisualizersViewportOverlay& GetViewportOverlay() const = 0;
};
