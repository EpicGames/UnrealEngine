// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaDataView.h"

namespace UE::Ava
{

FDataView::FDataView(const UStruct* InStruct, void* InMemory)
	: Struct(InStruct)
	, Memory(InMemory)
{
	// Avoid cases where a non-null memory is provided with a null struct.
	check(!Memory || (Memory && Struct));
}

FDataView::FDataView(UObject* Object)
	: FDataView(Object ? Object->GetClass() : nullptr, Object)
{
}

bool FDataView::IsValid() const
{
	return Memory && Struct;
}

bool FDataView::IsValidFor(const UStruct* InStruct) const
{
	return IsValid() && Struct->IsChildOf(InStruct);
}

FConstDataView::FConstDataView(const UStruct* InStruct, const void* InMemory)
	: Struct(InStruct)
	, Memory(InMemory)
{
	// Avoid cases where a non-null memory is provided with a null struct.
	check(!Memory || (Memory && Struct));
}

FConstDataView::FConstDataView(const UObject* Object)
	: FConstDataView(Object ? Object->GetClass() : nullptr, Object)
{
}

bool FConstDataView::IsValid() const
{
	return Memory && Struct;
}

bool FConstDataView::IsValidFor(const UStruct* InStruct) const
{
	return IsValid() && Struct->IsChildOf(InStruct);
}

} // UE::Ava
