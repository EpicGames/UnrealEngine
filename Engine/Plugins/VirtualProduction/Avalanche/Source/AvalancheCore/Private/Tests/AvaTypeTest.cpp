// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTypeTest.h"
#include "Misc/AutomationTest.h"
#include "Templates/SharedPointer.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAvaTypeTests, "Avalanche.Core.TypeInheritance", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter)
bool FAvaTypeTests::RunTest(const FString& Parameters)
{
	using namespace UE::AvaCore::Tests::Private;

	const TSharedRef<IAvaTypeCastable> Inst1 = MakeShared<FSubTypeA>();

	TestTrue (TEXT("Inst1 is FSubTypeA")          , Inst1->IsA<FSubTypeA>());
	TestFalse(TEXT("Inst1 is NOT FSubTypeB")      , Inst1->IsA<FSubTypeB>());

	TestTrue (TEXT("Inst1 is FTypeA")             , Inst1->IsA<FTypeA>());
	TestFalse(TEXT("Inst1 is NOT FTypeB")         , Inst1->IsA<FTypeB>());

	TestTrue (TEXT("Inst1 is ISuperType")         , Inst1->IsA<ISuperType>());
	TestTrue (TEXT("Inst1 is FSuperTypeA")        , Inst1->IsA<FSuperTypeA>());
	TestFalse(TEXT("Inst1 is NOT FSuperTypeB")    , Inst1->IsA<FSuperTypeB>());

	TestFalse(TEXT("Inst1 is NOT IExternalType")  , Inst1->IsA<IExternalType>());
	TestTrue (TEXT("Inst1 is FExternalTypeA")     , Inst1->IsA<FExternalTypeA>());
	TestFalse(TEXT("Inst1 is NOT FExternalTypeB") , Inst1->IsA<FExternalTypeB>());

	const TSharedRef<IAvaTypeCastable> Inst2 = MakeShared<FSubTypeB>();

	TestFalse(TEXT("Inst2 is NOT FSubTypeA")     , Inst2->IsA<FSubTypeA>());
	TestTrue (TEXT("Inst2 is FSubTypeB")         , Inst2->IsA<FSubTypeB>());

	TestFalse(TEXT("Inst2 is NOT FTypeA")        , Inst2->IsA<FTypeA>());
	TestTrue (TEXT("Inst2 is FTypeB")            , Inst2->IsA<FTypeB>());

	TestTrue (TEXT("Inst2 is a ISuperType")      , Inst2->IsA<ISuperType>());
	TestTrue (TEXT("Inst2 is FSuperTypeA")       , Inst2->IsA<FSuperTypeA>());
	TestTrue (TEXT("Inst2 is FSuperTypeB")       , Inst2->IsA<FSuperTypeB>());

	TestTrue (TEXT("Inst2 is IExternalType")     , Inst2->IsA<IExternalType>());
	TestTrue (TEXT("Inst2 is FExternalTypeA")    , Inst2->IsA<FExternalTypeA>());
	TestTrue (TEXT("Inst2 is FExternalTypeB")    , Inst2->IsA<FExternalTypeB>());

	return true;
}
