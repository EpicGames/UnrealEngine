// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"

namespace UE::AvaCore::Tests::Private
{
	/** Types with no UE_AVA_TYPE macro within */
	struct IExternalType {};
	struct FExternalTypeA : IExternalType {};
	struct FExternalTypeB : IExternalType {};
}

/**
 * Even Though both FExternalTypeA and FExternalTypeB inherit from IExternalType,
 * only FExternalTypeB is specified to inherit from it here, deliberately.
 * Test for an instance of a derived type inheriting only from FExternalTypeA should return false for IsA<IExternalType>()
 */
UE_AVA_TYPE_EXTERNAL(UE::AvaCore::Tests::Private::IExternalType);
UE_AVA_TYPE_EXTERNAL(UE::AvaCore::Tests::Private::FExternalTypeA);
UE_AVA_TYPE_EXTERNAL(UE::AvaCore::Tests::Private::FExternalTypeB, UE::AvaCore::Tests::Private::IExternalType);

namespace UE::AvaCore::Tests::Private
{
	struct ISuperType
	{
		UE_AVA_TYPE(ISuperType);
	};

	struct FSuperTypeA : ISuperType
	{
		UE_AVA_TYPE(FSuperTypeA, ISuperType)
	};

	struct FSuperTypeB : ISuperType
	{
		UE_AVA_TYPE(FSuperTypeB, ISuperType)
	};

	struct FTypeA : FSuperTypeA, FExternalTypeA, IAvaTypeCastable
	{
		UE_AVA_INHERITS(FTypeA, FSuperTypeA, FExternalTypeA, IAvaTypeCastable)
	};

	struct FTypeB : FSuperTypeA, FSuperTypeB, FExternalTypeA, FExternalTypeB, IAvaTypeCastable
	{
		UE_AVA_INHERITS(FTypeB, FSuperTypeA, FSuperTypeB, FExternalTypeA, FExternalTypeB, IAvaTypeCastable)
	};

	struct FSubTypeA : FTypeA
	{
		UE_AVA_INHERITS(FSubTypeA, FTypeA)
	};

	struct FSubTypeB : FTypeB
	{
		UE_AVA_INHERITS(FSubTypeB, FTypeB)
	};
}
