// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/StructUtils.h"
#include "StructUtils/StructUtilsTypes.h"
#include "UObject/Class.h"

#define UE_API AVALANCHECORE_API

namespace UE::Ava
{

/** Short-lived pointer to an object or struct. */
struct FDataView
{
	FDataView() = default;

	/**
	 * Generic struct constructor.
	 * Valid UStruct is required when passing in valid memory pointer.
	 * Both can be null and the constructed data view will be considered as invalid.
	 * @see IsValid
	 */
	UE_API explicit FDataView(const UStruct* InStruct, void* InMemory);

	/** UObject constructor. */
	UE_API explicit FDataView(UObject* Object);

	/**
	 * Check is the view is valid (both pointer and type are set).
	 * @return True if the view is valid.
	 */
	UE_API bool IsValid() const;

	/** Returns true if both the type and memory are set for the given struct. */
	UE_API bool IsValidFor(const UStruct* InStruct) const;

	/** Returns true if both the type and memory are set for the given struct. */
	template<typename T>
	bool IsValidFor() const
	{
		return this->IsValidFor(StructUtils::GetAsUStruct<T>());
	}

	template<typename T>
	const T* GetPtr() const
	{
		return GetMutablePtr<T>();
	}

	template<typename T>
	const T& Get() const
	{
		return GetMutable<T>();
	}

	template<typename T>
	T* GetMutablePtr() const
	{
		if (IsValidFor<T>())
		{
			return static_cast<T*>(Memory);
		}
		return nullptr;
	}

	template<typename T>
	T& GetMutable() const
	{
		check(IsValidFor<T>());
		return *static_cast<T*>(Memory);
	}

	const UStruct* GetStruct() const
	{
		return Struct;
	}

	const void* GetMemory() const
	{
		return GetMutableMemory();
	}

	void* GetMutableMemory() const
	{
		return Memory;
	}

protected:
	/** UClass or UScriptStruct of the data. */
	const UStruct* Struct = nullptr;
	/** Memory pointing at the class or struct */
	void* Memory = nullptr;
};

/** Short-lived const pointer to an object or struct. */
struct FConstDataView
{
	FConstDataView() = default;

	/**
	 * Generic struct constructor.
	 * Valid UStruct is required when passing in valid memory pointer.
	 * Both can be null and the constructed data view will be considered as invalid.
	 * @see IsValid
	 */
	UE_API explicit FConstDataView(const UStruct* InStruct, const void* InMemory);

	/** UObject constructor. */
	UE_API explicit FConstDataView(const UObject* Object);

	/** Returns true if both the type and memory are set. */
	UE_API bool IsValid() const;

	/** Returns true if both the type and memory are set for the given struct. */
	UE_API bool IsValidFor(const UStruct* InStruct) const;

	template<typename T>
	bool IsValidFor() const
	{
		return this->IsValidFor(StructUtils::GetAsUStruct<T>());
	}

	template<typename T> requires (std::is_const_v<T>)
	T* GetPtr() const
	{
		if (IsValidFor<T>())
		{
			return static_cast<T*>(Memory);
		}
		return nullptr;
	}

	template<typename T> requires (std::is_const_v<T>)
	T& Get() const
	{
		check(IsValidFor<T>());
		return *static_cast<T*>(Memory);
	}

	const UStruct* GetStruct() const
	{
		return Struct;
	}

	const void* GetMemory() const
	{
		return Memory;
	}

protected:
	/** UClass or UScriptStruct of the data. */
	const UStruct* Struct = nullptr;
	/** Const memory pointing at the class or struct */
	const void* Memory = nullptr;
};
	
} // UE::Ava

#undef UE_API
