// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::Ava
{

/** Indicates how iteration should be handled */
enum class EControlFlow : uint8
{
	/** Continue iteration */
	Continue,
	/** Break iteration */
	Break,
};

} // UE::Ava
