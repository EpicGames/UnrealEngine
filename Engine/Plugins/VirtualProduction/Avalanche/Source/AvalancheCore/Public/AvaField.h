// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

#include "Concepts/StaticClassProvider.h"
#include "Concepts/StaticStructProvider.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Models.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/Field.h"
#include "UObject/UnrealType.h"

namespace UE::AvaCore
{
	/**
	 * Find Property Conditions:
	 * 1) InStructType has a way to get UStruct (either via T::StaticClass(), or T::StaticStruct())
	 * AND
	 * 2) InPropertyType is derived from FProperty
	 */
	template<typename InStructType, typename InPropertyType>
	constexpr bool TFindPropertyCondition_V =
		(
			TModels_V<CStaticClassProvider, InStructType> ||
			TModels_V<CStaticStructProvider, InStructType>
		) &&
		std::is_base_of_v<FProperty, InPropertyType>;

	template<typename InStructType, typename InPropertyType = FProperty
		UE_REQUIRES(TFindPropertyCondition_V<InStructType, InPropertyType>)>
	InPropertyType* GetProperty(FName InMemberPropertyName)
	{
		if constexpr (TModels_V<CStaticClassProvider, InStructType>)
		{
			return FindFProperty<InPropertyType>(InStructType::StaticClass(), InMemberPropertyName);
		}

		else if constexpr (TModels_V<CStaticStructProvider, InStructType>)
		{
			return FindFProperty<InPropertyType>(InStructType::StaticStruct(), InMemberPropertyName);
		}

		else
		{
			checkNoEntry();
			return nullptr;
		}
	}
}
