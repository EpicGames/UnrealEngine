// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "Containers/Map.h"
#include "Delegates/DelegateInstanceInterface.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"

template<typename InClassType>
class TAvaPropertyChangeDispatcher
{
public:
	using FMemberFunctionType = typename TMemFunPtrType</*bConst*/false, InClassType, void()>::Type;

	TAvaPropertyChangeDispatcher(std::initializer_list<TPairInitializer<const FName&, const FMemberFunctionType&>> InCallbacks)
		: PropertyChangedFunctions(InCallbacks)
	{}

	void OnPropertyChanged(InClassType* InObject, const FPropertyChangedEvent& InPropertyChangedEvent) const
	{
		const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

		if (const FMemberFunctionType* CallbackFunction = PropertyChangedFunctions.Find(MemberName))
		{
			::Invoke(*CallbackFunction, InObject);
		}
	}

private:
	TMap<FName, FMemberFunctionType> PropertyChangedFunctions;
};
#endif
