// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTypeConcepts.h"
#include "AvaTypeId.h"
#include "AvaTypeTraits.h"
#include "Templates/AndOrNot.h"
#include "Templates/Models.h"

template<typename T>
struct TAvaExternalType;

/**
 * Determines that a typename T is a valid "Ava Type" if a FAvaTypeId can be retrieved by calling either
 * T::GetStaticTypeId() (via UE_AVA_TYPE) or TAvaExternalType<T>::GetStaticTypeId() (via UE_AVA_TYPE_EXTERNAL)
 */
template<typename T>
using TIsValidAvaType = TOr<TModels<CAvaStaticTypeable, T>, TModels<CAvaStaticTypeable, TAvaExternalType<T>>>;

/**
 * Helps retrieve the FAvaTypeId for a given type T that satisfies the TIsValidAvaType requirement
 * If this is requirement is not satisfied, TAvaType<T>::GetTypeId will be undefined for T.
 * @see TIsValidAvaType
 */
template<typename T, bool = TIsValidAvaType<T>::Value>
struct TAvaType
{
	static FAvaTypeId GetTypeId()
	{
		if constexpr (TModels_V<CAvaStaticTypeable, T>)
		{
			return T::GetStaticTypeId();
		}
		else if constexpr (TModels_V<CAvaStaticTypeable, TAvaExternalType<T>>)
		{
			return TAvaExternalType<T>::GetStaticTypeId();
		}
		else
		{
			checkNoEntry();
			return FAvaTypeId::Invalid();
		}
	}
};

/** No support for types that do not have a way to get their FAvaTypeId */
template<typename T>
struct TAvaType<T, false>
{
};

/**
 * Holds the direct super types of a given type T.
 * Also serves to Cast a provided pointer to a desired type id.
 */
template<typename T, typename... InSuperTypes>
class TAvaInherits
{
	static_assert(TIsValidAvaType<T>::Value, "Typename does not satisfy the requirements to be an Ava Type. See: TIsValidAvaType");
	static_assert(UE::AvaCore::TIsDerivedFromAll<T, InSuperTypes...>::Value, "Some provided super types are not base types of T");

	template<typename, typename InSuperType, typename... InOtherSuperTypes>
	static const void* Cast(const T* InPtr, const FAvaTypeId& InCastToType)
	{
		// Check first super type
		if (TAvaType<InSuperType>::GetTypeId() == InCastToType)
		{
			return static_cast<const InSuperType*>(InPtr);
		}
		// Check rest of super types
		else if (const void* CastedPtr = Cast<void, InOtherSuperTypes...>(InPtr, InCastToType))
		{
			return CastedPtr;
		}
		// Recurse up the super type
		else if constexpr (TModels_V<CAvaInheritable, InSuperType>)
		{
			return InSuperType::FAvaInherits::Cast(InPtr, InCastToType);
		}
		// Recurse up the super type for External Types
		else if constexpr (TModels_V<CAvaInheritable, TAvaExternalType<InSuperType>>)
		{
			return TAvaExternalType<InSuperType>::FAvaInherits::Cast(InPtr, InCastToType);
		}
		else
		{
			return nullptr;
		}
	}

	template<typename>
	static const void* Cast(const T* InPtr, const FAvaTypeId& InCastToType)
	{
		return nullptr;
	}

public:
	static const void* Cast(const T* InPtr, const FAvaTypeId& InCastToType)
	{
		// Check that Cast To Type is valid
		if (!InCastToType.IsValid())
		{
			return nullptr;
		}
		// Check whether Cast To Type is this Type
		if (TAvaType<T>::GetTypeId() == InCastToType)
		{
			return InPtr;
		}
		return Cast<void, InSuperTypes...>(InPtr, InCastToType);
	}
};

/**
 * Macro to provide type information (type id and super types).
 * This should be used within the class/struct scope only.
 * @see UE_AVA_TYPE_EXTERNAL for providing type information outside the class/struct scope
 */
#define UE_AVA_TYPE(InThisType, ...)                                        \
	using FAvaInherits = TAvaInherits<InThisType, ##__VA_ARGS__>;           \
	static FAvaTypeId GetStaticTypeId()                                     \
	{                                                                       \
		static const FAvaTypeId TypeId(TEXT(#InThisType));                  \
		return TypeId;                                                      \
	}                                                                       \

/**
 * Macro to implement a specialization of TAvaExternalType to specify the UE_AVA_TYPE for a given type.
 * This should only be done for types where a UE_AVA_TYPE internal implementation is not possible or desired.
 * @see UE_AVA_TYPE
 */
#define UE_AVA_TYPE_EXTERNAL(InExternalType, ...)                           \
	template<>                                                              \
	struct TAvaExternalType<InExternalType>                                 \
	{                                                                       \
		UE_AVA_TYPE(InExternalType, ##__VA_ARGS__);                         \
	};                                                                      \

/**
 * Macro to implement UE_AVA_TYPE for a given type and the overrides for the IAvaTypeCastable interface.
 * This should be used within a class/struct that ultimately inherits from IAvaTypeCastable
 * @see IAvaTypeCastable
 * @see UE_AVA_TYPE
 */
#define UE_AVA_INHERITS(InThisType, ...)                                    \
	UE_AVA_TYPE(InThisType, ##__VA_ARGS__)                                  \
	virtual FAvaTypeId GetTypeId() const override                           \
	{                                                                       \
		return TAvaType<InThisType>::GetTypeId();                           \
	}                                                                       \
	virtual const void* CastTo_Impl(FAvaTypeId InCastToType) const override \
	{                                                                       \
		return FAvaInherits::Cast(this, InCastToType);                      \
	}                                                                       \

#define UE_AVA_INHERITS_WITH_SUPER(InThisType, InSuperType, ...)            \
	UE_AVA_INHERITS(InThisType, InSuperType, ##__VA_ARGS__)                 \
	using Super = InSuperType;                                              \

/**
 * Interface Class to perform actions like IsA, or CastTo
 * @see UE_AVA_INHERITS
 */
class IAvaTypeCastable
{
public:
	UE_AVA_TYPE(IAvaTypeCastable)

	virtual ~IAvaTypeCastable() = default;

	virtual FAvaTypeId GetTypeId() const = 0;

	virtual const void* CastTo_Impl(FAvaTypeId InId) const = 0;

	template<typename T>
	bool IsA() const
	{
		return !!this->CastTo<T>();
	}

	template<typename T>
	bool IsExactlyA() const
	{
		return GetTypeId() == TAvaType<T>::GetTypeId();
	}

	template<typename T>
	T* CastTo()
	{
		const IAvaTypeCastable* ConstThis = this;
		return const_cast<T*>(ConstThis->CastTo<T>());
	}

	template<typename T>
	const T* CastTo() const
	{
		return static_cast<const T*>(this->CastTo_Impl(TAvaType<T>::GetTypeId()));
	}
};
