// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** Describes a type that has a GetStaticTypeId() static function that returns FAvaTypeId */
struct CAvaStaticTypeable
{
	template <typename T>
	auto Requires(class FAvaTypeId& Value)->decltype(
		Value = T::GetStaticTypeId()
	);
};

/**
 * Describes a type with a FAvaInherits typedef or using declaration
 * @see UE_AVA_TYPE macro
 */
struct CAvaInheritable
{
	template<typename T>
	auto Requires()->typename T::FAvaInherits&;
};
