// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "UObject/NameTypes.h"

class FAvaTypeId
{
public:
	static FAvaTypeId Invalid()
	{
		return FAvaTypeId(NAME_None);
	}

	explicit FAvaTypeId(FName InTypeName)
		: Name(InTypeName)
	{
	}

	friend uint32 GetTypeHash(FAvaTypeId InTypeId)
	{
		return GetTypeHash(InTypeId.Name);
	}

	bool operator==(FAvaTypeId InOther) const
	{
		return this->Name == InOther.Name;
	}

	bool IsValid() const
	{
		return !Name.IsNone();
	}

	FName ToName() const
	{
		return Name;
	}

	FString ToString() const
	{
		return Name.ToString();
	}

private:
	FName Name;
};
