// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "Templates/SharedPointer.h"

namespace UE::AvaCore
{
	template<typename InCastToType>
	TSharedPtr<InCastToType> CastSharedPtr(const TSharedPtr<IAvaTypeCastable>& InSharedPtr)
	{
		if (InSharedPtr.IsValid())
		{
			if (InCastToType* CastToPtr = InSharedPtr->CastTo<InCastToType>())
			{
				return TSharedPtr<InCastToType>(InSharedPtr, CastToPtr);
			}
		}
		return nullptr;
	}
}
