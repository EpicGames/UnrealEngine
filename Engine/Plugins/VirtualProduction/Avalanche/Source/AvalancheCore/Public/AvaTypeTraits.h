// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace UE::AvaCore
{
	/** Returns true if InDerivedType is a derived type of ALL (And) the given InBaseTypes */
	template<typename InDerivedType, typename... InBaseTypes>
	struct TIsDerivedFromAll;

	template<typename InDerivedType>
	struct TIsDerivedFromAll<InDerivedType>
	{
		static constexpr bool Value = true;
	};

	template<typename InDerivedType, typename InBaseType>
	struct TIsDerivedFromAll<InDerivedType, InBaseType>
	{
		static constexpr bool Value = std::is_base_of_v<InBaseType, InDerivedType>;
	};

	template<typename InDerivedType, typename InBaseType, typename... InOtherBaseTypes>
	struct TIsDerivedFromAll<InDerivedType, InBaseType, InOtherBaseTypes...>
	{
		static constexpr bool Value = TIsDerivedFromAll<InDerivedType, InBaseType>::Value
			&& TIsDerivedFromAll<InDerivedType, InOtherBaseTypes...>::Value;
	};
}
