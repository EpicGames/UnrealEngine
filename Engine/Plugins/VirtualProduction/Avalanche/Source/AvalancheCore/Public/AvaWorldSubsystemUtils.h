// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/World.h"
#include "Misc/AssertionMacros.h"
#include "Subsystems/WorldSubsystem.h"
#include "Templates/UnrealTypeTraits.h"

/**
 * Class to add some boilerplate functionality to world subsystems. Supports UObject and
 * non-UObject world contexts. If bInGenerateErrors is true, it will generate ensures.
 *
 * Example usage:
 *
 * Given a class that inherits from TAvaWorldUtility:
 *   class UMySubsystem : public UWorldSubsystem, public TAvaWorldSubsystemInterface<UMySubsystem>
 *
 * Then you can call Get using the subsystem class:
 *   UMySubsystem* Subsystem = UMySubsystem::Get(WorldContext);
 *
 * Without an inheriting class, you can use the utility struct directly:
 *   UMySubsystem* Subsystem = FAvaWorldSubsystemUtils::GetWorldSubstem<UMyOtherSubsystem>(WorldContext);
 */

struct CAvaWorldProvider
{
	template <typename T>
	auto Requires(UWorld*& OutWorld, const T& InObject) -> decltype(
		OutWorld = InObject.GetWorld()
	);
};

struct FAvaWorldSubsystemUtils
{
	template<typename InWorldSubsystemType, typename InWorldContextType
		UE_REQUIRES(TIsDerivedFrom<InWorldSubsystemType, UWorldSubsystem>::Value && TModels_V<CAvaWorldProvider, InWorldContextType>)>
	static InWorldSubsystemType* GetWorldSubsystem(const InWorldContextType* InWorldContext, bool bInGenerateErrors = false)
	{
		if constexpr (TIsDerivedFrom<InWorldContextType, UObject>::Value)
		{
			if (!IsValid(InWorldContext))
			{
				if (bInGenerateErrors)
				{
					ensureMsgf(IsValid(InWorldContext), TEXT("Invalid world context object when trying to get %s subsystem."), *InWorldSubsystemType::StaticClass()->GetName());
				}

				return nullptr;
			}
		}
		else
		{
			if (!InWorldContext)
			{
				if (bInGenerateErrors)
				{
					ensureMsgf(InWorldContext, TEXT("Invalid world context object when trying to get %s subsystem."), *InWorldSubsystemType::StaticClass()->GetName());
				}

				return nullptr;
			}
		}

		UWorld* World = InWorldContext->GetWorld();

		if (!IsValid(World))
		{
			if (bInGenerateErrors)
			{
				ensureMsgf(IsValid(World), TEXT("Invalid world when trying to get %s subsystem."), *InWorldSubsystemType::StaticClass()->GetName());
			}

			return nullptr;
		}

		InWorldSubsystemType* Subsystem = World->GetSubsystem<InWorldSubsystemType>();

		if (!Subsystem && bInGenerateErrors)
		{
			ensureMsgf(Subsystem, TEXT("Subsystem %s missing from world."), *InWorldSubsystemType::StaticClass()->GetName());
		}

		return Subsystem;
	}
};

template<typename InWorldSubsystemType>
struct TAvaWorldSubsystemInterface
{
	template< typename InWorldContextType
		UE_REQUIRES(TModels_V<CAvaWorldProvider, InWorldContextType>)>
	static InWorldSubsystemType* Get(const InWorldContextType* InWorldContext, bool bInGenerateErrors = false)
	{
		return FAvaWorldSubsystemUtils::GetWorldSubsystem<InWorldSubsystemType, InWorldContextType>(InWorldContext, bInGenerateErrors);
	}

	template< typename InWorldContextType
		UE_REQUIRES(TModels_V<CAvaWorldProvider, InWorldContextType>)>
	static bool SupportsWorldType(InWorldContextType* InWorldContext)
	{
		if constexpr (TIsDerivedFrom<InWorldContextType, UObject>::Value)
		{
			if (!IsValid(InWorldContext))
			{
				return false;
			}
		}
		else
		{
			if (!InWorldContext)
			{
				return false;
			}
		}

		UWorld* World = InWorldContext->GetWorld();		
		return GetDefault<InWorldSubsystemType>()->ShouldCreateSubsystem(World);
	}
};
