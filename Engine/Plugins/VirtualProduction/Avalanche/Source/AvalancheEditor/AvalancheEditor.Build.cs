// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheEditor : ModuleRules
{
	public AvalancheEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Avalanche",
				"Core",
				"CoreUObject",
				"Engine",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ActorModifierCore",
				"ActorModifier",
				"ActorModifierCoreEditor",
				"AdvancedPreviewScene",
				"AdvancedRenamer",
				"AppFramework",
				"ApplicationCore",
				"AssetRegistry",
				"AssetTools",
				"AvalancheAttributeEditor",
				"AvalancheComponentVisualizers",
				"AvalancheCore",
				"AvalancheEditorCore",
				"AvalancheEffectors",
				"AvalancheInteractiveTools",
				"AvalancheLevelViewport",
				"AvalancheMedia",
				"AvalancheMediaEditor",
				"AvalancheModifiers",
				"AvalancheOutliner",
				"AvalanchePropertyAnimator",
				"AvalancheRemoteControlEditor",
				"AvalancheSceneRig",
				"AvalancheSceneRigEditor",
				"AvalancheSceneTree",
				"AvalancheSequence",
				"AvalancheSequencer",
				"AvalancheShapes",
				"AvalancheTag",
				"AvalancheText",
				"AvalancheTransition",
				"AvalancheTransitionEditor",
				"AvalancheViewport",
				"BlueprintGraph",
				"CinematicCamera",
				"ClonerEffector",
				"CommonMenuExtensions",
				"ContentBrowser",
				"CustomDetailsView",
				"DeveloperSettings",
				"DynamicMaterial",
				"DynamicMaterialEditor",
				"EditorFramework",
				"EditorWidgets",
				"Engine",
				"GeometryAlgorithms",
				"GeometryCore",
				"GraphEditor",
				"InputCore",
				"InteractiveToolsFramework",
				"Kismet",
				"KismetCompiler",
				"Landscape",
				"LevelEditor",
				"LevelSequence",
				"LevelSequenceEditor",
				"MaterialEditor",
				"MediaCompositing",
				"MediaPlate",
				"ModelingEditorUI",
				"MovieScene",
				"OperatorStackEditor",
				"PlacementMode",
				"Projects",
				"PropertyAnimatorCore",
				"PropertyEditor",
				"RHI",
				"RawMesh",
				"RemoteControl",
				"RemoteControlComponents",
				"RemoteControlUI",
				"RenderCore",
				"SVGImporter",
				"SVGImporterEditor",
				"SceneOutliner",
				"Sequencer",
				"Slate",
				"SlateCore",
				"StateTreeEditorModule",
				"StateTreeModule",
				"StatusBar",
				"SubobjectDataInterface",
				"SubobjectEditor",
				"Text3D",
				"ToolMenus",
				"ToolWidgets",
				"TypedElementFramework",
				"TypedElementRuntime",
				"UElibPNG",
				"UMG",
				"UMGEditor",
				"UnrealEd",
				"XmlParser",
				"zlib",
			}
		);

		if (Target.Type != TargetType.Server)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target,
				new string[]
				{
					"FreeType2"
				});
		}
	}
}
