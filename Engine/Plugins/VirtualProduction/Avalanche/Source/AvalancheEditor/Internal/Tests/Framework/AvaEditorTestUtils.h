// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Math/MathFwd.h"
#include "Tests/Framework/AvaTestUtils.h"

// General Motion Design Editor unit test log
DECLARE_LOG_CATEGORY_EXTERN(LogAvaEditorTest, Log, All);

class AActor;

struct AVALANCHEEDITOR_API FAvaEditorTestUtils : FAvaTestUtils
{
	TArray<FVector> GetActorLocations(const TArray<AActor*>& InActors);
	void SetActorLocations(const TArray<AActor*>& InActors, const TArray<FVector>& InLocations);

	TArray<FVector> GenerateRandomVectors(int32 InVectorCount);
	int32 GetLowestAxisActorForCamera(const TArray<FVector>& InActorLocations, const FVector& InCameraLocation, int32 InAlignmentAxis);
	int32 GetHighestAxisActorForCamera(const TArray<FVector>& InActorLocations, const FVector& InCameraLocation, int32 InAlignmentAxis);
	int32 GetLowestAxisActor(const TArray<FVector>& InActorLocations, int32 InAlignmentAxis);
	int32 GetHighestAxisActor(const TArray<FVector>& InActorLocations, int32 InAlignmentAxis);
	FVector GetCenterAnchorLocation(const TArray<FVector>& InActorLocations, const FVector& InCameraLocation, int32 InAlignmentAxis);
	double GetExpectedAxisValue(const FVector& InAnchorActorLocation, const FVector& InTargetActorLocation, double InCameraLocationX, int32 AlignmentAxis);

private:
	double GetRandomValue(int32 InMinValue = -200, int32 InMaxValue = 200);
	double CalculateTangentAngle(double InCameraDepthLocation, const FVector& InActorLocation, int32 InAlignmentAxis);

	/** @param bInIsLowestAxisValue defines wether we searching for the lowest or highest axis value */
	int32 GetHighestTangentAngleActorIndex(const TArray<FVector>& InActorLocations, const FVector& InCameraLocation, int32 InEdgeActorIndex, bool bInIsLowestAxisValue, int32 InAlignmentAxis);
	int32 GetLowestTangentAngleActorIndex(const TArray<FVector>& InActorLocations, const FVector& InCameraLocation, int32 InEdgeActorIndex, bool bInIsLowestAxisValue, int32 InAlignmentAxis);
};
