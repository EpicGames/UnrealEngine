// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaAdvancedRenamerExtension.h"
#include "Algo/Transform.h"
#include "AvaEditorCommands.h"
#include "EditorModeManager.h"
#include "EngineUtils.h"
#include "IAdvancedRenamerModule.h"
#include "Selection.h"
#include "Toolkits/IToolkitHost.h"
#include "UObject/WeakObjectPtr.h"

void FAvaAdvancedRenamerExtension::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	FAvaEditorExtension::BindCommands(InCommandList);

	const FAvaEditorCommands& AvaEditorCommands = FAvaEditorCommands::Get();

	InCommandList->MapAction(
		AvaEditorCommands.OpenAdvancedRenamerTool_SelectedActors,
		FExecuteAction::CreateSP(this, &FAvaAdvancedRenamerExtension::OpenAdvancedRenamerTool_SelectedActors),
		FCanExecuteAction::CreateSP(this, &FAvaAdvancedRenamerExtension::CanOpenAdvancedRenamerTool_SelectedActors)
	);

	InCommandList->MapAction(
		AvaEditorCommands.OpenAdvancedRenamerTool_SharedClassActors,
		FExecuteAction::CreateSP(this, &FAvaAdvancedRenamerExtension::OpenAdvancedRenamerTool_SharedClassActors),
		FCanExecuteAction::CreateSP(this, &FAvaAdvancedRenamerExtension::CanOpenAdvancedRenamerTool_ClassActors)
	);
}

bool FAvaAdvancedRenamerExtension::CanOpenAdvancedRenamerTool_SelectedActors() const
{
	TArray<AActor*> SelectedActors = GetSelectedActors();

	if (SelectedActors.IsEmpty())
	{
		return false;
	}

	for (AActor* Actor : SelectedActors)
	{
		// rather than denying rename if there's a single item that isn't an actor or not able to rename,
		// allow rename if there's at least 1 item fitting this criteria (since the provider itself handles items that don't fit the criteria)
		if (IsValid(Actor))
		{
			return true;
		}
	}

	return false;
}

void FAvaAdvancedRenamerExtension::OpenAdvancedRenamerTool_SelectedActors() const
{
	OpenAdvancedRenamerTool(GetSelectedActors());
}

bool FAvaAdvancedRenamerExtension::CanOpenAdvancedRenamerTool_ClassActors() const
{
	// Selecting only actors that can't be renamed should not let the user rename related actors as this is confusing behavior.
	return CanOpenAdvancedRenamerTool_SelectedActors();
}

void FAvaAdvancedRenamerExtension::OpenAdvancedRenamerTool_SharedClassActors() const
{
	TArray<AActor*> SelectedActors = GetSelectedActors();
	SelectedActors = IAdvancedRenamerModule::Get().GetActorsSharingClassesInWorld(SelectedActors);
	
	OpenAdvancedRenamerTool(SelectedActors);
}

TArray<AActor*> FAvaAdvancedRenamerExtension::GetSelectedActors() const
{
	TSharedPtr<IAvaEditor> Editor = GetEditor();

	if (!Editor)
	{
		return {};
	}

	TSharedPtr<IToolkitHost> ToolkitHost = Editor->GetToolkitHost();

	if (!ToolkitHost.IsValid())
	{
		return {};
	}

	USelection* ActorSelection = ToolkitHost->GetEditorModeManager().GetSelectedActors();

	if (!ActorSelection)
	{
		return {};
	}

	TArray<AActor*> SelectedActors;
	int32 Count = ActorSelection->GetSelectedObjects<AActor>(SelectedActors);

	return SelectedActors;
}

void FAvaAdvancedRenamerExtension::OpenAdvancedRenamerTool(const TArray<AActor*>& InActors) const
{
	TSharedPtr<SWidget> HostWindow;

	if (TSharedPtr<IAvaEditor> Editor = GetEditor())
	{
		if (TSharedPtr<IToolkitHost> ToolkitHost = Editor->GetToolkitHost())
		{
			HostWindow = ToolkitHost->GetParentWidget();
		}
	}

	IAdvancedRenamerModule::Get().OpenAdvancedRenamerForActors(InActors, HostWindow);
}
