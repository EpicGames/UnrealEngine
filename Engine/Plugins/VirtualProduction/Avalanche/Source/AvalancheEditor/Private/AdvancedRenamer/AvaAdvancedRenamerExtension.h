// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaEditorExtension.h"

class UDynamicMaterialInstance;
class UDynamicMaterialModel;
struct FDMObjectMaterialProperty;

class FAvaAdvancedRenamerExtension : public FAvaEditorExtension
{
public:
	UE_AVA_INHERITS(FAvaAdvancedRenamerExtension, FAvaEditorExtension);

	//~ Begin IAvaEditorExtension
	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandList) override;
	//~ End IAvaEditorExtension

private:
	bool CanOpenAdvancedRenamerTool_SelectedActors() const;

	void OpenAdvancedRenamerTool_SelectedActors() const;

	bool CanOpenAdvancedRenamerTool_ClassActors() const;

	void OpenAdvancedRenamerTool_SharedClassActors() const;

	TArray<AActor*> GetSelectedActors() const;

	void OpenAdvancedRenamerTool(const TArray<AActor*>& InActors) const;
};
