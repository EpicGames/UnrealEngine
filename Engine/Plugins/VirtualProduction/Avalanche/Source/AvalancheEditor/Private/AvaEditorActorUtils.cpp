// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEditorActorUtils.h"
#include "AvaEditorSettings.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"

#define LOCTEXT_NAMESPACE "AvaEditorActorUtils"

void FAvaEditorActorUtils::GetActorsToEdit(TArray<AActor*>& InOutSelectedActors)
{
	if (GetDefault<UAvaEditorSettings>()->bAutoIncludeAttachedActorsInEditActions)
	{
		TSet<AActor*> SelectedActors(InOutSelectedActors);
		TArray<AActor*> ActorsRemaining(InOutSelectedActors);

		while (ActorsRemaining.Num() > 0)
		{
			if (AActor* const Actor = ActorsRemaining.Pop())
			{
				TArray<AActor*> AttachedActors;

				Actor->GetAttachedActors(AttachedActors, true, true);
				ActorsRemaining.Append(AttachedActors);

				const int32 Index = InOutSelectedActors.Find(Actor);
				check(Index != INDEX_NONE);

				// Reverse Attached Actors since they're emplaced at a fixed index so that they appear in-order as they were gotten
				Algo::Reverse(AttachedActors);
				for (AActor* const AttachedActor : AttachedActors)
				{
					// Make sure Attached Actor isn't already in list
					if (!SelectedActors.Contains(AttachedActor))
					{
						SelectedActors.Add(AttachedActor);
						InOutSelectedActors.EmplaceAt(Index + 1, AttachedActor);
					}
				}
			}
		}
	}
}

void FAvaEditorActorUtils::GetAttachedComponents(const USceneComponent* InParent, TSet<USceneComponent*>& OutAttachedComponents)
{
	if (!InParent)
	{
		return;
	}

	AActor* ParentOwner = InParent->GetOwner();
	TArray<USceneComponent*> Children = InParent->GetAttachChildren();

	for (USceneComponent* Child : Children)
	{
		if (Child->GetOwner() != ParentOwner)
		{
			continue;
		}

		OutAttachedComponents.Add(Child);
		GetAttachedComponents(Child, OutAttachedComponents);
	}
}

void FAvaEditorActorUtils::GetDirectlyAttachedActors(const USceneComponent* InParent, TSet<AActor*>& OutAttachedActors)
{
	if (!InParent)
	{
		return;
	}

	AActor* ParentOwner = InParent->GetOwner();
	TArray<USceneComponent*> Children = InParent->GetAttachChildren();

	for (USceneComponent* Child : Children)
	{
		AActor* ChildOwner = Child->GetOwner();

		if (ChildOwner == ParentOwner)
		{
			continue;
		}

		OutAttachedActors.Add(ChildOwner);
	}
}

void FAvaEditorActorUtils::GetAllAttachedActors(const USceneComponent* InParent, TSet<AActor*>& OutAttachedActors)
{
	TSet<USceneComponent*> SceneComponents;
	SceneComponents.Add(const_cast<USceneComponent*>(InParent));
	GetAttachedComponents(InParent, SceneComponents);

	for (USceneComponent* SceneComponent : SceneComponents)
	{
		GetDirectlyAttachedActors(SceneComponent, OutAttachedActors);
	}
}

namespace UE::AvaEditor::Private
{
	bool IsComponentChildOf(const USceneComponent* InSceneComponent, const USceneComponent* InPossibleParent)
	{
		if (!InSceneComponent || !InPossibleParent)
		{
			return false;
		}

		if (InSceneComponent == InPossibleParent)
		{
			return true;
		}

		USceneComponent* ActualParent = InSceneComponent->GetAttachParent();

		if (!ActualParent)
		{
			return false;
		}

		if (ActualParent == InPossibleParent)
		{
			return true;
		}

		return IsComponentChildOf(ActualParent, InPossibleParent);
	}
}

void FAvaEditorActorUtils::GetDirectlyAttachedActors(const TSet<USceneComponent*>& InComponentList, TSet<AActor*>& OutAttachedActors)
{
	for (USceneComponent* Component : InComponentList)
	{
		GetDirectlyAttachedActors(Component, OutAttachedActors);
	}
}

void FAvaEditorActorUtils::GetAllAttachedActors(const TSet<USceneComponent*>& InComponentList, TSet<AActor*>& OutAttachedActors)
{
	for (USceneComponent* Component : InComponentList)
	{
		bool bIsChild = false;

		// If Component is the child of another selected component, we can skip it.
		// The parent component will build the hierarchy itself.
		for (USceneComponent* ComponentInner : InComponentList)
		{
			if (ComponentInner == Component)
			{
				continue;
			}

			if (UE::AvaEditor::Private::IsComponentChildOf(Component, ComponentInner))
			{
				bIsChild = true;
				break;
			}
		}

		if (bIsChild)
		{
			continue;
		}

		GetAllAttachedActors(Component, OutAttachedActors);
	}
}

#undef LOCTEXT_NAMESPACE
