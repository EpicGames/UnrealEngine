// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"

class AActor;
class FViewport;
class USceneComponent;

struct FAvaEditorActorUtils
{
	static void GetActorsToEdit(TArray<AActor*>& InOutSelectedActors);

	/** Fills the OutAttachedComponents array with all the components attached to the given component within the same actor. */
	static void GetAttachedComponents(const USceneComponent* InParent, TSet<USceneComponent*>& OutAttachedComponents);

	/** Returns the actors directly attached to this component. */
	static void GetDirectlyAttachedActors(const USceneComponent* InParent, TSet<AActor*>& OutAttachedActors);

	/** Returns all the actors attached directly or indirectly to this component. */
	static void GetAllAttachedActors(const USceneComponent* InParent, TSet<AActor*>& OutAttachedActors);

	/** Returns all actors attached directly to the given components. */
	static void GetDirectlyAttachedActors(const TSet<USceneComponent*>& InComponentList, TSet<AActor*>& OutAttachedActors);

	/** Returns all actors attached to these components directly or indirectly. */
	static void GetAllAttachedActors(const TSet<USceneComponent*>& InComponentList, TSet<AActor*>& OutAttachedActors);
};
