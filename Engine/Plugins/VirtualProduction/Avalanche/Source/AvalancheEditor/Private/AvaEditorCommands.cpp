// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEditorCommands.h"
#include "AvaEditorStyle.h"
#include "MediaPlate.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "AvaEditorCommands"

FAvaEditorCommands::FAvaEditorCommands()
	: TCommands<FAvaEditorCommands>(TEXT("AvaEditor")
		, LOCTEXT("MotionDesignEditor", "Motion Design Editor")
		, NAME_None
		, FAvaEditorStyle::Get().GetStyleSetName()
	)
{
}

void FAvaEditorCommands::RegisterCommands()
{
	RegisterViewportCommands();
	RegisterPivotCommands();
	RegisterToolsCommands();
	RegisterAdvancedRenamerCommands();
	RegisterClonerEffectorCommands();
}

void FAvaEditorCommands::RegisterViewportCommands()
{
	UI_COMMAND(SwitchViewports
		, "Switch Viewports"
		, "Switch between the Camera Viewport and the 3D Viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord(EKeys::C));

	UI_COMMAND(SetMotionDesignViewportType
		, "Motion Design Viewport"
		, "Viewport layout to view a fixed camera in an Motion Design Scene"
		, EUserInterfaceActionType::RadioButton
		, FInputChord());

	UI_COMMAND(GroupActors
		, "Group Actors"
		, "Groups the selected actors under a Null Actor."
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Control, EKeys::G));

	UI_COMMAND(UngroupActors
		, "Ungroup Actors"
		, "Ungroups the selected null actors."
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Shift, EKeys::G));
}

void FAvaEditorCommands::RegisterPivotCommands()
{
	UI_COMMAND(PivotTopLeftActor
		, "Set Pivot Top Left (Actor)"
		, "Moves the pivot of the currently selected actor(s) to the top left of the actor."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadSeven, EModifierKey::Alt))

	UI_COMMAND(PivotTopMiddleActor
		, "Set Pivot Top (Actor)"
		, "Moves the pivot of the currently selected actor(s) to the top middle of the actor."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadEight, EModifierKey::Alt))

	UI_COMMAND(PivotTopRightActor
		, "Set Pivot Top Right (Actor)"
		, "Moves the pivot of the currently selected actor(s) to the top right of the actor."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadNine, EModifierKey::Alt))

	UI_COMMAND(PivotMiddleLeftActor
		, "Set Pivot Middle Left (Actor)"
		, "Moves the pivot of the currently selected actor(s) to the middle left of the actor."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadFour, EModifierKey::Alt))

	UI_COMMAND(PivotCenterActor
		, "Set Pivot Center (Actor)"
		, "Moves the pivot of the currently selected actor(s) to the center of the actor."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadFive, EModifierKey::Alt))

	UI_COMMAND(PivotMiddleRightActor
		, "Set Pivot Middle Right (Actor)"
		, "Moves the pivot of the currently selected actor(s) to the middle right of the actor."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadSix, EModifierKey::Alt))

	UI_COMMAND(PivotBottomLeftActor
		, "Set Pivot Bottom Left (Actor)"
		, "Moves the pivot of the currently selected actor(s) to the bottom left of the actor."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadOne, EModifierKey::Alt))

	UI_COMMAND(PivotBottomMiddleActor
		, "Set Pivot Bottom Middle (Actor)"
		, "Moves the pivot of the currently selected actor(s) to the bottom middle of the actor."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadTwo, EModifierKey::Alt))

	UI_COMMAND(PivotBottomRightActor
		, "Set Pivot Bottom Right (Actor)"
		, "Moves the pivot of the currently selected actor(s) to the bottom right of the actor."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadThree, EModifierKey::Alt))

	UI_COMMAND(PivotDepthFrontActor
		, "Set Pivot Front (Depth, Actor)"
		, "Moves the pivot of the currently selected actor(s) to the front of the actor (depth only)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Divide, EModifierKey::Alt))

	UI_COMMAND(PivotDepthMiddleActor
		, "Set Pivot Middle (Depth, Actor)"
		, "Moves the pivot of the currently selected actor(s) to the middle of the actor (depth only)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Multiply, EModifierKey::Alt))

	UI_COMMAND(PivotDepthBackActor
		, "Set Pivot Back (Depth, Actor)"
		, "Moves the pivot of the currently selected actor(s) to the back of the actor (depth only)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Subtract, EModifierKey::Alt))

	UI_COMMAND(PivotTopLeftActorAndChildren
		, "Set Pivot Top Left (Actor)"
		, "Moves the pivot of the currently selected actor(s) to the top left of the actor and its children."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadSeven, EModifierKey::Control))

	UI_COMMAND(PivotTopMiddleActorAndChildren
		, "Set Pivot Top (Actor and Children)"
		, "Moves the pivot of the currently selected actor(s) to the top middle of the actor and its children."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadEight, EModifierKey::Control))

	UI_COMMAND(PivotTopRightActorAndChildren
		, "Set Pivot Top Right (Actor and Children)"
		, "Moves the pivot of the currently selected actor(s) to the top right of the actor and its children."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadNine, EModifierKey::Control))

	UI_COMMAND(PivotMiddleLeftActorAndChildren
		, "Set Pivot Middle Left (Actor and Children)"
		, "Moves the pivot of the currently selected actor(s) to the middle left of the actor and its children."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadFour, EModifierKey::Control))

	UI_COMMAND(PivotCenterActorAndChildren
		, "Set Pivot Center (Actor and Children)"
		, "Moves the pivot of the currently selected actor(s) to the center of the actor and its children."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadFive, EModifierKey::Control))

	UI_COMMAND(PivotMiddleRightActorAndChildren
		, "Set Pivot Middle Right (Actor and Children)"
		, "Moves the pivot of the currently selected actor(s) to the middle right of the actor and its children."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadSix, EModifierKey::Control))

	UI_COMMAND(PivotBottomLeftActorAndChildren
		, "Set Pivot Bottom Left (Actor and Children)"
		, "Moves the pivot of the currently selected actor(s) to the bottom left of the actor and its children."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadOne, EModifierKey::Control))

	UI_COMMAND(PivotBottomMiddleActorAndChildren
		, "Set Pivot Bottom Middle (Actor and Children)"
		, "Moves the pivot of the currently selected actor(s) to the bottom middle of the actor and its children."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadTwo, EModifierKey::Control))

	UI_COMMAND(PivotBottomRightActorAndChildren
		, "Set Pivot Bottom Right (Actor and Children)"
		, "Moves the pivot of the currently selected actor(s) to the bottom right of the actor and its children."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadThree, EModifierKey::Control))

	UI_COMMAND(PivotDepthFrontActorAndChildren
		, "Set Pivot Front (Depth, Actor and Children)"
		, "Moves the pivot of the currently selected actor(s) to the front of the actor and its children (depth only)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Divide, EModifierKey::Control))

	UI_COMMAND(PivotDepthMiddleActorAndChildren
		, "Set Pivot Middle (Depth, Actor and Children)"
		, "Moves the pivot of the currently selected actor(s) to the middle of the actor and its children (depth only)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Multiply, EModifierKey::Control))

	UI_COMMAND(PivotDepthBackActorAndChildren
		, "Set Pivot Back (Depth, Actor and Children)"
		, "Moves the pivot of the currently selected actor(s) to the back of the actor and its children (depth only)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Subtract, EModifierKey::Control))

	UI_COMMAND(PivotTopLeftSelection
		, "Set Pivot Top Left (Selection)"
		, "Moves the pivot of the currently selected actor(s) to the top left of the combined selected actor and children bounds."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadSeven))

	UI_COMMAND(PivotTopMiddleSelection
		, "Set Pivot Top (Selection)"
		, "Moves the pivot of the currently selected actor(s) to the top middle of the combined selected actor and children bounds."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadEight))

	UI_COMMAND(PivotTopRightSelection
		, "Set Pivot Top Right (Selection)"
		, "Moves the pivot of the currently selected actor(s) to the top right of the combined selected actor and children bounds."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadNine))

	UI_COMMAND(PivotMiddleLeftSelection
		, "Set Pivot Middle Left (Selection)"
		, "Moves the pivot of the currently selected actor(s) to the middle left of the combined selected actor and children bounds."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadFour))

	UI_COMMAND(PivotCenterSelection
		, "Set Pivot Center (Selection)"
		, "Moves the pivot of the currently selected actor(s) to the center of the combined selected actor and children bounds."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadFive))

	UI_COMMAND(PivotMiddleRightSelection
		, "Set Pivot Middle Right (Selection)"
		, "Moves the pivot of the currently selected actor(s) to the middle right of the combined selected actor and children bounds."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadSix))

	UI_COMMAND(PivotBottomLeftSelection
		, "Set Pivot Bottom Left (Selection)"
		, "Moves the pivot of the currently selected actor(s) to the bottom left of the combined selected actor and children bounds."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadOne))

	UI_COMMAND(PivotBottomMiddleSelection
		, "Set Pivot Bottom Middle (Selection)"
		, "Moves the pivot of the currently selected actor(s) to the bottom middle of the combined selected actor and children bounds."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadTwo))

	UI_COMMAND(PivotBottomRightSelection
		, "Set Pivot Bottom Right (Selection)"
		, "Moves the pivot of the currently selected actor(s) to the bottom right of the combined selected actor and children bounds."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::NumPadThree))

	UI_COMMAND(PivotDepthFrontSelection
		, "Set Pivot Front (Depth, Selection)"
		, "Moves the pivot of the currently selected actor(s) to the front of the combined selected actor and children bounds (depth only)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Divide))

	UI_COMMAND(PivotDepthMiddleSelection
		, "Set Pivot Middle (Depth, Selection)"
		, "Moves the pivot of the currently selected actor(s) to the middle of the combined selected actor and children bounds (depth only)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Multiply))

	UI_COMMAND(PivotDepthBackSelection
		, "Set Pivot Back (Depth, Selection)"
		, "Moves the pivot of the currently selected actor(s) to the back of the combined selected actor and children bounds (depth only)."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Subtract))
}

void FAvaEditorCommands::RegisterToolsCommands()
{
	UI_COMMAND(StaticMeshToolsCategory
		, "Meshes"
		, "Tools for creating static meshes in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(CameraToolsCategory
		, "Cameras"
		, "Tools for creating camera and cinematic actors in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(LightsToolsCategory
		, "Lights"
		, "Tools for creating lights in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(CubeTool
		, "Cube"
		, "Create a cube actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(SphereTool
		, "Sphere"
		, "Create a sphere actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(CylinderTool
		, "Cylinder"
		, "Create a cylinder actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(ConeTool
		, "Cone"
		, "Create a cone actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(PlaneTool
		, "Plane"
		, "Create a plane actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(CycloramaTool
		, "Cyclorama"
		, "Create a cyclorama actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	FUICommandInfo::MakeCommandInfo(AsShared()
		, MediaPlateTool
		, TEXT("MediaPlateTool")
		, LOCTEXT("MediaPlateTool", "Media Plate")
		, LOCTEXT("MediaPlateTool_Tooltip", "Create a Media Plate actor in the viewport.")
		, FSlateIconFinder::FindIconForClass(AMediaPlate::StaticClass())
		, EUserInterfaceActionType::ToggleButton
		, FInputChord()
	);

	UI_COMMAND(CameraTool
		, "Camera"
		, "Create a camera actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(CineCameraTool
		, "Cine Camera"
		, "Create a cine camera actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(CameraRigCraneTool
		, "Camera Rig Crane"
		, "Create a camera rig crane actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(CameraRigRailTool
		, "Camera Rig Rail"
		, "Create a camera rig rail actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(CameraShakeSourceTool
		, "Camera Shake Source"
		, "Create a camera shake source actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(AvaPostProcessVolumeTool
		, "Ava Post Process Volume"
		, "Create a Motion Design post process volume in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	FUICommandInfo::MakeCommandInfo(AsShared()
		, TickerTool
		, TEXT("TickerTool")
		, LOCTEXT("TickerTool", "Ticker")
		, LOCTEXT("TickerTool_Tooltip", "Create a Motion Design Ticker actor in the viewport.")
		, FSlateIconFinder::FindIconForClass(AActor::StaticClass())
		, EUserInterfaceActionType::ToggleButton
		, FInputChord()
	);

	UI_COMMAND(PointLightTool
		, "Point Light"
		, "Create a point light actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(DirectionalLightTool
		, "Directional Light"
		, "Create a directional light actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(SpotLightTool
		, "Spot Light"
		, "Create a spot light actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(RectLightTool
		, "Rect Light"
		, "Create a rect light actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(SkyLightTool
		, "Sky Light"
		, "Create a sky light actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());
}

void FAvaEditorCommands::RegisterAdvancedRenamerCommands()
{
	UI_COMMAND(OpenAdvancedRenamerTool_SelectedActors
		, "Rename Selected Actors"
		, "Opens the Advanced Renamer Panel to rename all selected actors."
		, EUserInterfaceActionType::Button
		, FInputChord())

	UI_COMMAND(OpenAdvancedRenamerTool_SharedClassActors
		, "Rename Actors of Selected Actor Classes"
		, "Opens the Advanced Renamer Panel to rename all actors sharing a class with any selected actor."
		, EUserInterfaceActionType::Button
		, FInputChord())
}

void FAvaEditorCommands::RegisterClonerEffectorCommands()
{
	UI_COMMAND(DisableEffectors
		, "Disable Effectors"
		, "Disable effectors on the selected actors in the level"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::E, EModifierKey::Alt))

	UI_COMMAND(EnableEffectors
		, "Enable Effectors"
		, "Enable effectors on the selected actors in the level"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::E, EModifierKey::Shift))

	UI_COMMAND(DisableCloners
		, "Disable Cloners"
		, "Disable cloners on the selected actors in the level"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::C, EModifierKey::Alt))

	UI_COMMAND(EnableCloners
		, "Enable Cloners"
		, "Enable cloners on the selected actors in the level"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::C, EModifierKey::Shift))

	UI_COMMAND(CreateCloner
		, "Create Cloner"
		, "Create new cloner with selected actors in the level"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::C, EModifierKey::Control | EModifierKey::Alt))
}

#undef LOCTEXT_NAMESPACE
