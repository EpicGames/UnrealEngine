// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FAvaEditorCommands : public TCommands<FAvaEditorCommands>
{
public:
	FAvaEditorCommands();

	virtual void RegisterCommands() override;

	void RegisterViewportCommands();
	void RegisterPivotCommands();
	void RegisterToolsCommands();
	void RegisterAdvancedRenamerCommands();
	void RegisterClonerEffectorCommands();

	/** Switches between Camera "2D" viewport and 3D viewport. */
	TSharedPtr<FUICommandInfo> SwitchViewports;
	TSharedPtr<FUICommandInfo> SetMotionDesignViewportType;

	/** Groups the Selected Actors via a Null Actor. */
	TSharedPtr<FUICommandInfo> GroupActors;
	TSharedPtr<FUICommandInfo> UngroupActors;

	// Pivot
	TSharedPtr<FUICommandInfo> PivotTopLeftActor;
	TSharedPtr<FUICommandInfo> PivotTopMiddleActor;
	TSharedPtr<FUICommandInfo> PivotTopRightActor;
	TSharedPtr<FUICommandInfo> PivotMiddleLeftActor;
	TSharedPtr<FUICommandInfo> PivotCenterActor;
	TSharedPtr<FUICommandInfo> PivotMiddleRightActor;
	TSharedPtr<FUICommandInfo> PivotBottomLeftActor;
	TSharedPtr<FUICommandInfo> PivotBottomMiddleActor;
	TSharedPtr<FUICommandInfo> PivotBottomRightActor;
	TSharedPtr<FUICommandInfo> PivotDepthFrontActor;
	TSharedPtr<FUICommandInfo> PivotDepthMiddleActor;
	TSharedPtr<FUICommandInfo> PivotDepthBackActor;

	TSharedPtr<FUICommandInfo> PivotTopLeftActorAndChildren;
	TSharedPtr<FUICommandInfo> PivotTopMiddleActorAndChildren;
	TSharedPtr<FUICommandInfo> PivotTopRightActorAndChildren;
	TSharedPtr<FUICommandInfo> PivotMiddleLeftActorAndChildren;
	TSharedPtr<FUICommandInfo> PivotCenterActorAndChildren;
	TSharedPtr<FUICommandInfo> PivotMiddleRightActorAndChildren;
	TSharedPtr<FUICommandInfo> PivotBottomLeftActorAndChildren;
	TSharedPtr<FUICommandInfo> PivotBottomMiddleActorAndChildren;
	TSharedPtr<FUICommandInfo> PivotBottomRightActorAndChildren;
	TSharedPtr<FUICommandInfo> PivotDepthFrontActorAndChildren;
	TSharedPtr<FUICommandInfo> PivotDepthMiddleActorAndChildren;
	TSharedPtr<FUICommandInfo> PivotDepthBackActorAndChildren;

	TSharedPtr<FUICommandInfo> PivotTopLeftSelection;
	TSharedPtr<FUICommandInfo> PivotTopMiddleSelection;
	TSharedPtr<FUICommandInfo> PivotTopRightSelection;
	TSharedPtr<FUICommandInfo> PivotMiddleLeftSelection;
	TSharedPtr<FUICommandInfo> PivotCenterSelection;
	TSharedPtr<FUICommandInfo> PivotMiddleRightSelection;
	TSharedPtr<FUICommandInfo> PivotBottomLeftSelection;
	TSharedPtr<FUICommandInfo> PivotBottomMiddleSelection;
	TSharedPtr<FUICommandInfo> PivotBottomRightSelection;
	TSharedPtr<FUICommandInfo> PivotDepthFrontSelection;
	TSharedPtr<FUICommandInfo> PivotDepthMiddleSelection;
	TSharedPtr<FUICommandInfo> PivotDepthBackSelection;

	// Tools
	TSharedPtr<FUICommandInfo> StaticMeshToolsCategory;
	TSharedPtr<FUICommandInfo> CameraToolsCategory;
	TSharedPtr<FUICommandInfo> LightsToolsCategory;

	TSharedPtr<FUICommandInfo> CubeTool;
	TSharedPtr<FUICommandInfo> SphereTool;
	TSharedPtr<FUICommandInfo> CylinderTool;
	TSharedPtr<FUICommandInfo> ConeTool;
	TSharedPtr<FUICommandInfo> PlaneTool;
	TSharedPtr<FUICommandInfo> CycloramaTool;

	TSharedPtr<FUICommandInfo> MediaPlateTool;

	TSharedPtr<FUICommandInfo> CameraTool;
	TSharedPtr<FUICommandInfo> CineCameraTool;
	TSharedPtr<FUICommandInfo> CameraRigCraneTool;
	TSharedPtr<FUICommandInfo> CameraRigRailTool;
	TSharedPtr<FUICommandInfo> CameraShakeSourceTool;
	TSharedPtr<FUICommandInfo> AvaPostProcessVolumeTool;
	TSharedPtr<FUICommandInfo> TickerTool;

	TSharedPtr<FUICommandInfo> PointLightTool;
	TSharedPtr<FUICommandInfo> DirectionalLightTool;
	TSharedPtr<FUICommandInfo> SpotLightTool;
	TSharedPtr<FUICommandInfo> RectLightTool;
	TSharedPtr<FUICommandInfo> SkyLightTool;

	// Advanced Renamer
	TSharedPtr<FUICommandInfo> OpenAdvancedRenamerTool_SelectedActors;
	TSharedPtr<FUICommandInfo> OpenAdvancedRenamerTool_SharedClassActors;

	// ClonerEffector
	TSharedPtr<FUICommandInfo> DisableCloners;
	TSharedPtr<FUICommandInfo> EnableCloners;
	TSharedPtr<FUICommandInfo> DisableEffectors;
	TSharedPtr<FUICommandInfo> EnableEffectors;
	TSharedPtr<FUICommandInfo> CreateCloner;
};
