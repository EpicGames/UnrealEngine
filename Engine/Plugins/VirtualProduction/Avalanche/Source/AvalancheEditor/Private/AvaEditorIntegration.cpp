// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEditorIntegration.h"
#include "AdvancedRenamer/AvaAdvancedRenamerExtension.h"
#include "ClonerEffector/AvaClonerEffectorExtension.h"
#include "ComponentVisualizer/AvaComponentVisualizerExtension.h"
#include "DetailView/AvaDetailsExtension.h"
#include "EditorProvider/AvaEditorProvider.h"
#include "InteractiveTools/AvaInteractiveToolsExtension.h"
#include "LevelEditor/AvaLevelEditorBuilder.h"
#include "MaterialDesigner/AvaLevelMaterialDesignerExtension.h"
#include "OperatorStack/AvaOperatorStackExtension.h"
#include "Outliner/AvaOutlinerExtension.h"
#include "RemoteControl/AvaRCExtension.h"
#include "Scene/AvaSceneExtension.h"
#include "Scene/AvaSceneRigExtension.h"
#include "Selection/AvaSelectionProviderExtension.h"
#include "Sequencer/AvaLevelSequencerExtension.h"
#include "Transition/AvaTransitionExtension.h"
#include "Viewport/AvaLevelViewportExtension.h"
#include "Viewport/AvaViewportExtension.h"

TSharedRef<IAvaEditor> FAvaLevelEditorIntegration::BuildEditor()
{
	return FAvaLevelEditorBuilder()
		.SetIdentifier(TEXT("FAvaLevelEditorIntegration::BuildEditor"))
		.SetProvider<FAvaEditorProvider>()
		.AddExtension<FAvaSelectionProviderExtension>()
		.AddExtension<FAvaOutlinerExtension>()
		.AddExtension<FAvaSequencerExtension, FAvaLevelSequencerExtension>()
		.AddExtension<FAvaDetailsExtension>()
		.AddExtension<FAvaOperatorStackExtension>()
		.AddExtension<FAvaRCExtension>()
		.AddExtension<FAvaViewportExtension, FAvaLevelViewportExtension>()
		.AddExtension<FAvaInteractiveToolsExtension>()
		.AddExtension<FAvaLevelMaterialDesignerExtension>()
		.AddExtension<FAvaSceneExtension>()
		.AddExtension<FAvaSceneRigExtension>()
		.AddExtension<FAvaTransitionExtension>()
		.AddExtension<FAvaAdvancedRenamerExtension>()
		.AddExtension<FAvaComponentVisualizerExtension>()
		.AddExtension<FAvaClonerEffectorExtension>()
		.Build();
}
