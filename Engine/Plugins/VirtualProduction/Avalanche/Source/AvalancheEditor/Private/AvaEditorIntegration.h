// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

class IAvaEditor;

struct FAvaLevelEditorIntegration
{
	static TSharedRef<IAvaEditor> BuildEditor();
};
