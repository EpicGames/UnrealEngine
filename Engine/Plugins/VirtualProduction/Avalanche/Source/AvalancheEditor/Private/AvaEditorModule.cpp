// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEditorModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AvaEditorActorUtils.h"
#include "AvaEditorCommands.h"
#include "AvaEditorIntegration.h"
#include "AvaSceneRigSubsystem.h"
#include "AvaShapeActor.h"
#include "Camera/CameraActor.h"
#include "CineCameraActor.h"
#include "ColorPicker/AvaViewportColorPickerActorClassRegistry.h"
#include "ColorPicker/IAvaViewportColorPickerAdapter.h"
#include "Components/LightComponentBase.h"
#include "Containers/Set.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/DirectionalLight.h"
#include "Engine/Light.h"
#include "Engine/PointLight.h"
#include "Engine/PostProcessVolume.h"
#include "Engine/RectLight.h"
#include "Engine/SkyLight.h"
#include "Engine/SpotLight.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/AvaNullActor.h"
#include "IAssetTools.h"
#include "IAvaOutlinerModule.h"
#include "Icon/AvaOutlinerObjectIconCustomization.h"
#include "Item/AvaOutlinerActor.h"
#include "Materials/MaterialInterface.h"
#include "Misc/CoreDelegates.h"
#include "SVGImporter/AvaOutlinerSVGActorContextMenu.h"
#include "Styling/SlateIconFinder.h"
#include "UnrealEdGlobals.h"
#include "Viewport/AvaCineCameraActor.h"
#include "Viewport/AvaViewportQualitySettings.h"

// Details View
#include "AssetToolsModule.h"
#include "AvaSceneSettings.h"
#include "DetailView/Customizations/AvaCategoryHiderCustomization.h"
#include "DetailView/Customizations/AvaMeshesDetailCustomization.h"
#include "DetailView/Customizations/AvaSceneSettingsCustomization.h"
#include "DetailView/Customizations/AvaVectorPropertyTypeCustomization.h"
#include "DetailView/Customizations/AvaViewportQualitySettingsPropertyTypeCustomization.h"

DEFINE_LOG_CATEGORY(AvaLog);

#define LOCTEXT_NAMESPACE "AvalancheEditor"

struct FAvaViewportColorPickerLightAdapter : IAvaViewportColorPickerAdapter
{
	//~ Begin IAvaViewportColorPickerActorAdapter
	virtual bool GetColorData(const AActor& InActor, FAvaColorChangeData& OutColorData) const override
	{
		if (const ULightComponentBase* Component = InActor.FindComponentByClass<ULightComponentBase>())
		{
			OutColorData = { EAvaColorStyle::Solid, Component->GetLightColor(), FLinearColor::Black, /* bIsUnlit */ false };
			return true;
		}
		return false;
	}

	virtual void SetColorData(AActor& InActor, const FAvaColorChangeData& InColorData) const override
	{
		if (ULightComponentBase* Component = InActor.FindComponentByClass<ULightComponentBase>())
		{
			FProperty* LightColorProperty = Component->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ULightComponentBase, LightColor));

			Component->PreEditChange(LightColorProperty);

			Component->LightColor = InColorData.PrimaryColor.ToFColorSRGB();

			FPropertyChangedEvent PropertyChangedEvent = {
				LightColorProperty,
				EPropertyChangeType::Interactive
			};

			Component->PostEditChangeProperty(PropertyChangedEvent);
		}
	}
	//~ End IAvaViewportColorPickerActorAdapter
};

namespace UE::AvaEditor::Private
{
	static FString LevelTemplatesPath = FString::Printf(TEXT("/%hs/%s"), UE_PLUGIN_NAME, TEXT("LevelTemplates"));
	static FString DefaultLevelPath = FString::Printf(TEXT("/%hs/%s"), UE_PLUGIN_NAME, TEXT("DefaultMotionDesignLevel"));
	static FString DefaultLevelThumbnailPath = FString::Printf(TEXT("/%hs/%s"), UE_PLUGIN_NAME, TEXT("DefaultMotionDesignLevelThumbnail.DefaultMotionDesignLevelThumbnail"));
}

void FAvaEditorModule::StartupModule()
{
	FAvaEditorCommands::Register();

	// Add the menu subsection
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FAvaEditorModule::PostEngineInit);
	FCoreDelegates::OnPreExit.AddRaw(this, &FAvaEditorModule::PreExit);

	RegisterAssetTools();
	RegisterCustomLayouts();

	// Register Palettes
	using namespace UE::AvaEditor::Private;

	// Register Icon Customization
	IAvaOutlinerModule::Get().RegisterOverriddenIcon<FAvaOutlinerActor, FAvaOutlinerObjectIconCustomization>(AAvaShapeActor::StaticClass())
		.SetOverriddenIcon(FOnGetOverriddenObjectIcon::CreateStatic(&FAvaEditorModule::GetOutlinerShapeActorIcon));

	IAvaOutlinerModule::Get().GetOnExtendOutlinerItemContextMenu()
    	.AddStatic(&FAvaOutlinerSVGActorContextMenu::OnExtendOutlinerContextMenu);

	// Note: ASkylight Does not extend from ALight
	FAvaViewportColorPickerActorClassRegistry::RegisterClassAdapter<ALight, FAvaViewportColorPickerLightAdapter>();
	FAvaViewportColorPickerActorClassRegistry::RegisterClassAdapter<ASkyLight, FAvaViewportColorPickerLightAdapter>();

	UAvaSceneRigSubsystem::RegisterSupportedActorClasses(DefaultSceneRigActorClasses());
}

void FAvaEditorModule::ShutdownModule()
{
	FAvaEditorCommands::Unregister();

	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	FCoreDelegates::OnPreExit.RemoveAll(this);

	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		AssetRegistry->OnKnownGathersComplete().Remove(OnKnownGathersCompleteHandle);
		OnKnownGathersCompleteHandle.Reset();
	}

	if (UObjectInitialized() && !IsEngineExitRequested())
	{
		UnregisterCustomLayouts();

		if (IAvaOutlinerModule::IsLoaded())
		{
			IAvaOutlinerModule::Get().UnregisterOverriddenIcon<FAvaOutlinerActor>(AAvaShapeActor::StaticClass()->GetFName());
		}

		UAvaSceneRigSubsystem::UnregisterSupportedActorClasses(DefaultSceneRigActorClasses());
	}
}

void FAvaEditorModule::CreateAvaLevelEditor()
{
	AvaLevelEditor = FAvaLevelEditorIntegration::BuildEditor();
}

void FAvaEditorModule::PostEngineInit()
{
	CreateAvaLevelEditor();

	if (FSlateApplication::IsInitialized())
	{
		RegisterPropertyEditorCategories();

		if (IAssetRegistry* AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).TryGet())
		{
			// Asset Registry is still gathering assets, get notified when the Asset Registry is done scanning the current plugins to register template assets
			if (AssetRegistry->IsGathering())
			{
				OnKnownGathersCompleteHandle = AssetRegistry->OnKnownGathersComplete().AddRaw(this, &FAvaEditorModule::RegisterLevelTemplates);
			}
			// Asset Registry is done gathering assets so can register level templates immediately
			else
			{
				RegisterLevelTemplates();
			}
		}
	}
}

void FAvaEditorModule::PreExit()
{
	AvaLevelEditor.Reset();
}

FSlateIcon FAvaEditorModule::GetOutlinerShapeActorIcon(TSharedPtr<const FAvaOutlinerItem> InItem)
{
	if (!InItem.IsValid())
	{
		return FSlateIcon();
	}

	if (const FAvaOutlinerActor* ActorItem = InItem->CastTo<FAvaOutlinerActor>())
	{
		if (const AAvaShapeActor* ShapeActor = Cast<AAvaShapeActor>(ActorItem->GetActor()))
		{
			if (const UAvaShapeDynamicMeshBase* DynamicMesh = ShapeActor->GetDynamicMesh())
			{
				return FSlateIconFinder::FindIconForClass(DynamicMesh->GetClass());
			}
		}
	}

	return FSlateIcon();
}

void FAvaEditorModule::RegisterAssetTools()
{
	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();

	AssetTools.RegisterAdvancedAssetCategory(TEXT("MotionDesignCategory")
		, LOCTEXT("MotionDesignCategoryName", "Motion Design"));

}

void FAvaEditorModule::RegisterPropertyEditorCategories()
{
	static const FName PropertyEditor("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);

	TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("Object", "General", LOCTEXT("General", "General"));
	Section->AddCategory("Transform");
	Section->AddCategory("TransformCommon");
	Section->AddCategory("Mobility");

	Section = PropertyModule.FindOrCreateSection("Actor", "General", LOCTEXT("General", "General"));
	Section->AddCategory("Transform");
	Section->AddCategory("TransformCommon");
	Section->AddCategory("Mobility");

	// AvaShapeActor Sections
	{
		Section = PropertyModule.FindOrCreateSection("AvaShapeActor", "Shape", LOCTEXT("Shape", "Shape"));
		Section->AddCategory("Shape");

		Section = PropertyModule.FindOrCreateSection("AvaShapeActor", "Material", LOCTEXT("Material", "Material"));
		Section->AddCategory("Material");

		Section = PropertyModule.FindOrCreateSection("AvaShapeActor", "DynamicMesh", LOCTEXT("DynamicMesh", "Dynamic Mesh"));
		Section->AddCategory("DynamicMeshComponent");

		Section = PropertyModule.FindOrCreateSection("AvaShapeActor", "Rendering", LOCTEXT("Rendering", "Rendering"));
		Section->AddCategory("Rendering");
		Section->RemoveCategory("Lighting");
		Section->RemoveCategory("VirtualTexture");
		Section->RemoveCategory("MaterialParameters");
		Section->RemoveCategory("TextureStreaming");

		Section = PropertyModule.FindOrCreateSection("AvaShapeActor", "Lighting", LOCTEXT("Lighting", "Lighting"));
		Section->AddCategory("Lighting");
	}

	// AvaTextActor Sections
	{
		Section = PropertyModule.FindOrCreateSection("AvaTextActor", "Text", LOCTEXT("Text", "Text"));
		Section->AddCategory("Text");
		Section->AddCategory("TextAnimation");

		Section = PropertyModule.FindOrCreateSection("AvaTextActor", "Lighting", LOCTEXT("Lighting", "Lighting"));
		Section->AddCategory("Lighting");

		Section = PropertyModule.FindOrCreateSection("AvaTextActor", "Rendering", LOCTEXT("Rendering", "Rendering"));
		Section->AddCategory("Rendering");

		Section = PropertyModule.FindOrCreateSection("AvaTextActor", "Style", LOCTEXT("Style", "Style"));
		Section->AddCategory("Style");
		Section->AddCategory("Materials");

		Section = PropertyModule.FindOrCreateSection("AvaTextActor", "Geometry", LOCTEXT("Geometry", "Geometry"));
		Section->AddCategory("Geometry");

		Section = PropertyModule.FindOrCreateSection("AvaTextActor", "Layout", LOCTEXT("Layout", "Layout"));
		Section->AddCategory("Layout");
	}

	// AvaCineCameraActor Sections
	{
		Section = PropertyModule.FindOrCreateSection("AvaCineCameraActor", "Camera", LOCTEXT("Camera", "Camera"));
		Section->AddCategory("Camera");
		Section->AddCategory("CameraOptions");
		Section->AddCategory("CurrentCameraSettings");
	}

	// AvaTicker Sections
	{
		Section = PropertyModule.FindOrCreateSection("AvaTickerComponent", "Ticker", LOCTEXT("Ticker", "Ticker"));
		Section->AddCategory("Ticker");
	}
}

void FAvaEditorModule::RegisterCustomLayouts()
{
	static FName PropertyEditor("PropertyEditor");

	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);

	// Generic
	PropertyModule.RegisterCustomClassLayout(AAvaShapeActor::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(
		&FAvaCategoryHiderCustomization::MakeInstance));
	PropertyModule.RegisterCustomClassLayout(UAvaShapeDynamicMeshBase::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(
		&FAvaMeshesDetailCustomization::MakeInstance));

	PropertyModule.RegisterCustomClassLayout(UAvaSceneSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(
		&FAvaSceneSettingsCustomization::MakeDefaultInstance));

	const TSharedRef<FAvaVectorPropertyTypeIdentifier> VectorPropertyTypeIdentifier = MakeShared<FAvaVectorPropertyTypeIdentifier>();

	PropertyModule.RegisterCustomPropertyTypeLayout(TEXT("Vector"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(
		&FAvaVectorPropertyTypeCustomization::MakeInstance), VectorPropertyTypeIdentifier);
	PropertyModule.RegisterCustomPropertyTypeLayout(TEXT("Vector2D"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(
		&FAvaVectorPropertyTypeCustomization::MakeInstance), VectorPropertyTypeIdentifier);
	
	PropertyModule.RegisterCustomPropertyTypeLayout(FAvaViewportQualitySettings::StaticStruct()->GetFName()
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAvaViewportQualitySettingsPropertyTypeCustomization::MakeInstance));
}

void FAvaEditorModule::UnregisterCustomLayouts()
{
	static FName PropertyEditor("PropertyEditor");

	if (FModuleManager::Get().IsModuleLoaded(PropertyEditor))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);

		// Generic
		PropertyModule.UnregisterCustomClassLayout(AActor::StaticClass()->GetFName());
		PropertyModule.UnregisterCustomClassLayout(AAvaShapeActor::StaticClass()->GetFName());
		PropertyModule.UnregisterCustomClassLayout(UAvaShapeDynamicMeshBase::StaticClass()->GetFName());

		PropertyModule.UnregisterCustomClassLayout(UAvaSceneSettings::StaticClass()->GetFName());

		PropertyModule.UnregisterCustomPropertyTypeLayout(UMaterialInterface::StaticClass()->GetFName());

		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("Vector"));
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("Vector2D"));

		PropertyModule.UnregisterCustomPropertyTypeLayout(FAvaViewportQualitySettings::StaticStruct()->GetFName());
	}
}

void FAvaEditorModule::RegisterLevelTemplates()
{
	if (GUnrealEd)
	{
		// Register original template map,
		if (!GUnrealEd->IsTemplateMap(UE::AvaEditor::Private::DefaultLevelPath))
		{
			FTemplateMapInfo MapTemplate;
			MapTemplate.Category = TEXT("Motion Design");
			MapTemplate.DisplayName = LOCTEXT("Map Template", "Motion Design");
			MapTemplate.ThumbnailTexture = UE::AvaEditor::Private::DefaultLevelThumbnailPath;
			MapTemplate.Map = UE::AvaEditor::Private::DefaultLevelPath;
				
			GUnrealEd->AppendTemplateMaps({ MapTemplate });	
		}

		IAssetRegistry* AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).TryGet();
		if (AssetRegistry)
		{
			TArray<FString> LevelTemplatePaths;
			AssetRegistry->GetSubPaths(UE::AvaEditor::Private::LevelTemplatesPath, LevelTemplatePaths, false);

			TArray<FTemplateMapInfo> TemplateMaps;
			
			for (const FString& LevelTemplatePath : LevelTemplatePaths)
			{
				FString LevelTemplateName = FPaths::GetPathLeaf(LevelTemplatePath);

				TArray<FAssetData> LevelTemplateAssetData;				
				if (AssetRegistry->GetAssetsByPath(FName(LevelTemplatePath), LevelTemplateAssetData, true))
				{
					FAssetData LevelAsset;
					FAssetData ThumbnailAsset;
					
					for (FAssetData& AssetData : LevelTemplateAssetData)
					{
						if (LevelAsset.IsValid() && ThumbnailAsset.IsValid())
						{
							break;
						}

						if (!AssetData.AssetName.ToString().Contains(LevelTemplateName))
						{
							continue;
						}
						
						if (AssetData.AssetClassPath == UWorld::StaticClass()->GetClassPathName())
						{
							LevelAsset = AssetData;
						}
						else if (AssetData.AssetClassPath == UTexture2D::StaticClass()->GetClassPathName())
						{
							ThumbnailAsset = AssetData;
						}
					}

					if (LevelAsset.IsValid() && ThumbnailAsset.IsValid())
					{
						if (!GUnrealEd->IsTemplateMap(LevelAsset.GetObjectPathString()))
						{
							FString LevelTemplateDisplayName = FName::NameToDisplayString(LevelTemplateName, false);
							
							FTemplateMapInfo& MapTemplate = TemplateMaps.Emplace_GetRef();
							MapTemplate.Category = TEXT("Motion Design");
							MapTemplate.DisplayName = FText::FromString(LevelTemplateDisplayName);
							MapTemplate.ThumbnailTexture = ThumbnailAsset.GetSoftObjectPath();			
							MapTemplate.Map = LevelAsset.GetSoftObjectPath();
						}
					}
				}
			}

			GUnrealEd->AppendTemplateMaps(TemplateMaps);
		}
	}
}

const TSet<TSubclassOf<AActor>>& FAvaEditorModule::DefaultSceneRigActorClasses()
{
	static const TSet<TSubclassOf<AActor>> Classes =
	{
		// Cameras
		ACameraActor::StaticClass(),
		ACineCameraActor::StaticClass(),
		AAvaCineCameraActor::StaticClass(),
		// Lights
		ASkyLight::StaticClass(),
		ADirectionalLight::StaticClass(),
		APointLight::StaticClass(),
		ARectLight::StaticClass(),
		ASpotLight::StaticClass(),
		// Misc
		AAvaNullActor::StaticClass(),
		APostProcessVolume::StaticClass()
	};
	return Classes;
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FAvaEditorModule, AvalancheEditor)
