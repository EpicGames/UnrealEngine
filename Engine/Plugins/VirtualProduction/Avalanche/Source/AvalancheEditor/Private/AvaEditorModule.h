// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeCategories.h"
#include "Containers/ContainersFwd.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SubclassOf.h"
#include "Toolkits/AssetEditorToolkit.h"

class FAvaOutlinerItem;
class IAssetTypeActions;
class IAvaEditor;

DECLARE_LOG_CATEGORY_EXTERN(AvaLog, Log, All);

/**
 * Main Motion Design Editor Module
 */
class FAvaEditorModule : public IModuleInterface
{
public:
	//~ Begin IAvaEditorModule
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IAvaEditorModule

	static FSlateIcon GetOutlinerShapeActorIcon(TSharedPtr<const FAvaOutlinerItem> InItem);

	static const TSet<TSubclassOf<AActor>>& DefaultSceneRigActorClasses();
	
private:
	/** Motion Design Level Editor */
	TSharedPtr<IAvaEditor> AvaLevelEditor;

	void CreateAvaLevelEditor();

	void PostEngineInit();
	void PreExit();

	void RegisterAssetTools();
	void RegisterPropertyEditorCategories();

	void RegisterCustomLayouts();
	void UnregisterCustomLayouts();

	void RegisterLevelTemplates();

	/** Delegate handle to when asset registry is done scanning assets */
	FDelegateHandle OnKnownGathersCompleteHandle;
};
