// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEditorSettings.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "AvaEditorSettings"

UAvaEditorSettings::UAvaEditorSettings()
{
	CategoryName = TEXT("Motion Design");
	SectionName = TEXT("Editor");
}

UAvaEditorSettings* UAvaEditorSettings::Get()
{
	UAvaEditorSettings* DefaultSettings = GetMutableDefault<UAvaEditorSettings>();
	static bool bInitialized = false;
	if (!bInitialized)
	{
		bInitialized = true;
		DefaultSettings->SetFlags(RF_Transactional);
	}
	return DefaultSettings;
}

void UAvaEditorSettings::OpenEditorSettingsWindow() const
{
	static ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>(TEXT("Settings"));
	SettingsModule.ShowViewer(GetContainerName(), CategoryName, SectionName);
}

void UAvaEditorSettings::PostLoad()
{
	Super::PostLoad();

	if (!ViewportQualityPresets_DEPRECATED.IsEmpty())
	{
		ViewportQualitySettingsPresets.Empty(ViewportQualityPresets_DEPRECATED.Num());

		for (const TPair<FName, FAvaViewportQualitySettings>& Preset : ViewportQualityPresets_DEPRECATED)
		{
			const FString PresetName = Preset.Key.ToString();
			const FText PresetText = FText::AsLocalizable_Advanced(TEXT(LOCTEXT_NAMESPACE), PresetName, PresetName);
			ViewportQualitySettingsPresets.Add(FAvaViewportQualitySettingsPreset(PresetText, Preset.Value));
		}

		ViewportQualityPresets_DEPRECATED.Empty();
	}

	EnsureDefaultViewportQualityPresets();
	MaintainViewportQualityPresetsIntegrity();
}

void UAvaEditorSettings::EnsureDefaultViewportQualityPresets()
{
	auto AddPresetIfNotFound = [this](const FText& InPresetName)
	{
		FAvaViewportQualitySettingsPreset* const FoundPreset = ViewportQualitySettingsPresets.FindByPredicate(
			[InPresetName](const FAvaViewportQualitySettingsPreset& InPreset)
			{
				return InPreset.PresetName.EqualTo(InPresetName);
			});
		if (!FoundPreset)
		{
			ViewportQualitySettingsPresets.Add(FAvaViewportQualitySettingsPreset(InPresetName
				, FAvaViewportQualitySettings::Preset(InPresetName)));
		}
	};

	AddPresetIfNotFound(FAvaViewportQualitySettingsPreset::NoLumen);
	AddPresetIfNotFound(FAvaViewportQualitySettingsPreset::Reduced);
}

void UAvaEditorSettings::MaintainViewportQualityPresetsIntegrity()
{
	ViewportQualitySettingsPresets.RemoveAll([](const FAvaViewportQualitySettingsPreset& InPreset)
		{
			return InPreset.PresetName.IsEmptyOrWhitespace();
		});

	// Verifying integrity and sorting will ensure any new/removed entries are handled correctly.
	DefaultViewportQualitySettings.VerifyIntegrity();
	DefaultViewportQualitySettings.SortFeaturesByDisplayText();

	for (FAvaViewportQualitySettingsPreset& Preset : ViewportQualitySettingsPresets)
	{
		Preset.QualitySettings.VerifyIntegrity();
		Preset.QualitySettings.SortFeaturesByDisplayText();
	}
}

#undef LOCTEXT_NAMESPACE
