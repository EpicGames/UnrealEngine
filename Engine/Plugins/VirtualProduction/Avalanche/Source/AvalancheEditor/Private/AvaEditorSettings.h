// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaDefs.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Engine/DeveloperSettings.h"
#include "UObject/NameTypes.h"
#include "Viewport/AvaViewportQualitySettings.h"
#include "AvaEditorSettings.generated.h"

/**
 * Motion Design Editor Settings
 */
UCLASS(Config=EditorPerProjectUserSettings, meta = (DisplayName = "Editor"))
class UAvaEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UAvaEditorSettings();

	virtual ~UAvaEditorSettings() override = default;

	static UAvaEditorSettings* Get();

	/** Whether to allow the Motion Design Interface to show the current selected level rather than fixed at the persistent level */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Behavior")
	bool bEnableLevelContextSwitching = true;

	/** Whether to Automatically Include the Attached Actors when performing Edit Actions such as Cut, Copy, Duplicate. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Behavior")
	bool bAutoIncludeAttachedActorsInEditActions = true;

	/** When Grouping Actors with a Null Actor, whether to keep the relative transform of these Actors */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Behavior")
	bool bKeepRelativeTransformWhenGrouping = false;

	/*
	 * Distance from the camera that new actors are created via the toolbox or drag and drop.
	 * Also sets the distance from the origin that new Camera Preview Viewport cameras are created.
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Behavior")
	float CameraDistance = 500.0f;

	/**
	 * Whether to automatically switch to the Motion Design viewport when the mode is activated
	 * or a Motion Design level is opened. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Behavior")
	bool bAutoActivateMotionDesignViewport = true;

	/** Default viewport quality settings for all newly created Motion Design blueprints. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Quality")
	FAvaViewportQualitySettings DefaultViewportQualitySettings = FAvaViewportQualitySettings(true);

	/** Viewport quality settings user presets. */
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "This property type has changed. Use the updated ViewportQualityPresets instead."))
	TMap<FName, FAvaViewportQualitySettings> ViewportQualityPresets_DEPRECATED;

	/** Viewport quality settings user presets. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Quality")
	TArray<FAvaViewportQualitySettingsPreset> ViewportQualitySettingsPresets;

	//~ Begin UObject
	virtual void PostLoad() override;
	//~ End UObject

	void OpenEditorSettingsWindow() const;

private:
	void EnsureDefaultViewportQualityPresets();

	void MaintainViewportQualityPresetsIntegrity();
};
