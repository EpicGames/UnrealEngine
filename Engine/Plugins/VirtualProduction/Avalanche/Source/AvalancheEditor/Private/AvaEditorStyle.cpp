// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEditorStyle.h"
#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Brushes/SlateNoResource.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Containers/StringFwd.h"
#include "Interfaces/IPluginManager.h"
#include "Layout/Margin.h"
#include "Math/MathFwd.h"
#include "Misc/Paths.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"

FAvaEditorStyle::FAvaEditorStyle()
	: FSlateStyleSet(TEXT("AvaEditor"))
{
	const FVector2f Icon16(16.f);
	const FVector2f Icon22(22.f);
	const FVector2f Icon25(25.f);
	const FVector2f Icon40(40.f);

	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	SetContentRoot(Plugin->GetBaseDir() / TEXT("Resources"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));

	// Custom Class Icons
	Set("ClassIcon.AvaNullActor", new CORE_IMAGE_BRUSH(TEXT("Icons/SequencerIcons/icon_Sequencer_Move_24x"), Icon16));
	Set("ClassIcon.AvaToolboxStarDynamicMesh", new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/Favorite", Icon16));
	Set("ClassIcon.AvaToolboxLineDynamicMesh", new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/minus", Icon16));
	Set("ClassIcon.AvaToolboxRectangleDynamicMesh", new IMAGE_BRUSH("Icons/ToolboxIcons/rectangle", Icon16));
	Set("ClassIcon.AvaToolboxNGonDynamicMesh", new IMAGE_BRUSH("Icons/ToolboxIcons/regularpolygon", Icon16));
	Set("ClassIcon.AvaToolboxIrregularPolygonDynamicMesh", new IMAGE_BRUSH("Icons/ToolboxIcons/irregularpolygon", Icon16));
	Set("ClassIcon.AvaToolboxEllipseDynamicMesh", new IMAGE_BRUSH("Icons/ToolboxIcons/circle", Icon16));
	Set("ClassIcon.AvaToolboxRingDynamicMesh", new IMAGE_BRUSH("Icons/ToolboxIcons/ring", Icon16));
	Set("ClassIcon.AvaTextActor", new IMAGE_BRUSH("Icons/ToolboxIcons/3d-text", Icon16));
	Set("ClassIcon.AvaShapeActor", new IMAGE_BRUSH("Icons/ToolboxIcons/regularpolygon", Icon16));
	Set("ClassIcon.AvaClonerActor", new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/cloner", Icon16));
	Set("ClassIcon.AvaEffectorActor", new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/effector", Icon16));

	// Toolbox Icons
	Set("Icons.Chevron",          new IMAGE_BRUSH("Icons/ToolboxIcons/chevron",          Icon16));
	Set("Icons.RegularPolygon",   new IMAGE_BRUSH("Icons/ToolboxIcons/regularpolygon",   Icon16));
	Set("Icons.IrregularPolygon", new IMAGE_BRUSH("Icons/ToolboxIcons/irregularpolygon", Icon16));
	Set("Icons.Canvas",           new IMAGE_BRUSH("Icons/ToolboxIcons/Canvas",           Icon16));
	Set("Icons.LayoutGrid",       new IMAGE_BRUSH("Icons/ToolboxIcons/LayoutGrid",       Icon16));
	Set("Icons.Ellipse",          new IMAGE_BRUSH("Icons/ToolboxIcons/circle",           Icon16));
	Set("Icons.Ring",             new IMAGE_BRUSH("Icons/ToolboxIcons/ring",             Icon16));
	Set("Icons.Arrow",            new IMAGE_BRUSH("Icons/ToolboxIcons/arrow",            Icon16));
	Set("Icons.2DArrow",          new IMAGE_BRUSH("Icons/ToolboxIcons/arrow",            Icon16));
	Set("Icons.Sphere",           new IMAGE_BRUSH("Icons/ToolboxIcons/sphere",           Icon16));
	Set("Icons.Pyramid",          new IMAGE_BRUSH("Icons/ToolboxIcons/pyramid",          Icon16));
	Set("Icons.Cone",             new IMAGE_BRUSH("Icons/ToolboxIcons/cone",             Icon16));
	Set("Icons.Torus",            new IMAGE_BRUSH("Icons/ToolboxIcons/torus",            Icon16));
	Set("Icons.Cube",             new IMAGE_BRUSH("Icons/ToolboxIcons/cube",             Icon16));
	Set("Icons.Rectangle",        new IMAGE_BRUSH("Icons/ToolboxIcons/rectangle",        Icon16));
	Set("Icons.Plane",            new IMAGE_BRUSH("Icons/ToolboxIcons/plane",            Icon16));
	Set("Icons.Cylinder",         new IMAGE_BRUSH("Icons/ToolboxIcons/cylinder",         Icon16));
	Set("Icons.Line",             new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/minus",             Icon16));
	Set("Icons.Star",             new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/Favorite",          Icon16));
	Set("Icons.3DText",           new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/Toolbar_Text",      Icon16));
	Set("Icons.RoundedBox",       new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/Toolbar_Primitive", Icon16));
	
	Set("Icons.Alignment.Translation.TopLeft",     new IMAGE_BRUSH("Icons/DetailsPanelIcons/TopLeft",     Icon16));
	Set("Icons.Alignment.Translation.Top",         new IMAGE_BRUSH("Icons/DetailsPanelIcons/Top",         Icon16));
	Set("Icons.Alignment.Translation.TopRight",    new IMAGE_BRUSH("Icons/DetailsPanelIcons/TopRight",    Icon16));
	Set("Icons.Alignment.Translation.Left",        new IMAGE_BRUSH("Icons/DetailsPanelIcons/Left",        Icon16));
	Set("Icons.Alignment.Translation.Center",      new IMAGE_BRUSH("Icons/DetailsPanelIcons/Center",      Icon16));
	Set("Icons.Alignment.Translation.Right",       new IMAGE_BRUSH("Icons/DetailsPanelIcons/Right",       Icon16));
	Set("Icons.Alignment.Translation.BottomLeft",  new IMAGE_BRUSH("Icons/DetailsPanelIcons/BottomLeft",  Icon16));
	Set("Icons.Alignment.Translation.Bottom",      new IMAGE_BRUSH("Icons/DetailsPanelIcons/Bottom",      Icon16));
	Set("Icons.Alignment.Translation.BottomRight", new IMAGE_BRUSH("Icons/DetailsPanelIcons/BottomRight", Icon16));
	
	Set("Icons.Alignment.Translation.Back",        new IMAGE_BRUSH_SVG("Icons/DetailsPanelIcons/StackBack",   Icon22));
	Set("Icons.Alignment.Translation.Center_X",    new IMAGE_BRUSH_SVG("Icons/DetailsPanelIcons/StackCenter", Icon22));
	Set("Icons.Alignment.Translation.Front",       new IMAGE_BRUSH_SVG("Icons/DetailsPanelIcons/StackFront",  Icon22));

	Set("Icons.Alignment.Center_YZ",   new IMAGE_BRUSH_SVG("Icons/PaletteIcons/AlignCenterVertAndHoriz", Icon22));
	Set("Icons.Alignment.Left",        new IMAGE_BRUSH_SVG("Icons/PaletteIcons/AlignLeft",               Icon22));
	Set("Icons.Alignment.Center_Y",    new IMAGE_BRUSH_SVG("Icons/PaletteIcons/AlignCenterHoriz",        Icon22));
	Set("Icons.Alignment.Right",       new IMAGE_BRUSH_SVG("Icons/PaletteIcons/AlignRight",              Icon22));
	Set("Icons.Alignment.Top",         new IMAGE_BRUSH_SVG("Icons/PaletteIcons/AlignTop",                Icon22));
	Set("Icons.Alignment.Center_Z",    new IMAGE_BRUSH_SVG("Icons/PaletteIcons/AlignCenterVert",         Icon22));
	Set("Icons.Alignment.Bottom",      new IMAGE_BRUSH_SVG("Icons/PaletteIcons/AlignBottom",             Icon22));
	Set("Icons.Alignment.DistributeX", new IMAGE_BRUSH_SVG("Icons/PaletteIcons/DistributeDepth",         Icon22));
	Set("Icons.Alignment.DistributeY", new IMAGE_BRUSH_SVG("Icons/PaletteIcons/DistributeHorizontal",    Icon22));
	Set("Icons.Alignment.DistributeZ", new IMAGE_BRUSH_SVG("Icons/PaletteIcons/DistributeVertical",      Icon22));
	
	Set("Icons.Alignment.Rotation.Actor.Roll",   new IMAGE_BRUSH_SVG("Icons/PaletteIcons/RotationAlignActorRoll",   Icon22));
	Set("Icons.Alignment.Rotation.Actor.Pitch",  new IMAGE_BRUSH_SVG("Icons/PaletteIcons/RotationAlignActorPitch",  Icon22));
	Set("Icons.Alignment.Rotation.Actor.Yaw",    new IMAGE_BRUSH_SVG("Icons/PaletteIcons/RotationAlignActorYaw",    Icon22));
	Set("Icons.Alignment.Rotation.Actor.All",    new IMAGE_BRUSH_SVG("Icons/PaletteIcons/RotationAlignActorAll",    Icon22));
	Set("Icons.Alignment.Rotation.Camera.Roll",  new IMAGE_BRUSH_SVG("Icons/PaletteIcons/RotationAlignCameraRoll",  Icon22));
	Set("Icons.Alignment.Rotation.Camera.Pitch", new IMAGE_BRUSH_SVG("Icons/PaletteIcons/RotationAlignCameraPitch", Icon22));
	Set("Icons.Alignment.Rotation.Camera.Yaw",   new IMAGE_BRUSH_SVG("Icons/PaletteIcons/RotationAlignCameraYaw",   Icon22));
	Set("Icons.Alignment.Rotation.Camera.All",   new IMAGE_BRUSH_SVG("Icons/PaletteIcons/RotationAlignCameraAll",   Icon22));

	// Grid Icons
	Set("Icons.Grid.Toggle",           new IMAGE_BRUSH_SVG("Icons/PaletteIcons/GridToggle",              Icon22));
	Set("Icons.Grid.AlwaysShowToggle", new IMAGE_BRUSH_SVG("Icons/PaletteIcons/GridAlwaysVisibleToggle", Icon22));

	// Snap Icons
	Set("Icons.Snap.Toggle",       new IMAGE_BRUSH_SVG("Icons/PaletteIcons/SnapToggle",       Icon22));
	Set("Icons.Snap.ToggleGrid",   new IMAGE_BRUSH_SVG("Icons/PaletteIcons/SnapGridToggle",   Icon22));
	Set("Icons.Snap.ToggleActor",  new IMAGE_BRUSH_SVG("Icons/PaletteIcons/SnapActorToggle",  Icon22));
	Set("Icons.Snap.ToggleScreen", new IMAGE_BRUSH_SVG("Icons/PaletteIcons/SnapScreenToggle", Icon22));

	// Screen Icons
	Set("Icons.Screen.SizeToScreen",        new IMAGE_BRUSH_SVG("Icons/PaletteIcons/SizeToScreen",        Icon22));
	Set("Icons.Screen.SizeToScreenStretch", new IMAGE_BRUSH_SVG("Icons/PaletteIcons/SizeToScreenStretch", Icon22));
	Set("Icons.Screen.FitToScreen",         new IMAGE_BRUSH_SVG("Icons/PaletteIcons/FitToScreen", Icon22));

	// Logs Icons
	Set("Icons.Logs.Pin",   new IMAGE_BRUSH_SVG("Icons/LogsIcons/Pin",   Icon16));
	Set("Icons.Logs.Unpin", new IMAGE_BRUSH_SVG("Icons/LogsIcons/Unpin", Icon16));
	
	// Editor Icons
	Set("Icons.Editor.ButtonExpander",        new IMAGE_BRUSH("Icons/EditorIcons/ButtonExpander",          FVector2f(7.0f)));
	Set("Icons.Editor.ToolHightlight",        new IMAGE_BRUSH("Icons/EditorIcons/ToolIconHighlight",       Icon40));
	Set("Icons.Editor.GameView",              new IMAGE_BRUSH_SVG("Icons/EditorIcons/GameView",            Icon25));
	Set("Icons.Editor.Visualizers",           new IMAGE_BRUSH_SVG("Icons/EditorIcons/Visualizers",         Icon25));
	Set("Icons.Editor.Guides",                new IMAGE_BRUSH_SVG("Icons/EditorIcons/Guides",              Icon25));
	Set("Icons.Editor.Billboards",            new IMAGE_BRUSH("Icons/EditorIcons/Billboards",              Icon25));
	Set("Icons.Editor.SelectionLock",         new IMAGE_BRUSH("Icons/EditorIcons/SelectionLock",           Icon25));
	Set("Icons.Editor.IsolateActors",         new IMAGE_BRUSH("Icons/EditorIcons/IsolateActors",           Icon25));
	Set("Icons.Editor.Settings",              new IMAGE_BRUSH("Icons/EditorIcons/Settings",                Icon25));
	Set("Icons.Editor.Favorites",             new IMAGE_BRUSH("Icons/EditorIcons/Favorites",               FVector2f(9.0f, 14.0f)));
	Set("Icons.Editor.RemoteControlEditor",   new IMAGE_BRUSH("Icons/EditorIcons/RemoteControl",           Icon25));
	Set("Icons.Editor.BoundingBoxes",         new IMAGE_BRUSH_SVG("Icons/EditorIcons/Mode_Bounding-Box",   Icon25));
	Set("Icons.Editor.SafeFrames",            new IMAGE_BRUSH_SVG("Icons/EditorIcons/Mode_Broadcast-Safe", Icon25));
	Set("Icons.Editor.KeyPreview",            new IMAGE_BRUSH_SVG("Icons/EditorIcons/Mode_Key-Preview",    Icon25));
	Set("Icons.Editor.Stats",                 new IMAGE_BRUSH_SVG("Icons/EditorIcons/Mode_Performance",    Icon25));
	Set("Icons.Editor.Snapshot",              new IMAGE_BRUSH_SVG("Icons/EditorIcons/Mode_Snapshot",       Icon25));
	Set("Icons.Editor.WireframeMode",         new IMAGE_BRUSH_SVG("Icons/EditorIcons/Mode_Wireframe",      Icon25));
	Set("Icons.Editor.Snap.Toggle",           new IMAGE_BRUSH_SVG("Icons/EditorIcons/Snapping_Toggle",     Icon25));
	Set("Icons.Editor.Grid.Toggle",           new IMAGE_BRUSH_SVG("Icons/EditorIcons/Snapping_Grid",       Icon25));
	Set("Icons.Editor.Radio.BlackBackground", new IMAGE_BRUSH_SVG("Icons/EditorIcons/radio-background",    Icon16, FStyleColors::Foldout.GetSpecifiedColor()));
	Set("Icons.Editor.Mask.Toggle",           new IMAGE_BRUSH("Icons/EditorIcons/Mode_Mask",               Icon25));
	Set("Icons.Editor.PivotMode",             new IMAGE_BRUSH("Icons/EditorIcons/Pivot_Mode",              Icon25));
	
	Set("Icons.Lock2d", new IMAGE_BRUSH("Icons/DetailsPanelIcons/Lock2d", Icon16));
	Set("Icons.Lock3d", new IMAGE_BRUSH("Icons/DetailsPanelIcons/Lock3d", Icon16));
	Set("Icons.Unlock", new IMAGE_BRUSH("Icons/DetailsPanelIcons/Unlock", Icon16));

	Set("AvaEditor.Thumbnail.Invalid", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.0f, FLinearColor(1.0f, 0.2f, 0.2f, 1.0f), 1.0f));

	Set("AvaEditor.StaticMeshToolsCategory", new CORE_IMAGE_BRUSH_SVG("Starship/AssetIcons/StaticMeshActor_16", Icon16));
	Set("AvaEditor.CameraToolsCategory",     new CORE_IMAGE_BRUSH_SVG("Starship/AssetIcons/CameraActor_16", Icon16));
	Set("AvaEditor.LightsToolsCategory",     new CORE_IMAGE_BRUSH_SVG("Starship/AssetIcons/PointLight_16", Icon16));

	Set("AvaEditor.CubeTool",         new IMAGE_BRUSH("Icons/ToolboxIcons/cube",          Icon16));
	Set("AvaEditor.SphereTool",       new IMAGE_BRUSH("Icons/ToolboxIcons/sphere",        Icon16));
	Set("AvaEditor.CylinderTool",     new IMAGE_BRUSH("Icons/ToolboxIcons/cylinder",      Icon16));
	Set("AvaEditor.ConeTool",         new IMAGE_BRUSH("Icons/ToolboxIcons/cone",          Icon16));
	Set("AvaEditor.PlaneTool",        new IMAGE_BRUSH("Icons/ToolboxIcons/plane",         Icon16));
	Set("AvaEditor.CycloramaTool",    new IMAGE_BRUSH("Icons/ToolboxIcons/roundedsquare", Icon16));

	Set("AvaEditor.CameraTool",               new CORE_IMAGE_BRUSH_SVG("Starship/AssetIcons/CameraActor_16",       Icon16));
	Set("AvaEditor.CineCameraTool",           new CORE_IMAGE_BRUSH_SVG("Starship/AssetIcons/CineCameraActor_16",   Icon16));
	Set("AvaEditor.CameraRigCraneTool",       new CORE_IMAGE_BRUSH_SVG("Starship/AssetIcons/CameraRig_Crane_16",   Icon16));
	Set("AvaEditor.CameraRigRailTool",        new CORE_IMAGE_BRUSH_SVG("Starship/AssetIcons/CameraRig_Rail_16",    Icon16));
	Set("AvaEditor.CameraShakeSourceTool",    new CORE_IMAGE_BRUSH_SVG("Starship/Common/CameraShake",              Icon16));
	Set("AvaEditor.AvaPostProcessVolumeTool", new CORE_IMAGE_BRUSH_SVG("Starship/AssetIcons/PostProcessVolume_16", Icon16));

	Set("AvaEditor.PointLightTool",       new CORE_IMAGE_BRUSH_SVG("Starship/AssetIcons/PointLight_16",       Icon16));
	Set("AvaEditor.DirectionalLightTool", new CORE_IMAGE_BRUSH_SVG("Starship/AssetIcons/DirectionalLight_16", Icon16));
	Set("AvaEditor.RectLightTool",        new CORE_IMAGE_BRUSH_SVG("Starship/AssetIcons/RectLight_16",        Icon16));
	Set("AvaEditor.SpotLightTool",        new CORE_IMAGE_BRUSH_SVG("Starship/AssetIcons/SpotLight_16",        Icon16));
	Set("AvaEditor.SkyLightTool",         new CORE_IMAGE_BRUSH_SVG("Starship/AssetIcons/SkyLight_16",         Icon16));
	
	// Colors
	Set("AvaEditor.PalettesTab.ExpanderHeader", new FSlateColorBrush(FStyleColors::Header));

	// Buttons
	const FTextBlockStyle& AppStyle_ContentBrowserTopBarFont = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("ContentBrowser.TopBar.Font");
	FLinearColor ButtonTextColor = AppStyle_ContentBrowserTopBarFont.ColorAndOpacity.GetSpecifiedColor();
	ButtonTextColor.A /= 2;
	FLinearColor ButtonShadowColorAndOpacity = AppStyle_ContentBrowserTopBarFont.ShadowColorAndOpacity;
	ButtonShadowColorAndOpacity.A /= 2;
	Set("AvaEditor.Button.TextStyle", FTextBlockStyle(AppStyle_ContentBrowserTopBarFont)
		.SetColorAndOpacity(ButtonTextColor)
		.SetShadowColorAndOpacity(ButtonShadowColorAndOpacity));

	const FButtonStyle& AppStyle_SimpleButton = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton");

	Set("AvaEditor.BorderlessButton", FButtonStyle(AppStyle_SimpleButton)
		.SetNormalPadding(0.0f)
		.SetPressedPadding(0.0f));

	Set("AvaEditor.HighlightButton", FButtonStyle(AppStyle_SimpleButton)
		.SetNormal(FSlateColorBrush(FStyleColors::Secondary))
		.SetHovered(FSlateColorBrush(FStyleColors::Hover))
		.SetPressed(FSlateColorBrush(FStyleColors::Header))
		.SetDisabled(FSlateColorBrush(FStyleColors::Dropdown)));

	Set("AvaEditor.SuperBarButton", FButtonStyle(AppStyle_SimpleButton)
		.SetNormal(FSlateNoResource())
		.SetHovered(FSlateColorBrush(FStyleColors::Secondary))
		.SetPressed(FSlateColorBrush(FStyleColors::Hover))
		.SetDisabled(FSlateNoResource()));

	Set("AvaEditor.SuperBarButton.Rounded", FButtonStyle(AppStyle_SimpleButton)
		.SetNormal(FSlateNoResource())
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Secondary, 4.0f))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f))
		.SetDisabled(FSlateNoResource()));

	Set("AvaEditor.SuperBarSubMenuButton", FButtonStyle(AppStyle_SimpleButton)
		.SetNormal(FSlateNoResource())
		.SetHovered(FSlateNoResource())
		.SetPressed(FSlateNoResource())
		.SetDisabled(FSlateNoResource()));

	Set("AvaEditor.DarkButton", FButtonStyle(AppStyle_SimpleButton)
		.SetNormal(FSlateColorBrush(FStyleColors::Recessed))
		.SetHovered(FSlateColorBrush(FStyleColors::Hover))
		.SetPressed(FSlateColorBrush(FStyleColors::Header))
		.SetDisabled(FSlateColorBrush(FStyleColors::Dropdown)));

	const FButtonStyle& AppStyle_FlatButton = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("FlatButton");
	Set("LevelSnapshotsEditor.RemoveFilterButton", FButtonStyle(AppStyle_FlatButton)
		.SetNormal(FSlateNoResource())
		.SetNormalPadding(FMargin(0, 1.5f))
		.SetPressedPadding(FMargin(0, 1.5f)));

	// Text
	const FTextBlockStyle& AppStyle_GraphCompactNodeTitle = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("Graph.CompactNode.Title");
	Set("AvaEditor.FilterRow.And", FTextBlockStyle(AppStyle_GraphCompactNodeTitle)
		.SetFont(DEFAULT_FONT("BoldCondensed", 16)));
	Set("AvaEditor.FilterRow.Or", FTextBlockStyle(AppStyle_GraphCompactNodeTitle)
		.SetFont(DEFAULT_FONT("BoldCondensed", 18)));

	// Check Boxes
	const FCheckBoxStyle& AppStyle_RadioButton = FAppStyle::GetWidgetStyle<FCheckBoxStyle>("RadioButton");
	Set("AvaEditor.BlackRadioButton", FCheckBoxStyle(AppStyle_RadioButton)
		.SetBackgroundImage(*GetBrush("Icons.Editor.Radio.BlackBackground")));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FAvaEditorStyle::~FAvaEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
