// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FAvaEditorStyle final : public FSlateStyleSet
{
public:
	static FAvaEditorStyle& Get()
	{
		static FAvaEditorStyle Instance;
		return Instance;
	}

	FAvaEditorStyle();
	virtual ~FAvaEditorStyle() override;
};
