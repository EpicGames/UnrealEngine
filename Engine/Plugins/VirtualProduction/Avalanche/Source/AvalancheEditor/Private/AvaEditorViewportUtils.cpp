// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEditorViewportUtils.h"
#include "AvaEditorSettings.h"
#include "Templates/SharedPointer.h"
#include "ViewportClient/IAvaViewportClient.h"

bool FAvaEditorViewportUtils::MeshSizeToPixelSize(const TSharedRef<IAvaViewportClient>& InViewportClient, double InMeshSize, double& OutPixelSize)
{
	const FIntPoint CanvasSize = InViewportClient->GetVirtualViewportSize();

	if (CanvasSize.X <= 0)
	{
		return false;
	}

	const FVector2D FrustumSize = InViewportClient->GetFrustumSizeAtDistance(UAvaEditorSettings::Get()->CameraDistance);

	if (FrustumSize.X <= 0)
	{
		return false;
	}

	OutPixelSize = (InMeshSize / FrustumSize.X) * static_cast<double>(CanvasSize.X);

	return true;
}

bool FAvaEditorViewportUtils::PixelSizeToMeshSize(const TSharedRef<IAvaViewportClient>& InViewportClient, double InPixelSize, double& OutMeshSize)
{
	const FIntPoint CanvasSize = InViewportClient->GetVirtualViewportSize();

	if (CanvasSize.X <= 0)
	{
		return false;
	}

	const FVector2D FrustumSize = InViewportClient->GetFrustumSizeAtDistance(UAvaEditorSettings::Get()->CameraDistance);

	if (FrustumSize.X <= 0)
	{
		return false;
	}

	OutMeshSize = (InPixelSize / static_cast<double>(CanvasSize.X)) * FrustumSize.X;

	return true;
}
