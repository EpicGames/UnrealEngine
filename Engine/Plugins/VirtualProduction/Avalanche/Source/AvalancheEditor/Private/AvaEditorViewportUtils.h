// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/SharedPointerFwd.h"

class IAvaViewportClient;

struct FAvaEditorViewportUtils
{
	static bool MeshSizeToPixelSize(const TSharedRef<IAvaViewportClient>& InViewportClient, double InMeshSize, double& OutPixelSize);

	static bool PixelSizeToMeshSize(const TSharedRef<IAvaViewportClient>& InViewportClient, double InPixelSize, double& OutMeshSize);
};
