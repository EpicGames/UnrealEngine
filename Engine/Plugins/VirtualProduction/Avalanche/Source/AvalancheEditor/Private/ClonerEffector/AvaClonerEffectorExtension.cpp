// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaClonerEffectorExtension.h"

#include "AvaEditorCommands.h"
#include "Cloner/CEClonerComponent.h"
#include "EditorModeManager.h"
#include "Effector/CEEffectorComponent.h"
#include "Selection.h"
#include "Subsystems/CEClonerSubsystem.h"
#include "Subsystems/CEEffectorSubsystem.h"

#define LOCTEXT_NAMESPACE "AvaClonerEffectorExtension"

FAvaClonerEffectorExtension::FAvaClonerEffectorExtension()
	: ClonerEffectorCommands(MakeShared<FUICommandList>())
{
}

void FAvaClonerEffectorExtension::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	InCommandList->Append(ClonerEffectorCommands);

	const FAvaEditorCommands& EditorCommands = FAvaEditorCommands::Get();

	ClonerEffectorCommands->MapAction(EditorCommands.DisableEffectors
		, FExecuteAction::CreateSP(this, &FAvaClonerEffectorExtension::EnableEffectors, false));

	ClonerEffectorCommands->MapAction(EditorCommands.EnableEffectors
		, FExecuteAction::CreateSP(this, &FAvaClonerEffectorExtension::EnableEffectors, true));

	ClonerEffectorCommands->MapAction(EditorCommands.DisableCloners
		, FExecuteAction::CreateSP(this, &FAvaClonerEffectorExtension::EnableCloners, false));

	ClonerEffectorCommands->MapAction(EditorCommands.EnableCloners
		, FExecuteAction::CreateSP(this, &FAvaClonerEffectorExtension::EnableCloners, true));

	ClonerEffectorCommands->MapAction(EditorCommands.CreateCloner
		, FExecuteAction::CreateSP(this, &FAvaClonerEffectorExtension::CreateCloner));
}

TSet<AActor*> FAvaClonerEffectorExtension::GetSelectedActors() const
{
	TSet<AActor*> SelectedActors;

	const FEditorModeTools* ModeTools = GetEditorModeTools();
	if (!ModeTools)
	{
		return SelectedActors;
	}

	const UTypedElementSelectionSet* SelectionSet = ModeTools->GetEditorSelectionSet();
	if (!SelectionSet)
	{
		return SelectedActors;
	}

	SelectedActors.Append(SelectionSet->GetSelectedObjects<AActor>());

	return SelectedActors;
}

void FAvaClonerEffectorExtension::EnableEffectors(bool bInEnable) const
{
	TSet<AActor*> SelectedActors = GetSelectedActors();

	TSet<UCEEffectorComponent*> EffectorComponents;
	for (AActor* SelectedActor : SelectedActors)
	{
		if (IsValid(SelectedActor))
		{
			TArray<UCEEffectorComponent*> Components;
			SelectedActor->GetComponents(Components, /** IncludeChildren */false);
			EffectorComponents.Append(Components);
		}
	}

	UCEEffectorSubsystem* EffectorSubsystem = UCEEffectorSubsystem::Get();

	if (!EffectorSubsystem)
	{
		return;
	}

	constexpr bool bTransact = true;

	// If nothing selected, target all animators in level
	if (!SelectedActors.IsEmpty())
	{
		EffectorSubsystem->SetEffectorsEnabled(EffectorComponents, bInEnable, bTransact);
	}
	else
	{
		EffectorSubsystem->SetLevelEffectorsEnabled(GetWorld(), bInEnable, bTransact);
	}
}

void FAvaClonerEffectorExtension::EnableCloners(bool bInEnable) const
{
	TSet<AActor*> SelectedActors = GetSelectedActors();

	TSet<UCEClonerComponent*> ClonerComponents;
	for (AActor* SelectedActor : SelectedActors)
	{
		if (IsValid(SelectedActor))
		{
			TArray<UCEClonerComponent*> Components;
			SelectedActor->GetComponents(Components, /** IncludeChildren */false);
			ClonerComponents.Append(Components);
		}
	}

	UCEClonerSubsystem* ClonerSubsystem = UCEClonerSubsystem::Get();

	if (!ClonerSubsystem)
	{
		return;
	}

	constexpr bool bTransact = true;

	// If nothing selected, target all animators in level
	if (!SelectedActors.IsEmpty())
	{
		ClonerSubsystem->SetClonersEnabled(ClonerComponents, bInEnable, bTransact);
	}
	else
	{
		ClonerSubsystem->SetLevelClonersEnabled(GetWorld(), bInEnable, bTransact);
	}
}

void FAvaClonerEffectorExtension::CreateCloner() const
{
	const FEditorModeTools* ModeTools = GetEditorModeTools();
	if (!ModeTools)
	{
		return;
	}

	UCEClonerSubsystem* ClonerSubsystem = UCEClonerSubsystem::Get();
	if (!ClonerSubsystem)
	{
		return;
	}

	TSet<AActor*> SelectedActors = GetSelectedActors();
	UWorld* World = ModeTools->GetWorld();

	constexpr UCEClonerSubsystem::ECEClonerActionFlags Flags = UCEClonerSubsystem::ECEClonerActionFlags::All;
	ClonerSubsystem->CreateClonerWithActors(World, SelectedActors, Flags);
}

#undef LOCTEXT_NAMESPACE
