// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "IAvaEditorExtension.h"
#include "Templates/SharedPointer.h"

class AActor;
class FUICommandList;

class FAvaClonerEffectorExtension : public FAvaEditorExtension
{
public:
	UE_AVA_INHERITS(FAvaClonerEffectorExtension, FAvaEditorExtension);

	FAvaClonerEffectorExtension();

	//~ Begin IAvaEditorExtension
	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandList) override;
	//~ End IAvaEditorExtension

private:
	TSet<AActor*> GetSelectedActors() const;

	void EnableEffectors(bool bInEnable) const;
	void EnableCloners(bool bInEnable) const;
	void CreateCloner() const;

	TSharedRef<FUICommandList> ClonerEffectorCommands;
};
