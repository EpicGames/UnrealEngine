// Copyright Epic Games, Inc. All Rights Reserved.

#include "ComponentVisualizer/AvaComponentVisualizerExtension.h"
#include "AvaViewportSettings.h"
#include "IAvaComponentVisualizersViewportOverlay.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "Selection/AvaEditorSelection.h"
#include "Viewport/AvaViewportExtension.h"

FAvaComponentVisualizerExtension::~FAvaComponentVisualizerExtension()
{
	if (UAvaViewportSettings* Settings = GetMutableDefault<UAvaViewportSettings>())
	{
		Settings->OnChange.RemoveAll(this);
	}
}

void FAvaComponentVisualizerExtension::Activate()
{
	FAvaEditorExtension::Activate();

	if (UAvaViewportSettings* ViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		ViewportSettings->OnChange.AddRaw(this, &FAvaComponentVisualizerExtension::OnViewportSettingsChanged);
	}
}

void FAvaComponentVisualizerExtension::Deactivate()
{
	FAvaEditorExtension::Deactivate();

	IAvalancheComponentVisualizersModule* ComponentVisualizersModule = IAvalancheComponentVisualizersModule::GetIfLoaded();

	if (!ComponentVisualizersModule)
	{
		return;
	}

	if (UAvaViewportSettings* ViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		ViewportSettings->OnChange.RemoveAll(this);
	}

	TSharedPtr<IAvaEditor> Editor = GetEditor();

	if (!Editor.IsValid())
	{
		return;
	}

	TSharedPtr<FAvaViewportExtension> ViewportExtension = Editor->FindExtension<FAvaViewportExtension>();

	if (!ViewportExtension.IsValid())
	{
		return;
	}

	ComponentVisualizersModule->GetViewportOverlay().RemoveWidget(ViewportExtension->GetViewportClients());
}

void FAvaComponentVisualizerExtension::NotifyOnSelectionChanged(const FAvaEditorSelection& InSelection)
{
	IAvalancheComponentVisualizersModule* ComponentVisualizersModule = IAvalancheComponentVisualizersModule::GetIfLoaded();

	if (!ComponentVisualizersModule)
	{
		return;
	}

	TSharedPtr<IAvaEditor> Editor = GetEditor();

	if (!Editor.IsValid())
	{
		return;
	}

	TSharedPtr<FAvaViewportExtension> ViewportExtension = Editor->FindExtension<FAvaViewportExtension>();

	if (!ViewportExtension.IsValid())
	{
		return;
	}

	const TArray<TSharedPtr<IAvaViewportClient>> AvaViewportClients = ViewportExtension->GetViewportClients();

	ComponentVisualizersModule->GetViewportOverlay().RemoveWidget(AvaViewportClients);

	const UAvaViewportSettings* ViewportSettings = GetDefault<UAvaViewportSettings>();

	if (!ViewportSettings || !ViewportSettings->bEnableViewportOverlay || !ViewportSettings->bEnableShapesEditorOverlay)
	{
		return;
	}

	USelection* ActorSelection = InSelection.GetSelection<AActor>();

	if (!ActorSelection)
	{
		return;
	}

	TArray<AActor*> SelectedActors;
	ActorSelection->GetSelectedObjects<AActor>(SelectedActors);

	TArray<UObject*> SelectedObjects;
	SelectedObjects.Reserve(SelectedActors.Num() * 3); // Estimate 1 actor and 2 components per actor.

	for (AActor* SelectedActor : SelectedActors)
	{
		SelectedObjects.Add(SelectedActor);

		TArray<UActorComponent*> ActorComponents;
		SelectedActor->GetComponents<UActorComponent>(ActorComponents);

		SelectedObjects.Append(ActorComponents);
	}

	if (!SelectedObjects.IsEmpty())
	{
		ComponentVisualizersModule->GetViewportOverlay().AddWidget(AvaViewportClients, SelectedObjects);
	}
}

void FAvaComponentVisualizerExtension::OnViewportSettingsChanged(const UAvaViewportSettings* InSettings, FName InSetting)
{
	if (!InSettings)
	{
		return;
	}

	IAvalancheComponentVisualizersModule* ComponentVisualizersModule = IAvalancheComponentVisualizersModule::GetIfLoaded();

	if (!ComponentVisualizersModule)
	{
		return;
	}

	static const FName bEnableViewportOverlayName = GET_MEMBER_NAME_CHECKED(UAvaViewportSettings, bEnableViewportOverlay);
	static const FName bEnableShapesEditorOverlayName = GET_MEMBER_NAME_CHECKED(UAvaViewportSettings, bEnableShapesEditorOverlay);
	static const FName ShapeEditorOverlayTypeName = GET_MEMBER_NAME_CHECKED(UAvaViewportSettings, ShapeEditorOverlayType);

	if (InSetting != bEnableViewportOverlayName && InSetting != bEnableShapesEditorOverlayName && InSetting != ShapeEditorOverlayTypeName)
	{
		return;
	}

	const bool bAllowWidget = InSettings->bEnableViewportOverlay && InSettings->bEnableShapesEditorOverlay;

	if (bAllowWidget == ComponentVisualizersModule->GetViewportOverlay().IsWidgetActive() && InSetting != ShapeEditorOverlayTypeName)
	{
		return;
	}

	TSharedPtr<IAvaEditor> Editor = GetEditor();

	if (!Editor.IsValid())
	{
		return;
	}

	TSharedPtr<FAvaViewportExtension> ViewportExtension = Editor->FindExtension<FAvaViewportExtension>();

	if (!ViewportExtension.IsValid())
	{
		return;
	}

	if (!bAllowWidget)
	{
		ComponentVisualizersModule->GetViewportOverlay().RemoveWidget(ViewportExtension->GetViewportClients());
		return;
	}

	FEditorModeTools* const ModeTools = GetEditorModeTools();

	if (!ModeTools)
	{
		return;
	}

	const FAvaEditorSelection Selection(*ModeTools, ModeTools->GetSelectedActors());

	if (!Selection.IsValid())
	{
		return;
	}

	NotifyOnSelectionChanged(Selection);
}
