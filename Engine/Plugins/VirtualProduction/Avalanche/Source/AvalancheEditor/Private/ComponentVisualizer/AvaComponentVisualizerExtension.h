// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaEditorExtension.h"

class UAvaViewportSettings;

class FAvaComponentVisualizerExtension : public FAvaEditorExtension
{
public:
	UE_AVA_INHERITS(FAvaComponentVisualizerExtension, FAvaEditorExtension);

	virtual ~FAvaComponentVisualizerExtension() override;

	//~ Begin IAvaEditorExtension
	virtual void Activate() override;
	virtual void Deactivate() override;
	virtual void NotifyOnSelectionChanged(const FAvaEditorSelection& InSelection) override;
	//~ End IAvaEditorExtension

protected:
	void OnViewportSettingsChanged(const UAvaViewportSettings* InSettings, FName InSetting);
};
