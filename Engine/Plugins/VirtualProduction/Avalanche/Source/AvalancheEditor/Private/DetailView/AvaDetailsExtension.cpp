// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailView/AvaDetailsExtension.h"
#include "Framework/Docking/LayoutExtender.h"
#include "ISequencer.h"
#include "LevelEditor.h"
#include "MovieScene.h"
#include "PropertyHandle.h"
#include "Sequencer/AvaSequencerExtension.h"

FEditorModeTools* FAvaDetailsExtension::GetDetailsModeTools() const
{
	return GetEditorModeTools();
}

TSharedPtr<IDetailKeyframeHandler> FAvaDetailsExtension::GetDetailsKeyframeHandler() const
{
	return SharedThis(const_cast<FAvaDetailsExtension*>(this));
}

bool FAvaDetailsExtension::IsPropertyKeyingEnabled() const
{
	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid() || Sequencer->IsReadOnly())
	{
		return false;
	}

	return Sequencer->GetFocusedMovieSceneSequence()
		&& Sequencer->GetAllowEditsMode() != EAllowEditsMode::AllowLevelEditsOnly;
}

bool FAvaDetailsExtension::IsPropertyKeyable(const UClass* InObjectClass, const IPropertyHandle& InPropertyHandle) const
{
	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid() || Sequencer->IsReadOnly())
	{
		return false;
	}

	return Sequencer->CanKeyProperty(FCanKeyPropertyParams(InObjectClass, InPropertyHandle));
}

void FAvaDetailsExtension::OnKeyPropertyClicked(const IPropertyHandle& InPropertyHandle)
{
	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	TArray<UObject*> Objects;
	InPropertyHandle.GetOuterObjects(Objects);

	if (Objects.IsEmpty())
	{
		return;
	}

	FKeyPropertyParams KeyPropertyParams(Objects, InPropertyHandle, ESequencerKeyMode::ManualKeyForced);
	Sequencer->KeyProperty(KeyPropertyParams);
}

bool FAvaDetailsExtension::IsPropertyAnimated(const IPropertyHandle& InPropertyHandle, UObject* InParentObject) const
{
	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return false;
	}

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return false;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return false;
	}

	FGuid ObjectHandle = Sequencer->GetHandleToObject(InParentObject, /*bCreateHandleIfMissing*/false);
	if (!ObjectHandle.IsValid())
	{
		return false;
	}

	FName PropertyName;
	{
		FPropertyPath PropertyPath;
		PropertyPath.AddProperty(FPropertyInfo(InPropertyHandle.GetProperty()));
		PropertyName = *PropertyPath.ToString(TEXT("."));
	}

	return !!MovieScene->FindTrack<UMovieSceneTrack>(ObjectHandle, PropertyName);
}

EPropertyKeyedStatus FAvaDetailsExtension::GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const
{
	if (TSharedPtr<ISequencer> Sequencer = GetSequencer())
	{
		return Sequencer->GetPropertyKeyedStatus(PropertyHandle);
	}
	return EPropertyKeyedStatus::NotKeyed;
}

TSharedPtr<ISequencer> FAvaDetailsExtension::GetSequencer() const
{
	TSharedPtr<IAvaEditor> Editor = GetEditor();
	if (!Editor.IsValid())
	{
		return nullptr;
	}

	if (const TSharedPtr<FAvaSequencerExtension> SequencerExtension = Editor->FindExtension<FAvaSequencerExtension>())
	{
		return SequencerExtension->GetSequencer();
	}

	return nullptr;
}
