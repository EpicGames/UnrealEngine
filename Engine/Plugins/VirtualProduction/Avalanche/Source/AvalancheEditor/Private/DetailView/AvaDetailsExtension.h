// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaEditorExtension.h"
#include "IDetailKeyframeHandler.h"
#include "DetailView/IAvaDetailsProvider.h"

class ISequencer;

UE_AVA_TYPE_EXTERNAL(IAvaDetailsProvider);
UE_AVA_TYPE_EXTERNAL(IDetailKeyframeHandler);

class FAvaDetailsExtension: public FAvaEditorExtension, public IAvaDetailsProvider, public IDetailKeyframeHandler
{
public:
	UE_AVA_INHERITS(FAvaDetailsExtension, FAvaEditorExtension, IAvaDetailsProvider, IDetailKeyframeHandler);

	//~ Begin IAvaDetailsProvider
	virtual FEditorModeTools* GetDetailsModeTools() const override;
	virtual TSharedPtr<IDetailKeyframeHandler> GetDetailsKeyframeHandler() const override;
	//~ End IAvaDetailsProvider

	//~ Begin IDetailKeyframeHandler
	virtual bool IsPropertyKeyingEnabled() const override;
	virtual bool IsPropertyKeyable(const UClass* InObjectClass, const IPropertyHandle& InPropertyHandle) const override;
	virtual void OnKeyPropertyClicked(const IPropertyHandle& InPropertyHandle) override;
	virtual bool IsPropertyAnimated(const IPropertyHandle& InPropertyHandle, UObject* InParentObject) const override;
	virtual EPropertyKeyedStatus GetPropertyKeyedStatus(const IPropertyHandle& PropertyHandle) const override;
	//~ End IDetailKeyframeHandler

private:
	TSharedPtr<ISequencer> GetSequencer() const;
};
