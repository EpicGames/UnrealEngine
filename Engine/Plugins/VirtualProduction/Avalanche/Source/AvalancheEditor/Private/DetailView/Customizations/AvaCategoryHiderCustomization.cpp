// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaCategoryHiderCustomization.h"

#include "DetailLayoutBuilder.h"

#define LOCTEXT_NAMESPACE "CategoryHidersCustomization"

void FAvaCategoryHiderCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	HideCategories(DetailBuilder);
}

void FAvaCategoryHiderCustomization::HideCategories(IDetailLayoutBuilder& DetailBuilder)
{
	DetailBuilder.HideCategory("Physics");
	DetailBuilder.HideCategory("Collision");
	DetailBuilder.HideCategory("Activation");
	DetailBuilder.HideCategory("Cooking");
	DetailBuilder.HideCategory("HLOD");
	DetailBuilder.HideCategory("LOD");
	DetailBuilder.HideCategory("Navigation");
	DetailBuilder.HideCategory("Replication");
	DetailBuilder.HideCategory("Actor");
	DetailBuilder.HideCategory("WorldPartition");
	DetailBuilder.HideCategory("Input");
	DetailBuilder.HideCategory("AssetUserData");
	DetailBuilder.HideCategory("Mobile");
	DetailBuilder.HideCategory("RayTracing");
	DetailBuilder.HideCategory("DataLayers");
	// hide proc mesh comp materials
	DetailBuilder.HideCategory("Materials");
}

#undef LOCTEXT_NAMESPACE
