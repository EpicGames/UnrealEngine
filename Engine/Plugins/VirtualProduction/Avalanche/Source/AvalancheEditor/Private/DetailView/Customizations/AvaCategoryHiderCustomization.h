// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class FAvaCategoryHiderCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FAvaCategoryHiderCustomization>();
	}

	virtual ~FAvaCategoryHiderCustomization() override = default;

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	static void HideCategories(IDetailLayoutBuilder& DetailBuilder);
};
