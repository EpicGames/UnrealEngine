// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMeshesDetailCustomization.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "Components/DynamicMeshComponent.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailView/Widgets/SAvaDynamicMaterialWidget.h"
#include "DetailWidgetRow.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "DynamicMeshes/AvaShape2DDynMeshBase.h"
#include "DynamicMeshToMeshDescription.h"
#include "Engine/StaticMesh.h"
#include "IAssetTools.h"
#include "Modifiers/AvaSizeToTextureModifier.h"
#include "Modifiers/Utilities/ActorModifierCoreLibrary.h"
#include "UObject/Object.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "AvaMeshesDetailCustomization"

const FLazyName FAvaMeshesDetailCustomization::AutoUpdateTextureMetadata = TEXT("AutoUpdateTexture");

void FAvaMeshesDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	TSharedRef<IPropertyHandle> MeshDatasHandle = InDetailBuilder.GetProperty(
		GET_MEMBER_NAME_CHECKED(UAvaShapeDynamicMeshBase, MeshDatas),
		UAvaShapeDynamicMeshBase::StaticClass()
	);

	InDetailBuilder.HideProperty(MeshDatasHandle);

	TSharedRef<IPropertyHandle> UsePrimaryMaterialEverywhereHandle = InDetailBuilder.GetProperty(
		GET_MEMBER_NAME_CHECKED(UAvaShapeDynamicMeshBase, bUsePrimaryMaterialEverywhere),
		UAvaShapeDynamicMeshBase::StaticClass()
	);

	InDetailBuilder.HideProperty(UsePrimaryMaterialEverywhereHandle);

	MeshGeneratorsWeak = InDetailBuilder.GetObjectsOfTypeBeingCustomized<UAvaShapeDynamicMeshBase>();

	// Set material category after shape category to avoid jump when new materials slot becomes available
	IDetailCategoryBuilder& ShapeCategoryBuilder = InDetailBuilder.EditCategory(FName("Shape"));
	IDetailCategoryBuilder& MaterialCategoryBuilder = InDetailBuilder.EditCategory(FName("Material"));
	MaterialCategoryBuilder.SetSortOrder(ShapeCategoryBuilder.GetSortOrder() + 1);

	// Make sure we have common mesh sections to display material properties
	TMap<int32, FName> MeshSectionNames;
	bool bFirstElement = true;
	for (TArray<TWeakObjectPtr<UAvaShapeDynamicMeshBase>>::TIterator It(MeshGeneratorsWeak); It; ++It)
	{
		if (UAvaShapeDynamicMeshBase* MeshGenerator = It->Get())
		{
			if (bFirstElement)
			{
				const TArray<FName> MeshNames = MeshGenerator->GetMeshSectionNames();
				for (int32 Index : MeshGenerator->GetMeshesIndexes())
				{
					if (MeshNames.IsValidIndex(Index))
					{
						MeshSectionNames.Add(Index, MeshNames[Index]);
					}
				}
				bFirstElement = false;
			}
			else
			{
				const TSet<int32> MeshIndex = MeshGenerator->GetMeshesIndexes();
				for (TMap<int32, FName>::TIterator MeshIt(MeshSectionNames); MeshIt; ++MeshIt)
				{
					if (!MeshIndex.Contains(MeshIt->Key))
					{
						MeshIt.RemoveCurrent();
					}
				}
			}
		}
		else
		{
			It.RemoveCurrent();
		}
	}

	if (!MeshGeneratorsWeak.IsEmpty() && !MeshSectionNames.IsEmpty())
	{
		TSharedPtr<IPropertyHandleMap> MapHandle = MeshDatasHandle->AsMap();
		check(MapHandle.IsValid())

		uint32 NumSections = 0;
		MapHandle->GetNumElements(NumSections);
		for (int32 Index = 0; Index < static_cast<int32>(NumSections); ++Index)
		{
			if (!MeshSectionNames.Contains(Index))
			{
				continue;
			}

			TSharedPtr<IPropertyHandle> MeshPropertyHandle = MapHandle->GetElement(Index);

			if (!MeshPropertyHandle.IsValid())
			{
				continue;
			}

			const FString MeshName = Index != 0 ? MeshSectionNames[Index].ToString() + TEXT(" ") : TEXT("");

			// Material Type
			TSharedPtr<IPropertyHandle> MaterialTypeHandle = MeshPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaShapeMeshData, MaterialType));
			check(MaterialTypeHandle.IsValid());

			FString MaterialTypeName = MeshName + TEXT("Material Type");
			FDetailWidgetRow& MaterialTypeRow = MaterialCategoryBuilder.AddCustomRow(FText::FromString(MaterialTypeName));

			MaterialTypeRow.NameContent()[MaterialTypeHandle->CreatePropertyNameWidget(FText::FromString(MaterialTypeName))];
			MaterialTypeRow.ValueContent()[MaterialTypeHandle->CreatePropertyValueWidget()];
			MaterialTypeRow.Visibility(MakeAttributeLambda([=]() {
				return MaterialTypeHandle->IsEditable() ? EVisibility::Visible : EVisibility::Hidden;
			}));

			// Material Asset
			TSharedPtr<IPropertyHandle> MaterialHandle = MeshPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaShapeMeshData, Material));
			check(MaterialHandle.IsValid());

			FString MaterialName = MeshName + TEXT("Material Asset");
			FDetailWidgetRow& MaterialRow = MaterialCategoryBuilder.AddCustomRow(FText::FromString(MaterialName));

			MaterialRow.NameContent()
			[
				MaterialHandle->CreatePropertyNameWidget(FText::FromString(MaterialName))
			];

			MaterialRow.ValueContent()
			[
				SNew(SAvaDynamicMaterialWidget, MaterialHandle.ToSharedRef())
			];

			MaterialRow.Visibility(MakeAttributeLambda([MaterialHandle]()
			{
				return MaterialHandle->IsEditable() ? EVisibility::Visible : EVisibility::Hidden;
			}));

			// Parametric Material
			TSharedPtr<IPropertyHandle> ParametricMaterialHandle = MeshPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaShapeMeshData, ParametricMaterial));
			check(ParametricMaterialHandle.IsValid());

			uint32 NumChildren = 0;
			ParametricMaterialHandle->GetNumChildren(NumChildren);

			for (uint32 ChildIdx = 0; ChildIdx < NumChildren; ChildIdx++)
			{
				TSharedPtr<IPropertyHandle> ParametricChildHandle = ParametricMaterialHandle->GetChildHandle(ChildIdx);
				check(ParametricChildHandle.IsValid());

				TAttribute<EVisibility> VisibilityAttribute = MakeAttributeLambda([ParametricMaterialHandle, ParametricChildHandle]()
					{
						return ParametricMaterialHandle->IsEditable() ? (ParametricChildHandle->IsEditable() ? EVisibility::Visible : EVisibility::Hidden) : EVisibility::Hidden;
					});

				if (ParametricChildHandle->GetProperty()
					&& ParametricChildHandle->GetProperty()->GetFName() == TEXT("Texture")
					&& CanSizeToTexture())
				{
					FDetailWidgetRow& ParametricTextureRow = MaterialCategoryBuilder.AddCustomRow(ParametricChildHandle->GetPropertyDisplayName());
					ParametricTextureRow.PropertyHandleList({ParametricChildHandle});
					ParametricTextureRow.Visibility(VisibilityAttribute);

					ParametricTextureRow.NameContent()
					[
						ParametricChildHandle->CreatePropertyNameWidget()
					];

					ParametricChildHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FAvaMeshesDetailCustomization::OnTexturePropertyChanged));

					const FText AutoUpdateTooltip = LOCTEXT("AutoSizeToTexture.Tooltip", "EDITOR-ONLY : Auto update the size to texture modifier when this texture property changes");

					ParametricTextureRow.ValueContent()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					.MaxDesiredWidth(250)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							ParametricChildHandle->CreatePropertyValueWidget()
						]
						+ SHorizontalBox::Slot()
						.Padding(2.f, 0.f)
						.AutoWidth()
						[
							SNew(SVerticalBox)
							+ SVerticalBox::Slot()
							[
								SNew(SButton)
								.ContentPadding(1.f)
								.VAlign(VAlign_Center)
								.HAlign(HAlign_Center)
								.ToolTipText(LOCTEXT("SizeToTextureTooltip", "EDITOR-ONLY : Find or Add a size to texture modifier and link this texture to it"))
								.OnClicked(this, &FAvaMeshesDetailCustomization::OnSizeToTextureClicked)
								.Content()
								[
									SNew(STextBlock)
									.Font(IDetailLayoutBuilder::GetDetailFont())
									.Text(LOCTEXT("SizeToTextureLabel", "Size to Texture"))
								]
							]
							+ SVerticalBox::Slot()
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.HAlign(HAlign_Fill)
								.VAlign(VAlign_Center)
								[
									SNew(SCheckBox)
									.Padding(1.f)
									.ToolTipText(AutoUpdateTooltip)
									.IsChecked(this, &FAvaMeshesDetailCustomization::OnIsAutoSizeToTextureChecked)
									.OnCheckStateChanged(this, &FAvaMeshesDetailCustomization::OnAutoSizeToTextureStateChanged)
								]
								+ SHorizontalBox::Slot()
								.FillWidth(1.f)
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.Font(IDetailLayoutBuilder::GetDetailFont())
									.Text(LOCTEXT("AutoSizeToTexture.Label", "Auto Update"))
									.ToolTipText(AutoUpdateTooltip)
								]
							]
						]
					];
				}
				else
				{
					IDetailPropertyRow& NewParametricRow = MaterialCategoryBuilder.AddProperty(ParametricChildHandle.ToSharedRef());
					FString ParametricRowName = MeshName + ParametricChildHandle->GetPropertyDisplayName().ToString();
					NewParametricRow.DisplayName(FText::FromString(ParametricRowName));
					NewParametricRow.Visibility(VisibilityAttribute);
				}
			}

			// Use primary uv params
			TSharedPtr<IPropertyHandle> UsePrimaryUVParamsHandle = MeshPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaShapeMeshData, bOverridePrimaryUVParams));
			check(UsePrimaryUVParamsHandle.IsValid());

			FString UsePrimaryParamsName = MeshName + TEXT("Override UV");
			FDetailWidgetRow& UsePrimaryUVParamsRow = MaterialCategoryBuilder.AddCustomRow(FText::FromString(UsePrimaryParamsName));

			UsePrimaryUVParamsRow.NameContent()[UsePrimaryUVParamsHandle->CreatePropertyNameWidget(FText::FromString(UsePrimaryParamsName))];
			UsePrimaryUVParamsRow.ValueContent()[UsePrimaryUVParamsHandle->CreatePropertyValueWidget()];
			UsePrimaryUVParamsRow.Visibility(MakeAttributeLambda([=]() {
				return UsePrimaryUVParamsHandle->IsEditable() ? EVisibility::Visible : EVisibility::Hidden;
			}));

			// Only Add it the first time at this specific point
			if (Index == 0 && MeshSectionNames.Num() > 1)
			{
				MaterialCategoryBuilder.AddProperty(UsePrimaryMaterialEverywhereHandle);
			}

			// uv params
			TSharedPtr<IPropertyHandle> MaterialUVHandle = MeshPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaShapeMeshData, MaterialUVParams));
			check(MaterialUVHandle.IsValid());

			IDetailPropertyRow& MaterialUVRow = MaterialCategoryBuilder.AddProperty(MaterialUVHandle.ToSharedRef());
			FString MaterialUVName = MeshName + TEXT("Material UV");
			MaterialUVRow.DisplayName(FText::FromString(MaterialUVName));
			MaterialUVRow.Visibility(MakeAttributeLambda([=]() {
				return MaterialUVHandle->IsEditable() ? EVisibility::Visible : EVisibility::Hidden;
			}));

			if (Index < MeshSectionNames.Num())
			{
				// Separator row
				FDetailWidgetRow& SeparatorRow = MaterialCategoryBuilder.AddCustomRow(FText::GetEmpty());
				SeparatorRow.WholeRowContent()[
					SNullWidget::NullWidget
				];

				// visibility for the separator row
				SeparatorRow.Visibility(MakeAttributeLambda([=]() {
					return UsePrimaryUVParamsHandle->IsEditable() || MaterialUVHandle->IsEditable() ? EVisibility::Visible : EVisibility::Hidden;
				}));
			}
		}

		const FText ExportRowText = LOCTEXT("ExportMesh", "Export Mesh");
		FDetailWidgetRow& ExportRow = ShapeCategoryBuilder.AddCustomRow(ExportRowText, /** Advanced */true);

		ExportRow
			.NameContent()
			[
				SNew(STextBlock)
				.Text(ExportRowText)
				.Font(InDetailBuilder.GetDetailFont())
			]
			.ValueContent()
			.VAlign(VAlign_Center)
			.MaxDesiredWidth(250)
			[
				SNew(SButton)
				.VAlign(VAlign_Center)
				.ToolTipText(LOCTEXT("ConvertToStaticMeshTooltip", "Create a new StaticMesh asset using current geometry from this DynamicMeshComponent. Does not modify instance."))
				.OnClicked(this, &FAvaMeshesDetailCustomization::OnConvertToStaticMeshClicked)
				.IsEnabled(this, &FAvaMeshesDetailCustomization::CanConvertToStaticMesh)
				.Content()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ConvertToStaticMesh", "Create Static Mesh"))
				]
			];
	}
}

FReply FAvaMeshesDetailCustomization::OnConvertToStaticMeshClicked()
{
	if (!CanConvertToStaticMesh())
	{
		return FReply::Handled();
	}

	UAvaShapeDynamicMeshBase* DynMesh = MeshGeneratorsWeak[0].Get();

	if (!DynMesh)
	{
		return FReply::Handled();
	}

	// generate name for asset
	FString NewNameSuggestion = TEXT("SM_MotionDesign") + DynMesh->GetMeshName();
	FString PackageName = FString(TEXT("/Game/Meshes/")) + NewNameSuggestion;
	FString AssetName;

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(PackageName, TEXT(""), PackageName, AssetName);

	TSharedPtr<SDlgPickAssetPath> PickAssetPathWidget =
		SNew(SDlgPickAssetPath)
		.Title(LOCTEXT("ConvertToStaticMeshPickName", "Choose New StaticMesh Location"))
		.DefaultAssetPath(FText::FromString(PackageName));

	if (PickAssetPathWidget->ShowModal() != EAppReturnType::Ok)
	{
		return FReply::Handled();
	}

	// get input name provided by user
	FString UserPackageName = PickAssetPathWidget->GetFullAssetPath().ToString();
	FName MeshName(*FPackageName::GetLongPackageAssetName(UserPackageName));

	// is input name valid ?
	if (MeshName == NAME_None)
	{
		// Use default if invalid
		UserPackageName = PackageName;
		MeshName = *AssetName;
	}

	const UE::Geometry::FDynamicMesh3* MeshIn = DynMesh->GetShapeMeshComponent()->GetMesh();

	// empty mesh do not export
	if (!MeshIn || MeshIn->TriangleCount() == 0)
	{
		return FReply::Handled();
	}

	// find/create package
	UPackage* Package = CreatePackage(*UserPackageName);
	check(Package);

	// Create StaticMesh object
	UStaticMesh* StaticMesh = NewObject<UStaticMesh>(Package, MeshName, RF_Public | RF_Standalone);

	if (DynMesh->ExportToStaticMesh(StaticMesh))
	{
		// Notify asset registry of new asset
		FAssetRegistryModule::AssetCreated(StaticMesh);
	}

	return FReply::Handled();
}

bool FAvaMeshesDetailCustomization::CanSizeToTexture() const
{
	bool bAllowed = false;

	for (const TWeakObjectPtr<UAvaShapeDynamicMeshBase>& MeshGeneratorWeak : MeshGeneratorsWeak)
	{
		if (const UAvaShapeDynamicMeshBase* MeshGenerator = MeshGeneratorWeak.Get())
		{
			if (MeshGenerator->IsA<UAvaShape2DDynMeshBase>())
			{
				bAllowed = true;
			}
			else
			{
				bAllowed = false;
				break;
			}
		}
	}

	return bAllowed;
}

FReply FAvaMeshesDetailCustomization::OnSizeToTextureClicked()
{
	if (!CanSizeToTexture())
	{
		return FReply::Handled();
	}

	for (const TWeakObjectPtr<UAvaShapeDynamicMeshBase>& MeshGeneratorWeak : MeshGeneratorsWeak)
	{
		ApplySizeToTexture(MeshGeneratorWeak.Get());
	}

	return FReply::Handled();
}

ECheckBoxState FAvaMeshesDetailCustomization::OnIsAutoSizeToTextureChecked() const
{
	TOptional<ECheckBoxState> State;

	for (const TWeakObjectPtr<UAvaShapeDynamicMeshBase>& MeshGeneratorWeak : MeshGeneratorsWeak)
	{
		const UAvaShapeDynamicMeshBase* MeshGenerator = MeshGeneratorWeak.Get();
		if (!MeshGenerator)
		{
			continue;
		}

		if (!MeshGenerator->IsA<UAvaShape2DDynMeshBase>())
		{
			continue;
		}

		const ECheckBoxState AutoState = MeshGenerator->ComponentTags.Contains(AutoUpdateTextureMetadata) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;

		if (!State.IsSet())
		{
			State = AutoState;
		}
		else if (State.GetValue() != AutoState)
		{
			State = ECheckBoxState::Undetermined;
			break;
		}
	}

	return State.Get(ECheckBoxState::Undetermined);
}

void FAvaMeshesDetailCustomization::OnTexturePropertyChanged()
{
	// Update texture on next tick to give it time to load resources
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSPLambda(this, [this](float)
	{
		for (const TWeakObjectPtr<UAvaShapeDynamicMeshBase>& MeshGeneratorWeak : MeshGeneratorsWeak)
		{
			UAvaShapeDynamicMeshBase* MeshGenerator = MeshGeneratorWeak.Get();
			if (!MeshGenerator)
			{
				continue;
			}

			if (!MeshGenerator->IsA<UAvaShape2DDynMeshBase>())
			{
				continue;
			}

			if (!MeshGenerator->ComponentTags.Contains(AutoUpdateTextureMetadata))
			{
				continue;
			}
			
			ApplySizeToTexture(MeshGenerator);
		}

		return false; // Stop
	}));
}

void FAvaMeshesDetailCustomization::OnAutoSizeToTextureStateChanged(ECheckBoxState InState)
{
	for (const TWeakObjectPtr<UAvaShapeDynamicMeshBase>& MeshGeneratorWeak : MeshGeneratorsWeak)
	{
		UAvaShapeDynamicMeshBase* MeshGenerator = MeshGeneratorWeak.Get();
		if (!MeshGenerator)
		{
			continue;
		}

		if (!MeshGenerator->IsA<UAvaShape2DDynMeshBase>())
		{
			continue;
		}

		if (InState == ECheckBoxState::Checked)
		{
			MeshGenerator->ComponentTags.AddUnique(AutoUpdateTextureMetadata);
		}
		else
		{
			MeshGenerator->ComponentTags.Remove(AutoUpdateTextureMetadata);
		}
	}
}

void FAvaMeshesDetailCustomization::ApplySizeToTexture(UAvaShapeDynamicMeshBase* InMeshGenerator)
{
	if (!InMeshGenerator)
	{
		return;
	}

	const FAvaShapeParametricMaterial* PrimaryParametricMaterial = InMeshGenerator->GetParametricMaterialPtr(UAvaShapeDynamicMeshBase::MESH_INDEX_PRIMARY);
	if (!PrimaryParametricMaterial)
	{
		return;
	}

	AActor* ShapeActor = InMeshGenerator->GetShapeActor();
	if (!ShapeActor)
	{
		return;
	}

	UActorModifierCoreStack* Stack = nullptr;
	if (!UActorModifierCoreLibrary::FindModifierStack(ShapeActor, Stack, /** CreateIfNone */true))
	{
		return;
	}

	UActorModifierCoreBase* SizeToTextureModifier = UActorModifierCoreLibrary::FindModifierByClass(Stack, UAvaSizeToTextureModifier::StaticClass());
	if (!SizeToTextureModifier)
	{
		FActorModifierCoreInsertOperation InsertOp;
		InsertOp.ModifierClass = UAvaSizeToTextureModifier::StaticClass();
		InsertOp.InsertPosition = EActorModifierCoreStackPosition::After;
		InsertOp.InsertPositionContext = nullptr;

		if (!UActorModifierCoreLibrary::InsertModifier(Stack, InsertOp, SizeToTextureModifier))
		{
			return;
		}
	}

	UAvaSizeToTextureModifier* SizeToTextureModifierCasted = Cast<UAvaSizeToTextureModifier>(SizeToTextureModifier);
	SizeToTextureModifierCasted->SetTexture(PrimaryParametricMaterial->GetTexture());
	InMeshGenerator->SetMaterialUVMode(UAvaShapeDynamicMeshBase::MESH_INDEX_PRIMARY, EAvaShapeUVMode::Stretch);
}

bool FAvaMeshesDetailCustomization::CanConvertToStaticMesh() const
{
	return MeshGeneratorsWeak.Num() == 1 && MeshGeneratorsWeak[0].IsValid();
}

#undef LOCTEXT_NAMESPACE