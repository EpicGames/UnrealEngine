// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FReply;
class UAvaShapeDynamicMeshBase;
enum class ECheckBoxState : uint8;

/** Used to create the details materials meshes widget and export to StaticMesh */
class FAvaMeshesDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FAvaMeshesDetailCustomization>();
	}

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization

protected:
	static const FLazyName AutoUpdateTextureMetadata;

	/** Enable or disable the button when selected object is not compatible */
	bool CanConvertToStaticMesh() const;

	/** Handler when the convert button is clicked */
	FReply OnConvertToStaticMeshClicked();

	bool CanSizeToTexture() const;
	FReply OnSizeToTextureClicked();
	ECheckBoxState OnIsAutoSizeToTextureChecked() const;
	void OnTexturePropertyChanged();
	void OnAutoSizeToTextureStateChanged(ECheckBoxState InState);
	void ApplySizeToTexture(UAvaShapeDynamicMeshBase* InMeshGenerator);

	TArray<TWeakObjectPtr<UAvaShapeDynamicMeshBase>> MeshGeneratorsWeak;
};