// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneSettingsCustomization.h"
#include "AvaSceneSettings.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IAvaAttributeEditorModule.h"
#include "IAvaEditor.h"
#include "IAvaEditorExtension.h"
#include "IAvaSceneRigEditorModule.h"
#include "PropertyHandle.h"
#include "Templates/SharedPointer.h"

TSharedRef<IDetailCustomization> FAvaSceneSettingsCustomization::MakeDefaultInstance()
{
	return FAvaSceneSettingsCustomization::MakeInstance(nullptr);
}

TSharedRef<IDetailCustomization> FAvaSceneSettingsCustomization::MakeInstance(TWeakPtr<IAvaEditor> InEditorWeak)
{
	return MakeShared<FAvaSceneSettingsCustomization>(InEditorWeak);
}

FAvaSceneSettingsCustomization::FAvaSceneSettingsCustomization(const TWeakPtr<IAvaEditor>& InEditorWeak)
	: EditorWeak(InEditorWeak)
{
}

void FAvaSceneSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	if (TSharedPtr<IPropertyHandle> SceneRigHandle = InDetailBuilder.GetProperty(UAvaSceneSettings::GetSceneRigPropertyName()))
	{
		IAvaSceneRigEditorModule::Get().CustomizeSceneRig(SceneRigHandle.ToSharedRef(), InDetailBuilder);
	}

	if (TSharedPtr<IPropertyHandle> AttributesHandle = InDetailBuilder.GetProperty(UAvaSceneSettings::GetSceneAttributesPropertyName()))
	{
		IAvaAttributeEditorModule::Get().CustomizeAttributes(AttributesHandle.ToSharedRef(), InDetailBuilder);
	}

	if (TSharedPtr<IAvaEditor> Editor = EditorWeak.Pin())
	{
		for (const TSharedRef<IAvaEditorExtension>& Extension : Editor->GetExtensions())
		{
			FName CategoryName = Extension->GetCategoryName();
			if (!CategoryName.IsNone())
			{
				Extension->ExtendSettingsCategory(InDetailBuilder.EditCategory(CategoryName));
			}
		}
	}
}
