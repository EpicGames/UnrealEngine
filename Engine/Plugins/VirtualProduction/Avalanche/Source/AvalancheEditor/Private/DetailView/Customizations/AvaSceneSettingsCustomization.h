// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointerFwd.h"

class IAvaEditor;
class IDetailLayoutBuilder;

class FAvaSceneSettingsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeDefaultInstance();

	static TSharedRef<IDetailCustomization> MakeInstance(TWeakPtr<IAvaEditor> InEditorWeak);

	explicit FAvaSceneSettingsCustomization(const TWeakPtr<IAvaEditor>& InEditorWeak);

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization

private:
	TWeakPtr<IAvaEditor> EditorWeak;
};
