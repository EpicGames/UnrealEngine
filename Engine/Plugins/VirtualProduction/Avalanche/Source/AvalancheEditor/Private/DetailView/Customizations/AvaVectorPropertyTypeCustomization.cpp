// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaVectorPropertyTypeCustomization.h"
#include "AvaEditorStyle.h"
#include "AvaEditorSubsystem.h"
#include "AvaEditorViewportUtils.h"
#include "AvaViewportUtils.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DynamicMeshes/AvaShape2DDynMeshBase.h"
#include "DynamicMeshes/AvaShape3DDynMeshBase.h"
#include "Editor.h"
#include "Viewport/AvaViewportExtension.h"
#include "ViewportClient/IAvaViewportClient.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SScaleBox.h"

#define LOCTEXT_NAMESPACE "AvaVectorPropertyTypeCustomization"

void FAvaVectorPropertyTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle
	, FDetailWidgetRow& HeaderRow
	, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	VectorPropertyHandle = StructPropertyHandle;
	const FString Type = StructPropertyHandle->GetProperty()->GetCPPType();
	bIsVector3d = Type == "FVector";
	SelectedObjectNum = VectorPropertyHandle->GetNumPerObjectValues();

	if (SelectedObjectNum > 0)
	{
		TArray<UObject*> OuterObjects;
		VectorPropertyHandle->GetOuterObjects(OuterObjects);

		// All outer objects should have the same world? So just use the first
		if (OuterObjects.Num() > 0)
		{
			if (UAvaEditorSubsystem* AvaEditorSubsystem = UAvaEditorSubsystem::Get(OuterObjects[0]))
			{
				if (TSharedPtr<FAvaViewportExtension> ViewportExtension = AvaEditorSubsystem->FindExtension<FAvaViewportExtension>())
				{
					TArray<TSharedPtr<IAvaViewportClient>> ViewportClients = ViewportExtension->GetViewportClients();

					if (!ViewportClients.IsEmpty())
					{
						ViewportClient = ViewportClients[0];
					}
				}
			}
		}
	}

	// Assign Name Widget
	HeaderRow.NameContent()[StructPropertyHandle->CreatePropertyNameWidget()];

	// Fill available space
	HeaderRow.ValueWidget.HorizontalAlignment = EHorizontalAlignment::HAlign_Fill;
	HeaderRow.ValueWidget.VerticalAlignment = EVerticalAlignment::VAlign_Fill;

	FName RatioNone(TEXT("Ratio"));
	RatioNone.SetNumber(static_cast<int32>(ERatioMode::None));
	RatioModes.Add(RatioNone);

	FName RatioXY(TEXT("Ratio"));
	RatioXY.SetNumber(static_cast<int32>(ERatioMode::PreserveXY));
	RatioModes.Add(RatioXY);

	if (bIsVector3d)
	{
		FName RatioXZ(TEXT("Ratio"));
		RatioXZ.SetNumber(static_cast<int32>(ERatioMode::PreserveXZ));
		RatioModes.Add(RatioXZ);

		FName RatioYZ(TEXT("Ratio"));
		RatioYZ.SetNumber(static_cast<int32>(ERatioMode::PreserveYZ));
		RatioModes.Add(RatioYZ);

		FName RatioXYZ(TEXT("Ratio"));
		RatioXYZ.SetNumber(static_cast<int32>(ERatioMode::PreserveXYZ));
		RatioModes.Add(RatioXYZ);
	}

	const TSharedPtr<SWidget> PreserveRatioWidget = SNew(SBox)
		.MinDesiredWidth(60.f)
		.Visibility(this, &FAvaVectorPropertyTypeCustomization::GetRatioWidgetVisibility)
		[
			SNew(SComboBox<FName>)
			.ComboBoxStyle(&FAppStyle::Get().GetWidgetStyle<FComboBoxStyle>(TEXT("ComboBox")))
			.OptionsSource(&RatioModes)
			.HasDownArrow(false)
			.InitiallySelectedItem(GetRatioCurrentItem())
			.ToolTipText(LOCTEXT("PreserveRatioTooltip", "Select the ratio mode to lock for the component axis"))
			.ContentPadding(0.f)
			.OnGenerateWidget(this, &FAvaVectorPropertyTypeCustomization::OnGenerateRatioWidget)
			.OnSelectionChanged(this, &FAvaVectorPropertyTypeCustomization::OnRatioSelectionChanged)
			.Content()
			[
				OnGenerateRatioWidget(NAME_None)
			]
		];

	if (bIsVector3d)
	{
		bPixelSizeProperty = VectorPropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UAvaShape3DDynMeshBase, PixelSize3D);

		VectorComponentPropertyHandles =
		{
			StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FVector, X)),
			StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FVector, Y)),
			StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FVector, Z)),
		};

		if (StructPropertyHandle->HasMetaData("ClampMin"))
		{
			MinVectorClamp = FVector(StructPropertyHandle->GetFloatMetaData("ClampMin"));
		}

		if (StructPropertyHandle->HasMetaData("ClampMax"))
		{
			MaxVectorClamp = FVector(StructPropertyHandle->GetFloatMetaData("ClampMax"));
		}

		float SpinDelta = 1.f;
		if (StructPropertyHandle->HasMetaData("Delta"))
		{
			SpinDelta = StructPropertyHandle->GetFloatMetaData("Delta");
		}
		// compute spin delta based on min and max value in percentage (100%)
		else if (MinVectorClamp.IsSet() && MaxVectorClamp.IsSet())
		{
			SpinDelta = (MaxVectorClamp->X - MinVectorClamp->X) / 100.f;
		}

		HeaderRow.ValueContent()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f, 2.f, 0.f)
			[
				PreserveRatioWidget.ToSharedRef()
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SNumericVectorInputBox3D)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.X(this, &FAvaVectorPropertyTypeCustomization::GetVectorComponent,  static_cast<uint8>(0)) // X
				.Y(this, &FAvaVectorPropertyTypeCustomization::GetVectorComponent,  static_cast<uint8>(1)) // Y
				.Z(this, &FAvaVectorPropertyTypeCustomization::GetVectorComponent,  static_cast<uint8>(2)) // Z
				.bColorAxisLabels(true)
				.MinVector(MinVectorClamp)
				.MaxVector(MaxVectorClamp)
				.MinSliderVector(MinVectorClamp)
				.MaxSliderVector(MaxVectorClamp)
				.OnXChanged(this, &FAvaVectorPropertyTypeCustomization::SetVectorComponent,  static_cast<uint8>(0))
				.OnYChanged(this, &FAvaVectorPropertyTypeCustomization::SetVectorComponent,  static_cast<uint8>(1))
				.OnZChanged(this, &FAvaVectorPropertyTypeCustomization::SetVectorComponent,  static_cast<uint8>(2))
				.OnXCommitted(this, &FAvaVectorPropertyTypeCustomization::SetVectorComponent,  static_cast<uint8>(0))
				.OnYCommitted(this, &FAvaVectorPropertyTypeCustomization::SetVectorComponent,  static_cast<uint8>(1))
				.OnZCommitted(this, &FAvaVectorPropertyTypeCustomization::SetVectorComponent,  static_cast<uint8>(2))
				.AllowSpin(true)
				.SpinDelta(SpinDelta)
				.IsEnabled(this, &FAvaVectorPropertyTypeCustomization::CanEditValue)
				.OnBeginSliderMovement(this, &FAvaVectorPropertyTypeCustomization::OnBeginSliderMovement)
				.OnEndSliderMovement(this, &FAvaVectorPropertyTypeCustomization::OnEndSliderMovement)
			]
		];
	}
	else
	{
		bPixelSizeProperty = VectorPropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UAvaShape2DDynMeshBase, PixelSize2D);

		VectorComponentPropertyHandles =
		{
			StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FVector2D, X)),
			StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FVector2D, Y)),
		};

		if (StructPropertyHandle->HasMetaData("ClampMin"))
		{
			MinVector2DClamp = FVector2D(StructPropertyHandle->GetFloatMetaData("ClampMin"));
		}

		if (StructPropertyHandle->HasMetaData("ClampMax"))
		{
			MaxVector2DClamp = FVector2D(StructPropertyHandle->GetFloatMetaData("ClampMax"));
		}

		float SpinDelta = 1.f;
		if (StructPropertyHandle->HasMetaData("Delta"))
		{
			SpinDelta = StructPropertyHandle->GetFloatMetaData("Delta");
		}
		// compute spin delta based on min and max value in percentage (100%)
		else if (MinVector2DClamp.IsSet() && MaxVector2DClamp.IsSet())
		{
			SpinDelta = (MaxVector2DClamp->X - MinVector2DClamp->X) / 100.f;
		}

		HeaderRow.ValueContent()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f, 2.f, 0.f)
			[
				PreserveRatioWidget.ToSharedRef()
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SNumericVectorInputBox2D)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.X(this, &FAvaVectorPropertyTypeCustomization::GetVectorComponent, static_cast<uint8>(0)) // X
				.Y(this, &FAvaVectorPropertyTypeCustomization::GetVectorComponent,  static_cast<uint8>(1)) // Y
				.bColorAxisLabels(true)
				.MinVector(MinVector2DClamp)
				.MaxVector(MaxVector2DClamp)
				.MinSliderVector(MinVector2DClamp)
				.MaxSliderVector(MaxVector2DClamp)
				.OnXChanged(this, &FAvaVectorPropertyTypeCustomization::SetVectorComponent,  static_cast<uint8>(0))
				.OnYChanged(this, &FAvaVectorPropertyTypeCustomization::SetVectorComponent,  static_cast<uint8>(1))
				.OnXCommitted(this, &FAvaVectorPropertyTypeCustomization::SetVectorComponent,  static_cast<uint8>(0))
				.OnYCommitted(this, &FAvaVectorPropertyTypeCustomization::SetVectorComponent,  static_cast<uint8>(1))
				.AllowSpin(true)
				.SpinDelta(SpinDelta)
				.IsEnabled(this, &FAvaVectorPropertyTypeCustomization::CanEditValue)
				.OnBeginSliderMovement(this, &FAvaVectorPropertyTypeCustomization::OnBeginSliderMovement)
				.OnEndSliderMovement(this, &FAvaVectorPropertyTypeCustomization::OnEndSliderMovement)
			]
		];
	}
}

void FAvaVectorPropertyTypeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle,
	IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

const FSlateBrush* FAvaVectorPropertyTypeCustomization::GetRatioModeBrush(const ERatioMode InMode) const
{
	if (InMode == ERatioMode::None)
	{
		return FAvaEditorStyle::Get().GetBrush("Icons.Unlock");
	}

	if (InMode == ERatioMode::PreserveXY || InMode == ERatioMode::PreserveXZ || InMode == ERatioMode::PreserveYZ)
	{
		return FAvaEditorStyle::Get().GetBrush("Icons.Lock2d");
	}

	return FAvaEditorStyle::Get().GetBrush("Icons.Lock3d");
}

FText FAvaVectorPropertyTypeCustomization::GetRatioModeDisplayText(const ERatioMode InMode) const
{
	FText DisplayText = FText::GetEmpty();

	switch (InMode)
	{
		case ERatioMode::PreserveXY:
			DisplayText = FText::FromString("XY ");
			break;
		case ERatioMode::PreserveYZ:
			DisplayText = FText::FromString("YZ ");
			break;
		case ERatioMode::PreserveXZ:
			DisplayText = FText::FromString("XZ ");
			break;
		case ERatioMode::PreserveXYZ:
			DisplayText = FText::FromString("XYZ");
			break;
		default:
			DisplayText = FText::FromString("Free");
			break;
	}

	return DisplayText;
}

const FSlateBrush* FAvaVectorPropertyTypeCustomization::GetCurrentRatioModeBrush() const
{
	return GetRatioModeBrush(GetRatioModeMetadata());
}

FText FAvaVectorPropertyTypeCustomization::GetCurrentRatioModeDisplayText() const
{
	return GetRatioModeDisplayText(GetRatioModeMetadata());
}

ERatioMode FAvaVectorPropertyTypeCustomization::GetRatioModeMetadata() const
{
	ERatioMode RatioMode = ERatioMode::None;

	if (VectorPropertyHandle.IsValid() && VectorPropertyHandle->IsValidHandle())
	{
		const FString& MetadataValue = VectorPropertyHandle->GetMetaData(PropertyMetadata);

		if (MetadataValue.Contains(TEXT("X")))
		{
			RatioMode |= ERatioMode::X;
		}

		if (MetadataValue.Contains(TEXT("Y")))
		{
			RatioMode |= ERatioMode::Y;
		}

		if (bIsVector3d && MetadataValue.Contains(TEXT("Z")))
		{
			RatioMode |= ERatioMode::Z;
		}
	}

	return RatioMode;
}

void FAvaVectorPropertyTypeCustomization::SetRatioModeMetadata(ERatioMode InMode) const
{
	if (VectorPropertyHandle.IsValid() && VectorPropertyHandle->IsValidHandle())
	{
		FProperty* VectorProperty = VectorPropertyHandle->GetProperty();

		FString NewMetadataValue = TEXT("");

		if ((InMode & ERatioMode::X) != ERatioMode::None)
		{
			NewMetadataValue += TEXT("X");
		}

		if ((InMode & ERatioMode::Y) != ERatioMode::None)
		{
			NewMetadataValue += TEXT("Y");
		}

		if (bIsVector3d && (InMode & ERatioMode::Z) != ERatioMode::None)
		{
			NewMetadataValue += TEXT("Z");
		}

		VectorProperty->SetMetaData(PropertyMetadata, *NewMetadataValue);
	}
}

EVisibility FAvaVectorPropertyTypeCustomization::GetRatioWidgetVisibility() const
{
	if (VectorPropertyHandle.IsValid() && VectorPropertyHandle->IsValidHandle())
	{
		// Only show preserve ratio widget if AllowPreserveRatio is set
		return VectorPropertyHandle->HasMetaData(PropertyMetadata)
			? EVisibility::Visible
			: EVisibility::Collapsed;
	}

	return EVisibility::Collapsed;
}

TOptional<double> FAvaVectorPropertyTypeCustomization::GetVectorComponent(const uint8 InComponent) const
{
	if (!VectorPropertyHandle.IsValid() || SelectedObjectNum == 0)
	{
		return TOptional<double>();
	}

	if (!VectorComponentPropertyHandles.IsValidIndex(InComponent) || !VectorComponentPropertyHandles[InComponent].IsValid())
	{
		return TOptional<double>();
	}

	double OutValue = 0.f;

	if (VectorComponentPropertyHandles[InComponent]->GetValue(OutValue) != FPropertyAccess::Success)
	{
		return TOptional<double>();
	}

	// handle specific case
	if (bPixelSizeProperty)
	{
		return MeshSizeToPixelSize(OutValue);
	}

	return OutValue;
}

void FAvaVectorPropertyTypeCustomization::SetVectorComponent(double InNewValue, const uint8 InComponent)
{
	if (bMovingSlider)
	{
		SetVectorComponent(InNewValue, ETextCommit::Default, InComponent);
	}
}

void FAvaVectorPropertyTypeCustomization::SetVectorComponent(double InNewValue, ETextCommit::Type InCommitType, const uint8 InComponent)
{
	const bool bFinalCommit = InCommitType == ETextCommit::OnEnter || InCommitType == ETextCommit::OnUserMovedFocus;
	if (!bFinalCommit && InCommitType != ETextCommit::Default)
	{
		return;
	}
	if (!VectorPropertyHandle.IsValid())
	{
		return;
	}
	// init here in case we only input the value without slider movement
	if ((!bIsVector3d && Begin2DValues.Num() == 0) ||
		(bIsVector3d && Begin3DValues.Num() == 0))
	{
		InitVectorValuesForRatio();
	}
	if (SelectedObjectNum == 0)
	{
		return;
	}
	// handle interactive debounce to avoid slow behaviour
	LastComponentValueSet = InComponent;
	if (DebounceValueSet > 0 && !bFinalCommit)
	{
		DebounceValueSet--;
		return;
	}
	DebounceValueSet = FMath::Clamp(SelectedObjectNum > 1 ? (SelectedObjectNum * MULTI_OBJECT_DEBOUNCE) : SINGLE_OBJECT_DEBOUNCE, 0, 255);
	// handle specific case for pixel size property
	if (bPixelSizeProperty)
	{
		InNewValue = PixelSizeToMeshSize(InNewValue);
	}
	if (!bMovingSlider)
	{
		GEditor->BeginTransaction(VectorPropertyHandle->GetPropertyDisplayName());
	}
	// update objects value for property, we handle transaction ourselves to batch properties changes together
	const EPropertyValueSetFlags::Type Flags = bMovingSlider ? EPropertyValueSetFlags::InteractiveChange : EPropertyValueSetFlags::NotTransactable;
	SetComponentValue(InNewValue, InComponent, Flags);
	if (!bMovingSlider)
	{
		GEditor->EndTransaction();
		ResetVectorValuesForRatio();
	}
}

void FAvaVectorPropertyTypeCustomization::OnBeginSliderMovement()
{
	LastComponentValueSet = INVALID_COMPONENT_IDX;
	DebounceValueSet = 0;
	InitVectorValuesForRatio();
	bMovingSlider = true;
	GEditor->BeginTransaction(VectorPropertyHandle->GetPropertyDisplayName());
}

void FAvaVectorPropertyTypeCustomization::OnEndSliderMovement(double InNewValue)
{
	DebounceValueSet = 0;
	bMovingSlider = false;
	// set final value like enter pressed
	if (LastComponentValueSet != INVALID_COMPONENT_IDX)
	{
		SetVectorComponent(InNewValue, ETextCommit::OnEnter, LastComponentValueSet);
	}
	// end started transactions during process
	while(GEditor->IsTransactionActive())
	{
		GEditor->EndTransaction();
	}
	ResetVectorValuesForRatio();
}

TSharedRef<SWidget> FAvaVectorPropertyTypeCustomization::OnGenerateRatioWidget(FName InRatioMode)
{
	TSharedPtr<SImage> ImageWidget;
	TSharedPtr<STextBlock> TextWidget;

	const FVector2d ImageSize(16.f);

	if (InRatioMode.IsNone())
	{
		ImageWidget = SNew(SImage)
			.ColorAndOpacity(FAppStyle::GetSlateColor("SelectionColor"))
			.DesiredSizeOverride(ImageSize)
			.Image(this, &FAvaVectorPropertyTypeCustomization::GetCurrentRatioModeBrush);

		TextWidget = SNew(STextBlock)
			.Justification(ETextJustify::Right)
			.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
			.Text(this, &FAvaVectorPropertyTypeCustomization::GetCurrentRatioModeDisplayText);
	}
	else
	{
		const ERatioMode RatioMode = static_cast<ERatioMode>(InRatioMode.GetNumber());

		ImageWidget = SNew(SImage)
			.ColorAndOpacity(FAppStyle::GetSlateColor("SelectionColor"))
			.DesiredSizeOverride(ImageSize)
			.Image(GetRatioModeBrush(RatioMode));

		TextWidget = SNew(STextBlock)
			.Justification(ETextJustify::Center)
			.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
			.Text(GetRatioModeDisplayText(RatioMode));
	}

	return SNew(SHorizontalBox)
		.Visibility(EVisibility::Visible)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.f)
		[
			SNew(SScaleBox)
			.Visibility(EVisibility::HitTestInvisible)
			.Stretch(EStretch::UserSpecified)
			.UserSpecifiedScale(1.f)
			[
				ImageWidget.ToSharedRef()
			]
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.Padding(8.f, 0.f, 0.f, 0.f)
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			TextWidget.ToSharedRef()
		];
}

void FAvaVectorPropertyTypeCustomization::OnRatioSelectionChanged(FName InRatioMode, ESelectInfo::Type InSelectInfo) const
{
	if (!InRatioMode.IsNone())
	{
		const ERatioMode RatioMode = static_cast<ERatioMode>(InRatioMode.GetNumber());
		SetRatioModeMetadata(RatioMode);
	}
}

FName FAvaVectorPropertyTypeCustomization::GetRatioCurrentItem() const
{
	const ERatioMode CurrentRatioMode = GetRatioModeMetadata();
	FName RatioName(TEXT("Ratio"));
	RatioName.SetNumber(static_cast<int32>(CurrentRatioMode));
	return RatioName;
}

bool FAvaVectorPropertyTypeCustomization::CanEditValue() const
{
	if (!VectorPropertyHandle.IsValid())
	{
		return false;
	}

	if (bPixelSizeProperty && SelectedObjectNum > 0 && ViewportClient.IsValid())
	{
		if (FAvaViewportUtils::IsValidViewportSize(ViewportClient.Pin()->GetVirtualViewportSize()))
		{
			return VectorPropertyHandle->IsEditable();
		}

		return false;
	}

	return VectorPropertyHandle->IsEditable();
}

void FAvaVectorPropertyTypeCustomization::InitVectorValuesForRatio()
{
	Begin2DValues.Empty();
	Begin3DValues.Empty();

	TArray<FString> OutValues;
	VectorPropertyHandle->GetPerObjectValues(OutValues);
	SelectedObjectNum = VectorPropertyHandle->GetNumPerObjectValues();

	if (SelectedObjectNum > 0)
	{
		for (const FString& Val : OutValues)
		{
			if (bIsVector3d)
			{
				FVector Vector;
				if (Vector.InitFromString(Val))
				{
					Begin3DValues.Add(Vector);
				}
				else
				{
					TOptional<FVector> OptionalVector;
					Begin3DValues.Add(OptionalVector);
				}
			}
			else
			{
				FVector2D Vector;
				if (Vector.InitFromString(Val))
				{
					Begin2DValues.Add(Vector);
				}
				else
				{
					TOptional<FVector2D> OptionalVector;
					Begin2DValues.Add(OptionalVector);
				}
			}
		}
	}
}

void FAvaVectorPropertyTypeCustomization::ResetVectorValuesForRatio()
{
	Begin2DValues.Empty();
	Begin3DValues.Empty();
}

void FAvaVectorPropertyTypeCustomization::SetComponentValue(const double InNewValue, const uint8 InComponent, const EPropertyValueSetFlags::Type InFlags)
{
	const ERatioMode RatioMode = GetRatioModeMetadata();

	// check if we are preserving ratio for current component change
	bool bPreserveRatio = false;
	switch (InComponent)
	{
		case 0:
			bPreserveRatio = ((RatioMode & ERatioMode::X) != ERatioMode::None);
		break;
		case 1:
			bPreserveRatio = ((RatioMode & ERatioMode::Y) != ERatioMode::None);
		break;
		case 2:
			bPreserveRatio = ((RatioMode & ERatioMode::Z) != ERatioMode::None);
		break;
		default:
		break;
	}

	const TArray<bool> PreserveRatios
	{
		(RatioMode & ERatioMode::X) != ERatioMode::None && bPreserveRatio,
		(RatioMode & ERatioMode::Y) != ERatioMode::None && bPreserveRatio,
		(RatioMode & ERatioMode::Z) != ERatioMode::None && bPreserveRatio
	};

	// set property per object since we need to handle ratios for each objects
	const uint8 MaxComponentCount = bIsVector3d ? 3 : 2;

	TArray<TArray<FString>> ComponentValues;
	ComponentValues.SetNum(MaxComponentCount);

	for (int32 ObjIdx = 0; ObjIdx < SelectedObjectNum; ObjIdx++)
	{
		if (!bIsVector3d && !Begin2DValues.IsValidIndex(ObjIdx) && !Begin2DValues[ObjIdx].IsSet())
		{
			continue;
		}
		if (bIsVector3d && !Begin3DValues.IsValidIndex(ObjIdx) && !Begin3DValues[ObjIdx].IsSet())
		{
			continue;
		}

		// compute clamped ratio for value change
		const double ClampedRatio = GetClampedRatioValueChange(ObjIdx, InNewValue, InComponent, PreserveRatios);

		// loop for each component (X,Y,Z)
		for (uint8 ComponentIdx = 0; ComponentIdx < MaxComponentCount; ComponentIdx++)
		{
			// only assign value to specific component, skip others
			if (!bPreserveRatio && ComponentIdx != InComponent)
			{
				continue;
			}

			// compute new component value
			const double NewComponentValue = GetClampedComponentValue(ObjIdx, InNewValue, ClampedRatio, ComponentIdx, InComponent);

			switch(ComponentIdx)
			{
				case 0:
					if (PreserveRatios[0] || ComponentIdx == InComponent)
					{
						ComponentValues[ComponentIdx].Emplace(FString::SanitizeFloat(NewComponentValue));
					}
					break;
				case 1:
					if (PreserveRatios[1] || ComponentIdx == InComponent)
					{
						ComponentValues[ComponentIdx].Emplace(FString::SanitizeFloat(NewComponentValue));
					}
					break;
				case 2:
					if (PreserveRatios[2] || ComponentIdx == InComponent)
					{
						ComponentValues[ComponentIdx].Emplace(FString::SanitizeFloat(NewComponentValue));
					}
					break;
				default:;
			}
		}
	}

	for (int32 Index = 0; Index < VectorComponentPropertyHandles.Num(); Index++)
	{
		if (!ComponentValues[Index].IsEmpty() && VectorComponentPropertyHandles[Index].IsValid())
		{
			VectorComponentPropertyHandles[Index]->SetPerObjectValues(ComponentValues[Index], InFlags);
		}
	}
}

double FAvaVectorPropertyTypeCustomization::GetClampedRatioValueChange(const int32 InObjectIdx, const double InNewValue, const uint8 InComponent, const TArray<bool>& InPreserveRatios) const
{
	double Ratio = 1;
	// get pre change value for this component
	if (bIsVector3d)
	{
		FVector BeginValue = Begin3DValues[InObjectIdx].GetValue();
		if (BeginValue[InComponent] != 0)
		{
			Ratio = InNewValue / BeginValue[InComponent];
		}
		// apply min/max clamp
		if (MinVectorClamp.IsSet() || MaxVectorClamp.IsSet())
		{
			for (int32 ComponentIdx = 0; ComponentIdx < 3; ComponentIdx++)
			{
				if (InPreserveRatios[ComponentIdx] || ComponentIdx == InComponent)
				{
					const double EndValue = BeginValue[ComponentIdx] * Ratio;
					if (MinVectorClamp.IsSet())
					{
						const double MinValue = MinVectorClamp.GetValue()[ComponentIdx];
						if (EndValue < MinValue)
						{
							Ratio = MinValue / BeginValue[ComponentIdx];
						}
					}
					if (MaxVectorClamp.IsSet())
					{
						const double MaxValue = MaxVectorClamp.GetValue()[ComponentIdx];
						if (EndValue > MaxValue)
						{
							Ratio = MaxValue / BeginValue[ComponentIdx];
						}
					}
				}
			}
		}
	}
	else
	{
		FVector2D BeginValue = Begin2DValues[InObjectIdx].GetValue();
		if (BeginValue[InComponent] != 0)
		{
			Ratio = InNewValue / BeginValue[InComponent];
		}
		// apply min/max clamp
		if (MinVector2DClamp.IsSet() || MaxVector2DClamp.IsSet())
		{
			for (int32 ComponentIdx = 0; ComponentIdx < 2; ComponentIdx++)
			{
				if (InPreserveRatios[ComponentIdx] || ComponentIdx == InComponent)
				{
					const double EndValue = BeginValue[ComponentIdx] * Ratio;
					if (MinVector2DClamp.IsSet())
					{
						const double MinValue = MinVector2DClamp.GetValue()[ComponentIdx];
						if (EndValue < MinValue)
						{
							Ratio = MinValue / BeginValue[ComponentIdx];
						}
					}
					if (MaxVector2DClamp.IsSet())
					{
						const double MaxValue = MaxVector2DClamp.GetValue()[ComponentIdx];
						if (EndValue > MaxValue)
						{
							Ratio = MaxValue / BeginValue[ComponentIdx];
						}
					}
				}
			}
		}
	}
	return Ratio;
}

double FAvaVectorPropertyTypeCustomization::GetClampedComponentValue(const int32 InObjectIdx, double InNewValue, const double InRatio, const uint8 InComponentIdx, const uint8 InOriginalComponent)
{
	const double OldValue = (bIsVector3d ? Begin3DValues[InObjectIdx].GetValue()[InComponentIdx] : Begin2DValues[InObjectIdx].GetValue()[InComponentIdx]);
	const double SliderOriginalValue = (bIsVector3d ? Begin3DValues[InObjectIdx].GetValue()[InOriginalComponent] : Begin2DValues[InObjectIdx].GetValue()[InOriginalComponent]);
	if (SliderOriginalValue == 0 && OldValue == 0)
	{
		if (bIsVector3d)
		{
			if (MinVectorClamp.IsSet())
			{
				InNewValue = FMath::Max(InNewValue, MinVectorClamp.GetValue()[InComponentIdx]);
			}
			if (MaxVectorClamp.IsSet())
			{
				InNewValue = FMath::Min(InNewValue, MaxVectorClamp.GetValue()[InComponentIdx]);
			}
		}
		else
		{
			if (MinVector2DClamp.IsSet())
			{
				InNewValue = FMath::Max(InNewValue, MinVector2DClamp.GetValue()[InComponentIdx]);
			}
			if (MaxVector2DClamp.IsSet())
			{
				InNewValue = FMath::Min(InNewValue, MaxVector2DClamp.GetValue()[InComponentIdx]);
			}
		}
		return InNewValue;
	}
	return OldValue * InRatio;
}

double FAvaVectorPropertyTypeCustomization::MeshSizeToPixelSize(double InMeshSize) const
{
	if (ViewportClient.IsValid())
	{
		double PixelSize;

		if (FAvaEditorViewportUtils::MeshSizeToPixelSize(ViewportClient.Pin().ToSharedRef(), InMeshSize, PixelSize))
		{
			return PixelSize;
		}
	}

	return InMeshSize;
}

double FAvaVectorPropertyTypeCustomization::PixelSizeToMeshSize(double InPixelSize) const
{
	if (ViewportClient.IsValid())
	{
		double MeshSize;

		if (FAvaEditorViewportUtils::PixelSizeToMeshSize(ViewportClient.Pin().ToSharedRef(), InPixelSize, MeshSize))
		{
			return MeshSize;
		}
	}

	return InPixelSize;
}

#undef LOCTEXT_NAMESPACE
