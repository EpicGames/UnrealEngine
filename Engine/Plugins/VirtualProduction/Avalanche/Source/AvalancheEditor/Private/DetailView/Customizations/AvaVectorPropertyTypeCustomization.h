// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "AvaVectorPropertyTypeCustomization.generated.h"

class IAvaViewportClient;
class IDetailChildrenBuilder;
class SButton;
class SComboButton;
struct EVisibility;

/** Used to detect if the type customization can be applied to a property */
class FAvaVectorPropertyTypeIdentifier : public IPropertyTypeIdentifier
{
public:
	virtual bool IsPropertyTypeCustomized(const IPropertyHandle& InPropertyHandle) const override
	{
		static const FName PropertyMetaTag = TEXT("MotionDesignVectorWidget");
		return InPropertyHandle.HasMetaData(PropertyMetaTag);
	}
};

UENUM()
enum class ERatioMode : uint8
{
	None = 0,                        // Free
	X = 1 << 0,
	Y = 1 << 1,
	Z = 1 << 2,
	PreserveXY = X | Y,              // Lock XY
	PreserveYZ = Y | Z,              // Lock YZ  (3D)
	PreserveXZ = X | Z,              // Lock XZ  (3D)
	PreserveXYZ = X | Y | Z          // Lock XYZ (3D)
};

ENUM_CLASS_FLAGS(ERatioMode)

class FAvaVectorPropertyTypeCustomization : public IPropertyTypeCustomization
{
public:
	using SNumericVectorInputBox2D = SNumericVectorInputBox<double, UE::Math::TVector2<double>, 2>;
	using SNumericVectorInputBox3D = SNumericVectorInputBox<FVector::FReal, UE::Math::TVector<FVector::FReal>, 3>;

	static constexpr const TCHAR* PropertyMetadata = TEXT("AllowPreserveRatio");
	static constexpr uint8 MULTI_OBJECT_DEBOUNCE  = 3;
	static constexpr uint8 SINGLE_OBJECT_DEBOUNCE = 2;
	static constexpr uint8 INVALID_COMPONENT_IDX  = 5;

	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FAvaVectorPropertyTypeCustomization>();
	}

	explicit FAvaVectorPropertyTypeCustomization()
	{
	}

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle,
		FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle,
		IDetailChildrenBuilder& StructBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	//~ End IPropertyTypeCustomization

protected:
	TOptional<double> GetVectorComponent(const uint8 InComponent) const;
	void SetVectorComponent(double InNewValue, const uint8 InComponent);
	void SetVectorComponent(double InNewValue, ETextCommit::Type InCommitType, const uint8 InComponent);

	void OnBeginSliderMovement();
	void OnEndSliderMovement(double InNewValue);

	TSharedRef<SWidget> OnGenerateRatioWidget(FName InRatioMode);
	void OnRatioSelectionChanged(FName InRatioMode, ESelectInfo::Type InSelectInfo) const;
	FName GetRatioCurrentItem() const;
	EVisibility GetRatioWidgetVisibility() const;

	const FSlateBrush* GetRatioModeBrush(const ERatioMode InMode) const;
	FText GetRatioModeDisplayText(const ERatioMode InMode) const;

	const FSlateBrush* GetCurrentRatioModeBrush() const;
	FText GetCurrentRatioModeDisplayText() const;

	ERatioMode GetRatioModeMetadata() const;
	void SetRatioModeMetadata(ERatioMode InMode) const;

	bool CanEditValue() const;
	void InitVectorValuesForRatio();
	void ResetVectorValuesForRatio();

	void SetComponentValue(const double InNewValue, const uint8 InComponent, const EPropertyValueSetFlags::Type InFlags);

	/** get the correct clamped ratio if a component hit min/max value */
	double GetClampedRatioValueChange(const int32 InObjectIdx, const double InNewValue, const uint8 InComponent, const TArray<bool>& InPreserveRatios) const;

	/** get the new clamped value if a component original value is zero since ratio * 0 = 0  */
	double GetClampedComponentValue(const int32 InObjectIdx, double InNewValue, const double InRatio, const uint8 InComponentIdx, const uint8 InOriginalComponent);

	/** special case for the pixel property only available in editor */
	double MeshSizeToPixelSize(double InMeshSize) const;

	double PixelSizeToMeshSize(double InPixelSize) const;

private:
	TWeakPtr<IAvaViewportClient> ViewportClient;

	TSharedPtr<IPropertyHandle> VectorPropertyHandle;
	TArray<TSharedPtr<IPropertyHandle>> VectorComponentPropertyHandles;

	/** optional begin values to compute ratios change */
	TArray<TOptional<FVector>> Begin3DValues;
	TArray<TOptional<FVector2D>> Begin2DValues;

	int32 SelectedObjectNum = 0;
	uint8 DebounceValueSet = 0;
	uint8 LastComponentValueSet = INDEX_NONE;
	bool bMovingSlider = false;
	bool bIsVector3d = false;

	/** specific case to handle that needs conversion */
	bool bPixelSizeProperty = false;

	/** optional clamp values */
	TOptional<FVector> MinVectorClamp;
	TOptional<FVector> MaxVectorClamp;
	TOptional<FVector2D> MinVector2DClamp;
	TOptional<FVector2D> MaxVector2DClamp;

	/** Ratio modes available for the property dropdown */
	TArray<FName> RatioModes;
};
