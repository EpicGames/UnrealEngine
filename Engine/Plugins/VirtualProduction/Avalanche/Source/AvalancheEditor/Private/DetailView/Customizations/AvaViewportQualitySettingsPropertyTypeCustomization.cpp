// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaViewportQualitySettingsPropertyTypeCustomization.h"
#include "AvaEditorSettings.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/Text.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "Viewport/AvaViewportQualitySettings.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AvaViewportQualitySettingsPropertyTypeCustomization"

TSharedRef<IPropertyTypeCustomization> FAvaViewportQualitySettingsPropertyTypeCustomization::MakeInstance()
{
	return MakeShared<FAvaViewportQualitySettingsPropertyTypeCustomization>();
}

void FAvaViewportQualitySettingsPropertyTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructPropertyHandle = InStructPropertyHandle;

	if (InStructPropertyHandle->GetBoolMetaData(TEXT("HideHeader")))
	{
		HeaderRow.Visibility(EVisibility::Collapsed);
	}
	else
	{
		HeaderRow.NameContent()
			[
				InStructPropertyHandle->CreatePropertyNameWidget()
			];
	}
}

void FAvaViewportQualitySettingsPropertyTypeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle
	, IDetailChildrenBuilder& InOutDetailBuilder
	, IPropertyTypeCustomizationUtils& InOutStructCustomizationUtils)
{
	if (InStructPropertyHandle->GetBoolMetaData(TEXT("ShowPresets")))
	{
		InOutDetailBuilder.AddCustomRow(LOCTEXT("Presets", "Presets"))
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush(TEXT("ToolPanel.GroupBorder")))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SComboButton)
						.ToolTipText(LOCTEXT("EditorSettingToolTip", "Open the Motion Design editor settings to edit viewport quality presets"))
						.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
						.ContentPadding(FMargin(2.f))
						.HasDownArrow(false)
						.MenuPlacement(MenuPlacement_BelowAnchor)
						.OnComboBoxOpened(this, &FAvaViewportQualitySettingsPropertyTypeCustomization::OpenEditorSettings)
						.ButtonContent()
						[
							SNew(SImage)
							.ColorAndOpacity(FSlateColor::UseForeground())
							.Image(FAppStyle::Get().GetBrush(TEXT("Icons.Settings")))
						]
					]
					+ SHorizontalBox::Slot()
					.Padding(FMargin(3.f, 0.f, 0.f, 0.f))
					[
						SAssignNew(PresetsWrapBox, SWrapBox)
						.UseAllottedSize(true)
						.Orientation(Orient_Horizontal)
					]
				]
			];

		RefreshPresets();
	}

	const TSharedPtr<IPropertyHandle> FeaturesProperty = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaViewportQualitySettings, Features));
	const TSharedRef<FDetailArrayBuilder> FeaturesArrayBuilder = MakeShared<FDetailArrayBuilder>(FeaturesProperty.ToSharedRef(), /*InGenerateHeader*/ false, /*InDisplayResetToDefault*/ true, /*InDisplayElementNum*/ false);

	FeaturesArrayBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateLambda([](TSharedRef<IPropertyHandle> InElementPropertyHandle, const int32 InArrayIndex, IDetailChildrenBuilder& InOutChildrenBuilder)
		{
			TSharedPtr<IPropertyHandle> NameProperty = InElementPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaViewportQualitySettingsFeature, Name));
			TSharedPtr<IPropertyHandle> ValueProperty = InElementPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaViewportQualitySettingsFeature, bEnabled));

			FString FeatureName;
			NameProperty->GetValue(FeatureName);

			FText NameText, TooltipText;
			FAvaViewportQualitySettings::FeatureNameAndTooltipText(FeatureName, NameText, TooltipText);

			InOutChildrenBuilder.AddProperty(InElementPropertyHandle)
				.ToolTip(TooltipText)
				.CustomWidget()
				.NameContent()
				[
					SNew(STextBlock)
					.Text(NameText)
					.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				.ValueContent()
				[
					ValueProperty->CreatePropertyValueWidget(/*bDisplayDefaultPropertyButtons=*/false)
				];
		}));

	InOutDetailBuilder.AddCustomBuilder(FeaturesArrayBuilder);
}

void FAvaViewportQualitySettingsPropertyTypeCustomization::RefreshPresets()
{
	if (!PresetsWrapBox.IsValid())
	{
		return;
	}

	PresetsWrapBox->ClearChildren();

	auto AddSlotToWrapBox = [this](const FText& InDisplayName
		, FOnClicked InOnClicked = FOnClicked(), const TAttribute<bool>& InIsEnabled = true)
	{
		if (!InOnClicked.IsBound())
		{
			InOnClicked = FOnClicked::CreateSP(this, &FAvaViewportQualitySettingsPropertyTypeCustomization::HandlePresetButtonClick, InDisplayName);
		}
		PresetsWrapBox->AddSlot()
			[
				SNew(SBox)
				.Padding(2.f)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
					.OnClicked(InOnClicked)
					.IsEnabled(InIsEnabled)
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), TEXT("SmallText"))
						.Text(InDisplayName)
					]
				]
			];
	};

	AddSlotToWrapBox(LOCTEXT("PresetDefaults", "Defaults")
		, FOnClicked::CreateSP(this, &FAvaViewportQualitySettingsPropertyTypeCustomization::HandleDefaultsButtonClick)
		, TAttribute<bool>::CreateSP(this, &FAvaViewportQualitySettingsPropertyTypeCustomization::IsDefaultsButtonEnabled));

	AddSlotToWrapBox(LOCTEXT("PresetAll", "All")
		, FOnClicked::CreateSP(this, &FAvaViewportQualitySettingsPropertyTypeCustomization::HandleEnableAllButtonClick)
		, TAttribute<bool>::CreateSP(this, &FAvaViewportQualitySettingsPropertyTypeCustomization::IsAllButtonEnabled));

	AddSlotToWrapBox(LOCTEXT("PresetNone", "None")
		, FOnClicked::CreateSP(this, &FAvaViewportQualitySettingsPropertyTypeCustomization::HandleDisableAllButtonClick)
		, TAttribute<bool>::CreateSP(this, &FAvaViewportQualitySettingsPropertyTypeCustomization::IsNoneButtonEnabled));

	PresetsWrapBox->AddSlot()
		.Padding(5.f, 0.f)
		[
			SNew(SSeparator)
			.Orientation(Orient_Vertical)
		];

	for (const FAvaViewportQualitySettingsPreset& Preset : UAvaEditorSettings::Get()->ViewportQualitySettingsPresets)
	{
		AddSlotToWrapBox(Preset.PresetName
			, FOnClicked()
			, TAttribute<bool>::CreateSP(this, &FAvaViewportQualitySettingsPropertyTypeCustomization::IsPresetButtonEnabled, Preset.PresetName));
	}
}

FAvaViewportQualitySettings& FAvaViewportQualitySettingsPropertyTypeCustomization::GetStructRef() const
{
	check(StructPropertyHandle.IsValid());

	void* OutAddress = nullptr;
	StructPropertyHandle->GetValueData(OutAddress);

	return *reinterpret_cast<FAvaViewportQualitySettings*>(OutAddress);
}

FReply FAvaViewportQualitySettingsPropertyTypeCustomization::HandleDefaultsButtonClick()
{
	GetStructRef() = UAvaEditorSettings::Get()->DefaultViewportQualitySettings;

	return FReply::Handled();
}

FReply FAvaViewportQualitySettingsPropertyTypeCustomization::HandleEnableAllButtonClick()
{
	for (FAvaViewportQualitySettingsFeature& Feature : GetStructRef().Features)
	{
		Feature.bEnabled = true;
	}

	return FReply::Handled();
}

FReply FAvaViewportQualitySettingsPropertyTypeCustomization::HandleDisableAllButtonClick()
{
	for (FAvaViewportQualitySettingsFeature& Feature : GetStructRef().Features)
	{
		Feature.bEnabled = false;
	}

	return FReply::Handled();
}

FReply FAvaViewportQualitySettingsPropertyTypeCustomization::HandlePresetButtonClick(const FText InPresetName)
{
	UAvaEditorSettings* const EditorSettings = UAvaEditorSettings::Get();

	FAvaViewportQualitySettingsPreset* const FoundPreset = EditorSettings->ViewportQualitySettingsPresets.FindByPredicate(
		[InPresetName](const FAvaViewportQualitySettingsPreset& InPreset)
		{
			return InPreset.PresetName.EqualTo(InPresetName);
		});
	if (!FoundPreset)
	{
		return FReply::Unhandled();
	}

	GetStructRef() = FoundPreset->QualitySettings;

	return FReply::Handled();
}

bool FAvaViewportQualitySettingsPropertyTypeCustomization::IsDefaultsButtonEnabled() const
{
	return UAvaEditorSettings::Get()->DefaultViewportQualitySettings != GetStructRef();
}

bool FAvaViewportQualitySettingsPropertyTypeCustomization::IsAllButtonEnabled() const
{
	for (FAvaViewportQualitySettingsFeature& Feature : GetStructRef().Features)
	{
		if (!Feature.bEnabled)
		{
			return true;
		}
	}

	return false;
}

bool FAvaViewportQualitySettingsPropertyTypeCustomization::IsNoneButtonEnabled() const
{
	for (FAvaViewportQualitySettingsFeature& Feature : GetStructRef().Features)
	{
		if (Feature.bEnabled)
		{
			return true;
		}
	}

	return false;
}

bool FAvaViewportQualitySettingsPropertyTypeCustomization::IsPresetButtonEnabled(const FText InPresetName) const
{
	const FAvaViewportQualitySettings& StructRef = GetStructRef();

	for (const FAvaViewportQualitySettingsPreset& Preset : UAvaEditorSettings::Get()->ViewportQualitySettingsPresets)
	{
		if (Preset.PresetName.EqualTo(InPresetName))
		{
			return Preset.QualitySettings != StructRef;
		}
	}

	return true;
}

void FAvaViewportQualitySettingsPropertyTypeCustomization::OpenEditorSettings() const
{
	if (ISettingsModule* const SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>(TEXT("Settings")))
	{
		if (const UAvaEditorSettings* const Settings = GetDefault<UAvaEditorSettings>())
		{
			SettingsModule->ShowViewer(Settings->GetContainerName(), Settings->GetCategoryName(), Settings->GetSectionName());
		}
	}
}

#undef LOCTEXT_NAMESPACE
