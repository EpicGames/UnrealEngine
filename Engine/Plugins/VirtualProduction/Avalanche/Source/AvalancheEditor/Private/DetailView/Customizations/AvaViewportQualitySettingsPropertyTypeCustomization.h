// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Layout/SWrapBox.h"

class FReply;
struct FAvaViewportQualitySettings;

class FAvaViewportQualitySettingsPropertyTypeCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& DetailBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	//~ End IPropertyTypeCustomization

protected:
	void RefreshPresets();

	FAvaViewportQualitySettings& GetStructRef() const;

	FReply HandleDefaultsButtonClick();
	FReply HandleEnableAllButtonClick();
	FReply HandleDisableAllButtonClick();
	FReply HandlePresetButtonClick(const FText InPresetName);

	bool IsDefaultsButtonEnabled() const;
	bool IsAllButtonEnabled() const;
	bool IsNoneButtonEnabled() const;
	bool IsPresetButtonEnabled(const FText InPresetName) const;

	void OpenEditorSettings() const;

	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	TSharedPtr<SWrapBox> PresetsWrapBox;
};
