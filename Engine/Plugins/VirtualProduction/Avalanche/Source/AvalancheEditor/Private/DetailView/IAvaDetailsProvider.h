// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

class FEditorModeTools;
class IDetailKeyframeHandler;

class IAvaDetailsProvider
{
public:
	virtual FEditorModeTools* GetDetailsModeTools() const = 0;

	virtual TSharedPtr<IDetailKeyframeHandler> GetDetailsKeyframeHandler() const = 0;
};
