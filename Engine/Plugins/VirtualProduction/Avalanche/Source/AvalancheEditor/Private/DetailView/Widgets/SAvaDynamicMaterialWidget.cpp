// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaDynamicMaterialWidget.h"
#include "Components/DynamicMeshComponent.h"
#include "DetailLayoutBuilder.h"
#include "DMObjectMaterialProperty.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "IDynamicMaterialEditorModule.h"
#include "Material/DynamicMaterialInstance.h"
#include "Material/DynamicMaterialInstanceFactory.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaDynamicMaterialWidget"

void SAvaDynamicMaterialWidget::Construct(const FArguments& InArgs, TSharedRef<IPropertyHandle> InPropertyHandle)
{
	PropertyHandle = InPropertyHandle;

	TArray<UObject*> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.IsEmpty())
	{
		return;
	}

	UObject* Value = nullptr;
	InPropertyHandle->GetValue(Value);

	UDynamicMaterialInstance* Instance = GetDynamicMaterialInstance();

	// @formatter:off
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.VAlign(EVerticalAlignment::VAlign_Center)
		.Padding(10.f, 5.f, 10.f, 5.f)
		[
			SNew(SObjectPropertyEntryBox)
			.Visibility(this, &SAvaDynamicMaterialWidget::GetPickerVisibility)
			.AllowClear(true)
			.AllowedClass(UMaterialInterface::StaticClass())
			.DisplayBrowse(true)
			.DisplayThumbnail(true)
			.DisplayCompactSize(false)
			.DisplayUseSelected(true)
			.EnableContentPicker(true)
			.ThumbnailPool(UThumbnailManager::Get().GetSharedThumbnailPool())
			.ObjectPath(this, &SAvaDynamicMaterialWidget::GetAssetPath)
			.OnObjectChanged(this, &SAvaDynamicMaterialWidget::OnAssetChanged)
		]
		+ SVerticalBox::Slot()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.VAlign(EVerticalAlignment::VAlign_Center)
		.Padding(10.f, 5.f, 10.f, 5.f)
		.AutoHeight()
		[
			SNew(SButton)
			.Visibility(this, &SAvaDynamicMaterialWidget::GetButtonVisibility)
			.OnClicked(this, &SAvaDynamicMaterialWidget::OnButtonClicked)
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("OpenMaterialDesigner", "Edit with Material Designer"))
				.Font(IDetailLayoutBuilder::GetDetailFontBold())
			]
		]
	];
	// @formatter:on
}

UObject* SAvaDynamicMaterialWidget::GetAsset() const
{
	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.IsEmpty())
	{
		return nullptr;
	}

	UObject* Value = nullptr;
	PropertyHandle->GetValue(Value);

	return Value;
}

UDynamicMaterialInstance* SAvaDynamicMaterialWidget::GetDynamicMaterialInstance() const
{
	return Cast<UDynamicMaterialInstance>(GetAsset());
}

void SAvaDynamicMaterialWidget::SetAsset(UObject* NewAsset)
{
	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.IsEmpty())
	{
		return;
	}

	PropertyHandle->SetValueFromFormattedString(NewAsset ? NewAsset->GetPathName() : "");
}

void SAvaDynamicMaterialWidget::SetDynamicMaterialInstance(UDynamicMaterialInstance* NewInstance)
{
	SetAsset(NewInstance);
}

EVisibility SAvaDynamicMaterialWidget::GetPickerVisibility() const
{
	if (GetDynamicMaterialInstance())
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

EVisibility SAvaDynamicMaterialWidget::GetButtonVisibility() const
{
	if (GetDynamicMaterialInstance())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

FReply SAvaDynamicMaterialWidget::OnButtonClicked()
{
	if (GetDynamicMaterialInstance())
	{
		return OpenDynamicMaterialInstanceTab();
	}

	return CreateDynamicMaterialInstance();
}

FReply SAvaDynamicMaterialWidget::CreateDynamicMaterialInstance()
{
	UDynamicMaterialInstance* Instance = GetDynamicMaterialInstance();

	// We already have an instance, so we don't need to create one
	if (Instance)
	{
		return FReply::Unhandled();
	}

	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.IsEmpty())
	{
		return FReply::Unhandled();
	}

	UDynamicMaterialInstanceFactory* DynamicMaterialInstanceFactory = NewObject<UDynamicMaterialInstanceFactory>();
	check(DynamicMaterialInstanceFactory);

	UDynamicMaterialInstance* NewInstance = Cast<UDynamicMaterialInstance>(DynamicMaterialInstanceFactory->FactoryCreateNew(
		UDynamicMaterialInstance::StaticClass(),
		OuterObjects[0],
		"DynamicMaterialInstance",
		RF_Transactional,
		nullptr,
		GWarn
	));

	PropertyHandle->SetValueFromFormattedString(NewInstance->GetPathName());

	return OpenDynamicMaterialInstanceTab();
}

FReply SAvaDynamicMaterialWidget::ClearDynamicMaterialInstance()
{
	UDynamicMaterialInstance* Instance = GetDynamicMaterialInstance();

	// We don't have an instance, so we don't need to clear it (and don't clear non-MDIs)
	if (!Instance)
	{
		return FReply::Unhandled();
	}

	SetDynamicMaterialInstance(nullptr);

	return FReply::Handled();
}

FReply SAvaDynamicMaterialWidget::OpenDynamicMaterialInstanceTab()
{
	UDynamicMaterialInstance* Instance = GetDynamicMaterialInstance();

	// We don't have a MDI, so don't try to open it.
	if (!Instance)
	{
		return FReply::Unhandled();
	}

	const IDynamicMaterialEditorModule& MaterialDesignerModule = IDynamicMaterialEditorModule::Get();
	constexpr bool bInvokeTab = true;

	TArray<UObject*> Outers;
	PropertyHandle->GetOuterObjects(Outers);

	if (Outers.Num() == 0)
	{
		MaterialDesignerModule.OpenMaterialModel(Instance->GetMaterialModelBase(), nullptr, bInvokeTab);
		return FReply::Handled();
	}

	UWorld* OuterWorld = Outers[0]->GetWorld();
	UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Outers[0]);

	if (!PrimitiveComponent)
	{
		if (UAvaShapeDynamicMeshBase* ShapeMesh = Cast<UAvaShapeDynamicMeshBase>(Outers[0]))
		{
			PrimitiveComponent = ShapeMesh->GetShapeMeshComponent();
		}

		if (!PrimitiveComponent)
		{
			MaterialDesignerModule.OpenMaterialModel(Instance->GetMaterialModelBase(), nullptr, bInvokeTab);
			return FReply::Handled();
		}
	}

	const int32 NumMaterials = PrimitiveComponent->GetNumMaterials();

	for (int32 Index = 0; Index < NumMaterials; ++Index)
	{
		if (PrimitiveComponent->GetMaterial(Index) == Instance)
		{
			MaterialDesignerModule.OpenMaterialObjectProperty({PrimitiveComponent, Index}, OuterWorld, bInvokeTab);
			return FReply::Handled();
		}
	}

	MaterialDesignerModule.OpenMaterialModel(Instance->GetMaterialModelBase(), OuterWorld, bInvokeTab);
	return FReply::Handled();
}

FString SAvaDynamicMaterialWidget::GetAssetPath() const
{
	UObject* Asset = GetAsset();

	return Asset ? Asset->GetPathName() : "";
}

void SAvaDynamicMaterialWidget::OnAssetChanged(const FAssetData& AssetData)
{
	SetAsset(AssetData.GetAsset());
}

#undef LOCTEXT_NAMESPACE
