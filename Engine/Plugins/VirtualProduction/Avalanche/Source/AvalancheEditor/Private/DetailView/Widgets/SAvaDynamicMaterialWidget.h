// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class IPropertyHandle;
class UDynamicMaterialInstance;
class UDynamicMaterialModel;

class SAvaDynamicMaterialWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaDynamicMaterialWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<IPropertyHandle> InPropertyHandle);

protected:
	TSharedPtr<IPropertyHandle> PropertyHandle;

	UObject* GetAsset() const;
	void SetAsset(UObject* NewAsset);

	UDynamicMaterialInstance* GetDynamicMaterialInstance() const;
	void SetDynamicMaterialInstance(UDynamicMaterialInstance* NewInstance);

	FReply OnButtonClicked();
	FReply CreateDynamicMaterialInstance();
	FReply ClearDynamicMaterialInstance();
	FReply OpenDynamicMaterialInstanceTab();

	EVisibility GetButtonVisibility() const;
	EVisibility GetPickerVisibility() const;
	FString GetAssetPath() const;
	void OnAssetChanged(const FAssetData& AssetData);
};
