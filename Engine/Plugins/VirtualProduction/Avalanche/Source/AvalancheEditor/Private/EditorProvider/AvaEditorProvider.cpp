// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEditorProvider.h"
#include "AvaEditorActorUtils.h"
#include "AvaEditorSettings.h"
#include "AvaScene.h"
#include "Engine/World.h"
#include "IAvaEditor.h"

UObject* FAvaEditorProvider::GetSceneObject(UWorld* InWorld, EAvaEditorObjectQueryType InQueryType)
{
	if (!InWorld)
	{
		return nullptr;
	}

	ULevel* SceneLevel = InWorld->PersistentLevel;

	const UAvaEditorSettings* EditorSettings = UAvaEditorSettings::Get();
	if (EditorSettings && EditorSettings->bEnableLevelContextSwitching)
	{
		SceneLevel = InWorld->GetCurrentLevel();
	}

	const bool bCreateSceneIfNotFound = InQueryType == EAvaEditorObjectQueryType::CreateIfNotFound;
	return AAvaScene::GetScene(SceneLevel, bCreateSceneIfNotFound);
}

bool FAvaEditorProvider::ShouldAutoActivateScene(UObject* InSceneObject) const
{
	if (AAvaScene* Scene = Cast<AAvaScene>(InSceneObject))
	{
		return Scene->ShouldAutoStartMode();
	}
	return IAvaEditorProvider::ShouldAutoActivateScene(InSceneObject);
}

void FAvaEditorProvider::SetAutoActivateScene(UObject* InSceneObject, bool bInAutoActivateScene) const
{
	if (AAvaScene* Scene = Cast<AAvaScene>(InSceneObject))
	{
		Scene->Modify();
		Scene->SetAutoStartMode(bInAutoActivateScene);
	}
}

void FAvaEditorProvider::GetActorsToEdit(TArray<AActor*>& InOutActorsToEdit) const
{
	FAvaEditorActorUtils::GetActorsToEdit(InOutActorsToEdit);
}

void FAvaEditorProvider::OnSceneActivated()
{
	AAvaScene::NotifySceneEvent(AAvaScene::ESceneAction::Activated);
}

void FAvaEditorProvider::OnSceneDeactivated()
{
	AAvaScene::NotifySceneEvent(AAvaScene::ESceneAction::Deactivated);
}
