// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaEditorProvider.h"

/** Base Implementation for the Core Editor Provider */
class FAvaEditorProvider : public IAvaEditorProvider
{
public:
	//~ Begin IAvaEditorProvider
	virtual UObject* GetSceneObject(UWorld* InWorld, EAvaEditorObjectQueryType InQueryType) override;
	virtual bool ShouldAutoActivateScene(UObject* InSceneObject) const override;
	virtual void SetAutoActivateScene(UObject* InSceneObject, bool bInAutoActivateScene) const override;
	virtual void GetActorsToEdit(TArray<AActor*>& InOutActorsToEdit) const override;
	virtual void OnSceneActivated() override;
	virtual void OnSceneDeactivated() override;
	//~ End IAvaEditorProvider
};
