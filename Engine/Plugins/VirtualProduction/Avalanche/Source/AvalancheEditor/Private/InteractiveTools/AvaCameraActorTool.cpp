// Copyright Epic Games, Inc. All Rights Reserved.

#include "InteractiveTools/AvaCameraActorTool.h"
#include "AvaViewportUtils.h"
#include "EditorViewportClient.h"
#include "InteractiveToolManager.h"
#include "ViewportClient/IAvaViewportClient.h"

UAvaCameraActorTool::UAvaCameraActorTool()
{
}

void UAvaCameraActorTool::DefaultAction()
{
	Super::DefaultAction();

	if (IsValid(SpawnedActor))
	{
		if (IToolsContextQueriesAPI* ContextAPI = GetToolManager()->GetContextQueriesAPI())
		{
			if (UWorld* World = ContextAPI->GetCurrentEditingWorld())
			{
				if (TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAvaViewportClient(GetViewport(EAvaViewportStatus::Focused)))
				{
					if (const FEditorViewportClient* EditorViewportClient = AvaViewportClient->AsEditorViewportClient())
					{
						const FViewportCameraTransform ViewTransform = EditorViewportClient->GetViewTransform();

						SpawnedActor->SetActorLocationAndRotation(
							ViewTransform.GetLocation(),
							ViewTransform.GetRotation()
						);
					}
				}
			}
		}
	}
}
