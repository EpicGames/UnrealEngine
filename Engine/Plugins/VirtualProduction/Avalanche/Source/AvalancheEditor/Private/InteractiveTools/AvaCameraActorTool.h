// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaInteractiveToolsActorTool.h"
#include "AvaCameraActorTool.generated.h"

UCLASS(DisplayName = "Motion Design Camera Actor Tool")
class UAvaCameraActorTool : public UAvaInteractiveToolsActorTool
{
	GENERATED_BODY()

public:
	UAvaCameraActorTool();

protected:
	//~ Begin UAvaInteractiveToolsToolBase
	virtual void DefaultAction() override;
	// End UAvaInteractiveToolsToolBase
};
