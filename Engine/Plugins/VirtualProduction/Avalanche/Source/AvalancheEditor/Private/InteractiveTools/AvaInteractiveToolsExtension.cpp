// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaInteractiveToolsExtension.h"
#include "AvaEditorCommands.h"
#include "AvaInteractiveToolsDelegates.h"
#include "Builders/AvaInteractiveToolsActorToolBuilder.h"
#include "Builders/AvaInteractiveToolsStaticMeshActorToolBuilder.h"
#include "Camera/CameraActor.h"
#include "Camera/CameraShakeSourceActor.h"
#include "CameraRig_Crane.h"
#include "CameraRig_Rail.h"
#include "EditorModeManager.h"
#include "Engine/DirectionalLight.h"
#include "Engine/PointLight.h"
#include "Engine/RectLight.h"
#include "Engine/SkyLight.h"
#include "Engine/SpotLight.h"
#include "IAvalancheInteractiveToolsModule.h"
#include "InteractiveTools/AvaCameraActorTool.h"
#include "MediaPlate.h"
#include "Viewport/AvaCineCameraActor.h"
#include "Viewport/AvaPostProcessVolume.h"

namespace UE::AvaEditor::Private
{
	const FName StaticMeshCategory = FName(TEXT("StaticMeshes"));
	const FName CamerasCategory = FName(TEXT("Cameras"));
	const FName LightsCategory = FName(TEXT("Lights"));
}

FAvaInteractiveToolsExtension::~FAvaInteractiveToolsExtension()
{
	UnregisterToolDelegates();
}

void FAvaInteractiveToolsExtension::PostInvokeTabs()
{
	if (FEditorModeTools* const ModeTools = GetEditorModeTools())
	{
		ModeTools->ActivateMode(IAvalancheInteractiveToolsModule::EM_AvaInteractiveToolsEdModeId);
	}
}

void FAvaInteractiveToolsExtension::Activate()
{
	FAvaEditorExtension::Activate();

	RegisterToolDelegates();
	RegisterCategories(IAvalancheInteractiveToolsModule::GetPtr());
	RegisterTools(IAvalancheInteractiveToolsModule::GetPtr());
}

void FAvaInteractiveToolsExtension::Deactivate()
{
	// DestroyMode used instead of DeactivateMode so that this EdMode can be processed immediately rather than waiting through PendingDeactivateModes
	// This is needed especially because this can be called when Reloading Layout Config, and this Mode needs to be re-entered again
	// to update the underlying Toolkit Host in UEdMode::Enter
	if (FEditorModeTools* const ModeTools = GetEditorModeTools())
	{
		ModeTools->DestroyMode(IAvalancheInteractiveToolsModule::EM_AvaInteractiveToolsEdModeId);
	}

	UnregisterToolDelegates();
}

void FAvaInteractiveToolsExtension::RegisterToolDelegates()
{
	FAvaInteractiveToolsDelegates::GetRegisterCategoriesDelegate().AddRaw(this, &FAvaInteractiveToolsExtension::RegisterCategories);
	FAvaInteractiveToolsDelegates::GetRegisterToolsDelegate().AddRaw(this, &FAvaInteractiveToolsExtension::RegisterTools);
}

void FAvaInteractiveToolsExtension::UnregisterToolDelegates()
{
	FAvaInteractiveToolsDelegates::GetRegisterCategoriesDelegate().RemoveAll(this);
	FAvaInteractiveToolsDelegates::GetRegisterToolsDelegate().RemoveAll(this);
}

void FAvaInteractiveToolsExtension::RegisterCategories(IAvalancheInteractiveToolsModule* InModule)
{
	using namespace UE::AvaEditor::Private;

	const FAvaEditorCommands& EditorCommands = FAvaEditorCommands::Get();

	InModule->RegisterCategory(StaticMeshCategory, EditorCommands.StaticMeshToolsCategory);
	InModule->RegisterCategory(CamerasCategory, EditorCommands.CameraToolsCategory);
	InModule->RegisterCategory(LightsCategory, EditorCommands.LightsToolsCategory);
}

void FAvaInteractiveToolsExtension::RegisterTools(IAvalancheInteractiveToolsModule* InModule)
{
	using namespace UE::AvaEditor::Private;

	const FAvaEditorCommands& EditorCommands = FAvaEditorCommands::Get();

	InModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryNameActor, UAvaInteractiveToolsActorToolBuilder::CreateToolParameters<AMediaPlate>(
		IAvalancheInteractiveToolsModule::CategoryNameActor, EditorCommands.MediaPlateTool, FString(TEXT("Media Plate Tool")),
		7000));

	int32 Priority = 0;

	InModule->RegisterTool(StaticMeshCategory, UAvaInteractiveToolsStaticMeshActorToolBuilder::CreateToolParameters(StaticMeshCategory,
		EditorCommands.CubeTool, FString(TEXT("Static Mesh Cube Tool")), Priority += 1000,
		TEXT("/Script/Engine.StaticMesh'/Engine/BasicShapes/Cube.Cube'")));

	InModule->RegisterTool(StaticMeshCategory, UAvaInteractiveToolsStaticMeshActorToolBuilder::CreateToolParameters(StaticMeshCategory,
		EditorCommands.SphereTool, FString(TEXT("Static Mesh Sphere Tool")), Priority += 1000,
		TEXT("/Script/Engine.StaticMesh'/Engine/BasicShapes/Sphere.Sphere'")));

	InModule->RegisterTool(StaticMeshCategory, UAvaInteractiveToolsStaticMeshActorToolBuilder::CreateToolParameters(StaticMeshCategory,
		EditorCommands.CylinderTool, FString(TEXT("Static Mesh Cylinder Tool")), Priority += 1000,
		TEXT("/Script/Engine.StaticMesh'/Engine/BasicShapes/Cylinder.Cylinder'")));

	InModule->RegisterTool(StaticMeshCategory, UAvaInteractiveToolsStaticMeshActorToolBuilder::CreateToolParameters(StaticMeshCategory,
		EditorCommands.ConeTool, FString(TEXT("Static Mesh Cone Tool")), Priority += 1000,
		TEXT("/Script/Engine.StaticMesh'/Engine/BasicShapes/Cone.Cone'")));

	InModule->RegisterTool(StaticMeshCategory, UAvaInteractiveToolsStaticMeshActorToolBuilder::CreateToolParameters(StaticMeshCategory,
		EditorCommands.PlaneTool, FString(TEXT("Static Mesh Plane Tool")), Priority += 1000,
		TEXT("/Script/Engine.StaticMesh'/Engine/BasicShapes/Plane.Plane'")));
	
	InModule->RegisterTool(StaticMeshCategory, UAvaInteractiveToolsStaticMeshActorToolBuilder::CreateToolParameters(StaticMeshCategory,
		EditorCommands.CycloramaTool, FString(TEXT("Static Mesh Cyclorama Tool")), Priority += 1000,
		TEXT("/Script/Engine.StaticMesh'/Avalanche/Toolkit/SM_Cyc_02.SM_Cyc_02'")));

	Priority = 0;

	InModule->RegisterTool(CamerasCategory, UAvaInteractiveToolsActorToolBuilder::CreateToolParameters<ACameraActor>( 
		CamerasCategory, EditorCommands.CameraTool, FString(TEXT("Standard Camera Tool")), 
		Priority += 1000,
		UAvaCameraActorTool::StaticClass()));

	InModule->RegisterTool(CamerasCategory, UAvaInteractiveToolsActorToolBuilder::CreateToolParameters<AAvaCineCameraActor>( 
		CamerasCategory, EditorCommands.CineCameraTool, FString(TEXT("Cine Camera Tool")), 
		Priority += 1000,
		UAvaCameraActorTool::StaticClass()));

	InModule->RegisterTool(CamerasCategory, UAvaInteractiveToolsActorToolBuilder::CreateToolParameters<ACameraRig_Crane>(
		CamerasCategory, EditorCommands.CameraRigCraneTool, FString(TEXT("Camera Rig Crane Tool")), 
		Priority += 1000));

	InModule->RegisterTool(CamerasCategory, UAvaInteractiveToolsActorToolBuilder::CreateToolParameters<ACameraRig_Rail>(
		CamerasCategory, EditorCommands.CameraRigRailTool, FString(TEXT("Camera Rig Rail Tool")), 
		Priority += 1000));

	InModule->RegisterTool(CamerasCategory, UAvaInteractiveToolsActorToolBuilder::CreateToolParameters<ACameraShakeSourceActor>( 
		CamerasCategory, EditorCommands.CameraShakeSourceTool, FString(TEXT("Camera Shake Source Tool")), 
		Priority += 1000));

	InModule->RegisterTool(CamerasCategory, UAvaInteractiveToolsActorToolBuilder::CreateToolParameters<AAvaPostProcessVolume>(
		CamerasCategory, EditorCommands.AvaPostProcessVolumeTool, FString(TEXT("Motion Design Post Process Volume Tool")), 
		Priority += 1000));

	Priority = 0;

	InModule->RegisterTool(LightsCategory, UAvaInteractiveToolsActorToolBuilder::CreateToolParameters<APointLight>(
		LightsCategory, EditorCommands.PointLightTool, FString(TEXT("Point Light Tool")), 
		Priority += 1000));

	InModule->RegisterTool(LightsCategory, UAvaInteractiveToolsActorToolBuilder::CreateToolParameters<ADirectionalLight>(
		LightsCategory, EditorCommands.DirectionalLightTool, FString(TEXT("Directional Light Tool")), 
		Priority += 1000));

	InModule->RegisterTool(LightsCategory, UAvaInteractiveToolsActorToolBuilder::CreateToolParameters<ASpotLight>(
		LightsCategory, EditorCommands.SpotLightTool, FString(TEXT("Spot Light Tool")), 
		Priority += 1000));

	InModule->RegisterTool(LightsCategory, UAvaInteractiveToolsActorToolBuilder::CreateToolParameters<ARectLight>(
		LightsCategory, EditorCommands.RectLightTool, FString(TEXT("Rect Light Tool")),
		Priority += 1000));

	InModule->RegisterTool(LightsCategory, UAvaInteractiveToolsActorToolBuilder::CreateToolParameters<ASkyLight>(
		LightsCategory, EditorCommands.SkyLightTool, FString(TEXT("Sky Light Tool")), 
		Priority += 1000));
}
