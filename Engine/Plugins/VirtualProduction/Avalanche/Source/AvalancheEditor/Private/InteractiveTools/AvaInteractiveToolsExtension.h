// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaEditorExtension.h"
#include "IAvalancheInteractiveToolsModule.h"
#include "Delegates/IDelegateInstance.h"

class IAvalancheInteractiveToolsModule;
class UEdMode;

class FAvaInteractiveToolsExtension : public FAvaEditorExtension
{
public:
	UE_AVA_INHERITS(FAvaInteractiveToolsExtension, FAvaEditorExtension);

	virtual ~FAvaInteractiveToolsExtension() override;

	//~ Begin IAvaEditorExtension
	virtual void PostInvokeTabs() override;
	virtual void Activate() override;
	virtual void Deactivate() override;
	//~ End IAvaEditorExtension

protected:
	void RegisterToolDelegates();

	void UnregisterToolDelegates();

	void RegisterCategories(IAvalancheInteractiveToolsModule* InModule);

	void RegisterTools(IAvalancheInteractiveToolsModule* InModule);
};
