// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialDesigner/AvaLevelMaterialDesignerExtension.h"
#include "AvaEditorModule.h"
#include "AvaMaterialDesignerTextureAssetFactory.h"
#include "AvaScreenAlignmentUtils.h"
#include "ContentBrowserModule.h"
#include "Delegates/IDelegateInstance.h"
#include "Engine/Texture.h"
#include "Materials/Material.h"
#include "Styling/SlateIconFinder.h"
#include "Viewport/AvaLevelViewportExtension.h"
#include "ViewportClient/IAvaViewportClient.h"

#define LOCTEXT_NAMESPACE "AvaMaterialDesignerExtension"

namespace UE::AvaEditor::Private
{
	int32 AvaLevelMaterialDesignerExtensionInstances = 0;
	FDelegateHandle ContentBrowserExtenderDelegateHandle;
}

void FAvaLevelMaterialDesignerExtension::Activate()
{
	FAvaMaterialDesignerExtension::Activate();

	using namespace UE::AvaEditor::Private;

	++AvaLevelMaterialDesignerExtensionInstances;
	bIsActive = true;

	InitContentBrowserExtension();
}

void FAvaLevelMaterialDesignerExtension::Deactivate()
{
	FAvaMaterialDesignerExtension::Deactivate();

	using namespace UE::AvaEditor::Private;

	if (bIsActive)
	{
		AvaLevelMaterialDesignerExtensionInstances = FMath::Max(AvaLevelMaterialDesignerExtensionInstances - 1, 0);
		bIsActive = false;
	}

	DeinitContentBrowserExtension();
}

void FAvaLevelMaterialDesignerExtension::InitContentBrowserExtension()
{
	using namespace UE::AvaEditor::Private;

	// Register only with the first instance.
	if (AvaLevelMaterialDesignerExtensionInstances == 1)
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();
		CBMenuExtenderDelegates.Add(FContentBrowserMenuExtender_SelectedAssets::CreateStatic(&FAvaLevelMaterialDesignerExtension::OnExtendContentBrowserAssetSelectionMenu));
		ContentBrowserExtenderDelegateHandle = CBMenuExtenderDelegates.Last().GetHandle();
	}
}

void FAvaLevelMaterialDesignerExtension::DeinitContentBrowserExtension()
{
	using namespace UE::AvaEditor::Private;

	// Unregister only with the last instance
	if (AvaLevelMaterialDesignerExtensionInstances == 0 && ContentBrowserExtenderDelegateHandle.IsValid())
	{
		if (FContentBrowserModule* ContentBrowserModule = FModuleManager::GetModulePtr<FContentBrowserModule>("ContentBrowser"))
		{
			TArray<FContentBrowserMenuExtender_SelectedAssets>& CBMenuExtenderDelegates = ContentBrowserModule->GetAllAssetViewContextMenuExtenders();

			CBMenuExtenderDelegates.RemoveAll([](const FContentBrowserMenuExtender_SelectedAssets& InDelegate)
				{
					return InDelegate.GetHandle() == ContentBrowserExtenderDelegateHandle;
				});
		}
	}
}

TSharedRef<FExtender> FAvaLevelMaterialDesignerExtension::OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& InSelectedAssets)
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();

	if (InSelectedAssets.Num() == 1)
	{
		if (InSelectedAssets[0].IsInstanceOf<UTexture>(EResolveClass::Yes))
		{
			Extender->AddMenuExtension("GetAssetActions", EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateLambda(
				[InSelectedAssets](FMenuBuilder& InMenuBuilder) {
					InMenuBuilder.AddMenuEntry(
						LOCTEXT("AddTextureToScene", "Add Texture To Scene"),
						LOCTEXT("AddTextureToSceneTooltip", "Creates a parametric shape in the scene the same size as the texture and creates a Material Designer asset on using this texture."),
						FSlateIconFinder::FindIconForClass(UMaterial::StaticClass()),
						FUIAction(FExecuteAction::CreateStatic(&FAvaLevelMaterialDesignerExtension::AddTextureToSene, InSelectedAssets[0]))
					);
				})
			);
		}
	}

	return Extender;
}

void FAvaLevelMaterialDesignerExtension::AddTextureToSene(FAssetData InAssetData)
{
	const TArray<TSharedPtr<IAvaViewportClient>> LevelViewportClients = FAvaLevelViewportExtension::GetLevelEditorViewportClients();

	if (LevelViewportClients.IsEmpty() || !LevelViewportClients[0].IsValid())
	{
		return;
	}

	TSharedRef<IAvaViewportClient> LevelViewportClient = LevelViewportClients[0].ToSharedRef();

	UWorld* World = LevelViewportClient->GetViewportWorld();

	if (!World)
	{
		return;
	}

	UAvaMaterialDesignerTextureAssetFactory* AssetFactory = NewObject<UAvaMaterialDesignerTextureAssetFactory>(GetTransientPackage());
	check(AssetFactory);
	AssetFactory->SetCameraRotation(LevelViewportClient->GetViewportViewTransform().Rotator());

	FText ErrorMsg;

	if (!AssetFactory->CanCreateActorFrom(InAssetData, ErrorMsg))
	{
		UE_LOGF(AvaLog, Warning, "%ls", *ErrorMsg.ToString());
		return;
	}

	UTexture* Texture = Cast<UTexture>(InAssetData.GetAsset());

	if (!Texture)
	{
		return;
	}

	FActorSpawnParameters SpawnParameters;
	SpawnParameters.bNoFail = true;

	AActor* TextureActor = AssetFactory->CreateActor(Texture, World->PersistentLevel.Get(), FTransform::Identity, SpawnParameters);

	if (!TextureActor)
	{
		return;
	}

	const FVector2D ViewportSize = LevelViewportClient->GetFrustumSizeAtDistance(
		(LevelViewportClient->GetViewportViewTransform().GetLocation() - TextureActor->GetActorLocation()).Size()
	);

	if (Texture->GetSurfaceWidth() > ViewportSize.X || Texture->GetSurfaceHeight() > ViewportSize.Y)
	{
		FAvaScreenAlignmentUtils::FitActorToScreen(
			LevelViewportClient,
			*TextureActor,
			/* Stretch to fit */ false,
			/* Align to nearest axis */ true
		);
	}
}

#undef LOCTEXT_NAMESPACE
