// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialDesigner/AvaMaterialDesignerExtension.h"
#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointerFwd.h"

class FExtender;
class UDynamicMaterialInstance;
class UDynamicMaterialModel;
struct FAssetData;
struct FDMObjectMaterialProperty;

class FAvaLevelMaterialDesignerExtension : public FAvaMaterialDesignerExtension
{
public:
	UE_AVA_INHERITS(FAvaLevelMaterialDesignerExtension, FAvaMaterialDesignerExtension);

	//~ Begin IAvaEditorExtension
	virtual void Activate() override;
	virtual void Deactivate() override;
	//~ End IAvaEditorExtension

private:
	bool bIsActive = false;

	static void InitContentBrowserExtension();

	static void DeinitContentBrowserExtension();

	static TSharedRef<FExtender> OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& InSelectedAssets);

	static void AddTextureToSene(FAssetData InAssetData);
};
