// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialDesigner/AvaMaterialDesignerExtension.h"

#include "AvaShapeActor.h"
#include "DetailView/AvaDetailsExtension.h"
#include "DMObjectMaterialProperty.h"
#include "DMWorldSubsystem.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "Engine/World.h"
#include "IDynamicMaterialEditorModule.h"
#include "Material/DynamicMaterialInstance.h"
#include "Materials/Material.h"
#include "Model/DynamicMaterialModel.h"
#include "Styling/SlateIconFinder.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "AvaMaterialDesignerExtension"

void FAvaMaterialDesignerExtension::Activate()
{
	FAvaEditorExtension::Activate();

	InitWorldSubsystem();
}

void FAvaMaterialDesignerExtension::ExtendToolbarMenu(UToolMenu& InMenu)
{
	FToolMenuSection& Section = InMenu.FindOrAddSection(DefaultSectionName);

	FToolMenuEntry& Entry = Section.AddEntry(FToolMenuEntry::InitToolBarButton(
		TEXT("OpenMaterialDesigner"),
		FExecuteAction::CreateSP(this, &FAvaMaterialDesignerExtension::OpenEditor),
		LOCTEXT("OpenMaterialDesignerLabel", "Material Designer"),
		LOCTEXT("OpenMaterialDesignerTooltip", "Open the Material Designer tab."),
		FSlateIconFinder::FindIconForClass(UMaterial::StaticClass())
	));

	Entry.StyleNameOverride = "CalloutToolbar";
}

bool FAvaMaterialDesignerExtension::IsDynamicMaterialModelValid(UDynamicMaterialModelBase* InMaterialModel)
{
	if (!IsValid(InMaterialModel))
	{
		return false;
	}

	if (GetWorld() != InMaterialModel->GetWorld())
	{
		return false;
	}

	UAvaShapeDynamicMeshBase* const DynamicMesh = InMaterialModel->GetTypedOuter<UAvaShapeDynamicMeshBase>();

	// If we're not part of a dynamic mesh, just assume we're relevant
	if (!DynamicMesh)
	{
		return true;
	}

	for (int32 MeshIndex : DynamicMesh->GetMeshesIndexes())
	{
		if (UDynamicMaterialInstance* const MeshInstance = Cast<UDynamicMaterialInstance>(DynamicMesh->GetMaterial(MeshIndex)))
		{
			if (MeshInstance->GetMaterialModelBase() == InMaterialModel)
			{
				return true;
			}
		}
	}

	return false;
}

bool FAvaMaterialDesignerExtension::SetDynamicMaterialValue(const FDMObjectMaterialProperty& InObjectMaterialProperty, UDynamicMaterialInstance* InMaterial)
{
	if (!InObjectMaterialProperty.IsValid() || !IsValid(InMaterial))
	{
		return false;
	}

	if (const UActorComponent* const ActorComponent = Cast<UActorComponent>(InObjectMaterialProperty.GetOuter()))
	{
		if (const AAvaShapeActor* const ShapeActor = Cast<AAvaShapeActor>(ActorComponent->GetOwner()))
		{
			UAvaShapeDynamicMeshBase* const DynamicMesh = ShapeActor->GetDynamicMesh();
			if (DynamicMesh && DynamicMesh->GetMeshesIndexes().Contains(InObjectMaterialProperty.GetIndex()))
			{
				if (!InMaterial->IsAsset())
				{
					InMaterial->Rename(nullptr, DynamicMesh);
				}

				DynamicMesh->SetMaterial(InObjectMaterialProperty.GetIndex(), InMaterial);
				return true;
			}
		}
	}

	return false;
}

void FAvaMaterialDesignerExtension::InitWorldSubsystem()
{
	const UWorld* const World = GetWorld();
	const TSharedPtr<IAvaEditor> Editor = GetEditor();

	if (!IsValid(World) || !Editor.IsValid())
	{
		return;
	}

	UDMWorldSubsystem* const DMWorldSubsystem = World->GetSubsystem<UDMWorldSubsystem>();
	if (!DMWorldSubsystem)
	{
		return;
	}

	if (const TSharedPtr<FAvaDetailsExtension> DetailsExtension = Editor->FindExtension<FAvaDetailsExtension>())
	{
		DMWorldSubsystem->SetKeyframeHandler(DetailsExtension->GetDetailsKeyframeHandler());
	}

	DMWorldSubsystem->GetIsValidDelegate().BindSP(this, &FAvaMaterialDesignerExtension::IsDynamicMaterialModelValid);
	DMWorldSubsystem->GetMaterialValueSetterDelegate().BindSP(this, &FAvaMaterialDesignerExtension::SetDynamicMaterialValue);
}

void FAvaMaterialDesignerExtension::OpenEditor()
{
	TSharedPtr<IAvaEditor> Editor = GetEditor();

	if (!Editor.IsValid())
	{
		return;
	}

	UWorld* World = Editor->GetWorld();

	if (!World)
	{
		return;
	}

	IDynamicMaterialEditorModule::Get().OpenEditor(World);
}

#undef LOCTEXT_NAMESPACE
