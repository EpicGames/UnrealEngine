// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaEditorExtension.h"

class UDynamicMaterialInstance;
class UDynamicMaterialModelBase;
struct FDMObjectMaterialProperty;

class FAvaMaterialDesignerExtension : public FAvaEditorExtension
{
public:
	UE_AVA_INHERITS(FAvaMaterialDesignerExtension, FAvaEditorExtension);

	void OpenEditor();

	//~ Begin IAvaEditorExtension
	virtual void Activate() override;
	virtual void ExtendToolbarMenu(UToolMenu& InMenu) override;
	//~ End IAvaEditorExtension

private:
	bool IsDynamicMaterialModelValid(UDynamicMaterialModelBase* InMaterialModel);

	bool SetDynamicMaterialValue(const FDMObjectMaterialProperty& InObjectMaterialProperty, UDynamicMaterialInstance* InMaterial);

	void InitWorldSubsystem();
};
