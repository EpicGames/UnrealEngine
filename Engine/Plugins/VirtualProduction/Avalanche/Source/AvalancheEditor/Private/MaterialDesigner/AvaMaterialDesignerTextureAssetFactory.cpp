// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialDesigner/AvaMaterialDesignerTextureAssetFactory.h"

#include "AvaShapeActor.h"
#include "Components/DMMaterialLayer.h"
#include "Components/DMMaterialSlot.h"
#include "Components/DMMaterialStageBlend.h"
#include "Components/DMMaterialSubStage.h"
#include "Components/DynamicMeshComponent.h"
#include "Components/MaterialStageExpressions/DMMSETextureSample.h"
#include "Components/MaterialStageInputs/DMMSIExpression.h"
#include "Components/MaterialStageInputs/DMMSIValue.h"
#include "Components/MaterialValues/DMMaterialValueTexture.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "DynamicMeshes/AvaShapeRectangleDynMesh.h"
#include "Engine/Level.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "IDynamicMaterialEditorModule.h"
#include "Material/DynamicMaterialInstance.h"
#include "Material/DynamicMaterialInstanceFactory.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"

UAvaMaterialDesignerTextureAssetFactory::UAvaMaterialDesignerTextureAssetFactory()
{
	NewActorClass = AAvaShapeActor::StaticClass();
	bShowInEditorQuickMenu = false;
	bFactoryEnabled = false;
	CameraRotation = FRotator::ZeroRotator;
}

void UAvaMaterialDesignerTextureAssetFactory::SetCameraRotation(const FRotator& InRotation)
{
	CameraRotation = InRotation;
}

bool UAvaMaterialDesignerTextureAssetFactory::CanCreateActorFrom(const FAssetData& InAssetData, FText& OutErrorMsg)
{
	if (!InAssetData.IsValid())
	{
		return false;
	}

	UClass* Class = InAssetData.GetClass();

	return (Class && Class->IsChildOf<UTexture>());
}

AActor* UAvaMaterialDesignerTextureAssetFactory::SpawnActor(UObject* InAsset, ULevel* InLevel,
	const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams)
{
	AActor* InNewActor = Super::SpawnActor(InAsset, InLevel, InTransform, InSpawnParams);

	UTexture* Texture = Cast<UTexture>(InAsset);

	if (!Texture)
	{
		return InNewActor;
	}

	AAvaShapeActor* ShapeActor = Cast<AAvaShapeActor>(InNewActor);

	if (!ShapeActor)
	{
		return InNewActor;
	}

	UAvaShapeDynamicMeshBase* DynMesh = ShapeActor->GetDynamicMesh();
	UAvaShapeRectangleDynamicMesh* RectangleMesh = Cast<UAvaShapeRectangleDynamicMesh>(DynMesh);

	// The method has been called a second time so we don't need further setup.
	if (RectangleMesh)
	{
		return InNewActor;
	}

	// Some error has occurred. We have a mesh... and it's the wrong mesh.
	if (DynMesh)
	{
		return InNewActor;
	}

	FVector ActorForward = InNewActor->GetActorForwardVector();
	ActorForward.Z = 0.f;
	ActorForward.Normalize();

	FVector CameraForward = CameraRotation.Vector();
	CameraForward.Z = 0.f;
	CameraForward.Normalize();

	if (ActorForward.Dot(CameraForward) < 0)
	{
		InNewActor->AddActorWorldRotation(FRotator(0, 180, 0));
	}

	RectangleMesh = NewObject<UAvaShapeRectangleDynamicMesh>(ShapeActor);
	ShapeActor->SetDynamicMesh(RectangleMesh);
	RectangleMesh->SetSize2D({Texture->GetSurfaceWidth(), Texture->GetSurfaceHeight()});
	RectangleMesh->SetMaterial(UAvaShapeDynamicMeshBase::MESH_INDEX_PRIMARY, nullptr); // Force switch to parametric

	UDynamicMeshComponent* MeshComponent = ShapeActor->GetShapeMeshComponent();
	check(MeshComponent);

	UDynamicMaterialInstanceFactory* InstanceFactory = NewObject<UDynamicMaterialInstanceFactory>();
	check(InstanceFactory);

	UDynamicMaterialInstance* NewInstance = Cast<UDynamicMaterialInstance>(InstanceFactory->FactoryCreateNew(
		UDynamicMaterialInstance::StaticClass(), MeshComponent, NAME_None, RF_Transactional, nullptr, GWarn));
	check(NewInstance);

	UDynamicMaterialModel* Model = Cast<UDynamicMaterialModel>(NewInstance->GetMaterialModel());

	if (!Model)
	{
		return InNewActor;
	}

	UDynamicMaterialModelEditorOnlyData* EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(Model);

	if (!EditorOnlyData)
	{
		return InNewActor;
	}

	// Assume true if it's not a texture 2d.
	// It's not reasonable to check the texture format of every single type of texture.
	bool bHasAlphaChannel = true;

	if (UTexture2D* Texture2D = Cast<UTexture2D>(Texture))
	{
		bHasAlphaChannel = Texture2D->HasAlphaChannel();
	}

	if (bHasAlphaChannel)
	{
		EditorOnlyData->SetChannelListPreset(TEXT("Translucent"));
	}
	else
	{
		EditorOnlyData->SetChannelListPreset(TEXT("Emissive"));
	}

	EditorOnlyData->OnWizardComplete();

	const UDMMaterialSlot* Slot = EditorOnlyData->GetSlotForMaterialProperty(EDMMaterialPropertyType::EmissiveColor);

	if (!Slot)
	{
		return InNewActor;
	}

	const UDMMaterialLayerObject* Layer = Slot->GetLayer(0);

	if (!Layer)
	{
		return InNewActor;
	}

	UDMMaterialStage* BaseStage = Layer->GetStage(EDMMaterialLayerStage::Base);

	if (!BaseStage || !BaseStage->IsComponentValid())
	{
		return InNewActor;
	}

	UDMMaterialStageBlend* Blend = Cast<UDMMaterialStageBlend>(BaseStage->GetSource());

	if (!Blend)
	{
		return InNewActor;
	}

	UDMMaterialStageInputExpression* NewInput = UDMMaterialStageInputExpression::ChangeStageInput_Expression(
		BaseStage,
		UDMMaterialStageExpressionTextureSample::StaticClass(),
		UDMMaterialStageBlend::InputB,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		0,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	if (!NewInput)
	{
		return InNewActor;
	}

	UDMMaterialSubStage* SubStage = NewInput->GetSubStage();

	if (!SubStage)
	{
		return InNewActor;
	}

	UDMMaterialStageInputValue* NewInputValue = UDMMaterialStageInputValue::ChangeStageInput_NewLocalValue(
		SubStage,
		0, 
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL,
		EDMValueType::VT_Texture,
		FDMMaterialStageConnectorChannel::WHOLE_CHANNEL
	);

	UDMMaterialValueTexture* TextureValue = Cast<UDMMaterialValueTexture>(NewInputValue->GetValue());

	if (!TextureValue)
	{
		return InNewActor;
	}

	TextureValue->SetValue(Texture);
	RectangleMesh->SetMaterial(UAvaShapeDynamicMeshBase::MESH_INDEX_PRIMARY, NewInstance);
	RectangleMesh->SetMaterialUVMode(UAvaShapeDynamicMeshBase::MESH_INDEX_PRIMARY, EAvaShapeUVMode::Stretch);

	return InNewActor;
}

FString UAvaMaterialDesignerTextureAssetFactory::GetDefaultActorLabel(UObject* InAsset) const
{
	if (InAsset)
	{
		return InAsset->GetName();
	}

	static const FString DefaultName = TEXT("Rectangle");
	return DefaultName;
}
