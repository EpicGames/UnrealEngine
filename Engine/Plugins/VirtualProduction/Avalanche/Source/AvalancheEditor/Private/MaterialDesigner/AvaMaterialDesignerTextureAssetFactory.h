// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"
#include "AvaMaterialDesignerTextureAssetFactory.generated.h"

UCLASS()
class UAvaMaterialDesignerTextureAssetFactory : public UActorFactory
{
	GENERATED_BODY()

	friend class FAvaLevelViewportClient;

public:
	UAvaMaterialDesignerTextureAssetFactory();

	void SetCameraRotation(const FRotator& InRotation);

	//~ Begin UActorFactory
	virtual bool CanCreateActorFrom(const FAssetData& InAssetData, FText& OutErrorMsg) override;
	//~ End UActorFactory

protected:
	bool bFactoryEnabled;
	FRotator CameraRotation;

	//~ Begin UActorFactory
	virtual AActor* SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams) override;
	virtual FString GetDefaultActorLabel(UObject* InAsset) const override;
	//~ End UActorFactory
};
