// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOperatorStackExtension.h"

#include "AvaLevelViewportCommands.h"
#include "DetailView/AvaDetailsExtension.h"
#include "EditorModeManager.h"
#include "Framework/Docking/LayoutExtender.h"
#include "LevelEditor.h"
#include "Selection.h"
#include "Subsystems/OperatorStackEditorSubsystem.h"
#include "Subsystems/PropertyAnimatorCoreSubsystem.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SOperatorStackEditorWidget.h"

#define LOCTEXT_NAMESPACE "AvaOperatorStackExtension"

FAvaOperatorStackExtension::FAvaOperatorStackExtension()
	: AnimatorCommands(MakeShared<FUICommandList>())
{
}

TSharedPtr<SDockTab> FAvaOperatorStackExtension::FindOrOpenTab(bool bInOpen) const
{
	const TSharedPtr<IAvaEditor>& Editor = GetEditor();
	if (!Editor.IsValid())
	{
		return nullptr;
	}

	const TSharedPtr<FTabManager> TabManager = Editor->GetTabManager();
	if (!TabManager.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<SDockTab> Tab = TabManager->FindExistingLiveTab(UOperatorStackEditorSubsystem::TabId);

	if (!Tab.IsValid() && bInOpen)
	{
		Tab = TabManager->TryInvokeTab(UOperatorStackEditorSubsystem::TabId);
	}

	if (!Tab.IsValid())
	{
		return nullptr;
	}

	const TSharedRef<SOperatorStackEditorWidget> Widget = StaticCastSharedRef<SOperatorStackEditorWidget>(Tab->GetContent());

	if (const TSharedPtr<FAvaDetailsExtension> DetailsExtension = Editor->FindExtension<FAvaDetailsExtension>())
	{
		Widget->SetKeyframeHandler(DetailsExtension->GetDetailsKeyframeHandler());
	}

	return Tab;
}

void FAvaOperatorStackExtension::Activate()
{
	FAvaEditorExtension::Activate();

	if (!FindOrOpenTab(/** Open */false))
	{
		UOperatorStackEditorSubsystem::OnOperatorStackSpawned().AddSP(this, &FAvaOperatorStackExtension::OnOperatorStackSpawned);
	}
}

void FAvaOperatorStackExtension::Deactivate()
{
	FAvaEditorExtension::Deactivate();

	UOperatorStackEditorSubsystem::OnOperatorStackSpawned().RemoveAll(this);
}

void FAvaOperatorStackExtension::RegisterTabSpawners(const TSharedRef<IAvaEditor>& InEditor) const
{
}

void FAvaOperatorStackExtension::ExtendLevelEditorLayout(FLayoutExtender& InExtender) const
{
	InExtender.ExtendLayout(LevelEditorTabIds::LevelEditorSceneOutliner
		, ELayoutExtensionPosition::After
		, FTabManager::FTab(UOperatorStackEditorSubsystem::TabId, ETabState::ClosedTab));
}

void FAvaOperatorStackExtension::ExtendToolbarMenu(UToolMenu& InMenu)
{
	FToolMenuSection& Section = InMenu.FindOrAddSection(DefaultSectionName);

	FToolMenuEntry& Entry = Section.AddEntry(FToolMenuEntry::InitToolBarButton(
		TEXT("OpenOperatorStackButton"),
		FExecuteAction::CreateSPLambda(this, [this]() { FindOrOpenTab(/** Open */true); }),
		LOCTEXT("OpenOperatorStackLabel", "Operator Stack"),
		LOCTEXT("OpenOperatorStackTooltip", "Open the operator stack tab."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.UserDefinedStruct")
	));

	Entry.StyleNameOverride = "CalloutToolbar";
}

void FAvaOperatorStackExtension::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	InCommandList->Append(AnimatorCommands);

	const FAvaLevelViewportCommands& ViewportCommands = FAvaLevelViewportCommands::GetExternal();

	AnimatorCommands->MapAction(ViewportCommands.DisableAnimators
		, FExecuteAction::CreateSP(this, &FAvaOperatorStackExtension::EnableAnimators, false));

	AnimatorCommands->MapAction(ViewportCommands.EnableAnimators
		, FExecuteAction::CreateSP(this, &FAvaOperatorStackExtension::EnableAnimators, true));
}

void FAvaOperatorStackExtension::EnableAnimators(bool bInEnable) const
{
	const UWorld* const World = GetWorld();
	if (!IsValid(World))
	{
		return;
	}

	const FEditorModeTools* ModeTools = GetEditorModeTools();
	if (!ModeTools)
	{
		return;
	}

	const UTypedElementSelectionSet* SelectionSet = ModeTools->GetEditorSelectionSet();
	if (!SelectionSet)
	{
		return;
	}

	const TSet<AActor*> SelectedActors(SelectionSet->GetSelectedObjects<AActor>());

	UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get();

	if (!AnimatorSubsystem)
	{
		return;
	}

	// If nothing selected, target all animators in level
	if (!SelectedActors.IsEmpty())
	{
		AnimatorSubsystem->SetActorAnimatorsEnabled(SelectedActors, bInEnable, true);
	}
	else
	{
		AnimatorSubsystem->SetLevelAnimatorsEnabled(World, bInEnable, true);
	}
}

void FAvaOperatorStackExtension::OnOperatorStackSpawned(TSharedRef<SOperatorStackEditorWidget> InWidget)
{
	// Init tab with MD
	FindOrOpenTab(/** Open */false);
}

#undef LOCTEXT_NAMESPACE
