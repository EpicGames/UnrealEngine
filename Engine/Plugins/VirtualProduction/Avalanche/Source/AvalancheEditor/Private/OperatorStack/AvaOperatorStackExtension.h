// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaEditorExtension.h"

class SDockTab;
class SOperatorStackEditorWidget;

class FAvaOperatorStackExtension : public FAvaEditorExtension
{
public:
	UE_AVA_INHERITS(FAvaOperatorStackExtension, FAvaEditorExtension);

	FAvaOperatorStackExtension();

	TSharedPtr<SDockTab> FindOrOpenTab(bool bInOpen) const;

	//~ Begin IAvaEditorExtension
	virtual void Activate() override;
	virtual void Deactivate() override;
	virtual void RegisterTabSpawners(const TSharedRef<IAvaEditor>& InEditor) const override;
	virtual void ExtendLevelEditorLayout(FLayoutExtender& InExtender) const override;
	virtual void ExtendToolbarMenu(UToolMenu& InMenu) override;
	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandList) override;
	//~ End IAvaEditorExtension

private:
	void EnableAnimators(bool bInEnable) const;
	void OnOperatorStackSpawned(TSharedRef<SOperatorStackEditorWidget> InWidget);

	TSharedRef<FUICommandList> AnimatorCommands;
};
