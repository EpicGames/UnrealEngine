// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerExtension.h"
#include "Animation/SkeletalMeshActor.h"
#include "AvaEditorCommands.h"
#include "AvaEditorModule.h"
#include "AvaEditorSettings.h"
#include "AvaOutlinerSubsystem.h"
#include "AvaOutlinerTabSpawner.h"
#include "AvaScene.h"
#include "AvaSceneRigSubsystem.h"
#include "AvaSequence.h"
#include "AvaShapeActor.h"
#include "AvaTextActor.h"
#include "Camera/CameraActor.h"
#include "Components/PrimitiveComponent.h"
#include "EditorModeManager.h"
#include "Engine/PointLight.h"
#include "Engine/SkyLight.h"
#include "Engine/StaticMeshActor.h"
#include "Filters/AvaOutlinerItemTypeFilter.h"
#include "Framework/AvaNullActor.h"
#include "Framework/Docking/LayoutExtender.h"
#include "IAvaOutliner.h"
#include "IAvaOutlinerModule.h"
#include "IAvaSceneRigEditorModule.h"
#include "Item/AvaOutlinerActor.h"
#include "Item/AvaOutlinerComponent.h"
#include "ItemProxies/AvaOutlinerItemProxyRegistry.h"
#include "LevelEditor.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Outliner/AvaOutlinerMaterialDesignerProxy.h"
#include "ScopedTransaction.h"
#include "Sequencer/AvaSequencerExtension.h"
#include "Styling/SlateIconFinder.h"
#include "Text3DActor.h"
#include "ToolMenuContext/AvaOutlinerItemsContext.h"
#include "ToolMenu.h"
#include "Modifiers/ActorModifierCoreBase.h"
#include "Viewport/AvaViewportExtension.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerExtension"

void FAvaOutlinerExtension::StaticStartup()
{
	if (!IAvaOutlinerModule::IsLoaded())
	{
		UE_LOGF(AvaLog, Error, "Outliner module not loaded");
		return;
	}

	IAvaOutlinerModule& OutlinerModule = IAvaOutlinerModule::Get();

	OutlinerModule.GetItemProxyRegistry().RegisterItemProxyWithDefaultFactory<FAvaOutlinerMaterialDesignerProxy, 5>();

	OutlinerModule.GetOnExtendItemProxiesForItem().AddLambda(
		[](IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InItemPtr, TArray<TSharedPtr<FAvaOutlinerItemProxy>>& OutItemProxies)
		{
			if (FAvaOutlinerComponent* ComponentItem = InItemPtr->CastTo<FAvaOutlinerComponent>())
			{
				if (UPrimitiveComponent* const PrimitiveComponent = Cast<UPrimitiveComponent>(ComponentItem->GetComponent()))
				{
					if (TSharedPtr<FAvaOutlinerItemProxy> MaterialItemProxy = InOutliner.GetOrCreateItemProxy<FAvaOutlinerMaterialDesignerProxy>(InItemPtr))
					{
						OutItemProxies.Add(MaterialItemProxy);
					}
				}
			}
		}
	);
}

FAvaOutlinerExtension::FAvaOutlinerExtension()
	: OutlinerCommands(MakeShared<FUICommandList>())
{
}

void FAvaOutlinerExtension::Activate()
{
	if (UWorld* const World = GetWorld())
	{
		if (UAvaOutlinerSubsystem* OutlinerSubsystem = World->GetSubsystem<UAvaOutlinerSubsystem>())
		{
			AvaOutliner = OutlinerSubsystem->GetOrCreateOutliner(*this);
			AvaOutliner->SetBaseCommandList(OutlinerCommands);
		}
		else
		{
			UE_LOGF(AvaLog, Warning, "Missing outliner world subsystem on extension activation.");
		}
	}
}

void FAvaOutlinerExtension::PostInvokeTabs()
{
	if (AvaOutliner.IsValid())
	{
		AvaOutliner->Refresh();
	}
}

void FAvaOutlinerExtension::Deactivate()
{
	AvaOutliner.Reset();
}

void FAvaOutlinerExtension::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	InCommandList->Append(OutlinerCommands);

	OutlinerCommands->MapAction(FAvaEditorCommands::Get().GroupActors
		, FExecuteAction::CreateSP(this, &FAvaOutlinerExtension::GroupSelection));
}

void FAvaOutlinerExtension::RegisterTabSpawners(const TSharedRef<IAvaEditor>& InEditor) const
{
	for (int32 Index = 0; Index < FAvaOutlinerTabSpawner::MaxTabCount; ++Index)
	{
		InEditor->AddTabSpawner<FAvaOutlinerTabSpawner>(Index, InEditor);
	}
}

void FAvaOutlinerExtension::ExtendLevelEditorLayout(FLayoutExtender& InExtender) const
{
	for (int32 Index = 0; Index < FAvaOutlinerTabSpawner::MaxTabCount; ++Index)
	{
		InExtender.ExtendLayout(LevelEditorTabIds::LevelEditorSceneOutliner
			, ELayoutExtensionPosition::Before
			, FTabManager::FTab(FAvaOutlinerTabSpawner::GetTabID(Index), ETabState::ClosedTab));
	}
}

void FAvaOutlinerExtension::Save()
{
	AAvaScene* const Scene = GetSceneObject<AAvaScene>();
	if (!AvaOutliner.IsValid() || !Scene)
	{
		return;
	}

	FMemoryWriter Writer(Scene->GetOutlinerData());
	AvaOutliner->Serialize(Writer);
	Writer.FlushCache();
	Writer.Close();
}

void FAvaOutlinerExtension::Load()
{
	AAvaScene* const Scene = GetSceneObject<AAvaScene>();
	if (!AvaOutliner.IsValid() || !Scene || Scene->GetOutlinerData().IsEmpty())
	{
		return;
	}

	FMemoryReader Reader(Scene->GetOutlinerData());
	Reader.Seek(0);
	AvaOutliner->Serialize(Reader);
	Reader.FlushCache();
	Reader.Close();
}

void FAvaOutlinerExtension::NotifyOnSelectionChanged(const FAvaEditorSelection& InSelection)
{
	if (AvaOutliner.IsValid())
	{
		AvaOutliner->OnObjectSelectionChanged(InSelection);
	}
}

void FAvaOutlinerExtension::OnCopyActors(FString& OutCopyData, TConstArrayView<AActor*> InActorsToCopy)
{
	if (AvaOutliner.IsValid())
	{
		AvaOutliner->OnActorsCopied(OutCopyData, InActorsToCopy);
	}
}

void FAvaOutlinerExtension::PrePasteActors()
{
	// Ignore Spawn/Duplication prior to processing the Paste Data so that Outliner does not automatically add these items in
	if (AvaOutliner.IsValid())
	{
		AvaOutliner->SetIgnoreNotify(FAvaOutlinerExtension::NotifyFlags, /*bIgnore*/true);
	}
}

void FAvaOutlinerExtension::PostPasteActors(bool bInPasteSucceeded)
{
	if (AvaOutliner.IsValid())
	{
		AvaOutliner->SetIgnoreNotify(FAvaOutlinerExtension::NotifyFlags, /*bIgnore*/false);
	}
}

void FAvaOutlinerExtension::OnPasteActors(FStringView InPastedData, TConstArrayView<FAvaEditorPastedActor> InPastedActors)
{
	if (!AvaOutliner.IsValid())
	{
		return;
	}

	constexpr bool bIncludeDuplicatedActors = false;
	TMap<FName, AActor*> PastedActors       = FAvaEditorPastedActor::BuildPastedActorMap(InPastedActors, bIncludeDuplicatedActors);
	TMap<AActor*, AActor*> DuplicatedActors = FAvaEditorPastedActor::BuildDuplicatedActorMap(InPastedActors);

	AvaOutliner->SetIgnoreNotify(FAvaOutlinerExtension::NotifyFlags, /*bIgnore*/false);

	// Process pasted actors that had a source (i.e. deemed a duplicate operation)
	AvaOutliner->OnActorsDuplicated(DuplicatedActors);

	// Process the rest of the pasted actors that did not have a source
	AvaOutliner->OnActorsPasted(InPastedData, PastedActors);
}

bool FAvaOutlinerExtension::ShouldLockOutliner() const
{
	return false;
}

bool FAvaOutlinerExtension::CanOutlinerProcessActorSpawn(AActor* InActor) const
{
	if (!IsValid(InActor) || InActor->bIsEditorPreviewActor)
	{
		return false;
	}

	const TSharedPtr<IAvaEditor> Editor = GetEditor();
	if (!Editor.IsValid())
	{
		return false;
	}

	const TSharedPtr<FAvaViewportExtension> ViewportExtension = Editor->FindExtension<FAvaViewportExtension>();

	return !ViewportExtension.IsValid() || !ViewportExtension->IsDroppingPreviewActor();
}

bool FAvaOutlinerExtension::ShouldHideItem(const FAvaOutlinerItemPtr& InItem) const
{
	// TODO: Decouple Logic from Ava VPC/ModeTools so that it's used here
	return false;
}

void FAvaOutlinerExtension::OutlinerDuplicateActors(const TArray<AActor*>& InTemplateActors)
{
	if (const TSharedPtr<IAvaEditor> Editor = GetEditor())
	{
		// TODO: Set Selection to Template Actors if different
		Editor->EditDuplicate();
	}
}

void FAvaOutlinerExtension::OutlinerDeleteActors(const TArray<AActor*>& InDeleteActors)
{
	if (const TSharedPtr<IAvaEditor> Editor = GetEditor())
	{
		// TODO: Set Selection to Delete Actors if different
		Editor->EditDelete();
	}
}

FEditorModeTools* FAvaOutlinerExtension::GetOutlinerModeTools() const
{
	return GetEditorModeTools();
}

FAvaSceneTree* FAvaOutlinerExtension::GetSceneTree() const
{
	IAvaSceneInterface* const SceneInterface = GetSceneObject<IAvaSceneInterface>();
	return SceneInterface ? &SceneInterface->GetSceneTree() : nullptr;
}

UWorld* FAvaOutlinerExtension::GetOutlinerWorld() const
{
	return GetWorld();
}

FTransform FAvaOutlinerExtension::GetOutlinerDefaultActorSpawnTransform() const
{
	// TODO: Decouple Logic from Ava VPC/ModeTools so that it's used here
	return FTransform::Identity;
}

void FAvaOutlinerExtension::ExtendOutlinerToolBar(UToolMenu* InToolBarMenu)
{
	TSharedPtr<IAvaEditor> Editor = GetEditor();
	if (!Editor.IsValid())
	{
		return;
	}

	if (FToolMenuSection* const MainSection = InToolBarMenu->FindSection("Main"))
	{
		FToolMenuEntry GroupActorsEntry = FToolMenuEntry::InitToolBarButton(FAvaEditorCommands::Get().GroupActors);
		GroupActorsEntry.Icon           = FSlateIconFinder::FindIconForClass(AAvaNullActor::StaticClass());
		GroupActorsEntry.InsertPosition = FToolMenuInsert("ViewOptions", EToolMenuInsertType::Before);
		GroupActorsEntry.SetCommandList(Editor->GetCommandList());

		MainSection->AddEntry(GroupActorsEntry);
	}
}

void FAvaOutlinerExtension::ExtendOutlinerItemContextMenu(UToolMenu* InItemContextMenu)
{
	UAvaOutlinerItemsContext* const ItemsContext = InItemContextMenu->FindContext<UAvaOutlinerItemsContext>();
	if (!ItemsContext || ItemsContext->GetItems().IsEmpty())
	{
		return;
	}

	FToolMenuSection* ContextActionsSection = InItemContextMenu->FindSection("ContextActions");
	if (!ContextActionsSection)
	{
		ContextActionsSection = &InItemContextMenu->AddSection("ContextActions"
			, LOCTEXT("ContextActions", "Context Actions")
			, FToolMenuInsert(NAME_None, EToolMenuInsertType::First));
	}

	// Note: Since the Outliner Command List is linked to the Ava Command List (see IAvaOutliner::SetBaseCommandList),
	// we do NOT need to add the entry with a different Command List
	const FAvaEditorCommands& AvaEditorCommands = FAvaEditorCommands::Get();

	ContextActionsSection->AddMenuEntry(
		AvaEditorCommands.OpenAdvancedRenamerTool_SelectedActors,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Rename")
	);

	ContextActionsSection->AddMenuEntry(
		AvaEditorCommands.OpenAdvancedRenamerTool_SharedClassActors,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Rename")
	);
}

void FAvaOutlinerExtension::ExtendOutlinerItemFilters(TArray<TSharedPtr<IAvaOutlinerItemFilter>>& OutItemFilters)
{
	OutItemFilters.Add(MakeShared<FAvaOutlinerItemTypeFilter>(TEXT("Null")
		, TArray<TSubclassOf<UObject>>{ AAvaNullActor::StaticClass() }
		, EAvaOutlinerTypeFilterMode::MatchesType
		, nullptr
		, LOCTEXT("NullActorFilterTooltip", "Null Actor")));

	OutItemFilters.Add(MakeShared<FAvaOutlinerItemTypeFilter>(TEXT("Modifiers")
		, TArray<TSubclassOf<UObject>>{ UActorModifierCoreBase::StaticClass() }
		, EAvaOutlinerTypeFilterMode::MatchesType | EAvaOutlinerTypeFilterMode::ContainerOfType
		, FSlateIconFinder::FindIconForClass(UActorModifierCoreBase::StaticClass()).GetIcon()
		, LOCTEXT("ModifiersFilterTooltip", "Modifiers")));

	OutItemFilters.Add(MakeShared<FAvaOutlinerItemTypeFilter>(TEXT("Materials")
		, TArray<TSubclassOf<UObject>>{ UMaterialInterface::StaticClass() }
		, EAvaOutlinerTypeFilterMode::MatchesType | EAvaOutlinerTypeFilterMode::ContainerOfType
		, FSlateIconFinder::FindIconForClass(UMaterial::StaticClass()).GetIcon()
		, LOCTEXT("MaterialsFilterTooltip", "Materials")));

	OutItemFilters.Add(MakeShared<FAvaOutlinerItemTypeFilter>(TEXT("Animations")
		, TArray<TSubclassOf<UObject>>{ UAvaSequence::StaticClass() }
		, EAvaOutlinerTypeFilterMode::MatchesType | EAvaOutlinerTypeFilterMode::ContainerOfType
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "UMGEditor.AnimTabIcon").GetIcon()
		, LOCTEXT("AnimationsFilterTooltip", "Animations")));

	OutItemFilters.Add(MakeShared<FAvaOutlinerItemTypeFilter>(TEXT("Text")
		, TArray<TSubclassOf<UObject>>{ AAvaTextActor::StaticClass(), AText3DActor::StaticClass() }));

	OutItemFilters.Add(MakeShared<FAvaOutlinerItemTypeFilter>(TEXT("BlueprintActors")
		, TArray<TSubclassOf<UObject>>{ AActor::StaticClass() }
		, EAvaOutlinerTypeFilterMode::MatchesType
		, nullptr
		, LOCTEXT("BlueprintActorFilterTooltip", "Blueprint Actor")
		, EClassFlags::CLASS_CompiledFromBlueprint));

	OutItemFilters.Add(MakeShared<FAvaOutlinerItemTypeFilter>(TEXT("ToolboxShapes")
		, TArray<TSubclassOf<UObject>>{ AAvaShapeActor::StaticClass() }
		, EAvaOutlinerTypeFilterMode::MatchesType
		, nullptr
		, LOCTEXT("ToolboxActorFilterTooltip", "Toolbox Actor")));

	OutItemFilters.Add(MakeShared<FAvaOutlinerItemTypeFilter>(TEXT("Lights")
		, TArray<TSubclassOf<UObject>>{ ALight::StaticClass(), ASkyLight::StaticClass() }
		, EAvaOutlinerTypeFilterMode::MatchesType
		, FSlateIconFinder::FindIconForClass(APointLight::StaticClass()).GetIcon()));

	OutItemFilters.Add(MakeShared<FAvaOutlinerItemTypeFilter>(TEXT("Camera")
		, TArray<TSubclassOf<UObject>>{ ACameraActor::StaticClass() }));

	OutItemFilters.Add(MakeShared<FAvaOutlinerItemTypeFilter>(TEXT("StaticMesh")
		, TArray<TSubclassOf<UObject>>{ AStaticMeshActor::StaticClass() }));

	OutItemFilters.Add(MakeShared<FAvaOutlinerItemTypeFilter>(TEXT("SkeletalMesh")
		, TArray<TSubclassOf<UObject>>{ ASkeletalMeshActor::StaticClass() }));

	OutItemFilters.Add(MakeShared<FAvaOutlinerItemTypeFilter>(TEXT("SceneRig")
		, FAvaOutlinerItemTypeFilterPassDelegate::CreateSP(this, &FAvaOutlinerExtension::OnSceneRigFilterPass)
		, FAppStyle::GetBrush(TEXT("LandscapeEditor.NoiseTool"))
		, LOCTEXT("SceneRigFilterTooltip", "Scene Rig Actors")));
}

bool FAvaOutlinerExtension::OnSceneRigFilterPass(FAvaOutlinerFilterType InItem) const
{
	if (const FAvaOutlinerActor* const OutlinerActor = InItem.CastTo<FAvaOutlinerActor>())
	{
		return IAvaSceneRigEditorModule::Get().IsActiveSceneRigActor(GetWorld(), OutlinerActor->GetActor());
	}
	return false;
}

TOptional<EItemDropZone> FAvaOutlinerExtension::OnOutlinerItemCanAcceptDrop(const FDragDropEvent& InDragDropEvent
	, EItemDropZone InDropZone
	, FAvaOutlinerItemPtr InTargetItem) const
{
	return TOptional<EItemDropZone>();
}

FReply FAvaOutlinerExtension::OnOutlinerItemAcceptDrop(const FDragDropEvent& InDragDropEvent
	, EItemDropZone InDropZone
	, FAvaOutlinerItemPtr InTargetItem)
{
	return FReply::Unhandled();
}

void FAvaOutlinerExtension::NotifyOutlinerItemRenamed(const FAvaOutlinerItemPtr& InItem)
{
	TSharedPtr<IAvaEditor> Editor = GetEditor();
	if (!InItem.IsValid() || !Editor.IsValid())
	{
		return;
	}

	// Update Sequencer Item Renames
	const FAvaOutlinerObject* ObjectItem = InItem->CastTo<FAvaOutlinerObject>();
	const TSharedPtr<FAvaSequencerExtension> SequencerExtension = Editor->FindExtension<FAvaSequencerExtension>();

	if (ObjectItem && SequencerExtension.IsValid())
	{
		SequencerExtension->OnObjectRenamed(ObjectItem->GetObject(), ObjectItem->GetDisplayName());
	}
}

void FAvaOutlinerExtension::NotifyOutlinerItemLockChanged(const FAvaOutlinerItemPtr& InItem)
{
	// TODO: Decouple Logic from Ava VPC/ModeTools so that it's used here
}

const FAttachmentTransformRules& FAvaOutlinerExtension::GetTransformRule(bool bIsPrimaryTransformRule) const
{
	const bool bKeepRelativeWhenGrouping = UAvaEditorSettings::Get()->bKeepRelativeTransformWhenGrouping;
	const bool bReturnRelative = bIsPrimaryTransformRule == bKeepRelativeWhenGrouping;
	return bReturnRelative ? FAttachmentTransformRules::KeepRelativeTransform : FAttachmentTransformRules::KeepWorldTransform;
}

void FAvaOutlinerExtension::GroupSelection()
{
	UWorld* const World = GetOutlinerWorld();
	if (!IsValid(World))
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("EditGroup", "Group"));
	AvaOutliner->SetIgnoreNotify(EAvaOutlinerIgnoreNotifyFlags::Spawn, true);

	FTransform SpawnTransform;
	TOptional<FAttachmentTransformRules> TransformRules;

	if (UAvaEditorSettings::Get()->bKeepRelativeTransformWhenGrouping)
	{
		SpawnTransform = FTransform::Identity;
		TransformRules = FAttachmentTransformRules::KeepRelativeTransform;
	}
	else
	{
		SpawnTransform.SetLocation(GetOutlinerModeTools()->GetWidgetLocation());
		TransformRules = FAttachmentTransformRules::KeepWorldTransform;
	}

	AActor* const GroupActor = World->SpawnActor(AAvaNullActor::StaticClass(), &SpawnTransform);

	AvaOutliner->SetIgnoreNotify(EAvaOutlinerIgnoreNotifyFlags::Spawn, false);
	AvaOutliner->GroupSelection(GroupActor, TransformRules);

	// Force Immediate Execution of Items while it's in a Scoped Transaction
	AvaOutliner->Refresh();
}

#undef LOCTEXT_NAMESPACE
