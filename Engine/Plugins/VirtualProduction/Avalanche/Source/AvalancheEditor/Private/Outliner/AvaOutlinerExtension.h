// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "IAvaEditorExtension.h"
#include "IAvaOutlinerProvider.h"
#include "Templates/SharedPointer.h"

class IAvaOutliner;

UE_AVA_TYPE_EXTERNAL(IAvaOutlinerProvider);

class FAvaOutlinerExtension : public FAvaEditorExtension, public IAvaOutlinerProvider
{
	/** Flags to use while processing Copy/Paste */
	static constexpr EAvaOutlinerIgnoreNotifyFlags NotifyFlags = EAvaOutlinerIgnoreNotifyFlags::Spawn | EAvaOutlinerIgnoreNotifyFlags::Duplication;

public:
	UE_AVA_INHERITS(FAvaOutlinerExtension, FAvaEditorExtension, IAvaOutlinerProvider);

	static void StaticStartup();

	FAvaOutlinerExtension();

	TSharedPtr<IAvaOutliner> GetAvaOutliner() const { return AvaOutliner; }

	//~ Begin IAvaEditorExtension
	virtual void Activate() override;
	virtual void PostInvokeTabs() override;
	virtual void Deactivate() override;
	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandList) override;
	virtual void RegisterTabSpawners(const TSharedRef<IAvaEditor>& InEditor) const override;
	virtual void ExtendLevelEditorLayout(FLayoutExtender& InExtender) const override;
	virtual void Save() override;
	virtual void Load() override;
	virtual void NotifyOnSelectionChanged(const FAvaEditorSelection& InSelection) override;
	virtual void OnCopyActors(FString& OutCopyData, TConstArrayView<AActor*> InActorsToCopy) override;
	virtual void PrePasteActors() override;
	virtual void PostPasteActors(bool bInPasteSucceeded) override;
	virtual void OnPasteActors(FStringView InPastedData, TConstArrayView<FAvaEditorPastedActor> InPastedActors) override;
	//~ End IAvaEditorExtension

	//~ Begin IAvaOutlinerProvider
	virtual bool ShouldCreateWidget() const override { return true; }
	virtual bool ShouldLockOutliner() const override;
	virtual bool CanOutlinerProcessActorSpawn(AActor* InActor) const override;
	virtual bool ShouldHideItem(const FAvaOutlinerItemPtr& InItem) const override;
	virtual void OutlinerDuplicateActors(const TArray<AActor*>& InTemplateActors) override;
	virtual void OutlinerDeleteActors(const TArray<AActor*>& InDeleteActors) override;
	virtual FEditorModeTools* GetOutlinerModeTools() const override;
	virtual FAvaSceneTree* GetSceneTree() const override;
	virtual UWorld* GetOutlinerWorld() const override;
	virtual FTransform GetOutlinerDefaultActorSpawnTransform() const override;
	virtual void ExtendOutlinerToolBar(UToolMenu* InToolBarMenu) override;
	virtual void ExtendOutlinerItemContextMenu(UToolMenu* InItemContextMenu) override;
	virtual void ExtendOutlinerItemFilters(TArray<TSharedPtr<class IAvaOutlinerItemFilter>>& OutItemFilters) override;
	virtual TOptional<EItemDropZone> OnOutlinerItemCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem) const override;
	virtual FReply OnOutlinerItemAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem) override;
	virtual void NotifyOutlinerItemRenamed(const FAvaOutlinerItemPtr& InItem) override;
	virtual void NotifyOutlinerItemLockChanged(const FAvaOutlinerItemPtr& InItem) override;
	virtual const FAttachmentTransformRules& GetTransformRule(bool bIsPrimaryTransformRule) const override;
	//~ End IAvaOutlinerProvider

private:
	void GroupSelection();

	bool OnSceneRigFilterPass(FAvaOutlinerFilterType InItem) const;

	TSharedPtr<IAvaOutliner> AvaOutliner;

	TSharedRef<FUICommandList> OutlinerCommands;
};

