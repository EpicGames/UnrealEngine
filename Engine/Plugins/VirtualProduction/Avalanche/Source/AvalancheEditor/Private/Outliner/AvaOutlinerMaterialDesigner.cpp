// Copyright Epic Games, Inc. All Rights Reserved.

#include "Outliner/AvaOutlinerMaterialDesigner.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "Components/PrimitiveComponent.h"
#include "Item/AvaOutlinerComponent.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "ScopedTransaction.h"
#include "Material/DynamicMaterialInstance.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerMaterialDesigner"

FAvaOutlinerMaterialDesigner::FAvaOutlinerMaterialDesigner(IAvaOutliner& InOutliner
		, UMaterialInterface* InMaterial
		, const FAvaOutlinerItemPtr& InReferencingItem
		, int32 InMaterialIndex)
	: FAvaOutlinerMaterial(InOutliner, InMaterial, InReferencingItem, InMaterialIndex)
{
}

void FAvaOutlinerMaterialDesigner::Select(FAvaOutlinerScopedSelection& InSelection) const
{
	FAvaOutlinerObjectReference::Select(InSelection);

	if (UDynamicMaterialInstance* MaterialDesignerAsset = Cast<UDynamicMaterialInstance>(Material.Get()))
	{
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		AssetTools.OpenEditorForAssets({MaterialDesignerAsset});
	}
}

#undef LOCTEXT_NAMESPACE
