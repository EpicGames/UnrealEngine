// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Item/AvaOutlinerMaterial.h"
#include "Item/AvaOutlinerObjectReference.h"
#include "Textures/SlateIcon.h"

class UPrimitiveComponent;
class UMaterialInterface;

/**
 * Item in Outliner representing a Material. Inherits from FAvaOutlinerObjectReference as multiple objects can have the same material
 */
class FAvaOutlinerMaterialDesigner : public FAvaOutlinerMaterial
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerMaterialDesigner, FAvaOutlinerMaterial)

	FAvaOutlinerMaterialDesigner(IAvaOutliner& InOutliner
		, UMaterialInterface* InMaterial
		, const FAvaOutlinerItemPtr& InReferencingItem
		, int32 InMaterialIndex);

	//~ Begin IAvaOutlinerItem
	virtual void Select(FAvaOutlinerScopedSelection& InSelection) const override;
	//~ End IAvaOutlinerItem
};
