// Copyright Epic Games, Inc. All Rights Reserved.

#include "Outliner/AvaOutlinerMaterialDesignerProxy.h"
#include "Components/PrimitiveComponent.h"
#include "IAvaOutliner.h"
#include "Item/AvaOutlinerComponent.h"
#include "Item/AvaOutlinerItemParameters.h"
#include "Item/AvaOutlinerMaterial.h"
#include "Material/DynamicMaterialInstance.h"
#include "Materials/Material.h"
#include "Outliner/AvaOutlinerMaterialDesigner.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerMaterialDesignerProxy"

FAvaOutlinerMaterialDesignerProxy::FAvaOutlinerMaterialDesignerProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem)
	: FAvaOutlinerMaterialProxy(InOutliner, InParentItem)
{
}

void FAvaOutlinerMaterialDesignerProxy::GetProxiedItems(const TSharedRef<IAvaOutlinerItem>& InParent, TArray<FAvaOutlinerItemPtr>& OutChildren, bool bInRecursive)
{
	if (const FAvaOutlinerComponent* const ComponentItem = InParent->CastTo<FAvaOutlinerComponent>())
	{
		if (UPrimitiveComponent* const PrimitiveComponent = Cast<UPrimitiveComponent>(ComponentItem->GetComponent()))
		{
			TArray<UMaterialInterface*> Materials;
			PrimitiveComponent->GetUsedMaterials(Materials);

			for (int32 Index = 0; Index < Materials.Num(); ++Index)
			{
				UMaterialInterface* const Material = Materials[Index];
				if (!Material || !Material->IsA<UDynamicMaterialInstance>())
				{
					continue;
				}

				// Note that we pass in the parent of this proxy rather than the proxy itself because
				// the parent (component item) is what's referencing this material
				// Also, the Material Item already has a way to get the proxy (via GetParent())
				FAvaOutlinerItemPtr MaterialItem = Outliner.FindOrAdd<FAvaOutlinerMaterial>(Material, InParent, Index);

				// Upgrade it to a material designer item
				if (!MaterialItem->IsExactlyA<FAvaOutlinerMaterialDesigner>())
				{
					MaterialItem->GetOwnerOutliner()->UnregisterItem(MaterialItem->GetItemId());

					if (FAvaOutlinerItemPtr Parent = MaterialItem->GetParent())
					{
						FAvaOutlinerRemoveItemParams RemoveParams;
						RemoveParams.Item = MaterialItem;
						Parent->RemoveChild(RemoveParams);
					}

					MaterialItem = Outliner.FindOrAdd<FAvaOutlinerMaterialDesigner>(Material, InParent, Index);
					MaterialItem->SetParent(SharedThis(this));

					FAvaOutlinerAddItemParams AddParams;
					AddParams.Items.Add(MaterialItem);
					AddParams.Flags = EAvaOutlinerAddItemFlags::None;
					AddChildren(AddParams);
				}

				// Check if it's already in the array
				const int32 CurrentIndex = OutChildren.IndexOfByPredicate(
					[MaterialItemId = MaterialItem->GetItemId()](const FAvaOutlinerItemPtr& InElement)
					{
						return MaterialItemId == InElement->GetItemId();
					});

				if (CurrentIndex == INDEX_NONE)
				{
					OutChildren.Add(MaterialItem);
				}
				else
				{
					OutChildren[CurrentIndex] = MaterialItem;
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
