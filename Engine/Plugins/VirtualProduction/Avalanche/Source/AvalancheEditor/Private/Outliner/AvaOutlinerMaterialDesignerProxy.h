// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Item/AvaOutlinerMaterialProxy.h"
#include "ItemProxies/IAvaOutlinerItemProxyFactory.h"

class FAvaOutlinerMaterialDesignerProxy : public FAvaOutlinerMaterialProxy
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerMaterialDesignerProxy, FAvaOutlinerMaterialProxy);

	FAvaOutlinerMaterialDesignerProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem);

	virtual ~FAvaOutlinerMaterialDesignerProxy() override = default;

	//~ Begin FAvaOutlinerItemProxy
	virtual void GetProxiedItems(const TSharedRef<IAvaOutlinerItem>& InParent, TArray<FAvaOutlinerItemPtr>& OutChildren, bool bInRecursive) override;
	//~ End FAvaOutlinerItemProxy
};
