// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerTabSpawner.h"
#include "AvaOutlinerExtension.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "IAvaEditor.h"
#include "IAvaOutliner.h"
#include "IAvaOutlinerView.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerTabSpawner"

FName FAvaOutlinerTabSpawner::GetTabID(int32 InOutlinerId)
{
	return *FString::Printf(TEXT("AvaOutliner%d"), InOutlinerId + 1);
}

FAvaOutlinerTabSpawner::FAvaOutlinerTabSpawner(int32 InOutlinerId, const TSharedRef<IAvaEditor>& InEditor)
	: FAvaTabSpawner(InEditor, FAvaOutlinerTabSpawner::GetTabID(InOutlinerId))
	, OutlinerViewId(InOutlinerId)
{
	TabLabel = FText::Format(LOCTEXT("OtherTabLabels", "Motion Design Outliner {0}"), FText::AsNumber(OutlinerViewId + 1));

	TabTooltipText = LOCTEXT("TabTooltip", "Motion Design Outliner");

	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner");
}

TSharedRef<SWidget> FAvaOutlinerTabSpawner::CreateTabBody()
{
	const TSharedPtr<IAvaEditor> Editor = EditorWeak.Pin();
	if (!ensure(Editor.IsValid()))
	{
		return GetNullWidget();
	}

	const TSharedPtr<FAvaOutlinerExtension> OutlinerExtension = Editor->FindExtension<FAvaOutlinerExtension>();
	if (!ensure(OutlinerExtension.IsValid()))
	{
		return GetNullWidget();
	}

	const TSharedPtr<IAvaOutliner> Outliner = OutlinerExtension->GetAvaOutliner();
	if (!ensure(Outliner.IsValid()))
	{
		return GetNullWidget();
	}

	TSharedPtr<IAvaOutlinerView> OutlinerView = Outliner->GetOutlinerView(OutlinerViewId);
	if (!OutlinerView.IsValid())
	{
		OutlinerView = Outliner->RegisterOutlinerView(OutlinerViewId);
	}

	if (!ensure(OutlinerView.IsValid()))
	{
		return GetNullWidget();
	}
	return OutlinerView->GetOutlinerWidget();
}

FTabSpawnerEntry& FAvaOutlinerTabSpawner::RegisterTabSpawner(const TSharedRef<FTabManager>& InTabManager, const TSharedPtr<FWorkspaceItem>& InWorkspaceMenu)
{
	TSharedRef<FWorkspaceItem> GroupItem = InWorkspaceMenu.ToSharedRef();

	if (const TSharedPtr<IAvaEditor> Editor = EditorWeak.Pin())
	{
		if (const TSharedPtr<FWorkspaceItem>& OutlinerCategory = Editor->GetOutlinerCategory())
		{
			GroupItem = OutlinerCategory.ToSharedRef();
		}
	}

	return FAvaTabSpawner::RegisterTabSpawner(InTabManager, nullptr)
		.SetGroup(GroupItem);
}

#undef LOCTEXT_NAMESPACE
