// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTabSpawner.h"

class FAvaOutlinerTabSpawner : public FAvaTabSpawner
{
public:
	static constexpr int32 MaxTabCount = 4;
	static FName GetTabID(int32 InOutlinerId = 0);

	explicit FAvaOutlinerTabSpawner(int32 InOutlinerId, const TSharedRef<IAvaEditor>& InEditor);

	//~ Begin IAvaTabSpawner
	virtual TSharedRef<SWidget> CreateTabBody() override;
	virtual FTabSpawnerEntry& RegisterTabSpawner(const TSharedRef<FTabManager>& InTabManager, const TSharedPtr<FWorkspaceItem>& InWorkspaceMenu) override;
	//~ End IAvaTabSpawner

private:
	int32 OutlinerViewId = -1;
};
