// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRCExtension.h"
#include "AvaRCSignatureCustomization.h"
#include "Editor.h"
#include "Framework/Docking/LayoutExtender.h"
#include "IAvaSceneInterface.h"
#include "IRemoteControlUIModule.h"
#include "LevelEditor.h"
#include "Misc/MessageDialog.h"
#include "RemoteControlPreset.h"
#include "RemoteControlTrackerComponent.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Subsystems/RemoteControlComponentsSubsystem.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AvaRCExtension"

URemoteControlPreset* FAvaRCExtension::GetRemoteControlPreset() const
{
	const IAvaSceneInterface* const Scene = GetSceneObject<IAvaSceneInterface>();
	return Scene ? Scene->GetRemoteControlPreset() : nullptr;
}

void FAvaRCExtension::Activate()
{
	RegisterSignatureCustomization();
	OpenRemoteControlTab(/*bCreateIfNotFound*/false);
}

void FAvaRCExtension::Deactivate()
{
	CloseRemoteControlTab();
	UnregisterSignatureCustomization();
}

void FAvaRCExtension::ExtendLevelEditorLayout(FLayoutExtender& InExtender) const
{
	const FName RemoteControlTabId(TEXT("RemoteControl_RemoteControlPanel"));

	InExtender.ExtendLayout(LevelEditorTabIds::Sequencer
		, ELayoutExtensionPosition::Before
		, FTabManager::FTab(RemoteControlTabId, ETabState::ClosedTab));
}

void FAvaRCExtension::ExtendToolbarMenu(UToolMenu& InMenu)
{
	FToolMenuSection& Section = InMenu.FindOrAddSection(DefaultSectionName);

	const FSlateIcon RemoteControlIcon(TEXT("RemoteControlPanelStyle"), "Icons.RemoteControlPanel");

	FToolMenuEntry& Entry = Section.AddEntry(FToolMenuEntry::InitToolBarButton(TEXT("RemoteControlButton")
		,  FExecuteAction::CreateSP(this, &FAvaRCExtension::OpenRemoteControlTab, /*bCreateIfNotFound*/true)
		, LOCTEXT("RemoteControlLabel"  , "Remote Control")
		, LOCTEXT("RemoteControlTooltip", "Opens the Remote Control Editor for the given Scene")
		, RemoteControlIcon));

	Entry.StyleNameOverride = "CalloutToolbar";

	Section.AddEntry(FToolMenuEntry::InitComboButton(TEXT("RemoteControlComboButton")
		, FUIAction()
		, FNewToolMenuDelegate::CreateSP(this, &FAvaRCExtension::GenerateRemoteControlOptions)
		, LOCTEXT("RemoteControlOptionsLabel", "Remote Control Options")
		, LOCTEXT("RemoteControlOptionsTooltip", "Remote Control Options")
		, RemoteControlIcon
		, /*bSimpleComboBox*/true));
}

void FAvaRCExtension::OnSceneObjectChanged(UObject* InOldSceneObject, UObject* InNewSceneObject)
{
	TSharedPtr<IAvaEditor> Editor = GetEditor();
	if (!Editor.IsValid())
	{
		return;
	}

	TSharedPtr<IToolkitHost> ToolkitHost = Editor->GetToolkitHost();
	if (!ToolkitHost.IsValid())
	{
		return;
	}

	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	check(AssetEditorSubsystem);

	if (const IAvaSceneInterface* OldScene = Cast<IAvaSceneInterface>(InOldSceneObject))
	{
		if (URemoteControlPreset* OldPreset = OldScene->GetRemoteControlPreset())
		{
			AssetEditorSubsystem->CloseAllEditorsForAsset(OldPreset);
		}
	}

	if (const IAvaSceneInterface* NewScene = Cast<IAvaSceneInterface>(InNewSceneObject))
	{
		if (URemoteControlPreset* NewPreset = NewScene->GetRemoteControlPreset())
		{
			// Level-dependent bindings are resolved via the Selected World in the Preset. By default, this should be the current edited world.
			// However, the Current Edited World does not hold ownership of the other sublevels, and so doing a "FindObject" or similar with this world
			// will fail for any actor/subobject within these sublevels.
			// So set the Selected World to be the true outer world of the new scene object level.
			NewPreset->SelectedWorld = InNewSceneObject->GetTypedOuter<UWorld>();

			AssetEditorSubsystem->OpenEditorForAsset(NewPreset, EToolkitMode::WorldCentric, ToolkitHost);
		}
	}
}

void FAvaRCExtension::OpenRemoteControlTab(bool bInCreateIfNotFound) const
{
	IAvaSceneInterface* const Scene = GetSceneObject<IAvaSceneInterface>();
	if (!Scene)
	{
		return;
	}

	URemoteControlPreset* RemoteControlPreset = Scene->GetRemoteControlPreset();
	if (!RemoteControlPreset)
	{
		if (!bInCreateIfNotFound)
		{
			return;
		}

		const EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::YesNo
			, LOCTEXT("CreateRCPresetMessage", "Remote Control Preset is not present in this level. Do you wish to create it?")
			, LOCTEXT("CreateRCPresetTitle", "Create Remote Control Preset"));

		if (Response != EAppReturnType::Yes)
		{
			return;
		}

		RemoteControlPreset = Scene->GetOrCreateRemoteControlPreset();
		if (!RemoteControlPreset)
		{
			return;
		}
	}

	TSharedPtr<IAvaEditor> Editor = GetEditor();
	if (!Editor.IsValid())
	{
		return;
	}

	TSharedPtr<IToolkitHost> ToolkitHost = Editor->GetToolkitHost();
	if (!ToolkitHost.IsValid())
	{
		return;
	}

	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	check(AssetEditorSubsystem);
	AssetEditorSubsystem->OpenEditorForAsset(RemoteControlPreset, EToolkitMode::WorldCentric, ToolkitHost);
}

void FAvaRCExtension::CloseRemoteControlTab() const
{
	if (URemoteControlPreset* const RemoteControlPreset = GetRemoteControlPreset())
	{
		if (UAssetEditorSubsystem* const AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
		{
			AssetEditorSubsystem->CloseAllEditorsForAsset(RemoteControlPreset);
		}
	}
}

void FAvaRCExtension::RegisterSignatureCustomization()
{
	UnregisterSignatureCustomization();

	SignatureCustomization = MakeShared<FAvaRCSignatureCustomization>();
	IRemoteControlUIModule::Get().RegisterSignatureCustomization(SignatureCustomization);
}

void FAvaRCExtension::UnregisterSignatureCustomization()
{
	if (!SignatureCustomization.IsValid())
	{
		return;
	}

	if (IRemoteControlUIModule* RCUIModule = FModuleManager::GetModulePtr<IRemoteControlUIModule>(TEXT("RemoteControlUI")))
	{
		RCUIModule->UnregisterSignatureCustomization(SignatureCustomization);
	}

	SignatureCustomization.Reset();
}

bool FAvaRCExtension::CanDeleteRemoteControlPreset() const
{
	return !!GetRemoteControlPreset();
}

void FAvaRCExtension::DeleteRemoteControlPreset()
{
	const EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::YesNo
		, LOCTEXT("DeleteRemoteControlMessage", "Do you wish to delete the Remote Control Preset? (experimental)")
		, LOCTEXT("DeleteRemoteControlTitle", "Delete Remote Control Preset"));

	if (Response != EAppReturnType::Yes)
	{
		return;
	}

	CloseRemoteControlTab();

	if (IAvaSceneInterface* const Scene = GetSceneObject<IAvaSceneInterface>())
	{
		constexpr bool bCallModify = true;
PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		Scene->DestroyRemoteControlPreset(bCallModify);
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	}
}

void FAvaRCExtension::GenerateRemoteControlOptions(UToolMenu* InMenu)
{
	FToolMenuSection& GeneralSection = InMenu->FindOrAddSection(TEXT("GeneralSection"), LOCTEXT("GeneralSectionLabel", "General"));

	GeneralSection.AddMenuEntry(TEXT("DeleteRemoteControl")
		, LOCTEXT("DeleteRemoteControlLabel", "Delete Remote Control (experimental)")
		, LOCTEXT("DeleteRemoteControlTooltip", "Deletes the existing embedded Remote Control.")
		, TAttribute<FSlateIcon>()
		, FUIAction(FExecuteAction::CreateSP(this, &FAvaRCExtension::DeleteRemoteControlPreset)
			, FCanExecuteAction::CreateSP(this, &FAvaRCExtension::CanDeleteRemoteControlPreset))
		, EUserInterfaceActionType::Button);
}

#undef LOCTEXT_NAMESPACE
