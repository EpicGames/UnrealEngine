// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaEditorExtension.h"

class IRCSignatureCustomization;
class URemoteControlPreset;
class URemoteControlTrackerComponent;

class FAvaRCExtension: public FAvaEditorExtension
{
public:
	UE_AVA_INHERITS(FAvaRCExtension, FAvaEditorExtension);

	virtual ~FAvaRCExtension() override = default;

	URemoteControlPreset* GetRemoteControlPreset() const;

	//~ Begin IAvaEditorExtension
	virtual void Activate() override;
	virtual void Deactivate() override;
	virtual void ExtendLevelEditorLayout(FLayoutExtender& InExtender) const override;
	virtual void ExtendToolbarMenu(UToolMenu& InMenu) override;
	virtual void OnSceneObjectChanged(UObject* InOldSceneObject, UObject* InNewSceneObject) override;
	//~ End IAvaEditorExtension

protected:
	void OpenRemoteControlTab(bool bInCreateIfNotFound) const;
	void CloseRemoteControlTab() const;

	void RegisterSignatureCustomization();
	void UnregisterSignatureCustomization();

	/** Determines whether the scene's rc preset can be deleted */
	bool CanDeleteRemoteControlPreset() const;

	/** Deletes the scene's rc preset (if any) */
	void DeleteRemoteControlPreset();

	/** Generates the sub-menu options for remote control */
	void GenerateRemoteControlOptions(UToolMenu* InMenu);

	TSharedPtr<IRCSignatureCustomization> SignatureCustomization;
};
