// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRCSignatureCustomization.h"
#include "DragDropOps/AvaOutlinerItemDragDropOp.h"
#include "GameFramework/Actor.h"
#include "UI/Signature/IRCSignatureItem.h"

bool FAvaRCSignatureCustomization::CanAcceptDrop(const FDragDropEvent& InDragDropEvent, IRCSignatureItem* InSignatureItem) const
{
	return InDragDropEvent.GetOperationAs<FAvaOutlinerItemDragDropOp>().IsValid();
}

FReply FAvaRCSignatureCustomization::AcceptDrop(const FDragDropEvent& InDragDropEvent, IRCSignatureItem* InSignatureItem) const
{
	// Motion Design Outliner Drag Drop
	if (TSharedPtr<FAvaOutlinerItemDragDropOp> OutlinerDragDrop = InDragDropEvent.GetOperationAs<FAvaOutlinerItemDragDropOp>())
	{
		TArray<TWeakObjectPtr<AActor>> DragDropActors;
		OutlinerDragDrop->GetDragDropOpActors(DragDropActors);

		TArray<TWeakObjectPtr<UObject>> DragDropObjects(MoveTemp(DragDropActors));
		InSignatureItem->ApplySignature(DragDropObjects);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}
