// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/Signature/IRCSignatureCustomization.h"

class FAvaRCSignatureCustomization : public IRCSignatureCustomization
{
	//~ Begin IRCSignatureCustomization
	virtual bool CanAcceptDrop(const FDragDropEvent& InDragDropEvent, IRCSignatureItem* InSignatureItem) const override;
	virtual FReply AcceptDrop(const FDragDropEvent& InDragDropEvent, IRCSignatureItem* InSignatureItem) const override;
	//~ End IRCSignatureCustomization
};
