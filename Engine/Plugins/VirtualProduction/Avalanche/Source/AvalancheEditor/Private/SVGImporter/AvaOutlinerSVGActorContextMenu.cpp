// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerSVGActorContextMenu.h"
#include "GameFramework/Actor.h"
#include "ISVGImporterEditorModule.h"
#include "Item/AvaOutlinerActor.h"
#include "ToolMenuContext/AvaOutlinerItemsContext.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerSVGActorContextMenu"

void FAvaOutlinerSVGActorContextMenu::OnExtendOutlinerContextMenu(UToolMenu* InToolMenu)
{
	if (!InToolMenu)
	{
		return;
	}

	TSet<TWeakObjectPtr<AActor>> OutObjects;
	GetActorItems(InToolMenu, OutObjects);

	if (!OutObjects.IsEmpty())
	{
		AddSVGActorMenuEntries(InToolMenu, OutObjects);
	}
}

void FAvaOutlinerSVGActorContextMenu::GetActorItems(const UToolMenu* InToolMenu, TSet<TWeakObjectPtr<AActor>>& InActors)
{
	if (const UAvaOutlinerItemsContext* OutlinerItemsContext = InToolMenu->Context.FindContext<UAvaOutlinerItemsContext>())
	{
		if (!IsValid(OutlinerItemsContext))
		{
			return;
		}

		for (FAvaOutlinerItemWeakPtr ItemWeak : OutlinerItemsContext->GetItems())
		{
			const TSharedPtr<IAvaOutlinerItem> Item = ItemWeak.Pin();

			// Is it an actor
			if (const FAvaOutlinerActor* ActorItem = Item->CastTo<FAvaOutlinerActor>())
			{
				if (AActor* Actor = ActorItem->GetActor())
				{
					InActors.Add(Actor);
				}
			}
		}
	}
}

void FAvaOutlinerSVGActorContextMenu::AddSVGActorMenuEntries(UToolMenu* InToolMenu, const TSet<TWeakObjectPtr<AActor>>& InActors)
{
	ISVGImporterEditorModule::Get().AddSVGActorMenuEntries(InToolMenu, InActors);
}

#undef LOCTEXT_NAMESPACE
