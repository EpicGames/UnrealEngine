// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "ToolMenus.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FAvaOutlinerSVGActorContextMenu
{
public:
	/** Extend outliner context menu */
	static void OnExtendOutlinerContextMenu(UToolMenu* InToolMenu);

private:
	static void GetActorItems(const UToolMenu* InToolMenu, TSet<TWeakObjectPtr<AActor>>& InActors);
	static void AddSVGActorMenuEntries(UToolMenu* InToolMenu, const TSet<TWeakObjectPtr<AActor>>& InActors);
};
