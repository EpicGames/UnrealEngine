// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scene/AvaSceneDefaults.h"
#include "ActorFactories/ActorFactory.h"
#include "AssetViewerSettings.h"
#include "AvaEditorSettings.h"
#include "Builders/CubeBuilder.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/SkyLightComponent.h"
#include "Engine/BrushBuilder.h"
#include "Engine/DirectionalLight.h"
#include "Engine/PostProcessVolume.h"
#include "Engine/SkyLight.h"
#include "Engine/TextureCube.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/AvaNullActor.h"
#include "GameFramework/Actor.h"
#include "IAvaEditor.h"
#include "Scene/SAvaSceneDefaultActorResponses.h"
#include "Templates/SharedPointer.h"
#include "Viewport/AvaCineCameraActor.h"
#include "Viewport/AvaPostProcessVolume.h"
#include "Viewport/AvaViewportExtension.h"
#include "ViewportClient/IAvaViewportClient.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "AvaSceneDefaults"

using FAvaSceneDefaultActorList = TMap<FName, TSharedRef<FAvaSceneDefaultActorResponse>>;

DECLARE_DELEGATE_ThreeParams(FAvaSceneDefaultActorInitDelegate, TSharedRef<IAvaEditor>, const FAvaSceneDefaultActorList&, AActor*)

namespace UE::AvaEditor::Private
{
	static const FPreviewSceneProfile DefaultSceneProfile;

	struct FAvaSceneDefaultActor
	{
		/** The class of the actor to create/replace/update*/
		TWeakObjectPtr<UClass> ClassWeak;
		/** Describes the actor to create/replace/update */
		FText Description;
		/** Delegate called to initialize the actor */
		FAvaSceneDefaultActorInitDelegate InitDelegate;
		/** Response to apply when there are available actors. */
		EAvaSceneDefaultActorResponse DesiredResponse = EAvaSceneDefaultActorResponse::ReplaceActor;
	};

	namespace DefaultSceneActorNames
	{
		static FName DefaultRoot = "DefaultRoot";
		static FName DirectionalLight = "DirectionalLight";
		static FName SkyLight = "SkyLight";
		static FName PostProcessVolume = "PostProcessVolume";
		static FName CineCamera = "CineCamera";
	};

	void InitDefaultSceneActor_DefaultRoot(TSharedRef<IAvaEditor> InEditor, const FAvaSceneDefaultActorList& InActorResponses, AActor* InActor);
	void InitDefaultSceneActor_DirectionalLight(TSharedRef<IAvaEditor> InEditor, const FAvaSceneDefaultActorList& InActorResponses, AActor* InActor);
	void InitDefaultSceneActor_SkyLight(TSharedRef<IAvaEditor> InEditor, const FAvaSceneDefaultActorList& InActorResponses, AActor* InActor);
	void InitDefaultSceneActor_PostProcessVolume(TSharedRef<IAvaEditor> InEditor, const FAvaSceneDefaultActorList& InActorResponses, AActor* InActor);
	void InitDefaultSceneActor_Camera(TSharedRef<IAvaEditor> InEditor, const FAvaSceneDefaultActorList& InActorResponses, AActor* InActor);

	static const TMap<FName, FAvaSceneDefaultActor> DefaultSceneActors = {
		{
			DefaultSceneActorNames::DefaultRoot,
			{
				AAvaNullActor::StaticClass(),
				LOCTEXT("DefaultRoot", "Default Scene Root"),
				FAvaSceneDefaultActorInitDelegate::CreateStatic(&InitDefaultSceneActor_DefaultRoot),
				EAvaSceneDefaultActorResponse::CreateNewActor
			}
		},
		{
			DefaultSceneActorNames::DirectionalLight,
			{
				ADirectionalLight::StaticClass(),
				LOCTEXT("DirectionalLight", "Directional Light"),
				FAvaSceneDefaultActorInitDelegate::CreateStatic(&InitDefaultSceneActor_DirectionalLight)
			}
		},
		{

			DefaultSceneActorNames::SkyLight,
			{
				ASkyLight::StaticClass(),
				LOCTEXT("SkyLight", "Sky Light"),
				FAvaSceneDefaultActorInitDelegate::CreateStatic(&InitDefaultSceneActor_SkyLight)
			}
		},
		{ DefaultSceneActorNames::PostProcessVolume,
			{
				AAvaPostProcessVolume::StaticClass(),
				LOCTEXT("PostProcessVolume", "Post Process Volume"),
				FAvaSceneDefaultActorInitDelegate::CreateStatic(&InitDefaultSceneActor_PostProcessVolume)
			}
		},
		{
			DefaultSceneActorNames::CineCamera, 
			{
				AAvaCineCameraActor::StaticClass(),
				LOCTEXT("CineCamera", "Cine Camera"),
				FAvaSceneDefaultActorInitDelegate::CreateStatic(&InitDefaultSceneActor_Camera)
			}
		}
	};

	AActor* GetActorFromResponses(TSharedRef<IAvaEditor> InEditor, const FAvaSceneDefaultActorList& InActorResponses, FName InName)
	{
		if (const TSharedRef<FAvaSceneDefaultActorResponse>* ActorInput = InActorResponses.Find(InName))
		{
			return (*ActorInput)->SelectedActor.Get();
		}

		return nullptr;
	}

	void InitDefaultSceneActor_DefaultRoot(TSharedRef<IAvaEditor> InEditor, const FAvaSceneDefaultActorList& InActorResponses, AActor* InActor)
	{
		if (!InActor)
		{
			return;
		}

		InActor->SetActorLocation(FVector(-500, 0, 250));
		InActor->SetActorLabel(TEXT("Default Scene"));
	}

	void InitDefaultSceneActor_DirectionalLight(TSharedRef<IAvaEditor> InEditor, const FAvaSceneDefaultActorList& InActorResponses, AActor* InActor)
	{
		ADirectionalLight* DirectionalLight = Cast<ADirectionalLight>(InActor);

		if (!DirectionalLight)
		{
			return;
		}

		using namespace UE::AvaEditor::Private;

		ULightComponent* const LightComponent = DirectionalLight->GetLightComponent();

		LightComponent->SetIntensity(DefaultSceneProfile.DirectionalLightIntensity);
		LightComponent->SetLightColor(DefaultSceneProfile.DirectionalLightColor.ToFColor(true));
		LightComponent->SetMobility(EComponentMobility::Movable);

		if (AActor* DefaultRoot = GetActorFromResponses(InEditor, InActorResponses, DefaultSceneActorNames::DefaultRoot))
		{
			static FTransform Transform = {
				FRotator::ZeroRotator,
				FVector(0, 0, 50),
				FVector::OneVector
			};

			DirectionalLight->AttachToActor(DefaultRoot, FAttachmentTransformRules::KeepWorldTransform);
			DirectionalLight->SetActorRelativeTransform(Transform);
		}
	}

	void InitDefaultSceneActor_SkyLight(TSharedRef<IAvaEditor> InEditor, const FAvaSceneDefaultActorList& InActorResponses, AActor* InActor)
	{
		ASkyLight* SkyLight = Cast<ASkyLight>(InActor);

		if (!SkyLight)
		{
			return;
		}

		USkyLightComponent* const SkyLightComponent = SkyLight->GetLightComponent();
		SkyLightComponent->bLowerHemisphereIsBlack = false;
		SkyLightComponent->SourceType = ESkyLightSourceType::SLS_SpecifiedCubemap;
		SkyLightComponent->Mobility = EComponentMobility::Movable;
		SkyLightComponent->Intensity = DefaultSceneProfile.SkyLightIntensity;
		SkyLightComponent->SourceCubemapAngle = DefaultSceneProfile.LightingRigRotation;

		// Load cube map from stored path
		UObject* LoadedObject = LoadObject<UObject>(nullptr, *DefaultSceneProfile.EnvironmentCubeMapPath);

		while (const UObjectRedirector* Redirector = Cast<UObjectRedirector>(LoadedObject))
		{
			LoadedObject = Redirector->DestinationObject;
		}

		SkyLightComponent->SetCubemap(Cast<UTextureCube>(LoadedObject));
		SkyLightComponent->SetVisibility(DefaultSceneProfile.bUseSkyLighting, true);

		if (AActor* DefaultRoot = GetActorFromResponses(InEditor, InActorResponses, DefaultSceneActorNames::DefaultRoot))
		{
			static FTransform Transform = {
				FRotator::ZeroRotator,
				FVector(0, 0, 100),
				FVector::OneVector
			};

			SkyLight->AttachToActor(DefaultRoot, FAttachmentTransformRules::KeepWorldTransform);
			SkyLight->SetActorRelativeTransform(Transform);
		}
	}

	void InitDefaultSceneActor_PostProcessVolume(TSharedRef<IAvaEditor> InEditor, const FAvaSceneDefaultActorList& InActorResponses, AActor* InActor)
	{
		if (!InActor)
		{
			return;
		}

		if (AActor* DefaultRoot = GetActorFromResponses(InEditor, InActorResponses, DefaultSceneActorNames::DefaultRoot))
		{
			static FTransform Transform = {
				FRotator::ZeroRotator,
				FVector(0, 0, 150),
				FVector::OneVector
			};

			InActor->AttachToActor(DefaultRoot, FAttachmentTransformRules::KeepWorldTransform);
			InActor->SetActorRelativeTransform(Transform);
		}
	}

	void InitDefaultSceneActor_Camera(TSharedRef<IAvaEditor> InEditor, const FAvaSceneDefaultActorList& InActorResponses, AActor* InActor)
	{
		AAvaCineCameraActor* Camera = Cast<AAvaCineCameraActor>(InActor);

		if (!Camera)
		{
			return;
		}

		if (AActor* DefaultRoot = GetActorFromResponses(InEditor, InActorResponses, DefaultSceneActorNames::DefaultRoot))
		{
			FTransform Transform = FTransform::Identity;
			Transform.SetLocation(FVector(-UAvaEditorSettings::Get()->CameraDistance, 0, 0));

			Camera->AttachToActor(DefaultRoot, FAttachmentTransformRules::KeepWorldTransform);
			Camera->SetActorRelativeTransform(Transform);
		}

		Camera->Configure(UAvaEditorSettings::Get()->CameraDistance);
		Camera->SetLockLocation(true);

		TSharedPtr<FAvaViewportExtension> ViewportExtension = InEditor->FindExtension<FAvaViewportExtension>();

		for (const TSharedPtr<IAvaViewportClient>& ViewportClient : ViewportExtension->GetViewportClients())
		{
			if (!ViewportClient->IsMotionDesignViewport())
			{
				continue;
			}

			ViewportClient->SetViewTarget(Camera);
			break;
		}
	}

	TMap<FName, TSharedRef<FAvaSceneDefaultActorResponse>> GeneratorInitialActorResponses(UWorld* InWorld)
	{
		TMap<FName, TSharedRef<FAvaSceneDefaultActorResponse>> ActorResponses;

		// Associate a list of actors and a pointer to the next index to use in that list
		TMap<FObjectKey, TTuple<TArray<TWeakObjectPtr<AActor>>, int32>> FoundActors;

		for (const TPair<FName, FAvaSceneDefaultActor>& DefaultPair : DefaultSceneActors)
		{
			UClass* const Class = DefaultPair.Value.ClassWeak.Get();
			if (!Class)
			{
				continue;
			}

			TSharedRef<FAvaSceneDefaultActorResponse> ActorResponse = MakeShared<FAvaSceneDefaultActorResponse>();
			ActorResponse->ActorClassWeak = Class;
			ActorResponse->Description = DefaultPair.Value.Description;

			const FObjectKey ClassKey = FObjectKey(Class);

			if (TTuple<TArray<TWeakObjectPtr<AActor>>, int32>* const FoundActorsPtr = FoundActors.Find(ClassKey))
			{
				const TArray<TWeakObjectPtr<AActor>>& ActorList = FoundActorsPtr->Key;
				int32& ActorIndex = FoundActorsPtr->Value;

				if (ActorList.IsValidIndex(ActorIndex))
				{
					ActorResponse->SelectedActor = ActorList[ActorIndex];
					++ActorIndex;

					ActorResponse->Response = EAvaSceneDefaultActorResponse::ReplaceActor;
				}
			}
			else
			{
				// Could be any number of actors.
				ActorResponse->AvailableActors.Reset();

				for (AActor* Actor : FActorRange(InWorld))
				{
					// Ensure exact class match
					if (Actor->GetClass() == Class)
					{
						ActorResponse->AvailableActors.Add(Actor);
					}
				}

				if (!ActorResponse->AvailableActors.IsEmpty())
				{
					ActorResponse->SelectedActor = ActorResponse->AvailableActors[0];
					ActorResponse->Response = DefaultPair.Value.DesiredResponse;
				}

				// The next index to be used in 1. 0 was just used.
				FoundActors.Add(ClassKey, {ActorResponse->AvailableActors, 1});
			}

			ActorResponses.Add(DefaultPair.Key, ActorResponse);
		}

		return ActorResponses;
	}

	bool ShowActorResponsesWindow(UWorld* InWorld, TMap<FName, TSharedRef<FAvaSceneDefaultActorResponse>>& InOutResponses)
	{
		TArray<TSharedRef<FAvaSceneDefaultActorResponse>> ResponseValues;
		InOutResponses.GenerateValueArray(ResponseValues);

		TSharedPtr<SAvaSceneDefaultActorResponses> UserResponsesWidget = SNew(SAvaSceneDefaultActorResponses, InWorld, ResponseValues);

		const TAttribute<FText> WindowTitle = LOCTEXT("UserResponsesWindowTitle", "Configure Default Scene Actors");

		const float Height = DefaultSceneActors.Num() * 30.f + 80.f;

		const TSharedRef<SWindow> UserResponsesWindow =
			SNew(SWindow)
			.Title(WindowTitle)
			.ClientSize(FVector2f(575.0f, Height))
			.SizingRule(ESizingRule::FixedSize)
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.FocusWhenFirstShown(true)
			.Content()
			[
				UserResponsesWidget.ToSharedRef()
			];

		FSlateApplication::Get().AddModalWindow(UserResponsesWindow, nullptr, false);

		return UserResponsesWidget->WasAccepted();
	}

	void CreateNewSceneActors(UWorld* InWorld, TMap<FName, TSharedRef<FAvaSceneDefaultActorResponse>>& InOutResponses)
	{
		for (TPair<FName, TSharedRef<FAvaSceneDefaultActorResponse>>& ResponsePair : InOutResponses)
		{
			const FAvaSceneDefaultActor* DefaultSceneActor = DefaultSceneActors.Find(ResponsePair.Key);

			if (!DefaultSceneActor)
			{
				continue;
			}

			FString NewActorLabel;

			switch (ResponsePair.Value->Response)
			{
				case EAvaSceneDefaultActorResponse::UpdateActor:
					continue;

				case EAvaSceneDefaultActorResponse::SkipActor:
					ResponsePair.Value->SelectedActor = nullptr;
					break;

				case EAvaSceneDefaultActorResponse::ReplaceActor:
					if (AActor* OldActor = ResponsePair.Value->SelectedActor.Get())
					{
						NewActorLabel = OldActor->GetActorLabel(/* Create if none */ false);
						OldActor->Destroy();
					}

					// Fall through to create new actor

				case EAvaSceneDefaultActorResponse::CreateNewActor:
				{
					FActorSpawnParameters SpawnParameters;
					SpawnParameters.bNoFail = true;

					AActor* NewActor = nullptr;

					if (UClass* ActorClass = DefaultSceneActor->ClassWeak.Get())
					{
						NewActor = InWorld->SpawnActor(ActorClass, nullptr, SpawnParameters);

						if (NewActor)
						{
							if (AVolume* VolumeActor = Cast<AVolume>(NewActor))
							{
								UCubeBuilder* Builder = NewObject<UCubeBuilder>();
								UActorFactory::CreateBrushForVolumeActor(VolumeActor, Builder);
							}

							if (!NewActorLabel.IsEmpty())
							{
								NewActor->SetActorLabel(NewActorLabel);
							}
						}
					}

					ResponsePair.Value->SelectedActor = NewActor;
					break;
				}
			}
		}
	}

	void ApplyActorSettings(TSharedRef<IAvaEditor> InEditor, TMap<FName, TSharedRef<FAvaSceneDefaultActorResponse>>& InResponses)
	{
		for (TPair<FName, TSharedRef<FAvaSceneDefaultActorResponse>>& ResponsePair : InResponses)
		{
			if (ResponsePair.Value->Response == EAvaSceneDefaultActorResponse::SkipActor)
			{
				continue;
			}

			const FAvaSceneDefaultActor* DefaultSceneActor = DefaultSceneActors.Find(ResponsePair.Key);

			if (!DefaultSceneActor)
			{
				continue;
			}

			DefaultSceneActor->InitDelegate.ExecuteIfBound(InEditor, InResponses, ResponsePair.Value->SelectedActor.Get());
		}
	}
}

void FAvaSceneDefaults::CreateDefaultScene(TSharedRef<IAvaEditor> InEditor, UWorld* InWorld)
{
	if (!InWorld)
	{
		return;
	}

	using namespace UE::AvaEditor::Private;

	TMap<FName, TSharedRef<FAvaSceneDefaultActorResponse>> ActorResponses = GeneratorInitialActorResponses(InWorld);

	if (ShowActorResponsesWindow(InWorld, ActorResponses))
	{
		CreateNewSceneActors(InWorld, ActorResponses);
		ApplyActorSettings(InEditor, ActorResponses);
	}
}

#undef LOCTEXT_NAMESPACE
