// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class IAvaEditor;
class UWorld;

enum class EAvaSceneDefaultActorResponse : uint8
{
	SkipActor,
	CreateNewActor,
	ReplaceActor,
	UpdateActor
};

struct FAvaSceneDefaultActorResponse
{
	TWeakObjectPtr<UClass> ActorClassWeak;
	FText Description = FText::GetEmpty();
	TArray<TWeakObjectPtr<AActor>> AvailableActors = {};
	TWeakObjectPtr<AActor> SelectedActor = nullptr;
	EAvaSceneDefaultActorResponse Response = EAvaSceneDefaultActorResponse::CreateNewActor;
};

class FAvaSceneDefaults
{
public:
	static void CreateDefaultScene(TSharedRef<IAvaEditor> InEditor, UWorld* InWorld);
};
