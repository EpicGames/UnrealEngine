// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneExtension.h"
#include "AvaSceneSettingsTabSpawner.h"
#include "Engine/World.h"
#include "Framework/Docking/LayoutExtender.h"
#include "LevelEditor.h"
#include "Scene/AvaSceneDefaults.h"
#include "ScopedTransaction.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "AvaSceneExtension"

void FAvaSceneExtension::ExtendToolbarMenu(UToolMenu& InMenu)
{
	FToolMenuSection& Section = InMenu.FindOrAddSection(DefaultSectionName);

	FToolMenuEntry& Entry = Section.AddEntry(FToolMenuEntry::InitToolBarButton(TEXT("CreateDefaultsButton")
		, FExecuteAction::CreateSP(this, &FAvaSceneExtension::SpawnDefaultScene)
		, LOCTEXT("CreateDefaultSceneLabel", "Create Defaults")
		, LOCTEXT("CreateDefaultSceneTooltip", "Open the Spawn Defaults menu to add a basic scene setup.")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.SpawnActor_16x")));

	Entry.StyleNameOverride = "CalloutToolbar";
}

void FAvaSceneExtension::RegisterTabSpawners(const TSharedRef<IAvaEditor>& InEditor) const
{
	InEditor->AddTabSpawner<FAvaSceneSettingsTabSpawner>(InEditor);
}

void FAvaSceneExtension::ExtendLevelEditorLayout(FLayoutExtender& InExtender) const
{
	InExtender.ExtendLayout(LevelEditorTabIds::LevelEditorSelectionDetails
		, ELayoutExtensionPosition::After
		, FTabManager::FTab(FAvaSceneSettingsTabSpawner::GetTabID(), ETabState::ClosedTab));
}

void FAvaSceneExtension::SpawnDefaultScene()
{
	TSharedPtr<IAvaEditor> Editor = GetEditor();

	if (!Editor.IsValid())
	{
		return;
	}

	UWorld* const World = GetWorld();

	if (!IsValid(World))
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("SpawnDefaultScene", "Spawn Default Scene"));

	FAvaSceneDefaults::CreateDefaultScene(Editor.ToSharedRef(), World);
}

#undef LOCTEXT_NAMESPACE
