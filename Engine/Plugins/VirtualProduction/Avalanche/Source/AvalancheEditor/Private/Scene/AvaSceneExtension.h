// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaEditorExtension.h"

class FAvaSceneExtension : public FAvaEditorExtension
{
public:
	UE_AVA_INHERITS(FAvaSceneExtension, FAvaEditorExtension);

	//~ Begin IAvaEditorExtension
	virtual void ExtendToolbarMenu(UToolMenu& InMenu) override;
	virtual void RegisterTabSpawners(const TSharedRef<IAvaEditor>& InEditor) const override;
	virtual void ExtendLevelEditorLayout(FLayoutExtender& InExtender) const override;
	//~ End IAvaEditorExtension

private:
	void SpawnDefaultScene();
};
