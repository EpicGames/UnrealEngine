// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneRigExtension.h"
#include "AvaSceneRigEditorCommands.h"
#include "AvaSceneRigSubsystem.h"
#include "AvaSceneSettings.h"
#include "AvaSceneSubsystem.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "Engine/World.h"
#include "IAvaSceneInterface.h"
#include "IAvaSceneRigEditorModule.h"
#include "Item/AvaOutlinerActor.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "ToolMenu.h"
#include "ToolMenuContext/AvaOutlinerItemsContext.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AvaSceneRigExtension"

void FAvaSceneRigExtension::Activate()
{
	if (IAvaOutlinerModule::IsLoaded())
	{
		OutlinerItemContextMenuDelegate = IAvaOutlinerModule::Get().GetOnExtendOutlinerItemContextMenu()
			.AddSP(this, &FAvaSceneRigExtension::ExtendOutlinerItemContextMenu);
	}
}

void FAvaSceneRigExtension::Deactivate()
{
	if (IAvaOutlinerModule::IsLoaded() && OutlinerItemContextMenuDelegate.IsValid())
	{
		IAvaOutlinerModule::Get().GetOnExtendOutlinerItemContextMenu().Remove(OutlinerItemContextMenuDelegate);
		OutlinerItemContextMenuDelegate.Reset();
	}
}

void FAvaSceneRigExtension::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	FAvaEditorExtension::BindCommands(InCommandList);

	const FAvaSceneRigEditorCommands& SceneRigEditorCommands = FAvaSceneRigEditorCommands::GetExternal();
	
	InCommandList->MapAction(
		SceneRigEditorCommands.PromptToSaveSceneRigFromOutlinerItems,
		FExecuteAction::CreateSP(this, &FAvaSceneRigExtension::PromptToSaveSceneRigFromOutlinerItems),
		FCanExecuteAction::CreateSP(this, &FAvaSceneRigExtension::CanSaveSceneRigFromOutlinerItems));

	InCommandList->MapAction(
		SceneRigEditorCommands.AddOutlinerItemsToSceneRig,
		FExecuteAction::CreateSP(this, &FAvaSceneRigExtension::AddOutlinerItemsToSceneRig),
		FCanExecuteAction::CreateSP(this, &FAvaSceneRigExtension::CanAddOutlinerItemsToSceneRig));

	InCommandList->MapAction(
		SceneRigEditorCommands.RemoveOutlinerItemsToSceneRig,
		FExecuteAction::CreateSP(this, &FAvaSceneRigExtension::RemoveOutlinerItemsFromSceneRig),
		FCanExecuteAction::CreateSP(this, &FAvaSceneRigExtension::CanRemoveOutlinerItemsFromSceneRig));

	CommandListWeak = InCommandList;
}

void FAvaSceneRigExtension::ExtendOutlinerItemContextMenu(UToolMenu* const InToolMenu)
{
	if (!InToolMenu)
	{
		return;
	}

	UAvaOutlinerItemsContext* const ItemsContext = InToolMenu->FindContext<UAvaOutlinerItemsContext>();
	if (!ItemsContext)
	{
		return;
	}

	const TConstArrayView<FAvaOutlinerItemWeakPtr> SelectedItems = ItemsContext->GetItems();
	if (SelectedItems.IsEmpty())
	{
		return;
	}

	ItemsContextWeak = ItemsContext;

	const TArray<AActor*> SelectedActors = OutlinerItemsToActors(ItemsContextWeak.Get()->GetItems(), true);
	if (!SelectedActors.IsEmpty() && UAvaSceneRigSubsystem::AreActorsSupported(SelectedActors))
	{
		const FAvaSceneRigEditorCommands& SceneRigEditorCommands = FAvaSceneRigEditorCommands::GetExternal();

		FToolMenuSection& NewSection = InToolMenu->AddSection(TEXT("SceneRig"), LOCTEXT("SceneRig", "Scene Rig")
			, FToolMenuInsert(TEXT("ContextActions"), EToolMenuInsertType::After));

		FToolMenuEntry SceneRigMenuEntry = NewSection.AddSubMenu(TEXT("SceneRig")
			, LOCTEXT("SceneRig", "Scene Rig")
			, LOCTEXT("SceneRigToolTip", "Scene Rig")
			, FNewToolMenuDelegate::CreateSP(this, &FAvaSceneRigExtension::CreateSubMenu)
			, false
			, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("LandscapeEditor.NoiseTool")));
	}
}

void FAvaSceneRigExtension::CreateSubMenu(UToolMenu* const InToolMenu)
{
	InToolMenu->AddDynamicSection(TEXT("SceneRig"), FNewToolMenuDelegate::CreateSPLambda(this, [this](UToolMenu* const InToolMenu)
		{
			const TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin();
			const FAvaSceneRigEditorCommands& SceneRigEditorCommands = FAvaSceneRigEditorCommands::GetExternal();

			FToolMenuSection& NewSection = InToolMenu->AddSection(TEXT("SceneRig"), LOCTEXT("SceneRigActions", "Scene Rig"));

			if (UWorld* const World = GetWorld())
			{
				if (UAvaSceneRigSubsystem* const SceneRigSubsystem = UAvaSceneRigSubsystem::ForWorld(World))
				{
					if (const ULevelStreaming* const StreamingLevel = SceneRigSubsystem->FindFirstActiveSceneRig())
					{
						TArray<AActor*> SelectedActors;

						if (const UAvaOutlinerItemsContext* const ItemsContext = ItemsContextWeak.Get())
						{
							const TConstArrayView<FAvaOutlinerItemWeakPtr> SelectedItems = ItemsContext->GetItems();
							SelectedActors = OutlinerItemsToActors(ItemsContext->GetItems(), true);
						}

						if (!SelectedActors.IsEmpty())
						{
							NewSection.AddMenuEntryWithCommandList(SceneRigEditorCommands.RemoveOutlinerItemsToSceneRig
								, CommandList
								, LOCTEXT("RemoveFromSceneRig", "Remove from Scene Rig")
								, SceneRigEditorCommands.RemoveOutlinerItemsToSceneRig->GetDescription()
								, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("MeshPaint.NextTexture.Small")));

							NewSection.AddMenuEntryWithCommandList(SceneRigEditorCommands.AddOutlinerItemsToSceneRig
								, CommandList
								, LOCTEXT("MoveToSceneRig", "Move to Scene Rig")
								, SceneRigEditorCommands.AddOutlinerItemsToSceneRig->GetDescription()
								, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("MeshPaint.PreviousTexture.Small")));

							NewSection.AddSeparator(NAME_None);
						}
					}
				}
			}

			NewSection.AddMenuEntryWithCommandList(SceneRigEditorCommands.PromptToSaveSceneRigFromOutlinerItems
				, CommandList
				, SceneRigEditorCommands.PromptToSaveSceneRigFromOutlinerItems->GetLabel()
				, SceneRigEditorCommands.PromptToSaveSceneRigFromOutlinerItems->GetDescription()
				, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("AssetEditor.SaveAssetAs")));
		}));
}

void FAvaSceneRigExtension::PromptToSaveSceneRigFromOutlinerItems()
{
	if (!CanSaveSceneRigFromOutlinerItems())
	{
		return;
	}

	const UAvaOutlinerItemsContext* const ItemsContext = ItemsContextWeak.Get();
	if (!ItemsContext)
	{
		return;
	}

	UWorld* const World = GetWorld();
	if (!World)
	{
		return;
	}

	const TArray<AActor*> SelectedActors = OutlinerItemsToActors(ItemsContext->GetItems());
	if (SelectedActors.IsEmpty())
	{
		return;
	}

	IAvaSceneRigEditorModule& SceneRigEditorModule = IAvaSceneRigEditorModule::Get();
	
	const FSoftObjectPath NewAssetPath = SceneRigEditorModule.CreateSceneRigAssetWithDialog();
	if (NewAssetPath.IsValid())
	{
		if (SceneRigEditorModule.SetActiveSceneRig(World, NewAssetPath))
		{
			SceneRigEditorModule.AddActiveSceneRigActors(World, SelectedActors);
		}
	}
}

bool FAvaSceneRigExtension::CanSaveSceneRigFromOutlinerItems() const
{
	TArray<AActor*> SelectedActors;

	if (const TObjectPtr<UAvaOutlinerItemsContext> ItemsContext = ItemsContextWeak.Get())
	{
		SelectedActors = OutlinerItemsToActors(ItemsContext->GetItems());
	}

	return !SelectedActors.IsEmpty() && UAvaSceneRigSubsystem::AreActorsSupported(SelectedActors);
}

void FAvaSceneRigExtension::AddOutlinerItemsToSceneRig()
{
	if (!CanAddOutlinerItemsToSceneRig())
	{
		return;
	}

	const UAvaOutlinerItemsContext* const ItemsContext = ItemsContextWeak.Get();
	if (!ItemsContext)
	{
		return;
	}

	const TArray<AActor*> SelectedActors = OutlinerItemsToActors(ItemsContext->GetItems());
	if (SelectedActors.IsEmpty())
	{
		return;
	}

	UWorld* const World = GetWorld();
	if (!World)
	{
		return;
	}

	UAvaSceneRigSubsystem* const SceneRigSubsystem = UAvaSceneRigSubsystem::ForWorld(World);
	if (!SceneRigSubsystem)
	{
		return;
	}

	IAvaSceneRigEditorModule& SceneRigEditorModule = IAvaSceneRigEditorModule::Get();

	if (ULevelStreaming* const SceneRig = SceneRigSubsystem->FindFirstActiveSceneRig())
	{
		FScopedTransaction Transaction(LOCTEXT("AddSceneRig", "Add Scene Rig Actors"));
		SceneRig->Modify();

		SceneRigEditorModule.AddActiveSceneRigActors(World, SelectedActors);
	}
	else
	{
		const FSoftObjectPath NewAssetPath = SceneRigEditorModule.CreateSceneRigAssetWithDialog();
		if (NewAssetPath.IsValid())
		{
			if (SceneRigEditorModule.SetActiveSceneRig(World, NewAssetPath))
			{
				SceneRigEditorModule.AddActiveSceneRigActors(World, SelectedActors);
			}
		}
	}
}

bool FAvaSceneRigExtension::CanAddOutlinerItemsToSceneRig() const
{
	const UAvaOutlinerItemsContext* const ItemsContext = ItemsContextWeak.Get();
	if (!ItemsContext)
	{
		return false;
	}

	const TArray<AActor*> SelectedActors = OutlinerItemsToActors(ItemsContext->GetItems());
	if (SelectedActors.IsEmpty() || !UAvaSceneRigSubsystem::AreActorsSupported(SelectedActors))
	{
		return false;
	}

	const UAvaSceneRigSubsystem* const SceneRigSubsystem = UAvaSceneRigSubsystem::ForWorld(GetWorld());
	if (!SceneRigSubsystem)
	{
		return false;
	}

	UWorld* const SceneRigAsset = SceneRigSubsystem->FindFirstActiveSceneRigAsset();
	if (!SceneRigAsset || !SceneRigAsset->PersistentLevel)
	{
		return false;
	}

	return !UAvaSceneRigSubsystem::AreAllActorsInLevel(SceneRigAsset->PersistentLevel, SelectedActors);
}

void FAvaSceneRigExtension::RemoveOutlinerItemsFromSceneRig()
{
	if (!CanRemoveOutlinerItemsFromSceneRig())
	{
		return;
	}

	const UAvaOutlinerItemsContext* const ItemsContext = ItemsContextWeak.Get();
	if (!ItemsContext)
	{
		return;
	}

	UWorld* const World = GetWorld();
	if (!World)
	{
		return;
	}
	
	UAvaSceneRigSubsystem* const SceneRigSubsystem = UAvaSceneRigSubsystem::ForWorld(World);
	if (!SceneRigSubsystem)
	{
		return;
	}

	ULevelStreaming* const SceneRig = SceneRigSubsystem->FindFirstActiveSceneRig();
	if (!SceneRig)
	{
		return;
	}

	const TArray<AActor*> SelectedActors = OutlinerItemsToActors(ItemsContext->GetItems());

	FScopedTransaction Transaction(LOCTEXT("RemoveSceneRig", "Remove Scene Rig"));
	SceneRig->Modify();

	IAvaSceneRigEditorModule::Get().RemoveActiveSceneRigActors(World, SelectedActors);
}

bool FAvaSceneRigExtension::CanRemoveOutlinerItemsFromSceneRig() const
{
	const UAvaOutlinerItemsContext* const ItemsContext = ItemsContextWeak.Get();
	if (!ItemsContext)
	{
		return false;
	}

	const TArray<AActor*> SelectedActors = OutlinerItemsToActors(ItemsContext->GetItems());

	if (SelectedActors.IsEmpty() || !UAvaSceneRigSubsystem::AreActorsSupported(SelectedActors))
	{
		return false;
	}

	const UAvaSceneRigSubsystem* const SceneRigSubsystem = UAvaSceneRigSubsystem::ForWorld(GetWorld());
	if (!SceneRigSubsystem)
	{
		return false;
	}

	UWorld* const SceneRigAsset = SceneRigSubsystem->FindFirstActiveSceneRigAsset();
	if (!SceneRigAsset || !SceneRigAsset->PersistentLevel)
	{
		return false;
	}

	return UAvaSceneRigSubsystem::AreSomeActorsInLevel(SceneRigAsset->PersistentLevel, SelectedActors);
}

TArray<AActor*> FAvaSceneRigExtension::OutlinerItemsToActors(const TConstArrayView<FAvaOutlinerItemWeakPtr>& InOutlinerItems
	, const bool bInIncludeLocked)
{
	TArray<AActor*> OutActors;

	for (const FAvaOutlinerItemWeakPtr& ItemWeak : InOutlinerItems)
	{
		if (const FAvaOutlinerItemPtr& Item = ItemWeak.Pin())
		{
			if (const FAvaOutlinerActor* const ActorItem = Item->CastTo<FAvaOutlinerActor>())
			{
				if (bInIncludeLocked || !ActorItem->IsLocked())
				{
					if (AActor* const Actor = ActorItem->GetActor())
					{
						OutActors.Add(Actor);
					}
				}
			}
		}
	}

	return OutActors;
}

#undef LOCTEXT_NAMESPACE
