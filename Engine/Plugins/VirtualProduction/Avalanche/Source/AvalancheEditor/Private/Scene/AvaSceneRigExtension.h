// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Containers/ArrayView.h"
#include "IAvaEditorExtension.h"
#include "IAvaOutlinerModule.h"

class AActor;
class FUICommandList;
class IAvaEditor;
class UAvaOutlinerItemsContext;
class UToolMenu;

class FAvaSceneRigExtension : public FAvaEditorExtension
{
public:
	UE_AVA_INHERITS(FAvaSceneRigExtension, FAvaEditorExtension);

	void PromptToSaveSceneRigFromOutlinerItems();
	bool CanSaveSceneRigFromOutlinerItems() const;

	void AddOutlinerItemsToSceneRig();
	bool CanAddOutlinerItemsToSceneRig() const;

	void RemoveOutlinerItemsFromSceneRig();
	bool CanRemoveOutlinerItemsFromSceneRig() const;

	//~ Begin IAvaEditorExtension
	virtual void Activate() override;
	virtual void Deactivate() override;
	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandList) override;
	//~ End IAvaEditorExtension

protected:
	static TArray<AActor*> OutlinerItemsToActors(const TConstArrayView<FAvaOutlinerItemWeakPtr>& InOutlinerItems
		, const bool bInIncludeLocked = false);

	void ExtendOutlinerItemContextMenu(UToolMenu* const InToolMenu);

	void CreateSubMenu(UToolMenu* const InToolMenu);

	FDelegateHandle OutlinerItemContextMenuDelegate;

	TWeakPtr<FUICommandList> CommandListWeak;

	TWeakObjectPtr<UAvaOutlinerItemsContext> ItemsContextWeak;
};
