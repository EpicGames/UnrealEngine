// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneSettingsTabSpawner.h"
#include "AvaSceneSettings.h"
#include "DetailView/Customizations/AvaSceneSettingsCustomization.h"
#include "IAvaEditor.h"
#include "IAvaSceneInterface.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "AvaSceneSettingsTabSpawner"

FName FAvaSceneSettingsTabSpawner::GetTabID()
{
	return TEXT("AvalancheSceneSettingsTabSpawner");
}

FAvaSceneSettingsTabSpawner::FAvaSceneSettingsTabSpawner(const TSharedRef<IAvaEditor>& InEditor)
	: FAvaTabSpawner(InEditor, FAvaSceneSettingsTabSpawner::GetTabID())
{
	TabLabel       = LOCTEXT("TabLabel", "Scene Settings");
	TabTooltipText = LOCTEXT("TabTooltip", "Scene Settings");
	TabIcon        = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.World");
}

TSharedRef<SWidget> FAvaSceneSettingsTabSpawner::CreateTabBody()
{
	const TSharedPtr<IAvaEditor> Editor = EditorWeak.Pin();
	if (!Editor.IsValid())
	{
		return GetNullWidget();
	}

	IAvaSceneInterface* SceneObject = Cast<IAvaSceneInterface>(Editor->GetSceneObject());
	if (!SceneObject)
	{
		return GetNullWidget();
	}

	UAvaSceneSettings* SceneSettings = SceneObject->GetSceneSettings();
	if (!SceneSettings)
	{
		return GetNullWidget();
	}

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	DetailsView->RegisterInstancedCustomPropertyLayout(UAvaSceneSettings::StaticClass()
		, FOnGetDetailCustomizationInstance::CreateStatic(&FAvaSceneSettingsCustomization::MakeInstance, EditorWeak));

	DetailsView->SetObject(SceneSettings);
	return DetailsView;
}

#undef LOCTEXT_NAMESPACE
