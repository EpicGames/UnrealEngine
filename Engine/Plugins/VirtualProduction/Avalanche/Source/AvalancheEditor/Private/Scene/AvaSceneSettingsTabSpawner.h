// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTabSpawner.h"

class FAvaSceneSettingsTabSpawner : public FAvaTabSpawner
{
public:
	static FName GetTabID();

	explicit FAvaSceneSettingsTabSpawner(const TSharedRef<IAvaEditor>& InEditor);

	//~ Begin IAvaTabSpawner
	virtual TSharedRef<SWidget> CreateTabBody() override;
	//~ End IAvaTabSpawner
};
