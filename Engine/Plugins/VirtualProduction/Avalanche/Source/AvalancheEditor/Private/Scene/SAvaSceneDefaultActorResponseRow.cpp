// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scene/SAvaSceneDefaultActorResponseRow.h"
#include "ActorTreeItem.h"
#include "PropertyCustomizationHelpers.h"
#include "GameFramework/Actor.h"
#include "Modules/ModuleManager.h"
#include "Scene/SAvaSceneDefaultActorResponses.h"
#include "SceneOutlinerModule.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/StyleColors.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "SAvaSceneDefaultActorResponseRow"

namespace UE::AvaEditor::Private
{
	static const FMargin AvaSceneDefaultActorResponseRowPadding = FMargin(5.f, 5.f);
}

void SAvaSceneDefaultActorResponseRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView,
	const TSharedRef<SAvaSceneDefaultActorResponses>& InActorResponses, const TSharedRef<FAvaSceneDefaultActorResponse>& InActorResponse)
{
	ActorResponsesWeak = InActorResponses;
	ActorResponse = InActorResponse;

	SMultiColumnTableRow<TSharedRef<FAvaSceneDefaultActorResponse>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

TSharedRef<SWidget> SAvaSceneDefaultActorResponseRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	if (!ActorResponse.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	if (InColumnName == SAvaSceneDefaultActorResponses::DescriptionRowName)
	{
		return GenerateRow_Description();
	}

	if (InColumnName == SAvaSceneDefaultActorResponses::TargetActorRowName)
	{
		return GenerateRow_Actor();
	}

	if (InColumnName == SAvaSceneDefaultActorResponses::CreateNewRowName)
	{
		return GenerateRow_CreateNew();
	}

	if (InColumnName == SAvaSceneDefaultActorResponses::ReplaceRowName)
	{
		return GenerateRow_Replace();
	}

	if (InColumnName == SAvaSceneDefaultActorResponses::UpdateRowName)
	{
		return GenerateRow_Update();
	}

	if (InColumnName == SAvaSceneDefaultActorResponses::EnabledRowName)
	{
		return GenerateRow_Enabled();
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SAvaSceneDefaultActorResponseRow::GenerateRow_Description()
{
	using namespace UE::AvaEditor::Private;

	FSlateIcon ActorIcon = FSlateIconFinder::FindIconForClass(AActor::StaticClass());

	if (ActorResponse.IsValid())
	{
		if (UClass* ActorClass = ActorResponse->ActorClassWeak.Get())
		{
			ActorIcon = FSlateIconFinder::FindIconForClass(ActorClass);
		}
	}

	return SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(AvaSceneDefaultActorResponseRowPadding)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(16.f))
				.Image(ActorIcon.GetIcon())
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.f, 0.f, 0.f, .0f)
			[
				SNew(STextBlock)
				.Text(ActorResponse->Description)
				.ColorAndOpacity(this, &SAvaSceneDefaultActorResponseRow::GetDescriptionColor)
			]
		];
}

TSharedRef<SWidget> SAvaSceneDefaultActorResponseRow::GenerateRow_Actor()
{
	using namespace UE::AvaEditor::Private;

	TSharedPtr<SWidget> ActorContent;
	TSharedPtr<SAvaSceneDefaultActorResponses> ActorResponses = ActorResponsesWeak.Pin();

	if (!ActorResponse.IsValid() || ActorResponse->AvailableActors.IsEmpty() || !ActorResponses.IsValid())
	{
		ActorContent = SNew(STextBlock)
			.Text(LOCTEXT("Error", "Error"));
	}
	else
	{
		auto ActorHasValidClass = [](const AActor* InActor, TWeakObjectPtr<UClass> InClass)
			{
				return InActor->GetClass() == InClass.Get();
			};

		TSharedRef<SWidget> Picker = PropertyCustomizationHelpers::MakeActorPickerAnchorButton(
			FOnGetActorFilters::CreateLambda(
				[InClass = ActorResponse->ActorClassWeak, ActorHasValidClass](TSharedPtr<FSceneOutlinerFilters>& OutFilters)
				{
					OutFilters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateLambda(ActorHasValidClass, InClass));
				}),
			FOnActorSelected::CreateSP(this, &SAvaSceneDefaultActorResponseRow::OnActorPickerSelected)
		);

		Picker->SetEnabled(ActorResponse->AvailableActors.Num() > 1);

		ActorContent = 
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SAvaSceneDefaultActorResponseRow::GetActorSelectRowLabel)
				.ColorAndOpacity(this, &SAvaSceneDefaultActorResponseRow::GetActorSelectRowColor)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.f, 0.f, 0.f, 0.f)
			[
				Picker
			];
	}

	return SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(AvaSceneDefaultActorResponseRowPadding)
		.Visibility(this, &SAvaSceneDefaultActorResponseRow::GetActorSelectVisibility)
		[
			ActorContent.ToSharedRef()
		];
}

TSharedRef<SWidget> SAvaSceneDefaultActorResponseRow::GenerateRow_CreateNew()
{
	using namespace UE::AvaEditor::Private;

	return SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(AvaSceneDefaultActorResponseRowPadding)
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "RadioButton")
			.IsChecked(this, &SAvaSceneDefaultActorResponseRow::GetCreateNewCheckBoxState)
			.Visibility(this, &SAvaSceneDefaultActorResponseRow::GetCreateNewCheckBoxVisibility)
			.OnCheckStateChanged(this, &SAvaSceneDefaultActorResponseRow::OnCreateNewCheckBoxChanged)
		];
}

TSharedRef<SWidget> SAvaSceneDefaultActorResponseRow::GenerateRow_Replace()
{
	using namespace UE::AvaEditor::Private;

	return SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(AvaSceneDefaultActorResponseRowPadding)
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "RadioButton")
			.IsChecked(this, &SAvaSceneDefaultActorResponseRow::GetReplaceCheckBoxState)
			.Visibility(this, &SAvaSceneDefaultActorResponseRow::GetReplaceCheckBoxVisibility)
			.OnCheckStateChanged(this, &SAvaSceneDefaultActorResponseRow::OnReplaceCheckBoxChanged)
		];
}

TSharedRef<SWidget> SAvaSceneDefaultActorResponseRow::GenerateRow_Update()
{
	using namespace UE::AvaEditor::Private;

	return SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(AvaSceneDefaultActorResponseRowPadding)
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "RadioButton")
			.IsChecked(this, &SAvaSceneDefaultActorResponseRow::GetUpdateCheckBoxState)
			.Visibility(this, &SAvaSceneDefaultActorResponseRow::GetUpdateCheckBoxVisibility)
			.OnCheckStateChanged(this, &SAvaSceneDefaultActorResponseRow::OnUpdateCheckBoxChanged)
		];
}

TSharedRef<SWidget> SAvaSceneDefaultActorResponseRow::GenerateRow_Enabled()
{
	using namespace UE::AvaEditor::Private;

	return SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(AvaSceneDefaultActorResponseRowPadding)
		[
			SNew(SCheckBox)
			.IsChecked(this, &SAvaSceneDefaultActorResponseRow::GetEnabledCheckBoxState)
			.OnCheckStateChanged(this, &SAvaSceneDefaultActorResponseRow::OnEnabledCheckBoxChanged)
		];
}

FSlateColor SAvaSceneDefaultActorResponseRow::GetDescriptionColor() const
{
	static const FSlateColor Normal = FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText").ColorAndOpacity;
	static const FSlateColor Ignored = FSlateColor(EStyleColor::AccentGray);

	if (ActorResponse.IsValid() && ActorResponse->Response != EAvaSceneDefaultActorResponse::SkipActor)
	{
		return Normal;
	}

	return Ignored;
}

ECheckBoxState SAvaSceneDefaultActorResponseRow::GetCreateNewCheckBoxState() const
{
	if (ActorResponse.IsValid())
	{
		return ActorResponse->Response == EAvaSceneDefaultActorResponse::CreateNewActor
			? ECheckBoxState::Checked
			: ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Undetermined;
}

EVisibility SAvaSceneDefaultActorResponseRow::GetCreateNewCheckBoxVisibility() const
{
	if (ActorResponse.IsValid() && ActorResponse->Response != EAvaSceneDefaultActorResponse::SkipActor)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

void SAvaSceneDefaultActorResponseRow::OnCreateNewCheckBoxChanged(ECheckBoxState InState)
{
	if (ActorResponse.IsValid() && InState == ECheckBoxState::Checked)
	{
		ActorResponse->Response = EAvaSceneDefaultActorResponse::CreateNewActor;
	}
}

ECheckBoxState SAvaSceneDefaultActorResponseRow::GetReplaceCheckBoxState() const
{
	if (ActorResponse.IsValid())
	{
		return ActorResponse->Response == EAvaSceneDefaultActorResponse::ReplaceActor
			? ECheckBoxState::Checked
			: ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Undetermined;
}

EVisibility SAvaSceneDefaultActorResponseRow::GetReplaceCheckBoxVisibility() const
{
	if (ActorResponse.IsValid() && !ActorResponse->AvailableActors.IsEmpty()
		&& ActorResponse->Response != EAvaSceneDefaultActorResponse::SkipActor)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

void SAvaSceneDefaultActorResponseRow::OnReplaceCheckBoxChanged(ECheckBoxState InState)
{
	if (ActorResponse.IsValid() && InState == ECheckBoxState::Checked)
	{
		ActorResponse->Response = EAvaSceneDefaultActorResponse::ReplaceActor;
	}
}

ECheckBoxState SAvaSceneDefaultActorResponseRow::GetUpdateCheckBoxState() const
{
	if (ActorResponse.IsValid())
	{
		return ActorResponse->Response == EAvaSceneDefaultActorResponse::UpdateActor
			? ECheckBoxState::Checked
			: ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Undetermined;
}

EVisibility SAvaSceneDefaultActorResponseRow::GetUpdateCheckBoxVisibility() const
{
	if (ActorResponse.IsValid() && !ActorResponse->AvailableActors.IsEmpty()
		&& ActorResponse->Response != EAvaSceneDefaultActorResponse::SkipActor)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

void SAvaSceneDefaultActorResponseRow::OnUpdateCheckBoxChanged(ECheckBoxState InState)
{
	if (ActorResponse.IsValid() && InState == ECheckBoxState::Checked)
	{
		ActorResponse->Response = EAvaSceneDefaultActorResponse::UpdateActor;
	}
}

ECheckBoxState SAvaSceneDefaultActorResponseRow::GetEnabledCheckBoxState() const
{
	if (ActorResponse.IsValid())
	{
		// Enabled != ignored
		return ActorResponse->Response == EAvaSceneDefaultActorResponse::SkipActor
			? ECheckBoxState::Unchecked
			: ECheckBoxState::Checked;
	}

	return ECheckBoxState::Undetermined;
}

void SAvaSceneDefaultActorResponseRow::OnEnabledCheckBoxChanged(ECheckBoxState InState)
{
	if (!ActorResponse.IsValid())
	{
		return;
	}

	// Enabled != ignored
	switch (InState)
	{
		case ECheckBoxState::Unchecked:
			ActorResponse->Response = EAvaSceneDefaultActorResponse::SkipActor;
			break;

		case ECheckBoxState::Checked:
			ActorResponse->Response = EAvaSceneDefaultActorResponse::CreateNewActor;
			break;
	}
}

EVisibility SAvaSceneDefaultActorResponseRow::GetActorSelectVisibility() const
{
	if (ActorResponse.IsValid() && !ActorResponse->AvailableActors.IsEmpty() &&
		(ActorResponse->Response == EAvaSceneDefaultActorResponse::ReplaceActor
			|| ActorResponse->Response == EAvaSceneDefaultActorResponse::UpdateActor))
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

void SAvaSceneDefaultActorResponseRow::OnActorPickerSelected(AActor* InActor)
{
	SelectedActorLabel.Reset();
	SelectedActorColor.Reset();

	if (!ActorResponse.IsValid())
	{
		return;
	}

	ActorResponse->SelectedActor = InActor;

	if (!HasSelectedActor())
	{
		ActorResponse->Response = EAvaSceneDefaultActorResponse::CreateNewActor;
	}
	else if (ActorResponse->Response != EAvaSceneDefaultActorResponse::UpdateActor)
	{
		ActorResponse->Response = EAvaSceneDefaultActorResponse::ReplaceActor;
	}
}

FSlateColor SAvaSceneDefaultActorResponseRow::GetActorSelectRowColor() const
{
	if (!SelectedActorColor.IsSet())
	{
		static const FSlateColor Valid = FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText").ColorAndOpacity;
		static const FSlateColor Invalid = FSlateColor(EStyleColor::AccentRed);

		if (!ActorResponse.IsValid())
		{
			SelectedActorColor = Invalid;
		}
		else if (!IsSelectableActor(ActorResponse->SelectedActor))
		{
			SelectedActorColor = Valid;
		}
		else if (TSharedPtr<SAvaSceneDefaultActorResponses> ActorResponses = ActorResponsesWeak.Pin())
		{
			int32 MatchCount = 0;

			for (const TSharedRef<FAvaSceneDefaultActorResponse>& ActorResponseIter : ActorResponses->GetResponses())
			{
				if (IsSelectableActor(ActorResponseIter->SelectedActor)
					&& ActorResponseIter->Response != EAvaSceneDefaultActorResponse::ReplaceActor
					&& ActorResponseIter->SelectedActor == ActorResponse->SelectedActor)
				{
					++MatchCount;
				}
			}

			SelectedActorColor = (MatchCount < 2) ? Valid : Invalid;
		}
		else
		{
			SelectedActorColor = Invalid;
		}
	}

	return SelectedActorColor.GetValue();
}

FText SAvaSceneDefaultActorResponseRow::GetActorSelectRowLabel() const
{
	if (!SelectedActorLabel.IsSet())
	{
		if (ActorResponse.IsValid())
		{
			if (IsSelectableActor(ActorResponse->SelectedActor))
			{
				SelectedActorLabel = FText::FromString(ActorResponse->SelectedActor->GetActorNameOrLabel());
				return SelectedActorLabel.GetValue();
			}
		}

		SelectedActorLabel = INVTEXT("-");
	}

	return SelectedActorLabel.GetValue();
}

bool SAvaSceneDefaultActorResponseRow::HasSelectedActor() const
{
	if (ActorResponse.IsValid())
	{
		return IsSelectableActor(ActorResponse->SelectedActor);
	}

	return false;
}

bool SAvaSceneDefaultActorResponseRow::IsSelectableActor(TWeakObjectPtr<AActor> InActor)
{
	return !!InActor.Get();
}

#undef LOCTEXT_NAMESPACE
