// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/STableRow.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "Scene/AvaSceneDefaults.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrFwd.h"
#include "UObject/WeakObjectPtrTemplatesFwd.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"

class AActor;
class SAvaSceneDefaultActorResponses;

class SAvaSceneDefaultActorResponseRow : public SMultiColumnTableRow<TSharedRef<FAvaSceneDefaultActorResponse>>
{
	SLATE_BEGIN_ARGS(SAvaSceneDefaultActorResponseRow) {}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView,
		const TSharedRef<SAvaSceneDefaultActorResponses>& InActorResponses, const TSharedRef<FAvaSceneDefaultActorResponse>& InActorResponse);

	//~ Begin SMultiColumnTableRow
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName & InColumnName) override;
	//~ End SMultiColumnTableRow

	static bool IsSelectableActor(TWeakObjectPtr<AActor> InActor);

protected:
	TWeakPtr<SAvaSceneDefaultActorResponses> ActorResponsesWeak;
	TSharedPtr<FAvaSceneDefaultActorResponse> ActorResponse;

	TSharedPtr<SComboBox<TWeakObjectPtr<AActor>>> ActorSelect;
	TSharedPtr<SCheckBox> CreateNewCheckBox;
	TSharedPtr<SCheckBox> ReplaceCheckBox;
	TSharedPtr<SCheckBox> UpdateCheckBox;
	TSharedPtr<SCheckBox> EnabledCheckBox;

	mutable TOptional<FText> SelectedActorLabel;
	mutable TOptional<FSlateColor> SelectedActorColor;

	TSharedRef<SWidget> GenerateRow_Description();
	TSharedRef<SWidget> GenerateRow_Actor();
	TSharedRef<SWidget> GenerateRow_CreateNew();
	TSharedRef<SWidget> GenerateRow_Replace();
	TSharedRef<SWidget> GenerateRow_Update();
	TSharedRef<SWidget> GenerateRow_Enabled();

	FSlateColor GetDescriptionColor() const;

	ECheckBoxState GetCreateNewCheckBoxState() const;
	EVisibility GetCreateNewCheckBoxVisibility() const;
	void OnCreateNewCheckBoxChanged(ECheckBoxState InState);

	ECheckBoxState GetReplaceCheckBoxState() const;
	EVisibility GetReplaceCheckBoxVisibility() const;
	void OnReplaceCheckBoxChanged(ECheckBoxState InState);

	ECheckBoxState GetUpdateCheckBoxState() const;
	EVisibility GetUpdateCheckBoxVisibility() const;
	void OnUpdateCheckBoxChanged(ECheckBoxState InState);

	ECheckBoxState GetEnabledCheckBoxState() const;
	void OnEnabledCheckBoxChanged(ECheckBoxState InState);

	EVisibility GetActorSelectVisibility() const;
	void OnActorPickerSelected(AActor* InActor);

	FSlateColor GetActorSelectRowColor() const;
	FText GetActorSelectRowLabel() const;

	bool HasSelectedActor() const;
};
