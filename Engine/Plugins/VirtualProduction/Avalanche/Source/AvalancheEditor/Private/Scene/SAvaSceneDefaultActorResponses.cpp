// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scene/SAvaSceneDefaultActorResponses.h"
#include "Engine/World.h"
#include "Scene/SAvaSceneDefaultActorResponseRow.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "AvaSceneDefaultActorResponses"

const FName SAvaSceneDefaultActorResponses::DescriptionRowName = "Description";
const FName SAvaSceneDefaultActorResponses::TargetActorRowName = "TargetActor";
const FName SAvaSceneDefaultActorResponses::CreateNewRowName = "CreateNew";
const FName SAvaSceneDefaultActorResponses::ReplaceRowName = "Replace";
const FName SAvaSceneDefaultActorResponses::UpdateRowName = "Update";
const FName SAvaSceneDefaultActorResponses::EnabledRowName = "Enabled";

namespace UE::AvaEditor::Private
{
	struct FAvaSceneDefaultActorResponseColumnData
	{
		FName Name;
		FText Description;
		FText Tooltip;
		TAttribute<float> FillWidth;
		TOptional<float> FixedWidth;
	};

	static const TArray<FAvaSceneDefaultActorResponseColumnData> ColumnData = {
		{
			SAvaSceneDefaultActorResponses::EnabledRowName,
			FText(),
			LOCTEXT("Enabled.Tooltip", "The default scene actor will be added to the scene."),
			TAttribute<float>(),
			TOptional<float>(32.f)
		},
		{
			SAvaSceneDefaultActorResponses::DescriptionRowName,
			LOCTEXT("DefaultActor", "Default Scene Actor"),
			LOCTEXT("DefaultActor.Tooltip", "Name of the default actor to be added to the scene."),
			TAttribute<float>(0.5f),
			TOptional<float>()
		},
		{
			SAvaSceneDefaultActorResponses::CreateNewRowName,
			LOCTEXT("CreateNew", "Add"),
			LOCTEXT("CreateNew.Tooltip", "Create a new actor in the scene."),
			TAttribute<float>(),
			TOptional<float>(55.f)
		},
		{
			SAvaSceneDefaultActorResponses::ReplaceRowName,
			LOCTEXT("Replace", "Replace"),
			LOCTEXT("Replace.Tooltip", "Replace the selected actor in the scene."),
			TAttribute<float>(),
			TOptional<float>(55.f)
		},
		{
			SAvaSceneDefaultActorResponses::UpdateRowName,
			LOCTEXT("Update", "Update"),
			LOCTEXT("Update.Tooltip", "Updated the selected actor in the scene with new values."),
			TAttribute<float>(),
			TOptional<float>(55.f)
		},
		{
			SAvaSceneDefaultActorResponses::TargetActorRowName,
			LOCTEXT("SelectedActor", "Target Actor"),
			LOCTEXT("SelectedActor.Tooltip", "Actor to be replaced or updated in the scene. If no actor is selected, a new actor will be created."),
			TAttribute<float>(0.5f),
			TOptional<float>()
		}
	};
}

void SAvaSceneDefaultActorResponses::Construct(const FArguments& InArgs, UWorld* InWorld, const TArray<TSharedRef<FAvaSceneDefaultActorResponse>>& InResponses)
{
	using namespace UE::AvaEditor::Private;

	World = InWorld;
	Responses = InResponses;
	bAccepted = false;

	TSharedRef<SHeaderRow> HeaderRow = SNew(SHeaderRow);

	for (const FAvaSceneDefaultActorResponseColumnData& PerColumnData : ColumnData)
	{
		if (PerColumnData.Name != EnabledRowName)
		{
			HeaderRow->AddColumn(
				SHeaderRow::Column(PerColumnData.Name)
				.DefaultLabel(PerColumnData.Description)
				.DefaultTooltip(PerColumnData.Tooltip)
				.FillWidth(PerColumnData.FillWidth)
				.FixedWidth(PerColumnData.FixedWidth)
				.HAlignHeader(HAlign_Center)
			);
		}
		else
		{
			HeaderRow->AddColumn(
				SHeaderRow::Column(PerColumnData.Name)
				.FillWidth(PerColumnData.FillWidth)
				.FixedWidth(PerColumnData.FixedWidth)
				.HAlignHeader(HAlign_Center)
				.HeaderContent()
				[
					SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.ToolTipText(PerColumnData.Tooltip)
					[
						SNew(SCheckBox)
						.IsChecked(this, &SAvaSceneDefaultActorResponses::GetEnableAllCheckState)
						.OnCheckStateChanged(this, &SAvaSceneDefaultActorResponses::OnEnableAllChecked)
					]
				]
			);
		}
	}

	ListView = SNew(SListView<TSharedRef<FAvaSceneDefaultActorResponse>>)
		.HeaderRow(HeaderRow)
		.ListItemsSource(&Responses)
		.Orientation(Orient_Vertical)
		.SelectionMode(ESelectionMode::None)
		.OnGenerateRow(this, &SAvaSceneDefaultActorResponses::GenerateRow);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			ListView.ToSharedRef()
		]
		+ SVerticalBox::Slot()
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.f, 10.f)
			[
				SNew(SButton)
				.Text(LOCTEXT("Spawn", "Spawn"))
				.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
				.OnClicked(this, &SAvaSceneDefaultActorResponses::OnOkayClick)
				.IsEnabled(this, &SAvaSceneDefaultActorResponses::IsOkayEnabled)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.f, 10.f)
			[
				SNew(SButton)
				.Text(LOCTEXT("Cancel", "Cancel"))
				.ButtonStyle(FAppStyle::Get(), "Button")
				.OnClicked(this, &SAvaSceneDefaultActorResponses::OnCancelClick)
			]
		]
	];
}

FReply SAvaSceneDefaultActorResponses::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		CloseWindow();
		return FReply::Handled();
	}

	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

TSharedRef<ITableRow> SAvaSceneDefaultActorResponses::GenerateRow(TSharedRef<FAvaSceneDefaultActorResponse> InListItem, 
	const TSharedRef<STableViewBase>& InListView)
{
	return SNew(SAvaSceneDefaultActorResponseRow, InListView, SharedThis(this), InListItem);
}

bool SAvaSceneDefaultActorResponses::IsOkayEnabled() const
{
	TSet<AActor*> OverriddenActors;

	for (const TSharedRef<FAvaSceneDefaultActorResponse>& ActorResponse : Responses)
	{
		if (ActorResponse->Response != EAvaSceneDefaultActorResponse::ReplaceActor
			&& ActorResponse->Response != EAvaSceneDefaultActorResponse::UpdateActor)
		{
			continue;
		}

		AActor* Actor = ActorResponse->SelectedActor.Get();

		if (!Actor)
		{
			continue;
		}

		if (OverriddenActors.Contains(Actor))
		{
			return false;
		}

		OverriddenActors.Add(Actor);
	}

	return true;
}

FReply SAvaSceneDefaultActorResponses::OnOkayClick()
{
	bAccepted = true;
	CloseWindow();

	return FReply::Handled();
}

FReply SAvaSceneDefaultActorResponses::OnCancelClick()
{
	CloseWindow();

	return FReply::Handled();
}

void SAvaSceneDefaultActorResponses::CloseWindow()
{
	if (TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(SharedThis(this)))
	{
		FSlateApplication::Get().RequestDestroyWindow(Window.ToSharedRef());
	}
}

ECheckBoxState SAvaSceneDefaultActorResponses::GetEnableAllCheckState() const
{
	ECheckBoxState State = ECheckBoxState::Undetermined;

	for (const TSharedRef<FAvaSceneDefaultActorResponse>& ActorResponse : Responses)
	{
		const ECheckBoxState ActorResponseCheckState = ActorResponse->Response == EAvaSceneDefaultActorResponse::SkipActor
			? ECheckBoxState::Unchecked
			: ECheckBoxState::Checked;

		if (State == ECheckBoxState::Undetermined)
		{
			State = ActorResponseCheckState;
		}
		else if (State != ActorResponseCheckState)
		{
			return ECheckBoxState::Undetermined;
		}
	}

	return State;
}

void SAvaSceneDefaultActorResponses::OnEnableAllChecked(ECheckBoxState InState)
{
	for (const TSharedRef<FAvaSceneDefaultActorResponse>& ActorResponse : Responses)
	{
		switch (InState)
		{
			case ECheckBoxState::Unchecked:
				ActorResponse->Response = EAvaSceneDefaultActorResponse::SkipActor;
				break;

			case ECheckBoxState::Checked:
				ActorResponse->Response = EAvaSceneDefaultActorResponse::CreateNewActor;
				break;
		}
	}
}

#undef LOCTEXT_NAMESPACE
