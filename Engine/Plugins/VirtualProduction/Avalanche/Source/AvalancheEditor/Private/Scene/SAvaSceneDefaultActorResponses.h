// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Containers/Array.h"
#include "Scene/AvaSceneDefaults.h"
#include "Templates/SharedPointer.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Views/SListView.h"

class ITableRow;
class UWorld;

class SAvaSceneDefaultActorResponses : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SAvaSceneDefaultActorResponses) {}
	SLATE_END_ARGS()

public:
	static const FName DescriptionRowName;
	static const FName CreateNewRowName;
	static const FName ReplaceRowName;
	static const FName UpdateRowName;
	static const FName EnabledRowName;
	static const FName TargetActorRowName;

	void Construct(const FArguments& InArgs, UWorld* InWorld, const TArray<TSharedRef<FAvaSceneDefaultActorResponse>>& InResponses);

	UWorld* GetWorld() const { return World.Get(); }

	const TArray<TSharedRef<FAvaSceneDefaultActorResponse>>& GetResponses() const { return Responses; }

	bool WasAccepted() const { return bAccepted; }

	//~ Start SWidget
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget

protected:
	TWeakObjectPtr<UWorld> World;
	TArray<TSharedRef<FAvaSceneDefaultActorResponse>> Responses;
	bool bAccepted;

	TSharedPtr<SListView<TSharedRef<FAvaSceneDefaultActorResponse>>> ListView;

	TSharedRef<ITableRow> GenerateRow(TSharedRef<FAvaSceneDefaultActorResponse> InListItem, const TSharedRef<STableViewBase>& InListView);

	bool IsOkayEnabled() const;

	FReply OnOkayClick();

	FReply OnCancelClick();

	void CloseWindow();

	ECheckBoxState GetEnableAllCheckState() const;

	void OnEnableAllChecked(ECheckBoxState InState);
};
