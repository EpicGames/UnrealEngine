// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/AvaPivotSetOperation.h"
#include "Bounds/AvaBoundsProviderSubsystem.h"
#include "Containers/Map.h"
#include "Engine/World.h"
#include "Framework/AvaNullActor.h"
#include "Misc/MessageDialog.h"
#include "Selection/AvaEditorSelection.h"
#include "Selection/AvaSelectionProviderSubsystem.h"

#define LOCTEXT_NAMESPACE "AvaPivotSetOperation"

FAvaPivotSetOperation::FAvaPivotSetOperation(UWorld* InWorld, EAvaPivotBoundsType InBoundsType, PivotSetCallbackType InPivotSetPredicate)
{
	World = InWorld;
	BoundsType = InBoundsType;
	PivotSetCallback = InPivotSetPredicate;
}

void FAvaPivotSetOperation::SetPivot()
{
	SelectionProvider = UAvaSelectionProviderSubsystem::Get(World, /* bInGenerateErrors */ true);

	if (!SelectionProvider)
	{
		return;
	}

	BoundsProvider = UAvaBoundsProviderSubsystem::Get(World, /* bInGenerateErrors */ true);

	if (!BoundsProvider)
	{
		return;
	}

	SelectedActors = SelectionProvider->GetSelectedActors();

	if (SelectedActors.IsEmpty())
	{
		return;
	}

	// Better lookup
	SelectedActorSet.Reserve(SelectedActors.Num());

	Algo::Transform(
		SelectedActors,
		SelectedActorSet,
		[](const TWeakObjectPtr<AActor>& InElement) { return InElement.Get(); }
	);

	// Filter out selected actors which are children of other selected actors.
	ValidActors.Reserve(SelectedActorSet.Num());
	ValidActorsSet.Reserve(SelectedActorSet.Num());

	GenerateValidActors();

	if (ValidActors.Num() == 0)
	{
		return;
	}

	if (BoundsType == EAvaPivotBoundsType::Selection)
	{
		AxisAlignedBounds = BoundsProvider->GetSelectionBounds(true);

		if (!AxisAlignedBounds.IsValid)
		{
			return;
		}

		// Even if the first selected actor isn't valid for pivot changing, it is still the basis for the selection bounds transform.
		AxisAlignedTransform = SelectedActors[0]->GetActorTransform();

		// Let's see if we should place all the selected actors under the same new null actor
		bool bHasCommonParent = true;
		AActor* CommonParent = ValidActors[0]->GetAttachParentActor();

		for (AActor* ValidActor : ValidActors)
		{
			if (ValidActor->GetAttachParentActor() != CommonParent)
			{
				bHasCommonParent = false;
				break;
			}
		}

		// If we have a common parent but it's not a null actor, create one.
		if (bHasCommonParent)
		{
			SetPivotCommonParent(CommonParent);
			return;
		}
	}

	SetPivotIndividual();
}

void FAvaPivotSetOperation::GenerateValidActors()
{
	for (const TWeakObjectPtr<AActor>& SelectedActorWeak : SelectedActors)
	{
		if (AActor* SelectedActor = SelectedActorWeak.Get())
		{
			TSet<AActor*> ActorChain;
			bool bFoundSelectedParent = false;

			for (AActor* ParentActor = SelectedActor; IsValid(ParentActor); ParentActor = ParentActor->GetAttachParentActor())
			{
				if ((ParentActor != SelectedActor && SelectedActorSet.Contains(ParentActor)) || InvalidActors.Contains(ParentActor))
				{
					bFoundSelectedParent = true;
					break;
				}

				ActorChain.Add(ParentActor);
			}

			if (bFoundSelectedParent)
			{
				InvalidActors.Append(ActorChain);
			}
			else
			{
				ValidActors.Add(SelectedActor);
				ValidActorsSet.Add(SelectedActor);
			}
		}
	}
}

void FAvaPivotSetOperation::SetPivotCommonParent(AActor* InCommonParent)
{
	TMap<AActor*, FTransform> OldChildrenTransforms;
	OldChildrenTransforms.Reserve(ValidActors.Num());

	for (AActor* ValidActor : ValidActors)
	{
		OldChildrenTransforms.Emplace(ValidActor, ValidActor->GetActorTransform());
	}

	// If our common parent is either invalid or not a null actor, create a null actor to act as the pivot.
	if (!IsValid(InCommonParent) || !InCommonParent->IsA<AAvaNullActor>())
	{
		InCommonParent = SpawnPivot(ValidActors[0]->GetLevel());

		if (!InCommonParent)
		{
			return;
		}

		FVector PivotLocation = AxisAlignedBounds.GetCenter();

		// Set pivot coordinates
		PivotSetCallback(AxisAlignedBounds, PivotLocation);

		// Transform back into world space
		PivotLocation = AxisAlignedTransform.TransformPositionNoScale(PivotLocation);

		InCommonParent->SetActorLocation(PivotLocation);
		InCommonParent->SetActorRotation(SelectedActors[0]->GetActorRotation());
	}
	// We're already a null actor parent, so also save the locations of the children we aren't changing the pivot for.
	else
	{
		TConstArrayView<TWeakObjectPtr<AActor>> PivotChildren = SelectionProvider->GetAttachedActors(InCommonParent, false);

		for (const TWeakObjectPtr<AActor>& PivotChildWeak : PivotChildren)
		{
			if (AActor* PivotChild = PivotChildWeak.Get())
			{
				// No need to add them twice
				if (!ValidActorsSet.Contains(PivotChild))
				{
					OldChildrenTransforms.Emplace(PivotChild, PivotChild->GetActorTransform());
				}
			}
		}

		FVector PivotLocation = AxisAlignedTransform.InverseTransformPositionNoScale(InCommonParent->GetActorLocation());

		// Set pivot coordinates
		PivotSetCallback(AxisAlignedBounds, PivotLocation);

		// Transform back into world space
		PivotLocation = AxisAlignedTransform.TransformPositionNoScale(PivotLocation);

		InCommonParent->SetActorLocation(PivotLocation);
	}

	// Attach to pivot
	for (AActor* ValidActor : ValidActors)
	{
		AActor* AttachParent = ValidActor->GetAttachParentActor();

		if (AttachParent != InCommonParent)
		{
			ValidActor->AttachToActor(InCommonParent, FAttachmentTransformRules::SnapToTargetIncludingScale);
		}
	}

	// Reset transforms
	for (const TPair<AActor*, FTransform>& Pair : OldChildrenTransforms)
	{
		Pair.Key->SetActorTransform(Pair.Value);
	}
}

void FAvaPivotSetOperation::SetPivotIndividual()
{
	if (ValidActors.Num() > 1)
	{
		const FText MessageBoxTitle = LOCTEXT("MoveMultiplePivotsTitle", "Move Multiple Pivots");

		const EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::YesNo
			, LOCTEXT("MoveMultiplePivotsMessage", "You are about move/create multiple pivots. Are you sure?")
			, MessageBoxTitle);

		switch (Response)
		{
			case EAppReturnType::Yes:
				break;

				// Cancel
			default:
				return;
		}
	}

	for (AActor* ValidActor : ValidActors)
	{
		switch (BoundsType)
		{
			case EAvaPivotBoundsType::Actor:
				AxisAlignedBounds = BoundsProvider->GetActorLocalBounds(ValidActor);

				if (!AxisAlignedBounds.IsValid)
				{
					continue;
				}

				AxisAlignedTransform = ValidActor->GetActorTransform();
				break;

			case EAvaPivotBoundsType::ActorAndChildren:
				AxisAlignedBounds = BoundsProvider->GetActorAndChildrenLocalBounds(ValidActor);

				if (!AxisAlignedBounds.IsValid)
				{
					continue;
				}

				AxisAlignedTransform = ValidActor->GetActorTransform();
				break;

			case EAvaPivotBoundsType::Selection:
				// Already done.
				break;

			default:
				checkNoEntry();
				return;
		}

		AActor* Parent = ValidActor->GetAttachParentActor();
		AAvaNullActor* ParentPivot = Cast<AAvaNullActor>(Parent);
		bool bIsNewPivot = false;
		const FTransform CurrentActorTransform = ValidActor->GetActorTransform();
		TMap<AActor*, FTransform> OldChildrenTransforms;

		// Store other child transforms
		if (IsValid(ParentPivot))
		{
			TConstArrayView<TWeakObjectPtr<AActor>> PivotChildren = SelectionProvider->GetAttachedActors(ParentPivot, false);

			for (const TWeakObjectPtr<AActor>& PivotChildWeak : PivotChildren)
			{
				if (AActor* PivotChild = PivotChildWeak.Get())
				{
					if (PivotChild != ValidActor)
					{
						OldChildrenTransforms.Emplace(PivotChild, PivotChild->GetActorTransform());
					}
				}
			}
		}
		// Create new pivot (selected actor child transform already stored)
		else
		{
			bIsNewPivot = true;
			ParentPivot = SpawnPivot(ValidActor->GetLevel());

			if (!ParentPivot)
			{
				continue;
			}

			// Attach the actor to the new parent null
			ValidActor->AttachToActor(ParentPivot, FAttachmentTransformRules::SnapToTargetIncludingScale);

			if (Parent)
			{
				ParentPivot->AttachToComponent(
					ValidActor->GetRootComponent()->GetAttachParent(),
					FAttachmentTransformRules::SnapToTargetIncludingScale
				);
			}
		}

		// Transform pivot location into aligned space
		FVector PivotLocation = AxisAlignedTransform.InverseTransformPositionNoScale(ParentPivot->GetActorLocation());

		// Set pivot coordinates
		PivotSetCallback(AxisAlignedBounds, PivotLocation);

		// Transform back into world space
		PivotLocation = AxisAlignedTransform.TransformPositionNoScale(PivotLocation);

		// Set the parent null to the position
		ParentPivot->SetActorLocation(PivotLocation);

		if (bIsNewPivot)
		{
			ParentPivot->SetActorRotation(CurrentActorTransform.GetRotation());
		}

		// Set the transform of the child actor back to what it was
		ValidActor->SetActorTransform(CurrentActorTransform);

		// Reset other child transforms
		for (const TPair<AActor*, FTransform>& Pair : OldChildrenTransforms)
		{
			Pair.Key->SetActorTransform(Pair.Value);
		}
	}
}

AAvaNullActor* FAvaPivotSetOperation::SpawnPivot(ULevel* InLevel)
{
	FActorSpawnParameters SpawnParams;
	SpawnParams.OverrideLevel = InLevel;
	SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnParams.bNoFail = true;

	AAvaNullActor* NewPivot = World->SpawnActor<AAvaNullActor>(SpawnParams);

	if (!ensureMsgf(NewPivot, TEXT("Unable to spawn new null actor as pivot.")))
	{
		return nullptr;;
	}

	return NewPivot;
}

#undef LOCTEXT_NAMESPACE
