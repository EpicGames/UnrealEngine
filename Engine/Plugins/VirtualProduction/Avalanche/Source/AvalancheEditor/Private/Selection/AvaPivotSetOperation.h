// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "GameFramework/Actor.h"
#include "Math/Box.h"
#include "Math/MathFwd.h"
#include "Math/Transform.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AAvaNullActor;
class UWorld;
class UAvaSelectionProviderSubsystem;
class UAvaBoundsProviderSubsystem;
class ULevel;

enum class EAvaPivotBoundsType : uint8
{
	Actor,
	ActorAndChildren,
	Selection
};

class FAvaPivotSetOperation
{
public:
	using PivotSetCallbackType = TFunction<void(const FBox, FVector&)>;

	FAvaPivotSetOperation(UWorld* InWorld, EAvaPivotBoundsType InBoundsType, PivotSetCallbackType InPivotSetPredicate);

	void SetPivot();

protected:
	UWorld* World;

	EAvaPivotBoundsType BoundsType;

	PivotSetCallbackType PivotSetCallback;

	UAvaSelectionProviderSubsystem* SelectionProvider;

	UAvaBoundsProviderSubsystem* BoundsProvider;

	TConstArrayView<TWeakObjectPtr<AActor>> SelectedActors;

	TSet<AActor*> SelectedActorSet;

	TArray<AActor*> ValidActors;

	TSet<AActor*> ValidActorsSet;

	TSet<AActor*> InvalidActors;

	FBox AxisAlignedBounds;

	FTransform AxisAlignedTransform;

	void GenerateValidActors();

	void SetPivotCommonParent(AActor* InCommonParent);

	void SetPivotIndividual();

	AAvaNullActor* SpawnPivot(ULevel* InLevel);
};
