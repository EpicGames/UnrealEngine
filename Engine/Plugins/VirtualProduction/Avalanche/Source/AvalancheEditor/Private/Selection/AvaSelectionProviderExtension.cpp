// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/AvaSelectionProviderExtension.h"
#include "AvaEditorCommands.h"
#include "Bounds/AvaBoundsProviderSubsystem.h"
#include "Engine/Engine.h"
#include "Framework/Commands/Commands.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Selection/AvaEditorSelection.h"
#include "Selection/AvaPivotSetOperation.h"
#include "Selection/AvaSelectionProviderSubsystem.h"
#include "Viewport/AvaViewportExtension.h"
#include "ViewportClient/IAvaViewportClient.h"

FAvaSelectionProviderExtension::~FAvaSelectionProviderExtension()
{
	if (GEngine)
	{
		GEngine->OnLevelActorAttached().RemoveAll(this);
	}
}

void FAvaSelectionProviderExtension::Activate()
{
	FAvaEditorExtension::Activate();

	if (GEngine)
	{
		GEngine->OnLevelActorAttached().AddSP(this, &FAvaSelectionProviderExtension::OnLevelAttachmentChange);
	}
}

void FAvaSelectionProviderExtension::Deactivate()
{
	FAvaEditorExtension::Deactivate();

	if (GEngine)
	{
		GEngine->OnLevelActorAttached().RemoveAll(this);
	}
}

void FAvaSelectionProviderExtension::NotifyOnSelectionChanged(const FAvaEditorSelection& InSelection)
{
	UAvaSelectionProviderSubsystem* SelectionProvider = UAvaSelectionProviderSubsystem::Get(this, /* bInGenerateErrors */ true);

	if (!SelectionProvider)
	{
		return;
	}

	SelectionProvider->UpdateSelection(InSelection);

	TSharedPtr<IAvaEditor> Editor = GetEditor();

	if (!Editor.IsValid())
	{
		return;
	}

	TSharedPtr<FAvaViewportExtension> ViewportExtension = Editor->FindExtension<FAvaViewportExtension>();

	if (!ViewportExtension.IsValid())
	{
		return;
	}

	for (const TSharedPtr<IAvaViewportClient>& AvaViewportClient : ViewportExtension->GetViewportClients())
	{
		if (AvaViewportClient.IsValid())
		{
			AvaViewportClient->OnActorSelectionChanged();
		}
	}
}

void FAvaSelectionProviderExtension::OnLevelAttachmentChange(AActor* InAttachedActor, const AActor* InAttachedToActor)
{
	UAvaSelectionProviderSubsystem* SelectionProvider = UAvaSelectionProviderSubsystem::Get(this, /* bInGenerateErrors */ true);

	if (!SelectionProvider)
	{
		return;
	}

	SelectionProvider->ClearAttachedActorCache();
}

void FAvaSelectionProviderExtension::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	FAvaEditorExtension::BindCommands(InCommandList);

	const FAvaEditorCommands& AvaEditorCommands = FAvaEditorCommands::Get();
	FUICommandList& CommandListRef = *InCommandList;

	// Pivot positions
	CommandListRef.MapAction(
		AvaEditorCommands.PivotTopLeftActor,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(-1, 1), EAvaPivotBoundsType::Actor)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotTopMiddleActor,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(0, 1), EAvaPivotBoundsType::Actor)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotTopRightActor,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(1, 1), EAvaPivotBoundsType::Actor)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotMiddleLeftActor,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(-1, 0), EAvaPivotBoundsType::Actor)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotCenterActor,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(0, 0), EAvaPivotBoundsType::Actor)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotMiddleRightActor,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(1, 0), EAvaPivotBoundsType::Actor)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotBottomLeftActor,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(-1, -1), EAvaPivotBoundsType::Actor)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotBottomMiddleActor,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(0, -1), EAvaPivotBoundsType::Actor)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotBottomRightActor,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(1, -1), EAvaPivotBoundsType::Actor)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotDepthFrontActor,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetDepthPivot, 1.0, EAvaPivotBoundsType::Actor)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotDepthMiddleActor,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetDepthPivot, 0.0, EAvaPivotBoundsType::Actor)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotDepthBackActor,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetDepthPivot, -1.0, EAvaPivotBoundsType::Actor)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotTopLeftActorAndChildren,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(-1, 1), EAvaPivotBoundsType::ActorAndChildren)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotTopMiddleActorAndChildren,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(0, 1), EAvaPivotBoundsType::ActorAndChildren)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotTopRightActorAndChildren,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(1, 1), EAvaPivotBoundsType::ActorAndChildren)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotMiddleLeftActorAndChildren,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(-1, 0), EAvaPivotBoundsType::ActorAndChildren)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotCenterActorAndChildren,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(0, 0), EAvaPivotBoundsType::ActorAndChildren)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotMiddleRightActorAndChildren,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(1, 0), EAvaPivotBoundsType::ActorAndChildren)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotBottomLeftActorAndChildren,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(-1, -1), EAvaPivotBoundsType::ActorAndChildren)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotBottomMiddleActorAndChildren,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(0, -1), EAvaPivotBoundsType::ActorAndChildren)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotBottomRightActorAndChildren,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(1, -1), EAvaPivotBoundsType::ActorAndChildren)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotDepthFrontActorAndChildren,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetDepthPivot, 1.0, EAvaPivotBoundsType::ActorAndChildren)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotDepthMiddleActorAndChildren,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetDepthPivot, 0.0, EAvaPivotBoundsType::ActorAndChildren)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotDepthBackActorAndChildren,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetDepthPivot, -1.0, EAvaPivotBoundsType::ActorAndChildren)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotTopLeftSelection,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(-1, 1), EAvaPivotBoundsType::Selection)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotTopMiddleSelection,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(0, 1), EAvaPivotBoundsType::Selection)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotTopRightSelection,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(1, 1), EAvaPivotBoundsType::Selection)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotMiddleLeftSelection,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(-1, 0), EAvaPivotBoundsType::Selection)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotCenterSelection,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(0, 0), EAvaPivotBoundsType::Selection)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotMiddleRightSelection,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(1, 0), EAvaPivotBoundsType::Selection)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotBottomLeftSelection,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(-1, -1), EAvaPivotBoundsType::Selection)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotBottomMiddleSelection,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(0, -1), EAvaPivotBoundsType::Selection)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotBottomRightSelection,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetPlanePivot, FVector2D(1, -1), EAvaPivotBoundsType::Selection)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotDepthFrontSelection,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetDepthPivot, 1.0, EAvaPivotBoundsType::Selection)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotDepthMiddleSelection,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetDepthPivot, 0.0, EAvaPivotBoundsType::Selection)
	);

	CommandListRef.MapAction(
		AvaEditorCommands.PivotDepthBackSelection,
		FExecuteAction::CreateSP(this, &FAvaSelectionProviderExtension::SetDepthPivot, -1.0, EAvaPivotBoundsType::Selection)
	);
}

void FAvaSelectionProviderExtension::SetPlanePivot(FVector2D InPivotUV, EAvaPivotBoundsType InBoundsType)
{
	FAvaPivotSetOperation PivotOperation(
		GetWorld(),
		InBoundsType,
		[InPivotUV](const FBox AxisAlignedBounds, FVector& InOutPivotLocation)
		{
			const FVector AxisAlignedBoundsCenter = AxisAlignedBounds.GetCenter();
			const FVector AxisAlignedBoundsSize = AxisAlignedBounds.GetSize();

			InOutPivotLocation.Y = AxisAlignedBoundsCenter.Y + (InPivotUV.X * 0.5) * AxisAlignedBoundsSize.Y;
			InOutPivotLocation.Z = AxisAlignedBoundsCenter.Z + (InPivotUV.Y * 0.5) * AxisAlignedBoundsSize.Z;
		}
	);

	PivotOperation.SetPivot();
}

void FAvaSelectionProviderExtension::SetDepthPivot(double InPivotUV, EAvaPivotBoundsType InBoundsType)
{
	FAvaPivotSetOperation PivotOperation(
		GetWorld(),
		InBoundsType,
		[InPivotUV](const FBox AxisAlignedBounds, FVector& InOutPivotLocation)
		{
			const FVector AxisAlignedBoundsCenter = AxisAlignedBounds.GetCenter();
			const FVector AxisAlignedBoundsSize = AxisAlignedBounds.GetSize();

			InOutPivotLocation.X = AxisAlignedBoundsCenter.X + (InPivotUV * 0.5) * AxisAlignedBoundsSize.X;
		}
	);

	PivotOperation.SetPivot();
}
