// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaEditorExtension.h"
#include "Math/MathFwd.h"

class FAvaEditorSelection;
enum class EAvaPivotBoundsType : uint8;

class FAvaSelectionProviderExtension : public FAvaEditorExtension
{
public:
	UE_AVA_INHERITS(FAvaSelectionProviderExtension, FAvaEditorExtension);

	virtual ~FAvaSelectionProviderExtension() override;

	//~ Begin IAvaEditorExtension
	virtual void Activate() override;
	virtual void Deactivate() override;
	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandList) override;
	virtual void NotifyOnSelectionChanged(const FAvaEditorSelection& InSelection) override;
	//~ End IAvaEditorExtension

	/** InPivotUV should be a UV-style value between -1 and 1 based on the bounds of the selection. */
	void SetPlanePivot(FVector2D InPivotUV, EAvaPivotBoundsType InBoundsType);

	/** InPivotUV should be a UV-style value between -1 and 1 based on the bounds of the selection. */
	void SetDepthPivot(double InPivotUV, EAvaPivotBoundsType InBoundsType);

protected:
	void OnLevelAttachmentChange(AActor* InAttachedActor, const AActor* InAttachedToActor);
};
