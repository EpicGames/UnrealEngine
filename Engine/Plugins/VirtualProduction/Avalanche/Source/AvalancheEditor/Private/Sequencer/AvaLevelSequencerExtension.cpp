// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaLevelSequencerExtension.h"
#include "AvaSceneSubsystem.h"
#include "AvaSequence.h"
#include "AvaSequencerTabSpawner.h"
#include "AvaSequencerUtils.h"
#include "AvaWorldSubsystemUtils.h"
#include "IAvaSceneInterface.h"
#include "IAvaSequenceProvider.h"
#include "IAvaSequencer.h"
#include "ILevelSequenceEditorToolkit.h"
#include "ILevelSequenceModule.h"
#include "ISequencer.h"
#include "LevelEditor.h"
#include "LevelEditor/AvaLevelEditorUtils.h"
#include "LevelEditorMenuContext.h"
#include "LevelSequence.h"
#include "LevelSequenceEditorBlueprintLibrary.h"
#include "Styling/SlateIconFinder.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "AvaLevelSequencerExtension"

namespace UE::AvaEditor::Private
{
	static constexpr const TCHAR* CinematicToolbarName   = TEXT("LevelEditor.LevelEditorToolBar.Cinematics");
	static constexpr const TCHAR* CinematicExtensionName = TEXT("LevelEditorExistingCinematic");
	static constexpr const TCHAR* AvaSequenceSectionName = TEXT("AvaSequenceSection");

	ILevelSequenceEditorToolkit* OpenSequenceEditor(ULevelSequence* InSequence)
	{
		check(InSequence);

		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		check(AssetEditorSubsystem);

		constexpr bool bFocusIfOpen = false;
		IAssetEditorInstance* AssetEditor = AssetEditorSubsystem->FindEditorForAsset(InSequence, bFocusIfOpen);

		// Return early if it's already opened
		if (AssetEditor)
		{
			return static_cast<ILevelSequenceEditorToolkit*>(AssetEditor);
		}

		AssetEditorSubsystem->OpenEditorForAsset(InSequence);

		AssetEditor = AssetEditorSubsystem->FindEditorForAsset(InSequence, bFocusIfOpen);
		if (ensureAlwaysMsgf(AssetEditor, TEXT("Unable to find Editor for Sequence: %s"), *InSequence->GetName()))
		{
			return static_cast<ILevelSequenceEditorToolkit*>(AssetEditor);
		}
		return nullptr;
	}
}

FAvaLevelSequencerExtension::FAvaLevelSequencerExtension()
	: FAvaSequencerExtension(LevelEditorTabIds::Sequencer, /*bSupportsDrawerWidget*/false)
	, SequencerController(FAvaSequencerUtils::CreateSequencerController())
{
}

FAvaLevelSequencerExtension::~FAvaLevelSequencerExtension()
{
	RemoveCinematicsToolbarExtension();
}

FAvaSequencerArgs FAvaLevelSequencerExtension::MakeSequencerArgs() const
{
	FAvaSequencerArgs Args = FAvaSequencerExtension::MakeSequencerArgs();

	Args.SequencerController = SequencerController;

	// Disable Custom Clean Playback Mode as FSequencer Clean playback mode deals with the Level Editor Viewport Clients already
	Args.bUseCustomCleanPlaybackMode = false;

	// Don't allow FAvaSequencer from processing Selections. For Level Editor Sequences, this is already handled by the Sequencer itself
	Args.bCanProcessSequencerSelections = false;

	return Args;
}

void FAvaLevelSequencerExtension::Construct(const TSharedRef<IAvaEditor>& InEditor)
{
	FAvaSequencerExtension::Construct(InEditor);
	AddCinematicsToolbarExtension();
}

void FAvaLevelSequencerExtension::ExtendLevelEditorLayout(FLayoutExtender& InExtender) const
{
	// do nothing: re-use the Sequencer Tab
}

void FAvaLevelSequencerExtension::RegisterTabSpawners(const TSharedRef<IAvaEditor>& InEditor) const
{
	// do nothing: re-use the Sequencer Tab
}

void FAvaLevelSequencerExtension::Activate()
{
	if (!OnAssetEditorOpenedHandle.IsValid())
	{
		UAssetEditorSubsystem* const AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		check(AssetEditorSubsystem);

		OnAssetEditorOpenedHandle = AssetEditorSubsystem->OnAssetEditorOpened().AddSP(this, &FAvaLevelSequencerExtension::OnAssetEditorOpened);
	}

	FAvaSequencerExtension::Activate();
}

void FAvaLevelSequencerExtension::Deactivate()
{
	// This may be unavailable during engine shutdown
	UAssetEditorSubsystem* const AssetEditorSubsystem = GEditor ? GEditor->GetEditorSubsystem<UAssetEditorSubsystem>() : nullptr;

	if (AssetEditorSubsystem)
	{
		RestoreTabContent();
	}

	FAvaSequencerExtension::Deactivate();

	if (AssetEditorSubsystem)
	{
		if (OnAssetEditorOpenedHandle.IsValid())
		{
			AssetEditorSubsystem->OnAssetEditorOpened().Remove(OnAssetEditorOpenedHandle);
			OnAssetEditorOpenedHandle.Reset();
		}

		// If an Ava Sequence is still opened, close the Sequencer Tab
		// only keep it alive if the viewed sequence is a level sequence
		if (UAvaSequence* ViewedSequence = Cast<UAvaSequence>(GetViewedSequence()))
		{
			AssetEditorSubsystem->CloseAllEditorsForAsset(ViewedSequence);
		}
	}
}

void FAvaLevelSequencerExtension::PostInvokeTabs()
{
	FAvaSequencerExtension::PostInvokeTabs();
	OpenSequencerWithViewedSequence();
	ApplyTabContent();
}

TSharedPtr<ISequencer> FAvaLevelSequencerExtension::GetExternalSequencer() const
{
	if (ILevelSequenceEditorToolkit* Toolkit = OpenSequencerWithViewedSequence())
	{
		return Toolkit->GetSequencer();
	}
	return nullptr;
}

void FAvaLevelSequencerExtension::OnViewedSequenceChanged(UAvaSequence* InOldSequence, UAvaSequence* InNewSequence)
{
	IAvaSequenceProvider* SequenceProvider = GetSequenceProvider();
	if (!SequenceProvider || !InNewSequence)
	{
		return;
	}

	UObject* const Outer = SequenceProvider->ToUObject();

	bool bIsOldOwnedByProvider = InOldSequence && InOldSequence->GetOuter() == Outer;
	bool bIsNewOwnedByProvider = InNewSequence->GetOuter() == Outer;

	// Only avoid re-opening the asset editor if it's changing between two sequences that are owned by the provider
	if (!bIsOldOwnedByProvider || !bIsNewOwnedByProvider)
	{
		OpenSequencerWithViewedSequence();	
	}
}

ULevelSequence* FAvaLevelSequencerExtension::GetViewedSequence() const
{
	if (!AvaSequencer.IsValid())
	{
		return ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence();
	}

	ULevelSequence* OutSequence = nullptr;

	ON_SCOPE_EXIT
	{
		static bool bReentrant = false;
		if (!bReentrant && AvaSequencer.IsValid())
		{
			TGuardValue<bool> ReentrantGuard(bReentrant, true);
			AvaSequencer->SetViewedSequence(Cast<UAvaSequence>(OutSequence));
		}
	};

	OutSequence = AvaSequencer->GetViewedSequence();
	if (OutSequence)
	{
		return OutSequence;
	}

	OutSequence = ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence();
	if (OutSequence)
	{
		return OutSequence;
	}

	OutSequence = AvaSequencer->GetDefaultSequence();
	if (OutSequence)
	{
		return OutSequence;
	}

	return OutSequence;
}

ILevelSequenceEditorToolkit* FAvaLevelSequencerExtension::OpenSequencerWithViewedSequence() const
{
	if (ULevelSequence* Sequence = GetViewedSequence())
	{
		return UE::AvaEditor::Private::OpenSequenceEditor(Sequence);
	}
	return nullptr;
}

void FAvaLevelSequencerExtension::OnAssetEditorOpened(UObject* InObject)
{
	if (ULevelSequence* LevelSequence = Cast<ULevelSequence>(InObject))
	{
		if (AvaSequencer.IsValid())
		{
			AvaSequencer->SetViewedSequence(Cast<UAvaSequence>(LevelSequence));
		}
		ApplyTabContent();
	}
}

void FAvaLevelSequencerExtension::ApplyTabContent()
{
	TSharedPtr<IAvaEditor> Editor = GetEditor();
	if (!Editor.IsValid())
	{
		return;
	}

	TSharedPtr<FTabManager> TabManager = Editor->GetTabManager();
	if (!TabManager.IsValid())
	{
		return;
	}

	TSharedPtr<SDockTab> SequencerTab = TabManager->TryInvokeTab(LevelEditorTabIds::Sequencer);
	if (SequencerTab.IsValid())
	{
		SequencerTabWeak = SequencerTab;
		SequencerTab->SetContent(FAvaSequencerTabSpawner(Editor.ToSharedRef(), GetSequencerTabId()).CreateTabBody());
	}
}

void FAvaLevelSequencerExtension::RestoreTabContent()
{
	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	TSharedPtr<SDockTab> SequencerTab = SequencerTabWeak.Pin();
	if (!SequencerTab.IsValid())
	{
		return;
	}

	SequencerTab->SetContent(Sequencer->GetSequencerWidget());
	SequencerTabWeak.Reset();
	SequencerTabContentWeak.Reset();
}

void FAvaLevelSequencerExtension::AddCinematicsToolbarExtension()
{
	UToolMenus* const ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return;
	}

	UToolMenu* const ToolMenu = ToolMenus->ExtendMenu(UE::AvaEditor::Private::CinematicToolbarName);
	if (!ToolMenu)
	{
		return;
	}

	FToolMenuSection& AvaSequenceSection = ToolMenu->AddSection(UE::AvaEditor::Private::AvaSequenceSectionName
		, LOCTEXT("AvaSequenceSectionLabel", "Edit Motion Design Sequence")
		, FToolMenuInsert(UE::AvaEditor::Private::CinematicExtensionName, EToolMenuInsertType::Before));

	AvaSequenceSection.AddDynamicEntry("AvaSequenceEntry", FNewToolMenuSectionDelegate::CreateSPLambda(this,
		[this](FToolMenuSection& InSection)
		{
			if (!IsEditorActive())
			{
				return;
			}

			ULevelEditorMenuContext* const Context = InSection.Context.FindContext<ULevelEditorMenuContext>();
			if (!Context)
			{
				return;
			}

			UAvaSceneSubsystem* SceneSubsystem = FAvaWorldSubsystemUtils::GetWorldSubsystem<UAvaSceneSubsystem>(Context->GetLevelEditor().Pin().Get());
			if (!SceneSubsystem)
			{
				return;
			}

			if (IAvaSceneInterface* SceneInterface = SceneSubsystem->GetSceneInterface())
			{
				AddSequenceEntries(InSection, SceneInterface->GetSequenceProvider());
			}
		}));
}

void FAvaLevelSequencerExtension::RemoveCinematicsToolbarExtension()
{
	if (!UObjectInitialized())
	{
		return;
	}

	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return;
	}

	UToolMenu* ToolMenu = ToolMenus->FindMenu(UE::AvaEditor::Private::CinematicToolbarName);
	if (!ToolMenu)
	{
		return;
	}

	ToolMenu->RemoveSection(UE::AvaEditor::Private::AvaSequenceSectionName);
}

void FAvaLevelSequencerExtension::AddSequenceEntries(FToolMenuSection& InSection, IAvaSequenceProvider* InSequenceProvider)
{
	if (!InSequenceProvider)
	{
		return;
	}

	auto OpenSequenceEditor = [](TWeakObjectPtr<UAvaSequence> InSequence)
		{
			if (UAvaSequence* Sequence = InSequence.Get())
			{
				UE::AvaEditor::Private::OpenSequenceEditor(InSequence.Get());
			}
		};

	const FSlateIcon ClassIcon = FSlateIconFinder::FindIconForClass(UAvaSequence::StaticClass());
	for (UAvaSequence* Sequence : InSequenceProvider->GetSequences())
	{
		TWeakObjectPtr<UAvaSequence> SequenceWeak = Sequence;
		if (!SequenceWeak.IsValid())
		{
			continue;
		}

		InSection.AddMenuEntry(Sequence->GetFName()
			, Sequence->GetDisplayName()
			, TAttribute<FText>()
			, ClassIcon
			, FExecuteAction::CreateLambda(OpenSequenceEditor, SequenceWeak)
			, EUserInterfaceActionType::Button);
	}
}

#undef LOCTEXT_NAMESPACE
