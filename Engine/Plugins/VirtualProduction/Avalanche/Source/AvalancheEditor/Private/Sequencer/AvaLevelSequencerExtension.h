// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequencerExtension.h"
#include "Delegates/Delegate.h"

class IAvaSequenceProvider;
class IAvaSequencerController;
class ILevelSequenceEditorToolkit;
class SDockTab;
class SWidget;
class ULevelSequence;
class UObject;
struct FToolMenuSection;

class FAvaLevelSequencerExtension : public FAvaSequencerExtension
{
public:
	UE_AVA_INHERITS(FAvaLevelSequencerExtension, FAvaSequencerExtension);

	FAvaLevelSequencerExtension();

	virtual ~FAvaLevelSequencerExtension() override;

	//~ Begin FAvaSequencerExtension
	virtual FAvaSequencerArgs MakeSequencerArgs() const override;
	//~ End FAvaSequencerExtension

	//~ Begin IAvaEditorExtension
	virtual void Construct(const TSharedRef<IAvaEditor>& InEditor) override;
	virtual void ExtendLevelEditorLayout(FLayoutExtender& InExtender) const override;
	virtual void RegisterTabSpawners(const TSharedRef<IAvaEditor>& InEditor) const override;
	virtual void Activate() override;
	virtual void Deactivate() override;
	virtual void PostInvokeTabs() override;
	//~ Begin IAvaEditorExtension

	//~ Begin IAvaSequencerProvider
	virtual TSharedPtr<ISequencer> GetExternalSequencer() const override;
	virtual void OnViewedSequenceChanged(UAvaSequence* InOldSequence, UAvaSequence* InNewSequence) override;
	//~ End IAvaSequenceProvider

private:
	ULevelSequence* GetViewedSequence() const;

	ILevelSequenceEditorToolkit* OpenSequencerWithViewedSequence() const;

	void OnAssetEditorOpened(UObject* InObject);

	void ApplyTabContent();

	void RestoreTabContent();

	void AddCinematicsToolbarExtension();

	void RemoveCinematicsToolbarExtension();

	void AddSequenceEntries(FToolMenuSection& InSection, IAvaSequenceProvider* InSequenceProvider);

	TSharedRef<IAvaSequencerController> SequencerController;

	TWeakPtr<SWidget> SequencerTabContentWeak;

	TWeakPtr<SDockTab> SequencerTabWeak;

	FDelegateHandle OnAssetEditorOpenedHandle;
};
