// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequencerExtension.h"
#include "AvaEditorModule.h"
#include "AvaSequencerSubsystem.h"
#include "AvaSequencerTabSpawner.h"
#include "EditorModeManager.h"
#include "Framework/Docking/LayoutExtender.h"
#include "IAssetTools.h"
#include "IAvaSceneInterface.h"
#include "IAvaSequenceProvider.h"
#include "IAvaSequencer.h"
#include "ISequencer.h"
#include "LevelEditor.h"
#include "Misc/PackageName.h"
#include "MovieScene.h"
#include "Settings/AvaSequencerSettings.h"
#include "Settings/AvaTransitionEditorSettings.h"
#include "StatusBarSubsystem.h"
#include "UObject/Package.h"
#include "Viewport/AvaViewportExtension.h"
#include "WidgetDrawerConfig.h"

#define LOCTEXT_NAMESPACE "AvaSequencerExtension"

FAvaSequencerExtension::FAvaSequencerExtension()
	: SequencerTabId(FAvaSequencerTabSpawner::GetTabID())
	, bSupportsDrawerWidget(true)
{
}

FAvaSequencerExtension::FAvaSequencerExtension(FName InSequencerTabId, bool bInSupportsDrawerWidget)
	: SequencerTabId(InSequencerTabId)
	, bSupportsDrawerWidget(bInSupportsDrawerWidget)
{
}

TSharedPtr<ISequencer> FAvaSequencerExtension::GetSequencer() const
{
	return AvaSequencer.IsValid() ? AvaSequencer->GetSequencerPtr() : TSharedPtr<ISequencer>();
}

void FAvaSequencerExtension::OnObjectRenamed(UObject* InRenamedObject, const FText& InDisplayNameText)
{
	if (!InRenamedObject || !AvaSequencer.IsValid())
	{
		return;
	}

	if (const TSharedPtr<ISequencer> Sequencer = AvaSequencer->GetSequencerPtr())
	{
		const FGuid ObjectGuid = Sequencer->FindObjectId(*InRenamedObject, Sequencer->GetFocusedTemplateID());
		if (ObjectGuid.IsValid())
		{
			Sequencer->SetDisplayName(ObjectGuid, InDisplayNameText);
		}
	}
}

FAvaSequencerArgs FAvaSequencerExtension::MakeSequencerArgs() const
{
	return FAvaSequencerArgs();
}

void FAvaSequencerExtension::Activate()
{
	if (UWorld* const World = GetWorld())
	{
		if (UAvaSequencerSubsystem* SequencerSubsystem = World->GetSubsystem<UAvaSequencerSubsystem>())
		{
			AvaSequencer = SequencerSubsystem->GetOrCreateSequencer(*this, MakeSequencerArgs());
			ConditionallyCreateDefaultSequences();
			ValidateDirectorBlueprints();
		}
		else
		{
			UE_LOGF(AvaLog, Warning, "Missing sequencer world subsystem on extension activation.");
		}

		RegisterSequenceDrawerWidget();
		BindDelegates();
	}
}

void FAvaSequencerExtension::Deactivate()
{
	UnbindDelegates();
	UnregisterSequenceDrawerWidget();

	// Reset Sequencer as ISequencer uses FGCObject to hold reference to the Root Sequence and other objects
	// that should otherwise be GC'd along with the Level on Level Change
	AvaSequencer.Reset();
}

void FAvaSequencerExtension::Cleanup()
{
	const IAvaSequenceProvider* SequenceProvider = GetSequenceProvider();
	if (!SequenceProvider)
	{
		return;
	}

	// Clear the Asset Actions for each Director Blueprint so that the World gets properly GC'd
	for (UAvaSequence* Sequence : SequenceProvider->GetSequences())
	{
		if (Sequence)
		{
			Sequence->Cleanup();
		}
	}
}

void FAvaSequencerExtension::OnSceneObjectCreated(UObject* InSceneObject)
{
	bCreateDefaultSequences = true;
}

void FAvaSequencerExtension::OnSceneObjectChanged(UObject* InOldSceneObject, UObject* InNewSceneObject)
{
	if (IAvaSequenceProvider* SequenceProvider = Cast<IAvaSequenceProvider>(InNewSceneObject))
	{
		SequenceProvider->RebuildSequenceTree();

		const TArray<TObjectPtr<UAvaSequence>>& Sequences = SequenceProvider->GetSequences();

		// Select First Sequence from the new List
		if (!Sequences.IsEmpty() && AvaSequencer.IsValid())
		{
			AvaSequencer->SetViewedSequence(Sequences[0]);
			AvaSequencer->NotifyOnSequenceTreeChanged();
		}
	}
}

void FAvaSequencerExtension::RegisterTabSpawners(const TSharedRef<IAvaEditor>& InEditor) const
{
	InEditor->AddTabSpawner<FAvaSequencerTabSpawner>(InEditor, GetSequencerTabId());
}

void FAvaSequencerExtension::ExtendLevelEditorLayout(FLayoutExtender& InExtender) const
{
	InExtender.ExtendLayout(LevelEditorTabIds::Sequencer
		, ELayoutExtensionPosition::Before
		, FTabManager::FTab(GetSequencerTabId(), ETabState::OpenedTab));
}

void FAvaSequencerExtension::NotifyOnSelectionChanged(const FAvaEditorSelection& InSelection)
{
	if (AvaSequencer.IsValid())
	{
		AvaSequencer->OnEditorSelectionChanged(InSelection);
	}
}

void FAvaSequencerExtension::OnCopyActors(FString& OutExtensionData, TConstArrayView<AActor*> InActorsToCopy)
{
	if (AvaSequencer.IsValid())
	{
		AvaSequencer->OnActorsCopied(OutExtensionData, InActorsToCopy);
	}
}

void FAvaSequencerExtension::OnPasteActors(FStringView InPastedData, TConstArrayView<FAvaEditorPastedActor> InPastedActors)
{
	if (AvaSequencer.IsValid())
	{
		constexpr bool bIncludeDuplicatedActors = true;
		TMap<FName, AActor*> PastedActors = FAvaEditorPastedActor::BuildPastedActorMap(InPastedActors, bIncludeDuplicatedActors);
		AvaSequencer->OnActorsPasted(InPastedData, PastedActors);
	}
}

IAvaSequenceProvider* FAvaSequencerExtension::GetSequenceProvider() const
{
	return GetSceneObject<IAvaSequenceProvider>();
}

FEditorModeTools* FAvaSequencerExtension::GetSequencerModeTools() const
{
	return GetEditorModeTools();
}

IAvaSequencePlaybackObject* FAvaSequencerExtension::GetPlaybackObject() const
{
	if (IAvaSceneInterface* const Scene = GetSceneObject<IAvaSceneInterface>())
	{
		return Scene->GetPlaybackObject();
	}
	return nullptr;
}

TSharedPtr<IToolkitHost> FAvaSequencerExtension::GetSequencerToolkitHost() const
{
	return GetToolkitHost();
}

UObject* FAvaSequencerExtension::GetPlaybackContext() const
{
	return GetWorld();
}

bool FAvaSequencerExtension::CanEditOrPlaySequences() const
{
	const bool bPlayInEditorActive = GEditor && GEditor->PlayWorld != nullptr;
	return !bPlayInEditorActive;
}

void FAvaSequencerExtension::ExportSequences(TConstArrayView<UAvaSequence*> InSequencesToExport)
{
	IAssetTools& AssetTools = IAssetTools::Get();

	for (UAvaSequence* const SourceSequence : InSequencesToExport)
	{
		if (!SourceSequence)
		{
			continue;
		}

		if (UPackage* const SourcePackage = SourceSequence->GetPackage())
		{
			AssetTools.DuplicateAsset(SourceSequence->GetName() + TEXT("_Exported")
				, FPackageName::GetLongPackagePath(SourcePackage->GetName())
				, SourceSequence);
		}
	}
}

void FAvaSequencerExtension::BindDelegates()
{
	IAvaSequenceProvider* const SequenceProvider = GetSequenceProvider();
	if (!SequenceProvider || !AvaSequencer.IsValid())
	{
		return;
	}
	UnbindDelegates();
	SequenceProvider->GetOnSequenceTreeRebuilt().AddSP(AvaSequencer.ToSharedRef(), &IAvaSequencer::NotifyOnSequenceTreeChanged);
}

void FAvaSequencerExtension::UnbindDelegates()
{
	IAvaSequenceProvider* const SequenceProvider = GetSequenceProvider();
	if (!SequenceProvider || !AvaSequencer.IsValid())
	{
		return;
	}
	SequenceProvider->GetOnSequenceTreeRebuilt().RemoveAll(AvaSequencer.Get());
}

TSharedRef<SWidget> FAvaSequencerExtension::GetSequenceDrawerWidget()
{
	if (!ensureAlways(bSupportsDrawerWidget))
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<IAvaEditor> Editor = GetEditor();
	if (!Editor.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	if (!SequencerDrawerWidget.IsValid())
	{
		SequencerDrawerWidget = FAvaSequencerTabSpawner(Editor.ToSharedRef(), GetSequencerTabId(), true).CreateTabBody();
	}

	return SequencerDrawerWidget.ToSharedRef();
}

void FAvaSequencerExtension::RegisterSequenceDrawerWidget()
{
	if (!bSupportsDrawerWidget)
	{
		return;
	}

	TSharedPtr<IToolkitHost> ToolkitHost = GetToolkitHost();
	if (!ToolkitHost.IsValid())
	{
		return;	
	}

	FWidgetDrawerConfig DrawerConfig(GetSequencerTabId());
	DrawerConfig.GetDrawerContentDelegate.BindSP(this, &FAvaSequencerExtension::GetSequenceDrawerWidget);
	DrawerConfig.ButtonText  = LOCTEXT("SequencerStatusBarButtonText", "Sequencer");
	DrawerConfig.ToolTipText = LOCTEXT("SequencerStatusBarTooltip", "Opens the Sequencer Drawer Widget");
	DrawerConfig.Icon        = FAppStyle::Get().GetBrush("UMGEditor.AnimTabIcon");

	GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->RegisterDrawer(ToolkitHost->GetStatusBarName(), MoveTemp(DrawerConfig), 1);
}

void FAvaSequencerExtension::UnregisterSequenceDrawerWidget()
{
	if (!bSupportsDrawerWidget)
	{
		ensureAlways(!SequencerDrawerWidget.IsValid());
		return;
	}

	SequencerDrawerWidget.Reset();

	TSharedPtr<IToolkitHost> ToolkitHost = GetToolkitHost();
	if (!ToolkitHost.IsValid())
	{
		return;	
	}

	if (GEditor)
	{
		if (UStatusBarSubsystem* StatusBarSubsystem = GEditor->GetEditorSubsystem<UStatusBarSubsystem>())
		{
			StatusBarSubsystem->UnregisterDrawer(ToolkitHost->GetStatusBarName(), GetSequencerTabId());
		}
	}
}

void FAvaSequencerExtension::ConditionallyCreateDefaultSequences()
{
	if (!bCreateDefaultSequences || !AvaSequencer.IsValid())
	{
		return;
	}

	const UAvaSequencerSettings* const SequencerSettings = GetDefault<UAvaSequencerSettings>();
	if (!SequencerSettings)
	{
		return;
	}

	// Transition Logic Default Sequences
	const UAvaTransitionEditorSettings* TransitionEditorSettings = GetDefault<UAvaTransitionEditorSettings>();
	if (TransitionEditorSettings && TransitionEditorSettings->ShouldCreateTransitionLogicDefaultScene())
	{
		const TArray<const FAvaSequencePreset*> SequencePresets = SequencerSettings->GatherPresetsFromGroup(UAvaSequencerSettings::TransitionLogicStarterGroup);
		AvaSequencer->AddSequenceFromPresets(SequencePresets);
	}

	bCreateDefaultSequences = false;
}

void FAvaSequencerExtension::ValidateDirectorBlueprints()
{
	IAvaSequenceProvider* SequenceProvider = GetSequenceProvider();
	if (!SequenceProvider)
	{
		return;
	}

	for (UAvaSequence* Sequence : SequenceProvider->GetSequences())
	{
		UBlueprint* Blueprint = Sequence ? Sequence->GetDirectorBlueprint() : nullptr;

		if (Blueprint && Blueprint->GetOuter() != Sequence)
		{
			if (Blueprint->Rename(nullptr, Sequence, REN_Test))
			{
				Blueprint->Rename(nullptr, Sequence);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
