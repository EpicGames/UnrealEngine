// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequencerArgs.h"
#include "IAvaEditorExtension.h"
#include "IAvaSequencerProvider.h"
#include "Misc/EngineVersionComparison.h"

class FText;
class IAvaSequencer;
class ISequencer;
class SWidget;
class UObject;

UE_AVA_TYPE_EXTERNAL(IAvaSequencerProvider);

class FAvaSequencerExtension: public FAvaEditorExtension, public IAvaSequencerProvider
{
public:
	UE_AVA_INHERITS(FAvaSequencerExtension, FAvaEditorExtension, IAvaSequencerProvider);

	FAvaSequencerExtension();

	explicit FAvaSequencerExtension(FName InSequencerTabId, bool bInSupportsDrawerWidget);

	TSharedPtr<ISequencer> GetSequencer() const;

	void OnObjectRenamed(UObject* InRenamedObject, const FText& InDisplayNameText);

	FName GetSequencerTabId() const { return SequencerTabId; }

	virtual FAvaSequencerArgs MakeSequencerArgs() const;

	//~ Begin IAvaEditorExtension
	virtual TSharedPtr<IAvaSequencer> GetAvaSequencer() const override { return AvaSequencer; }
	virtual void Activate() override;
	virtual void Deactivate() override;
	virtual void Cleanup() override;
	virtual void OnSceneObjectCreated(UObject* InSceneObject) override;
	virtual void OnSceneObjectChanged(UObject* InOldSceneObject, UObject* InNewSceneObject) override;
	virtual void RegisterTabSpawners(const TSharedRef<IAvaEditor>& InEditor) const override;
	virtual void ExtendLevelEditorLayout(FLayoutExtender& InExtender) const override;
	virtual void NotifyOnSelectionChanged(const FAvaEditorSelection& InSelection) override;
	virtual void OnCopyActors(FString& OutExtensionData, TConstArrayView<AActor*> InActorsToCopy) override;
	virtual void OnPasteActors(FStringView InPastedData, TConstArrayView<FAvaEditorPastedActor> InPastedActors) override;
	//~ End IAvaEditorExtension

	//~ Begin IAvaSequencerProvider
	virtual IAvaSequenceProvider* GetSequenceProvider() const override;
	virtual FEditorModeTools* GetSequencerModeTools() const override;
	virtual IAvaSequencePlaybackObject* GetPlaybackObject() const override;
	virtual TSharedPtr<IToolkitHost> GetSequencerToolkitHost() const override;
	virtual UObject* GetPlaybackContext() const override;
	virtual bool CanEditOrPlaySequences() const override;
	virtual void ExportSequences(TConstArrayView<UAvaSequence*> InSequencesToExport) override;
	//~ End IAvaSequencerProvider

protected:
	TSharedPtr<IAvaSequencer> AvaSequencer;

private:
	void BindDelegates();

	void UnbindDelegates();

	TSharedRef<SWidget> GetSequenceDrawerWidget();

	void RegisterSequenceDrawerWidget();

	void UnregisterSequenceDrawerWidget();

	/** Creates the default sequences gotten from the default preset group to use */
	void ConditionallyCreateDefaultSequences();

	/** Temporary fix ensuring that any Director Blueprint that got renamed to the incorrect outer gets fixed to the correct outer*/
	void ValidateDirectorBlueprints();

	/** The Drawer Widget for the Animations */
	TSharedPtr<SWidget> SequencerDrawerWidget;

	const FName SequencerTabId;

	const bool bSupportsDrawerWidget;

	/**
	 * Flag to indicate whether to create the default sequences when activating
	 * Set to true when the scene object is first created, but before the editor itself is activated.
	 */
	bool bCreateDefaultSequences = false;
};
