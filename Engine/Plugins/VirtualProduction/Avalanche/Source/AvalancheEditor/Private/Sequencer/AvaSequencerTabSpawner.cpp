// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequencerTabSpawner.h"
#include "AvaSequencerExtension.h"
#include "IAvaEditor.h"
#include "IAvaSequencer.h"
#include "UMGStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AvaSequencerTabSpawner"

FName FAvaSequencerTabSpawner::GetTabID()
{
	return TEXT("AvalancheAnimation");
}

FAvaSequencerTabSpawner::FAvaSequencerTabSpawner(const TSharedRef<IAvaEditor>& InEditor, FName InTabId, bool bInIsDrawerTab)
	: FAvaTabSpawner(InEditor, InTabId)
	, bIsDrawerTab(bInIsDrawerTab)
{
	TabLabel       = LOCTEXT("TabLabel", "Motion Design Animation");
	TabTooltipText = LOCTEXT("TabTooltip", "Motion Design Animation");
	TabIcon        = FSlateIcon(FUMGStyle::GetStyleSetName(), "Animations.TabIcon");
}

TSharedRef<SWidget> FAvaSequencerTabSpawner::CreateTabBody()
{
	const TSharedPtr<IAvaEditor> Editor = EditorWeak.Pin();
	if (!ensure(Editor.IsValid()))
	{
		return GetNullWidget();
	}

	const TSharedPtr<FAvaSequencerExtension> SequencerExtension = Editor->FindExtension<FAvaSequencerExtension>();
	if (!ensure(SequencerExtension.IsValid()))
	{
		return GetNullWidget();
	}

	const TSharedPtr<IAvaSequencer> AvaSequencer = SequencerExtension->GetAvaSequencer();
	if (!ensure(AvaSequencer.IsValid()))
	{
		return GetNullWidget();
	}

	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(FMargin(bIsDrawerTab ? 8.f : 2.f, 2.f))
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				AvaSequencer->CreateSequenceWidget()
			]
			+SOverlay::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Top)
			.Padding(FMargin(1.f, 3.f))
			[
				CreateDrawerDockButton(Editor.ToSharedRef())
			]
		];
}

TSharedRef<SWidget> FAvaSequencerTabSpawner::CreateDrawerDockButton(TSharedRef<IAvaEditor> InEditor) const
{
	if (!bIsDrawerTab)
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.ToolTipText(LOCTEXT("DockInLayoutTooltip", "Docks animation drawer in tab."))
		.ContentPadding(FMargin(3.f, 1.f))
		.OnClicked(InEditor, &IAvaEditor::DockInLayout, GetId())
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::Get().GetBrush("Icons.Layout"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(4.f, 0.f, 0.f, 0.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DockInLayoutLabel", "Dock in Layout"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
}

#undef LOCTEXT_NAMESPACE
