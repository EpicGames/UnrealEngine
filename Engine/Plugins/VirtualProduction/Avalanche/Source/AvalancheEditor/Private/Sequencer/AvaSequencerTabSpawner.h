// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTabSpawner.h"

class FAvaSequencerTabSpawner : public FAvaTabSpawner
{
public:
	static FName GetTabID();

	explicit FAvaSequencerTabSpawner(const TSharedRef<IAvaEditor>& InEditor, FName InTabId, bool bInIsDrawerTab = false);

	//~ Begin IAvaTabSpawner
	virtual TSharedRef<SWidget> CreateTabBody() override;
	//~ End IAvaTabSpawner

	TSharedRef<SWidget> CreateDrawerDockButton(TSharedRef<IAvaEditor> InEditor) const;

private:
	bool bIsDrawerTab;
};
