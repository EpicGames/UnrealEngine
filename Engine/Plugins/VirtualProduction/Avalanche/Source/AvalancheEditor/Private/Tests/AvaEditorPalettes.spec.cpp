// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaDefs.h"
#include "AvaScreenAlignmentEnums.h"
#include "AvaScreenAlignmentUtils.h"
#include "Misc/AutomationTest.h"
#include "Test/AvaViewportWorldCoordinateConverterProvider.h"
#include "Tests/Framework/AvaEditorTestUtils.h"
#include "Tests/Framework/AvaTestDynamicMeshActor.h"

BEGIN_DEFINE_SPEC(AvalancheEditorAligmentPalettes, "Avalanche.Editor.AlignmentPalettes", EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	TArray<AActor*> Actors;
	int32 ActorCount = 3;

	static constexpr int32 AxisX = 0;
	static constexpr int32 AxisY = 1;
	static constexpr int32 AxisZ = 2;

	TArray<FVector> InitialActorLocations;
	TArray<FVector> ActorLocations;

	FAvaEditorTestUtils TestUtils;

	// Camera
	const FVector CameraLocation = {-500, 0, 0};
	const FRotator Rotation = {0, 0, 0};
	const FVector2f ViewportSize = {1920, 1080};
	const float FieldOfView = 90.f;

	TSharedPtr<FAvaViewportWorldCoordinateConverterProviderPerspective> Camera;

END_DEFINE_SPEC(AvalancheEditorAligmentPalettes);

void AvalancheEditorAligmentPalettes::Define()
{
	BeforeEach([this]
	{
		TestUtils.Init();

		// Spawn actors
		Actors.Empty();
		Actors.Append(TestUtils.SpawnTestDynamicMeshActors(ActorCount));

		// Define Camera
		Camera = MakeShared<FAvaViewportWorldCoordinateConverterProviderPerspective>(CameraLocation, Rotation, ViewportSize, FieldOfView);
	});

	AfterEach([this]
	{
		TestUtils.Destroy();
	});

	Describe("Horizontal Alignmetn", [this]
	{
		BeforeEach([this]
		{
			// Define Actor locations
			ActorLocations = TestUtils.GenerateRandomVectors(ActorCount);

			// Place Actors on the scene
			TestUtils.SetActorLocations(Actors, ActorLocations);

			// Store initial Actor locations
			InitialActorLocations = TestUtils.GetActorLocations(Actors);
		});

		It("Left: Should align actors to leftmost Actor in array accrding to camera's position", [this]
		{
			// Find Leftmost Actor according to the Camera
			const int32 LeftmostActorIndex = TestUtils.GetLowestAxisActorForCamera(InitialActorLocations, CameraLocation, AxisY);

			// Apply alignment to the Array of actors
			FAvaScreenAlignmentUtils::AlignActorsHorizontal(Camera.ToSharedRef(), Actors, EAvaHorizontalAlignment::Left, EAvaAlignmentSizeMode::Self, EAvaAlignmentContext::SelectedActors);

			TArray<FVector> NewActorLocations = TestUtils.GetActorLocations(Actors);

			// Test X and Z not changed
			for (int32 LocationIndex = 0; LocationIndex < Actors.Num(); LocationIndex++)
			{
				TestEqual("X axis was not affected", NewActorLocations[LocationIndex].X, InitialActorLocations[LocationIndex].X);
				TestEqual("Z axis was not affected", NewActorLocations[LocationIndex].Z, InitialActorLocations[LocationIndex].Z);
			}

			// Test Y Aligned correctly
			for (int32 ActorIndex = 0; ActorIndex < Actors.Num(); ActorIndex++)
			{
				if (ActorIndex == LeftmostActorIndex)
				{
					TestEqual("Leftmost Actor remained unchanged", NewActorLocations[ActorIndex].Y, InitialActorLocations[ActorIndex].Y);
				}
				else
				{
					TestEqual("Actor is aligned as expected", NewActorLocations[ActorIndex].Y, TestUtils.GetExpectedAxisValue(InitialActorLocations[LeftmostActorIndex], InitialActorLocations[ActorIndex], CameraLocation.X, AxisY), 0.001);
				}
			}
		});

		It("Right: Should align actors to rightmost Actor in array according to camera's position", [this]
		{
			// Find Rightmost Actor according to the Camera
			const int32 RightmostActorIndex = TestUtils.GetHighestAxisActorForCamera(InitialActorLocations, CameraLocation, AxisY);

			// Apply alignment to the Array of actors
			FAvaScreenAlignmentUtils::AlignActorsHorizontal(Camera.ToSharedRef(), Actors, EAvaHorizontalAlignment::Right, EAvaAlignmentSizeMode::Self, EAvaAlignmentContext::SelectedActors);

			TArray<FVector> NewActorLocations = TestUtils.GetActorLocations(Actors);

			// Test X and Z not changed
			for (int32 LocationIndex = 0; LocationIndex < Actors.Num(); LocationIndex++)
			{
				TestEqual("X axis was not affected", NewActorLocations[LocationIndex].X, InitialActorLocations[LocationIndex].X);
				TestEqual("Z axis was not affected", NewActorLocations[LocationIndex].Z, InitialActorLocations[LocationIndex].Z);
			}

			// Test Y Aligned correctly
			for (int32 ActorIndex = 0; ActorIndex < Actors.Num(); ActorIndex++)
			{
				if (ActorIndex == RightmostActorIndex)
				{
					TestEqual("Rightmost Actor remained unchanged", NewActorLocations[ActorIndex].Y, InitialActorLocations[ActorIndex].Y, 0.001);
				}
				else
				{
					TestEqual("Actor is aligned as expected", NewActorLocations[ActorIndex].Y, TestUtils.GetExpectedAxisValue(InitialActorLocations[RightmostActorIndex], InitialActorLocations[ActorIndex], CameraLocation.X, AxisY), 0.001);
				}
			}
		});

		It("Center: Should align array of actors horizontally in the center according to camera's position", [this]
		{
			// Find anchor location for further expected actors alignment locations calculations
			const FVector CenterAnchorActor = TestUtils.GetCenterAnchorLocation(InitialActorLocations, CameraLocation, AxisY);

			// Apply alignment to the Array of actors
			FAvaScreenAlignmentUtils::AlignActorsHorizontal(Camera.ToSharedRef(), Actors, EAvaHorizontalAlignment::Center, EAvaAlignmentSizeMode::Self, EAvaAlignmentContext::SelectedActors);

			TArray<FVector> NewActorLocations = TestUtils.GetActorLocations(Actors);

			// Test Y Aligned correctly and X, Z were not effected
			for (int32 LocationIndex = 0; LocationIndex < InitialActorLocations.Num(); LocationIndex++)
			{
				TestEqual("X axis was not affected", NewActorLocations[LocationIndex].X, InitialActorLocations[LocationIndex].X);
				TestEqual("Z axis was not affected", NewActorLocations[LocationIndex].Z, InitialActorLocations[LocationIndex].Z);
				TestEqual("Y axis aligned as expected", NewActorLocations[LocationIndex].Y, TestUtils.GetExpectedAxisValue(CenterAnchorActor, InitialActorLocations[LocationIndex], CameraLocation.X, AxisY), 0.001);
			}
		});
	});

	Describe("Vertical Alignment", [this]
	{
		BeforeEach([this]
		{
			// Define Actor locations
			ActorLocations = TestUtils.GenerateRandomVectors(ActorCount);

			// Place Actors on the scene
			TestUtils.SetActorLocations(Actors, ActorLocations);

			// Store initial Actor locations;
			InitialActorLocations = TestUtils.GetActorLocations(Actors);
		});

		It("Bottom: Should align actors to bottom most Actor in array according to camera's position", [this]
		{
			const int32 BottommostActorIndex = TestUtils.GetLowestAxisActorForCamera(InitialActorLocations, CameraLocation, AxisZ);

			FAvaScreenAlignmentUtils::AlignActorsVertical(Camera.ToSharedRef(), Actors, EAvaVerticalAlignment::Bottom, EAvaAlignmentSizeMode::Self, EAvaAlignmentContext::SelectedActors);

			TArray<FVector> NewActorLocations = TestUtils.GetActorLocations(Actors);

			// Test X and Y not changed
			for (int32 LocationIndex = 0; LocationIndex < Actors.Num(); LocationIndex++)
			{
				TestEqual("X axis was not affected", NewActorLocations[LocationIndex].X, InitialActorLocations[LocationIndex].X);
				TestEqual("Y axis was not affected", NewActorLocations[LocationIndex].Y, InitialActorLocations[LocationIndex].Y);
			}

			// Test Z Aligned correctly
			for (int32 ActorIndex = 0; ActorIndex < Actors.Num(); ActorIndex++)
			{
				if (ActorIndex == BottommostActorIndex)
				{
					TestEqual("Bottommost Actor remained unchanged", NewActorLocations[ActorIndex].Z, InitialActorLocations[ActorIndex].Z);
				}
				else
				{
					TestEqual("Actor is aligned as expected", NewActorLocations[ActorIndex].Z, TestUtils.GetExpectedAxisValue(InitialActorLocations[BottommostActorIndex], InitialActorLocations[ActorIndex], CameraLocation.X, AxisZ), 0.001);
				}
			}
		});

		It("Top: Should align actors to top most Actor in array according to camera's position", [this]
		{
			// Find Topmost Actor according to the Camera
			const int32 TopmostActorIndex = TestUtils.GetHighestAxisActorForCamera(InitialActorLocations, CameraLocation, AxisZ);

			FAvaScreenAlignmentUtils::AlignActorsVertical(Camera.ToSharedRef(), Actors, EAvaVerticalAlignment::Top, EAvaAlignmentSizeMode::Self, EAvaAlignmentContext::SelectedActors);

			TArray<FVector> NewActorLocations = TestUtils.GetActorLocations(Actors);

			// Test X and Y not changed
			for (int32 LocationIndex = 0; LocationIndex < Actors.Num(); LocationIndex++)
			{
				TestEqual("X axis was not affected", NewActorLocations[LocationIndex].X, InitialActorLocations[LocationIndex].X);
				TestEqual("Y axis was not affected", NewActorLocations[LocationIndex].Y, InitialActorLocations[LocationIndex].Y);
			}

			// Test Z Aligned correctly
			for (int32 ActorIndex = 0; ActorIndex < Actors.Num(); ActorIndex++)
			{
				if (ActorIndex == TopmostActorIndex)
				{
					TestEqual("Rightmost Actor remained unchanged", NewActorLocations[ActorIndex].Z, InitialActorLocations[ActorIndex].Z, 0.001);
				}
				else
				{
					TestEqual("Actor is aligned as expected", NewActorLocations[ActorIndex].Z, TestUtils.GetExpectedAxisValue(InitialActorLocations[TopmostActorIndex], InitialActorLocations[ActorIndex], CameraLocation.X, AxisZ), 0.001);
				}
			}
		});

		It("Center:  Should align array of actors vertically in the center according to camera's position", [this]
		{
			const FVector CenterAnchorActor = TestUtils.GetCenterAnchorLocation(InitialActorLocations, CameraLocation, AxisZ);

			FAvaScreenAlignmentUtils::AlignActorsVertical(Camera.ToSharedRef(), Actors, EAvaVerticalAlignment::Center, EAvaAlignmentSizeMode::Self, EAvaAlignmentContext::SelectedActors);

			TArray<FVector> NewActorLocations = TestUtils.GetActorLocations(Actors);

			// Test Z Aligned correctly and X, Y were not effected
			for (int32 LocationIndex = 0; LocationIndex < InitialActorLocations.Num(); LocationIndex++)
			{
				TestEqual("X axis was not affected", NewActorLocations[LocationIndex].X, InitialActorLocations[LocationIndex].X);
				TestEqual("Y axis was not affected", NewActorLocations[LocationIndex].Y, InitialActorLocations[LocationIndex].Y);
				TestEqual("Z axis aligned as expected", NewActorLocations[LocationIndex].Z, TestUtils.GetExpectedAxisValue(CenterAnchorActor, InitialActorLocations[LocationIndex], CameraLocation.X, AxisZ), 0.001);
			}
		});
	});

	Describe("Depth Alignment", [this]
	{
		BeforeEach([this]
		{
			// Define Actor locations
			ActorLocations = TestUtils.GenerateRandomVectors(ActorCount);

			// Place Actors on the scene
			TestUtils.SetActorLocations(Actors, ActorLocations);

			// Store initial Actor locations;
			InitialActorLocations = TestUtils.GetActorLocations(Actors);
		});

		It("Should align array of actors in depth to the Front according to camera's position", [this]
		{
			const int32 ExpectedFrontActorIndex = TestUtils.GetLowestAxisActor(ActorLocations, AxisX);

			FAvaScreenAlignmentUtils::AlignActorsDepth(Camera.ToSharedRef(), Actors, EAvaDepthAlignment::Front, EAvaAlignmentSizeMode::Self);
			TArray<FVector> NewActorLocations = TestUtils.GetActorLocations(Actors);

			for (int32 LocationIndex = 0; LocationIndex < InitialActorLocations.Num(); LocationIndex++)
			{
				TestEqual("Y axis was not affected", NewActorLocations[LocationIndex].Y, InitialActorLocations[LocationIndex].Y);
				TestEqual("Z axis was not affected", NewActorLocations[LocationIndex].Z, InitialActorLocations[LocationIndex].Z);
				TestEqual("X (Depth) axis aligned as expected", NewActorLocations[LocationIndex].X, ActorLocations[ExpectedFrontActorIndex].X, 0.001);
			}
		});

		It("Should align array of actors in depth to the Back according to camera's position", [this]
		{
			const int32 ExpectedBackActorIndex = TestUtils.GetHighestAxisActor(ActorLocations, AxisX);

			FAvaScreenAlignmentUtils::AlignActorsDepth(Camera.ToSharedRef(), Actors, EAvaDepthAlignment::Back, EAvaAlignmentSizeMode::Self);
			TArray<FVector> NewActorLocations = TestUtils.GetActorLocations(Actors);

			for (int32 LocationIndex = 0; LocationIndex < InitialActorLocations.Num(); LocationIndex++)
			{
				TestEqual("Y axis was not affected", NewActorLocations[LocationIndex].Y, InitialActorLocations[LocationIndex].Y);
				TestEqual("Z axis was not affected", NewActorLocations[LocationIndex].Z, InitialActorLocations[LocationIndex].Z);
				TestEqual("X (Depth) axis aligned as expected", NewActorLocations[LocationIndex].X, ActorLocations[ExpectedBackActorIndex].X, 0.001);
			}
		});

		It("Should align array of actors in depth to the Center according to camera's position", [this]
		{
			const int32 ExpectedFrontActorIndex = TestUtils.GetLowestAxisActor(ActorLocations, AxisX);
			const int32 ExpectedBackActorIndex = TestUtils.GetHighestAxisActor(ActorLocations, AxisX);
			const double ExpectedCenteredDepthValue = (ActorLocations[ExpectedBackActorIndex].X + ActorLocations[ExpectedFrontActorIndex].X) / 2;

			FAvaScreenAlignmentUtils::AlignActorsDepth(Camera.ToSharedRef(), Actors, EAvaDepthAlignment::Center, EAvaAlignmentSizeMode::Self);
			TArray<FVector> NewActorLocations = TestUtils.GetActorLocations(Actors);

			for (int32 LocationIndex = 0; LocationIndex < InitialActorLocations.Num(); LocationIndex++)
			{
				TestEqual("Y axis was not affected", NewActorLocations[LocationIndex].Y, InitialActorLocations[LocationIndex].Y);
				TestEqual("Z axis was not affected", NewActorLocations[LocationIndex].Z, InitialActorLocations[LocationIndex].Z);
				TestEqual("X (Depth) axis aligned as expected", NewActorLocations[LocationIndex].X, ExpectedCenteredDepthValue, 0.001);
			}
		});
	});

	Describe("Distribute Actors", [this]
	{
		BeforeEach([this]
		{
			// Define Actor locations
			ActorLocations = TestUtils.GenerateRandomVectors(ActorCount);

			// Place Actors on the scene
			TestUtils.SetActorLocations(Actors, ActorLocations);

			// Store initial Actor locations;
			InitialActorLocations = TestUtils.GetActorLocations(Actors);
		});

		It("Distribute Actors: Horizontal", [this]
		{
			const FVector CenterAnchorActor = TestUtils.GetCenterAnchorLocation(InitialActorLocations, CameraLocation, AxisY);
			const int32 LeftmostActorIndex = TestUtils.GetLowestAxisActorForCamera(ActorLocations, CameraLocation, AxisY);
			const int32 RightmostActorIndex = TestUtils.GetHighestAxisActorForCamera(ActorLocations, CameraLocation, AxisY);

			FAvaScreenAlignmentUtils::DistributeActorsHorizontal(Camera.ToSharedRef(), Actors, EAvaAlignmentSizeMode::Self, EAvaActorDistributionMode::CenterDistance, EAvaAlignmentContext::SelectedActors);
			TArray<FVector> NewActorLocations = TestUtils.GetActorLocations(Actors);

			for (int32 LocationIndex = 0; LocationIndex < InitialActorLocations.Num(); LocationIndex++)
			{
				TestEqual("X axis was not affected", NewActorLocations[LocationIndex].X, InitialActorLocations[LocationIndex].X);
				TestEqual("Z axis was not affected", NewActorLocations[LocationIndex].Z, InitialActorLocations[LocationIndex].Z);
			}

			for (int32 LocationIndex = 0; LocationIndex < InitialActorLocations.Num(); LocationIndex++)
			{
				if (LocationIndex == LeftmostActorIndex || LocationIndex == RightmostActorIndex)
				{
					TestEqual("Y value didn't change for edge actor", ActorLocations[LocationIndex].Y, NewActorLocations[LocationIndex].Y);
				}
				else
				{
					TestEqual("Y value calculated correctly for the actor in the middle", NewActorLocations[LocationIndex].Y, TestUtils.GetExpectedAxisValue(CenterAnchorActor, InitialActorLocations[LocationIndex], CameraLocation.X, AxisY), 0.001);
				}
			}
		});

		It("Distribute Actors: Vertical", [this]
		{
			const FVector CenterAnchorActor = TestUtils.GetCenterAnchorLocation(InitialActorLocations, CameraLocation, AxisZ);
			const int32 LowestActorIndex = TestUtils.GetLowestAxisActorForCamera(ActorLocations, CameraLocation, AxisZ);
			const int32 HighestActorIndex = TestUtils.GetHighestAxisActorForCamera(ActorLocations, CameraLocation, AxisZ);

			FAvaScreenAlignmentUtils::DistributeActorsVertical(Camera.ToSharedRef(), Actors, EAvaAlignmentSizeMode::Self, EAvaActorDistributionMode::CenterDistance, EAvaAlignmentContext::SelectedActors);
			TArray<FVector> NewActorLocations = TestUtils.GetActorLocations(Actors);

			for (int32 LocationIndex = 0; LocationIndex < InitialActorLocations.Num(); LocationIndex++)
			{
				TestEqual("Y axis was not affected", NewActorLocations[LocationIndex].Y, InitialActorLocations[LocationIndex].Y);
				TestEqual("X axis was not affected", NewActorLocations[LocationIndex].X, InitialActorLocations[LocationIndex].X);
			}

			for (int32 LocationIndex = 0; LocationIndex < InitialActorLocations.Num(); LocationIndex++)
			{
				if (LocationIndex == LowestActorIndex || LocationIndex == HighestActorIndex)
				{
					TestEqual("Z value didn't change for edge actor", ActorLocations[LocationIndex].Z, NewActorLocations[LocationIndex].Z);
				}
				else
				{
					TestEqual("Z value calculated correctly for the actor in the middle", NewActorLocations[LocationIndex].Z, TestUtils.GetExpectedAxisValue(CenterAnchorActor, InitialActorLocations[LocationIndex], CameraLocation.X, AxisZ), 0.001);
				}
			}
		});

		It("Distribute Actors: Depth", [this]
		{
			const int32 FrontmostActorIndex = TestUtils.GetLowestAxisActorForCamera(ActorLocations, CameraLocation, AxisX);
			const int32 BackmostActorIndex = TestUtils.GetHighestAxisActorForCamera(ActorLocations, CameraLocation, AxisX);
			const double ExpectedAxisXValue = (ActorLocations[FrontmostActorIndex].X + ActorLocations[BackmostActorIndex].X) / 2;

			FAvaScreenAlignmentUtils::DistributeActorsDepth(Camera.ToSharedRef(), Actors, EAvaAlignmentSizeMode::Self, EAvaActorDistributionMode::CenterDistance);
			TArray<FVector> NewActorLocations = TestUtils.GetActorLocations(Actors);

			for (int32 LocationIndex = 0; LocationIndex < InitialActorLocations.Num(); LocationIndex++)
			{
				TestEqual("Y axis was not affected", NewActorLocations[LocationIndex].Y, InitialActorLocations[LocationIndex].Y);
				TestEqual("Z axis was not affected", NewActorLocations[LocationIndex].Z, InitialActorLocations[LocationIndex].Z);
			}

			for (int32 LocationIndex = 0; LocationIndex < InitialActorLocations.Num(); LocationIndex++)
			{
				if (LocationIndex == FrontmostActorIndex || LocationIndex == BackmostActorIndex)
				{
					TestEqual("X value didn't change for edge actor", ActorLocations[LocationIndex].X, NewActorLocations[LocationIndex].X);
				}
				else
				{
					TestEqual("X value calculated correctly for the actor in the middle", NewActorLocations[LocationIndex].X, ExpectedAxisXValue);
				}
			}
		});
	});
}
