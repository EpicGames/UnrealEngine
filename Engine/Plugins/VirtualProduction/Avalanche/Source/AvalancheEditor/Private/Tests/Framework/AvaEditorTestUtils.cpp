// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Framework/AvaEditorTestUtils.h"

#include "Logging/LogVerbosity.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Math/Vector.h"
#include "Math/Rotator.h"
#include "Misc/AutomationTest.h"

// General Motion Design Editor unit test log
DEFINE_LOG_CATEGORY(LogAvaEditorTest);

TArray<FVector> FAvaEditorTestUtils::GetActorLocations(const TArray<AActor*>& InActors)
{
	TArray<FVector> ActorLocations;
	ActorLocations.Reserve(InActors.Num());

	for (const AActor* Actor : InActors)
	{
		if (IsValid(Actor))
		{
			ActorLocations.Add(Actor->GetActorLocation());
		}
	}

	return ActorLocations;
}

void FAvaEditorTestUtils::SetActorLocations(const TArray<AActor*>& InActors, const TArray<FVector>& InLocations)
{
	const int32 ActorCount = InActors.Num();
	const int32 LocationCount = InLocations.Num();

	if (ActorCount <= LocationCount)
	{
		for (int32 ActorIdx = 0; ActorIdx < ActorCount; ActorIdx++)
		{
			InActors[ActorIdx]->SetActorLocation(InLocations[ActorIdx]);
		}
	}
	else
	{
		UE_LOGF(LogAvaEditorTest, Error, "Number of locations %d exceed number of given actors %d", LocationCount, ActorCount);
	}
}

TArray<FVector> FAvaEditorTestUtils::GenerateRandomVectors(int32 InVectorCount)
{
	TArray<FVector> ResultVectors;
	ResultVectors.Reserve(InVectorCount);

	for (int32 VectorCount = 0; VectorCount < InVectorCount; VectorCount++)
	{
		ResultVectors.Add({GetRandomValue(), GetRandomValue(), GetRandomValue()});
	}

	return ResultVectors;
}

int32 FAvaEditorTestUtils::GetLowestAxisActorForCamera(const TArray<FVector>& InActorLocations, const FVector& InCameraLocation, int32 InAlignmentAxis)
{
	int32 LowestActorIndex = GetLowestAxisActor(InActorLocations, InAlignmentAxis);

	// First: we define lowest position Horizontaly
	const double LowestAxisValue = InActorLocations[LowestActorIndex][InAlignmentAxis];

	// Second: We define lowest axis position according to camera location
	static constexpr bool bIsLowestAxisValue = true;

	if (LowestAxisValue < 0)
	{
		LowestActorIndex = GetHighestTangentAngleActorIndex(InActorLocations, InCameraLocation, LowestActorIndex, bIsLowestAxisValue, InAlignmentAxis);
	}
	else
	{
		// Otherwise searching for the smallest TangentAngle value above camera position
		if (LowestAxisValue != 0)
		{
			LowestActorIndex = GetLowestTangentAngleActorIndex(InActorLocations, InCameraLocation, LowestActorIndex, bIsLowestAxisValue, InAlignmentAxis);
		}
	}
	return LowestActorIndex;
}

int32 FAvaEditorTestUtils::GetHighestAxisActorForCamera(const TArray<FVector>& InActorLocations, const FVector& InCameraLocation, int32 InAlignmentAxis)
{
	int32 HighestAxisActorIndex = GetHighestAxisActor(InActorLocations, InAlignmentAxis);

	// First: we define highest axis position
	const double HighestYValue = InActorLocations[HighestAxisActorIndex][InAlignmentAxis];

	// Second: We define highest position according to camera location
	static constexpr bool bIsLowestAxisValue = false;

	if (HighestYValue > 0)
	{
		HighestAxisActorIndex = GetHighestTangentAngleActorIndex(InActorLocations, InCameraLocation, HighestAxisActorIndex, bIsLowestAxisValue, InAlignmentAxis);
	}
	else
	{
		// Otherwise searching for the smallest TangentAngle value below camera
		if (HighestYValue != 0)
		{
			HighestAxisActorIndex = GetLowestTangentAngleActorIndex(InActorLocations, InCameraLocation, HighestAxisActorIndex, bIsLowestAxisValue, InAlignmentAxis);
		}
	}

	return HighestAxisActorIndex;
}

int32 FAvaEditorTestUtils::GetLowestAxisActor(const TArray<FVector>& InActorLocations, int32 InAlignmentAxis)
{
	int32 LowestActorIndex = 0;

	if (!ensureMsgf(!InActorLocations.IsEmpty(), TEXT("No actor locations passed in.")))
	{
		return LowestActorIndex;
	}

	double LowestAxisValue = InActorLocations[0][InAlignmentAxis];

	for (int32 LocationIndex = 0; LocationIndex < InActorLocations.Num(); LocationIndex++)
	{
		if (InActorLocations[LocationIndex][InAlignmentAxis] < LowestAxisValue)
		{
			LowestAxisValue = InActorLocations[LocationIndex][InAlignmentAxis];
			LowestActorIndex = LocationIndex;
		}
	}

	return LowestActorIndex;
}

int32 FAvaEditorTestUtils::GetHighestAxisActor(const TArray<FVector>& InActorLocations, int32 InAlignmentAxis)
{
	int32 HighestAxisActorIndex = 0;

	if (!ensureMsgf(!InActorLocations.IsEmpty(), TEXT("No actor locations passed in.")))
	{
		return HighestAxisActorIndex;
	}

	double HighestYValue = InActorLocations[0][InAlignmentAxis];

	// First: we define highest axis position
	for (int32 LocationIndex = 0; LocationIndex < InActorLocations.Num(); LocationIndex++)
	{
		if (InActorLocations[LocationIndex][InAlignmentAxis] > HighestYValue)
		{
			HighestYValue = InActorLocations[LocationIndex][InAlignmentAxis];
			HighestAxisActorIndex = LocationIndex;
		}
	}

	return HighestAxisActorIndex;
}

FVector FAvaEditorTestUtils::GetCenterAnchorLocation(const TArray<FVector>& InActorLocations, const FVector& InCameraLocation, int32 InAlignmentAxis)
{
	// Anchor location is needed for further calculations of a new expected centered location
	const int32 LowestActorIndex = GetLowestAxisActorForCamera(InActorLocations, InCameraLocation, InAlignmentAxis);
	const int32 HighestActorIndex = GetHighestAxisActorForCamera(InActorLocations, InCameraLocation, InAlignmentAxis);

	const double LowestTangentAngle = CalculateTangentAngle(InCameraLocation.X, InActorLocations[LowestActorIndex], InAlignmentAxis);
	const double LowestEdgeXValue = InCameraLocation.X - InActorLocations[HighestActorIndex].X;
	const double CorrespondingEdgeAxisBelow = LowestTangentAngle * LowestEdgeXValue;
	const double CenterValue = (CorrespondingEdgeAxisBelow + InActorLocations[HighestActorIndex][InAlignmentAxis]) / 2;
	FVector ResultLocation;

	switch (InAlignmentAxis)
	{
	// Y axis - Horizontal alignment
	case 1: ResultLocation = FVector(InActorLocations[HighestActorIndex].X, CenterValue, InActorLocations[HighestActorIndex].Z);
		break;

	// Z axis - Vertical alignment
	case 2: ResultLocation = FVector(InActorLocations[HighestActorIndex].X, InActorLocations[HighestActorIndex].Y, CenterValue);
		break;

	// Invalid
	default: checkNoEntry()
		break;
	}

	return ResultLocation;
}

double FAvaEditorTestUtils::GetExpectedAxisValue(const FVector& InAnchorActorLocation, const FVector& InTargetActorLocation, double InCameraLocationX, int32 InAlignmentAxis)
{
	// Using Right-angled triangle formula
	const double AnchorAdjacentX = InCameraLocationX - InAnchorActorLocation.X;
	const double AnchorOppositeValue = InAnchorActorLocation[InAlignmentAxis];
	const double ResultAdjacentX = InCameraLocationX - InTargetActorLocation.X;
	const double ResultOppositeValue = (AnchorOppositeValue / AnchorAdjacentX) * ResultAdjacentX;

	return ResultOppositeValue;
}

double FAvaEditorTestUtils::GetRandomValue(int32 InMinValue, int32 InMaxValue)
{
	return FMath::RandRange(InMinValue, InMaxValue);
}

double FAvaEditorTestUtils::CalculateTangentAngle(double InCameraDepthLocation, const FVector& InActorLocation, int32 InAlignmentAxis)
{
	// Using Right-angled triangle formula
	const double AdjacentX = InCameraDepthLocation - InActorLocation.X;
	const double OppositeValue = InActorLocation[InAlignmentAxis];
	const double TangentAngle = OppositeValue / AdjacentX;

	return TangentAngle;
}

int32 FAvaEditorTestUtils::GetHighestTangentAngleActorIndex(const TArray<FVector>& InActorLocations, const FVector& InCameraLocation, int32 InEdgeActorIndex, bool bInIsLowestAxisValue, int32 InAlignmentAxis)
{
	int32 ResultIndex = InEdgeActorIndex;
	double BiggestTangentAngle = CalculateTangentAngle(InCameraLocation.X, InActorLocations[InEdgeActorIndex], InAlignmentAxis);
	bool bSteerHighOrLowValue;

	// Searching for the biggest TangentAngle value
	for (int32 LocationIndex = 0; LocationIndex < InActorLocations.Num(); LocationIndex++)
	{
		if (bInIsLowestAxisValue)
		{
			// for the lowest axis value
			bSteerHighOrLowValue = InActorLocations[LocationIndex][InAlignmentAxis] < 0;
		}
		else
		{
			// for the highest axis value
			bSteerHighOrLowValue = InActorLocations[LocationIndex][InAlignmentAxis] > 0;
		}

		const double TangentAngle = CalculateTangentAngle(InCameraLocation.X, InActorLocations[LocationIndex], InAlignmentAxis);

		if (bSteerHighOrLowValue && FMath::Abs(TangentAngle) >= FMath::Abs(BiggestTangentAngle))
		{
			BiggestTangentAngle = TangentAngle;
			ResultIndex = LocationIndex;
		}
	}

	return ResultIndex;
}

int32 FAvaEditorTestUtils::GetLowestTangentAngleActorIndex(const TArray<FVector>& InActorLocations, const FVector& InCameraLocation, int32 InEdgeActorIndex, bool bInIsLowestAxisValue, int32 InAlignmentAxis)
{
	int32 ResultIndex = InEdgeActorIndex;
	double SmallestTangentAngle = CalculateTangentAngle(InCameraLocation.X, InActorLocations[InEdgeActorIndex], InAlignmentAxis);
	bool bSteerHighOrLowValue;

	// Searching for the smallest TangentAngle value
	for (int32 LocationIndex = 0; LocationIndex < InActorLocations.Num(); LocationIndex++)
	{
		if (bInIsLowestAxisValue)
		{
			// for the lowest axis value
			bSteerHighOrLowValue = InActorLocations[LocationIndex][InAlignmentAxis] > 0;
		}
		else
		{
			// for the highest axis value
			bSteerHighOrLowValue = InActorLocations[LocationIndex][InAlignmentAxis] < 0;
		}

		const double TangentAngle = CalculateTangentAngle(InCameraLocation.X, InActorLocations[LocationIndex], InAlignmentAxis);

		if (bSteerHighOrLowValue && FMath::Abs(TangentAngle) <= FMath::Abs(SmallestTangentAngle))
		{
			SmallestTangentAngle = TangentAngle;
			ResultIndex = LocationIndex;
		}
	}

	return ResultIndex;
}
