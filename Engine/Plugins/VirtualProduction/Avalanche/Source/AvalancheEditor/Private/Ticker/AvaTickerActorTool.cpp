// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTickerActorTool.h"
#include "AvaEditorCommands.h"
#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "Framework/Ticker/AvaTickerActor.h"

UAvaTickerActorTool::UAvaTickerActorTool()
{
	ActorClass = AAvaTickerActor::StaticClass();
}

void UAvaTickerActorTool::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	FAvaInteractiveToolsToolParameters ToolParameters = CreateDefaultToolParameters();
	ToolParameters.UICommand = FAvaEditorCommands::Get().TickerTool;

	InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryNameActor, MoveTemp(ToolParameters));
}
