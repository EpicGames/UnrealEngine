// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionExtension.h"
#include "AvaSequenceDefaultTags.h"
#include "AvaTagCollection.h"
#include "AvaTransitionSubsystem.h"
#include "AvaTransitionTree.h"
#include "AvaTransitionTreeEditorData.h"
#include "AvaWorldSubsystemUtils.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "Conditions/AvaTransitionSceneMatchCondition.h"
#include "Conditions/AvaTransitionTypeMatchCondition.h"
#include "DetailCategoryBuilder.h"
#include "Editor.h"
#include "Framework/AvaNullActor.h"
#include "IAvaTransitionEditorModule.h"
#include "IRemoteControlUIModule.h"
#include "Modifiers/ActorModifierCoreBase.h"
#include "Modifiers/Utilities/ActorModifierCoreLibrary.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Settings/AvaSequencerSettings.h"
#include "Settings/AvaTransitionEditorSettings.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Tasks/AvaTransitionWaitForLayerTask.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Transition/AvaTransitionPlaySequenceTask.h"

#define LOCTEXT_NAMESPACE "AvaTransitionExtension"

namespace UE::AvaEditor::Private
{
	void AddSceneTypeCondition(UStateTreeState& InState, EAvaTransitionComparisonResult InComparisonType)
	{
		TStateTreeEditorNode<FAvaTransitionSceneMatchCondition>& ConditionNode = InState.AddEnterCondition<FAvaTransitionSceneMatchCondition>();

		FAvaTransitionSceneMatchCondition::FInstanceDataType& Condition = ConditionNode.GetInstanceData();

		Condition.SceneComparisonType  = InComparisonType;
		ConditionNode.ExpressionOperand = EStateTreeExpressionOperand::Or;
	}

	void AddTransitionCondition(UStateTreeState& InState, EAvaTransitionType InTransitionType)
	{
		FAvaTransitionTypeMatchCondition::FInstanceDataType& Condition = InState.AddEnterCondition<FAvaTransitionTypeMatchCondition>().GetInstanceData();
		Condition.TransitionType = InTransitionType;
	}

	void SetStateCompletedResult(UStateTreeState& InState, EStateTreeTransitionType InStateTransitionType)
	{
		InState.Transitions.Empty(1);
		InState.AddTransition(EStateTreeTransitionTrigger::OnStateCompleted, InStateTransitionType);
	}

	FAvaSequencePlayParams& AddPlayTask(UStateTreeState& InState, const FAvaTagHandle& InSequenceTag)
	{
		FAvaTransitionPlaySequenceTask::FInstanceDataType& PlayTask = InState.AddTask<FAvaTransitionPlaySequenceTask>().GetInstanceData();

		PlayTask.QueryType   = EAvaTransitionSequenceQueryType::Tag;
		PlayTask.SequenceTag = InSequenceTag;
		PlayTask.WaitType    = EAvaTransitionSequenceWaitType::WaitUntilStop;

		return PlayTask.PlaySettings;
	}
}

FDelegateHandle FAvaTransitionExtension::PropertyFilterHandle;

FAvaTransitionRundownExtension FAvaTransitionExtension::RundownExtension;

void FAvaTransitionExtension::StaticStartup()
{
	RundownExtension.Startup();

	IRemoteControlUIModule& RemoteControlUIModule = IRemoteControlUIModule::Get();

	PropertyFilterHandle = RemoteControlUIModule.AddPropertyFilter(FOnDisplayExposeIcon::CreateLambda(
		[](const FRCExposesPropertyArgs& InPropertyArgs)->bool
		{
			if (InPropertyArgs.PropertyHandle.IsValid())
			{
				if (const UClass* BaseClass = InPropertyArgs.PropertyHandle->GetOuterBaseClass())
				{
					// filter out any property coming from State Tree
					return !BaseClass->IsChildOf<UStateTreeEditorData>()
						&& !BaseClass->IsChildOf<UStateTreeState>();
				}
			}
			return true;
		}));
}

void FAvaTransitionExtension::StaticShutdown()
{
	RundownExtension.Shutdown();

	if (IRemoteControlUIModule* RemoteControlUIModule = FModuleManager::Get().GetModulePtr<IRemoteControlUIModule>(TEXT("RemoteControlUI")))
	{
		RemoteControlUIModule->RemovePropertyFilter(PropertyFilterHandle);
		PropertyFilterHandle.Reset();
	}
}

void FAvaTransitionExtension::Construct(const TSharedRef<IAvaEditor>& InEditor)
{
	FAvaEditorExtension::Construct(InEditor);

	IAvaTransitionEditorModule::Get().GetOnBuildDefaultTransitionTree()
		.BindSP(this, &FAvaTransitionExtension::BuildDefaultTransitionTree);
}

void FAvaTransitionExtension::Activate()
{
	if (UWorld* const World = GetWorld())
	{
		if (UAvaTransitionSubsystem* TransitionSubsystem = World->GetSubsystem<UAvaTransitionSubsystem>())
		{
			// Ensure that the Transition Behavior is created for the Persistent Level
			TransitionSubsystem->GetOrCreateTransitionBehavior();
		}
	}
}

void FAvaTransitionExtension::Deactivate()
{
	CloseTransitionEditor();
}

FName FAvaTransitionExtension::GetCategoryName() const
{
	return TEXT("TransitionLogic");
}

void FAvaTransitionExtension::ExtendSettingsCategory(IDetailCategoryBuilder& InCategoryBuilder)
{
	if (IAvaTransitionBehavior* TransitionBehavior = GetTransitionBehavior())
	{
		const TArray<UObject*> TransitionBehaviorObjects = { &TransitionBehavior->AsUObject() };

		InCategoryBuilder.SetDisplayName(LOCTEXT("CategoryDisplayName", "Transition Logic"));

		TransitionBehavior->ForEachDetailsEditableProperty(
			[&InCategoryBuilder, &TransitionBehaviorObjects](const IAvaTransitionBehavior::FPropertyContext& InPropertyContext)
			{
				InCategoryBuilder.AddExternalObjectProperty(TransitionBehaviorObjects, InPropertyContext.Name);
			});
	}
}

void FAvaTransitionExtension::OnSceneObjectCreated(UObject* InSceneObject)
{
	const UAvaTransitionEditorSettings* TransitionEditorSettings = GetDefault<UAvaTransitionEditorSettings>();
	if (!TransitionEditorSettings || !TransitionEditorSettings->ShouldCreateTransitionLogicDefaultScene())
	{
		return;
	}

	UWorld* const World = GetWorld();
	if (!World)
	{
		return;
	}

	UAvaTransitionSubsystem* const TransitionSubsystem = World->GetSubsystem<UAvaTransitionSubsystem>();
	if (!TransitionSubsystem)
	{
		return;
	}

	IAvaTransitionBehavior* const TransitionBehavior = TransitionSubsystem->GetOrCreateTransitionBehavior();
	if (!TransitionBehavior)
	{
		return;
	}

	// Enable TL if creating the TL default scene
	TransitionBehavior->SetEnabled(true);
	TransitionBehavior->SetInstancingMode(EAvaTransitionInstancingMode::Reuse);

	// Spawn Layer 1 Content null
	FActorSpawnParameters SpawnParameters;
	SpawnParameters.InitialActorLabel = TEXT("Layer 1 Content");
	AAvaNullActor* const Layer1Actor = World->SpawnActor<AAvaNullActor>(SpawnParameters);

	// Add the Sub Layer Modifier (BP) to this Layer 1 Content null
	UActorModifierCoreStack* ModifierStack;
	if (UActorModifierCoreLibrary::FindModifierStack(Layer1Actor, ModifierStack, /*bCreateIfNone*/true))
	{
		constexpr const TCHAR* SubLayerModifierClassPath = TEXT("/Avalanche/TransitionLogic/SubLayer/BP_SubLayerModifier.BP_SubLayerModifier_C");

		FActorModifierCoreInsertOperation InsertOperation;
		InsertOperation.ModifierClass = StaticLoadClass(UActorModifierCoreBase::StaticClass(), nullptr, SubLayerModifierClassPath);

		UActorModifierCoreBase* Modifier;
		if (UActorModifierCoreLibrary::InsertModifier(ModifierStack, InsertOperation, Modifier))
		{
			auto SetSequenceName =
				[Modifier, ModifierClass = InsertOperation.ModifierClass](FName InPropertyName, FAvaSequenceName InSequenceName)
				{
					FStructProperty* const StructProperty = CastField<FStructProperty>(ModifierClass->FindPropertyByName(InPropertyName));
					if (StructProperty && StructProperty->Struct == FAvaSequenceName::StaticStruct())
					{
						StructProperty->SetValue_InContainer(Modifier, &InSequenceName);
					}
				};

			SetSequenceName(TEXT("Change In Sequence"), FAvaSequenceName(UAvaSequencerSettings::DefaultChangeInName));
			SetSequenceName(TEXT("Change Out Sequence"), FAvaSequenceName(UAvaSequencerSettings::DefaultChangeOutName));
		}
	}
}

void FAvaTransitionExtension::ExtendToolbarMenu(UToolMenu& InMenu)
{
	FToolMenuSection& Section = InMenu.FindOrAddSection(DefaultSectionName);

	FSlateIcon TransitionTreeIcon = FSlateIconFinder::FindCustomIconForClass(UAvaTransitionTree::StaticClass(), TEXT("ClassThumbnail"));

	FToolMenuEntry& Entry = Section.AddEntry(FToolMenuEntry::InitToolBarButton(TEXT("TransitionLogicButton")
		, FExecuteAction::CreateSP(this, &FAvaTransitionExtension::OpenTransitionEditor)
		, LOCTEXT("TransitionLogicLabel", "Transition Logic")
		, LOCTEXT("TransitionLogicTooltip", "Opens the Transition Logic Editor for the given Scene")
		, TransitionTreeIcon));

	Entry.StyleNameOverride = TEXT("CalloutToolbar");

	Section.AddEntry(FToolMenuEntry::InitComboButton(TEXT("TransitionLogicComboButton")
		, FUIAction()
		, FNewToolMenuDelegate::CreateSP(this, &FAvaTransitionExtension::GenerateTransitionLogicOptions)
		, LOCTEXT("TransitionLogicOptionsLabel", "Transition Logic Options")
		, LOCTEXT("TransitionLogicOptionsTooltip", "Transition Logic Options")
		, TransitionTreeIcon
		, /*bSimpleComboBox*/true));
}

void FAvaTransitionExtension::ExtendCreateSceneOptions(UToolMenu* InMenu)
{
	auto ToggleCreateTLDefaultScene = []()->void
		{
			if (UAvaTransitionEditorSettings* TransitionEditorSettings = GetMutableDefault<UAvaTransitionEditorSettings>())
			{
				FScopedTransaction Transaction(LOCTEXT("ToggleTransitionTreeEnabled", "Toggle Transition Tree Enabled"));
				TransitionEditorSettings->Modify();
				TransitionEditorSettings->ToggleCreateTransitionLogicDefaultScene();
				TransitionEditorSettings->SaveConfig();
			}
		};

	auto ShouldCreateTLDefaultScene = []()->bool
		{
			const UAvaTransitionEditorSettings* TransitionEditorSettings = GetDefault<UAvaTransitionEditorSettings>();
			return TransitionEditorSettings && TransitionEditorSettings->ShouldCreateTransitionLogicDefaultScene();
		};

	FToolMenuSection& Section = InMenu->FindOrAddSection(DefaultSectionName);

	Section.AddMenuEntry(TEXT("CreateTransitionLogicDefaultScene")
		, LOCTEXT("CreateTransitionLogicDefaultSceneLabel", "Create Transition Logic Default Scene")
		, LOCTEXT("CreateTransitionLogicDefaultSceneTooltip", "Determines whether to create the default setup for Transition Logic when creating the Motion Design Scene")
		, TAttribute<FSlateIcon>()
		, FUIAction(FExecuteAction::CreateLambda(ToggleCreateTLDefaultScene)
			, FCanExecuteAction()
			, FIsActionChecked::CreateLambda(ShouldCreateTLDefaultScene))
		, EUserInterfaceActionType::ToggleButton);
}

IAvaTransitionBehavior* FAvaTransitionExtension::GetTransitionBehavior() const
{
	if (UAvaTransitionSubsystem* TransitionSubsystem = FAvaWorldSubsystemUtils::GetWorldSubsystem<UAvaTransitionSubsystem>(this))
	{
		return TransitionSubsystem->GetTransitionBehavior();
	}
	return nullptr;
}

void FAvaTransitionExtension::BuildDefaultTransitionTree(UAvaTransitionTreeEditorData& InTreeEditorData)
{
	using namespace UE::AvaEditor::Private;

	UStateTreeState& RootState = InTreeEditorData.AddRootState();
	SetStateCompletedResult(RootState, EStateTreeTransitionType::Succeeded);

	const FAvaSequenceDefaultTags& DefaultTags = FAvaSequenceDefaultTags::Get();

	// Same scene transition
	{
		UStateTreeState& State = RootState.AddChildState(TEXT("Same scene transition"));
		AddSceneTypeCondition(State, EAvaTransitionComparisonResult::Same);

		const FAvaTagHandle ChangeTag   = DefaultTags.Change.MakeTagHandle();
		const FAvaSequenceTime MarkTime = FAvaSequenceTime(TEXT("A"));

		// Change out (0 to A)
		{
			UStateTreeState& ChangeOut = State.AddChildState(TEXT("Change Out"));
			AddTransitionCondition(ChangeOut , EAvaTransitionType::Out);
			SetStateCompletedResult(ChangeOut, EStateTreeTransitionType::Succeeded);

			FAvaSequencePlayParams& PlaySettings = AddPlayTask(ChangeOut, ChangeTag);
			PlaySettings.Start.bHasTimeConstraint = false;
            PlaySettings.End = MarkTime;
		}

		// Change in (A to End)
		{
			UStateTreeState& ChangeIn = State.AddChildState(TEXT("Change In"));
			AddTransitionCondition(ChangeIn , EAvaTransitionType::In);

			UStateTreeState& WaitState = ChangeIn.AddChildState(TEXT("Wait for change out"));
			SetStateCompletedResult(WaitState, EStateTreeTransitionType::NextSelectableState);

			FAvaTransitionWaitForLayerTask::FInstanceDataType& WaitTask = WaitState.AddTask<FAvaTransitionWaitForLayerTask>().GetInstanceData();
			WaitTask.LayerType = EAvaTransitionLayerCompareType::Same;

			UStateTreeState& PlayChangeInState = ChangeIn.AddChildState(TEXT("Play change in"));
			SetStateCompletedResult(PlayChangeInState, EStateTreeTransitionType::Succeeded);

			FAvaSequencePlayParams& PlaySettings = AddPlayTask(PlayChangeInState, ChangeTag);
			PlaySettings.Start = MarkTime;
			PlaySettings.End.bHasTimeConstraint = false;
		}
	}

	// Different or no scene transition
	{
		UStateTreeState& State = RootState.AddChildState(TEXT("Different or no scene transition"));
		AddSceneTypeCondition(State, EAvaTransitionComparisonResult::Different);
		AddSceneTypeCondition(State, EAvaTransitionComparisonResult::None);

		// Out
		UStateTreeState& Out = State.AddChildState(TEXT("Out"));
		AddTransitionCondition(Out , EAvaTransitionType::Out);
		SetStateCompletedResult(Out, EStateTreeTransitionType::Succeeded);
		AddPlayTask(Out, DefaultTags.Out.MakeTagHandle());

		// In
		UStateTreeState& In = State.AddChildState(TEXT("In"));
		AddTransitionCondition(In , EAvaTransitionType::In);
		SetStateCompletedResult(In, EStateTreeTransitionType::Succeeded);
		AddPlayTask(In, DefaultTags.In.MakeTagHandle());
	}
}

void FAvaTransitionExtension::OpenTransitionEditor()
{
	if (const IAvaTransitionBehavior* TransitionBehavior = GetTransitionBehavior())
	{
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		check(AssetEditorSubsystem);
		AssetEditorSubsystem->OpenEditorForAsset(TransitionBehavior->GetTransitionTree());
	}
}

void FAvaTransitionExtension::CloseTransitionEditor()
{
	if (const IAvaTransitionBehavior* TransitionBehavior = GetTransitionBehavior())
	{
		if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
		{
			AssetEditorSubsystem->CloseAllEditorsForAsset(TransitionBehavior->GetTransitionTree());
		}
	}
}

void FAvaTransitionExtension::GenerateTransitionLogicOptions(UToolMenu* InMenu)
{
	if (IAvaTransitionEditorModule::IsLoaded())
	{
		IAvaTransitionEditorModule::Get().GenerateTransitionTreeOptionsMenu(InMenu, GetTransitionBehavior());
	}
}

#undef LOCTEXT_NAMESPACE
