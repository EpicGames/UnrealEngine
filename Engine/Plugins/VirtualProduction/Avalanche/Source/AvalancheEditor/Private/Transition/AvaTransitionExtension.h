// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionRundownExtension.h"
#include "Delegates/Delegate.h"
#include "IAvaEditorExtension.h"

class IAvaTransitionBehavior;
class UAvaTransitionTreeEditorData;

class FAvaTransitionExtension : public FAvaEditorExtension
{
public:
	UE_AVA_INHERITS(FAvaTransitionExtension, FAvaEditorExtension);

	static void StaticStartup();
	static void StaticShutdown();

	//~ Begin IAvaEditorExtension
	virtual void Construct(const TSharedRef<IAvaEditor>& InEditor) override;
	virtual void Activate() override;
	virtual void Deactivate() override;
	virtual FName GetCategoryName() const override;
	virtual void ExtendSettingsCategory(IDetailCategoryBuilder& InCategoryBuilder) override;
	virtual void OnSceneObjectCreated(UObject* InSceneObject) override;
	virtual void ExtendToolbarMenu(UToolMenu& InMenu) override;
	virtual void ExtendCreateSceneOptions(UToolMenu* InMenu) override;
	//~ End IAvaEditorExtension

private:
	IAvaTransitionBehavior* GetTransitionBehavior() const;

	void BuildDefaultTransitionTree(UAvaTransitionTreeEditorData& InTreeEditorData);

	void OpenTransitionEditor();

	void CloseTransitionEditor();

	void GenerateTransitionLogicOptions(UToolMenu* InMenu);

	static FDelegateHandle PropertyFilterHandle;

	static FAvaTransitionRundownExtension RundownExtension;
};
