// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionRundownExtension.h"
#include "AvaTransitionLog.h"
#include "AvaTransitionSubsystem.h"
#include "AvaTransitionTree.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "IAvaMediaEditorModule.h"
#include "Playback/AvaPlaybackUtils.h"
#include "Rundown/AvaRundownEditor.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/Pages/AvaRundownPageContext.h"
#include "Styling/SlateIconFinder.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AvaTransitionRundownExtension"

FAvaTransitionRundownExtension::FAvaTransitionRundownExtension()
	: ExtensionSectionName(TEXT("AvaTransitionRundownExtension"))
{
}

FAvaTransitionRundownExtension::~FAvaTransitionRundownExtension()
{
	Shutdown();
}

void FAvaTransitionRundownExtension::Startup()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	check(ToolMenus);

	UToolMenu* PageContextMenu = ToolMenus->ExtendMenu(IAvaMediaEditorModule::GetRundownPageMenuName());
	check(PageContextMenu);

	PageContextMenuWeak = PageContextMenu;

	PageContextMenu->AddDynamicSection(ExtensionSectionName
		, FNewToolMenuDelegate::CreateRaw(this, &FAvaTransitionRundownExtension::ExtendPageContextMenu)
		, FToolMenuInsert(TEXT("PageListOperations"), EToolMenuInsertType::After));
}

void FAvaTransitionRundownExtension::Shutdown()
{
	if (UToolMenu* ToolMenu = PageContextMenuWeak.Get())
	{
		ToolMenu->RemoveSection(ExtensionSectionName);
		PageContextMenuWeak = nullptr;
	}
}

void FAvaTransitionRundownExtension::ExtendPageContextMenu(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	UAvaRundownPageContext* PageContext = InMenu->FindContext<UAvaRundownPageContext>();
	if (!PageContext)
	{
		return;
	}

	TWeakPtr<FAvaRundownEditor> PlaylistEditorWeak = PageContext->GetRundownEditor();
	if (!PlaylistEditorWeak.IsValid())
	{
		return;
	}

	FToolMenuSection& Section = InMenu->AddSection(TEXT("TransitionLogicSection"), LOCTEXT("TransitionLogicSectionLabel", "Transition Logic"));
	Section.AddMenuEntry(TEXT("OpenTransitionLogic")
		, LOCTEXT("OpenTransitionLogicLabel", "Open Transition Logic")
		, LOCTEXT("OpenTransitionLogicTooltip", "Opens the Transition Logic Editor for the selected pages")
		, FSlateIconFinder::FindIconForClass(UAvaTransitionTree::StaticClass())
		, FExecuteAction::CreateRaw(this, &FAvaTransitionRundownExtension::OpenTransitionTree, PlaylistEditorWeak));
}

void FAvaTransitionRundownExtension::OpenTransitionTree(TWeakPtr<FAvaRundownEditor> InPlaylistEditorWeak)
{
	TSharedPtr<FAvaRundownEditor> PlaylistEditor = InPlaylistEditorWeak.Pin();
	if (!PlaylistEditor.IsValid())
	{
		UE_LOGF(LogAvaTransition, Warning, "Unable to open Transition Logic. Rundown Editor is invalid");
		return;
	}

	UAvaRundown* Playlist = PlaylistEditor->GetRundown();
	if (!Playlist)
	{
		UE_LOGF(LogAvaTransition, Warning, "Unable to open Transition Logic. Rundown is invalid");
		return;
	}

	TConstArrayView<int32> SelectedPageIds = PlaylistEditor->GetSelectedPagesOnFocusedWidget();
	if (SelectedPageIds.IsEmpty())
	{
		UE_LOGF(LogAvaTransition, Log, "Rundown '%ls' did not have selected pages on focused widget.", *Playlist->GetName());
		return;
	}

	const FString PlaylistName = Playlist->GetName();

	for (int32 PageId : SelectedPageIds)
	{
		const FAvaRundownPage& Page = Playlist->GetPage(PageId);
		if (!Page.IsValidPage())
		{
			return;
		}

		TArray<FSoftObjectPath> PageAssetPaths = Page.GetAssetPaths(Playlist);
		for (const FSoftObjectPath& PageAssetPath : PageAssetPaths)
		{
			OpenTransitionTree(PageAssetPath, *PlaylistName, PageId);
		}
	}
}

void FAvaTransitionRundownExtension::OpenTransitionTree(const FSoftObjectPath& InPageAssetPath, const TCHAR* InPlaylistName, int32 InPageId)
{
	if (!FAvaPlaybackUtils::IsMapAsset(InPageAssetPath.GetLongPackageName()))
	{
		UE_LOGF(LogAvaTransition, Warning
			, "Page asset path '%ls' (Page Id: '%d', Rundown '%ls) is not a map asset and will not be processed"
			, *InPageAssetPath.ToString()
			, InPageId
			, InPlaylistName);
		return;
	}

	UWorld* const World = Cast<UWorld>(InPageAssetPath.TryLoad());
	if (!World)
	{
		UE_LOGF(LogAvaTransition, Warning
			, "World path '%ls' (Page Id: '%d', Rundown '%ls) did not load a valid world and will not be processed"
			, *InPageAssetPath.ToString()
			, InPageId
			, InPlaylistName);
		return;
	}

	IAvaTransitionBehavior* TransitionBehavior = UAvaTransitionSubsystem::FindTransitionBehavior(World->PersistentLevel);
	if (!TransitionBehavior)
	{
		UE_LOGF(LogAvaTransition, Warning
			, "World '%ls' (Page Id: '%d', Rundown '%ls) does not have a valid Transition Behavior"
			, *InPageAssetPath.ToString()
			, InPageId
			, InPlaylistName);
		return;
	}

	UAvaTransitionTree* TransitionTree = TransitionBehavior->GetTransitionTree();
	if (!TransitionTree)
	{
		UE_LOGF(LogAvaTransition, Warning
			, "Transition Behavior of World '%ls' (Page Id: '%d', Rundown '%ls) did not have a valid Transition Tree"
			, *InPageAssetPath.ToString()
			, InPageId
			, InPlaylistName);
		return;
	}

	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	check(AssetEditorSubsystem);

	AssetEditorSubsystem->OpenEditorForAsset(TransitionTree
		, EToolkitMode::Standalone
		, TSharedPtr<IToolkitHost>()
		, /*bShowProgressWindow*/true
		, EAssetTypeActivationOpenedMethod::View);
}

#undef LOCTEXT_NAMESPACE
