// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"

class FAvaRundownEditor;
class UToolMenu;
class UAvaRundown;

class FAvaTransitionRundownExtension
{
public:
	FAvaTransitionRundownExtension();

	~FAvaTransitionRundownExtension();

	void Startup();

	void Shutdown();

private:
	void ExtendPageContextMenu(UToolMenu* InMenu);

	void OpenTransitionTree(TWeakPtr<FAvaRundownEditor> InPlaylistEditorWeak);

	void OpenTransitionTree(const FSoftObjectPath& InPageAssetPath, const TCHAR* InPlaylistName, int32 InPageId);

	TWeakObjectPtr<UToolMenu> PageContextMenuWeak;

	const FName ExtensionSectionName;
};
