// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaLevelViewportExtension.h"
#include "AvaEditorCommands.h"
#include "AvaEditorSettings.h"
#include "AvaLevelViewportCommands.h"
#include "AvaLevelViewportLayoutEntity.h"
#include "AvaViewportUtils.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorModeManager.h"
#include "Elements/Actor/ActorElementEditorViewportInteractionCustomization.h"
#include "ISequencer.h"
#include "LevelEditor.h"
#include "LevelEditor/AvaLevelEditorUtils.h"
#include "LevelViewportActions.h"
#include "ScopedTransaction.h"
#include "SEditorViewport.h"
#include "Selection/AvaEditorSelection.h"
#include "Sequencer/AvaSequencerExtension.h"
#include "SLevelViewport.h"
#include "UnrealEdGlobals.h"
#include "ViewportClient/IAvaViewportClient.h"

#define LOCTEXT_NAMESPACE "AvaLevelViewportExtension"

namespace UE::AvaEditor::Private
{
	class SAvaLevelViewportExposeBinds : public SLevelViewport
	{
	public:
		using SLevelViewport::BindCommands;
	};

	void FixupInvalidFocusedLevelEditorViewport()
	{
		const FLevelEditorModule* LevelEditorModule = FAvaLevelEditorUtils::GetLevelEditorModule();
		if (!LevelEditorModule)
		{
			return;
		}

		TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule->GetFirstLevelEditor();
		if (!LevelEditor.IsValid())
		{
			return;
		}

		TArray<TSharedPtr<SLevelViewport>> Viewports = LevelEditor->GetViewports();
		if (Viewports.IsEmpty())
		{
			return;
		}

		const bool bHasValidFocusedViewport = Viewports.ContainsByPredicate(
			[FocusedViewportClient = LevelEditor->GetEditorModeManager().GetFocusedViewportClient()](const TSharedPtr<SLevelViewport>& InViewport)
			{
				return InViewport.IsValid() && InViewport->GetViewportClient().Get() == FocusedViewportClient;
			});

		// return early if the existing focused viewport is valid (and not some dangling ptr)
		if (bHasValidFocusedViewport)
		{
			return;
		}

		FEditorViewportClient* ViewportClientToFocus = nullptr;

		// Get the first valid Motion Design viewport
		for (const TSharedPtr<SLevelViewport>& Viewport : Viewports)
		{
			if (Viewport->IsVisible())
			{
				TSharedPtr<FEditorViewportClient> ViewportClient = Viewport->GetViewportClient();
				TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(ViewportClient.Get());

				if (AvaViewportClient && AvaViewportClient->IsMotionDesignViewport())
				{
					ViewportClientToFocus = ViewportClient.Get();
					break;
				}
			}
		}

		// if there isn't any Motion Design viewport, use the global viewport client that was set by the level editor
		if (!ViewportClientToFocus)
		{
			ViewportClientToFocus = GCurrentLevelEditingViewportClient;
		}

		if (ensure(ViewportClientToFocus))
		{
			ViewportClientToFocus->ReceivedFocus(ViewportClientToFocus->Viewport);
		}
	}
}

FAvaLevelViewportExtension::~FAvaLevelViewportExtension()
{
	if (FLevelEditorModule* LevelEditorModule = FAvaLevelEditorUtils::GetLevelEditorModule())
	{
		LevelEditorModule->UnregisterViewportType(FAvaLevelViewportLayoutEntity::GetStaticType());
		if (OnMapChangedHandle.IsValid())
		{
			LevelEditorModule->OnMapChanged().Remove(OnMapChangedHandle);
			OnMapChangedHandle.Reset();
		}
	}

	UnbindCameraCutDelegate();
}

bool FAvaLevelViewportExtension::IsCameraCutEnabled() const
{
	if (TSharedPtr<IAvaEditor> Editor = GetEditor())
	{
		if (TSharedPtr<FAvaSequencerExtension> SequencerExtension = Editor->FindExtension<FAvaSequencerExtension>())
		{
			if (TSharedPtr<ISequencer> Sequencer = SequencerExtension->GetSequencer())
			{
				return Sequencer->IsPerspectiveViewportCameraCutEnabled();
			}
		}
	}

	return false;
}

void FAvaLevelViewportExtension::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	FAvaViewportExtension::BindCommands(InCommandList);

	InCommandList->MapAction(FAvaEditorCommands::Get().SwitchViewports
		, FExecuteAction::CreateSP(this, &FAvaLevelViewportExtension::OnSwitchViewports));

	const FAvaLevelViewportCommands& LevelViewportCommands = FAvaLevelViewportCommands::GetExternal();

	InCommandList->MapAction(LevelViewportCommands.ResetLocation
		, FExecuteAction::CreateSP(this, &FAvaLevelViewportExtension::ExecuteResetLocation));

	InCommandList->MapAction(LevelViewportCommands.ResetRotation
		, FExecuteAction::CreateSP(this, &FAvaLevelViewportExtension::ExecuteResetRotation));

	InCommandList->MapAction(LevelViewportCommands.ResetScale
		, FExecuteAction::CreateSP(this, &FAvaLevelViewportExtension::ExecuteResetScale));
}

void FAvaLevelViewportExtension::Activate()
{
	FAvaViewportExtension::Activate();

	if (FLevelEditorModule* LevelEditorModule = FAvaLevelEditorUtils::LoadLevelEditorModule())
	{
		LevelEditorModule->RegisterViewportType(FAvaLevelViewportLayoutEntity::GetStaticType(), MakeViewportTypeDefinition());
		if (TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule->GetFirstLevelEditor())
		{
			for (TSharedPtr<SLevelViewport> LevelViewport : LevelEditor->GetViewports())
			{
				StaticCastSharedPtr<UE::AvaEditor::Private::SAvaLevelViewportExposeBinds>(LevelViewport)->BindCommands();
			}
		}
	}

	CheckValidViewportType();

	BindCameraCutDelegate();
}

void FAvaLevelViewportExtension::Deactivate()
{
	FAvaViewportExtension::Deactivate();

	// If the command is invalid, we're shutting down and this doesn't need to be run.
	if (!FAvaEditorCommands::IsRegistered() || !FAvaEditorCommands::Get().SetMotionDesignViewportType.IsValid())
	{
		return;
	}

	if (FLevelEditorModule* LevelEditorModule = FAvaLevelEditorUtils::GetLevelEditorModule())
	{
		if (TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule->GetFirstLevelEditor())
		{
			TSharedPtr<FUICommandInfo> ToggleViewportCommandInfo = FAvaEditorCommands::Get().SetMotionDesignViewportType;

			for (TSharedPtr<SLevelViewport> LevelViewport : LevelEditor->GetViewports())
			{
				if (LevelViewport.IsValid())
				{
					if (TSharedPtr<FUICommandList> CommandList = LevelViewport->GetCommandList())
					{
						CommandList->UnmapAction(ToggleViewportCommandInfo);
					}
				}
			}
		}

		LevelEditorModule->UnregisterViewportType(FAvaLevelViewportLayoutEntity::GetStaticType());
	}

	UnbindCameraCutDelegate();
}

void FAvaLevelViewportExtension::Construct(const TSharedRef<IAvaEditor>& InEditor)
{
	FAvaViewportExtension::Construct(InEditor);

	if (FLevelEditorModule* LevelEditorModule = FAvaLevelEditorUtils::LoadLevelEditorModule())
	{
		OnMapChangedHandle = LevelEditorModule->OnMapChanged().AddSP(this, &FAvaLevelViewportExtension::OnMapChanged);
	}
}

TArray<TSharedPtr<IAvaViewportClient>> FAvaLevelViewportExtension::GetViewportClients() const
{
	return GetLevelEditorViewportClients();
}

void FAvaLevelViewportExtension::NotifyOnSelectionChanged(const FAvaEditorSelection& InSelection)
{
	FEditorModeTools* const ModeTools = GetEditorModeTools();

	if (!ModeTools)
	{
		return;
	}

	// Only update pivot if actors changed selection
	if (InSelection.GetChangedSelection() != InSelection.GetSelection<AActor>())
	{
		return;
	}

	const TArray<AActor*> SelectedActors = InSelection.GetSelectedObjects<AActor>();

	if (SelectedActors.IsEmpty())
	{
		return;
	}

	const double ActorCountInverse = 1.0 / static_cast<double>(SelectedActors.Num());
	FVector PivotLocation(EForceInit::ForceInitToZero);

	for (AActor* const Actor : SelectedActors)
	{
		PivotLocation += Actor->GetActorLocation() * ActorCountInverse;
	}

	constexpr bool bIncludeGridBase = false;
	ModeTools->SetPivotLocation(PivotLocation, bIncludeGridBase);
	GUnrealEd->SetPivotMovedIndependently(true);
}

bool FAvaLevelViewportExtension::IsDroppingPreviewActor() const
{
	return FLevelEditorViewportClient::IsDroppingPreviewActor();
}

TArray<TSharedPtr<IAvaViewportClient>> FAvaLevelViewportExtension::GetLevelEditorViewportClients()
{
	TArray<TSharedPtr<IAvaViewportClient>> AvaViewportClients;

	if (FLevelEditorModule* LevelEditorModule = FAvaLevelEditorUtils::GetLevelEditorModule())
	{
		if (TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule->GetLevelEditorInstance().Pin())
		{
			if (TSharedPtr<SLevelViewport> ActiveLevelViewport = LevelEditor->GetActiveViewportInterface())
			{
				if (TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(ActiveLevelViewport->GetViewportClient().Get()))
				{
					AvaViewportClients.Add(AvaViewportClient);
				}

				for (TSharedPtr<SLevelViewport> LevelViewport : LevelEditor->GetViewports())
				{
					if (LevelViewport != ActiveLevelViewport)
					{
						if (TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(LevelViewport->GetViewportClient().Get()))
						{
							AvaViewportClients.Add(AvaViewportClient);
						}
					}
				}
			}
		}
	}

	return AvaViewportClients;
}

void FAvaLevelViewportExtension::SetDefaultViewportType()
{
	SetViewportType(FLevelViewportCommands::Get().SetDefaultViewportType.ToSharedRef(), /* Set Active Camera */ false);
}

void FAvaLevelViewportExtension::SetMotionDesignViewportType()
{
	SetViewportType(FAvaEditorCommands::Get().SetMotionDesignViewportType.ToSharedRef(), /* Set Active Camera */ true);
}

void FAvaLevelViewportExtension::SetViewportType(const TSharedRef<FUICommandInfo>& InViewportCommand, bool bInSetActiveCamera)
{
	FLevelEditorModule* LevelEditorModule = FAvaLevelEditorUtils::GetLevelEditorModule();

	if (!LevelEditorModule)
	{
		return;
	}

	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule->GetLevelEditorInstance().Pin();

	if (!LevelEditor.IsValid())
	{
		return;
	}

	TSharedPtr<FUICommandList> CommandList;
	if (TSharedPtr<SLevelViewport> ActiveLevelViewport = LevelEditor->GetActiveViewportInterface())
	{
		CommandList = ActiveLevelViewport->GetCommandList();
	}

	if (!CommandList.IsValid())
	{
		return;
	}

	CommandList->ExecuteAction(InViewportCommand);

	UE::AvaEditor::Private::FixupInvalidFocusedLevelEditorViewport();

	if (bInSetActiveCamera)
	{
		if (AActor* LastCameraCutActor = LastCameraCutActorWeak.Get())
		{
			SetActiveCamera(LastCameraCutActor, true);
		}
	}
}

FViewportTypeDefinition FAvaLevelViewportExtension::MakeViewportTypeDefinition()
{
	return FViewportTypeDefinition(
		[](const FAssetEditorViewportConstructionArgs& InArgs, TSharedPtr<ILevelEditor> InLevelEditor)
		{
			return MakeShared<FAvaLevelViewportLayoutEntity>(InArgs, InLevelEditor);
		}
		, FAvaEditorCommands::Get().SetMotionDesignViewportType);
}

void FAvaLevelViewportExtension::OnSwitchViewports()
{
	FLevelEditorModule* LevelEditorModule = FAvaLevelEditorUtils::GetLevelEditorModule();
	if (!LevelEditorModule)
	{
		return;
	}

	TSharedPtr<SLevelViewport> ViewportWidget = LevelEditorModule->GetFirstActiveLevelViewport();
	if (!ViewportWidget.IsValid())
	{
		return;
	}

	UWorld* const ViewportWorld = ViewportWidget->GetWorld();
	if (!ViewportWorld || ViewportWorld != GetWorld())
	{
		return;
	}

	TSharedPtr<FEditorViewportClient> ViewportClient = ViewportWidget->GetViewportClient();
	if (!ViewportClient)
	{
		return;
	}

	TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(ViewportClient.Get());
	if (!AvaViewportClient.IsValid())
	{
		return;
	}

	UObject* const SceneObject = GetSceneObject();
	if (!SceneObject || AvaViewportClient->IsMotionDesignViewport())
	{
		SetDefaultViewportType();
	}
	else
	{
		SetMotionDesignViewportType();
	}
}

void FAvaLevelViewportExtension::OnMapChanged(UWorld* InWorld, EMapChangeType InChangeType)
{
	if (InChangeType == EMapChangeType::SaveMap || InChangeType == EMapChangeType::TearDownWorld)
	{
		return;
	}

	CheckValidViewportType();

	if (AActor* LastCameraCutActor = LastCameraCutActorWeak.Get())
	{
		SetActiveCamera(LastCameraCutActor, true);
	}
}

void FAvaLevelViewportExtension::CheckValidViewportType()
{
	TSharedPtr<IAvaEditor> Editor = GetEditor();

	// Set the active viewport to Motion Design if there is a valid scene object (not all the viewports)
	if (Editor.IsValid() && Editor->IsActive() && GetSceneObject())
	{
		if (GetDefault<UAvaEditorSettings>()->bAutoActivateMotionDesignViewport)
		{
			SetMotionDesignViewportType();
		}

		return;
	}

	// Reset all Motion Design viewports to default if there was an invalid scene object
	FLevelEditorModule* LevelEditorModule = FAvaLevelEditorUtils::GetLevelEditorModule();
	if (!LevelEditorModule)
	{
		return;
	}

	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule->GetLevelEditorInstance().Pin();
	if (!LevelEditor.IsValid())
	{
		return;
	}

	// Execute SetDefaultViewportType on all Motion Design viewports, not only the Active one
	for (TSharedPtr<SLevelViewport> LevelViewport : LevelEditor->GetViewports())
	{
		TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(LevelViewport->GetViewportClient().Get());
		if (AvaViewportClient.IsValid() && AvaViewportClient->IsMotionDesignViewport())
		{
			LevelViewport->GetCommandList()->ExecuteAction(FLevelViewportCommands::Get().SetDefaultViewportType.ToSharedRef());
		}
	}

	UE::AvaEditor::Private::FixupInvalidFocusedLevelEditorViewport();
}

void FAvaLevelViewportExtension::BindCameraCutDelegate()
{
	if (TSharedPtr<IAvaEditor> Editor = GetEditor())
	{
		if (TSharedPtr<FAvaSequencerExtension> SequencerExtension = Editor->FindExtension<FAvaSequencerExtension>())
		{
			if (TSharedPtr<ISequencer> Sequencer = SequencerExtension->GetSequencer())
			{
				Sequencer->OnCameraCut().AddSP(this, &FAvaLevelViewportExtension::OnCameraCut);
			}
		}
	}

	if (AActor* LastCameraCutActor = LastCameraCutActorWeak.Get())
	{
		SetActiveCamera(LastCameraCutActor, true);
	}
}

void FAvaLevelViewportExtension::UnbindCameraCutDelegate()
{
	if (TSharedPtr<IAvaEditor> Editor = GetEditor())
	{
		if (TSharedPtr<FAvaSequencerExtension> SequencerExtension = Editor->FindExtension<FAvaSequencerExtension>())
		{
			if (TSharedPtr<ISequencer> Sequencer = SequencerExtension->GetSequencer())
			{
				Sequencer->OnCameraCut().RemoveAll(this);
			}
		}
	}
}

void FAvaLevelViewportExtension::OnCameraCut(UObject* InCameraObject, bool bInJumpCut)
{
	/**
	 * The event is always called with bInJumpCut set to true. However, it's only really a jump cut
	 * if ther camera object changes. Here we're checking to see if there was a change and ignoring
	 * the camera cut update if the camera hasn't changed.
	 *
	 * Always updating the camera and sending the jump cut signal to the base viewport class causes
	 * rendering artifacts (such as flickering).
	 */
	if (InCameraObject == LastCameraCutActorWeak.Get())
	{
		return;
	}

	LastCameraCutActorWeak = nullptr;

	if (!IsValid(InCameraObject))
	{
		return;
	}

	AActor* Actor = Cast<AActor>(InCameraObject);

	if (!Actor)
	{
		return;
	}

	SetActiveCamera(Actor, bInJumpCut);
}

void FAvaLevelViewportExtension::SetActiveCamera(AActor* InActiveCameraActor, bool bInJumpCut)
{
	LastCameraCutActorWeak = InActiveCameraActor;

	// If camera cutting isn't enabled, store the cut to actor but don't do anything with it
	if (!IsCameraCutEnabled())
	{
		return;
	}

	for (TSharedPtr<IAvaViewportClient> ViewportClient : GetViewportClients())
	{
		if (ViewportClient->IsMotionDesignViewport())
		{
			ViewportClient->OnCameraCut(InActiveCameraActor, bInJumpCut);
		}
	}
}

void FAvaLevelViewportExtension::ExecuteResetLocation()
{
	TSharedPtr<IAvaEditor> Editor = GetEditor();
	
	if (!Editor.IsValid())
	{
		return;
	}

	FEditorModeTools* ModeTools = GetEditorModeTools();

	if (!ModeTools)
	{
		return;
	}

	USelection* ActorSelection = ModeTools->GetSelectedActors();

	if (!ActorSelection)
	{
		return;
	}

	TArray<AActor*> SelectedActors;
	ActorSelection->GetSelectedObjects<AActor>(SelectedActors);

	if (SelectedActors.IsEmpty())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ResetLocation", "Reset Actor Location"));

	for (AActor* Actor : SelectedActors)
	{
		FActorElementEditorViewportInteractionCustomization::ApplyDeltaToActor(
			Actor,
			/* bIsDelta */ false,
			/* Location */ &FVector::ZeroVector,
			/* Rotation */ nullptr,
			/* Scale */ nullptr,
			/* Pivot */ FVector::ZeroVector,
			FInputDeviceState()
		);
	}
}

void FAvaLevelViewportExtension::ExecuteResetRotation()
{
	TSharedPtr<IAvaEditor> Editor = GetEditor();

	if (!Editor.IsValid())
	{
		return;
	}

	FEditorModeTools* ModeTools = GetEditorModeTools();

	if (!ModeTools)
	{
		return;
	}

	USelection* ActorSelection = ModeTools->GetSelectedActors();

	if (!ActorSelection)
	{
		return;
	}

	TArray<AActor*> SelectedActors;
	ActorSelection->GetSelectedObjects<AActor>(SelectedActors);

	if (SelectedActors.IsEmpty())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ResetRotation", "Reset Actor Rotation"));

	for (AActor* Actor : SelectedActors)
	{
		FActorElementEditorViewportInteractionCustomization::ApplyDeltaToActor(
			Actor,
			/* bIsDelta */ false,
			/* Location */ nullptr,
			/* Rotation */ &FRotator::ZeroRotator,
			/* Scale */ nullptr,
			/* Pivot */ FVector::ZeroVector,
			FInputDeviceState()
		);
	}
}

void FAvaLevelViewportExtension::ExecuteResetScale()
{
	TSharedPtr<IAvaEditor> Editor = GetEditor();

	if (!Editor.IsValid())
	{
		return;
	}

	FEditorModeTools* ModeTools = GetEditorModeTools();

	if (!ModeTools)
	{
		return;
	}

	USelection* ActorSelection = ModeTools->GetSelectedActors();

	if (!ActorSelection)
	{
		return;
	}

	TArray<AActor*> SelectedActors;
	ActorSelection->GetSelectedObjects<AActor>(SelectedActors);

	if (SelectedActors.IsEmpty())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ResetScale", "Reset Actor Scale"));

	for (AActor* Actor : SelectedActors)
	{
		FActorElementEditorViewportInteractionCustomization::ApplyDeltaToActor(
			Actor,
			/* bIsDelta */ false,
			/* Location */ nullptr,
			/* Rotation */ nullptr,
			/* Scale */ &FVector::OneVector,
			/* Pivot */ FVector::ZeroVector,
			FInputDeviceState()
		);
	}
}

#undef LOCTEXT_NAMESPACE
