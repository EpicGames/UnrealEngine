// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaViewportExtension.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class FUICommandInfo;
class IAvaEditor;
class UObject;
class UWorld;
enum class EMapChangeType : uint8;
struct FViewportTypeDefinition;

class FAvaLevelViewportExtension : public FAvaViewportExtension
{
public:
	UE_AVA_INHERITS(FAvaLevelViewportExtension, FAvaViewportExtension);

	static TArray<TSharedPtr<IAvaViewportClient>> GetLevelEditorViewportClients();

	void SetDefaultViewportType();
	void SetMotionDesignViewportType();
	void SetViewportType(const TSharedRef<FUICommandInfo>& InViewportCommand, bool bInSetActiveCamera);

	virtual ~FAvaLevelViewportExtension() override;

	bool IsCameraCutEnabled() const;

	//~ Begin IAvaEditorExtension
	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandList) override;
	virtual void Activate() override;
	virtual void Deactivate() override;
	//~ End IAvaEditorExtension

	//~ Begin IAvaViewportExtension
	virtual void Construct(const TSharedRef<IAvaEditor>& InEditor) override;
	virtual TArray<TSharedPtr<IAvaViewportClient>> GetViewportClients() const override;
	virtual void NotifyOnSelectionChanged(const FAvaEditorSelection& InSelection) override;
	//~ End IAvaViewportExtension

	//~ Begin IAvaViewportProvider
	virtual bool IsDroppingPreviewActor() const override;
	//~ End IAvaViewportProvider

private:
	FDelegateHandle OnMapChangedHandle;

	FViewportTypeDefinition MakeViewportTypeDefinition();

	TWeakObjectPtr<AActor> LastCameraCutActorWeak;

	void OnSwitchViewports();

	void OnMapChanged(UWorld* InWorld, EMapChangeType InChangeType);

	void CheckValidViewportType();

	void BindCameraCutDelegate();
	void UnbindCameraCutDelegate();

	void OnCameraCut(UObject* InCameraObject, bool bInJumpCut);

	void SetActiveCamera(AActor* InActiveCameraActor, bool bInJumpCut);

	void ExecuteResetLocation();
	void ExecuteResetRotation();
	void ExecuteResetScale();
};
