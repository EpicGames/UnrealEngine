// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "IAvaEditorExtension.h"
#include "Templates/SharedPointerFwd.h"

class IAvaViewportClient;

class IAvaViewportProvider
{
public:	
	UE_AVA_TYPE(IAvaViewportProvider);

	virtual ~IAvaViewportProvider() = default;

	/** Returns if the User is currently dropping an Actor that is Preview (e.g. Dragging an Actor to Spawn but not dropping it yet) */
	virtual bool IsDroppingPreviewActor() const = 0;
};

class FAvaViewportExtension : public FAvaEditorExtension, public IAvaViewportProvider
{
public:
	UE_AVA_INHERITS(FAvaViewportExtension, FAvaEditorExtension, IAvaViewportProvider);

	virtual TArray<TSharedPtr<IAvaViewportClient>> GetViewportClients() const { return {}; }
};
