// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheEditorCore : ModuleRules
{
	public AvalancheEditorCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AvalancheCore",
				"Core",
				"InputCore",
				"Slate",
				"SlateCore",
				"UnrealEd",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"CoreUObject",
				"EditorFramework",
				"Engine",
				"LevelEditor",
				"Projects",
				"StatusBar",
				"ToolMenus",
				"TypedElementFramework",
				"TypedElementRuntime",
				"WorkspaceMenuStructure",
			}
		);
	}
}