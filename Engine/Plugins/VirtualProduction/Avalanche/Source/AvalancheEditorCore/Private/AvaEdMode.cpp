// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEdMode.h"
#include "AvaEdModeToolkit.h"
#include "AvaEditorSubsystem.h"
#include "EditorModeManager.h"
#include "IAvaEditor.h"
#include "Tools/Modes.h"

#define LOCTEXT_NAMESPACE "AvaEdMode"

const FEditorModeID UAvaEdMode::ModeID(TEXT("EM_MotionDesign"));

UAvaEdMode::UAvaEdMode()
{
	Info = FEditorModeInfo(UAvaEdMode::ModeID, LOCTEXT("ModeDisplayName", "Motion Design"), FSlateIcon(), false);
}

void UAvaEdMode::Initialize()
{
	Super::Initialize();

	if (UAvaEditorSubsystem* EditorSubsystem = UAvaEditorSubsystem::Get(Owner))
	{
		EditorWeak = EditorSubsystem->GetActiveEditor();
	}
}

bool UAvaEdMode::UsesToolkits() const
{
	// UsesToolkits() is only used in UEdMode::CreateToolkit, FEditorModeTools::ShouldShowModeToolbox (deprecated) and where the Editor Mode Tools processes input.
	// CreateToolkit is already overriden by UAvaEdMode (below), so UsesToolkits() should return false to block EditorModeTools from processing input.
	// Note: EditorModeTools processes input via SViewport (before SEditorViewport).
	// This toolkit command list ends up being appended to the Global Level Editor command list as a parent.
	// This is so that any action bound to any command list in that parent-child chain can be processed without needing to remap.
	// However, UToolMenu objects do not fully release their command list shared ref until GC.
	// Due to these tool menu objects, changing viewport types temporarily breaks the command input processing for viewport-related actions,
	// and because of how FUICommandList input processing returns true for actions that have invalid objects (since CanExecute will return true if not bound and Execute is not checked).
	// By having this false, the toolkit commands still get processed in SLevelEditor::OnKeyDown,
	// but at a much later time, allowing others like valid SEditorViewport instances to process their command list first.
	return false;
}

void UAvaEdMode::CreateToolkit()
{
	Toolkit = MakeShared<FAvaEdModeToolkit>(this);
}

bool UAvaEdMode::ProcessEditCut()
{
	TSharedPtr<IAvaEditor> Editor = GetEditor();
	return Editor.IsValid() && Editor->EditCut();
}

bool UAvaEdMode::ProcessEditCopy()
{
	TSharedPtr<IAvaEditor> Editor = GetEditor();
	return Editor.IsValid() && Editor->EditCopy();
}

bool UAvaEdMode::ProcessEditPaste()
{
	TSharedPtr<IAvaEditor> Editor = GetEditor();
	return Editor.IsValid() && Editor->EditPaste();
}

bool UAvaEdMode::ProcessEditDuplicate()
{
	TSharedPtr<IAvaEditor> Editor = GetEditor();
	return Editor.IsValid() && Editor->EditDuplicate();
}

bool UAvaEdMode::ProcessEditDelete()
{
	TSharedPtr<IAvaEditor> Editor = GetEditor();
	return Editor.IsValid() && Editor->EditDelete();
}

TSharedPtr<FUICommandList> UAvaEdMode::GetToolkitCommands() const
{
	if (Toolkit.IsValid())
	{
		return Toolkit->GetToolkitCommands();
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
