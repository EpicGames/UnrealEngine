// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/UEdMode.h"
#include "Templates/SharedPointer.h"
#include "AvaEdMode.generated.h"

class IAvaEditor;

/**
 * Used primarily as means to add/override functionality such as Edit (Copy,Paste,etc)
 * This Ed Mode gets automatically activated by the Ava Editor Init
 */
UCLASS()
class UAvaEdMode : public UEdMode
{
	GENERATED_BODY()

public:
	static const FEditorModeID ModeID;

	UAvaEdMode();

	virtual ~UAvaEdMode() override = default;

	TSharedPtr<IAvaEditor> GetEditor() const { return EditorWeak.Pin(); }

	//~ Begin UEdMode
	virtual void Initialize();
	virtual bool IsCompatibleWith(FEditorModeID InOtherModeID) const override { return true; }
	virtual EEditAction::Type GetActionEditCut() override { return EEditAction::Process; }
	virtual EEditAction::Type GetActionEditCopy() override { return EEditAction::Process; }
	virtual EEditAction::Type GetActionEditPaste() override { return EEditAction::Process; }
	virtual EEditAction::Type GetActionEditDuplicate() override { return EEditAction::Process; }
	virtual EEditAction::Type GetActionEditDelete() override { return EEditAction::Process; }
	virtual bool UsesToolkits() const override;
	virtual void CreateToolkit() override;
	virtual bool ProcessEditCut() override;
	virtual bool ProcessEditCopy() override;
	virtual bool ProcessEditPaste() override;
	virtual bool ProcessEditDuplicate() override;
	virtual bool ProcessEditDelete() override;
	//~ End UEdMode

	TSharedPtr<FUICommandList> GetToolkitCommands() const;

private:
	TWeakPtr<IAvaEditor> EditorWeak;
};
