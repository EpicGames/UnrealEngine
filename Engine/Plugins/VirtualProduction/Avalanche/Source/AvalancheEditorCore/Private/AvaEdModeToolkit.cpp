// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEdModeToolkit.h"
#include "AvaEdMode.h"
#include "IAvaEditor.h"
#include "IAvaEditorCoreModule.h"

#define LOCTEXT_NAMESPACE "AvaEdModeToolkit"

namespace UE::AvaEditorCore::Private
{
	FName ToolkitName = "AvaEdModeToolkit";
	FText ToolkitBaseName = LOCTEXT("ToolkitName", "Motion Design Editor Toolkit");
}

FAvaEdModeToolkit::FAvaEdModeToolkit(UAvaEdMode* InEdMode)
	: EdModeWeak(InEdMode)
{
}

FName FAvaEdModeToolkit::GetToolkitFName() const
{
	return UE::AvaEditorCore::Private::ToolkitName;
}

FText FAvaEdModeToolkit::GetBaseToolkitName() const
{
	return UE::AvaEditorCore::Private::ToolkitBaseName;
}

void FAvaEdModeToolkit::ExtendSecondaryModeToolbar(UToolMenu* InToolbarMenu)
{
	if (!InToolbarMenu)
	{
		return;
	}

	UAvaEdMode* EdMode = EdModeWeak.Get();
	if (!EdMode)
	{
		return;
	}

	if (TSharedPtr<IAvaEditor> Editor = EdMode->GetEditor())
	{
		Editor->ExtendToolbarMenu(InToolbarMenu);
	}
	IAvaEditorCoreModule::Get().GetOnExtendEditorToolbar().Broadcast(*InToolbarMenu);
}

#undef LOCTEXT_NAMESPACE
