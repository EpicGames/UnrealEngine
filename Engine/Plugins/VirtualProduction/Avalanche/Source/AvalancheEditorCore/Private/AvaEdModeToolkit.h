// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/BaseToolkit.h"

class UAvaEdMode;

class FAvaEdModeToolkit : public FModeToolkit
{
public:
	FAvaEdModeToolkit(UAvaEdMode* InEdMode);

	//~ Begin IToolkit
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	//~ End IToolkit

	//~ Begin FModeToolkit
	virtual void RequestModeUITabs() override {}
	virtual void InvokeUI() override {}
	virtual void ExtendSecondaryModeToolbar(UToolMenu* InToolbarMenu);
	//~ End FModeToolkit

private:
	TWeakObjectPtr<UAvaEdMode> EdModeWeak;
};
