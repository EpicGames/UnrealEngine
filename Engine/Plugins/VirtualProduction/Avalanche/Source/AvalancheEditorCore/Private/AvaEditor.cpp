// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEditor.h"
#include "Algo/ForEach.h"
#include "AvaEdMode.h"
#include "AvaEditorBuilder.h"
#include "AvaEditorCoreStyle.h"
#include "AvaEditorExtensionTypeRegistry.h"
#include "AvaEditorSubsystem.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorModeManager.h"
#include "Elements/Framework/TypedElementCommonActions.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementUtil.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "EngineAnalytics.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IAvaEditorExtension.h"
#include "IAvaTabSpawner.h"
#include "Misc/CoreDelegates.h"
#include "Misc/MessageDialog.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "Selection/AvaEditorSelection.h"
#include "StatusBarSubsystem.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "Toolkits/IToolkitHost.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "UnrealEdGlobals.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "AvaEditor"

namespace UE::AvaEditorCore::Private
{
	static constexpr const TCHAR* AvalancheCopyToken = TEXT("AvalancheCopy");

	// Gathers the commonly used objects and functionality for Common Edit Actions
	struct FAvaActorEditActionScope
	{
		FAvaActorEditActionScope(const IAvaEditor& InEditor)
		{
			ToolkitHost = InEditor.GetToolkitHost();
			if (ToolkitHost.IsValid())
			{
				CommonActions  = ToolkitHost->GetCommonActions();
				World          = ToolkitHost->GetWorld();
				ActorSelection = ToolkitHost->GetEditorModeManager().GetSelectedActors();
			}
		}

		bool HasSelectedComponents() const
		{
			USelection* const ComponentSelection = ToolkitHost.IsValid()
				? ToolkitHost->GetEditorModeManager().GetSelectedComponents()
				: nullptr;

			if (!ComponentSelection)
			{
				return false;
			}

			for (FSelectionIterator Iter(*ComponentSelection); Iter; ++Iter)
			{
				if (::IsValid(*Iter))
				{
					return true;
				}
			}

			return false;
		}

		bool IsValid() const
		{
			return ToolkitHost.IsValid() && CommonActions && World && ActorSelection;
		}

		TSharedPtr<IToolkitHost> ToolkitHost;
		UTypedElementCommonActions* CommonActions = nullptr;
		USelection* ActorSelection = nullptr;
		UWorld* World = nullptr;
	};

	// Temporarily selects Actors for Edit (e.g. Copy Selected, Cut Selected, etc)
	// Note: Does not Modify the Selection as it's restored when exiting scope
	struct FAvaActorEditSelectionScope
	{
		FAvaActorEditSelectionScope(USelection& InActorSelection) : ActorSelection(InActorSelection)
		{
			ActorSelection.GetSelectedObjects(InitialSelection);
			if (UTypedElementSelectionSet* const SelectionSet = ActorSelection.GetElementSelectionSet())
			{
				ClearNewPendingChange = SelectionSet->GetScopedClearNewPendingChange();
			}			
		}

		~FAvaActorEditSelectionScope()
		{
			SetActorSelection(InitialSelection);
		}

		void SetActorSelection(TConstArrayView<AActor*> InActors)
		{
			ActorSelection.BeginBatchSelectOperation();
			ActorSelection.DeselectAll();
			for (AActor* const Actor : InActors)
			{
				ActorSelection.Select(Actor);
			}
			ActorSelection.EndBatchSelectOperation(/*bNotify*/false);
		}

		TArray<AActor*> GetSelectedActors() const
		{
			TArray<AActor*> SelectedActors;
			ActorSelection.GetSelectedObjects(SelectedActors);
			return SelectedActors;
		}

	private:
		FTypedElementList::FScopedClearNewPendingChange ClearNewPendingChange;
		USelection& ActorSelection;
		TArray<AActor*> InitialSelection;
	};

	// Temporarily adds a tag to the Actor to identify it for Edit (e.g. Duplication, Copy, Paste)
	// Note: Does not Modify the Actor as the original Tag state is restored when exiting scope
	struct FAvaActorEditTagScope
	{
		FAvaActorEditTagScope(TConstArrayView<AActor*> InActors) : Actors(InActors)
		{
			for (AActor* const Actor : Actors)
			{
				Actor->Tags.Push(*TSoftObjectPtr<AActor>(Actor).ToString());
			}
		}

		~FAvaActorEditTagScope()
		{
			for (AActor* const Actor : Actors)
			{
				const FName LastTag = Actor->Tags.Pop();
				check(LastTag == *TSoftObjectPtr<AActor>(Actor).ToString());
			}
		}

	private:
		TConstArrayView<AActor*> Actors;
	};
}

FAvaEditor::FAvaEditor(FAvaEditorBuilder& Initializer)
	: Provider(MoveTemp(Initializer.Provider).ToSharedRef())
	, EditorExtensions(MoveTemp(Initializer.Extensions))
{
}

FAvaEditor::~FAvaEditor()
{
	FAvaEditor::UnregisterTabSpawners();
	UnbindDelegates();
	EditorExtensions.Reset();
	FAvaEditorExtensionTypeRegistry::Get().RemoveStaleEntries();
}

void FAvaEditor::InvokeTabs()
{
	const TSharedPtr<IToolkitHost> ToolkitHost = GetToolkitHost();
	const TSharedPtr<FTabManager> TabManager = GetTabManager();

	if (!ToolkitHost.IsValid() || !TabManager.IsValid())
	{
		return;
	}

	const TSharedRef<FTabManager::FLayout> Layout = TabManager->PersistLayout();

	// HACK: Way to iterate Layout Vars via a Const Array View public func call
	// TODO: Public API in Tab Manager to allow const iteration without this hack
	class FLayout   : public FTabManager::FLayout   { public: TConstArrayView<TSharedRef<FTabManager::FArea>> GetAreas() { return Areas; } };
	class FSplitter : public FTabManager::FSplitter { public: TConstArrayView<TSharedRef<FTabManager::FLayoutNode>> GetChildNodes() { return ChildNodes; } };
	class FStack    : public FTabManager::FStack    { public: TConstArrayView<FTabManager::FTab> GetTabs() { return Tabs; } };

	TArray<FTabId> TabsToInvoke;

	// Gather all the Motion Design Tabs that are supposed to be opened, but were possibly not due to not having a valid Toolkit Host at the time
	TArray<TSharedRef<FTabManager::FSplitter>> Splitters(StaticCastSharedRef<FLayout>(Layout)->GetAreas());

	while (!Splitters.IsEmpty())
	{
		const TSharedRef<FTabManager::FSplitter> Splitter = Splitters.Pop();

		for (const TSharedRef<FTabManager::FLayoutNode>& Child : StaticCastSharedRef<FSplitter>(Splitter)->GetChildNodes())
		{
			if (TSharedPtr<FTabManager::FStack> ChildStack = Child->AsStack())
			{
				for (const FTabManager::FTab& Tab : StaticCastSharedPtr<FStack>(ChildStack)->GetTabs())
				{
					if (Tab.TabState == ETabState::OpenedTab && TabSpawners.Contains(Tab.TabId.TabType))
					{
						TabsToInvoke.Add(Tab.TabId);
					}
				}
			}
			// This consider FAreas as well, (FArea inherits from FSplitter)
			else if (TSharedPtr<FTabManager::FSplitter> ChildSplitter = Child->AsSplitter())
			{
				Splitters.Add(ChildSplitter.ToSharedRef());
			}
		}
	}

	for (const FTabId& TabId : TabsToInvoke)
	{
		// Do not re-use the existing tabs here.
		// Instead, invoke a whole new tab
		if (TSharedPtr<SDockTab> Tab = TabManager->FindExistingLiveTab(TabId))
		{
			Tab->RequestCloseTab();
		}
		TabManager->TryInvokeTab(TabId);
	}
}

void FAvaEditor::ForEachExtension(TFunctionRef<void(const TSharedRef<IAvaEditorExtension>&)> InFunc) const
{
	for (const TPair<FAvaTypeId, TSharedRef<IAvaEditorExtension>>& Pair : EditorExtensions)
	{
		InFunc(Pair.Value);
	}
}

void FAvaEditor::NotifySceneActivationEvent()
{
	if (bIsActive)
	{
		Provider->OnSceneActivated();
	}
	else
	{
		Provider->OnSceneDeactivated();
	}
}

void FAvaEditor::Construct()
{
	ForEachExtension(
		[This = SharedThis(this)](const TSharedRef<IAvaEditorExtension>& InExtension)
		{
			InExtension->Construct(This);
		});
}

void FAvaEditor::Activate(TSharedPtr<IToolkitHost> InOverrideToolkitHost)
{
	if (!CanActivate())
	{
		return;
	}

	TGuardValue<bool> StateChangeGuard(bActiveStateChanging, true);

	bIsActive = true;

	NotifySceneActivationEvent();

	BindDelegates();

	if (InOverrideToolkitHost.IsValid())
	{
		SetToolkitHost(InOverrideToolkitHost.ToSharedRef());	
	}

	const TSharedPtr<IToolkitHost> ToolkitHost = GetToolkitHost();
	checkf(ToolkitHost.IsValid(), TEXT("Toolkit Host must be valid when Initializing Ava Editor"));

	if (UAvaEditorSubsystem* AvaEditorSubsystem = UAvaEditorSubsystem::Get(GetWorld()))
	{
		AvaEditorSubsystem->OnEditorActivated(SharedThis(this));
	}

	FEditorModeTools& ModeTools = ToolkitHost->GetEditorModeManager();
	ModeTools.ActivateMode(UAvaEdMode::ModeID);

	// Opt out of ITF gizmos and viewport interactions
	ModeTools.SetSupportsViewportITF(false);

	// Disable viewport interactions to allow legacy gizmos to receive input
	if (UModeManagerInteractiveToolsContext* ToolsContext = ModeTools.GetInteractiveToolsContext())
	{
		ToolsContext->SetViewportInteractionsEnabled(false);
	}

	RegisterTabSpawners();

	if (UAvaEdMode* const AvaEdMode = Cast<UAvaEdMode>(ModeTools.GetActiveScriptableMode(UAvaEdMode::ModeID)))
	{
		if (TSharedPtr<FUICommandList> ToolkitCommands = AvaEdMode->GetToolkitCommands())
		{
			BindCommands(ToolkitCommands.ToSharedRef());
		}
	}

	ForEachExtension([](const TSharedRef<IAvaEditorExtension>& InExtension)
	{
		InExtension->Activate();
	});

	InvokeTabs();

	ForEachExtension([](const TSharedRef<IAvaEditorExtension>& InExtension)
	{
		InExtension->PostInvokeTabs();
	});
}

void FAvaEditor::Deactivate()
{
	if (!CanDeactivate())
	{
		return;
	}

	TGuardValue<bool> StateChangeGuard(bActiveStateChanging, true);

	bIsActive = false;

	NotifySceneActivationEvent();

	UnbindDelegates();

	DeactivateExtensions();

	if (UAvaEditorSubsystem* AvaEditorSubsystem = UAvaEditorSubsystem::Get(GetWorld()))
	{
		AvaEditorSubsystem->OnEditorDeactivated();
	}

	if (FEditorModeTools* const ModeTools = GetEditorModeTools())
	{
		ModeTools->DeactivateMode(UAvaEdMode::ModeID);

		// Standard Level Editor default is to support ITF gizmos and interactions
		ModeTools->SetSupportsViewportITF(true);

		// Re-enable viewport interactions for the Level Editor
		if (UModeManagerInteractiveToolsContext* ToolsContext = ModeTools->GetInteractiveToolsContext())
		{
			ToolsContext->SetViewportInteractionsEnabled(true);
		}
	}
}

void FAvaEditor::Cleanup()
{
	ForEachExtension([](const TSharedRef<IAvaEditorExtension>& InExtension)
	{
		InExtension->Cleanup();
	});
}

bool FAvaEditor::CanActivate() const
{
	return !IsActive() && !bActiveStateChanging;
}

bool FAvaEditor::ShouldAutoActivate() const
{
	return Provider->ShouldAutoActivateScene(GetSceneObject(EAvaEditorObjectQueryType::SearchOnly));
}

void FAvaEditor::SetAutoActivate(bool bInAutoActivate)
{
	Provider->SetAutoActivateScene(GetSceneObject(EAvaEditorObjectQueryType::SkipSearch), bInAutoActivate);
}

bool FAvaEditor::CanDeactivate() const
{
	return IsActive() && !bActiveStateChanging;
}

void FAvaEditor::SetToolkitHost(TSharedRef<IToolkitHost> InToolkitHost)
{
	ToolkitHostWeak = InToolkitHost;
}

void FAvaEditor::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	ForEachExtension([&InCommandList](const TSharedRef<IAvaEditorExtension>& InExtension)
	{
		InExtension->BindCommands(InCommandList);
	});
}

void FAvaEditor::Save()
{
	ForEachExtension([](const TSharedRef<IAvaEditorExtension>& InExtension)
	{
		InExtension->Save();
	});
}

void FAvaEditor::Load()
{
	ForEachExtension([](const TSharedRef<IAvaEditorExtension>& InExtension)
	{
		InExtension->Load();
	});
}

TSharedPtr<FUICommandList> FAvaEditor::GetCommandList() const
{
	if (const TSharedPtr<IToolkitHost> ToolkitHost = GetToolkitHost())
	{
		FEditorModeTools& ModeTools = ToolkitHost->GetEditorModeManager();

		if (UAvaEdMode* const AvaEdMode = Cast<UAvaEdMode>(ModeTools.GetActiveScriptableMode(UAvaEdMode::ModeID)))
		{
			if (TSharedPtr<FUICommandList> ToolkitCommands = AvaEdMode->GetToolkitCommands())
			{
				return ToolkitCommands.ToSharedRef();
			}
		}
	}

	return nullptr;
}

TSharedPtr<FTabManager> FAvaEditor::GetTabManager() const
{
	if (TSharedPtr<IToolkitHost> ToolkitHost = GetToolkitHost())
	{
		return ToolkitHost->GetTabManager();
	}
	return nullptr;
}

FEditorModeTools* FAvaEditor::GetEditorModeTools() const
{
	if (TSharedPtr<IToolkitHost> ToolkitHost = GetToolkitHost())
	{
		return &ToolkitHost->GetEditorModeManager();
	}
	return nullptr;
}

UWorld* FAvaEditor::GetWorld() const
{
	if (TSharedPtr<IToolkitHost> ToolkitHost = GetToolkitHost())
	{
		return ToolkitHost->GetWorld();
	}
	return nullptr;
}

void FAvaEditor::OnSceneObjectChanged()
{
	UObject* OldSceneObject = GetSceneObject(EAvaEditorObjectQueryType::SkipSearch);

	// Clear cached scene object as scene object changed
	SceneObjectWeak.Reset();

	UObject* NewSceneObject = GetSceneObject(EAvaEditorObjectQueryType::SearchOnly);

	ForEachExtension([OldSceneObject, NewSceneObject](const TSharedRef<IAvaEditorExtension>& InExtension)
	{
		InExtension->OnSceneObjectChanged(OldSceneObject, NewSceneObject);
	});
}

UObject* FAvaEditor::GetSceneObject(EAvaEditorObjectQueryType InQueryType) const
{
	// Return cached version if we are skipping search or if already valid
	if (InQueryType == EAvaEditorObjectQueryType::SkipSearch || SceneObjectWeak.IsValid())
	{
		return SceneObjectWeak.Get();
	}

	UObject* SceneObject = nullptr;
	bool bWasCreated = false;

	if (UWorld* SceneWorld = GetWorld())
	{
		// Check that the query types from here on are either find or create
		check(InQueryType == EAvaEditorObjectQueryType::SearchOnly || InQueryType == EAvaEditorObjectQueryType::CreateIfNotFound);

		SceneObject = Provider->GetSceneObject(SceneWorld, EAvaEditorObjectQueryType::SearchOnly);

		if (!SceneObject && InQueryType == EAvaEditorObjectQueryType::CreateIfNotFound)
		{
			SceneObject = Provider->GetSceneObject(SceneWorld, EAvaEditorObjectQueryType::CreateIfNotFound);
			bWasCreated = true;
		}
	}

	const_cast<FAvaEditor*>(this)->SceneObjectWeak = SceneObject;
	if (bWasCreated)
	{
		NotifySceneObjectCreated(SceneObject);
	}
	return SceneObject;
}

void FAvaEditor::RegisterTabSpawners()
{
	TSharedPtr<FTabManager> TabManager = GetTabManager();
	if (!TabManager.IsValid())
	{
		return;
	}

	if (!RootCategory.IsValid())
	{
		const TSharedRef<FWorkspaceItem> LevelEditorCategory = WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory();

		RootCategory = LevelEditorCategory->AddGroup(TEXT("MotionDesignGroup")
			, LOCTEXT("RootCategoryLabel", "Motion Design")
			, FSlateIcon(FAvaEditorCoreStyle::Get().GetStyleSetName(), "Icons.MotionDesign")
			, /*bSortChildren*/false);

		OutlinerCategory = RootCategory->AddGroup(TEXT("MotionDesignOutlinerGroup")
			, LOCTEXT("OutlinerCategoryLabel", "Motion Design Outliner")
			, FSlateIcon()
			, /*bSortChildren*/true);

		ToolsCategory = RootCategory->AddGroup(TEXT("MotionDesignToolsGroup")
			, LOCTEXT("ToolsCategoryLabel", "Motion Design Tools")
			, FSlateIcon()
			, /*bSortChildren*/true);
	}

	TSharedRef<IAvaEditor> This = SharedThis(this);

	// Allow Extensions to Register their own Tabs
	ForEachExtension([This](const TSharedRef<IAvaEditorExtension>& InExtension)
	{
		InExtension->RegisterTabSpawners(This);
	});

	TSharedRef<FTabManager> TabManagerRef = TabManager.ToSharedRef();

	for (const TPair<FName, TSharedRef<IAvaTabSpawner>>& Pair : TabSpawners)
	{
		if (!TabManager->HasTabSpawner(Pair.Key))
		{
			Pair.Value->RegisterTabSpawner(TabManagerRef, ToolsCategory);
		}
	}
}

void FAvaEditor::UnregisterTabSpawners()
{
	TSharedPtr<FTabManager> TabManager = GetTabManager();
	if (!TabManager.IsValid())
	{
		return;
	}

	for (const TPair<FName, TSharedRef<IAvaTabSpawner>>& Pair : TabSpawners)
	{
		TabManager->UnregisterTabSpawner(Pair.Key);
	}

	TabSpawners.Empty();

	if (RootCategory.IsValid() && FModuleManager::Get().IsModuleLoaded("WorkspaceMenuStructure"))
	{
		const TSharedRef<FWorkspaceItem> LevelEditorCategory = WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory();
		LevelEditorCategory->RemoveItem(RootCategory.ToSharedRef());
		RootCategory.Reset();
	}
}

void FAvaEditor::ExtendToolbarMenu(UToolMenu* InMenu)
{
	check(InMenu);
	ForEachExtension([InMenu](const TSharedRef<IAvaEditorExtension>& InExtension)
	{
		InExtension->ExtendToolbarMenu(*InMenu);
	});
}

void FAvaEditor::ExtendCreateSceneOptions(UToolMenu* InMenu)
{
	check(InMenu);

	ForEachExtension(
		[InMenu](const TSharedRef<IAvaEditorExtension>& InExtension)
		{
			InExtension->ExtendCreateSceneOptions(InMenu);
		});
}

void FAvaEditor::CloseTabs()
{
	TSharedPtr<FTabManager> TabManager = GetTabManager();
	if (!TabManager.IsValid())
	{
		return;
	}

	for (const TPair<FName, TSharedRef<IAvaTabSpawner>>& Pair : TabSpawners)
	{
		if (TSharedPtr<SDockTab> Tab = TabManager->FindExistingLiveTab(Pair.Key))
		{
			Tab->RequestCloseTab();
		}
	}
}

FReply FAvaEditor::DockInLayout(FName InTabId)
{
	TSharedPtr<FTabManager> TabManager = GetTabManager();

	if (!TabManager.IsValid())
	{
		return FReply::Unhandled();
	}

	if (GEditor)
	{
		GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->ForceDismissDrawer();
	}

	if (TSharedPtr<SDockTab> ExistingTab = TabManager->TryInvokeTab(InTabId))
	{
		ExistingTab->ActivateInParent(ETabActivationCause::SetDirectly);
	}

	return FReply::Handled();
}

TArray<TSharedRef<IAvaEditorExtension>> FAvaEditor::GetExtensions() const
{
	TArray<TSharedRef<IAvaEditorExtension>> OutExtensions;
	EditorExtensions.GenerateValueArray(OutExtensions);
	return OutExtensions;
}

TSharedPtr<IAvaEditorExtension> FAvaEditor::FindExtensionImpl(FAvaTypeId InExtensionId) const
{
	if (const TSharedRef<IAvaEditorExtension>* const FoundExtension = EditorExtensions.Find(InExtensionId))
	{
		return *FoundExtension;
	}
	return nullptr;
}

void FAvaEditor::AddTabSpawnerImpl(TSharedRef<IAvaTabSpawner> InTabSpawner)
{
	const FName Id = InTabSpawner->GetId();
	if (!TabSpawners.Contains(Id))
	{
		TabSpawners.Add(Id, InTabSpawner);
	}
}

bool FAvaEditor::EditCut()
{
	FString CopyData;
	if (CopyToString(CopyData))
	{
		FEditorDelegates::OnEditCutActorsBegin.Broadcast();

		FPlatformApplicationMisc::ClipboardCopy(*CopyData);

		FScopedTransaction Transaction(LOCTEXT("CutActors", "Cut Actors"));
		if (!EditDelete())
		{
			Transaction.Cancel();
		}

		FEditorDelegates::OnEditCutActorsEnd.Broadcast();
		return true;
	}
	return false;
}

bool FAvaEditor::EditCopy()
{
	FString CopyData;
	if (CopyToString(CopyData))
	{
		FEditorDelegates::OnEditCopyActorsBegin.Broadcast();

		FPlatformApplicationMisc::ClipboardCopy(*CopyData);

		FEditorDelegates::OnEditCopyActorsEnd.Broadcast();
		return true;
	}
	return false;
}

bool FAvaEditor::EditPaste()
{
	FString PastedData;
	FPlatformApplicationMisc::ClipboardPaste(PastedData);

	FScopedTransaction Transaction(LOCTEXT("PasteActors", "Paste Actors"));

	if (!PasteFromString(PastedData))
	{
		Transaction.Cancel();
		return false;
	}

	return true;
}

bool FAvaEditor::EditDuplicate()
{
	TGuardValue<bool> Guard(bDuplicating, true);

	FString CopyData;
	if (!CopyToString(CopyData))
	{
		return false;
	}

	FScopedTransaction Transaction(LOCTEXT("DuplicateActors", "Duplicate Actors"));

	if (!PasteFromString(CopyData))
	{
		Transaction.Cancel();
		return false;
	}

	return true;
}

bool FAvaEditor::EditDelete()
{
	UE::AvaEditorCore::Private::FAvaActorEditActionScope ActionScope(*this);

	if (!ActionScope.IsValid() || ActionScope.HasSelectedComponents())
	{
		return false;
	}

	TArray<AActor*> ActorsToDelete;
	ActionScope.ActorSelection->GetSelectedObjects(ActorsToDelete);

	ActorsToDelete.RemoveAll([](const AActor* InActor) { return !IsValid(InActor); });

	if (ActorsToDelete.IsEmpty())
	{
		return false;
	}

	TSet<AActor*> UnselectedAttachedActors;

	// Gather all Attached Actors to the Selected Actors that are not yet selected to optionally consider them for deletion
	{
		TSet<AActor*> ProcessedActors(ActorsToDelete);

		Algo::ForEach(ActorsToDelete,
			[ActorsToIgnore = TSet<AActor*>(ActorsToDelete), &UnselectedAttachedActors](const AActor* InActor)
			{
				if (!IsValid(InActor))
				{
					return;
				}

				TArray<AActor*> AttachedActors;
				InActor->GetAttachedActors(AttachedActors, /*bResetArray*/false, /*bRecursivelyIncludeAttachedActors*/true);

				for (AActor* const AttachedActor : AttachedActors)
				{
					if (!ActorsToIgnore.Contains(AttachedActor))
					{
						UnselectedAttachedActors.Add(AttachedActor);
					}
				}
			});
	}

	bool bDeleteUnselectedChildren = false;

	// If there are attached actors that are not selected, warn the user and ask whether to delete these attached actors as well
	if (!UnselectedAttachedActors.IsEmpty())
	{
		const FText MessageBoxTitle = LOCTEXT("DeleteChildrenTitle", "Children in Selection");

		const EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::YesNoCancel
			, LOCTEXT("DeleteChildrenMessage", "Would you like to delete the children of the selected object(s)?")
			, MessageBoxTitle);

		switch (Response)
		{
		case EAppReturnType::Yes:
			bDeleteUnselectedChildren = true;
			break;

		// Cancel Operation, no Actors to Delete
		case EAppReturnType::Cancel:
			return true;
		}
	}

	FScopedTransaction Transaction(LOCTEXT("DeleteActors", "Delete Actors"));

	if (bDeleteUnselectedChildren)
	{
		ActorsToDelete.Append(UnselectedAttachedActors.Array());

		ActionScope.ActorSelection->Modify();
		ActionScope.ActorSelection->BeginBatchSelectOperation();
		for (AActor* const Actor : UnselectedAttachedActors)
		{
			ActionScope.ActorSelection->Select(Actor);
		}
		ActionScope.ActorSelection->EndBatchSelectOperation(/*bNotify*/false);
	}

	bool bDeletedActors = false;

	// TODO: Delete is handled via ITypedElementWorldInterface::DeleteElements instead of UTypedElementCommonActions::DeleteSelectedElements
	// This is because Common Actions goes through FTypedElementCommonActionsCustomization which for an Actor calls FEditorModeTools::ProcessEditDelete (marked as TODO)
	// So current impl is not ideal because it would run on all Ed Modes including this one (and need to handle re-entry)
	UTypedElementSelectionSet* const SelectionSet = ActionScope.ActorSelection->GetElementSelectionSet();

	TMap<FTypedHandleTypeId, TArray<FTypedElementHandle>> ElementsToDeleteByType;
	TypedElementUtil::BatchElementsByType(SelectionSet->GetElementList(), ElementsToDeleteByType);

	UTypedElementRegistry* const Registry = UTypedElementRegistry::GetInstance();
	UTypedElementRegistry::FDisableElementDestructionOnGC GCGuard(Registry);

	const FTypedElementDeletionOptions DeletionOptions;

	for (const TPair<FTypedHandleTypeId, TArray<FTypedElementHandle>>& Pair : ElementsToDeleteByType)
	{
		ITypedElementWorldInterface* const WorldInterface = Registry->GetElementInterface<ITypedElementWorldInterface>(Pair.Key);
		bDeletedActors |= WorldInterface && WorldInterface->DeleteElements(Pair.Value, ActionScope.World, SelectionSet, DeletionOptions);
	}

	return bDeletedActors;
}

const TSharedPtr<FWorkspaceItem>& FAvaEditor::GetRootCategory() const
{
	return RootCategory;
}

const TSharedPtr<FWorkspaceItem>& FAvaEditor::GetOutlinerCategory() const
{
	return OutlinerCategory;
}

const TSharedPtr<FWorkspaceItem>& FAvaEditor::GetToolsCategory() const
{
	return ToolsCategory;
}

void FAvaEditor::DeactivateExtensions()
{
	// Guard Value to temporarily set Editor Active to false (while Deactivating the Extensions)
	// and restoring back to its original value (which could well still be false)
	TGuardValue<bool> EditorActive(bIsActive, false);

	// Force deactivation
	ForEachExtension([](const TSharedRef<IAvaEditorExtension>& InExtension)
		{
			InExtension->Deactivate();
		});
}

void FAvaEditor::NotifySceneObjectCreated(UObject* InSceneObject) const
{
	ForEachExtension(
		[InSceneObject](const TSharedRef<IAvaEditorExtension>& InExtension)
		{
			InExtension->OnSceneObjectCreated(InSceneObject);
		});
}

bool FAvaEditor::CopyToString(FString& OutCopyData) const
{
	UE::AvaEditorCore::Private::FAvaActorEditActionScope ActionScope(*this);

	if (!ActionScope.IsValid() || ActionScope.HasSelectedComponents())
	{
		return false;
	}

	UE::AvaEditorCore::Private::FAvaActorEditSelectionScope SelectionScope(*ActionScope.ActorSelection);

	TArray<AActor*> ActorsToCopy(SelectionScope.GetSelectedActors());
	Provider->GetActorsToEdit(ActorsToCopy);

	if (ActorsToCopy.IsEmpty())
	{
		return false;
	}

	SelectionScope.SetActorSelection(ActorsToCopy);

	// Copy Scope
	FString CopyData;
	{
		UE::AvaEditorCore::Private::FAvaActorEditTagScope TagScope(ActorsToCopy);

		if (TypedElementCommonActionsUtils::IsElementCopyAndPasteEnabled())
		{
			UTypedElementSelectionSet* const SelectionSet = ActionScope.ActorSelection->GetElementSelectionSet();
			if (!ActionScope.CommonActions->CopySelectedElementsToString(SelectionSet, CopyData))
			{
				return false;
			}
		}
		else
		{
			GUnrealEd->CopyActors(ActorsToCopy, ActionScope.World, &CopyData);
			if (CopyData.IsEmpty())
			{
				return false;
			}
		}

		ForEachExtension([&CopyData, &ActorsToCopy](const TSharedRef<IAvaEditorExtension>& InExtension)
		{
			FString ExtensionCopyData;
			InExtension->OnCopyActors(ExtensionCopyData, ActorsToCopy);
			CopyData.Append(MoveTemp(ExtensionCopyData));
		});
	}

	// Token added at the End as UUnrealEdEngine::PasteActors will fail if the Pasted String doesn't begin with "Begin Map"
	// and it's not guaranteed that this copied data will be used in a World that has AvaEditor
	// so pasting this should work as if it was a normal copy when destination has no AvaEditor to handle it
	CopyData += UE::AvaEditorCore::Private::AvalancheCopyToken;

	OutCopyData = MoveTemp(CopyData);
	return true;
}

bool FAvaEditor::PasteFromString(FString& InPastedData) const
{
	// Return failed early if Pasted Data is Empty, or did not have the Copy Token at the End
	// @see FAvaEditor::CopySelection for why the Token is added at the end
	if (InPastedData.IsEmpty() || !InPastedData.RemoveFromEnd(UE::AvaEditorCore::Private::AvalancheCopyToken))
	{
		return false;
	}

	UE::AvaEditorCore::Private::FAvaActorEditActionScope ActionScope(*this);
	if (!ActionScope.IsValid())
	{
		return false;
	}

	// Notify Extensions that Paste is about to begin
	ForEachExtension([](const TSharedRef<IAvaEditorExtension>& InExtension)
	{
		InExtension->PrePasteActors();
	});

	TArray<FAvaEditorPastedActor> PastedActors;

	auto ProcessPastedActor = [&PastedActors](AActor* InPastedActor)
	{
		if (IsValid(InPastedActor) && ensureAlways(!InPastedActor->Tags.IsEmpty()))
		{
			// Pop the Temporary Tag added on Copy, which would be the Actor Path of Source Actor
			const FName CopiedTag = InPastedActor->Tags.Pop();

			TSoftObjectPtr<AActor> SourceActor = TSoftObjectPtr<AActor>(FSoftObjectPath(CopiedTag.ToString()));

			PastedActors.Emplace(InPastedActor, MoveTemp(SourceActor));
		}
	};

	if (TypedElementCommonActionsUtils::IsElementCopyAndPasteEnabled())
	{
		UTypedElementSelectionSet* const SelectionSet = ActionScope.ActorSelection->GetElementSelectionSet();

		FTypedElementPasteOptions PasteOptions;
		PasteOptions.SelectionSetToModify = SelectionSet;
		PasteOptions.bPasteAtLocation     = false;

		const TArray<FTypedElementHandle> PastedHandles = ActionScope.CommonActions->PasteElements(SelectionSet
			, ActionScope.World, PasteOptions, &InPastedData);

		UTypedElementRegistry* const Registry = UTypedElementRegistry::GetInstance();
		check(Registry);

		// Worst case: all pasted handles are valid actors
		PastedActors.Reserve(PastedHandles.Num());

		for (const FTypedElementHandle& PastedHandle : PastedHandles)
		{
			if (TTypedElement<ITypedElementObjectInterface> PastedElement = Registry->GetElement<ITypedElementObjectInterface>(PastedHandle))
			{
				if (AActor* const Actor = PastedElement.GetObjectAs<AActor>())
				{
					ProcessPastedActor(Actor);
				}
			}
		}
	}
	else
	{
		TArray<AActor*> PastedActorsRaw;
		GUnrealEd->PasteActors(PastedActorsRaw, ActionScope.World, FVector::ZeroVector, bDuplicating, /*bWarnIfHidden*/false, &InPastedData);

		// Worst case: all pasted actors are valid
		PastedActors.Reserve(PastedActorsRaw.Num());

		Algo::ForEach(PastedActorsRaw, ProcessPastedActor);
	}

	if (PastedActors.IsEmpty())
	{
		// Notify that paste had no actors to paste
		ForEachExtension([](const TSharedRef<IAvaEditorExtension>& InExtension)
		{
			InExtension->PostPasteActors(/*bPasteSucceeded*/false);
		});
		return false;
	}

	ForEachExtension([&InPastedData, &PastedActors](const TSharedRef<IAvaEditorExtension>& InExtension)
	{
		InExtension->OnPasteActors(InPastedData, PastedActors);
	});

	ActionScope.ActorSelection->BeginBatchSelectOperation();
	ActionScope.ActorSelection->DeselectAll();
	for (const FAvaEditorPastedActor& PastedActor : PastedActors)
	{
		if (AActor* const Actor = PastedActor.GetActor())
		{
			ActionScope.ActorSelection->Select(Actor);
		}
	}
	ActionScope.ActorSelection->EndBatchSelectOperation(/*bNotify*/true);

	// Notify extensions that paste has ended
	ForEachExtension([](const TSharedRef<IAvaEditorExtension>& InExtension)
	{
		InExtension->PostPasteActors(/*bPasteSucceeded*/true);
	});

	return true;
}

void FAvaEditor::BindDelegates()
{
	UnbindDelegates();
	OnSelectionChangedHandle = USelection::SelectionChangedEvent.AddSP(this, &FAvaEditor::OnSelectionChanged);
	OnEnginePreExitHandle = FCoreDelegates::OnEnginePreExit.AddSP(this, &FAvaEditor::OnEnginePreExit);
}

void FAvaEditor::UnbindDelegates()
{
	USelection::SelectionChangedEvent.Remove(OnSelectionChangedHandle);
	OnSelectionChangedHandle.Reset();

	FCoreDelegates::OnEnginePreExit.Remove(OnEnginePreExitHandle);
	OnEnginePreExitHandle.Reset();
}

void FAvaEditor::OnEnginePreExit()
{
	Deactivate();
}

void FAvaEditor::OnSelectionChanged(UObject* InSelection)
{
	FEditorModeTools* const ModeTools = GetEditorModeTools();
	if (!ModeTools)
	{
		return;
	}

	const FAvaEditorSelection Selection(*ModeTools, InSelection);

	if (!Selection.IsValid())
	{
		return;
	}

	ForEachExtension([&Selection](const TSharedRef<IAvaEditorExtension>& InExtension)
	{
		InExtension->NotifyOnSelectionChanged(Selection);
	});
}

#undef LOCTEXT_NAMESPACE
