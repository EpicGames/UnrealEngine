// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "Delegates/IDelegateInstance.h"
#include "IAvaEditor.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

class FAvaEditorBuilder;
class FLayoutExtender;
class FUICommandList;
class FWorkspaceItem;
class IAvaEditorExtension;
class IAvaEditorProvider;
class IAvaTabSpawner;

class FAvaEditor : public IAvaEditor
{
public:
	UE_AVA_INHERITS(FAvaEditor, IAvaEditor);

	explicit FAvaEditor(FAvaEditorBuilder& Initializer);

	virtual ~FAvaEditor() override;

protected:
	virtual void ExtendLayout(FLayoutExtender& InExtender) {}

	void InvokeTabs();

	void ForEachExtension(TFunctionRef<void(const TSharedRef<IAvaEditorExtension>&)> InFunc) const;

	void NotifySceneActivationEvent();

	//~ Begin IAvaEditor
	virtual void Construct() override;
	virtual void Activate(TSharedPtr<IToolkitHost> InOverrideToolkitHost = nullptr) override;
	virtual void Deactivate() override;
	virtual void Cleanup() override;
	virtual bool IsActive() const { return bIsActive; }
	virtual bool CanActivate() const override;
	virtual bool ShouldAutoActivate() const override;
	virtual void SetAutoActivate(bool bInAutoActivate) override;
	virtual bool CanDeactivate() const override;
	virtual void SetToolkitHost(TSharedRef<IToolkitHost> InToolkitHost) override;
	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandList) override;
	virtual void Save() override;
	virtual void Load() override;
	virtual TSharedPtr<FUICommandList> GetCommandList() const override;
	virtual TSharedPtr<IToolkitHost> GetToolkitHost() const override { return ToolkitHostWeak.Pin(); }
	virtual TSharedPtr<FTabManager> GetTabManager() const override;
	virtual FEditorModeTools* GetEditorModeTools() const override;
	virtual UWorld* GetWorld() const override;
	virtual void OnSceneObjectChanged() override;
	virtual UObject* GetSceneObject(EAvaEditorObjectQueryType InQueryType) const override;
	virtual void RegisterTabSpawners() override;
	virtual void UnregisterTabSpawners() override;
	virtual void ExtendToolbarMenu(UToolMenu* InMenu) override;
	virtual void ExtendCreateSceneOptions(UToolMenu* InMenu) override;
	virtual void CloseTabs() override;
	virtual FReply DockInLayout(FName InTabId) override;
	virtual TArray<TSharedRef<IAvaEditorExtension>> GetExtensions() const override;
	virtual TSharedPtr<IAvaEditorExtension> FindExtensionImpl(FAvaTypeId InExtensionId) const override;
	virtual void AddTabSpawnerImpl(TSharedRef<IAvaTabSpawner> InTabSpawner) override;
	virtual void OnSelectionChanged(UObject* InSelection) override;
	virtual bool EditCut() override;
	virtual bool EditCopy() override;
	virtual bool EditPaste() override;
	virtual bool EditDuplicate() override;
	virtual bool EditDelete() override;
	virtual const TSharedPtr<FWorkspaceItem>& GetRootCategory() const override;
	virtual const TSharedPtr<FWorkspaceItem>& GetOutlinerCategory() const override;
	virtual const TSharedPtr<FWorkspaceItem>& GetToolsCategory() const override;
	//~ End IAvaEditor

	void DeactivateExtensions();

private:
	/** Called when the scene object has been created for the first time */
	void NotifySceneObjectCreated(UObject* InSceneObject) const;

	bool CopyToString(FString& OutCopyData) const;

	bool PasteFromString(FString& InPastedData) const;

	void BindDelegates();

	void UnbindDelegates();

	void OnEnginePreExit();

	TSharedRef<IAvaEditorProvider> Provider;

	TMap<FAvaTypeId, TSharedRef<IAvaEditorExtension>> EditorExtensions;

	TMap<FName, TSharedRef<IAvaTabSpawner>> TabSpawners;

	TWeakPtr<IToolkitHost> ToolkitHostWeak;

	/** Cached Scene Object to avoid having to call OnGetSceneObject everytime */
	TWeakObjectPtr<UObject> SceneObjectWeak;

	/** Tab Categories */
	TSharedPtr<FWorkspaceItem> RootCategory;
	TSharedPtr<FWorkspaceItem> OutlinerCategory;
	TSharedPtr<FWorkspaceItem> ToolsCategory;

	FDelegateHandle OnSelectionChangedHandle;

	FDelegateHandle OnEnginePreExitHandle;

	bool bIsActive = false;

	bool bDuplicating = false;

	bool bActiveStateChanging = false;
};
