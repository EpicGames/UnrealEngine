// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEditorBuilder.h"
#include "AvaEditor.h"

FAvaEditorBuilder::FOnEditorBuild FAvaEditorBuilder::OnEditorBuild;

FAvaEditorBuilder::~FAvaEditorBuilder()
{
	ensureAlwaysMsgf(bFinalized, TEXT("FAvaEditorBuilder being destroyed without having ever finalized!"));
}

FName FAvaEditorBuilder::GetIdentifier() const
{
	return Identifier;
}

FAvaEditorBuilder& FAvaEditorBuilder::SetIdentifier(FName InIdentifier)
{
	Identifier = InIdentifier;
	return *this;
}

TSharedRef<IAvaEditor> FAvaEditorBuilder::Build()
{
	checkf(!bFinalized, TEXT("Calling Build again on the same FAvaEditorBuilder is not allowed"));

	bFinalized = true;

	OnEditorBuild.Broadcast(*this);

	checkf(Provider.IsValid(), TEXT("Editor Provider invalid! Calling FAvaEditorBuilder::SetProvider prior to building the editor is compulsory"));
	Provider->Construct();

	TSharedRef<IAvaEditor> Editor = CreateEditor();
	Editor->Construct();
	return Editor;
}

TSharedRef<IAvaEditor> FAvaEditorBuilder::CreateEditor()
{
	return MakeShared<FAvaEditor>(*this);
}
