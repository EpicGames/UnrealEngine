// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEditorCoreModule.h"
#include "AvaEditorExtensionTypeRegistry.h"
#include "LevelEditor/AvaLevelEditorCommands.h"

IMPLEMENT_MODULE(FAvaEditorCoreModule, AvalancheEditorCore)

void FAvaEditorCoreModule::StartupModule()
{
	FAvaLevelEditorCommands::Register();
}

void FAvaEditorCoreModule::ShutdownModule()
{
	FAvaLevelEditorCommands::Unregister();
	FAvaEditorExtensionTypeRegistry::Get().Shutdown();
}

IAvaEditorCoreModule::FOnExtendEditorToolbar& FAvaEditorCoreModule::GetOnExtendEditorToolbar()
{
	return OnExtendEditorToolbar;
}
