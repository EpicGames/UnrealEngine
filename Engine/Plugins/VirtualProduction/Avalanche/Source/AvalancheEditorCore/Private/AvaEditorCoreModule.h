// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaEditorCoreModule.h"

class FAvaEditorCoreModule : public IAvaEditorCoreModule
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	//~ Begin IAvaEditorCoreModule
	virtual FOnExtendEditorToolbar& GetOnExtendEditorToolbar();
	//~ End IAvaEditorCoreModule

private:
	FOnExtendEditorToolbar OnExtendEditorToolbar;
};
