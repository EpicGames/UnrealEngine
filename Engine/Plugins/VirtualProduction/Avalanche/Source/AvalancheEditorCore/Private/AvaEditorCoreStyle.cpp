// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEditorCoreStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

FAvaEditorCoreStyle& FAvaEditorCoreStyle::Get()
{
	static FAvaEditorCoreStyle Instance;
	return Instance;
}

FAvaEditorCoreStyle::FAvaEditorCoreStyle()
	: FSlateStyleSet(TEXT("AvaEditorCore"))
{
	const FVector2f Icon16(16.f);

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	FSlateStyleSet::SetContentRoot(Plugin->GetBaseDir() / TEXT("Resources"));
	FSlateStyleSet::SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));

	Set("Icons.MotionDesign", new IMAGE_BRUSH_SVG("Icons/MotionDesign_16", Icon16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FAvaEditorCoreStyle::~FAvaEditorCoreStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
