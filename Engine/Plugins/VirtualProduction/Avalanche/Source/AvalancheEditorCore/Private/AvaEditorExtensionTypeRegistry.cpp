// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEditorExtensionTypeRegistry.h"

FAvaEditorExtensionTypeRegistry::~FAvaEditorExtensionTypeRegistry()
{
	Shutdown();
}

FAvaEditorExtensionTypeRegistry& FAvaEditorExtensionTypeRegistry::Get()
{
	static FAvaEditorExtensionTypeRegistry Registry;
	return Registry;
}

void FAvaEditorExtensionTypeRegistry::RegisterExtensionType(const TSharedRef<IAvaEditorExtension>& InExtension, FAvaEditorExtensionType&& InType)
{
	bool bIsAlreadyInSet;
	FSetElementId ElementId = ExtensionTypes.Emplace(MoveTemp(InType), &bIsAlreadyInSet);

	FAvaEditorExtensionType& ExtensionType = ExtensionTypes[ElementId];
	ExtensionType.Extensions.Add(InExtension);

	// If first time initializing call StaticStartup()
	if (!bIsAlreadyInSet && ExtensionType.Startup)
	{
		(*ExtensionType.Startup)();
	}
}

void FAvaEditorExtensionTypeRegistry::Shutdown()
{
	for (FAvaEditorExtensionType& ExtensionType : ExtensionTypes)
	{
		if (ExtensionType.Shutdown)
		{
			(*ExtensionType.Shutdown)();
		}
	}
	ExtensionTypes.Empty();
}

void FAvaEditorExtensionTypeRegistry::RemoveStaleEntries()
{
	for (TSet<FAvaEditorExtensionType>::TIterator Iter(ExtensionTypes); Iter; ++Iter)
	{
		FAvaEditorExtensionType& ExtensionType = *Iter;
		ExtensionType.Extensions.RemoveAll([](const TWeakPtr<IAvaEditorExtension>& InExtension)
		{
			return !InExtension.IsValid();
		});

		// If no live extensions remaining, call StaticShutdown()
		if (ExtensionType.Extensions.IsEmpty())
		{
			if (ExtensionType.Shutdown)
			{
				(*ExtensionType.Shutdown)();	
			}
			Iter.RemoveCurrent();
		}
	}
}
