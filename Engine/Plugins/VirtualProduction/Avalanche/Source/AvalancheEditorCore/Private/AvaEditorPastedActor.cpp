// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEditorPastedActor.h"
#include "Containers/Map.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"

FAvaEditorPastedActor::FAvaEditorPastedActor(AActor* InActor, TSoftObjectPtr<AActor>&& InTemplateActor)
	: Actor(InActor)
	, TemplateActor(MoveTemp(InTemplateActor))
{
}

AActor* FAvaEditorPastedActor::GetActor() const
{
	return Actor.Get();
}

const TSoftObjectPtr<AActor>& FAvaEditorPastedActor::GetTemplateActor() const
{
	return TemplateActor;
}

bool FAvaEditorPastedActor::IsDuplicate() const
{
	AActor* const ResolvedTemplateActor = GetTemplateActor().Get();
	AActor* const ResolvedPastedActor = GetActor();

	return IsValid(ResolvedTemplateActor)
		&& IsValid(ResolvedPastedActor)
		&& ResolvedTemplateActor->GetWorld() == ResolvedPastedActor->GetWorld();
}

TMap<FName, AActor*> FAvaEditorPastedActor::BuildPastedActorMap(TConstArrayView<FAvaEditorPastedActor> InPastedActors, bool bIncludeDuplicatedActors)
{
	TMap<FName, AActor*> OutPastedActorMap;
	OutPastedActorMap.Reserve(InPastedActors.Num());

	for (const FAvaEditorPastedActor& PastedActor : InPastedActors)
	{
		AActor* const Actor = PastedActor.GetActor();
		if (!Actor)
		{
			continue;
		}

		FName TemplateName = NAME_None;

		if (PastedActor.IsDuplicate())
		{
			if (!bIncludeDuplicatedActors)
			{
				continue;
			}
			TemplateName = *PastedActor.GetTemplateActor()->GetPathName(PastedActor.GetTemplateActor()->GetWorld());
		}
		else
		{
			// If it comes from a Copy where the Template Actor is invalid, get its name via the sub path string
			TemplateName = *PastedActor.GetTemplateActor().ToSoftObjectPath().GetSubPathString();
		}

		OutPastedActorMap.Add(TemplateName, Actor);
	}

	return OutPastedActorMap;
}

TMap<AActor*, AActor*> FAvaEditorPastedActor::BuildDuplicatedActorMap(TConstArrayView<FAvaEditorPastedActor> InPastedActors)
{
	TMap<AActor*, AActor*> OutDuplicatedActorMap;

	// worst case: all pasted actors are duplicate
	OutDuplicatedActorMap.Reserve(InPastedActors.Num());

	for (const FAvaEditorPastedActor& PastedActor : InPastedActors)
	{
		// Pasted actor considered duplicate if template actor is valid and belongs in the same world as the pasted actor
		if (PastedActor.IsDuplicate())
		{
			OutDuplicatedActorMap.Add(PastedActor.GetActor(), PastedActor.GetTemplateActor().Get());
		}
	}

	return OutDuplicatedActorMap;
}
