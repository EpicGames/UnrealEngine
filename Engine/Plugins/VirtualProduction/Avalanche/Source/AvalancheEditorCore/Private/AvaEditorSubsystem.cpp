// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEditorSubsystem.h"

bool UAvaEditorSubsystem::DoesSupportWorldType(const EWorldType::Type InWorldType) const
{
	return InWorldType == EWorldType::Editor;
}

void UAvaEditorSubsystem::OnEditorActivated(const TSharedRef<IAvaEditor>& InAvaEditor)
{
	ActiveEditorWeak = InAvaEditor;
}

void UAvaEditorSubsystem::OnEditorDeactivated()
{
	ActiveEditorWeak.Reset();
}
