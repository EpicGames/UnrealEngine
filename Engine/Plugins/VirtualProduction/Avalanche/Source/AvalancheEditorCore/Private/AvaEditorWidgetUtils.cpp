// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEditorWidgetUtils.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Internationalization/Text.h"
#include "Layout/ChildrenBase.h"
#include "Math/MathFwd.h"
#include "UObject/NameTypes.h"
#include "Widgets/SlateControlledConstruction.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "AvaEditorWidgetUtils"

namespace UE::AvaEditor::Private
{
	void GetWidgetChildrenOfClass(const TSharedRef<SWidget>& Widget, const FName& WidgetClassName, TArray<TSharedRef<SWidget>>& Widgets)
	{
		if (Widget->GetWidgetClass().GetWidgetType() == WidgetClassName)
		{
			Widgets.Add(Widget);
		}

		FChildren* Children = Widget->GetChildren();

		if (Children && Children->Num() > 0)
		{
			for (int32 ChildIdx = 0; ChildIdx < Children->Num(); ++ChildIdx)
			{
				GetWidgetChildrenOfClass(Children->GetChildAt(ChildIdx), WidgetClassName, Widgets);
			}
		}
	}
}

TArray<TSharedRef<SWidget>> FAvaEditorWidgetUtils::GetWidgetChildrenOfClass(const TSharedPtr<SWidget>& InWidget, const FSlateWidgetClassData& InWidgetClass)
{
	TArray<TSharedRef<SWidget>> Widgets;

	UE::AvaEditor::Private::GetWidgetChildrenOfClass(InWidget.ToSharedRef(), InWidgetClass.GetWidgetType(), Widgets);

	return Widgets;
}

TSharedPtr<SWidget> FAvaEditorWidgetUtils::FindParentWidgetWithClass(const TSharedPtr<SWidget>& InWidget, const FName& InWidgetClassName)
{
	if (InWidget->GetWidgetClass().GetWidgetType() == InWidgetClassName)
	{
		return InWidget;
	}

	if (!InWidget->GetParentWidget().IsValid())
	{
		return nullptr;
	}

	return FindParentWidgetWithClass(InWidget->GetParentWidget(), InWidgetClassName);
}

FVector2f FAvaEditorWidgetUtils::GetWidgetScale(const SWidget* InWidget)
{
	if (!InWidget)
	{
		return FVector2f::ZeroVector;
	}

	const FVector2f AbsoluteSize = InWidget->GetTickSpaceGeometry().GetAbsoluteSize();
	const FVector2f LocalSize = InWidget->GetTickSpaceGeometry().GetLocalSize();

	if (FMath::IsNearlyZero(AbsoluteSize.X) || FMath::IsNearlyZero(AbsoluteSize.Y) || FMath::IsNearlyZero(LocalSize.X) || FMath::IsNearlyZero(LocalSize.Y))
	{
		return FVector2f(1.f, 1.f);
	}

	return AbsoluteSize / LocalSize;
}

FVector2f FAvaEditorWidgetUtils::GetWidgetScale(const TSharedRef<SWidget> InWidget)
{
	const FVector2f AbsoluteSize = InWidget->GetTickSpaceGeometry().GetAbsoluteSize();
	const FVector2f LocalSize = InWidget->GetTickSpaceGeometry().GetLocalSize();

	if (FMath::IsNearlyZero(AbsoluteSize.X) || FMath::IsNearlyZero(AbsoluteSize.Y) || FMath::IsNearlyZero(LocalSize.X) || FMath::IsNearlyZero(LocalSize.Y))
	{
		return FVector2f(1.f, 1.f);
	}

	return AbsoluteSize / LocalSize;
}

FText FAvaEditorWidgetUtils::AddKeybindToTooltip(FText InDefaultTooltip, TSharedPtr<FUICommandInfo> InCommand)
{
	if (InCommand.IsValid())
	{
		const TSharedRef<const FInputChord>& Chord = InCommand->GetFirstValidChord();

		if (Chord->IsValidChord())
		{
			return FText::Format(LOCTEXT("LineBreak", "{0}\n\nKeybind: {1}"), InDefaultTooltip, Chord->GetInputText(false));
		}
	}

	return InDefaultTooltip;
}

#undef LOCTEXT_NAMESPACE
