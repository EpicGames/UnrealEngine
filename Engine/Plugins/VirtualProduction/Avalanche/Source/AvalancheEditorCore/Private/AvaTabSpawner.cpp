// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTabSpawner.h"
#include "IAvaEditor.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SNullWidget.h"

FAvaTabSpawner::FAvaTabSpawner(const TSharedRef<IAvaEditor>& InEditor, FName InTabId)
	: EditorWeak(InEditor)
	, TabId(InTabId)
	, TabRole(ETabRole::PanelTab)
{
}

bool FAvaTabSpawner::CanSpawnTab(const FSpawnTabArgs& InArgs, TWeakPtr<FTabManager> InTabManagerWeak) const
{
	return HasValidScene();
}

TSharedRef<SDockTab> FAvaTabSpawner::SpawnTab(const FSpawnTabArgs& InArgs, TWeakPtr<FTabManager> InTabManagerWeak)
{
	TSharedRef<SDockTab> NewTab = SNew(SDockTab)
		.TabRole(TabRole)
		.Label(TabLabel)
		.ToolTipText(TabTooltipText)
		.ShouldAutosize(bAutosizeTab);

	DockTab = NewTab;

	TSharedRef<SWidget> TabBody = CreateTabBody();

	// Pad the content if requested
	if (TabInnerPadding > 0.0f)
	{
		// propagate tag from original content, or use TabId
		TSharedPtr<FTagMetaData> MetaData = TabBody->GetMetaData<FTagMetaData>();

		TabBody = SNew(SBorder)
			.Padding(TabInnerPadding)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.AddMetaData<FTagMetaData>(MetaData.IsValid() ? MetaData->Tag : GetId())
			[
				TabBody
			];
	}

	NewTab->SetContent(TabBody);
	return NewTab;
}

FTabSpawnerEntry& FAvaTabSpawner::RegisterTabSpawner(const TSharedRef<FTabManager>& InTabManager, const TSharedPtr<FWorkspaceItem>& InWorkspaceMenu)
{
	TWeakPtr<FTabManager> WeakTabManager(InTabManager);

	TSharedRef<FAvaTabSpawner> This = SharedThis(this);

	const FCanSpawnTab CanSpawnTab = FCanSpawnTab::CreateSP(This, &IAvaTabSpawner::CanSpawnTab, WeakTabManager);
	const FOnSpawnTab SpawnTab = FOnSpawnTab::CreateSP(This, &IAvaTabSpawner::SpawnTab, WeakTabManager);

	FTabSpawnerEntry& SpawnerEntry = InTabManager->RegisterTabSpawner(GetId(), SpawnTab, CanSpawnTab)
		.SetDisplayName(TabLabel)
		.SetTooltipText(TabTooltipText);

	if (InWorkspaceMenu.IsValid())
	{
		SpawnerEntry.SetGroup(InWorkspaceMenu.ToSharedRef());
	}

	if (TabIcon.IsSet())
	{
		SpawnerEntry.SetIcon(TabIcon);
	}

	return SpawnerEntry;
}

bool FAvaTabSpawner::HasValidScene() const
{
	const TSharedPtr<IAvaEditor> Editor = EditorWeak.Pin();
	return Editor.IsValid() && IsValid(Editor->GetSceneObject(EAvaEditorObjectQueryType::SkipSearch));
}

TSharedRef<SWidget> FAvaTabSpawner::GetNullWidget() const
{
	return SNullWidget::NullWidget;
}
