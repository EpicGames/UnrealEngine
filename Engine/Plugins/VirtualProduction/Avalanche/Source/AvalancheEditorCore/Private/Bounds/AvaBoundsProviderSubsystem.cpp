// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bounds/AvaBoundsProviderSubsystem.h"
#include "Containers/Array.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Selection.h"
#include "Selection/AvaSelectionProviderSubsystem.h"

namespace UE::AvaEditorCore::Private
{
	FBox InvalidBox = FBox(EForceInit::ForceInit);
	FOrientedBox DefaultOrientedBox = FOrientedBox();

	FOrientedBox MakeOrientedBox(const FBox& InLocalBox, const FTransform& InWorldTransform)
	{
		FOrientedBox OutOrientedBox;

		OutOrientedBox.Center = InWorldTransform.TransformPosition(InLocalBox.GetCenter());

		OutOrientedBox.AxisX = InWorldTransform.TransformVector(FVector::UnitX());
		OutOrientedBox.AxisY = InWorldTransform.TransformVector(FVector::UnitY());
		OutOrientedBox.AxisZ = InWorldTransform.TransformVector(FVector::UnitZ());

		OutOrientedBox.ExtentX = (InLocalBox.Max.X - InLocalBox.Min.X) * 0.5;
		OutOrientedBox.ExtentY = (InLocalBox.Max.Y - InLocalBox.Min.Y) * 0.5;
		OutOrientedBox.ExtentZ = (InLocalBox.Max.Z - InLocalBox.Min.Z) * 0.5;

		return OutOrientedBox;
	}
}

bool UAvaBoundsProviderSubsystem::DoesSupportWorldType(const EWorldType::Type InWorldType) const
{
	return InWorldType == EWorldType::Editor;
}

bool UAvaBoundsProviderSubsystem::CacheComponentLocalBounds(UPrimitiveComponent* InComponent)
{
	if (!InComponent)
	{
		return false;
	}

	if (!InComponent->IsRegistered())
	{
		return false;
	}

	if (InComponent->IsVisualizationComponent())
	{
		return false;
	}

	// Pre-scale component to be consistent with actor bounding boxes
	const FBoxSphereBounds BoxSphereBounds = InComponent->CalcBounds(FTransform::Identity);
	FBox Box = BoxSphereBounds.GetBox();
	Box.IsValid = 1;

	CachedComponentLocalBounds.Emplace(InComponent, MoveTemp(Box));

	return true;
}

bool UAvaBoundsProviderSubsystem::CacheComponentWorldOrientedBounds(UPrimitiveComponent* InComponent)
{
	if (!InComponent)
	{
		return false;
	}

	if (CachedComponentWorldOrientedBounds.Contains(InComponent))
	{
		return true;
	}

	using namespace UE::AvaEditorCore::Private;

	const FBox LocalBounds = GetComponentLocalBounds(InComponent);

	if (!LocalBounds.IsValid)
	{
		return false;
	}

	FOrientedBox OrientedBounds = MakeOrientedBox(LocalBounds, InComponent->GetComponentTransform());
	CachedComponentWorldOrientedBounds.Emplace(InComponent, MoveTemp(OrientedBounds));

	return true;
}

bool UAvaBoundsProviderSubsystem::CacheComponentActorOrientedBounds(UPrimitiveComponent* InComponent)
{
	if (!InComponent)
	{
		return false;
	}

	AActor* Owner = InComponent->GetOwner();

	if (!Owner || !Owner->GetRootComponent())
	{
		return false;
	}

	if (CachedComponentActorOrientedBounds.Contains(InComponent))
	{
		return true;
	}

	using namespace UE::AvaEditorCore::Private;

	const FBox LocalBounds = GetComponentLocalBounds(InComponent);

	if (!LocalBounds.IsValid)
	{
		return false;
	}

	FTransform ComponentTransform = FTransform::Identity;

	if (InComponent != Owner->GetRootComponent())
	{
		FTransform ComponentToWorld = InComponent->GetComponentTransform();
		FTransform ActorToWorld = Owner->GetActorTransform();

		ComponentTransform = ComponentToWorld * ActorToWorld.Inverse();
	}

	FOrientedBox OrientedBounds = MakeOrientedBox(LocalBounds, ComponentTransform);
	CachedComponentActorOrientedBounds.Emplace(InComponent, MoveTemp(OrientedBounds));

	return true;
}

bool UAvaBoundsProviderSubsystem::CacheActorLocalBounds(AActor* InActor)
{
	if (!InActor || !InActor->GetRootComponent())
	{
		return false;
	}

	if (CachedActorLocalBounds.Contains(InActor))
	{
		return true;
	}

	constexpr bool bIncludeFromChildActors = false;

	FBox Box(ForceInit);
	Box.IsValid = 0;

	FTransform ActorToWorld = InActor->GetTransform();
	ActorToWorld.SetScale3D(FVector::OneVector);
	FTransform WorldToActor = ActorToWorld.Inverse();

	uint32 FailedComponentCount = 0;
	InActor->ForEachComponent<UPrimitiveComponent>(bIncludeFromChildActors, [this, &FailedComponentCount, &WorldToActor, &Box](UPrimitiveComponent* InPrimitiveComponent)
		{
			FOrientedBox OrientedBox;
			const bool bIsValidBounds = GetComponentActorOrientedBounds(InPrimitiveComponent, OrientedBox);

			if (!bIsValidBounds)
			{
				FailedComponentCount++;
				return;
			}

			FVector Vertices[8];
			OrientedBox.CalcVertices(Vertices);

			for (const FVector& Vertex : Vertices)
			{
				Box += Vertex;
			}
		});

	// Actors with no Failed Primitives should still return a valid Box with no Extents and 0,0,0 origin (local).
	if (FailedComponentCount == 0)
	{
		Box.IsValid = 1;
	}

	CachedActorLocalBounds.Emplace(InActor, MoveTemp(Box));

	return true;
}

bool UAvaBoundsProviderSubsystem::CacheActorOrientedBounds(AActor* InActor)
{
	if (!InActor || !InActor->GetRootComponent())
	{
		return false;
	}

	if (CachedActorOrientedBounds.Contains(InActor))
	{
		return true;
	}

	using namespace UE::AvaEditorCore::Private;

	const FBox LocalBounds = GetActorLocalBounds(InActor);

	if (!LocalBounds.IsValid)
	{
		return false;
	}

	FOrientedBox OrientedBounds = MakeOrientedBox(LocalBounds, InActor->GetActorTransform());
	CachedActorOrientedBounds.Emplace(InActor, MoveTemp(OrientedBounds));

	return true;
}

bool UAvaBoundsProviderSubsystem::CacheActorAndChildrenLocalBounds(AActor* InActor)
{
	if (!InActor || !InActor->GetRootComponent())
	{
		return false;
	}

	if (CachedActorAndChildrenLocalBounds.Contains(InActor))
	{
		return true;
	}

	FBox TotalSize = GetActorLocalBounds(InActor);

	if (!TotalSize.IsValid)
	{
		return false;
	}

	UAvaSelectionProviderSubsystem* SelectionProvider = UAvaSelectionProviderSubsystem::Get(this, /* bInGenerateErrors */ true);

	if (!SelectionProvider)
	{
		return false;
	}

	TConstArrayView<TWeakObjectPtr<AActor>> ChildActors = SelectionProvider->GetAttachedActors(InActor, true);

	if (ChildActors.IsEmpty())
	{
		TotalSize.IsValid = 1;
	}
	else
	{
		using namespace UE::AvaEditorCore::Private;

		FTransform ActorTransform = InActor->GetActorTransform();
		FTransform InverseActorTransformNoScale = FTransform(ActorTransform.GetRotation(), ActorTransform.GetLocation()).Inverse();
		FVector Vertices[8];

		for (TWeakObjectPtr<AActor> ChildActorWeak : ChildActors)
		{
			if (AActor* ChildActor = ChildActorWeak.Get())
			{
				const FBox ChildLocalBounds = GetActorLocalBounds(ChildActor);

				if (ChildLocalBounds.IsValid)
				{
					// Child scale must be calculated at the final orientation due to local scaling.
					const FTransform ChildTransform = ChildActor->GetActorTransform();
					const FTransform ChildTransformRelativeScale(ChildTransform.GetRotation(), ChildTransform.GetLocation(), ChildTransform.GetScale3D() / ActorTransform.GetScale3D());

					// Cannot use precomputed oriented box because it needs to be relative to the base actor
					const FOrientedBox ChildOrientedBounds = MakeOrientedBox(ChildLocalBounds, ChildTransformRelativeScale * InverseActorTransformNoScale);
					ChildOrientedBounds.CalcVertices(Vertices);

					for (FVector& Vertex : Vertices)
					{
						TotalSize += Vertex;
					}

					TotalSize.IsValid = 1;
				}
			}
		}
	}

	CachedActorAndChildrenLocalBounds.Emplace(InActor, MoveTemp(TotalSize));

	return true;
}

bool UAvaBoundsProviderSubsystem::CacheActorAndChildrenOrientedBounds(AActor* InActor)
{
	if (!InActor || !InActor->GetRootComponent())
	{
		return false;
	}

	if (CachedActorAndChildrenOrientedBounds.Contains(InActor))
	{
		return true;
	}

	using namespace UE::AvaEditorCore::Private;

	const FBox LocalBounds = GetActorAndChildrenLocalBounds(InActor);

	if (!LocalBounds.IsValid)
	{
		return false;
	}

	FOrientedBox OrientedBounds = MakeOrientedBox(LocalBounds, InActor->GetActorTransform());
	CachedActorAndChildrenOrientedBounds.Emplace(InActor, MoveTemp(OrientedBounds));

	return true;
}

void UAvaBoundsProviderSubsystem::ClearCachedBounds()
{
	CachedComponentLocalBounds.Empty();
	CachedComponentWorldOrientedBounds.Empty();
	CachedComponentActorOrientedBounds.Empty();
	CachedActorLocalBounds.Empty();
	CachedActorOrientedBounds.Empty();
	CachedActorAndChildrenLocalBounds.Empty();
	CachedActorAndChildrenOrientedBounds.Empty();
	CachedSelectionBounds = FBox(EForceInit::ForceInit);
	CachedSelectionWithChildrenBounds = FBox(EForceInit::ForceInit);
}

bool UAvaBoundsProviderSubsystem::HasCachedComponentLocalBounds(UPrimitiveComponent* InComponent) const
{
	return CachedComponentLocalBounds.Contains(InComponent);
}

bool UAvaBoundsProviderSubsystem::HasCachedComponentWorldOrientedBounds(UPrimitiveComponent* InComponent) const
{
	return CachedComponentWorldOrientedBounds.Contains(InComponent);
}

bool UAvaBoundsProviderSubsystem::HasCachedComponentActorOrientedBounds(UPrimitiveComponent* InComponent) const
{
	return CachedComponentActorOrientedBounds.Contains(InComponent);
}

bool UAvaBoundsProviderSubsystem::HasCachedActorLocalBounds(AActor* InActor) const
{
	return CachedActorLocalBounds.Contains(InActor);
}

bool UAvaBoundsProviderSubsystem::HasCachedActorOrientedBounds(AActor* InActor) const
{
	return CachedActorOrientedBounds.Contains(InActor);
}

bool UAvaBoundsProviderSubsystem::HasCachedActorAndChildrenLocalBounds(AActor* InActor) const
{
	return CachedActorAndChildrenLocalBounds.Contains(InActor);
}

bool UAvaBoundsProviderSubsystem::HasCachedActorAndChildrenOrientedBounds(AActor* InActor) const
{
	return CachedActorAndChildrenOrientedBounds.Contains(InActor);
}

FBox UAvaBoundsProviderSubsystem::GetComponentLocalBounds(UPrimitiveComponent* InComponent)
{
	if (CacheComponentLocalBounds(InComponent))
	{
		return CachedComponentLocalBounds[InComponent];
	}

	return UE::AvaEditorCore::Private::InvalidBox;
}

bool UAvaBoundsProviderSubsystem::GetComponentWorldOrientedBounds(UPrimitiveComponent* InComponent, FOrientedBox& OutOrientedBounds)
{
	if (CacheComponentWorldOrientedBounds(InComponent))
	{
		OutOrientedBounds = CachedComponentWorldOrientedBounds[InComponent];
		return true;
	}

	return false;
}

bool UAvaBoundsProviderSubsystem::GetComponentActorOrientedBounds(UPrimitiveComponent* InComponent, FOrientedBox& OutOrientedBounds)
{
	if (CacheComponentActorOrientedBounds(InComponent))
	{
		OutOrientedBounds = CachedComponentActorOrientedBounds[InComponent];
		return true;
	}

	return false;
}

FBox UAvaBoundsProviderSubsystem::GetActorLocalBounds(AActor* InActor)
{
	if (CacheActorLocalBounds(InActor))
	{
		return CachedActorLocalBounds[InActor];
	}

	return UE::AvaEditorCore::Private::InvalidBox;
}

bool UAvaBoundsProviderSubsystem::GetActorOrientedBounds(AActor* InActor, FOrientedBox& OutOrientedBounds)
{
	if (CacheActorOrientedBounds(InActor))
	{
		OutOrientedBounds = CachedActorOrientedBounds[InActor];
		return true;
	}

	return false;
}

FBox UAvaBoundsProviderSubsystem::GetActorAndChildrenLocalBounds(AActor* InActor)
{
	if (CacheActorAndChildrenLocalBounds(InActor))
	{
		return CachedActorAndChildrenLocalBounds[InActor];
	}

	return UE::AvaEditorCore::Private::InvalidBox;
}

bool UAvaBoundsProviderSubsystem::GetActorAndChildrenOrientedBounds(AActor* InActor, FOrientedBox& OutOrientedBounds)
{
	if (CacheActorAndChildrenOrientedBounds(InActor))
	{
		OutOrientedBounds = CachedActorAndChildrenOrientedBounds[InActor];
		return true;
	}

	return false;
}

const TMap<TWeakObjectPtr<UPrimitiveComponent>, FBox>& UAvaBoundsProviderSubsystem::GetCachedComponentLocalBounds() const
{
	return CachedComponentLocalBounds;
}

const TMap<TWeakObjectPtr<UPrimitiveComponent>, FOrientedBox>& UAvaBoundsProviderSubsystem::GetCachedComponentWorldOrientedBounds() const
{
	return CachedComponentWorldOrientedBounds;
}

const TMap<TWeakObjectPtr<UPrimitiveComponent>, FOrientedBox>& UAvaBoundsProviderSubsystem::GetCachedComponentActorOrientedBounds() const
{
	return CachedComponentActorOrientedBounds;
}

const TMap<TWeakObjectPtr<AActor>, FBox>& UAvaBoundsProviderSubsystem::GetCachedActorLocalBounds() const
{
	return CachedActorLocalBounds;
}

const TMap<TWeakObjectPtr<AActor>, FOrientedBox>& UAvaBoundsProviderSubsystem::GetCachedActorOrientedBounds() const
{
	return CachedActorOrientedBounds;
}

const TMap<TWeakObjectPtr<AActor>, FBox>& UAvaBoundsProviderSubsystem::GetCachedActorAndChildrenLocalBounds() const
{
	return CachedActorAndChildrenLocalBounds;
}

const TMap<TWeakObjectPtr<AActor>, FOrientedBox>& UAvaBoundsProviderSubsystem::GetCachedActorAndChildrenOrientedBounds() const
{
	return CachedActorAndChildrenOrientedBounds;
}

FBox UAvaBoundsProviderSubsystem::GetSelectionBounds(bool bInIncludeChildrenOfSelectedActors)
{
	if (bInIncludeChildrenOfSelectedActors)
	{
		if (CachedSelectionWithChildrenBounds.IsValid)
		{
			return CachedSelectionWithChildrenBounds;
		}
	}
	else
	{
		if (CachedSelectionBounds.IsValid)
		{
			return CachedSelectionBounds;
		}
	}

	UAvaSelectionProviderSubsystem* SelectionProvider = UAvaSelectionProviderSubsystem::Get(this, /* bInGenerateErrors */ true);

	if (!SelectionProvider)
	{
		return FBox(EForceInit::ForceInit);
	}

	TConstArrayView<TWeakObjectPtr<AActor>> SelectedActors = SelectionProvider->GetSelectedActors();

	if (SelectedActors.IsEmpty())
	{
		return FBox(EForceInit::ForceInit);
	}

	// A little optimization if there's just a single selected actor
	if (SelectedActors.Num() == 1 && !bInIncludeChildrenOfSelectedActors)
	{
		AActor* OnlyActor = SelectedActors[0].Get();

		if (!OnlyActor)
		{
			return FBox(EForceInit::ForceInit);
		}

		FBox LocalBounds = bInIncludeChildrenOfSelectedActors
			? GetActorAndChildrenLocalBounds(OnlyActor)
			: GetActorLocalBounds(OnlyActor);

		if (!LocalBounds.IsValid)
		{
			return LocalBounds;
		}

		// Convert the scale of the bounds into world space
		const FVector ActorScale = OnlyActor->GetActorScale3D();
		LocalBounds.Min *= ActorScale;
		LocalBounds.Max *= ActorScale;

		return LocalBounds;
	}

	FBox Bounds(EForceInit::ForceInit);
	bool bIsFirstActor = true;
	FTransform InverseFirstActorTransform;

	for (const TWeakObjectPtr<AActor>& ActorWeak : SelectedActors)
	{
		if (AActor* Actor = ActorWeak.Get())
		{
			if (bIsFirstActor)
			{
				InverseFirstActorTransform = FTransform(
					Actor->GetActorRotation(),
					Actor->GetActorLocation(),
					FVector::OneVector
				).Inverse();

				bIsFirstActor = false;
			}

			FOrientedBox OrientedBounds;
			const bool bValidBounds = bInIncludeChildrenOfSelectedActors
				? GetActorAndChildrenOrientedBounds(Actor, OrientedBounds)
				: GetActorOrientedBounds(Actor, OrientedBounds);

			if (bValidBounds)
			{
				FVector Vertices[8];
				OrientedBounds.CalcVertices(Vertices);

				for (const FVector& Vertex : Vertices)
				{
					Bounds += InverseFirstActorTransform.TransformPositionNoScale(Vertex);
				}
			}
		}
	}

	if (bInIncludeChildrenOfSelectedActors)
	{
		CachedSelectionWithChildrenBounds = Bounds;
	}
	else
	{
		CachedSelectionBounds = Bounds;
	}

	return Bounds;
}

bool UAvaBoundsProviderSubsystem::GetSelectionOrientedBounds(bool bInIncludeChildrenOfSelectedActors, FOrientedBox& OutOrientedBounds)
{
	UAvaSelectionProviderSubsystem* SelectionProvider = UAvaSelectionProviderSubsystem::Get(this, /* bInGenerateErrors */ true);

	if (!SelectionProvider)
	{
		return false;
	}

	TConstArrayView<TWeakObjectPtr<AActor>> SelectedActors = SelectionProvider->GetSelectedActors();

	if (SelectedActors.IsEmpty())
	{
		return false;
	}

	const FBox SelectionAxisBounds = GetSelectionBounds(bInIncludeChildrenOfSelectedActors);

	if (!SelectionAxisBounds.IsValid)
	{
		return false;
	}

	bool bFoundActor = false;
	FTransform FirstActorTransform;

	for (const TWeakObjectPtr<AActor>& ActorWeak : SelectedActors)
	{
		if (AActor* Actor = ActorWeak.Get())
		{
			FirstActorTransform = Actor->GetActorTransform();
			bFoundActor = true;
			break;
		}
	}

	if (!bFoundActor)
	{
		return false;
	}

	OutOrientedBounds = UE::AvaEditorCore::Private::MakeOrientedBox(SelectionAxisBounds, FirstActorTransform);

	return true;
}

bool UAvaBoundsProviderSubsystem::IsTickableWhenPaused() const
{
	return true;
}

bool UAvaBoundsProviderSubsystem::IsTickableInEditor() const
{
	return true;
}

TStatId UAvaBoundsProviderSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UAvaBoundsProviderSubsystem, STATGROUP_Tickables);
}

void UAvaBoundsProviderSubsystem::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);

	ClearCachedBounds();
}
