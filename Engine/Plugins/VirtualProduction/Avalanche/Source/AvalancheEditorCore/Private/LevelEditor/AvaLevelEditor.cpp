// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaLevelEditor.h"
#include "AvaLevelEditorToolbar.h"
#include "Delegates/Delegate.h"
#include "IAvaEditorExtension.h"
#include "IAvaEditorProvider.h"
#include "LevelEditor.h"
#include "LevelEditor/AvaLevelEditorUtils.h"
#include "Styling/SlateIconFinder.h"
#include "UnrealEdMisc.h"

#define LOCTEXT_NAMESPACE "AvaLevelEditor"

FAvaLevelEditor::FAvaLevelEditor(FAvaEditorBuilder& Initializer)
	: FAvaEditor(Initializer)
	, Toolbar(MakeShared<FAvaLevelEditorToolbar>())
{
}

FAvaLevelEditor::~FAvaLevelEditor()
{
	if (FLevelEditorModule* const LevelEditorModule = FAvaLevelEditorUtils::GetLevelEditorModule())
	{
		LevelEditorModule->OnMapChanged().Remove(OnMapChangedHandle);
		LevelEditorModule->OnRegisterLayoutExtensions().Remove(OnRegisterLayoutHandle);

		OnMapChangedHandle.Reset();
		OnRegisterLayoutHandle.Reset();
	}

	FWorldDelegates::OnCurrentLevelChanged.Remove(OnCurrentLevelChangedHandle);
	OnCurrentLevelChangedHandle.Reset();
}

void FAvaLevelEditor::Construct()
{
	// Make sure that there's only one Ava Level Editor due to the limitations of Single World/LevelEditor
	// For example, the Level Editor Instance is handled at the Level Editor Module instance
	static TWeakPtr<FAvaLevelEditor> AvaLevelEditor;
	if (!ensureMsgf(!AvaLevelEditor.IsValid(), TEXT("Trying to construct more than one Ava Level Editor! This is not allowed")))
	{
		return;
	}

	const TSharedRef<FAvaLevelEditor> This = SharedThis(this);

	AvaLevelEditor = This;

	FAvaEditor::Construct();

	OnCurrentLevelChangedHandle = FWorldDelegates::OnCurrentLevelChanged.AddSP(This, &FAvaLevelEditor::OnCurrentLevelChanged);

	Toolbar->Construct(This);

	if (FLevelEditorModule* const LevelEditorModule = FAvaLevelEditorUtils::GetLevelEditorModule())
	{
		OnRegisterLayoutHandle     = LevelEditorModule->OnRegisterLayoutExtensions().AddSP(This, &FAvaLevelEditor::ExtendLayout);
		OnLevelEditorCreatedHandle = LevelEditorModule->OnLevelEditorCreated().AddSP(This, &FAvaLevelEditor::OnLevelEditorCreated);
		OnMapChangedHandle         = LevelEditorModule->OnMapChanged().AddSP(This, &FAvaLevelEditor::OnMapChanged);
	}
}

void FAvaLevelEditor::ExtendLayout(FLayoutExtender& InExtender)
{
	ForEachExtension([&InExtender](const TSharedRef<IAvaEditorExtension>& InExtension)
	{
		InExtension->ExtendLevelEditorLayout(InExtender);
	});
}

void FAvaLevelEditor::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	if (FLevelEditorModule* LevelEditorModule = FAvaLevelEditorUtils::LoadLevelEditorModule())
	{
		LevelEditorModule->GetGlobalLevelEditorActions()->Append(InCommandList);
	}
	FAvaEditor::BindCommands(InCommandList);
}

void FAvaLevelEditor::OnCurrentLevelChanged(ULevel* InNewLevel, ULevel* InOldLevel, UWorld* InWorld)
{
	OnSceneObjectChanged();
}

void FAvaLevelEditor::TryOpenScene(EAvaEditorObjectQueryType InQueryType)
{
	FLevelEditorModule* LevelEditorModule = FAvaLevelEditorUtils::LoadLevelEditorModule();
	if (!LevelEditorModule)
	{
		return;
	}

	const TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule->GetLevelEditorInstance().Pin();
	if (!LevelEditor.IsValid())
	{
		return;
	}

	// Update Toolkit Host
	SetToolkitHost(LevelEditor.ToSharedRef());

	// Only Init if there's a Valid Scene Object Found
	if (UObject* const SceneObject = GetSceneObject(InQueryType))
	{
		Activate();
	}
}

void FAvaLevelEditor::CreateScene()
{
	if (CanCreateScene())
	{
		TryOpenScene(EAvaEditorObjectQueryType::CreateIfNotFound);
	}
}

bool FAvaLevelEditor::CanCreateScene() const
{
	// Can only create Scene if there's no valid one already.
	// Skip search as this gets called frequently
	return !IsValid(GetSceneObject(EAvaEditorObjectQueryType::SkipSearch));
}

void FAvaLevelEditor::OnMapChanged(UWorld* InWorld, EMapChangeType InChangeType)
{
	switch (InChangeType)
	{
	case EMapChangeType::NewMap:
		if (ShouldAutoActivate())
		{
			TryOpenScene();
		}
		break;

	case EMapChangeType::LoadMap:
		if (ShouldAutoActivate())
		{
			TryOpenScene();
			Load();
		}
		break;

	case EMapChangeType::SaveMap:
		Save();
		break;

	case EMapChangeType::TearDownWorld:
		Deactivate();
		Cleanup();
		break;

	default:
		break;
	}
}

void FAvaLevelEditor::OnLevelEditorCreated(TSharedPtr<ILevelEditor> InLevelEditor)
{
	if (!InLevelEditor.IsValid())
	{
		return;
	}

	SetToolkitHost(InLevelEditor.ToSharedRef());

	DeactivateExtensions();
	UnregisterTabSpawners();

	if (ShouldAutoActivate())
	{
		TryOpenScene();
		RegisterTabSpawners();
	}
}

#undef LOCTEXT_NAMESPACE
