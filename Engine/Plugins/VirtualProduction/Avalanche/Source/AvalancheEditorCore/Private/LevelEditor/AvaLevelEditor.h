// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaEditor.h"

class FAvaLevelEditorToolbar;
class ILevelEditor;
class ULevel;
enum class EMapChangeType : uint8;

class FAvaLevelEditor : public FAvaEditor
{
public:
	UE_AVA_INHERITS(FAvaLevelEditor, FAvaEditor);

	explicit FAvaLevelEditor(FAvaEditorBuilder& Initializer);

	virtual ~FAvaLevelEditor() override;

	void CreateScene();

	bool CanCreateScene() const;

protected:
	//~ Begin FAvaEditor
	virtual void Construct() override;
	virtual void ExtendLayout(FLayoutExtender& InExtender) override;
	//~ End FAvaEditor

	//~ Begin IAvaEditor
	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandList) override;
	//~ End IAvaEditor

private:
	void OnCurrentLevelChanged(ULevel* InNewLevel, ULevel* InOldLevel, UWorld* InWorld);

	void TryOpenScene(EAvaEditorObjectQueryType InQueryType = EAvaEditorObjectQueryType::SearchOnly);

	void OnMapChanged(UWorld* InWorld, EMapChangeType InChangeType);

	void OnLevelEditorCreated(TSharedPtr<ILevelEditor> InLevelEditor);

	TSharedRef<FAvaLevelEditorToolbar> Toolbar;

	FDelegateHandle OnCurrentLevelChangedHandle;

	FDelegateHandle OnRegisterLayoutHandle;

	FDelegateHandle OnMapChangedHandle;

	FDelegateHandle OnLevelEditorCreatedHandle;
};
