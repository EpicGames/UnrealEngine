// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaLevelEditorCommands.h"
#include "AvaLevelEditorStyle.h"

#define LOCTEXT_NAMESPACE "AvaLevelEditorCommands"

FAvaLevelEditorCommands::FAvaLevelEditorCommands()
	: TCommands<FAvaLevelEditorCommands>(TEXT("AvaLevelEditor")
	, LOCTEXT("MotionDesignLevelEditor", "Motion Design Level Editor")
	, NAME_None
	, FAvaLevelEditorStyle::Get().GetStyleSetName())
{
}

void FAvaLevelEditorCommands::RegisterCommands()
{
	UI_COMMAND(CreateScene
		, "Create Scene"
		, "Creates a Motion Design Scene for the Level and opens the Motion Design Editor Interface"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(ActivateScene
		, "Activate Scene"
		, "Re-opens the Motion Design Editor Interface"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(DeactivateScene
		, "Deactivate Scene"
		, "Closes the Motion Design Editor Interface. The Motion Design Scene data is kept."
		, EUserInterfaceActionType::Button
		, FInputChord());
}

#undef LOCTEXT_NAMESPACE
