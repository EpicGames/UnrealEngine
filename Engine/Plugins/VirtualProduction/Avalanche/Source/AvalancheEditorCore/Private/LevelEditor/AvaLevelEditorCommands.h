// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FAvaLevelEditorCommands : public TCommands<FAvaLevelEditorCommands>
{
public:
	FAvaLevelEditorCommands();

	//~ Begin TCommands
	virtual void RegisterCommands() override;
	//~ End TCommands

	TSharedPtr<FUICommandInfo> CreateScene;

	TSharedPtr<FUICommandInfo> ActivateScene;

	TSharedPtr<FUICommandInfo> DeactivateScene;
};
