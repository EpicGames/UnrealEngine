// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaLevelEditorStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

FAvaLevelEditorStyle::FAvaLevelEditorStyle()
	: FSlateStyleSet(TEXT("AvaLevelEditor"))
{
	const FVector2f Icon16x16(16.f, 16.f);

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	SetContentRoot(Plugin->GetBaseDir() / TEXT("Resources"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
	
	Set("AvaLevelEditor.CreateScene", new IMAGE_BRUSH_SVG("Icons/EditorIcons/LevelEditor_CreateScene_16", Icon16x16));
	Set("AvaLevelEditor.ActivateScene", new IMAGE_BRUSH_SVG("Icons/EditorIcons/LevelEditor_ActivateScene_16", Icon16x16));
	Set("AvaLevelEditor.DeactivateScene", new IMAGE_BRUSH_SVG("Icons/EditorIcons/LevelEditor_DeactivateScene_16", Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FAvaLevelEditorStyle::~FAvaLevelEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
