// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FAvaLevelEditorStyle final : public FSlateStyleSet
{
public:
	static FAvaLevelEditorStyle& Get()
	{
		static FAvaLevelEditorStyle Instance;
		return Instance;
	}

	FAvaLevelEditorStyle();
	virtual ~FAvaLevelEditorStyle() override;
};
