// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaLevelEditorToolbar.h"
#include "AvaEditorContext.h"
#include "AvaLevelEditor.h"
#include "AvaLevelEditorCommands.h"
#include "Engine/World.h"
#include "LevelEditor/AvaLevelEditorUtils.h"
#include "Misc/MessageDialog.h"
#include "Styling/SlateIconFinder.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenus.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "AvaLevelEditorToolbar"

namespace UE::AvaEditorCore::Private
{
	static constexpr const TCHAR* LevelEditorToolbarName = TEXT("LevelEditor.LevelEditorToolBar.User");
	static constexpr const TCHAR* SceneSectionName = TEXT("Scene");

	bool ShouldShowActivateScene(TWeakPtr<IAvaEditor> InEditorWeak)
	{
		TSharedPtr<IAvaEditor> Editor = InEditorWeak.Pin();
		return Editor.IsValid() && Editor->CanActivate() && IsValid(Editor->GetSceneObject(EAvaEditorObjectQueryType::SkipSearch));
	}

	bool ShouldShowDeactivateScene(TWeakPtr<IAvaEditor> InEditorWeak)
	{
		TSharedPtr<IAvaEditor> Editor = InEditorWeak.Pin();
		return Editor.IsValid() && Editor->CanDeactivate() && IsValid(Editor->GetSceneObject(EAvaEditorObjectQueryType::SkipSearch));
	}

	void ActivateScene(TWeakPtr<IAvaEditor> InEditorWeak)
	{
		if (TSharedPtr<IAvaEditor> Editor = InEditorWeak.Pin())
		{
			Editor->SetAutoActivate(true);
			Editor->Activate();
		}
	}

	void DeactivateScene(TWeakPtr<IAvaEditor> InEditorWeak)
	{
		const EAppReturnType::Type Reply = FMessageDialog::Open(EAppMsgCategory::Warning
			, EAppMsgType::YesNo
			, LOCTEXT("DeactivateSceneConfirmationDesc", "Are you sure you want to deactivate the Motion Design Interface? \n"
				"Note: The scene data will be kept.")
			, LOCTEXT("DeactivateSceneConfirmationTitle", "Deactivate Motion Design Interface"));

		if (Reply != EAppReturnType::Yes)
		{
			return;
		}

		if (TSharedPtr<IAvaEditor> Editor = InEditorWeak.Pin())
		{
			Editor->SetAutoActivate(false);
			Editor->Deactivate();
		}
	}
}

FAvaLevelEditorToolbar::FAvaLevelEditorToolbar()
	: CommandList(MakeShared<FUICommandList>())
{
}

FAvaLevelEditorToolbar::~FAvaLevelEditorToolbar()
{
	Destroy();
}

void FAvaLevelEditorToolbar::Construct(const TSharedRef<FAvaLevelEditor>& InLevelEditor)
{
	LevelEditorWeak = InLevelEditor;
	BindLevelEditorToolbarCommands(InLevelEditor);
	ExtendLevelEditorToolbar(InLevelEditor);
}

void FAvaLevelEditorToolbar::Destroy()
{
	using namespace UE::AvaEditorCore;

	if (UObjectInitialized())
	{
		UToolMenus* const ToolMenus = UToolMenus::Get();
		if (IsValid(ToolMenus))
		{
			ToolMenus->RemoveSection(Private::LevelEditorToolbarName, Private::SceneSectionName);		
		}
	}
}

void FAvaLevelEditorToolbar::BindLevelEditorToolbarCommands(const TSharedRef<FAvaLevelEditor>& InLevelEditor)
{
	const FAvaLevelEditorCommands& AvaLevelEditorCommands = FAvaLevelEditorCommands::Get();

	TWeakPtr<IAvaEditor> EditorWeak = InLevelEditor;

	using namespace UE::AvaEditorCore;

	CommandList->MapAction(AvaLevelEditorCommands.CreateScene
		, FExecuteAction::CreateSP(InLevelEditor, &FAvaLevelEditor::CreateScene)
		, FIsActionChecked()
		, FCanExecuteAction()
		, FIsActionButtonVisible::CreateSP(InLevelEditor, &FAvaLevelEditor::CanCreateScene));

	CommandList->MapAction(AvaLevelEditorCommands.ActivateScene
		, FExecuteAction::CreateStatic(Private::ActivateScene, EditorWeak)
		, FIsActionChecked()
		, FCanExecuteAction()
		, FIsActionButtonVisible::CreateStatic(Private::ShouldShowActivateScene, EditorWeak));

	CommandList->MapAction(AvaLevelEditorCommands.DeactivateScene
		, FExecuteAction::CreateStatic(Private::DeactivateScene, EditorWeak)
		, FCanExecuteAction()
		, FIsActionChecked()
		, FIsActionButtonVisible::CreateStatic(Private::ShouldShowDeactivateScene, EditorWeak));
}

void FAvaLevelEditorToolbar::ExtendLevelEditorToolbar(const TSharedRef<IAvaEditor>& InLevelEditor)
{
	using namespace UE::AvaEditorCore;

	UToolMenus* const ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return;
	}

	if (UToolMenu* const LevelEditorToolbar = ToolMenus->ExtendMenu(Private::LevelEditorToolbarName))
	{
		const FAvaLevelEditorCommands& AvaLevelEditorCommands = FAvaLevelEditorCommands::Get();
		FToolMenuSection& Section = LevelEditorToolbar->FindOrAddSection(Private::SceneSectionName);

		auto AddEntry = [this, &Section](const TSharedPtr<FUICommandInfo>& InCommand)
		{
			FToolMenuEntry& Entry = Section.AddEntry(FToolMenuEntry::InitToolBarButton(InCommand, LOCTEXT("MotionDesignScene", "Motion Design")));
			Entry.SetCommandList(CommandList);
			Entry.StyleNameOverride = "CalloutToolbar";
		};

		AddEntry(AvaLevelEditorCommands.CreateScene);

		Section.AddEntry(FToolMenuEntry::InitComboButton(TEXT("CreateSceneOptions")
			, FUIAction()
			, FNewToolMenuDelegate::CreateSPLambda(this,
				[this](UToolMenu* InMenu)
				{
					if (const TSharedPtr<IAvaEditor> LevelEditor = LevelEditorWeak.Pin())
					{
						LevelEditor->ExtendCreateSceneOptions(InMenu);
					}
				})
			, LOCTEXT("CreateSceneOptionsLabel", "Create Scene Options")
			, LOCTEXT("CreateSceneOptionsTooltip", "Options for creating a Motion Design scene")
			, AvaLevelEditorCommands.CreateScene->GetIcon()
			, /*bSimpleComboBox*/true));

		AddEntry(AvaLevelEditorCommands.ActivateScene);
		AddEntry(AvaLevelEditorCommands.DeactivateScene);
	}
}

#undef LOCTEXT_NAMESPACE
