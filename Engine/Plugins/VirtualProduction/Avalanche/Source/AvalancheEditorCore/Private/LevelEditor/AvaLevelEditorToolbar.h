// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FAvaLevelEditor;
class FUICommandList;
class IAvaEditor;
class SBorder;
class SWidget;
class UToolMenu;

/** Responsible for manipulating the Level Editor toolbar for the Motion Design Editor */
class FAvaLevelEditorToolbar : public TSharedFromThis<FAvaLevelEditorToolbar>
{
public:
	FAvaLevelEditorToolbar();

	~FAvaLevelEditorToolbar();

	void Construct(const TSharedRef<FAvaLevelEditor>& InLevelEditor);

	void Destroy();

private:
	void BindLevelEditorToolbarCommands(const TSharedRef<FAvaLevelEditor>& InLevelEditor);

	void ExtendLevelEditorToolbar(const TSharedRef<IAvaEditor>& InLevelEditor);

	TWeakPtr<FAvaLevelEditor> LevelEditorWeak;

	TSharedRef<FUICommandList> CommandList;
};
