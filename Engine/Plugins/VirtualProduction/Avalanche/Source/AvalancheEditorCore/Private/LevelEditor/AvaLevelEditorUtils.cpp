// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelEditor/AvaLevelEditorUtils.h"
#include "HAL/Platform.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"

namespace UE::AvaEditorCore::Private
{
	static constexpr const TCHAR* LevelEditorModuleName = TEXT("LevelEditor");
}

FLevelEditorModule* FAvaLevelEditorUtils::GetLevelEditorModule()
{
	return FModuleManager::Get().GetModulePtr<FLevelEditorModule>(UE::AvaEditorCore::Private::LevelEditorModuleName);
}

FLevelEditorModule* FAvaLevelEditorUtils::LoadLevelEditorModule()
{
	return FModuleManager::Get().LoadModulePtr<FLevelEditorModule>(UE::AvaEditorCore::Private::LevelEditorModuleName);
}

TConstArrayView<FName> FAvaLevelEditorUtils::GetDetailsViewNames()
{
	static const TArray<FName> DetailsViewIds = {
		LevelEditorTabIds::LevelEditorSelectionDetails,
		LevelEditorTabIds::LevelEditorSelectionDetails2,
		LevelEditorTabIds::LevelEditorSelectionDetails3,
		LevelEditorTabIds::LevelEditorSelectionDetails4
	};

	return DetailsViewIds;
}
