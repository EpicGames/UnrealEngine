// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/AvaSelectionProviderSubsystem.h"
#include "Algo/Transform.h"
#include "Bounds/AvaBoundsProviderSubsystem.h"
#include "Engine/World.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Selection/AvaEditorSelection.h"

bool UAvaSelectionProviderSubsystem::DoesSupportWorldType(const EWorldType::Type InWorldType) const
{
	return InWorldType == EWorldType::Editor || InWorldType == EWorldType::PIE;
}

void UAvaSelectionProviderSubsystem::UpdateSelection(const FAvaEditorSelection& InSelection)
{
	const TArray<AActor*> SelectedActors = InSelection.GetSelectedObjects<AActor>();

	CachedSelectedActors.Empty(SelectedActors.Num());
	CachedSelectedActors.Reserve(SelectedActors.Num());

	Algo::Transform(
		SelectedActors,
		CachedSelectedActors,
		[](AActor* InElement) { return InElement; }
	);

	const TArray<UActorComponent*> SelectedComponents = InSelection.GetSelectedObjects<UActorComponent>();

	CachedSelectedComponents.Empty();
	CachedSelectedComponents.Reserve(SelectedComponents.Num());

	for (UActorComponent* ActorComponent : SelectedComponents)
	{
		if (AActor* Actor = ActorComponent->GetOwner())
		{
			if (Actor->GetRootComponent() != ActorComponent)
			{
				CachedSelectedComponents.Add(ActorComponent);
			}
		}
	}

	// So we don't get out of date
	ClearAttachedActorCache();
}

void UAvaSelectionProviderSubsystem::ClearAttachedActorCache()
{
	CachedDirectlyAttachedActors.Empty();
	CachedRecursivelyAttachedActors.Empty();
}

FTransform UAvaSelectionProviderSubsystem::GetSelectionTransform() const
{
	if (CachedSelectedActors.IsEmpty())
	{
		return FTransform::Identity;
	}

	UAvaBoundsProviderSubsystem* BoundsProvider = UAvaBoundsProviderSubsystem::Get(this, /* bInGenerateErrors */ true);

	if (!BoundsProvider)
	{
		return FTransform::Identity;
	}

	for (const TWeakObjectPtr<AActor>& ActorWeak : CachedSelectedActors)
	{
		if (AActor* Actor = ActorWeak.Get())
		{
			const FBox Bounds = BoundsProvider->GetSelectionBounds(false);

			if (!Bounds.IsValid)
			{
				return FTransform::Identity;
			}

			// Selection bounds are based on the transform of the first selected actor.
			return FTransform(Actor->GetActorRotation(), Actor->GetActorTransform().TransformPositionNoScale(Bounds.GetCenter()), FVector::OneVector);
		}
	}

	return FTransform::Identity;
}

TConstArrayView<TWeakObjectPtr<AActor>> UAvaSelectionProviderSubsystem::GetAttachedActors(AActor* InActor, bool bInRecursive)
{
	static TArray<TWeakObjectPtr<AActor>> EmptyArray;

	if (!IsValid(InActor))
	{
		return EmptyArray;
	}

	TMap<TWeakObjectPtr<AActor>, TArray<TWeakObjectPtr<AActor>>>& CachedAttachedActors = bInRecursive
		? CachedRecursivelyAttachedActors
		: CachedDirectlyAttachedActors;

	if (const TArray<TWeakObjectPtr<AActor>>* CachedList = CachedAttachedActors.Find(InActor))
	{
		return *CachedList;
	}

	TArray<AActor*> Children;
	InActor->GetAttachedActors(Children, false, bInRecursive);

	TArray<TWeakObjectPtr<AActor>> ChildrenWeak;
	ChildrenWeak.Reserve(Children.Num());

	Algo::Transform(
		Children,
		ChildrenWeak,
		[](AActor* InElement) { return InElement; }
	);

	CachedAttachedActors.Emplace(InActor, MoveTemp(ChildrenWeak));

	return CachedAttachedActors[InActor];
}
