// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DataTypes/AvaUserInputDialogDataTypeStruct.h"
#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "UObject/StructOnScope.h"

FAvaUserInputDialogDataTypeStruct::FAvaUserInputDialogDataTypeStruct(const FParams& InParams)
	: StructOnScope(MakeShared<FStructOnScope>(InParams.Struct))
	, IsValidDelegate(InParams.IsValidDelegate)
{
}

const uint8* FAvaUserInputDialogDataTypeStruct::GetStructMemory() const
{
	return StructOnScope->GetStructMemory();
}

const UStruct* FAvaUserInputDialogDataTypeStruct::GetStruct() const
{
	return StructOnScope->GetStruct();
}

TSharedRef<SWidget> FAvaUserInputDialogDataTypeStruct::CreateInputWidget()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
		DetailsViewArgs.bShowObjectLabel = false;
		DetailsViewArgs.bForceHiddenPropertyVisibility = true;
		DetailsViewArgs.bShowScrollBar = false;
	}

	FStructureDetailsViewArgs StructureViewArgs;
	{
		StructureViewArgs.bShowObjects = true;
		StructureViewArgs.bShowAssets = true;
		StructureViewArgs.bShowClasses = true;
		StructureViewArgs.bShowInterfaces = true;
	}

	TSharedRef<IStructureDetailsView> DetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, StructOnScope);
	return DetailsView->GetWidget().ToSharedRef();
}

bool FAvaUserInputDialogDataTypeStruct::IsValueValid()
{
	if (IsValidDelegate.IsBound())
	{
		return IsValidDelegate.Execute(StructOnScope);
	}

	return true;
}
