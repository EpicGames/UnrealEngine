// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/DataTypes/AvaUserInputDialogDataTypeText.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/SMultiLineEditableText.h"

#define LOCTEXT_NAMESPACE "FAvaUserInputDialogTextData"

FAvaUserInputDialogTextData::FAvaUserInputDialogTextData(const FParams& InParams)
	: Value(InParams.InitialValue)
	, bAllowMultiline(InParams.bAllowMultiline)
	, MaxLength(InParams.MaxLength)
	, OnVerifyDelegate(InParams.OnVerifyDelegate)
{
}

const FText& FAvaUserInputDialogTextData::GetValue() const
{
	return Value;
}

TSharedRef<SWidget> FAvaUserInputDialogTextData::CreateInputWidget()
{
	if (bAllowMultiline)
	{
		TSharedRef<SScrollBar> ScrollBar = SNew(SScrollBar)
			.Orientation(Orient_Vertical);

		return SNew(SBox)
			.WidthOverride(500.f)
			.HeightOverride(200.f)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SBorder)
					.BorderImage(&FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox").BackgroundImageFocused)
				]
				+ SOverlay::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.f)
					.Padding(5.f)
					[
						SNew(SMultiLineEditableText)
						.Text(Value)
						.OnTextChanged(this, &FAvaUserInputDialogTextData::OnTextChanged)
						.OnTextCommitted(this, &FAvaUserInputDialogTextData::OnTextCommitted)
						.AllowMultiLine(true)
						.VScrollBar(ScrollBar)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						ScrollBar
					]
				]
			];
	}

	return SNew(SBox)
		.WidthOverride(FMath::Max(200.f, MaxLength.IsSet() ? MaxLength.GetValue() * 8.f : 200.f))
		.HAlign(HAlign_Fill)
		[
			SNew(SEditableTextBox)
			.Text(Value)
			.OnTextChanged(this, &FAvaUserInputDialogTextData::OnTextChanged)
			.OnTextCommitted(this, &FAvaUserInputDialogTextData::OnTextCommitted)
			.OnVerifyTextChanged(this, &FAvaUserInputDialogTextData::OnTextVerify)
		];
}

void FAvaUserInputDialogTextData::OnTextChanged(const FText& InValue)
{
	Value = InValue;
}

void FAvaUserInputDialogTextData::OnTextCommitted(const FText& InValue, ETextCommit::Type InCommitType)
{
	Value = InValue;

	if (!bAllowMultiline && InCommitType == ETextCommit::OnEnter)
	{
		OnUserCommit();
	}
}

bool FAvaUserInputDialogTextData::OnTextVerify(const FText& InValue, FText& OutErrorText)
{
	if (MaxLength.IsSet() && InValue.ToString().Len() > MaxLength.GetValue())
	{
		OutErrorText = LOCTEXT("InputTooLong", "Input too long");
		return false;
	}

	if (OnVerifyDelegate.IsBound())
	{
		return OnVerifyDelegate.Execute(InValue, OutErrorText);
	}

	return true;
}

bool FAvaUserInputDialogTextData::IsValueValid()
{
	FText Unused;
	return OnTextVerify(Value, Unused);
}

#undef LOCTEXT_NAMESPACE
