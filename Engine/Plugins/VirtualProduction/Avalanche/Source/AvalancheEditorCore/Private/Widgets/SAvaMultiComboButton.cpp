// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * This is a copy+paste of the engine SComboButton because it does not support some required
 * functionality (the ability to use right click to open the menu and left click to activate
 * the button).
 */

#include "Widgets/SAvaMultiComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Framework/Application/SlateApplication.h"

void SAvaMultiComboButton::Construct( const FArguments& InArgs )
{
	check(InArgs._ComboButtonStyle);
	Style = InArgs._ComboButtonStyle;
	// Work out which values we should use based on whether we were given an override, or should use the style's version
	const FButtonStyle* const OurButtonStyle = InArgs._ButtonStyle ? InArgs._ButtonStyle : &InArgs._ComboButtonStyle->ButtonStyle;

	MenuBorderBrush = &InArgs._ComboButtonStyle->MenuBorderBrush;
	MenuBorderPadding = InArgs._ComboButtonStyle->MenuBorderPadding;
	
	OnComboBoxOpened = InArgs._OnComboBoxOpened;
	ContentWidgetPtr = InArgs._MenuContent.Widget;
	bIsFocusable = InArgs._IsFocusable;

	const bool bHasDownArrowShadow = !InArgs._ComboButtonStyle->ShadowOffset.IsZero();

	SMenuAnchor::Construct( SMenuAnchor::FArguments()
		.Placement(InArgs._MenuPlacement)
		.Method(InArgs._Method)
		.OnMenuOpenChanged(InArgs._OnMenuOpenChanged)
		.OnGetMenuContent(InArgs._OnGetMenuContent)
		.IsCollapsedByParent(InArgs._CollapseMenuOnParentFocus)
		[
			SAssignNew(ButtonPtr, SButton )
			.ButtonStyle( OurButtonStyle )
			.ClickMethod( EButtonClickMethod::MouseDown )
			.OnClicked(InArgs._OnButtonClicked)
			.ToolTipText( this, &SAvaMultiComboButton::GetFilteredToolTipText, InArgs._ToolTipText)
			.ContentPadding( InArgs._ContentPadding.IsSet() ? InArgs._ContentPadding : InArgs._ComboButtonStyle->ContentPadding )
			.ForegroundColor( InArgs._ForegroundColor )
			.ButtonColorAndOpacity( InArgs._ButtonColorAndOpacity )
			.IsFocusable( InArgs._IsFocusable )
			// We set the button to not be accessible as there are issues interacting with the menus that pop up e.g in SComboBox
			.AccessibleParams(FAccessibleWidgetData(EAccessibleBehavior::NotAccessible, EAccessibleBehavior::NotAccessible, false))
			[
				// Button and down arrow on the right
				// +-------------------+---+
				// | Button Content    | v |
				// +-------------------+---+
				SAssignNew( HBox, SHorizontalBox )
				+ SHorizontalBox::Slot()
				.Expose( ButtonContentSlot )
				.FillWidth( 1 )
				.HAlign( InArgs._HAlign )
				.VAlign( InArgs._VAlign )
				[
					InArgs._ButtonContent.Widget
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign( HAlign_Right )
				.VAlign(InArgs._HasDownArrow ? (EVerticalAlignment) InArgs._ComboButtonStyle->DownArrowAlign : VAlign_Center)
				.Padding(InArgs._HasDownArrow ? InArgs._ComboButtonStyle->DownArrowPadding : FMargin(0))
				[
					SNew(SOverlay)
					// drop shadow
					+ SOverlay::Slot()
					.VAlign(VAlign_Top)
					.Padding(FMargin(InArgs._ComboButtonStyle->ShadowOffset.X, InArgs._ComboButtonStyle->ShadowOffset.Y, 0, 0))
					[
						SAssignNew(ShadowImage, SImage)
						.Visibility( InArgs._HasDownArrow && bHasDownArrowShadow ? EVisibility::Visible : EVisibility::Collapsed )
						.Image( &InArgs._ComboButtonStyle->DownArrowImage )
						.ColorAndOpacity( InArgs._ComboButtonStyle->ShadowColorAndOpacity )
					]
					+ SOverlay::Slot()
					.VAlign(VAlign_Top)
					[
						SAssignNew(ForegroundArrowImage,SImage)
						.Visibility( InArgs._HasDownArrow ? EVisibility::Visible : EVisibility::Collapsed )
						.Image( &InArgs._ComboButtonStyle->DownArrowImage )
						// Inherit tinting from parent
						.ColorAndOpacity( FSlateColor::UseForeground() )
					]
				]
			]
		]
	);
	
	// The menu that pops up when we press the button.
	// We keep this content around, and then put it into a new window when we need to pop
	// it up.
	SetMenuContent( InArgs._MenuContent.Widget );
}

FText SAvaMultiComboButton::GetFilteredToolTipText(TAttribute<FText> ToolTipText) const
{
	if (IsOpen())
	{
		return FText::GetEmpty();
	}

	return ToolTipText.Get();
}

FReply SAvaMultiComboButton::OnButtonClicked()
{
	// Button was clicked; show the popup.
	// Do nothing if clicking on the button also dismissed the menu, because we will end up doing the same thing twice.
	// Don't explicitly focus the menu, here, we're going to do it in the button reply so that it's properly focused
	// to the correct user.
	SetIsOpen( ShouldOpenDueToClick(), false );

	// If the menu is open, execute the related delegate.
	if( IsOpen() && OnComboBoxOpened.IsBound() )
	{
		OnComboBoxOpened.Execute();
	}

	// Focusing any newly-created widgets must occur after they have been added to the UI root.
	FReply ButtonClickedReply = FReply::Handled();
	
	// Don't try to focus the menu if the menu is closing
	if (!IsOpen())
	{
		return ButtonClickedReply;
	}

	TSharedPtr<SWidget> WidgetToFocus = WidgetToFocusPtr.Pin();
	
	if (bIsFocusable)
	{
		if (!WidgetToFocus.IsValid())
		{
			// no explicitly focused widget, try to focus the content that is a child of the border
			if (WrappedContent.IsValid() && WrappedContent->GetChildren()->Num() == 1)
			{
				WidgetToFocus = WrappedContent->GetChildren()->GetChildAt(0);
			}
		}

		if (!WidgetToFocus.IsValid())
		{
			// no explicitly focused widget, try to focus the content
			WidgetToFocus = MenuContent;
		}

		if (!WidgetToFocus.IsValid())
		{
			// no content, so try to focus the original widget set on construction
			WidgetToFocus = ContentWidgetPtr.Pin();
		}
	}

	if (WidgetToFocus.IsValid())
	{
		ButtonClickedReply.SetUserFocus(WidgetToFocus.ToSharedRef(), EFocusCause::SetDirectly);
	}

	return ButtonClickedReply;
}

FReply SAvaMultiComboButton::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply Reply = FReply::Unhandled();

	if (FSlateApplication::Get().GetNavigationActionFromKey(InKeyEvent) == EUINavigationAction::Accept)
	{
		// Handle click button with controller.
		ButtonPtr->SimulateClick();
		Reply = FReply::Handled();
	}

	return Reply;
}

void SAvaMultiComboButton::SetMenuContent(TSharedRef<SWidget> InContent)
{
	WrappedContent = MenuContent =
		SNew(SBorder)
		.BorderImage(MenuBorderBrush)
		.Padding(MenuBorderPadding)
		[
			InContent
		];
}

void SAvaMultiComboButton::SetOnGetMenuContent(FOnGetContent InOnGetMenuContent)
{
	OnGetMenuContent = InOnGetMenuContent;
}

void SAvaMultiComboButton::SetButtonContentPadding(FMargin InPadding)
{
	check(ButtonPtr);
	ButtonPtr->SetContentPadding(InPadding);
}

void SAvaMultiComboButton::SetHasDownArrow(bool InHasArrowDown)
{
	const bool bHasDownArrowShadow = !Style->ShadowOffset.IsZero();

	check(HBox && HBox->NumSlots() >= 2);
	HBox->GetSlot(1).SetVerticalAlignment(InHasArrowDown ? (EVerticalAlignment)Style->DownArrowAlign : VAlign_Center);
	HBox->GetSlot(1).SetPadding(InHasArrowDown ? Style->DownArrowPadding : FMargin(0));

	check(ForegroundArrowImage);
	ForegroundArrowImage->SetVisibility(InHasArrowDown ? EVisibility::Visible : EVisibility::Collapsed);
	check(ShadowImage);
	ShadowImage->SetVisibility(InHasArrowDown && bHasDownArrowShadow ? EVisibility::Visible : EVisibility::Collapsed);
}

FReply SAvaMultiComboButton::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		return OnButtonClicked();
	}

	return SMenuAnchor::OnMouseButtonDown(InGeometry, InMouseEvent);
}
