// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAvaUserInputDialog.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaUserInputDialog"

bool SAvaUserInputDialog::CreateModalDialog(const TSharedRef<FAvaUserInputDialogDataTypeBase>& InInputType, const TSharedPtr<SWidget>& InParent,
	const TOptional<FText>& InPrompt, const TOptional<FText>& InTitle)
{
	static const FText DefaultPrompt = LOCTEXT("DefaultPrompt", "Value requested:");
	static const FText DefaultTitle = LOCTEXT("DefaultTitle", "User Input Required");

	const FText Prompt = InPrompt.IsSet() && !InPrompt.GetValue().IsEmpty()
		? InPrompt.GetValue()
		: DefaultPrompt;

	const FText Title = InTitle.IsSet() && !InTitle.GetValue().IsEmpty()
		? InTitle.GetValue()
		: DefaultTitle;

	TSharedPtr<SWindow> ParentWindow = InParent.IsValid() ? FSlateApplication::Get().FindWidgetWindow(InParent.ToSharedRef()) : nullptr;

	TSharedRef<SAvaUserInputDialog> InputDialog = SNew(SAvaUserInputDialog, InInputType)
		.Prompt(Prompt);

	TSharedRef<SWindow> Window = SNew(SWindow)
		.SizingRule(ESizingRule::Autosized)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.Title(Title)
		[
			InputDialog
		];

	FSlateApplication::Get().AddModalWindow(Window, InParent, /* Slow Task */ false);

	return InputDialog->WasAccepted();
}

void SAvaUserInputDialog::PrivateRegisterAttributes(struct FSlateAttributeDescriptor::FInitializer& InInitializer)
{
}

void SAvaUserInputDialog::Construct(const FArguments& InArgs, const TSharedRef<FAvaUserInputDialogDataTypeBase>& InInputType)
{
	InputType = InInputType;
	InputType->OnCommit.BindSP(this, &SAvaUserInputDialog::OnUserCommit);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.Padding(15.f)
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(InArgs._Prompt)
		]
		+ SVerticalBox::Slot()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.Padding(15.f, 0.f, 15.f, 15.f)
		.AutoHeight()
		[
			InputType->CreateInputWidget()
		]
		+ SVerticalBox::Slot()
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.Padding(15.f, 0.f, 15.f, 15.f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f, 5.f, 0.f)
			[
				SNew(SButton)
				.Text(LOCTEXT("Accept", "Accept"))
				.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
				.OnClicked(this, &SAvaUserInputDialog::OnAcceptClicked)
				.IsEnabled(this, &SAvaUserInputDialog::GetAcceptedEnabled)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.f, 0.f, 0.f, 0.f)
			[
				SNew(SButton)
				.Text(LOCTEXT("Cancel", "Cancel"))
				.ButtonStyle(FAppStyle::Get(), "Button")
				.OnClicked(this, &SAvaUserInputDialog::OnCancelClicked)
			]
		]
	];
}

TSharedPtr<FAvaUserInputDialogDataTypeBase> SAvaUserInputDialog::GetInputType() const
{
	return InputType;
}

bool SAvaUserInputDialog::WasAccepted() const
{
	return bAccepted;
}

FReply SAvaUserInputDialog::OnAcceptClicked()
{
	if (!InputType.IsValid())
	{
		Close(/* Accepted */ false);
	}
	else if (InputType->IsValueValid())
	{
		Close(/* Accepted */ true);
	}

	return FReply::Handled();
}

bool SAvaUserInputDialog::GetAcceptedEnabled() const
{
	return !InputType.IsValid() || InputType->IsValueValid();
}

FReply SAvaUserInputDialog::OnCancelClicked()
{
	Close(/* Accepted */ false);

	return FReply::Handled();
}

void SAvaUserInputDialog::OnUserCommit()
{
	Close(true);
}

void SAvaUserInputDialog::Close(bool bInAccepted)
{
	bAccepted = bInAccepted;

	TSharedPtr<SWindow> CurrentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());

	if (CurrentWindow.IsValid())
	{
		CurrentWindow->RequestDestroyWindow();
	}
}

#undef LOCTEXT_NAMESPACE
