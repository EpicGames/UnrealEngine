// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaEditorExtensionTypeRegistry.h"
#include "AvaTypeId.h"
#include "Containers/Map.h"
#include "IAvaEditorExtension.h"
#include "IAvaEditorProvider.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/NameTypes.h"
#include <type_traits>

class AVALANCHEEDITORCORE_API FAvaEditorBuilder
{
	friend class FAvaEditor;

	/**
	 * For an Extension to be Consider valid, all these conditions must be met:
	 * 1) Both the Base Type and Override Type must be derived from IAvaEditorExtension
	 * 2) Override Type is the Base Type or a Derived Type of it
	 * 3) Override Type is not abstract
	 */
	template<typename InBaseExtensionType, typename InOverrideExtensionType>
	static constexpr bool TIsValidExtension_V = 
		!std::is_abstract_v<InOverrideExtensionType> &&
		std::is_base_of_v<IAvaEditorExtension, InBaseExtensionType> &&
		std::is_base_of_v<InBaseExtensionType, InOverrideExtensionType>;

public:
	virtual ~FAvaEditorBuilder();

	/** Delegate to give an opportunity for others to override provider class or add extensions */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnEditorBuild, FAvaEditorBuilder&);
	static FOnEditorBuild OnEditorBuild;

	FName GetIdentifier() const;

	FAvaEditorBuilder& SetIdentifier(FName InIdentifier);

	template<typename InProviderType, typename... InArgTypes
		UE_REQUIRES(std::is_base_of_v<IAvaEditorProvider, InProviderType>)>
	FAvaEditorBuilder& SetProvider(InArgTypes&&... InArgs)
	{
		Provider = MakeShared<InProviderType>(Forward<InArgTypes>(InArgs)...);
		return *this;
	}

	template<typename InBaseType, typename InOverrideType = InBaseType, typename... InArgTypes
		UE_REQUIRES(TIsValidExtension_V<InBaseType, InOverrideType>)>
	FAvaEditorBuilder& AddExtension(InArgTypes&&... InArgs)
	{
		TSharedRef<InOverrideType> Extension = MakeShared<InOverrideType>(Forward<InArgTypes>(InArgs)...);
		Extensions.Add(TAvaType<InBaseType>::GetTypeId(), Extension);
		FAvaEditorExtensionTypeRegistry::Get().RegisterExtension(Extension);
		return *this;
	}

	/** Creates the editor and initializes it and its extensions */
	TSharedRef<IAvaEditor> Build();

protected:
	virtual TSharedRef<IAvaEditor> CreateEditor();

private:
	/** Optional identifier for this Initializer instance */
	FName Identifier = NAME_None;

	TSharedPtr<IAvaEditorProvider> Provider;

	TMap<FAvaTypeId, TSharedRef<IAvaEditorExtension>> Extensions;

	bool bFinalized = false;
};
