// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "AvaEditorContext.generated.h"

class IAvaEditor;

UCLASS(MinimalAPI)
class UAvaEditorContext : public UObject
{
	GENERATED_BODY()

public:
	UAvaEditorContext() = default;

	void SetAvaEditor(const TSharedRef<IAvaEditor>& InAvaEditor) { AvaEditorWeak = InAvaEditor; }

	TSharedPtr<IAvaEditor> GetAvaEditor() const { return AvaEditorWeak.Pin(); }

private:
	TWeakPtr<IAvaEditor> AvaEditorWeak;
};
