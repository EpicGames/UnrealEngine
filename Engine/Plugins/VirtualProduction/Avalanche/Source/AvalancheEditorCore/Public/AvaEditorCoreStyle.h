// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class AVALANCHEEDITORCORE_API FAvaEditorCoreStyle final : public FSlateStyleSet
{
public:
	static FAvaEditorCoreStyle& Get();

	FAvaEditorCoreStyle();
	virtual ~FAvaEditorCoreStyle() override;
};
