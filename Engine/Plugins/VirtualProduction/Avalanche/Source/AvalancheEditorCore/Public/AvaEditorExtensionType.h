// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTypeId.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

class IAvaEditorExtension;

struct FAvaEditorExtensionType
{
	bool IsValid() const
	{
		return TypeId.IsValid() && (Startup || Shutdown);
	}

	friend uint32 GetTypeHash(const FAvaEditorExtensionType& InType)
	{
		return GetTypeHash(InType.TypeId);
	}

	bool operator==(const FAvaEditorExtensionType& InOther) const
	{
		return TypeId == InOther.TypeId;
	}

	FAvaTypeId TypeId = FAvaTypeId::Invalid();

	/** Weak pointers to the instances of the type */
	TArray<TWeakPtr<IAvaEditorExtension>> Extensions;

	void(*Startup)() = nullptr;

	void(*Shutdown)() = nullptr;
};
