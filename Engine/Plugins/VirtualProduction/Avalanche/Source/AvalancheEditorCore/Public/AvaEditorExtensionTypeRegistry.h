// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaEditorExtensionType.h"
#include "AvaType.h"
#include "Containers/Set.h"

class IAvaEditorExtension;

/** Requires a given type to have StaticStartup() */
struct CAvaEditorStaticStartupProvider
{
	template<typename T>
	auto Requires()->decltype(T::StaticStartup());
};

/** Requires a given type to have StaticShutdown() */
struct CAvaEditorStaticShutdownProvider
{
	template<typename T>
	auto Requires()->decltype(T::StaticShutdown());
};

class FAvaEditorExtensionTypeRegistry 
{
	friend class FAvaEditorCoreModule;

	FAvaEditorExtensionTypeRegistry() = default;

	~FAvaEditorExtensionTypeRegistry();

public:
	AVALANCHEEDITORCORE_API static FAvaEditorExtensionTypeRegistry& Get();

	template<typename InExtensionType
		UE_REQUIRES(std::is_base_of_v<IAvaEditorExtension, InExtensionType>)>
	void RegisterExtension(const TSharedRef<InExtensionType>& InExtension)
	{
		FAvaEditorExtensionType ExtensionEntry;
		ExtensionEntry.TypeId = TAvaType<InExtensionType>::GetTypeId();

		if constexpr (TModels_V<CAvaEditorStaticStartupProvider, InExtensionType>)
		{
			ExtensionEntry.Startup  = &InExtensionType::StaticStartup;
		}

		if constexpr (TModels_V<CAvaEditorStaticShutdownProvider, InExtensionType>)
		{
			ExtensionEntry.Shutdown = &InExtensionType::StaticShutdown;
		}

		// Only register if either Startup/Shutdown exist
		if (ExtensionEntry.IsValid())
		{
			this->RegisterExtensionType(InExtension, MoveTemp(ExtensionEntry));
		}
	}

	AVALANCHEEDITORCORE_API void RemoveStaleEntries();

private:
	AVALANCHEEDITORCORE_API void RegisterExtensionType(const TSharedRef<IAvaEditorExtension>& InExtension, FAvaEditorExtensionType&& InType);

	void Shutdown();

	TSet<FAvaEditorExtensionType> ExtensionTypes;
};
