// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "UObject/NameTypes.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/WeakObjectPtr.h"

class AActor;

/** Data struct used by the editor and its extensions to process a pasted actor */
class AVALANCHEEDITORCORE_API FAvaEditorPastedActor
{
public:
	FAvaEditorPastedActor(AActor* InActor, TSoftObjectPtr<AActor>&& InTemplateActor);

	AActor* GetActor() const;

	const TSoftObjectPtr<AActor>& GetTemplateActor() const;

	bool IsDuplicate() const;

	/**
	 * Helper to build a map of the original actor name to its created actor on paste
	 * @param InPastedActors the list of pasted actors to convert to the map
	 * @param bIncludeDuplicatedActors whether to add duplicates to the actor map 
	 */
	static TMap<FName, AActor*> BuildPastedActorMap(TConstArrayView<FAvaEditorPastedActor> InPastedActors, bool bIncludeDuplicatedActors);

	/** Helper to build a map of the template actor to its duplicated actor */
	static TMap<AActor*, AActor*> BuildDuplicatedActorMap(TConstArrayView<FAvaEditorPastedActor> InPastedActors);

private:
	/** Pasted Actor */
	TWeakObjectPtr<AActor> Actor;

	/** Template Actor used for the Paste */
	TSoftObjectPtr<AActor> TemplateActor;
};
