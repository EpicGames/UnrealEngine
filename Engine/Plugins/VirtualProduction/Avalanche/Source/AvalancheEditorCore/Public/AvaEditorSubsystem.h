// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaWorldSubsystemUtils.h"
#include "IAvaEditor.h"
#include "IAvaEditorExtension.h"
#include "Subsystems/WorldSubsystem.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTypeTraits.h"
#include "AvaEditorSubsystem.generated.h"

UCLASS(MinimalAPI)
class UAvaEditorSubsystem : public UWorldSubsystem, public TAvaWorldSubsystemInterface<UAvaEditorSubsystem>
{
	GENERATED_BODY()

public:
	virtual ~UAvaEditorSubsystem() override = default;
	
	template<typename InExtensionType
		UE_REQUIRES(TIsDerivedFrom<InExtensionType, IAvaEditorExtension>::Value)>
	TSharedPtr<InExtensionType> FindExtension() const
	{
		if (TSharedPtr<IAvaEditor> Editor = ActiveEditorWeak.Pin())
		{
			return Editor->FindExtension<InExtensionType>();
		}
		return nullptr;
	}

	TSharedPtr<IAvaEditor> GetActiveEditor() const
	{
		return ActiveEditorWeak.Pin();
	}

	void OnEditorActivated(const TSharedRef<IAvaEditor>& InAvaEditor);
	void OnEditorDeactivated();

protected:
	TWeakPtr<IAvaEditor> ActiveEditorWeak;

	//~ Begin UWorldSubsystem
	AVALANCHEEDITORCORE_API virtual bool DoesSupportWorldType(const EWorldType::Type InWorldType) const override;
	//~ End UWorldSubsystem
};
