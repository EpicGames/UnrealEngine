// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/MathFwd.h"
#include "Templates/SharedPointer.h"

class FName;
class FSlateWidgetClassData;
class FText;
class FUICommandInfo;
class SWidget;

struct AVALANCHEEDITORCORE_API FAvaEditorWidgetUtils
{
	static TArray<TSharedRef<SWidget>> GetWidgetChildrenOfClass(const TSharedPtr<SWidget>& InWidget, const FSlateWidgetClassData& InWidgetClass);

	static TSharedPtr<SWidget> FindParentWidgetWithClass(const TSharedPtr<SWidget>& InWidget, const FName& InWidgetClassName);

	static FVector2f GetWidgetScale(const SWidget* InWidget);
	static FVector2f GetWidgetScale(const TSharedRef<SWidget> InWidget);

	static FText AddKeybindToTooltip(FText InDefaultTooltip, TSharedPtr<FUICommandInfo> InCommand);
};
