// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaTabSpawner.h"
#include "Internationalization/Text.h"
#include "SlateFwd.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"

class IAvaEditor;

/** Base Implementation of IAvaTabSpawner with functionality and data likely to be used across most Tab Spawners */
class AVALANCHEEDITORCORE_API FAvaTabSpawner : public IAvaTabSpawner
{
public:
	explicit FAvaTabSpawner(const TSharedRef<IAvaEditor>& InEditor, FName InTabId);

	//~ Begin IAvaTabSpawner
	virtual FName GetId() const override { return TabId; }
	virtual bool CanSpawnTab(const FSpawnTabArgs& InArgs, TWeakPtr<FTabManager> InTabManagerWeak) const override;
	virtual TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& InArgs, TWeakPtr<FTabManager> InTabManagerWeak) override;
	virtual FTabSpawnerEntry& RegisterTabSpawner(const TSharedRef<FTabManager>& InTabManager, const TSharedPtr<FWorkspaceItem>& InWorkspaceMenu) override;
	//~ End IAvaTabSpawner

protected:
	bool HasValidScene() const;

	TSharedRef<SWidget> GetNullWidget() const;

	TWeakPtr<IAvaEditor> EditorWeak;

	TWeakPtr<SDockTab> DockTab;

	FName TabId;

	FText TabLabel;

	FText TabTooltipText;

	FSlateIcon TabIcon;

	ETabRole TabRole;

	float TabInnerPadding = 0.f;

	bool bAutosizeTab = false;
};
