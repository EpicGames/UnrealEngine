// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "AvaWorldSubsystemUtils.h"
#include "Containers/Map.h"
#include "Components/PrimitiveComponent.h"
#include "GameFramework/Actor.h"
#include "Math/MathFwd.h"
#include "Math/Box.h"
#include "Math/OrientedBox.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "AvaBoundsProviderSubsystem.generated.h"

UCLASS()
class AVALANCHEEDITORCORE_API UAvaBoundsProviderSubsystem : public UTickableWorldSubsystem, public TAvaWorldSubsystemInterface<UAvaBoundsProviderSubsystem>
{
	GENERATED_BODY()

public:
	virtual ~UAvaBoundsProviderSubsystem() override = default;

	/** These methods return true if the actor was succesfully/already cached. */

	bool CacheComponentLocalBounds(UPrimitiveComponent* InComponent);

	/** The cached oriented box of the component to the world. */
	bool CacheComponentWorldOrientedBounds(UPrimitiveComponent* InComponent);

	/** The cached oriented box of the component to its container actor. */
	bool CacheComponentActorOrientedBounds(UPrimitiveComponent* InComponent);

	bool CacheActorLocalBounds(AActor* InActor);
	bool CacheActorOrientedBounds(AActor* InActor);
	bool CacheActorAndChildrenLocalBounds(AActor* InActor);
	bool CacheActorAndChildrenOrientedBounds(AActor* InActor);

	void ClearCachedBounds();

	bool HasCachedComponentLocalBounds(UPrimitiveComponent* InComponent) const;
	bool HasCachedComponentWorldOrientedBounds(UPrimitiveComponent* InComponent) const;
	bool HasCachedComponentActorOrientedBounds(UPrimitiveComponent* InComponent) const;
	bool HasCachedActorLocalBounds(AActor* InActor) const;
	bool HasCachedActorOrientedBounds(AActor* InActor) const;
	bool HasCachedActorAndChildrenLocalBounds(AActor* InActor) const;
	bool HasCachedActorAndChildrenOrientedBounds(AActor* InActor) const;

	FBox GetComponentLocalBounds(UPrimitiveComponent* InComponent);
	bool GetComponentWorldOrientedBounds(UPrimitiveComponent* InComponent, FOrientedBox& OutOrientedBounds);
	bool GetComponentActorOrientedBounds(UPrimitiveComponent* InComponent, FOrientedBox& OutOrientedBounds);

	FBox GetActorLocalBounds(AActor* InActor);
	bool GetActorOrientedBounds(AActor* InActor, FOrientedBox& OutOrientedBounds);

	FBox GetActorAndChildrenLocalBounds(AActor* InActor);
	bool GetActorAndChildrenOrientedBounds(AActor* InActor, FOrientedBox& OutOrientedBounds);

	/** These call the internal non-methods, but return data. */

	const TMap<TWeakObjectPtr<UPrimitiveComponent>, FBox>& GetCachedComponentLocalBounds() const;
	const TMap<TWeakObjectPtr<UPrimitiveComponent>, FOrientedBox>& GetCachedComponentWorldOrientedBounds() const;
	const TMap<TWeakObjectPtr<UPrimitiveComponent>, FOrientedBox>& GetCachedComponentActorOrientedBounds() const;
	const TMap<TWeakObjectPtr<AActor>, FBox>& GetCachedActorLocalBounds() const;
	const TMap<TWeakObjectPtr<AActor>, FOrientedBox>& GetCachedActorOrientedBounds() const;
	const TMap<TWeakObjectPtr<AActor>, FBox>& GetCachedActorAndChildrenLocalBounds() const;
	const TMap<TWeakObjectPtr<AActor>, FOrientedBox>& GetCachedActorAndChildrenOrientedBounds() const;

	/**
	 * Gets the axis-aligned, world-scaled bounds of the selected actors.
	 * The base transform of this box is the first selected actor's location and rotation.Actor
	 */
	FBox GetSelectionBounds(bool bInIncludeChildrenOfSelectedActors);

	/** Returns the oriented box representing the bounds selected actors. */
	bool GetSelectionOrientedBounds(bool bInIncludeChildrenOfSelectedActors, FOrientedBox& OutOrientedBounds);

	//~ Begin FTickableGameObject
	virtual bool IsTickableWhenPaused() const override;
	virtual bool IsTickableInEditor() const override;
	virtual TStatId GetStatId() const override;
	//~ End FTickableGameObject

	//~ Begin FTickableObjectBase
	virtual void Tick(float DeltaTime) override;
	//~ End FTickableGameObject

protected:
	TMap<TWeakObjectPtr<UPrimitiveComponent>, FBox> CachedComponentLocalBounds;
	TMap<TWeakObjectPtr<UPrimitiveComponent>, FOrientedBox> CachedComponentWorldOrientedBounds;
	TMap<TWeakObjectPtr<UPrimitiveComponent>, FOrientedBox> CachedComponentActorOrientedBounds;

	TMap<TWeakObjectPtr<AActor>, FBox> CachedActorLocalBounds;
	TMap<TWeakObjectPtr<AActor>, FOrientedBox> CachedActorOrientedBounds;

	TMap<TWeakObjectPtr<AActor>, FBox> CachedActorAndChildrenLocalBounds;
	TMap<TWeakObjectPtr<AActor>, FOrientedBox> CachedActorAndChildrenOrientedBounds;

	FBox CachedSelectionBounds;
	FBox CachedSelectionWithChildrenBounds;

	//~ Begin UWorldSubsystem
	virtual bool DoesSupportWorldType(const EWorldType::Type InWorldType) const override;
	//~ End UWorldSubsystem
};
