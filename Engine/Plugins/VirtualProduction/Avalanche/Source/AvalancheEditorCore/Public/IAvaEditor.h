// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "Templates/SharedPointer.h"

class AActor;
class FEditorModeTools;
class FReply;
class FTabManager;
class FUICommandList;
class FWorkspaceItem;
class IAvaEditor;
class IAvaEditorExtension;
class IAvaTabSpawner;
class IToolkitHost;
class UObject;
class UToolMenu;
class UWorld;

enum class EAvaEditorObjectQueryType : uint8
{
	/** Returns cached version if valid. If invalid, it will try to find the object via Provider GetSceneObject */
	SearchOnly,

	/** Skip looking through the GetSceneObject and just get the cached version if valid, or return null */
	SkipSearch,

	/** If there is no valid object (invalid cached version and not found), create it */
	CreateIfNotFound,
};

/** Interface used to Extend an Editor (e.g. Level Editor) for the Motion Design Workflow that extends beyond the limits of Mode UI Layer*/
class IAvaEditor : public IAvaTypeCastable, public TSharedFromThis<IAvaEditor>
{
public:
	UE_AVA_INHERITS(IAvaEditor, IAvaTypeCastable);

	virtual ~IAvaEditor() override = default;

	/** Called to setup the basics without needing a Toolkit Host or Scene */
	virtual void Construct() {}

	virtual void SetToolkitHost(TSharedRef<IToolkitHost> InToolkitHost) = 0;

	virtual void Activate(TSharedPtr<IToolkitHost> InOverrideToolkitHost = nullptr) = 0;

	virtual void Deactivate() = 0;

	/** Gives all extension an opportunity to cleanup prior to Scene Object tear down */
	virtual void Cleanup() = 0;

	virtual bool IsActive() const = 0;

	virtual bool CanActivate() const = 0;

	/** Whether the editor should auto-activate if available */
	virtual bool ShouldAutoActivate() const = 0;

	/** Sets whether the editor should auto-activate */
	virtual void SetAutoActivate(bool bInAutoActivate) = 0;

	virtual bool CanDeactivate() const = 0;

	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandList) = 0;

	virtual void Save() = 0;

	virtual void Load() = 0;

	virtual TSharedPtr<FUICommandList> GetCommandList() const = 0;

	virtual TSharedPtr<IToolkitHost> GetToolkitHost() const = 0;

	virtual TSharedPtr<FTabManager> GetTabManager() const = 0;

	virtual FEditorModeTools* GetEditorModeTools() const = 0;

	virtual UWorld* GetWorld() const = 0;

	virtual void OnSceneObjectChanged() = 0;

	virtual UObject* GetSceneObject(EAvaEditorObjectQueryType InQueryType = EAvaEditorObjectQueryType::SearchOnly) const = 0;

	virtual void RegisterTabSpawners() = 0;

	virtual void UnregisterTabSpawners() = 0;

	virtual void ExtendToolbarMenu(UToolMenu* InMenu) {}

	/** Adds entries to the create scene options menu */
	virtual void ExtendCreateSceneOptions(UToolMenu* InMenu) {}

	virtual void CloseTabs() = 0;

	virtual FReply DockInLayout(FName InTabId) = 0;

	virtual TArray<TSharedRef<IAvaEditorExtension>> GetExtensions() const = 0;

	template<typename InExtensionType, typename = typename TEnableIf<TIsDerivedFrom<InExtensionType, IAvaEditorExtension>::Value>::Type>
	TSharedPtr<InExtensionType> FindExtension() const
	{
		if (TSharedPtr<IAvaEditorExtension> FoundExtension = this->FindExtensionImpl(TAvaType<InExtensionType>::GetTypeId()))
		{
			return StaticCastSharedPtr<InExtensionType>(FoundExtension);
		}
		return nullptr;
	}

	template<typename InTabSpawnerType, typename... InArgTypes, typename = typename TEnableIf<TIsDerivedFrom<InTabSpawnerType, IAvaTabSpawner>::Value>::Type>
	void AddTabSpawner(InArgTypes&&... InArgs)
	{
		TSharedRef<IAvaTabSpawner> TabSpawner = MakeShared<InTabSpawnerType>(Forward<InArgTypes>(InArgs)...);
		this->AddTabSpawnerImpl(TabSpawner);
	}

	virtual void OnSelectionChanged(UObject* InSelection) = 0;

	virtual bool EditCut() = 0;

	virtual bool EditCopy() = 0;

	virtual bool EditPaste() = 0;

	virtual bool EditDuplicate() = 0;

	virtual bool EditDelete() = 0;

	/** Returns the root category for motion design tabs and other categories */
	virtual const TSharedPtr<FWorkspaceItem>& GetRootCategory() const = 0;

	/** Returns the outliner sub-category for motion design tabs */
	virtual const TSharedPtr<FWorkspaceItem>& GetOutlinerCategory() const = 0;

	/** Returns the tools sub-category for motion design tabs */
	virtual const TSharedPtr<FWorkspaceItem>& GetToolsCategory() const = 0;

protected:
	virtual TSharedPtr<IAvaEditorExtension> FindExtensionImpl(FAvaTypeId InExtensionId) const = 0;

	virtual void AddTabSpawnerImpl(TSharedRef<IAvaTabSpawner> InTabSpawner) = 0;
};
