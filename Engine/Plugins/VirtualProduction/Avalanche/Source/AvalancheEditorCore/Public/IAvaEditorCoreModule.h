// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class UToolMenu;

class IAvaEditorCoreModule : public IModuleInterface
{
	static constexpr const TCHAR* ModuleName = TEXT("AvalancheEditorCore");

public:
	static bool IsLoaded()
	{
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	static IAvaEditorCoreModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IAvaEditorCoreModule>(ModuleName);
	}

	/** Delegate to extend the Editor Toolbar */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnExtendEditorToolbar, UToolMenu&);
	virtual FOnExtendEditorToolbar& GetOnExtendEditorToolbar() = 0;
};
