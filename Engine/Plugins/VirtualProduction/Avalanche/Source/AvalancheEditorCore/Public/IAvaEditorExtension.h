// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaEditorPastedActor.h"
#include "AvaType.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "IAvaEditor.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"

class AActor;
class FAvaEditorSelection;
class FEditorModeTools;
class FLayoutExtender;
class FUICommandList;
class IDetailCategoryBuilder;
class IToolkitHost;
class UToolMenu;
class UWorld;

class IAvaEditorExtension : public IAvaTypeCastable, public TSharedFromThis<IAvaEditorExtension>
{
public:
	UE_AVA_INHERITS(IAvaEditorExtension, IAvaTypeCastable)

	/** Name of the Default Section Name used to add or as reference when Extending the Toolbar Menu */
	static constexpr const TCHAR* DefaultSectionName = TEXT("DefaultExtensions");

	/** Called as soon as this Editor Extension is created. Can use TSharedFromThis as it will be outside Ctor */
	virtual void Construct(const TSharedRef<IAvaEditor>& InEditor) {}

	/** Called when the Ava Editor is fully active (e.g. Toolkit and Scene have been set). Called right before invoking tabs */
	virtual void Activate() {}

	/** Called after Activate and after all the Extension Tabs have been invoked */
	virtual void PostInvokeTabs() {}

	/**
	 * Called when deactivating the Scene.
	 * This does not necessarily mean the scene object is being destroyed.
	 * For Destruction/Cleanup, use IAvaEditorExtension::Cleanup
	 */
	virtual void Deactivate() {}

	/** Called when destroying the object containing the Scene Object for the Extension */
	virtual void Cleanup() {}

	virtual TSharedPtr<IAvaEditor> GetEditor() const = 0;

	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandList) {}

	/** Called when the scene object has been created */
	virtual void OnSceneObjectCreated(UObject* InSceneObject) {}

	/** Called when the scene object has been changed/replaced*/
	virtual void OnSceneObjectChanged(UObject* InOldSceneObject, UObject* InNewSceneObject) {}

	virtual void RegisterTabSpawners(const TSharedRef<IAvaEditor>& InEditor) const {}

	/** Opportunity for an Extension to extend the Editor Toolbar */
	virtual void ExtendToolbarMenu(UToolMenu& InMenu) {}

	/** Extend the Level Editor Layout (only called when instancing for Level Editor) */
	virtual void ExtendLevelEditorLayout(FLayoutExtender& InExtender) const {}

	/** Opportunity for an extension to add to the create scene options menu */
	virtual void ExtendCreateSceneOptions(UToolMenu* InMenu) {}

	/**
	 * Name of the category this Extension belongs in.
	 * Used in places like Setting Details Category
	 */
	virtual FName GetCategoryName() const
	{
		return NAME_None;
	}

	/**
	 * Extend the Settings Category by adding External Objects or Properties.
	 * Must have a valid category name to proceed.
	 * @see IAvaEditorExtension::GetCategoryName
	 */
	virtual void ExtendSettingsCategory(IDetailCategoryBuilder& InCategoryBuilder) {}

	virtual void Save() {}

	virtual void Load() {}

	/** Called when a USelection relevant to the Toolkit Mode Tools has changed */
	virtual void NotifyOnSelectionChanged(const FAvaEditorSelection& InSelection) {}

	/** Called when Actors are about to be copied and opportunity for Extension to add to the Copied String */
	virtual void OnCopyActors(FString& OutCopyData, TConstArrayView<AActor*> InActorsToCopy) {}

	/** Called prior to pasting the actors. Give opportunity for Extension to prepare for pasting */
	virtual void PrePasteActors() {}

	/** Called after pasting the actors regardless if it successfully completes or not */
	virtual void PostPasteActors(bool bInPasteSucceeded) {}

	/** Called when pasting new actors to give opportunity for Extension to handle Pasted Data and its Actors */
	virtual void OnPasteActors(FStringView InPastedData, TConstArrayView<FAvaEditorPastedActor> InPastedActors) {}

	UWorld* GetWorld() const
	{
		const TSharedPtr<IAvaEditor> Editor = GetEditor();
		return Editor.IsValid() ? Editor->GetWorld() : nullptr;
	}

	FEditorModeTools* GetEditorModeTools() const
	{
		const TSharedPtr<IAvaEditor> Editor = GetEditor();
		return Editor.IsValid() ? Editor->GetEditorModeTools() : nullptr;
	}

	template<typename InSceneObjectType = UObject>
	InSceneObjectType* GetSceneObject() const
	{
		const TSharedPtr<IAvaEditor> Editor = GetEditor();
		return Editor.IsValid()
			? Cast<InSceneObjectType>(Editor->GetSceneObject(EAvaEditorObjectQueryType::SearchOnly))
			: nullptr;
	}

	TSharedPtr<IToolkitHost> GetToolkitHost() const
	{
		const TSharedPtr<IAvaEditor> Editor = GetEditor();
		return Editor.IsValid() ? Editor->GetToolkitHost() : nullptr;
	}

	bool IsEditorActive() const
	{
		const TSharedPtr<IAvaEditor> Editor = GetEditor();
		return Editor.IsValid() && Editor->IsActive();
	}
};

class FAvaEditorExtension : public IAvaEditorExtension
{
public:
	UE_AVA_INHERITS(FAvaEditorExtension, IAvaEditorExtension)

	//~ Begin IAvaEditorExtension
	virtual void Construct(const TSharedRef<IAvaEditor>& InEditor) override { EditorWeak = InEditor; }
	virtual TSharedPtr<IAvaEditor> GetEditor() const override { return EditorWeak.Pin(); }
	//~ End IAvaEditorExtension

private:
	TWeakPtr<IAvaEditor> EditorWeak;
};
