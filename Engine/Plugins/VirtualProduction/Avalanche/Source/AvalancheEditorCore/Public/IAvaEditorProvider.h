// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointer.h"

class AActor;
class UObject;
class UWorld;
enum class EAvaEditorObjectQueryType : uint8;

class IAvaEditorProvider : public TSharedFromThis<IAvaEditorProvider>
{
public:
	virtual ~IAvaEditorProvider() = default;

	/** Called right after instantiating the provider */
	virtual void Construct()
	{
	}

	/**
	 * Gets the Scene Object this Ava Editor Instance is using.
	 * @param InWorld the world to optionally help get the Scene Object
	 * @param InQueryType the type of query in order to get the object either from cache, or create one if not found
	 */
	virtual UObject* GetSceneObject(UWorld* InWorld, EAvaEditorObjectQueryType InQueryType) = 0;

	/** Whether the scene object can be auto-activated if the functionality is available */
	virtual bool ShouldAutoActivateScene(UObject* InSceneObject) const
	{
		return true;
	}

	/** Sets the auto activate scene option for the scene object to persist it */
	virtual void SetAutoActivateScene(UObject* InSceneObject, bool bInAutoActivateScene) const
	{
		
	}
	
	/** Optional: Add/Remove from the Actor array when performing operations such as Copy/Cut */
	virtual void GetActorsToEdit(TArray<AActor*>& InOutActorsToEdit) const
	{
	}

	virtual void OnSceneActivated()
	{
	}

	virtual void OnSceneDeactivated()
	{
	}
};
