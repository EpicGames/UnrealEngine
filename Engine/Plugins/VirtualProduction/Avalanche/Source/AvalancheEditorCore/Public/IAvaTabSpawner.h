// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FName;
class FSpawnTabArgs;
class FTabManager;
class FWorkspaceItem;
class SDockTab;
class SWidget;
struct FTabSpawnerEntry;

class IAvaTabSpawner : public TSharedFromThis<IAvaTabSpawner>
{
public:
	virtual ~IAvaTabSpawner() = default;

	virtual FName GetId() const = 0;

	virtual bool CanSpawnTab(const FSpawnTabArgs& InArgs, TWeakPtr<FTabManager> InTabManagerWeak) const { return true; }

	virtual TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& InArgs, TWeakPtr<FTabManager> InTabManagerWeak) = 0;

	virtual TSharedRef<SWidget> CreateTabBody() = 0;

	virtual FTabSpawnerEntry& RegisterTabSpawner(const TSharedRef<FTabManager>& InTabManager, const TSharedPtr<FWorkspaceItem>& InWorkspaceMenu) = 0;
};
