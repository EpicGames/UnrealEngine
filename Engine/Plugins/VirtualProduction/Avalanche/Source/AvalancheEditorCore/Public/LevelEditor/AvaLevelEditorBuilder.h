// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaEditorBuilder.h"

class AVALANCHEEDITORCORE_API FAvaLevelEditorBuilder : public FAvaEditorBuilder
{
protected:
	//~ Begin FAvaEditorBuilder
	virtual TSharedRef<IAvaEditor> CreateEditor() override;
	//~ End FAvaEditorBuilder
};
