// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"

class FLevelEditorModule;
class FName;

/** Utility class for the Level Editor and the Motion Design extension editor of it */
class FAvaLevelEditorUtils
{
public:
	static AVALANCHEEDITORCORE_API FLevelEditorModule* GetLevelEditorModule();

	static AVALANCHEEDITORCORE_API FLevelEditorModule* LoadLevelEditorModule();

	static AVALANCHEEDITORCORE_API TConstArrayView<FName> GetDetailsViewNames();
};
