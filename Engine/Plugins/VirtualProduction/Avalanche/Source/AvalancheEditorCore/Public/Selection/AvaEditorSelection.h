// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "EditorModeManager.h"
#include "Selection.h"
#include "Templates/SharedPointer.h"

class AActor;
class FEditorModeTools;
class UActorComponent;
class UObject;

enum class EAvaSelectionSource
{
	/** Only consider one USelection Object, closest to the Object Type. E.g. AActor will use SelectedActors */
	Single,
	/** Consider all USelection Objects in Editor Mode Tools (i.e. SelectedActors, SelectedComponents and SelectedObjects) */
	All,
};

class FAvaEditorSelection
{
	template<typename InObjectType, typename InParentObjectType = UObject>
	using TDerivesFromUObject = typename TEnableIf<TIsDerivedFrom<InObjectType, InParentObjectType>::Value>::Type;

public:
	FAvaEditorSelection(FEditorModeTools& InModeTools, UObject* InChangedSelection = nullptr)
		: ModeTools(InModeTools)
		, ChangedSelectionWeak(InChangedSelection)
	{
	}

	/**
	 * Determines this Selection valid either if there's either no Changed Selection (i.e. just handling Mode Tools)
	 * or the given Selection Object is one of the 3 in the given Editor Mode Tools
	 */
	bool IsValid() const
	{
		return !ChangedSelectionWeak.IsValid()
			|| ChangedSelectionWeak == ModeTools.GetSelectedActors()
			|| ChangedSelectionWeak == ModeTools.GetSelectedComponents()
			|| ChangedSelectionWeak == ModeTools.GetSelectedObjects();
	}

	UObject* GetChangedSelection() const { return ChangedSelectionWeak.Get(); }

	/** Retrieves the USelection object closest to Object Type */
	template<typename InObjectType = UObject, typename = TDerivesFromUObject<InObjectType>>
	USelection* GetSelection() const
	{
		if constexpr (TIsDerivedFrom<InObjectType, AActor>::Value)
		{
			return ModeTools.GetSelectedActors();
		}
		else if constexpr (TIsDerivedFrom<InObjectType, UActorComponent>::Value)
		{
			return ModeTools.GetSelectedComponents();
		}
		else
		{
			return ModeTools.GetSelectedObjects();
		}
	}

	template<typename InObjectType = UObject, EAvaSelectionSource InSource = EAvaSelectionSource::All
		, typename InObjectReturnType = InObjectType
		, typename = TDerivesFromUObject<InObjectType>
		, typename = TDerivesFromUObject<InObjectType, InObjectReturnType>>
	TArray<InObjectReturnType*> GetSelectedObjects() const
	{
		TArray<InObjectReturnType*> OutSelectedObjects;

		if constexpr (InSource == EAvaSelectionSource::Single)
		{
			OutSelectedObjects = GetSelectedObjects<InObjectReturnType*>(GetSelection<InObjectType>());
		}
		else
		{
			static_assert(InSource == EAvaSelectionSource::All);
			OutSelectedObjects.Append(GetSelectedObjects<InObjectReturnType*>(GetSelection<AActor>()));
			OutSelectedObjects.Append(GetSelectedObjects<InObjectReturnType*>(GetSelection<UActorComponent>()));
			OutSelectedObjects.Append(GetSelectedObjects<InObjectReturnType*>(GetSelection<UObject>()));
		}
		return OutSelectedObjects;
	}

private:
	template<typename InElementType>
	TArray<InElementType> GetSelectedObjects(USelection* InSelection) const
	{
		if (!ensure(InSelection))
		{
			return TArray<InElementType>();
		}
		TArray<InElementType> OutSelectedObjects;
		InSelection->GetSelectedObjects(OutSelectedObjects);
		return OutSelectedObjects;
	}

	/** The Mode Tools used to get the USelection instances */
	FEditorModeTools& ModeTools;

	/** The Selection that changed that triggered this FAvaEditorSelection to be instanced. Can be null if nothing changed. */
	TWeakObjectPtr<UObject> ChangedSelectionWeak;
};
