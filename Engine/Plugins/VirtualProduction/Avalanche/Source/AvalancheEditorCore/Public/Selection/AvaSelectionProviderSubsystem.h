// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "AvaWorldSubsystemUtils.h"
#include "Components/ActorComponent.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "GameFramework/Actor.h"
#include "Math/MathFwd.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "AvaSelectionProviderSubsystem.generated.h"

class FAvaEditorSelection;
class UWorld;

UCLASS()
class AVALANCHEEDITORCORE_API UAvaSelectionProviderSubsystem : public UWorldSubsystem, public TAvaWorldSubsystemInterface<UAvaSelectionProviderSubsystem>
{
	GENERATED_BODY()

public:
	virtual ~UAvaSelectionProviderSubsystem() override = default;

	void UpdateSelection(const FAvaEditorSelection& InSelection);

	void ClearAttachedActorCache();

	TConstArrayView<TWeakObjectPtr<AActor>> GetSelectedActors() const { return CachedSelectedActors; }
	TConstArrayView<TWeakObjectPtr<UActorComponent>> GetSelectedComponents() const { return CachedSelectedComponents; }

	/**
	 * Location: Center of the selected actors' bounds.
	 * Rotation: Rotation of the first actor.
	 * Scale: 1, 1, 1
	 */
	FTransform GetSelectionTransform() const;

	TConstArrayView<TWeakObjectPtr<AActor>> GetAttachedActors(AActor* InActor, bool bInRecursive);

protected:
	TArray<TWeakObjectPtr<AActor>> CachedSelectedActors;
	TMap<TWeakObjectPtr<AActor>, TArray<TWeakObjectPtr<AActor>>> CachedDirectlyAttachedActors;
	TMap<TWeakObjectPtr<AActor>, TArray<TWeakObjectPtr<AActor>>> CachedRecursivelyAttachedActors;

	/** Does not cache selected root components. These should always be considered selected if the actor is. */
	TArray<TWeakObjectPtr<UActorComponent>> CachedSelectedComponents;

	//~ Begin UWorldSubsystem
	virtual bool DoesSupportWorldType(const EWorldType::Type InWorldType) const override;
	//~ End UWorldSubsystem
};
