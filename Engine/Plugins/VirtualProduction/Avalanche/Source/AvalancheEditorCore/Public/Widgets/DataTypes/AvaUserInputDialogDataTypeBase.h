// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Templates/SharedPointer.h"

class SWidget;

struct FAvaUserInputDialogDataTypeBase : public TSharedFromThis<FAvaUserInputDialogDataTypeBase>
{
	friend class SAvaUserInputDialog;

	virtual ~FAvaUserInputDialogDataTypeBase() = default;

	virtual TSharedRef<SWidget> CreateInputWidget() = 0;

	virtual bool IsValueValid()
	{
		return true;
	}

protected:
	FSimpleDelegate OnCommit;

	void OnUserCommit()
	{
		if (IsValueValid())
		{
			OnCommit.ExecuteIfBound();
		}
	}
};
