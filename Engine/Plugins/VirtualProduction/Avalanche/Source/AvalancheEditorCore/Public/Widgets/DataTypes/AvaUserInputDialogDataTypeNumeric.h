// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DataTypes/AvaUserInputDialogDataTypeBase.h"
#include "Misc/Optional.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBox.h"

template<typename InNumericType
	UE_REQUIRES(TAnd<TIsArithmetic<InNumericType>, TNot<std::is_same<bool, InNumericType>>>::Value)>
struct FAvaUserInputDialogNumericData : public FAvaUserInputDialogDataTypeBase
{
	using FNumericType = InNumericType;
	using ThisStruct = FAvaUserInputDialogNumericData<FNumericType>;

	struct FParams
	{
		FNumericType InitialValue = 0;
		TOptional<FNumericType> InMinValue = TOptional<FNumericType>();
		TOptional<FNumericType> InMaxValue = TOptional<FNumericType>();
	};

	FAvaUserInputDialogNumericData(const FParams& InParams)
		: Value(InParams.InitialValue)
		, MinValue(InParams.MinValue)
		, MaxValue(InParams.MaxValue)
	{
	}

	FNumericType GetValue() const { return Value; }

	//~ Begin FAvaUserInputDialogDataTypeBase
	virtual TSharedRef<SWidget> CreateInputWidget()
	{
		return SNew(SBox)
			.WidthOverride(200.f)
			.HAlign(HAlign_Fill)
			[
				SNew(SSpinBox<FNumericType>)
				.Value(Value)
				.MinValue(MinValue)
				.MaxSliderValue(MinValue)
				.MaxValue(MaxValue)
				.MaxSliderValue(MaxValue)
				.EnableSlider(true)
				.OnValueChanged(this, &ThisStruct::OnValueChanged)
				.OnValueCommitted(this, &ThisStruct::OnValueCommitted)
			];
	}
	//~ End FAvaUserInputDialogDataTypeBase

protected:
	FNumericType Value;
	TOptional<FNumericType> MinValue;
	TOptional<FNumericType> MaxValue;

	void OnValueChanged(FNumericType InValue)
	{
		Value = InValue;
	}

	void OnValueCommitted(FNumericType InValue, ETextCommit::Type InCommitType)
	{
		Value = InValue;
	}
};