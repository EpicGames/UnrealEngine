// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaUserInputDialogDataTypeBase.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"

class FStructOnScope;

struct FAvaUserInputDialogDataTypeStruct : FAvaUserInputDialogDataTypeBase
{
	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsValidDelegate, TSharedRef<FStructOnScope>)

	struct FParams
	{
		UScriptStruct* Struct = nullptr;
		FIsValidDelegate IsValidDelegate;
	};

	AVALANCHEEDITORCORE_API FAvaUserInputDialogDataTypeStruct(const FParams& InParams);

	virtual ~FAvaUserInputDialogDataTypeStruct() override = default;

	AVALANCHEEDITORCORE_API const uint8* GetStructMemory() const;

	AVALANCHEEDITORCORE_API const UStruct* GetStruct() const;

	//~ Begin FAvaUserInputDataTypeBase
	AVALANCHEEDITORCORE_API virtual TSharedRef<SWidget> CreateInputWidget() override;
	AVALANCHEEDITORCORE_API virtual bool IsValueValid() override;
	//~ End FAvaUserInputDataTypeBase

protected:
	TSharedRef<FStructOnScope> StructOnScope;
	FIsValidDelegate IsValidDelegate;
};

template<typename InStructType>
struct TAvaUserInputDialogDataTypeStruct : FAvaUserInputDialogDataTypeStruct
{
	struct FParams
	{
		FIsValidDelegate IsValidDelegate;
	};

	explicit TAvaUserInputDialogDataTypeStruct(const FParams& InParams)
		: FAvaUserInputDialogDataTypeStruct({TBaseStructure<InStructType>::Get(), InParams.IsValidDelegate})
	{
	}

	virtual ~TAvaUserInputDialogDataTypeStruct() override = default;

	const InStructType* GetPtr() const
	{
		const UStruct* ScriptStruct = GetStruct();
		if (ScriptStruct && (ScriptStruct == TBaseStructure<InStructType>::Get() || ScriptStruct->IsChildOf(TBaseStructure<InStructType>::Get())))
		{
			return reinterpret_cast<const InStructType*>(GetStructMemory());
		}
		return nullptr;
	}

	const InStructType& Get() const
	{
		const UStruct* ScriptStruct = GetStruct();
		const uint8* StructMemory = GetStructMemory();
		check(StructMemory);
		check(ScriptStruct && (ScriptStruct == TBaseStructure<InStructType>::Get() || ScriptStruct->IsChildOf(TBaseStructure<InStructType>::Get())));
		return *reinterpret_cast<const InStructType*>(StructMemory);
	}
};
