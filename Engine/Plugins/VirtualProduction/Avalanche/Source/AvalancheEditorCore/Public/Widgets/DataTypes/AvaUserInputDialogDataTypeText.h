// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DataTypes/AvaUserInputDialogDataTypeBase.h"
#include "Framework/SlateDelegates.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"

namespace ETextCommit { enum Type : int; }

struct FAvaUserInputDialogTextData : public FAvaUserInputDialogDataTypeBase
{
	struct FParams
	{
		FText InitialValue = FText::GetEmpty();
		bool bAllowMultiline = false;
		TOptional<int32> MaxLength = TOptional<int32>();
		FOnVerifyTextChanged OnVerifyDelegate;
	};

	AVALANCHEEDITORCORE_API FAvaUserInputDialogTextData(const FParams& InParams);

	virtual ~FAvaUserInputDialogTextData() override = default;

	AVALANCHEEDITORCORE_API const FText& GetValue() const;

	//~ Begin FAvaUserInputDialogDataTypeBase
	AVALANCHEEDITORCORE_API virtual TSharedRef<SWidget> CreateInputWidget() override;
	AVALANCHEEDITORCORE_API virtual bool IsValueValid() override;
	//~ End FAvaUserInputDialogDataTypeBase

protected:
	FText Value;
	bool bAllowMultiline;
	TOptional<int32> MaxLength;
	FOnVerifyTextChanged OnVerifyDelegate;

	void OnTextChanged(const FText& InValue);

	void OnTextCommitted(const FText& InValue, ETextCommit::Type InCommitType);

	bool OnTextVerify(const FText& InValue, FText& OutErrorText);
};