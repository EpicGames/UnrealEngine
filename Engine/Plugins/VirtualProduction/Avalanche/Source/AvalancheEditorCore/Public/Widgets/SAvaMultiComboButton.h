// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * This is a copy+paste of the engine SComboButton because it does not support some required
 * functionality (the ability to use right click to open the menu and left click to activate
 * the button).
 */

#pragma once

#include "Layout/Margin.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/AppStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SMenuAnchor.h"

class SButton;
class SImage;

/**
 * A button that, when clicked, brings up a popup.
 */
class SAvaMultiComboButton : public SMenuAnchor
{
public:

	SLATE_BEGIN_ARGS( SAvaMultiComboButton )
		: _ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle< FComboButtonStyle >( "ComboButton" ))
		, _ButtonStyle(nullptr)
		, _ButtonContent()
		, _MenuContent()
		, _IsFocusable(true)
		, _HasDownArrow(true)
		, _ForegroundColor(FSlateColor::UseStyle())
		, _ButtonColorAndOpacity(FLinearColor::White)
		, _MenuPlacement(MenuPlacement_ComboBox)
		, _HAlign(HAlign_Fill)
		, _VAlign(VAlign_Center)
		, _Method()
		, _CollapseMenuOnParentFocus(false)
		{}

		SLATE_STYLE_ARGUMENT( FComboButtonStyle, ComboButtonStyle )

		/** The visual style of the button (overrides ComboButtonStyle) */
		SLATE_STYLE_ARGUMENT( FButtonStyle, ButtonStyle )
		
		SLATE_NAMED_SLOT( FArguments, ButtonContent )
		
		/** Optional static menu content.  If the menu content needs to be dynamically built, use the OnGetMenuContent event */
		SLATE_NAMED_SLOT( FArguments, MenuContent )
		
		/** Sets an event handler to generate a widget dynamically when the menu is needed. */
		SLATE_EVENT( FOnGetContent, OnGetMenuContent )
		SLATE_EVENT( FOnIsOpenChanged, OnMenuOpenChanged )
		
		SLATE_EVENT( FOnComboBoxOpened, OnComboBoxOpened )

		SLATE_EVENT(FOnClicked, OnButtonClicked)

		SLATE_ARGUMENT( bool, IsFocusable )
		SLATE_ARGUMENT( bool, HasDownArrow )

		SLATE_ATTRIBUTE( FSlateColor, ForegroundColor )
		SLATE_ATTRIBUTE( FSlateColor, ButtonColorAndOpacity )
		SLATE_ATTRIBUTE( FMargin, ContentPadding )
		SLATE_ATTRIBUTE( EMenuPlacement, MenuPlacement )
		SLATE_ARGUMENT( EHorizontalAlignment, HAlign )
		SLATE_ARGUMENT( EVerticalAlignment, VAlign )

		/** Spawn a new window or reuse current window for this combo*/
		SLATE_ARGUMENT( TOptional<EPopupMethod>, Method )

		/** True if this combo's menu should be collapsed when our parent receives focus, false (default) otherwise */
		SLATE_ARGUMENT(bool, CollapseMenuOnParentFocus)

	SLATE_END_ARGS()

	// SMenuAnchor interface
	AVALANCHEEDITORCORE_API virtual void SetMenuContent(TSharedRef<SWidget> InContent) override;
	// End of SMenuAnchor interface

	/** See the OnGetMenuContent event */
	AVALANCHEEDITORCORE_API void SetOnGetMenuContent( FOnGetContent InOnGetMenuContent );

	/**
	 * Construct the widget from a declaration
	 *
	 * @param InArgs  The declaration from which to construct
	 */
	AVALANCHEEDITORCORE_API void Construct(const FArguments& InArgs);

	void SetMenuContentWidgetToFocus( TWeakPtr<SWidget> InWidgetToFocusPtr )
	{
		WidgetToFocusPtr = InWidgetToFocusPtr;
	}

	/** See the padding for button content. */
	AVALANCHEEDITORCORE_API void SetButtonContentPadding(FMargin InPadding);

	/** add/remove the expanding arrow. */
	AVALANCHEEDITORCORE_API void SetHasDownArrow(bool InHasArrowDown);

	//~ Begin SWidget
	AVALANCHEEDITORCORE_API virtual FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	//~ End SWidget

protected:
	/**
	 * Handle the button being clicked by summoning the ComboButton.
	 */
	AVALANCHEEDITORCORE_API virtual FReply OnButtonClicked();
	AVALANCHEEDITORCORE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/**
	 * Called to query the tool tip text for this widget, but will return an empty text when the menu is already open
	 *
	 * @param	ToolTipText	Tool tip text to display, if possible
	 *
	 * @return	Tool tip text, or an empty text if filtered out
	 */
	AVALANCHEEDITORCORE_API FText GetFilteredToolTipText(TAttribute<FText> ToolTipText) const;

protected:
	/** Area where the button's content resides */
	SHorizontalBox::FSlot* ButtonContentSlot;

	/** Delegate to execute when the combo list is opened */
	FOnComboBoxOpened OnComboBoxOpened;

	TWeakPtr<SWidget> WidgetToFocusPtr;

	/** Brush to use to add a "menu border" around the drop-down content */
	const FSlateBrush* MenuBorderBrush;

	/** Padding to use to add a "menu border" around the drop-down content */
	FMargin MenuBorderPadding;

	/** The content widget, if any, set by the user on creation */
	TWeakPtr<SWidget> ContentWidgetPtr;

	/** Can this button be focused? */
	bool bIsFocusable;

private:
	/** The button widget within the ComboButton */
	TSharedPtr<SButton> ButtonPtr;

	/** The box containing the arrow */
	TSharedPtr<SHorizontalBox> HBox;

	/** The arrow image shadow */
	TSharedPtr<SImage> ShadowImage;

	/** The arrow image */
	TSharedPtr<SImage> ForegroundArrowImage;

	/** The ComboButton style */
	const FComboButtonStyle* Style;
};
