// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DataTypes/AvaUserInputDialogDataTypeBase.h"

class SAvaUserInputDialog : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET_API(SAvaUserInputDialog, SCompoundWidget, AVALANCHEEDITORCORE_API)

public:
	static AVALANCHEEDITORCORE_API bool CreateModalDialog(const TSharedRef<FAvaUserInputDialogDataTypeBase>& InInputType, const TSharedPtr<SWidget>& InParent = nullptr,
		const TOptional<FText>& InPrompt = TOptional<FText>(), const TOptional<FText>& InTitle = TOptional<FText>());

	SLATE_BEGIN_ARGS(SAvaUserInputDialog) {}
		SLATE_ARGUMENT(FText, Prompt)
	SLATE_END_ARGS()

	AVALANCHEEDITORCORE_API void Construct(const FArguments& InArgs, const TSharedRef<FAvaUserInputDialogDataTypeBase>& InInputType);

	AVALANCHEEDITORCORE_API TSharedPtr<FAvaUserInputDialogDataTypeBase> GetInputType() const;

	AVALANCHEEDITORCORE_API bool WasAccepted() const;

private:
	TSharedPtr<FAvaUserInputDialogDataTypeBase> InputType;
	bool bAccepted = false;

	FReply OnAcceptClicked();

	bool GetAcceptedEnabled() const;

	FReply OnCancelClicked();

	void OnUserCommit();

	void Close(bool bInAccepted);
};
