// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheEffectors : ModuleRules
{
	public AvalancheEffectors(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Avalanche",
				"AvalancheCore",
				"AvalancheSceneTree",
				"ClonerEffector",
				"Core",
				"CoreUObject",
				"Engine",
			}
		);

		if (Target.Type == TargetRules.TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"AvalancheOutliner",
				"UnrealEd"
			});
		}
	}
}
