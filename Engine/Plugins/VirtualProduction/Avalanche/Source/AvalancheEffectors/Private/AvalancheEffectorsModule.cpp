// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvalancheEffectorsModule.h"

#include "AvalancheEffectorsSceneTreeResolver.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Modules/ModuleManager.h"
#include "Subsystems/CEClonerSubsystem.h"
#include "Templates/SharedPointer.h"

void FAvalancheEffectorsModule::StartupModule()
{
	UCEClonerSubsystem::OnGetSceneTreeResolver().BindStatic(&FAvalancheEffectorsModule::CreateSceneTreeResolver);
}

void FAvalancheEffectorsModule::ShutdownModule()
{
	UCEClonerSubsystem::OnGetSceneTreeResolver().Unbind();
}

TSharedPtr<ICEClonerSceneTreeCustomResolver> FAvalancheEffectorsModule::CreateSceneTreeResolver(ULevel* InLevel)
{
	if (IsValid(InLevel))
	{
		return MakeShared<FAvaEffectorsSceneTreeResolver>(InLevel);
	}

	return nullptr;
}

IMPLEMENT_MODULE(FAvalancheEffectorsModule, AvalancheEffectors)
