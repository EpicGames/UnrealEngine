// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointerFwd.h"

class AActor;
class ICEClonerSceneTreeCustomResolver;
class ULevel;

class FAvalancheEffectorsModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

protected:
	static TSharedPtr<ICEClonerSceneTreeCustomResolver> CreateSceneTreeResolver(ULevel* InLevel);
};
