// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvalancheEffectorsSceneTreeResolver.h"

#include "AvaActorUtils.h"
#include "AvaSceneSubsystem.h"
#include "AvaSceneTree.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "IAvaSceneInterface.h"

#if WITH_EDITOR
#include "AvaOutlinerSubsystem.h"
#include "AvaOutlinerUtils.h"
#include "IAvaOutliner.h"
#endif

FAvaEffectorsSceneTreeResolver::FAvaEffectorsSceneTreeResolver(ULevel* InLevel)
	: LevelWeak(InLevel)
{
}

void FAvaEffectorsSceneTreeResolver::Activate()
{
#if WITH_EDITOR
	const ULevel* Level = LevelWeak.Get();

	if (!Level)
	{
		return;
	}

	const UWorld* World = Level->GetWorld();

	if (!World)
	{
		return;
	}

	if (UAvaOutlinerSubsystem* const OutlinerSubsystem = World->GetSubsystem<UAvaOutlinerSubsystem>())
	{
		UAvaOutlinerSubsystem::FActorHierarchyChanged& ActorHierarchyChanged = OutlinerSubsystem->OnActorHierarchyChanged();

		ActorHierarchyChanged.RemoveAll(this);
		ActorHierarchyChanged.AddSP(this, &FAvaEffectorsSceneTreeResolver::OnOutlinerHierarchyChanged);

		if (const TSharedPtr<IAvaOutliner> Outliner = OutlinerSubsystem->GetOutliner())
		{
			IAvaOutliner::FOnOutlinerLoaded& OnOutlinerLoaded = Outliner->GetOnOutlinerLoaded();
			OnOutlinerLoaded.RemoveAll(this);
			OnOutlinerLoaded.AddSP(this, &FAvaEffectorsSceneTreeResolver::OnOutlinerLoaded);
		}
	}
#endif
}

void FAvaEffectorsSceneTreeResolver::Deactivate()
{
#if WITH_EDITOR
	const ULevel* Level = LevelWeak.Get();
		
	if (!Level)
	{
		return;
	}

	const UWorld* World = Level->GetWorld();

	if (!World)
	{
		return;
	}

	if (UAvaOutlinerSubsystem* const OutlinerSubsystem = World->GetSubsystem<UAvaOutlinerSubsystem>())
	{
		OutlinerSubsystem->OnActorHierarchyChanged().RemoveAll(this);

		if (const TSharedPtr<IAvaOutliner> Outliner = OutlinerSubsystem->GetOutliner())
		{
			Outliner->GetOnOutlinerLoaded().RemoveAll(this);
		}
	}
#endif
}

bool FAvaEffectorsSceneTreeResolver::GetDirectChildrenActor(AActor* InActor, TArray<AActor*>& OutActors) const
{
	if (!IsValid(InActor))
	{
		return false;	
	}
	
	bool bIsOutlinerAttachedActors = false;
	
#if WITH_EDITOR
	if (const UWorld* const World = InActor->GetWorld())
	{
		if (const UAvaOutlinerSubsystem* const OutlinerSubsystem = World->GetSubsystem<UAvaOutlinerSubsystem>())
		{
			if (const TSharedPtr<IAvaOutliner> AvaOutliner = OutlinerSubsystem->GetOutliner())
			{
				OutActors = FAvaOutlinerUtils::EditorOutlinerChildActors(AvaOutliner, InActor);
				bIsOutlinerAttachedActors = true;
			}
		}
	}
#endif

	if (!bIsOutlinerAttachedActors)
	{
		if (const IAvaSceneInterface* SceneInterface = FAvaActorUtils::GetSceneInterfaceFromActor(InActor))
		{
			const FAvaSceneTree& SceneTree = SceneInterface->GetSceneTree();
			OutActors = SceneTree.GetChildActors(InActor);
			bIsOutlinerAttachedActors = true;
		}
	}

	return bIsOutlinerAttachedActors;
}

FAvaEffectorsSceneTreeResolver::FOnActorHierarchyChanged::RegistrationType& FAvaEffectorsSceneTreeResolver::OnActorHierarchyChanged()
{
	return OnHierarchyChangedDelegate;
}

#if WITH_EDITOR
void FAvaEffectorsSceneTreeResolver::OnOutlinerLoaded()
{
	OnHierarchyChangedDelegate.Broadcast(/** Actor */nullptr);
}

void FAvaEffectorsSceneTreeResolver::OnOutlinerHierarchyChanged(AActor* InActor, const AActor* InParent, EAvaOutlinerHierarchyChangeType InChange)
{
	OnHierarchyChangedDelegate.Broadcast(InActor);
}
#endif
