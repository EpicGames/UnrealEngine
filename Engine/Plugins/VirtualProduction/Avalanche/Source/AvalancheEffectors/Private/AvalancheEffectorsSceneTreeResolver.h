// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cloner/Attachments/CEClonerSceneTreeCustomResolver.h"
#include "Containers/Array.h"
#include "UObject/WeakObjectPtr.h"

class AActor;
enum class EAvaOutlinerHierarchyChangeType : uint8;

/** This allows cloner to retrieve hierarchy of the Motion Design outliner and react accordingly */
class FAvaEffectorsSceneTreeResolver : public ICEClonerSceneTreeCustomResolver
{
public:
	explicit FAvaEffectorsSceneTreeResolver(ULevel* InLevel);

	//~ Begin ICEClonerSceneTreeCustomResolver
	virtual void Activate() override;
	virtual void Deactivate() override;
	virtual bool GetDirectChildrenActor(AActor* InActor, TArray<AActor*>& OutActors) const override;
	virtual FOnActorHierarchyChanged::RegistrationType& OnActorHierarchyChanged() override;
	//~ End ICEClonerSceneTreeCustomResolver

private:
#if WITH_EDITOR
	void OnOutlinerLoaded();
	void OnOutlinerHierarchyChanged(AActor* InActor, const AActor* InParent, EAvaOutlinerHierarchyChangeType InChange);
#endif

	FOnActorHierarchyChanged OnHierarchyChangedDelegate;
	TWeakObjectPtr<ULevel> LevelWeak;
};