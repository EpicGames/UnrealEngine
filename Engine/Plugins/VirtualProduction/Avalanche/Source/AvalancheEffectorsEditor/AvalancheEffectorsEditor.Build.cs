// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheEffectorsEditor : ModuleRules
{
	public AvalancheEffectorsEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateIncludePathModuleNames.AddRange(
			new string[]
			{
				"AvalancheShapesEditor",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AvalancheComponentVisualizers",
				"AvalancheCore",
				"AvalancheEffectors",
				"AvalancheInteractiveTools",
				"AvalancheOutliner",
				"AvalancheShapes",
				"ClonerEffector",
				"ClonerEffectorEditor",
				"ComponentVisualizers",
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"InteractiveToolsFramework",
				"Projects",
				"Slate",
				"SlateCore",
				"UnrealEd",
			}
		);
	}
}
