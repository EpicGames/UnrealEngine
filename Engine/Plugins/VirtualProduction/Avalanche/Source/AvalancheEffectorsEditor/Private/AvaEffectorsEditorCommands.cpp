// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEffectorsEditorCommands.h"
#include "AvaEffectorsEditorStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Utilities/CEClonerLibrary.h"
#include "Utilities/CEEffectorLibrary.h"

#define LOCTEXT_NAMESPACE "AvaEffectorsEditorCommands"

FAvaEffectorsEditorCommands::FAvaEffectorsEditorCommands()
	: TCommands<FAvaEffectorsEditorCommands>(
		TEXT("AvaEffectorsEditor")
		, LOCTEXT("MotionDesignEffectorsEditor", "Motion Design Effects Editor")
		, NAME_None
		, FAvaEffectorsEditorStyle::Get().GetStyleSetName()
	)
{
}

void FAvaEffectorsEditorCommands::RegisterCommands()
{
	TSet<FName> TypeNames;
	UCEEffectorLibrary::GetEffectorTypeNames(TypeNames);

	for (FName TypeName : TypeNames)
	{
		TSharedPtr<FUICommandInfo>& TypeCommand = Tool_Actor_Effectors.FindOrAdd(TypeName);

		FUICommandInfo::MakeCommandInfo(
			AsShared(),
			TypeCommand,
			TypeName,
			FText::FromString(FName::NameToDisplayString(TypeName.ToString(), /** IsBool */false)),
			FText::Format(LOCTEXT("EffectorCommandTooltip", "Create a {0} Effector Actor in the viewport."), FText::FromName(TypeName)),
			FSlateIconFinder::FindIcon(TEXT("AvaEffectorsEditor.Tool_Actor_Effector")),
			EUserInterfaceActionType::ToggleButton,
			FInputChord()
		);
	}

	TSet<FName> LayoutNames;
	UCEClonerLibrary::GetClonerLayoutNames(LayoutNames);

	for (FName LayoutName : LayoutNames)
	{
		TSharedPtr<FUICommandInfo>& LayoutCommand = Tool_Actor_Cloners.FindOrAdd(LayoutName);

		FUICommandInfo::MakeCommandInfo(
			AsShared(),
			LayoutCommand,
			LayoutName,
			FText::FromString(FName::NameToDisplayString(LayoutName.ToString(), /** IsBool */false)),
			FText::Format(LOCTEXT("ClonerCommandTooltip", "Create a {0} Cloner Actor in the viewport."), FText::FromName(LayoutName)),
			FSlateIconFinder::FindIcon(TEXT("AvaEffectorsEditor.Tool_Actor_Cloner")),
			EUserInterfaceActionType::ToggleButton,
			FInputChord()
		);
	}
}

#undef LOCTEXT_NAMESPACE
