// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FAvaEffectorsEditorCommands : public TCommands<FAvaEffectorsEditorCommands>
{
public:
	FAvaEffectorsEditorCommands();

	//~ Begin TCommands
	virtual void RegisterCommands() override;
	//~ End TCommands

	TMap<FName, TSharedPtr<FUICommandInfo>> Tool_Actor_Cloners;
	TMap<FName, TSharedPtr<FUICommandInfo>> Tool_Actor_Effectors;
};
