// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEffectorsEditorStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Math/MathFwd.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleRegistry.h"

FAvaEffectorsEditorStyle::FAvaEffectorsEditorStyle()
	: FSlateStyleSet(TEXT("AvaEffectorsEditor"))
{
	const FVector2f Icon16x16(16.0f, 16.0f);
	
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());
	
	SetContentRoot(Plugin->GetBaseDir() / TEXT("Resources"));

	Set("AvaEffectorsEditor.Tool_Actor_Effector", new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/effector", Icon16x16));
	Set("AvaEffectorsEditor.Tool_Actor_Cloner", new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/cloner", Icon16x16));
	
	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FAvaEffectorsEditorStyle::~FAvaEffectorsEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
