// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FAvaEffectorsEditorStyle final : public FSlateStyleSet
{
public:
	static FAvaEffectorsEditorStyle& Get()
	{
		static FAvaEffectorsEditorStyle Instance;
		return Instance;
	}

	FAvaEffectorsEditorStyle();
	virtual ~FAvaEffectorsEditorStyle() override;
};
