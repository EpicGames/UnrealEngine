// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvalancheEffectorsEditorModule.h"
#include "AvaEffectorsEditorCommands.h"
#include "AvaEffectorsEditorStyle.h"
#include "Cloner/AvaClonerActorVis.h"
#include "Cloner/AvaClonerEditorOutlinerContextMenu.h"
#include "Cloner/CEClonerComponent.h"
#include "ComponentVisualizers.h"
#include "Effector/AvaEffectorActorVis.h"
#include "Effector/AvaEffectorEditorOutlinerContextMenu.h"
#include "Effector/CEEffectorComponent.h"
#include "Framework/Application/SlateApplication.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "IAvaOutlinerModule.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

void FAvalancheEffectorsEditorModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FAvalancheEffectorsEditorModule::PostEngineInit);

	RegisterOutlinerItems();
}

void FAvalancheEffectorsEditorModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	FAvaEffectorsEditorCommands::Unregister();

	UnregisterOutlinerItems();
}

void FAvalancheEffectorsEditorModule::PostEngineInit()
{
	if (FSlateApplication::IsInitialized())
	{
		RegisterComponentVisualizers();
	}
}

void FAvalancheEffectorsEditorModule::RegisterComponentVisualizers()
{
	IAvalancheComponentVisualizersModule::RegisterComponentVisualizer<UCEEffectorComponent, FAvaEffectorActorVisualizer>(&Visualizers);
	IAvalancheComponentVisualizersModule::RegisterComponentVisualizer<UCEClonerComponent, FAvaClonerActorVisualizer>(&Visualizers);
}

void FAvalancheEffectorsEditorModule::RegisterOutlinerItems()
{
	IAvaOutlinerModule& OutlinerModule = IAvaOutlinerModule::Get();

	OutlinerContextClonerDelegateHandle = OutlinerModule.GetOnExtendOutlinerItemContextMenu()
		.AddStatic(&FAvaClonerEditorOutlinerContextMenu::OnExtendOutlinerContextMenu);

	OutlinerContextEffectorDelegateHandle = OutlinerModule.GetOnExtendOutlinerItemContextMenu()
		.AddStatic(&FAvaEffectorEditorOutlinerContextMenu::OnExtendOutlinerContextMenu);
}

void FAvalancheEffectorsEditorModule::UnregisterOutlinerItems()
{
	if (IAvaOutlinerModule::IsLoaded())
	{
		IAvaOutlinerModule& OutlinerModule = IAvaOutlinerModule::Get();

		OutlinerModule.GetOnExtendOutlinerItemContextMenu().Remove(OutlinerContextClonerDelegateHandle);
        OutlinerContextClonerDelegateHandle.Reset();

		OutlinerModule.GetOnExtendOutlinerItemContextMenu().Remove(OutlinerContextEffectorDelegateHandle);
		OutlinerContextEffectorDelegateHandle.Reset();
	}
}

IMPLEMENT_MODULE(FAvalancheEffectorsEditorModule, AvalancheEffectorsEditor)
