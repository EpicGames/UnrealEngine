// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointerFwd.h"

class FComponentVisualizer;
class IAvalancheInteractiveToolsModule;

class FAvalancheEffectorsEditorModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

private:
	void PostEngineInit();

	void RegisterComponentVisualizers();

	void RegisterOutlinerItems();
	void UnregisterOutlinerItems();

	TArray<TSharedPtr<FComponentVisualizer>> Visualizers;

	FDelegateHandle OutlinerContextClonerDelegateHandle;
	FDelegateHandle OutlinerContextEffectorDelegateHandle;
};
