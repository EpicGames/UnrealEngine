// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaClonerActorTool.h"
#include "AvaEffectorsEditorCommands.h"
#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "Cloner/CEClonerActor.h"
#include "Cloner/CEClonerActorFactory.h"

UAvaClonerActorTool::UAvaClonerActorTool()
{
	ActorClass = ACEClonerActor::StaticClass();
}

void UAvaClonerActorTool::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	// Register commands here, subsystem is init
	FAvaEffectorsEditorCommands::Register();

	const FString ToolIdentifier = TEXT("Clone Actor Tool ");
	for (const TPair<FName, TSharedPtr<FUICommandInfo>>& ClonerCommandPair : FAvaEffectorsEditorCommands::Get().Tool_Actor_Cloners)
	{
		UCEClonerActorFactory* ClonerActorFactory = CreateActorFactory<UCEClonerActorFactory>();
		ClonerActorFactory->SetClonerLayout(ClonerCommandPair.Key);
		
		FAvaInteractiveToolsToolParameters ToolParameters = CreateDefaultToolParameters();
		ToolParameters.UICommand = ClonerCommandPair.Value;
		ToolParameters.ToolIdentifier = ToolIdentifier + ClonerCommandPair.Key.ToString();
		ToolParameters.Priority = 3000;
		ToolParameters.ActorFactory.Set<TObjectPtr<UActorFactory>>(ClonerActorFactory);

		InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryNameCloner, MoveTemp(ToolParameters));
	}
}
