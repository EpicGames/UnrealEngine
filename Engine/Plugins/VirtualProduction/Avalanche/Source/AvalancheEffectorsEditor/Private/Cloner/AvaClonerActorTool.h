// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaInteractiveToolsActorPointToolBase.h"
#include "AvaClonerActorTool.generated.h"

UCLASS()
class UAvaClonerActorTool : public UAvaInteractiveToolsActorPointToolBase
{
	GENERATED_BODY()

public:
	UAvaClonerActorTool();

	//~ Begin UAvaInteractiveToolsToolBase
	virtual void OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule) override;
	//~ End UAvaInteractiveToolsToolBase
};
