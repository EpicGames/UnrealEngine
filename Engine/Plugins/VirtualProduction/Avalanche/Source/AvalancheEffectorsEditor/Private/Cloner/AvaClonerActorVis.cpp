// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/AvaClonerActorVis.h"
#include "AvaField.h"
#include "AvaShapeSprites.h"
#include "Cloner/CEClonerActor.h"
#include "Cloner/CEClonerComponent.h"
#include "Cloner/Layouts/CEClonerCircleLayout.h"
#include "Cloner/Layouts/CEClonerCylinderLayout.h"
#include "Cloner/Layouts/CEClonerGridLayout.h"
#include "Cloner/Layouts/CEClonerHoneycombLayout.h"
#include "Cloner/Layouts/CEClonerLineLayout.h"
#include "Cloner/Layouts/CEClonerSphereUniformLayout.h"
#include "EditorViewportClient.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "ScopedTransaction.h"
#include "TextureResource.h"

IMPLEMENT_HIT_PROXY(HAvaClonerActorSpacingHitProxy, HAvaHitProxy);

#define LOCTEXT_NAMESPACE "AvaClonerActorVisualizer"

FAvaClonerActorVisualizer::FAvaClonerActorVisualizer()
	: FAvaVisualizerBase()
{
}

void FAvaClonerActorVisualizer::StoreInitialValues()
{
	Super::StoreInitialValues();

	if (GetEditedComponent() == nullptr)
	{
		return;
	}

	const UCEClonerComponent* ClonerComponent = ClonerComponentWeak.Get();

	if (!ClonerComponent)
	{
		return;
	}

	if (const UCEClonerGridLayout* GridLayout = ClonerComponent->GetActiveLayout<UCEClonerGridLayout>())
	{
		InitialSpacing = FVector(GridLayout->GetSpacingX(), GridLayout->GetSpacingY(), GridLayout->GetSpacingZ());
	}
	else if (const UCEClonerLineLayout* LineLayout = ClonerComponent->GetActiveLayout<UCEClonerLineLayout>())
	{
		InitialSpacing = FVector(LineLayout->GetSpacing());
	}
	else if (const UCEClonerHoneycombLayout* HoneycombLayout = ClonerComponent->GetActiveLayout<UCEClonerHoneycombLayout>())
	{
		const ECEClonerPlane Plane = HoneycombLayout->GetPlane();

		FVector Spacing = FVector::ZeroVector;
		if (Plane == ECEClonerPlane::XY)
		{
			Spacing = FVector(HoneycombLayout->GetWidthSpacing(), HoneycombLayout->GetHeightSpacing(), 0);
		}
		else if (Plane == ECEClonerPlane::YZ)
		{
			Spacing = FVector(0, HoneycombLayout->GetWidthSpacing(), HoneycombLayout->GetHeightSpacing());
		}
		else if (Plane == ECEClonerPlane::XZ)
		{
			Spacing = FVector(HoneycombLayout->GetWidthSpacing(), 0, HoneycombLayout->GetHeightSpacing());
		}

		InitialSpacing = Spacing;
	}
	else if (const UCEClonerCircleLayout* CircleLayout = ClonerComponent->GetActiveLayout<UCEClonerCircleLayout>())
	{
		InitialSpacing = FVector(CircleLayout->GetRadius());
	}
	else if (const UCEClonerCylinderLayout* CylinderLayout = ClonerComponent->GetActiveLayout<UCEClonerCylinderLayout>())
	{
		InitialSpacing = FVector(0, CylinderLayout->GetRadius(), CylinderLayout->GetHeight());
	}
	else if (const UCEClonerSphereUniformLayout* SphereLayout = ClonerComponent->GetActiveLayout<UCEClonerSphereUniformLayout>())
	{
		InitialSpacing = FVector(0, SphereLayout->GetRadius(), 0);
	}
}

FBox FAvaClonerActorVisualizer::GetComponentBounds(const UActorComponent* InComponent) const
{
	if (const UCEClonerComponent* ClonerComponent = Cast<UCEClonerComponent>(InComponent))
	{
		if (const AActor* ClonerActor = ClonerComponent->GetOwner())
		{
			FVector Origin;
			FVector Extent;
			ClonerActor->GetActorBounds(false, Origin, Extent);
			return FBox(-Extent, Extent);
		}
	}

	return Super::GetComponentBounds(InComponent);
}

bool FAvaClonerActorVisualizer::HandleInputDeltaInternal(FEditorViewportClient* InViewportClient, FViewport* InViewport, const FVector& InAccumulatedTranslation, const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale)
{
	if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
	{
		return false;
	}

	const UCEClonerComponent* ClonerComponent = ClonerComponentWeak.Get();

	if (!ClonerComponent)
	{
		EndEditing();
	}
	else if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
	{
		const EAxisList::Type AxisList = GetViewportWidgetAxisList(InViewportClient);

		if (UCEClonerGridLayout* GridLayout = ClonerComponent->GetActiveLayout<UCEClonerGridLayout>())
		{
			if (AxisList & EAxisList::X)
			{
				ModifyProperty(
					GridLayout
					, GET_MEMBER_NAME_CHECKED(UCEClonerGridLayout, SpacingX)
					, EPropertyChangeType::Interactive
					, [this, &InAccumulatedTranslation, &GridLayout]()
				{
					const float SpacingX = InitialSpacing.X + InAccumulatedTranslation.X / FMath::Max(1, GridLayout->GetCountX() / 2.f);
					GridLayout->SetSpacingX(SpacingX);
				});
			}
			else if (AxisList & EAxisList::Y)
			{
				ModifyProperty(
					GridLayout
					, GET_MEMBER_NAME_CHECKED(UCEClonerGridLayout, SpacingY)
					, EPropertyChangeType::Interactive
					, [this, &InAccumulatedTranslation, &GridLayout]()
				{
					const float SpacingY = InitialSpacing.Y + InAccumulatedTranslation.Y / FMath::Max(1, GridLayout->GetCountY() / 2.f);
					GridLayout->SetSpacingY(SpacingY);
				});
			}
			else if (AxisList & EAxisList::Z)
			{
				ModifyProperty(
					GridLayout
					, GET_MEMBER_NAME_CHECKED(UCEClonerGridLayout, SpacingZ)
					, EPropertyChangeType::Interactive
					, [this, &InAccumulatedTranslation, &GridLayout]()
				{
					const float SpacingZ = InitialSpacing.Z + InAccumulatedTranslation.Z / FMath::Max(1, GridLayout->GetCountZ() / 2.f);
					GridLayout->SetSpacingZ(SpacingZ);
				});
			}

			return true;
		}
		else if (UCEClonerLineLayout* LineLayout = ClonerComponent->GetActiveLayout<UCEClonerLineLayout>())
		{
			const int32 Count = LineLayout->GetCount();
			float Spacing = LineLayout->GetSpacing();

			if (AxisList & EAxisList::X)
			{
				Spacing = InitialSpacing.X + InAccumulatedTranslation.X / FMath::Max(1, Count);
			}
			if (AxisList & EAxisList::Y)
			{
				Spacing = InitialSpacing.Y + InAccumulatedTranslation.Y / FMath::Max(1, Count);
			}
			if (AxisList & EAxisList::Z)
			{
				Spacing = InitialSpacing.Z + InAccumulatedTranslation.Z / FMath::Max(1, Count);
			}

			ModifyProperty(
				LineLayout
				, GET_MEMBER_NAME_CHECKED(UCEClonerLineLayout, Spacing)
				, EPropertyChangeType::Interactive
				, [this, &Spacing, &LineLayout]()
			{
				LineLayout->SetSpacing(Spacing);
			});

			return true;
		}
		else if (UCEClonerHoneycombLayout* HoneycombLayout = ClonerComponent->GetActiveLayout<UCEClonerHoneycombLayout>())
		{
			const ECEClonerPlane Plane = HoneycombLayout->GetPlane();
			const int32 WidthCount = HoneycombLayout->GetWidthCount();
			const int32 HeightCount = HoneycombLayout->GetHeightCount();
			float WidthSpacing = HoneycombLayout->GetWidthSpacing();
			float HeightSpacing = HoneycombLayout->GetHeightSpacing();

			if (Plane == ECEClonerPlane::XY)
			{
				WidthSpacing = InitialSpacing.X + InAccumulatedTranslation.X / FMath::Max(1, WidthCount / 2.f);
				HeightSpacing = InitialSpacing.Y + InAccumulatedTranslation.Y / FMath::Max(1, HeightCount / 2.f);
			}
			else if (Plane == ECEClonerPlane::YZ)
			{
				WidthSpacing = InitialSpacing.Y + InAccumulatedTranslation.Y / FMath::Max(1, WidthCount / 2.f);
				HeightSpacing = InitialSpacing.Z + InAccumulatedTranslation.Z / FMath::Max(1, HeightCount / 2.f);
			}
			else if (Plane == ECEClonerPlane::XZ)
			{
				WidthSpacing = InitialSpacing.X + InAccumulatedTranslation.X / FMath::Max(1, WidthCount / 2.f);
				HeightSpacing = InitialSpacing.Z + InAccumulatedTranslation.Z / FMath::Max(1, HeightCount / 2.f);
			}

			ModifyProperty(
				HoneycombLayout
				, GET_MEMBER_NAME_CHECKED(UCEClonerHoneycombLayout, WidthSpacing)
				, EPropertyChangeType::Interactive
				, [this, &WidthSpacing, &HoneycombLayout]()
			{
				HoneycombLayout->SetWidthSpacing(WidthSpacing);
			});

			ModifyProperty(
				HoneycombLayout
				, GET_MEMBER_NAME_CHECKED(UCEClonerHoneycombLayout, HeightSpacing)
				, EPropertyChangeType::Interactive
				, [this, &HeightSpacing, &HoneycombLayout]()
			{
				HoneycombLayout->SetHeightSpacing(HeightSpacing);
			});

			return true;
		}
		else if (UCEClonerCircleLayout* CircleLayout = ClonerComponent->GetActiveLayout<UCEClonerCircleLayout>())
		{
			float Radius = CircleLayout->GetRadius();

			if (AxisList & EAxisList::X)
			{
				Radius = InitialSpacing.X + InAccumulatedTranslation.X;
			}
			else if (AxisList & EAxisList::Y)
			{
				Radius = InitialSpacing.Y + InAccumulatedTranslation.Y;
			}
			else if (AxisList & EAxisList::Z)
			{
				Radius = InitialSpacing.Z + InAccumulatedTranslation.Z;
			}

			ModifyProperty(
				CircleLayout
				, GET_MEMBER_NAME_CHECKED(UCEClonerCircleLayout, Radius)
				, EPropertyChangeType::Interactive
				, [this, &Radius, &CircleLayout]()
			{
				CircleLayout->SetRadius(Radius);
			});

			return true;
		}
		else if (UCEClonerCylinderLayout* CylinderLayout = ClonerComponent->GetActiveLayout<UCEClonerCylinderLayout>())
		{
			const ECEClonerPlane Plane = CylinderLayout->GetPlane();
			float Radius = CylinderLayout->GetRadius();
			float Height = CylinderLayout->GetHeight();

			if (Plane == ECEClonerPlane::XY)
			{
				if (AxisList & EAxisList::Y)
				{
					Radius = InitialSpacing.Y + InAccumulatedTranslation.Y;
				}
				else
				{
					Height = InitialSpacing.Z + InAccumulatedTranslation.Z;
				}
			}
			else if (Plane == ECEClonerPlane::YZ)
			{
				if (AxisList & EAxisList::Y)
				{
					Radius = InitialSpacing.Y + InAccumulatedTranslation.Y;
				}
				else
				{
					Height = InitialSpacing.Z + InAccumulatedTranslation.X;
				}
			}
			else if (Plane == ECEClonerPlane::XZ)
			{
				if (AxisList & EAxisList::Z)
				{
					Radius = InitialSpacing.Y + InAccumulatedTranslation.Z;
				}
				else
				{
					Height = InitialSpacing.Z + InAccumulatedTranslation.Y;
				}
			}

			ModifyProperty(
				CylinderLayout
				, GET_MEMBER_NAME_CHECKED(UCEClonerCylinderLayout, Radius)
				, EPropertyChangeType::Interactive
				, [this, &Radius, &CylinderLayout]()
			{
				CylinderLayout->SetRadius(Radius);
			});

			ModifyProperty(
				CylinderLayout
				, GET_MEMBER_NAME_CHECKED(UCEClonerCylinderLayout, Height)
				, EPropertyChangeType::Interactive
				, [this, &Height, &CylinderLayout]()
			{
				CylinderLayout->SetHeight(Height);
			});

			return true;
		}
		else if (UCEClonerSphereUniformLayout* SphereLayout = ClonerComponent->GetActiveLayout<UCEClonerSphereUniformLayout>())
		{
			float Radius = SphereLayout->GetRadius();

			if (AxisList & EAxisList::Y)
			{
				Radius = InitialSpacing.Y + InAccumulatedTranslation.Y;
			}

			ModifyProperty(
				SphereLayout
				, GET_MEMBER_NAME_CHECKED(UCEClonerSphereUniformLayout, Radius)
				, EPropertyChangeType::Interactive
				, [this, &Radius, &SphereLayout]()
			{
				SphereLayout->SetRadius(Radius);
			});

			return true;
		}
	}

	return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
}

void FAvaClonerActorVisualizer::DrawVisualizationEditing(const UActorComponent* InComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationEditing(InComponent, InView, InPDI, InOutIconIndex);

	const UCEClonerComponent* ClonerComponent = Cast<UCEClonerComponent>(InComponent);

	if (!ClonerComponent || !ClonerComponent->GetEnabled() || ClonerComponent->GetMeshCount() == 0)
	{
		return;
	}

	if (const UCEClonerGridLayout* GridLayout = ClonerComponent->GetActiveLayout<UCEClonerGridLayout>())
	{
		if (GridLayout->GetCountX() > 0)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::X, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}

		if (GridLayout->GetCountY() > 0)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Y, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}

		if (GridLayout->GetCountZ() > 0)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Z, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
	}
	else if (const UCEClonerLineLayout* LineLayout = ClonerComponent->GetActiveLayout<UCEClonerLineLayout>())
	{
		if (LineLayout->GetCount() > 0)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, LineLayout->GetAxis(), FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
	}
	else if (const UCEClonerHoneycombLayout* HoneycombLayout = ClonerComponent->GetActiveLayout<UCEClonerHoneycombLayout>())
	{
		const ECEClonerPlane Plane = HoneycombLayout->GetPlane();

		if (Plane == ECEClonerPlane::XY)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::X, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Y, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
		else if (Plane == ECEClonerPlane::YZ)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Y, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Z, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
		else if (Plane == ECEClonerPlane::XZ)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::X, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Z, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
	}
	else if (const UCEClonerCircleLayout* CircleLayout = ClonerComponent->GetActiveLayout<UCEClonerCircleLayout>())
	{
		const ECEClonerPlane Plane = CircleLayout->GetPlane();

		if (Plane == ECEClonerPlane::XY)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Y, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
		else if (Plane == ECEClonerPlane::YZ)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Y, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
		else if (Plane == ECEClonerPlane::XZ)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Z, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
	}
	else if (const UCEClonerCylinderLayout* CylinderLayout = ClonerComponent->GetActiveLayout<UCEClonerCylinderLayout>())
	{
		const ECEClonerPlane Plane = CylinderLayout->GetPlane();

		if (Plane == ECEClonerPlane::XY)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Y, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Z, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
		else if (Plane == ECEClonerPlane::YZ)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Y, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::X, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
		else if (Plane == ECEClonerPlane::XZ)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Z, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Y, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
	}
	else if (ClonerComponent->IsActiveLayout<UCEClonerSphereUniformLayout>())
	{
		DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Y, FAvaVisualizerBase::Inactive);
		InOutIconIndex++;
	}
}

void FAvaClonerActorVisualizer::DrawVisualizationNotEditing(const UActorComponent* InComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationNotEditing(InComponent, InView, InPDI, InOutIconIndex);

	const UCEClonerComponent* ClonerComponent = Cast<UCEClonerComponent>(InComponent);

	if (!ClonerComponent || !ClonerComponent->GetEnabled() || ClonerComponent->GetMeshCount() == 0)
	{
		return;
	}

	if (const UCEClonerGridLayout* GridLayout = ClonerComponent->GetActiveLayout<UCEClonerGridLayout>())
	{
		if (GridLayout->GetCountX() > 0)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::X, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}

		if (GridLayout->GetCountY() > 0)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Y, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}

		if (GridLayout->GetCountZ() > 0)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Z, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
	}
	else if (const UCEClonerLineLayout* LineLayout = ClonerComponent->GetActiveLayout<UCEClonerLineLayout>())
	{
		if (LineLayout->GetCount() > 0)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, LineLayout->GetAxis(), FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
	}
	else if (const UCEClonerHoneycombLayout* HoneycombLayout = ClonerComponent->GetActiveLayout<UCEClonerHoneycombLayout>())
	{
		const ECEClonerPlane Plane = HoneycombLayout->GetPlane();

		if (Plane == ECEClonerPlane::XY)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::X, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Y, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
		else if (Plane == ECEClonerPlane::YZ)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Y, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Z, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
		else if (Plane == ECEClonerPlane::XZ)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::X, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Z, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
	}
	else if (const UCEClonerCircleLayout* CircleLayout = ClonerComponent->GetActiveLayout<UCEClonerCircleLayout>())
	{
		const ECEClonerPlane Plane = CircleLayout->GetPlane();

		if (Plane == ECEClonerPlane::XY)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Y, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
		else if (Plane == ECEClonerPlane::YZ)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Y, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
		else if (Plane == ECEClonerPlane::XZ)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Z, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
		else
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Custom, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
	}
	else if (const UCEClonerCylinderLayout* CylinderLayout = ClonerComponent->GetActiveLayout<UCEClonerCylinderLayout>())
	{
		const ECEClonerPlane Plane = CylinderLayout->GetPlane();

		if (Plane == ECEClonerPlane::XY)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Y, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Z, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
		else if (Plane == ECEClonerPlane::YZ)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Y, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::X, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
		else if (Plane == ECEClonerPlane::XZ)
		{
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Z, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
			DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Y, FAvaVisualizerBase::Inactive);
			InOutIconIndex++;
		}
	}
	else if (ClonerComponent->IsActiveLayout<UCEClonerSphereUniformLayout>())
	{
		DrawSpacingButton(ClonerComponent, InView, InPDI, InOutIconIndex, ECEClonerAxis::Y, FAvaVisualizerBase::Inactive);
		InOutIconIndex++;
	}
}

FVector FAvaClonerActorVisualizer::GetHandleSpacingLocation(const UCEClonerComponent* InClonerComponent, ECEClonerAxis InAxis) const
{
	const FRotator ClonerRotation = InClonerComponent->GetComponentRotation();
	const FVector ClonerScale = InClonerComponent->GetComponentScale();
	FVector OutLocation = FVector::ZeroVector;
	FVector Axis = InAxis == ECEClonerAxis::X ? FVector::XAxisVector : (InAxis == ECEClonerAxis::Z ? FVector::ZAxisVector : FVector::YAxisVector);

	if (const UCEClonerGridLayout* GridLayout = InClonerComponent->GetActiveLayout<UCEClonerGridLayout>())
	{
		const FVector Spacing(GridLayout->GetSpacingX(), GridLayout->GetSpacingY(), GridLayout->GetSpacingZ());
		const FVector Count(GridLayout->GetCountX(), GridLayout->GetCountY(), GridLayout->GetCountZ());

		OutLocation = ClonerRotation.RotateVector(Axis * Spacing * Count / 2);
	}
	else if (const UCEClonerLineLayout* LineLayout = InClonerComponent->GetActiveLayout<UCEClonerLineLayout>())
	{
		if (InAxis == ECEClonerAxis::Custom)
		{
			Axis = LineLayout->GetDirection().GetSafeNormal();
		}

		OutLocation = ClonerRotation.RotateVector(Axis) * LineLayout->GetSpacing() * FVector(LineLayout->GetCount());
	}
	else if (const UCEClonerHoneycombLayout* HoneycombLayout = InClonerComponent->GetActiveLayout<UCEClonerHoneycombLayout>())
	{
		const ECEClonerPlane Plane = HoneycombLayout->GetPlane();
		const float WidthSpacing = HoneycombLayout->GetWidthSpacing();
		const float HeightSpacing = HoneycombLayout->GetHeightSpacing();
		const int32 WidthCount = HoneycombLayout->GetWidthCount();
		const int32 HeightCount = HoneycombLayout->GetHeightCount();

		FVector Spacing;
		FVector Count;
		if (Plane == ECEClonerPlane::XY)
		{
			Spacing = FVector(WidthSpacing, HeightSpacing, 0);
			Count = FVector(WidthCount, HeightCount, 0);
		}
		else if (Plane == ECEClonerPlane::YZ)
		{
			Spacing = FVector(0, WidthSpacing, HeightSpacing);
			Count = FVector(0, WidthCount, HeightCount);
		}
		else if (Plane == ECEClonerPlane::XZ)
		{
			Spacing = FVector(WidthSpacing, 0, HeightSpacing);
			Count = FVector(WidthCount, 0, HeightCount);
		}

		OutLocation = ClonerRotation.RotateVector(Axis * Spacing * Count/2);
	}
	else if (const UCEClonerCircleLayout* CircleLayout = InClonerComponent->GetActiveLayout<UCEClonerCircleLayout>())
	{
		FRotator Rotation = CircleLayout->GetRotation();
		const FVector Scale = CircleLayout->GetScale();
		const ECEClonerPlane Plane = CircleLayout->GetPlane();
		const float Radius = CircleLayout->GetRadius();

		if (Plane == ECEClonerPlane::XY)
		{
			Rotation = FRotator::ZeroRotator;
		}
		else if (Plane == ECEClonerPlane::YZ)
		{
			Rotation = FRotator(90, 0, 0);
		}
		else if (Plane == ECEClonerPlane::XZ)
		{
			Rotation = FRotator(0, 90, 0);
		}

		OutLocation = ClonerRotation.RotateVector(Rotation.RotateVector(Axis * Scale)) * Radius;
	}
	else if (const UCEClonerCylinderLayout* CylinderLayout = InClonerComponent->GetActiveLayout<UCEClonerCylinderLayout>())
	{
		FRotator Rotation = CylinderLayout->GetRotation();
		const FVector Scale = CylinderLayout->GetScale();
		const ECEClonerPlane Plane = CylinderLayout->GetPlane();
		const float Radius = CylinderLayout->GetRadius();
		const float Height = CylinderLayout->GetHeight();
		float Dim = 0.f;

		if (Plane == ECEClonerPlane::XY)
		{
			Rotation = FRotator::ZeroRotator;
			if (InAxis == ECEClonerAxis::Y)
			{
				Dim = Radius;
			}
			else
			{
				Dim = Height / 2;
			}
		}
		else if (Plane == ECEClonerPlane::YZ)
		{
			if (InAxis == ECEClonerAxis::Y)
			{
				Rotation = FRotator(90, 0, 0);
				Dim = Radius;
			}
			else
			{
				Rotation = FRotator(0, 0, 90);
				Dim = Height / 2;
			}
		}
		else if (Plane == ECEClonerPlane::XZ)
		{
			if (InAxis == ECEClonerAxis::Z)
			{
				Rotation = FRotator(0, 90, 0);
				Dim = Radius;
			}
			else
			{
				Rotation = FRotator(90, 0, 0);
				Dim = Height / 2;
			}
		}

		OutLocation = ClonerRotation.RotateVector(Rotation.RotateVector(Axis * Scale)) * Dim;
	}
	else if (const UCEClonerSphereUniformLayout* SphereLayout = InClonerComponent->GetActiveLayout<UCEClonerSphereUniformLayout>())
	{
		const float Radius = SphereLayout->GetRadius();
		const FVector Scale = SphereLayout->GetScale();

		OutLocation = ClonerRotation.RotateVector(Axis) * Scale * Radius;
	}

	return InClonerComponent->GetComponentLocation() + OutLocation * ClonerScale;
}

void FAvaClonerActorVisualizer::DrawSpacingButton(const UCEClonerComponent* InClonerComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32 InIconIndex, ECEClonerAxis InAxis, FLinearColor InColor) const
{
	UTexture2D* SpacingSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::BevelSprite);

	if (!SpacingSprite || !SpacingSprite->GetResource() || !InClonerComponent->GetEnabled() || InClonerComponent->GetMeshCount() == 0)
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	IconLocation = GetHandleSpacingLocation(InClonerComponent, InAxis);

	InPDI->SetHitProxy(new HAvaClonerActorSpacingHitProxy(InClonerComponent, InAxis));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, SpacingSprite->GetResource(), InColor, SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

UActorComponent* FAvaClonerActorVisualizer::GetEditedComponent() const
{
	return GetClonerComponent();
}

TMap<UObject*, TArray<FProperty*>> FAvaClonerActorVisualizer::GatherEditableProperties(UObject* InObject) const
{
	if (const UCEClonerComponent* ClonerComponent = Cast<UCEClonerComponent>(InObject))
	{
		if (UCEClonerLayoutBase* Layout = ClonerComponent->GetActiveLayout())
		{
			TArray<FName> PropertyNames;

			if (UCEClonerGridLayout* GridLayout = Cast<UCEClonerGridLayout>(Layout))
			{
				PropertyNames = {
					GET_MEMBER_NAME_CHECKED(UCEClonerGridLayout, SpacingX),
					GET_MEMBER_NAME_CHECKED(UCEClonerGridLayout, SpacingY),
					GET_MEMBER_NAME_CHECKED(UCEClonerGridLayout, SpacingZ)
				};
			}
			else if (UCEClonerLineLayout* LineLayout = Cast<UCEClonerLineLayout>(Layout))
			{
				PropertyNames = {GET_MEMBER_NAME_CHECKED(UCEClonerLineLayout, Spacing)};
			}
			else if (UCEClonerHoneycombLayout* HoneycombLayout = Cast<UCEClonerHoneycombLayout>(Layout))
			{
				PropertyNames = {
					GET_MEMBER_NAME_CHECKED(UCEClonerHoneycombLayout, WidthSpacing),
					GET_MEMBER_NAME_CHECKED(UCEClonerHoneycombLayout, HeightSpacing)
				};
			}
			else if (UCEClonerCircleLayout* CircleLayout = Cast<UCEClonerCircleLayout>(Layout))
			{
				PropertyNames = {GET_MEMBER_NAME_CHECKED(UCEClonerCircleLayout, Radius)};
			}
			else if (UCEClonerCylinderLayout* CylinderLayout = Cast<UCEClonerCylinderLayout>(Layout))
			{
				PropertyNames = {
					GET_MEMBER_NAME_CHECKED(UCEClonerCylinderLayout, Radius),
					GET_MEMBER_NAME_CHECKED(UCEClonerCylinderLayout, Height)
				};
			}

			TArray<FProperty*> Properties;
			Properties.Reserve(PropertyNames.Num());

			UClass* LayoutClass = Layout->GetClass();

			for (FName PropertyName : PropertyNames)
			{
				if (FProperty* Property = LayoutClass->FindPropertyByName(PropertyName))
				{
					Properties.Add(Property);
				}
			}

			return {{Layout, Properties}};
		}
	}

	return {};
}

bool FAvaClonerActorVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* InVisProxy, const FViewportClick& InClick)
{
	if (InClick.GetKey() != EKeys::LeftMouseButton)
	{
		EndEditing();
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	MeshType* DynMesh = Cast<MeshType>(const_cast<UActorComponent*>(InVisProxy->Component.Get()));

	if (DynMesh == nullptr)
	{
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	if (InVisProxy->IsA(HAvaClonerActorSpacingHitProxy::StaticGetType()))
	{
		EndEditing();
		ClonerComponentWeak = DynMesh;
		bEditingSpacing = true;
		EditingAxis = static_cast<HAvaClonerActorSpacingHitProxy*>(InVisProxy)->Axis;
		StartEditing(InViewportClient, DynMesh);

		return true;
	}

	return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
}

bool FAvaClonerActorVisualizer::GetWidgetLocation(const FEditorViewportClient* InViewportClient, FVector& OutLocation) const
{
	if (const UCEClonerComponent* ClonerComponent = ClonerComponentWeak.Get())
	{
		if (bEditingSpacing)
		{
			OutLocation = GetHandleSpacingLocation(ClonerComponent, EditingAxis);
			return true;
		}
	}

	return Super::GetWidgetLocation(InViewportClient, OutLocation);
}

bool FAvaClonerActorVisualizer::GetWidgetMode(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode& OutMode) const
{
	if (bEditingSpacing)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Translate;
		return true;
	}

	return Super::GetWidgetMode(InViewportClient, OutMode);
}

bool FAvaClonerActorVisualizer::GetWidgetAxisList(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingSpacing)
	{
		if (EditingAxis == ECEClonerAxis::X)
		{
			OutAxisList = EAxisList::Type::X;
		}
		else if (EditingAxis == ECEClonerAxis::Y)
		{
			OutAxisList = EAxisList::Type::Y;
		}
		else if (EditingAxis == ECEClonerAxis::Z)
		{
			OutAxisList = EAxisList::Type::Z;
		}
		else if (EditingAxis == ECEClonerAxis::Custom)
		{
			OutAxisList = EAxisList::Type::XYZ;
		}
		return true;
	}

	return Super::GetWidgetAxisList(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaClonerActorVisualizer::GetWidgetAxisListDragOverride(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	return Super::GetWidgetAxisListDragOverride(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaClonerActorVisualizer::ResetValue(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy)
{
	if (!InHitProxy->IsA(HAvaClonerActorSpacingHitProxy::StaticGetType()))
	{
		return Super::ResetValue(InViewportClient, InHitProxy);
	}

	const HAvaClonerActorSpacingHitProxy* ComponentHitProxy = static_cast<HAvaClonerActorSpacingHitProxy*>(InHitProxy);

	if (!ComponentHitProxy->Component.IsValid() || !ComponentHitProxy->Component->IsA<UCEClonerComponent>())
	{
		return Super::ResetValue(InViewportClient, InHitProxy);
	}

	if (const UCEClonerComponent* ClonerComponent = Cast<UCEClonerComponent>(ComponentHitProxy->Component.Get()))
	{
		if (UCEClonerGridLayout* GridLayout = ClonerComponent->GetActiveLayout<UCEClonerGridLayout>())
		{
			FScopedTransaction Transaction(LOCTEXT("VisualizerResetValue", "Visualizer Reset Value"));

			ModifyProperty(
				GridLayout
				, GET_MEMBER_NAME_CHECKED(UCEClonerGridLayout, SpacingX)
				, EPropertyChangeType::ValueSet
				, [this, &GridLayout]()
			{
				GridLayout->SetSpacingX(100.f);
			});

			ModifyProperty(
				GridLayout
				, GET_MEMBER_NAME_CHECKED(UCEClonerGridLayout, SpacingY)
				, EPropertyChangeType::ValueSet
				, [this, &GridLayout]()
			{
				GridLayout->SetSpacingY(100.f);
			});

			ModifyProperty(
				GridLayout
				, GET_MEMBER_NAME_CHECKED(UCEClonerGridLayout, SpacingZ)
				, EPropertyChangeType::ValueSet
				, [this, &GridLayout]()
			{
				GridLayout->SetSpacingZ(100.f);
			});
		}
		else if (UCEClonerLineLayout* LineLayout = ClonerComponent->GetActiveLayout<UCEClonerLineLayout>())
		{
			FScopedTransaction Transaction(LOCTEXT("VisualizerResetValue", "Visualizer Reset Value"));

			ModifyProperty(
				LineLayout
				, GET_MEMBER_NAME_CHECKED(UCEClonerLineLayout, Spacing)
				, EPropertyChangeType::ValueSet
				, [this, &LineLayout]()
			{
				LineLayout->SetSpacing(500.f);
			});
		}
		else if (UCEClonerHoneycombLayout* HoneycombLayout = ClonerComponent->GetActiveLayout<UCEClonerHoneycombLayout>())
		{
			FScopedTransaction Transaction(LOCTEXT( "VisualizerResetValue", "Visualizer Reset Value"));

			ModifyProperty(
				HoneycombLayout
				, GET_MEMBER_NAME_CHECKED(UCEClonerHoneycombLayout, WidthSpacing)
				, EPropertyChangeType::ValueSet
				, [this, &HoneycombLayout]()
			{
				HoneycombLayout->SetWidthSpacing(100.f);
			});

			ModifyProperty(
				HoneycombLayout
				, GET_MEMBER_NAME_CHECKED(UCEClonerHoneycombLayout, HeightSpacing)
				, EPropertyChangeType::ValueSet
				, [this, &HoneycombLayout]()
			{
				HoneycombLayout->SetHeightSpacing(100.f);
			});
		}
		else if (UCEClonerCircleLayout* CircleLayout = ClonerComponent->GetActiveLayout<UCEClonerCircleLayout>())
		{
			FScopedTransaction Transaction(LOCTEXT("VisualizerResetValue", "Visualizer Reset Value"));

			ModifyProperty(
				CircleLayout
				, GET_MEMBER_NAME_CHECKED(UCEClonerCircleLayout, Radius)
				, EPropertyChangeType::ValueSet
				, [this, &CircleLayout]()
			{
				CircleLayout->SetRadius(500.f);
			});
		}
		else if (UCEClonerCylinderLayout* CylinderLayout = ClonerComponent->GetActiveLayout<UCEClonerCylinderLayout>())
		{
			FScopedTransaction Transaction(LOCTEXT("VisualizerResetValue", "Visualizer Reset Value"));

			ModifyProperty(
				CylinderLayout
				, GET_MEMBER_NAME_CHECKED(UCEClonerCylinderLayout, Radius)
				, EPropertyChangeType::ValueSet
				, [this, &CylinderLayout]()
			{
				CylinderLayout->SetRadius(500.f);
			});

			ModifyProperty(
				CylinderLayout
				, GET_MEMBER_NAME_CHECKED(UCEClonerCylinderLayout, Height)
				, EPropertyChangeType::ValueSet
				, [this, &CylinderLayout]()
			{
				CylinderLayout->SetHeight(1000.f);
			});
		}
	}

	return true;
}

bool FAvaClonerActorVisualizer::IsEditing() const
{
	if (bEditingSpacing)
	{
		return true;
	}

	return Super::IsEditing();
}

void FAvaClonerActorVisualizer::EndEditing()
{
	Super::EndEditing();

	ClonerComponentWeak.Reset();
	InitialSpacing = FVector::ZeroVector;
	bEditingSpacing = false;
}

#undef LOCTEXT_NAMESPACE