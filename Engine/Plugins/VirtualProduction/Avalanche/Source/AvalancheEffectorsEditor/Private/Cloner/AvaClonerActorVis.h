// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CEClonerEffectorShared.h"
#include "AvaVisBase.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UCEClonerComponent;

struct HAvaClonerActorSpacingHitProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaClonerActorSpacingHitProxy(const UActorComponent* InComponent, ECEClonerAxis InAxis)
		: HAvaHitProxy(InComponent)
		, Axis(InAxis)
	{}

	ECEClonerAxis Axis = ECEClonerAxis::Custom;
};

/** Custom visualization for cloner actor to handle spacing in various layouts */
class FAvaClonerActorVisualizer : public FAvaVisualizerBase
{
public:
	using Super = FAvaVisualizerBase;
	using MeshType = UCEClonerComponent;

	FAvaClonerActorVisualizer();

	//~ Begin FAvaVisualizerBase
	virtual UActorComponent* GetEditedComponent() const override;
	virtual TMap<UObject*, TArray<FProperty*>> GatherEditableProperties(UObject* InObject) const override;
	virtual bool VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* InVisProxy, const FViewportClick& InClick) override;
	virtual bool GetWidgetLocation(const FEditorViewportClient* InViewportClient, FVector& OutLocation) const override;
	virtual bool GetWidgetMode(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode& OutMode) const override;
	virtual bool GetWidgetAxisList(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const override;
	virtual bool GetWidgetAxisListDragOverride(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const override;
	virtual bool ResetValue(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy) override;
	virtual bool IsEditing() const override;
	virtual void EndEditing() override;
	virtual void StoreInitialValues() override;
	virtual FBox GetComponentBounds(const UActorComponent* InComponent) const override;
	virtual bool HandleInputDeltaInternal(FEditorViewportClient* InViewportClient, FViewport* InViewport, const FVector& InAccumulatedTranslation, const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale) override;
	virtual void DrawVisualizationEditing(const UActorComponent* InComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;
	virtual void DrawVisualizationNotEditing(const UActorComponent* InComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;
	//~ End FAvaVisualizerBase

	UCEClonerComponent* GetClonerComponent() const
	{
		return ClonerComponentWeak.Get();
	}

protected:
	FVector GetHandleSpacingLocation(const UCEClonerComponent* InClonerComponent, ECEClonerAxis InAxis) const;
	void DrawSpacingButton(const UCEClonerComponent* InClonerComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32 InIconIndex, ECEClonerAxis InAxis, FLinearColor InColor) const;

	TWeakObjectPtr<UCEClonerComponent> ClonerComponentWeak = nullptr;

	FVector InitialSpacing = FVector::ZeroVector;
	bool bEditingSpacing = false;
	ECEClonerAxis EditingAxis = ECEClonerAxis::X;
};
