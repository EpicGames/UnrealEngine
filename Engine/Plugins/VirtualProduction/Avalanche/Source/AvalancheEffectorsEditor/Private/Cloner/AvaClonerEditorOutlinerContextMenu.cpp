// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cloner/AvaClonerEditorOutlinerContextMenu.h"

#include "Cloner/Menus/CEEditorClonerMenuContext.h"
#include "Cloner/Menus/CEEditorClonerMenuOptions.h"
#include "GameFramework/Actor.h"
#include "Item/AvaOutlinerActor.h"
#include "Item/AvaOutlinerComponent.h"
#include "Subsystems/CEEditorClonerSubsystem.h"
#include "ToolMenuContext/AvaOutlinerItemsContext.h"

void FAvaClonerEditorOutlinerContextMenu::OnExtendOutlinerContextMenu(UToolMenu* InToolMenu)
{
	if (!InToolMenu)
	{
		return;
	}

	const UAvaOutlinerItemsContext* OutlinerItemsContext = InToolMenu->Context.FindContext<UAvaOutlinerItemsContext>();

	if (!IsValid(OutlinerItemsContext))
	{
		return;
	}

	UCEEditorClonerSubsystem* ClonerSubsystem = UCEEditorClonerSubsystem::Get();

	if (!IsValid(ClonerSubsystem))
	{
		return;
	}

	TSet<UObject*> ContextObjects;
	GetContextObjects(OutlinerItemsContext, ContextObjects);

	if (ContextObjects.IsEmpty())
	{
		return;
	}

	const FCEEditorClonerMenuContext Context(ContextObjects);
	FCEEditorClonerMenuOptions Options(
		{
			ECEEditorClonerMenuType::Enable
			, ECEEditorClonerMenuType::Disable
			, ECEEditorClonerMenuType::CreateEffector
			, ECEEditorClonerMenuType::Convert
			, ECEEditorClonerMenuType::CreateCloner
		}
	);
	Options.UseTransact(true);
	Options.CreateSubMenu(true);

	ClonerSubsystem->FillClonerMenu(InToolMenu, Context, Options);
}

void FAvaClonerEditorOutlinerContextMenu::GetContextObjects(const UAvaOutlinerItemsContext* InContext, TSet<UObject*>& OutObjects)
{
	OutObjects.Empty();

	if (!IsValid(InContext) || InContext->GetItems().IsEmpty())
	{
		return;
	}

	for (const FAvaOutlinerItemWeakPtr& ItemWeak : InContext->GetItems())
	{
		const TSharedPtr<IAvaOutlinerItem> Item = ItemWeak.Pin();

		// is it an actor
		if (const FAvaOutlinerActor* ActorItem = Item->CastTo<FAvaOutlinerActor>())
		{
			if (AActor* Actor = ActorItem->GetActor())
			{
				OutObjects.Add(Actor);
			}
		}
		// is it a component
		else if (const FAvaOutlinerComponent* ComponentItem = Item->CastTo<FAvaOutlinerComponent>())
		{
			if (USceneComponent* Component = ComponentItem->GetComponent())
			{
				OutObjects.Add(Component);
			}
		}
	}
}
