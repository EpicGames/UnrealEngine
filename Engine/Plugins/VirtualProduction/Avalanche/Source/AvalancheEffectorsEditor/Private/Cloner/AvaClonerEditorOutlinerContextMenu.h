// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "ToolMenus.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UAvaOutlinerItemsContext;

/** Context menu actions in outliner related to cloner */
class FAvaClonerEditorOutlinerContextMenu
{
public:
	/** Extend outliner context menu */
	static void OnExtendOutlinerContextMenu(UToolMenu* InToolMenu);

	/** Gets all pertinent context object in this context */
	static void GetContextObjects(const UAvaOutlinerItemsContext* InContext, TSet<UObject*>& OutObjects);
};