// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaEffectorActorTool.h"
#include "AvaEffectorsEditorCommands.h"
#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "Effector/CEEffectorActor.h"
#include "Effector/CEEffectorActorFactory.h"
#include "Effector/CEEffectorComponent.h"

UAvaEffectorActorTool::UAvaEffectorActorTool()
{
	ActorClass = ACEEffectorActor::StaticClass();
}

void UAvaEffectorActorTool::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	// Register commands here, subsystem is init
	FAvaEffectorsEditorCommands::Register();

	const FString ToolIdentifier = TEXT("Effector Actor Tool ");
	for (const TPair<FName, TSharedPtr<FUICommandInfo>>& EffectorCommandPair : FAvaEffectorsEditorCommands::Get().Tool_Actor_Effectors)
	{
		UCEEffectorActorFactory* EffectorActorFactory = CreateActorFactory<UCEEffectorActorFactory>();
		EffectorActorFactory->SetEffectorTypeName(EffectorCommandPair.Key);

		FAvaInteractiveToolsToolParameters ToolParameters = CreateDefaultToolParameters();
		ToolParameters.ToolIdentifier = ToolIdentifier + EffectorCommandPair.Key.ToString();
		ToolParameters.UICommand = EffectorCommandPair.Value;
		ToolParameters.Priority = 4000;
		ToolParameters.ActorFactory.Set<TObjectPtr<UActorFactory>>(EffectorActorFactory);

		InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryNameEffector, MoveTemp(ToolParameters));
	}
}