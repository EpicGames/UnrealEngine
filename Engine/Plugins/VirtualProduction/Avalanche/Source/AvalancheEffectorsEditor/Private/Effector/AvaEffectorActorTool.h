// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaInteractiveToolsActorPointToolBase.h"
#include "AvaEffectorActorTool.generated.h"

UCLASS()
class UAvaEffectorActorTool : public UAvaInteractiveToolsActorPointToolBase
{
	GENERATED_BODY()

public:
	UAvaEffectorActorTool();

	//~ Begin UAvaInteractiveToolsToolBase
	virtual void OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule) override;
	//~ End UAvaInteractiveToolsToolBase
};
