// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/AvaEffectorActorVis.h"
#include "AvaField.h"
#include "AvaShapeSprites.h"
#include "AvaVisBase.h"
#include "EditorViewportClient.h"
#include "Effector/Types/CEEffectorBoxType.h"
#include "Effector/Types/CEEffectorPlaneType.h"
#include "Effector/Types/CEEffectorRadialType.h"
#include "Effector/Types/CEEffectorSphereType.h"
#include "Effector/Types/CEEffectorTorusType.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "ScopedTransaction.h"
#include "TextureResource.h"

IMPLEMENT_HIT_PROXY(HAvaEffectorActorZoneHitProxy, HAvaHitProxy);

#define LOCTEXT_NAMESPACE "AvaEffectorActorVisualizer"

FAvaEffectorActorVisualizer::FAvaEffectorActorVisualizer()
	: FAvaVisualizerBase()
{
	using namespace UE::AvaCore;

	// Sphere
	InnerRadiusProperty  = GetProperty<UCEEffectorSphereType>(GET_MEMBER_NAME_CHECKED(UCEEffectorSphereType, InnerRadius));
	OuterRadiusProperty  = GetProperty<UCEEffectorSphereType>(GET_MEMBER_NAME_CHECKED(UCEEffectorSphereType, OuterRadius));

	// Box
	InnerExtentProperty  = GetProperty<UCEEffectorBoxType>(GET_MEMBER_NAME_CHECKED(UCEEffectorBoxType, InnerExtent));
	OuterExtentProperty  = GetProperty<UCEEffectorBoxType>(GET_MEMBER_NAME_CHECKED(UCEEffectorBoxType, OuterExtent));

	// Plane
	PlaneSpacingProperty = GetProperty<UCEEffectorPlaneType>(GET_MEMBER_NAME_CHECKED(UCEEffectorPlaneType, PlaneSpacing));

	// Radial
	RadialAngleProperty = GetProperty<UCEEffectorRadialType>(GET_MEMBER_NAME_CHECKED(UCEEffectorRadialType, RadialAngle));
	RadialMinRadiusProperty = GetProperty<UCEEffectorRadialType>(GET_MEMBER_NAME_CHECKED(UCEEffectorRadialType, RadialMinRadius));
	RadialMaxRadiusProperty = GetProperty<UCEEffectorRadialType>(GET_MEMBER_NAME_CHECKED(UCEEffectorRadialType, RadialMaxRadius));

	// Torus
	TorusRadiusProperty = GetProperty<UCEEffectorTorusType>(GET_MEMBER_NAME_CHECKED(UCEEffectorTorusType, TorusRadius));
	TorusInnerRadiusProperty = GetProperty<UCEEffectorTorusType>(GET_MEMBER_NAME_CHECKED(UCEEffectorTorusType, TorusInnerRadius));
	TorusOuterRadiusProperty = GetProperty<UCEEffectorTorusType>(GET_MEMBER_NAME_CHECKED(UCEEffectorTorusType, TorusOuterRadius));
}

void FAvaEffectorActorVisualizer::StoreInitialValues()
{
	Super::StoreInitialValues();

	const UCEEffectorComponent* EffectorComponent = EffectorComponentWeak.Get();

	if (!EffectorComponent)
	{
		return;
	}

	if (UCEEffectorSphereType* SphereType = EffectorComponent->GetActiveType<UCEEffectorSphereType>())
	{
		// Sphere
		InitialInnerRadius = SphereType->GetInnerRadius();
		InitialOuterRadius = SphereType->GetOuterRadius();
	}
	else if (UCEEffectorBoxType* BoxType = EffectorComponent->GetActiveType<UCEEffectorBoxType>())
	{
		// Box
		InitialInnerExtent = BoxType->GetInnerExtent();
		InitialOuterExtent = BoxType->GetOuterExtent();
	}
	else if (UCEEffectorPlaneType* PlaneType = EffectorComponent->GetActiveType<UCEEffectorPlaneType>())
	{
		// Plane
		InitialPlaneSpacing = PlaneType->GetPlaneSpacing();
	}
	else if (UCEEffectorRadialType* RadialType = EffectorComponent->GetActiveType<UCEEffectorRadialType>())
	{
		// Radial
		InitialRadialAngle = RadialType->GetRadialAngle();
		InitialRadialMinRadius = RadialType->GetRadialMinRadius();
		InitialRadialMaxRadius = RadialType->GetRadialMaxRadius();
	}
	else if (UCEEffectorTorusType* TorusType = EffectorComponent->GetActiveType<UCEEffectorTorusType>())
	{
		// Torus
		InitialTorusRadius = TorusType->GetTorusRadius();
		InitialTorusInnerRadius = TorusType->GetTorusInnerRadius();
		InitialTorusOuterRadius = TorusType->GetTorusOuterRadius();
	}
}

FBox FAvaEffectorActorVisualizer::GetComponentBounds(const UActorComponent* InComponent) const
{
	if (const UCEEffectorComponent* EffectorComponent = Cast<UCEEffectorComponent>(InComponent))
	{
		if (UCEEffectorSphereType* SphereType = EffectorComponent->GetActiveType<UCEEffectorSphereType>())
		{
			return FBox(-FVector(SphereType->GetOuterRadius() / 2), FVector(SphereType->GetOuterRadius() / 2));
		}
		else if (UCEEffectorBoxType* BoxType = EffectorComponent->GetActiveType<UCEEffectorBoxType>())
		{
			return FBox(-BoxType->GetOuterExtent(), BoxType->GetOuterExtent());
		}
		else if (UCEEffectorPlaneType* PlaneType = EffectorComponent->GetActiveType<UCEEffectorPlaneType>())
		{
			return FBox(-FVector(PlaneType->GetPlaneSpacing() / 2), FVector(PlaneType->GetPlaneSpacing() / 2));
		}
		else if (UCEEffectorRadialType* RadialType = EffectorComponent->GetActiveType<UCEEffectorRadialType>())
		{
			return FBox(-FVector(RadialType->GetRadialMaxRadius() / 2), FVector(RadialType->GetRadialMaxRadius() / 2));
		}
		else if (UCEEffectorTorusType* TorusType = EffectorComponent->GetActiveType<UCEEffectorTorusType>())
		{
			return FBox(-FVector((TorusType->GetTorusRadius() + TorusType->GetTorusOuterRadius()) / 2), FVector((TorusType->GetTorusRadius() + TorusType->GetTorusOuterRadius()) / 2));
		}
	}

	return Super::GetComponentBounds(InComponent);
}

bool FAvaEffectorActorVisualizer::HandleInputDeltaInternal(FEditorViewportClient* InViewportClient, FViewport* InViewport, const FVector& InAccumulatedTranslation, const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale)
{
	if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
	{
		return false;
	}

	if (UCEEffectorComponent* EffectorComponent = EffectorComponentWeak.Get())
	{
		if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
		{
			if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::XYZ)
			{
				if (UCEEffectorBoxType* BoxType = EffectorComponent->GetActiveType<UCEEffectorBoxType>())
				{
					if (EditingHandleType == HandleTypeInnerZone)
					{
						ModifyProperty(BoxType, InnerExtentProperty, EPropertyChangeType::Interactive, [this, &BoxType, &InAccumulatedTranslation]()
						{
							BoxType->SetInnerExtent(InitialInnerExtent + InAccumulatedTranslation);
						});
					}
					else if (EditingHandleType == HandleTypeOuterZone)
					{
						ModifyProperty(BoxType, OuterExtentProperty, EPropertyChangeType::Interactive, [this, &BoxType, &InAccumulatedTranslation]()
						{
							BoxType->SetOuterExtent(InitialOuterExtent + InAccumulatedTranslation);
						});
					}

					return true;
				}
			}

			if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
			{
				if (UCEEffectorPlaneType* PlaneType = EffectorComponent->GetActiveType<UCEEffectorPlaneType>())
				{
					if (EditingHandleType == HandleTypeInnerZone || EditingHandleType == HandleTypeOuterZone)
					{
						ModifyProperty(PlaneType, PlaneSpacingProperty, EPropertyChangeType::Interactive, [this, &PlaneType, &InAccumulatedTranslation]()
						{
							PlaneType->SetPlaneSpacing(InitialPlaneSpacing + InAccumulatedTranslation.Y);
						});
					}

					return true;
				}

				if (UCEEffectorSphereType* SphereType = EffectorComponent->GetActiveType<UCEEffectorSphereType>())
				{
					if (EditingHandleType == HandleTypeInnerZone)
					{
						ModifyProperty(SphereType, InnerRadiusProperty, EPropertyChangeType::Interactive, [this, &SphereType, &InAccumulatedTranslation]()
						{
							SphereType->SetInnerRadius(InitialInnerRadius + InAccumulatedTranslation.Y);
						});
					}
					else if (EditingHandleType == HandleTypeOuterZone)
					{
						ModifyProperty(SphereType, OuterRadiusProperty, EPropertyChangeType::Interactive, [this, &SphereType, &InAccumulatedTranslation]
						{
							SphereType->SetOuterRadius(InitialOuterRadius + InAccumulatedTranslation.Y);
						});
					}

					return true;
				}

				if (UCEEffectorRadialType* RadialType = EffectorComponent->GetActiveType<UCEEffectorRadialType>())
				{
					if (EditingHandleType == HandleTypeInnerZone)
					{
						ModifyProperty(RadialType, RadialMinRadiusProperty, EPropertyChangeType::Interactive, [this, &RadialType, &InAccumulatedTranslation]
						{
							RadialType->SetRadialMinRadius(InitialRadialMinRadius + InAccumulatedTranslation.Y);
						});
					}
					else if (EditingHandleType == HandleTypeOuterZone)
					{
						ModifyProperty(RadialType, RadialMaxRadiusProperty, EPropertyChangeType::Interactive, [this, &RadialType, &InAccumulatedTranslation]()
						{
							RadialType->SetRadialMaxRadius(InitialRadialMaxRadius + InAccumulatedTranslation.Y);
						});
					}

					return true;
				}

				if (UCEEffectorTorusType* TorusType = EffectorComponent->GetActiveType<UCEEffectorTorusType>())
				{
					if (EditingHandleType == HandleTypeRadius)
					{
						ModifyProperty(TorusType, TorusRadiusProperty, EPropertyChangeType::Interactive, [this, &TorusType, &InAccumulatedTranslation]()
						{
							TorusType->SetTorusRadius(InitialTorusRadius + InAccumulatedTranslation.Y);
						});
					}

					return true;
				}
			}

			if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
			{
				if (UCEEffectorTorusType* TorusType = EffectorComponent->GetActiveType<UCEEffectorTorusType>())
				{
					if (EditingHandleType == HandleTypeInnerZone)
					{
						ModifyProperty(TorusType, TorusInnerRadiusProperty, EPropertyChangeType::Interactive, [this, &TorusType, &InAccumulatedTranslation]()
						{
							TorusType->SetTorusInnerRadius(InitialTorusInnerRadius + InAccumulatedTranslation.Z);
						});
					}
					else if (EditingHandleType == HandleTypeOuterZone)
					{
						ModifyProperty(TorusType, TorusOuterRadiusProperty, EPropertyChangeType::Interactive, [this, &TorusType, &InAccumulatedTranslation]()
						{
							TorusType->SetTorusOuterRadius(InitialTorusOuterRadius + InAccumulatedTranslation.Z);
						});
					}

					return true;
				}
			}
		}
		else if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Rotate)
		{
			if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
			{
				if (UCEEffectorRadialType* RadialType = EffectorComponent->GetActiveType<UCEEffectorRadialType>())
				{
					if (EditingHandleType == HandleTypeAngle)
					{
						ModifyProperty(RadialType, RadialAngleProperty, EPropertyChangeType::Interactive, [this, &RadialType, &InAccumulatedRotation]()
						{
							RadialType->SetRadialAngle(InitialRadialAngle + InAccumulatedRotation.Yaw);
						});
					}

					return true;
				}
			}
		}
	}
	else
	{
		EndEditing();
	}

	return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
}

void FAvaEffectorActorVisualizer::DrawVisualizationEditing(const UActorComponent* InComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationEditing(InComponent, InView, InPDI, InOutIconIndex);

	const UCEEffectorComponent* EffectorComponent = Cast<UCEEffectorComponent>(InComponent);

	if (!EffectorComponent)
	{
		return;
	}

	if (!EffectorComponent->GetActiveType<UCEEffectorPlaneType>())
	{
		DrawZoneButton(EffectorComponent, InView, InPDI, InOutIconIndex, HandleTypeInnerZone, FAvaVisualizerBase::Active);
		InOutIconIndex++;
	}

	DrawZoneButton(EffectorComponent, InView, InPDI, InOutIconIndex, HandleTypeOuterZone, FAvaVisualizerBase::Active);
	InOutIconIndex++;

	if (EffectorComponent->GetActiveType<UCEEffectorRadialType>())
	{
		DrawZoneButton(EffectorComponent, InView, InPDI, InOutIconIndex, HandleTypeAngle, FAvaVisualizerBase::Active);
		InOutIconIndex++;
	}

	if (EffectorComponent->GetActiveType<UCEEffectorTorusType>())
	{
		DrawZoneButton(EffectorComponent, InView, InPDI, InOutIconIndex, HandleTypeRadius, FAvaVisualizerBase::Active);
		InOutIconIndex++;
	}
}

void FAvaEffectorActorVisualizer::DrawVisualizationNotEditing(const UActorComponent* InComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationNotEditing(InComponent, InView, InPDI, InOutIconIndex);

	const UCEEffectorComponent* EffectorComponent = Cast<UCEEffectorComponent>(InComponent);

	if (!EffectorComponent)
	{
		return;
	}

	if (!EffectorComponent->GetActiveType<UCEEffectorPlaneType>())
	{
		DrawZoneButton(EffectorComponent, InView, InPDI, InOutIconIndex, HandleTypeInnerZone, FAvaVisualizerBase::Inactive);
		InOutIconIndex++;
	}

	DrawZoneButton(EffectorComponent, InView, InPDI, InOutIconIndex, HandleTypeOuterZone, FAvaVisualizerBase::Inactive);
	InOutIconIndex++;

	if (EffectorComponent->GetActiveType<UCEEffectorRadialType>())
	{
		DrawZoneButton(EffectorComponent, InView, InPDI, InOutIconIndex, HandleTypeAngle, FAvaVisualizerBase::Inactive);
		InOutIconIndex++;
	}

	if (EffectorComponent->GetActiveType<UCEEffectorTorusType>())
	{
		DrawZoneButton(EffectorComponent, InView, InPDI, InOutIconIndex, HandleTypeRadius, FAvaVisualizerBase::Inactive);
		InOutIconIndex++;
	}
}

FVector FAvaEffectorActorVisualizer::GetHandleZoneLocation(const UCEEffectorComponent* InEffectorComponent, int32 InHandleType) const
{
	const FVector EffectorScale = InEffectorComponent->GetComponentScale();
	const FRotator EffectorRotation = InEffectorComponent->GetComponentRotation();
	FVector OutLocation = InEffectorComponent->GetComponentLocation();

	// To avoid inner/outer handle to be near actor gizmo and hard to select
	constexpr float MinHandleOffset = 50.f;
	constexpr float MaxHandleOffset = 100.f;

	if (UCEEffectorBoxType* BoxType = InEffectorComponent->GetActiveType<UCEEffectorBoxType>())
	{
		if (InHandleType == HandleTypeInnerZone)
		{
			OutLocation += EffectorRotation.RotateVector(BoxType->GetInnerExtent()) * EffectorScale;
		}
		else if (InHandleType == HandleTypeOuterZone)
		{
			OutLocation += EffectorRotation.RotateVector(BoxType->GetOuterExtent()) * EffectorScale;
		}
	}
	else if (UCEEffectorPlaneType* PlaneType = InEffectorComponent->GetActiveType<UCEEffectorPlaneType>())
	{
		if (InHandleType == HandleTypeInnerZone
			|| InHandleType == HandleTypeOuterZone)
		{
			const float ComponentScale = (EffectorRotation.RotateVector(-FVector::YAxisVector) * EffectorScale).Length();
			const FVector HandleAxis = EffectorRotation.RotateVector(FVector::YAxisVector);

			OutLocation += HandleAxis * (PlaneType->GetPlaneSpacing() / 2) * ComponentScale;
		}
	}
	else if (UCEEffectorSphereType* SphereType = InEffectorComponent->GetActiveType<UCEEffectorSphereType>())
	{
		const float MinComponentScale = FMath::Min<float>(FMath::Min<float>(EffectorScale.X, EffectorScale.Y), EffectorScale.Z);
		const FVector HandleAxis = EffectorRotation.RotateVector(FVector::YAxisVector);

		if (InHandleType == HandleTypeInnerZone)
		{
			OutLocation += FVector::Max(HandleAxis * SphereType->GetInnerRadius(), HandleAxis * MinHandleOffset) * MinComponentScale;
		}
		else if (InHandleType == HandleTypeOuterZone)
		{
			OutLocation += FVector::Max(HandleAxis * SphereType->GetOuterRadius(), HandleAxis * MaxHandleOffset) * MinComponentScale;
		}
	}
	else if (UCEEffectorRadialType* RadialType = InEffectorComponent->GetActiveType<UCEEffectorRadialType>())
	{
		const float MinComponentScale = FMath::Min<float>(FMath::Min<float>(EffectorScale.X, EffectorScale.Y), EffectorScale.Z);
		const FVector UpHandleAxis = EffectorRotation.RotateVector(FVector::ZAxisVector);
		const FVector RightHandleAxis = EffectorRotation.RotateVector(FVector::YAxisVector);

		if (InHandleType == HandleTypeInnerZone)
		{
			OutLocation += FVector::Max(RightHandleAxis * RadialType->GetRadialMinRadius(), RightHandleAxis * MinHandleOffset) * MinComponentScale;
		}
		else if (InHandleType == HandleTypeOuterZone)
		{
			OutLocation += FVector::Max(RightHandleAxis * RadialType->GetRadialMaxRadius(), RightHandleAxis * MaxHandleOffset) * MinComponentScale;
		}
		else if (InHandleType == HandleTypeAngle)
		{
			OutLocation += UpHandleAxis * MaxHandleOffset * MinComponentScale;
		}
	}
	else if (UCEEffectorTorusType* TorusType = InEffectorComponent->GetActiveType<UCEEffectorTorusType>())
	{
		const float MinComponentScale = FMath::Min<float>(FMath::Min<float>(EffectorScale.X, EffectorScale.Y), EffectorScale.Z);
		const FVector UpHandleAxis = EffectorRotation.RotateVector(FVector::ZAxisVector);
		const FVector RightHandleAxis = EffectorRotation.RotateVector(FVector::YAxisVector);

		if (InHandleType == HandleTypeInnerZone)
		{
			OutLocation += RightHandleAxis * TorusType->GetTorusRadius()
				+ FVector::Max(UpHandleAxis * TorusType->GetTorusInnerRadius(), UpHandleAxis * MinHandleOffset) * MinComponentScale;
		}
		else if (InHandleType == HandleTypeOuterZone)
		{
			OutLocation += RightHandleAxis * TorusType->GetTorusRadius()
				+ FVector::Max(UpHandleAxis * TorusType->GetTorusOuterRadius(), UpHandleAxis * MaxHandleOffset) * MinComponentScale;
		}
		else if (InHandleType == HandleTypeRadius)
		{
			OutLocation += FVector::Max(RightHandleAxis * TorusType->GetTorusRadius(), RightHandleAxis * MinHandleOffset) * MinComponentScale;
		}
	}

	return OutLocation;
}

void FAvaEffectorActorVisualizer::DrawZoneButton(const UCEEffectorComponent* InEffectorComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32 InIconIndex, int32 InHandleType, FLinearColor InColor) const
{
	UTexture2D* ZoneSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::BevelSprite);

	if (!ZoneSprite || !ZoneSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	IconLocation = GetHandleZoneLocation(InEffectorComponent, InHandleType);

	InPDI->SetHitProxy(new HAvaEffectorActorZoneHitProxy(InEffectorComponent, InHandleType));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, ZoneSprite->GetResource(), InColor, SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

UActorComponent* FAvaEffectorActorVisualizer::GetEditedComponent() const
{
	return EffectorComponentWeak.Get();
}

TMap<UObject*, TArray<FProperty*>> FAvaEffectorActorVisualizer::GatherEditableProperties(UObject* InObject) const
{
	if (const UCEEffectorComponent* EffectorComponent = Cast<UCEEffectorComponent>(InObject))
	{
		if (UCEEffectorSphereType* SphereType = EffectorComponent->GetActiveType<UCEEffectorSphereType>())
		{
			return {{SphereType, {InnerRadiusProperty, OuterRadiusProperty}}};
		}
		else if (UCEEffectorBoxType* BoxType = EffectorComponent->GetActiveType<UCEEffectorBoxType>())
		{
			return {{BoxType, {InnerExtentProperty, OuterExtentProperty}}};
		}
		else if (UCEEffectorPlaneType* PlaneType = EffectorComponent->GetActiveType<UCEEffectorPlaneType>())
		{
			return {{PlaneType, {PlaneSpacingProperty}}};
		}
		else if (UCEEffectorRadialType* RadialType = EffectorComponent->GetActiveType<UCEEffectorRadialType>())
		{
			return {{RadialType, {RadialAngleProperty, RadialMinRadiusProperty, RadialMaxRadiusProperty}}};
		}
		else if (UCEEffectorTorusType* TorusType = EffectorComponent->GetActiveType<UCEEffectorTorusType>())
		{
			return {{TorusType, {TorusRadiusProperty, TorusInnerRadiusProperty, TorusOuterRadiusProperty}}};
		}
	}

	return {};
}

bool FAvaEffectorActorVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* InVisProxy, const FViewportClick& InClick)
{
	if (InClick.GetKey() != EKeys::LeftMouseButton)
	{
		EndEditing();
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	UActorComponent* Component = const_cast<UActorComponent*>(InVisProxy->Component.Get());

	if (!Component || !Component->IsA<UCEEffectorComponent>())
	{
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	if (InVisProxy->IsA(HAvaEffectorActorZoneHitProxy::StaticGetType()))
	{
		EndEditing();
		EffectorComponentWeak = Cast<UCEEffectorComponent>(Component);
		EditingHandleType = static_cast<HAvaEffectorActorZoneHitProxy*>(InVisProxy)->HandleType;
		StartEditing(InViewportClient, Component);

		return true;
	}

	return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
}

bool FAvaEffectorActorVisualizer::GetWidgetLocation(const FEditorViewportClient* InViewportClient, FVector& OutLocation) const
{
	if (const UCEEffectorComponent* EffectorComponent = EffectorComponentWeak.Get())
	{
		OutLocation = GetHandleZoneLocation(EffectorComponent, EditingHandleType);
		return true;
	}

	return Super::GetWidgetLocation(InViewportClient, OutLocation);
}

bool FAvaEffectorActorVisualizer::GetWidgetMode(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode& OutMode) const
{
	if (EditingHandleType == HandleTypeInnerZone
		|| EditingHandleType == HandleTypeOuterZone
		|| EditingHandleType == HandleTypeRadius)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Translate;
		return true;
	}

	if (EditingHandleType == HandleTypeAngle)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Rotate;
		return true;
	}

	return Super::GetWidgetMode(InViewportClient, OutMode);
}

bool FAvaEffectorActorVisualizer::GetWidgetAxisList(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (UCEEffectorComponent* EffectorComponent = EffectorComponentWeak.Get())
	{
		if (EditingHandleType == HandleTypeInnerZone
		|| EditingHandleType == HandleTypeOuterZone)
		{
			if (!!EffectorComponent->GetActiveType<UCEEffectorTorusType>())
			{
				OutAxisList = EAxisList::Type::Z;
			}
			else if (!!EffectorComponent->GetActiveType<UCEEffectorBoxType>())
			{
				OutAxisList = EAxisList::Type::XYZ;
			}
			else
			{
				OutAxisList = EAxisList::Type::Y;
			}

			return true;
		}

		if (EditingHandleType == HandleTypeRadius)
		{
			if (!!EffectorComponent->GetActiveType<UCEEffectorTorusType>())
			{
				OutAxisList = EAxisList::Type::Y;
			}

			return true;
		}

		if (EditingHandleType == HandleTypeAngle)
		{
			if (!!EffectorComponent->GetActiveType<UCEEffectorRadialType>())
			{
				OutAxisList = EAxisList::Type::Z;
			}

			return true;
		}
	}

	return Super::GetWidgetAxisList(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaEffectorActorVisualizer::GetWidgetAxisListDragOverride(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (UCEEffectorComponent* EffectorComponent = EffectorComponentWeak.Get())
	{
		if (EditingHandleType == HandleTypeInnerZone
		|| EditingHandleType == HandleTypeOuterZone)
		{
			if (!!EffectorComponent->GetActiveType<UCEEffectorTorusType>())
			{
				OutAxisList = EAxisList::Type::Z;
				return true;
			}

			if (!EffectorComponent->GetActiveType<UCEEffectorBoxType>())
			{
				OutAxisList = EAxisList::Type::Y;
				return true;
			}
		}

		if (EditingHandleType == HandleTypeRadius)
		{
			if (!!EffectorComponent->GetActiveType<UCEEffectorTorusType>())
			{
				OutAxisList = EAxisList::Type::Y;
				return true;
			}
		}

		if (EditingHandleType == HandleTypeAngle)
		{
			if (!!EffectorComponent->GetActiveType<UCEEffectorRadialType>())
			{
				OutAxisList = EAxisList::Type::Z;
			}

			return true;
		}
	}

	return Super::GetWidgetAxisListDragOverride(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaEffectorActorVisualizer::ResetValue(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy)
{
	if (!InHitProxy->IsA(HAvaEffectorActorZoneHitProxy::StaticGetType()))
	{
		return Super::ResetValue(InViewportClient, InHitProxy);
	}

	const HAvaEffectorActorZoneHitProxy* ComponentHitProxy = static_cast<HAvaEffectorActorZoneHitProxy*>(InHitProxy);
	UActorComponent* Component = const_cast<UActorComponent*>(ComponentHitProxy->Component.Get());

	if (!Component || !Component->IsA<UCEEffectorComponent>())
	{
		return Super::ResetValue(InViewportClient, InHitProxy);
	}

	if (UCEEffectorComponent* EffectorComponent = Cast<UCEEffectorComponent>(Component))
	{
		const int32 HandleType = ComponentHitProxy->HandleType;

		FScopedTransaction Transaction(LOCTEXT("VisualizerResetValue", "Visualizer Reset Value"));

		if (UCEEffectorBoxType* BoxType = EffectorComponent->GetActiveType<UCEEffectorBoxType>())
		{
			if (HandleType == HandleTypeInnerZone)
			{
				ModifyProperty(BoxType, InnerExtentProperty, EPropertyChangeType::ValueSet, [&BoxType]()
				{
					BoxType->SetInnerExtent(FVector(50.f));
				});
			}
			else if (HandleType == HandleTypeOuterZone)
			{
				ModifyProperty(BoxType, OuterExtentProperty, EPropertyChangeType::ValueSet, [&BoxType]()
				{
					BoxType->SetOuterExtent(FVector(200.f));
				});
			}
		}
		else if (UCEEffectorPlaneType* PlaneType = EffectorComponent->GetActiveType<UCEEffectorPlaneType>())
		{
			if (HandleType == HandleTypeInnerZone
				|| HandleType == HandleTypeOuterZone)
			{
				ModifyProperty(PlaneType, PlaneSpacingProperty, EPropertyChangeType::ValueSet, [&PlaneType]()
				{
					PlaneType->SetPlaneSpacing(200.f);
				});
			}
		}
		else if (UCEEffectorSphereType* SphereType = EffectorComponent->GetActiveType<UCEEffectorSphereType>())
		{
			if (HandleType == HandleTypeInnerZone)
			{
				ModifyProperty(SphereType, InnerRadiusProperty, EPropertyChangeType::ValueSet, [&SphereType]()
				{
					SphereType->SetInnerRadius(50.f);
				});
			}
			else if (HandleType == HandleTypeOuterZone)
			{
				ModifyProperty(SphereType, OuterRadiusProperty, EPropertyChangeType::ValueSet, [&SphereType]()
				{
					SphereType->SetOuterRadius(200.f);
				});
			}
		}
		else if (UCEEffectorRadialType* RadialType = EffectorComponent->GetActiveType<UCEEffectorRadialType>())
		{
			if (HandleType == HandleTypeInnerZone)
			{
				ModifyProperty(RadialType, RadialMinRadiusProperty, EPropertyChangeType::ValueSet, [&RadialType]()
				{
					RadialType->SetRadialMinRadius(0.f);
				});
			}
			else if (HandleType == HandleTypeOuterZone)
			{
				ModifyProperty(RadialType, RadialMaxRadiusProperty, EPropertyChangeType::ValueSet, [&RadialType]()
				{
					RadialType->SetRadialMaxRadius(1000.f);
				});
			}
			else if (HandleType == HandleTypeAngle)
			{
				ModifyProperty(RadialType, RadialAngleProperty, EPropertyChangeType::ValueSet, [&RadialType]()
				{
					RadialType->SetRadialAngle(180.f);
				});
			}
		}
		else if (UCEEffectorTorusType* TorusType = EffectorComponent->GetActiveType<UCEEffectorTorusType>())
		{
			if (HandleType == HandleTypeInnerZone)
			{
				ModifyProperty(TorusType, TorusInnerRadiusProperty, EPropertyChangeType::ValueSet, [&TorusType]()
				{
					TorusType->SetTorusInnerRadius(50.f);
				});
			}
			else if (HandleType == HandleTypeOuterZone)
			{
				ModifyProperty(TorusType, TorusOuterRadiusProperty, EPropertyChangeType::ValueSet, [&TorusType]()
				{
					TorusType->SetTorusOuterRadius(200.f);
				});
			}
			else if (HandleType == HandleTypeRadius)
			{
				ModifyProperty(TorusType, TorusRadiusProperty, EPropertyChangeType::ValueSet, [&TorusType]()
				{
					TorusType->SetTorusRadius(250.f);
				});
			}
		}
	}

	return true;
}

bool FAvaEffectorActorVisualizer::IsEditing() const
{
	if (EditingHandleType != INDEX_NONE)
	{
		return true;
	}

	return Super::IsEditing();
}

void FAvaEffectorActorVisualizer::EndEditing()
{
	Super::EndEditing();

	EffectorComponentWeak.Reset();
	EditingHandleType = INDEX_NONE;
}

#undef LOCTEXT_NAMESPACE
