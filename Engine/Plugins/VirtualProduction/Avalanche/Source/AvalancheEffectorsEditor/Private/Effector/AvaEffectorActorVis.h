// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Effector/CEEffectorComponent.h"
#include "AvaVisBase.h"

struct HAvaEffectorActorZoneHitProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaEffectorActorZoneHitProxy(const UActorComponent* InComponent, int32 InHandleType)
		: HAvaHitProxy(InComponent)
		, HandleType(InHandleType)
	{}

	int32 HandleType = INDEX_NONE;
};

/** Custom visualization for effector actor to handle weight zones */
class FAvaEffectorActorVisualizer : public FAvaVisualizerBase
{
public:
	using Super = FAvaVisualizerBase;

	FAvaEffectorActorVisualizer();

	//~ Begin FAvaVisualizerBase
	virtual UActorComponent* GetEditedComponent() const override;
	virtual TMap<UObject*, TArray<FProperty*>> GatherEditableProperties(UObject* InObject) const override;
	virtual bool VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* InVisProxy, const FViewportClick& InClick) override;
	virtual bool GetWidgetLocation(const FEditorViewportClient* InViewportClient, FVector& OutLocation) const override;
	virtual bool GetWidgetMode(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode& OutMode) const override;
	virtual bool GetWidgetAxisList(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const override;
	virtual bool GetWidgetAxisListDragOverride(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const override;
	virtual bool ResetValue(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy) override;
	virtual bool IsEditing() const override;
	virtual void EndEditing() override;
	virtual void StoreInitialValues() override;
	virtual FBox GetComponentBounds(const UActorComponent* InComponent) const override;
	virtual bool HandleInputDeltaInternal(FEditorViewportClient* InViewportClient, FViewport* InViewport, const FVector& InAccumulatedTranslation, const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale) override;
	virtual void DrawVisualizationEditing(const UActorComponent* InComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;
	virtual void DrawVisualizationNotEditing(const UActorComponent* InComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;
	//~ End FAvaVisualizerBase

	UCEEffectorComponent* GetEffectorComponent() const
	{
		return EffectorComponentWeak.Get();
	}

protected:
	FVector GetHandleZoneLocation(const UCEEffectorComponent* InEffectorComponent, int32 InHandleType) const;
	void DrawZoneButton(const UCEEffectorComponent* InEffectorComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32 InIconIndex, int32 InHandleType, FLinearColor InColor) const;

	FProperty* InnerRadiusProperty;
	FProperty* OuterRadiusProperty;

	FProperty* InnerExtentProperty;
	FProperty* OuterExtentProperty;

	FProperty* PlaneSpacingProperty;

	FProperty* RadialAngleProperty;
	FProperty* RadialMinRadiusProperty;
	FProperty* RadialMaxRadiusProperty;

	FProperty* TorusRadiusProperty;
	FProperty* TorusInnerRadiusProperty;
	FProperty* TorusOuterRadiusProperty;

	TWeakObjectPtr<UCEEffectorComponent> EffectorComponentWeak = nullptr;

	float InitialInnerRadius = 0.f;
	float InitialOuterRadius = 0.f;

	FVector InitialInnerExtent = FVector(0.f);
	FVector InitialOuterExtent = FVector(0.f);

	float InitialPlaneSpacing = 0.f;

	float InitialRadialAngle = 0.f;
	float InitialRadialMinRadius = 0.f;
	float InitialRadialMaxRadius = 0.f;

	float InitialTorusRadius = 0.f;
	float InitialTorusInnerRadius = 0.f;
	float InitialTorusOuterRadius = 0.f;

	static constexpr int32 HandleTypeInnerZone = 0;
	static constexpr int32 HandleTypeOuterZone = 1;
	static constexpr int32 HandleTypeRadius = 2;
	static constexpr int32 HandleTypeAngle = 3;

	int32 EditingHandleType = INDEX_NONE;
};
