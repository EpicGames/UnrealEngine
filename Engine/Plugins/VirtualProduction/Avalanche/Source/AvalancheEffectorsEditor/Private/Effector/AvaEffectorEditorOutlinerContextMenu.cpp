// Copyright Epic Games, Inc. All Rights Reserved.

#include "Effector/AvaEffectorEditorOutlinerContextMenu.h"

#include "Effector/CEEffectorComponent.h"
#include "Effector/Menus/CEEditorEffectorMenuContext.h"
#include "Effector/Menus/CEEditorEffectorMenuOptions.h"
#include "GameFramework/Actor.h"
#include "Item/AvaOutlinerActor.h"
#include "Item/AvaOutlinerComponent.h"
#include "Subsystems/CEEditorEffectorSubsystem.h"
#include "ToolMenuContext/AvaOutlinerItemsContext.h"

void FAvaEffectorEditorOutlinerContextMenu::OnExtendOutlinerContextMenu(UToolMenu* InToolMenu)
{
	if (!InToolMenu)
	{
		return;
	}

	const UAvaOutlinerItemsContext* OutlinerItemsContext = InToolMenu->Context.FindContext<UAvaOutlinerItemsContext>();

	if (!IsValid(OutlinerItemsContext))
	{
		return;
	}

	UCEEditorEffectorSubsystem* EffectorSubsystem = UCEEditorEffectorSubsystem::Get();

	if (!IsValid(EffectorSubsystem))
	{
		return;
	}

	TSet<UObject*> ContextObjects;
	GetContextObjects(OutlinerItemsContext, ContextObjects);

	if (ContextObjects.IsEmpty())
	{
		return;
	}

	const FCEEditorEffectorMenuContext Context(ContextObjects);
	FCEEditorEffectorMenuOptions Options(
		{
			ECEEditorEffectorMenuType::Enable
			, ECEEditorEffectorMenuType::Disable
		}
	);
	Options.UseTransact(true);
	Options.CreateSubMenu(true);

	EffectorSubsystem->FillEffectorMenu(InToolMenu, Context, Options);
}

void FAvaEffectorEditorOutlinerContextMenu::GetContextObjects(const UAvaOutlinerItemsContext* InContext, TSet<UObject*>& OutObjects)
{
	OutObjects.Empty();

	if (!IsValid(InContext) || InContext->GetItems().IsEmpty())
	{
		return;
	}

	for (const FAvaOutlinerItemWeakPtr& ItemWeak : InContext->GetItems())
	{
		const TSharedPtr<IAvaOutlinerItem> Item = ItemWeak.Pin();

		// is it an actor
		if (const FAvaOutlinerActor* ActorItem = Item->CastTo<FAvaOutlinerActor>())
		{
			if (AActor* Actor = ActorItem->GetActor())
			{
				if (Actor->FindComponentByClass<UCEEffectorComponent>())
				{
					OutObjects.Add(Actor);
				}
			}
		}
		// is it a component
		else if (const FAvaOutlinerComponent* ComponentItem = Item->CastTo<FAvaOutlinerComponent>())
		{
			if (USceneComponent* Component = ComponentItem->GetComponent())
			{
				if (Component->IsA<UCEEffectorComponent>())
				{
					OutObjects.Add(Component);
				}
			}
		}
	}
}
