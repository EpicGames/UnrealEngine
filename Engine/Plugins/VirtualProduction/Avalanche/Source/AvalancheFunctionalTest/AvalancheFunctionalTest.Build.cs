// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheFunctionalTest : ModuleRules
{
	public AvalancheFunctionalTest(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Avalanche",
				"Core",
				"CoreUObject",
				"Engine",
				"FunctionalTesting",
				"Niagara",
				"RenderCore",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Slate",
				"SlateCore",
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"UnrealEd",
					"LevelEditor",
				}
			);
		}
	}
}
