// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScreenshotFunctionalTestAvalanche.h"
#include "DynamicResolutionState.h"
#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "Tests/AutomationCommon.h"
#include "UnrealClient.h"
#include "NiagaraComponent.h"
#include "NiagaraSystem.h"
#include "NiagaraSystemInstanceController.h"
#include "Components/PrimitiveComponent.h"
#include "UObject/UObjectIterator.h"

DEFINE_LOG_CATEGORY_STATIC(LogScreenshotAvalanche, Log, All);

AScreenshotFunctionalTestAvalanche::AScreenshotFunctionalTestAvalanche(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, AntiAliasingOverride(TEXT("r.AntiAliasingMethod"))
	, AutoExposureOverride(TEXT("r.DefaultFeature.AutoExposure"))
	, MotionBlurOverride(TEXT("r.DefaultFeature.MotionBlur"))
	, MotionBlurQualityOverride(TEXT("r.MotionBlurQuality"))
	, ContactShadowsOverride(TEXT("r.ContactShadows"))
	, ScreenSpaceReflectionQualityOverride(TEXT("r.SSR.Quality"))
	, EyeAdaptationQualityOverride(TEXT("r.EyeAdaptationQuality"))
	, TonemapperGammaOverride(TEXT("r.TonemapperGamma"))
	, ScreenPercentageOverride(TEXT("r.ScreenPercentage"))
	, DynamicResTestScreenPercentageOverride(TEXT("r.DynamicRes.TestScreenPercentage"))
	, DynamicResOperationModeOverride(TEXT("r.DynamicRes.OperationMode"))
	, SecondaryScreenPercentageOverride(TEXT("r.SecondaryScreenPercentage.GameViewport"))
{
}

void AScreenshotFunctionalTestAvalanche::PrepareTest()
{
	// Apply Avalanche-compatible CVar overrides BEFORE base class setup
	// This prevents priority conflicts with Avalanche's custom viewport
	ApplyAvalancheRenderingOverrides();

	// Record start time for actor readiness timeout
	ActorWaitStartTime = GetWorld()->GetTimeSeconds();

	// Now call base class PrepareTest which sets up the entire screenshot test
	Super::PrepareTest();
}

void AScreenshotFunctionalTestAvalanche::RequestScreenshot()
{
	// Call base class to register instance delegate callback and flush rendering
	Super::RequestScreenshot();

	// CRITICAL FIX for PIE: Also register to the static delegate used by FEditorViewportClient
	// PIE uses FEditorViewportClient::ProcessScreenShots() which fires FScreenshotRequest::OnScreenshotCaptured() (static)
	// instead of UGameViewportClient::OnScreenshotCaptured() (instance) that base class registers to
	FScreenshotRequest::OnScreenshotCaptured().AddUObject(this, &AScreenshotFunctionalTestAvalanche::OnScreenShotCaptured);
}

void AScreenshotFunctionalTestAvalanche::OnScreenShotCaptured(int32 InSizeX, int32 InSizeY, const TArray<FColor>& InImageData)
{
	// Remove callback from BOTH delegates (instance and static)
	UGameViewportClient* GameViewportClient = AutomationCommon::GetAnyGameViewportClient();
	if (GameViewportClient)
	{
		GameViewportClient->OnScreenshotCaptured().RemoveAll(this);
	}
	FScreenshotRequest::OnScreenshotCaptured().RemoveAll(this);

	// Call parent implementation which handles the screenshot capture and comparison pipeline
	Super::OnScreenShotCaptured(InSizeX, InSizeY, InImageData);
}

void AScreenshotFunctionalTestAvalanche::ApplyAvalancheRenderingOverrides()
{
	// Check if we should disable noisy rendering features (same behavior as base class)
	if (ScreenshotOptions.bDisableNoisyRenderingFeatures)
	{
		// Disable anti-aliasing for deterministic rendering
		AntiAliasingOverride.SetOverride(0);

		// Disable auto-exposure
		AutoExposureOverride.SetOverride(0);

		// Disable motion blur
		MotionBlurOverride.SetOverride(0);
		MotionBlurQualityOverride.SetOverride(0);

		// Disable screen space reflections
		ScreenSpaceReflectionQualityOverride.SetOverride(0);

		// Disable contact shadows
		ContactShadowsOverride.SetOverride(0);

		// Disable eye adaptation
		EyeAdaptationQualityOverride.SetOverride(0);

		// Set fixed tonemapper gamma
		TonemapperGammaOverride.SetOverride(2.2f);
	}
	else if (ScreenshotOptions.bDisableTonemapping)
	{
		// Only disable eye adaptation and set gamma
		EyeAdaptationQualityOverride.SetOverride(0);
		TonemapperGammaOverride.SetOverride(2.2f);
	}

	// Force screen percentage to 100% for consistent results
	{
		// Completely disable dynamic resolution
		DynamicResTestScreenPercentageOverride.SetOverride(0);
		DynamicResOperationModeOverride.SetOverride(0);

		// Emit dynamic resolution state change events
		if (GEngine)
		{
			GEngine->EmitDynamicResolutionEvent(EDynamicResolutionStateEvent::EndFrame);
			GEngine->EmitDynamicResolutionEvent(EDynamicResolutionStateEvent::BeginFrame);
		}

		// Set screen percentage to 100%
		ScreenPercentageOverride.SetOverride(100.0f);
	}

	// Ignore High-DPI settings
	SecondaryScreenPercentageOverride.SetOverride(100.0f);
}

bool AScreenshotFunctionalTestAvalanche::IsReady_Implementation()
{
	// If actor readiness check is disabled, use base class behavior (fixed delay)
	if (!bWaitForActorReady)
	{
		return Super::IsReady_Implementation();
	}

	// First check if base class timing requirements are met
	if (!Super::IsReady_Implementation())
	{
		return false;
	}

	// Check if we've exceeded max wait time
	const float CurrentTime = GetWorld()->GetTimeSeconds();
	const float ElapsedWaitTime = CurrentTime - ActorWaitStartTime;

	if (ElapsedWaitTime > MaxActorWaitTime)
	{
		UE_LOGF(LogScreenshotAvalanche, Warning,
			"Screenshot test '%ls' timed out waiting for actor to be ready after %.2f seconds. Proceeding with screenshot anyway.",
			*GetName(), ElapsedWaitTime);
		return true; // Proceed anyway to avoid hanging
	}

	// Resolve the actor reference
	AActor* Actor = ActorToWaitFor.Get();
	if (!Actor)
	{
		// Try to load the actor if it's a soft reference that hasn't been resolved yet
		Actor = ActorToWaitFor.LoadSynchronous();
	}

	if (!Actor)
	{
		UE_LOGF(LogScreenshotAvalanche, Warning,
			"Screenshot test '%ls' has bWaitForActorReady enabled but ActorToWaitFor is not set or couldn't be resolved. Proceeding with screenshot.",
			*GetName());
		return true; // Proceed if actor is not set
	}

	// Check if the actor is fully initialized
	if (!IsActorFullyInitialized(Actor))
	{
		UE_LOGF(LogScreenshotAvalanche, Verbose,
			"Screenshot test '%ls' waiting for actor '%ls' to be fully initialized (elapsed: %.2f / %.2f seconds)",
			*GetName(), *Actor->GetName(), ElapsedWaitTime, MaxActorWaitTime);
		return false;
	}

	UE_LOGF(LogScreenshotAvalanche, Log,
		"Screenshot test '%ls' confirmed actor '%ls' is ready after %.2f seconds",
		*GetName(), *Actor->GetName(), ElapsedWaitTime);

	return true;
}

bool AScreenshotFunctionalTestAvalanche::IsActorFullyInitialized(AActor* Actor) const
{
	if (!IsValid(Actor))
	{
		return true; // Skip invalid actors
	}

	// Check if actor has completed BeginPlay
	if (!Actor->HasActorBegunPlay())
	{
		UE_LOGF(LogScreenshotAvalanche, VeryVerbose,
			"Actor '%ls' has not completed BeginPlay yet", *Actor->GetName());
		return false;
	}

	// Check all Niagara components are initialized and active
	TArray<UNiagaraComponent*> NiagaraComponents;
	Actor->GetComponents<UNiagaraComponent>(NiagaraComponents);

	for (UNiagaraComponent* NiagaraComp : NiagaraComponents)
	{
		if (!IsValid(NiagaraComp))
		{
			continue;
		}

		// Check if Niagara system is active
		if (!NiagaraComp->IsActive())
		{
			UE_LOGF(LogScreenshotAvalanche, VeryVerbose,
				"Actor '%ls': Niagara component '%ls' is not active yet",
				*Actor->GetName(), *NiagaraComp->GetName());
			return false;
		}

		// Check if system instance controller is valid
		FNiagaraSystemInstanceControllerConstPtr SystemInstanceController = NiagaraComp->GetSystemInstanceController();
		if (!SystemInstanceController.IsValid())
		{
			UE_LOGF(LogScreenshotAvalanche, VeryVerbose,
				"Actor '%ls': Niagara component '%ls' does not have a valid system instance controller yet",
				*Actor->GetName(), *NiagaraComp->GetName());
			return false;
		}

		// Wait for system to have at least one tick to spawn particles
		// Some systems might need a frame or two to initialize properly
		const float SystemAge = SystemInstanceController->GetAge();
		if (SystemAge < 0.1f)
		{
			UE_LOGF(LogScreenshotAvalanche, VeryVerbose,
				"Actor '%ls': Niagara component '%ls' system is too young (age: %.3f, waiting for at least 0.1s)",
				*Actor->GetName(), *NiagaraComp->GetName(), SystemAge);
			return false;
		}
	}

	// Check all primitive components are registered
	TArray<UPrimitiveComponent*> PrimitiveComponents;
	Actor->GetComponents<UPrimitiveComponent>(PrimitiveComponents);

	for (UPrimitiveComponent* PrimComp : PrimitiveComponents)
	{
		if (IsValid(PrimComp) && !PrimComp->IsRegistered())
		{
			UE_LOGF(LogScreenshotAvalanche, VeryVerbose,
				"Actor '%ls': Primitive component '%ls' is not registered yet",
				*Actor->GetName(), *PrimComp->GetName());
			return false;
		}
	}

	return true;
}

void AScreenshotFunctionalTestAvalanche::FinishTest(EFunctionalTestResult TestResult, const FString& Message)
{
	// Unroot any systems left over from a previous FinishTest call (safety measure).
	for (UNiagaraSystem* NiagaraSystem : RootedNiagaraSystems)
	{
		if (IsValid(NiagaraSystem) && NiagaraSystem->IsRooted())
		{
			NiagaraSystem->RemoveFromRoot();
		}
	}
	RootedNiagaraSystems.Reset();

	// Root all UNiagaraSystem assets referenced by NiagaraComponents in this world.
	// Niagara async compilation tasks (kicked off by scalability CVar changes during
	// PrepareTest) hold a TWeakObjectPtr<UNiagaraSystem>. If the Cloner actor is torn
	// down and GC runs before DigestSystemInfo() executes, that weak ptr becomes null
	// and the task crashes at EmitterHandles (offset 0x390). AddToRoot() prevents GC
	// from marking these assets pending-kill until BeginDestroy() cleans them up.
	if (UWorld* World = GetWorld())
	{
		for (TObjectIterator<UNiagaraComponent> It; It; ++It)
		{
			UNiagaraComponent* NiagaraComp = *It;
			if (!IsValid(NiagaraComp) || NiagaraComp->GetWorld() != World)
			{
				continue;
			}

			if (UNiagaraSystem* NiagaraSystem = NiagaraComp->GetAsset())
			{
				if (!NiagaraSystem->IsRooted())
				{
					NiagaraSystem->AddToRoot();
					RootedNiagaraSystems.Add(NiagaraSystem);
					UE_LOGF(LogScreenshotAvalanche, Log,
						"FinishTest '%ls': Rooted NiagaraSystem '%ls' to outlive async compilation tasks",
						*GetName(), *NiagaraSystem->GetName());
				}
			}
		}

		UE_LOGF(LogScreenshotAvalanche, Log,
			"FinishTest '%ls': Rooted %d NiagaraSystem(s) to survive async compilation during teardown (result=%d)",
			*GetName(), RootedNiagaraSystems.Num(), (int32)TestResult);
	}

	Super::FinishTest(TestResult, Message);
}

void AScreenshotFunctionalTestAvalanche::BeginDestroy()
{
	// Unroot NiagaraSystems pinned in FinishTest. Releasing the root here is a
	// best-effort cleanup; FinishTest pins assets to extend their lifetime beyond
	// teardown, but BeginDestroy does not wait for in-flight compilation tasks to
	// complete. If tasks are still running when this fires, they hold
	// TWeakObjectPtr references that will become invalid on the next GC cycle.
	for (UNiagaraSystem* NiagaraSystem : RootedNiagaraSystems)
	{
		if (IsValid(NiagaraSystem) && NiagaraSystem->IsRooted())
		{
			NiagaraSystem->RemoveFromRoot();
			UE_LOGF(LogScreenshotAvalanche, Log,
				"BeginDestroy '%ls': Unrooted NiagaraSystem '%ls'",
				*GetName(), *NiagaraSystem->GetName());
		}
	}
	RootedNiagaraSystems.Reset();

	Super::BeginDestroy();
}
