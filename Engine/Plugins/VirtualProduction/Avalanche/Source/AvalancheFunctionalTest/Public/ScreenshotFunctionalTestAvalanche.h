// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/IConsoleManager.h"
#include "ScreenshotFunctionalTest.h"
#include "ScreenshotFunctionalTestAvalanche.generated.h"

class UNiagaraSystem;

/**
 * Screenshot functional test optimized for Avalanche (Motion Design) levels.
 *
 * This class works around two critical issues that prevent regular screenshot tests
 * from working correctly in Avalanche/Motion Design levels:
 *
 * 1. CVar Priority Conflicts:
 *    - Avalanche's custom viewport client (UAvaGameViewportClient) sets rendering CVars
 *      at Constructor/Code priority levels
 *    - Base AScreenshotFunctionalTest uses SetWithCurrentPriority() which cannot override
 *      Constructor-level CVars, causing 12 CVars to fail with priority errors
 *    - This class uses ECVF_SetByConsole priority to successfully override during test
 *
 * 2. PIE Screenshot Delegate Mismatch:
 *    - PIE uses FEditorViewportClient::ProcessScreenShots() which fires the static
 *      FScreenshotRequest::OnScreenshotCaptured() delegate
 *    - Base class only registers to the instance UGameViewportClient::OnScreenshotCaptured()
 *    - This class registers to BOTH delegates to support PIE and standalone execution
 *
 * Usage:
 *    Use this class instead of AScreenshotFunctionalTest when creating screenshot tests
 *    for Avalanche/Motion Design levels. It's only available when the Avalanche plugin
 *    is enabled.
 *
 * Note:
 *    CVars are NOT restored after test completion to avoid priority stickiness issues.
 *    Test-friendly values (AA=0, ScreenPercentage=100, MotionBlur=0) remain in place.
 */
UCLASS(Blueprintable, MinimalAPI, DisplayName = "Screenshot Functional Test Motion Design")
class AScreenshotFunctionalTestAvalanche : public AScreenshotFunctionalTest
{
	GENERATED_BODY()

public:
	AVALANCHEFUNCTIONALTEST_API AScreenshotFunctionalTestAvalanche(const FObjectInitializer& ObjectInitializer);

	/**
	 * If enabled, the test will wait for the specified actor to be fully initialized
	 * before taking the screenshot, instead of relying on fixed time/frame delays.
	 * This ensures actors with Niagara systems, async loading, or complex initialization
	 * are fully ready before screenshot capture.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Screenshot|Actor Readiness", meta = (DisplayName = "Wait For Actor Ready"))
	bool bWaitForActorReady = false;

	/**
	 * The actor to wait for before taking the screenshot.
	 * The test will check if this actor and all its components (especially Niagara systems)
	 * are fully initialized before proceeding.
	 * Only used when bWaitForActorReady is enabled.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Screenshot|Actor Readiness", meta = (EditCondition = "bWaitForActorReady", DisplayName = "Actor To Wait For"))
	TSoftObjectPtr<AActor> ActorToWaitFor;

	/**
	 * Maximum time to wait for the actor to be ready (in seconds).
	 * If the actor is not ready within this time, the test will proceed anyway
	 * to avoid hanging indefinitely.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Screenshot|Actor Readiness", meta = (EditCondition = "bWaitForActorReady", ClampMin = "0.1", ClampMax = "30.0", UIMin = "0.5", UIMax = "10.0"))
	float MaxActorWaitTime = 5.0f;

protected:
	// Override to set up CVars with explicit priority before base class setup
	AVALANCHEFUNCTIONALTEST_API virtual void PrepareTest() override;

	// Override to check if actor is ready before proceeding
	AVALANCHEFUNCTIONALTEST_API virtual bool IsReady_Implementation() override;

	// Override to register to both instance and static screenshot delegates for PIE support
	AVALANCHEFUNCTIONALTEST_API virtual void RequestScreenshot() override;
	AVALANCHEFUNCTIONALTEST_API virtual void OnScreenShotCaptured(int32 InSizeX, int32 InSizeY, const TArray<FColor>& InImageData) override;

	// Override to root NiagaraSystem assets before test teardown, preventing GC from
	// freeing them while async Niagara compilation tasks still hold TWeakObjectPtr refs.
	AVALANCHEFUNCTIONALTEST_API virtual void FinishTest(EFunctionalTestResult TestResult, const FString& Message) override;

	// Override to unroot NiagaraSystem assets once the test actor is being destroyed.
	AVALANCHEFUNCTIONALTEST_API virtual void BeginDestroy() override;

private:
	/**
	 * Checks if the specified actor is fully initialized and ready for screenshot.
	 * Returns true if:
	 * - Actor has completed BeginPlay
	 * - All Niagara components are active and have valid system instances
	 * - All primitive components are registered
	 * - Niagara systems have had time to spawn particles (at least 0.1s age)
	 */
	bool IsActorFullyInitialized(AActor* Actor) const;

	// Time when PrepareTest() was called, used to enforce MaxActorWaitTime
	float ActorWaitStartTime = 0.0f;

	// NiagaraSystem assets temporarily rooted in FinishTest() to outlive async
	// compilation tasks that may fire during test teardown. Unrooted in BeginDestroy().
	TArray<TObjectPtr<UNiagaraSystem>> RootedNiagaraSystems;

private:
	/**
	 * Helper struct to manage CVar overrides at Console priority.
	 * Note: CVars are NOT restored after test to avoid priority stickiness issues.
	 * Test-friendly values (AA=0, ScreenPercentage=100) are left in place.
	 */
	template<typename T>
	struct FScreenshotTestAvalancheCVarOverride
	{
		FScreenshotTestAvalancheCVarOverride() = default;

		FScreenshotTestAvalancheCVarOverride(const FString& InCVarName)
			: CVarName(InCVarName)
		{
		}

		void SetOverride(T NewValue)
		{
			IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(*CVarName);
			if (CVar)
			{
				// Use Console priority to override Scalability/Constructor/Code settings
				// High enough to actually take effect, but won't be restored to avoid
				// priority stickiness (Console priority would prevent lower priorities from working)
				CVar->Set(NewValue, ECVF_SetByConsole);
			}
		}

	private:
		FString CVarName;
	};

	// CVar overrides managed with explicit priority
	FScreenshotTestAvalancheCVarOverride<int32> AntiAliasingOverride;
	FScreenshotTestAvalancheCVarOverride<int32> AutoExposureOverride;
	FScreenshotTestAvalancheCVarOverride<int32> MotionBlurOverride;
	FScreenshotTestAvalancheCVarOverride<int32> MotionBlurQualityOverride;
	FScreenshotTestAvalancheCVarOverride<int32> ContactShadowsOverride;
	FScreenshotTestAvalancheCVarOverride<int32> ScreenSpaceReflectionQualityOverride;
	FScreenshotTestAvalancheCVarOverride<int32> EyeAdaptationQualityOverride;
	FScreenshotTestAvalancheCVarOverride<float> TonemapperGammaOverride;
	FScreenshotTestAvalancheCVarOverride<float> ScreenPercentageOverride;
	FScreenshotTestAvalancheCVarOverride<int32> DynamicResTestScreenPercentageOverride;
	FScreenshotTestAvalancheCVarOverride<int32> DynamicResOperationModeOverride;
	FScreenshotTestAvalancheCVarOverride<float> SecondaryScreenPercentageOverride;

	// Apply Avalanche-compatible rendering overrides
	void ApplyAvalancheRenderingOverrides();
};
