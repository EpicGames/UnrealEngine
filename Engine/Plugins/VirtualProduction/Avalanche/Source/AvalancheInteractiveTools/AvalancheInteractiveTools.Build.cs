// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheInteractiveTools : ModuleRules
{
	public AvalancheInteractiveTools(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Avalanche",
				"AvalancheInteractiveToolsRuntime",
				"AvalancheViewport",
				"AvalancheText",
				"DeveloperSettings",
				"EditorFramework",
				"EditorInteractiveToolsFramework",
				"InputCore",
				"InteractiveToolsFramework",
				"PlacementMode",
				"Projects",
				"ToolMenus",
				"TypedElementRuntime",
				"UnrealEd",
				"WidgetRegistration"
			}
		);

		ShortName = "AvIntTools";
	}
}
