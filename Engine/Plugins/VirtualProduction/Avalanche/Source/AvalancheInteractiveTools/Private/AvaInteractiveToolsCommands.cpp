// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaInteractiveToolsCommands.h"
#include "AvaInteractiveToolsStyle.h"

#define LOCTEXT_NAMESPACE "AvaInteractiveToolsCommands"

const FAvaInteractiveToolsCommands* FAvaInteractiveToolsCommands::GetExternal()
{
	if (IsRegistered())
	{
		return &FAvaInteractiveToolsCommands::Get();
	}

	return nullptr;
}

FAvaInteractiveToolsCommands::FAvaInteractiveToolsCommands()
	: TCommands<FAvaInteractiveToolsCommands>(
		TEXT("AvaInteractiveTools")
		, LOCTEXT("MotionDesignInteractiveTools", "Motion Design Interactive Tools")
		, NAME_None
		, FAvaInteractiveToolsStyle::Get().GetStyleSetName()
	)
{
}

void FAvaInteractiveToolsCommands::RegisterCommands()
{
	RegisterEdModeCommands();
	RegisterCategoryCommands();
	RegisterActorCommands();
}

void FAvaInteractiveToolsCommands::RegisterEdModeCommands()
{
	UI_COMMAND(CancelActiveTool
		, "Cancel"
		, "Cancel the active Tool"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Escape)
	);

	UI_COMMAND(ToggleViewportToolbar
		, "Toggle Toolbar"
		, "Toggle viewport overlay toolbar visibility"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord(EModifierKey::Control, EKeys::T)
	);
	
	UI_COMMAND(OpenViewportToolbarSettings
		, "Toolbar Settings"
		, "Opens editor toolbar settings"
		, EUserInterfaceActionType::Button
		, FInputChord()
	);
}

void FAvaInteractiveToolsCommands::RegisterCategoryCommands()
{
	UI_COMMAND(Category_2D
		, "2D Shapes"
		, "Tools for creating 2D shapes in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Category_3D
		, "3D Shapes"
		, "Tools for creating 3D shapes in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Category_Actor
		, "Actors"
		, "Tools for creating Actors in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());
	
	UI_COMMAND(Category_Cloner
		, "Cloners"
		, "Tools for creating cloners in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Category_Effector
		, "Effectors"
		, "Tools for creating effectors in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());
}

void FAvaInteractiveToolsCommands::RegisterActorCommands()
{
	UI_COMMAND(Tool_Actor_Null
		, "Null Actor"
		, "Create a Null Actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Tool_Actor_Spline
		, "Spline Actor"
		, "Create a Spline Actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());
}

#undef LOCTEXT_NAMESPACE
