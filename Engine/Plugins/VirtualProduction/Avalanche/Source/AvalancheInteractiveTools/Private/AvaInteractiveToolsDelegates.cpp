// Copyright Epic Games, Inc.All Rights Reserved.

#include "AvaInteractiveToolsDelegates.h"

FAvaInteractiveToolsRegisterToolsDelegate& FAvaInteractiveToolsDelegates::GetRegisterToolsDelegate()
{
	static FAvaInteractiveToolsRegisterToolsDelegate Delegate;
	return Delegate;
}

FAvaInteractiveToolsRegisterCategoriesDelegate& FAvaInteractiveToolsDelegates::GetRegisterCategoriesDelegate()
{
	static FAvaInteractiveToolsRegisterCategoriesDelegate Delegate;
	return Delegate;
}
