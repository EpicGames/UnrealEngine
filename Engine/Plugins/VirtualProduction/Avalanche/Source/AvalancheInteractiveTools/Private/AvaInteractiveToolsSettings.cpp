// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaInteractiveToolsSettings.h"

#include "EditorModeManager.h"
#include "EdMode/AvaInteractiveToolsEdModeToolkit.h"
#include "IAvalancheInteractiveToolsModule.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"

bool UAvaInteractiveToolsSettings::IsViewportToolbarProperty(FName InPropertyName)
{
	return InPropertyName == GET_MEMBER_NAME_CHECKED(UAvaInteractiveToolsSettings, ViewportToolbarPosition)
		|| InPropertyName == GET_MEMBER_NAME_CHECKED(UAvaInteractiveToolsSettings, bViewportToolbarLabelEnabled);
}

UAvaInteractiveToolsSettings* UAvaInteractiveToolsSettings::Get()
{
	return GetMutableDefault<UAvaInteractiveToolsSettings>();
}

UAvaInteractiveToolsSettings::UAvaInteractiveToolsSettings()
{
	CategoryName = TEXT("Motion Design");
	SectionName = TEXT("Interactive Tools");
}

void UAvaInteractiveToolsSettings::OpenEditorSettingsWindow() const
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>(TEXT("Settings")))
	{
		SettingsModule->ShowViewer(GetContainerName(), GetCategoryName(), GetSectionName());
	}
}

bool UAvaInteractiveToolsSettings::IsViewportToolbarSupported() const
{
	const FEditorModeTools& EditorModeTools = GLevelEditorModeTools();
	return EditorModeTools.IsModeActive(IAvalancheInteractiveToolsModule::EM_AvaInteractiveToolsEdModeId);
}

void UAvaInteractiveToolsSettings::SetViewportToolbarVisible(bool bInVisible) const
{
	const FEditorModeTools& EditorModeTools = GLevelEditorModeTools();
	if (UEdMode* ITEdMode = EditorModeTools.GetActiveScriptableMode(IAvalancheInteractiveToolsModule::EM_AvaInteractiveToolsEdModeId))
	{
		if (const TSharedPtr<FModeToolkit> ModeToolkit = ITEdMode->GetToolkit().Pin())
		{
			FAvaInteractiveToolsEdModeToolkit* ITMode = static_cast<FAvaInteractiveToolsEdModeToolkit*>(ModeToolkit.Get());
			ITMode->SetViewportToolbarVisibility(bInVisible);
		}
	}
}

bool UAvaInteractiveToolsSettings::GetViewportToolbarVisible() const
{
	const FEditorModeTools& EditorModeTools = GLevelEditorModeTools();
	if (UEdMode* ITEdMode = EditorModeTools.GetActiveScriptableMode(IAvalancheInteractiveToolsModule::EM_AvaInteractiveToolsEdModeId))
	{
		if (const TSharedPtr<FModeToolkit> ModeToolkit = ITEdMode->GetToolkit().Pin())
		{
			FAvaInteractiveToolsEdModeToolkit* ITMode = static_cast<FAvaInteractiveToolsEdModeToolkit*>(ModeToolkit.Get());
			return ITMode->GetViewportToolbarVisibility();
		}
	}

	return false;
}
