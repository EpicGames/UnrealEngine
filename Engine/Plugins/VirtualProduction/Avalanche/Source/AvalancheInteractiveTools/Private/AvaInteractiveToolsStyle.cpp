// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaInteractiveToolsStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Containers/StringFwd.h"
#include "Interfaces/IPluginManager.h"
#include "Math/MathFwd.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/ToolBarStyle.h"

FAvaInteractiveToolsStyle::FAvaInteractiveToolsStyle()
	: FSlateStyleSet(TEXT("AvaInteractiveTools"))
{
	const FVector2f Icon16x16(16.0f, 16.0f);

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	SetContentRoot(Plugin->GetBaseDir() / TEXT("Resources"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));

	const FTextBlockStyle TextStyle = FTextBlockStyle()
		.SetFont(FCoreStyle::GetDefaultFontStyle("Regular", 6))
		.SetColorAndOpacity(FLinearColor(0.4f, 0.4f, 0.4f))
		.SetOverflowPolicy(ETextOverflowPolicy::Clip);

	const FMargin Margin(0.5f);
	FToolBarStyle ToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("EditorViewportToolBar");
	ToolbarStyle
	    .SetComboLabelMaxWidth(38.f)
		.SetComboLabelMinWidth(38.f)
		.SetButtonContentMaxWidth(38.f)
		.SetAllowWrapButton(true)
		.SetIconSize(Icon16x16)
		.SetButtonPadding(Margin)
		.SetIconPadding(Margin)
		.SetLabelPadding(Margin)
		.SetLabelStyle(TextStyle)
		.SetBlockPadding(Margin)
		.SetBackgroundPadding(Margin)
		.SetCheckBoxPadding(Margin)
		.SetBackground(FSlateColorBrush(FStyleColors::Transparent));

	Set("ViewportToolbar", ToolbarStyle);

	Set("AvaInteractiveTools.ToggleViewportToolbar", new IMAGE_BRUSH_SVG("Icons/EditorIcons/visible", Icon16x16));
	Set("AvaInteractiveTools.OpenViewportToolbarSettings", new IMAGE_BRUSH_SVG("Icons/EditorIcons/settings", Icon16x16));
	Set("AvaInteractiveTools.Dropdown", new IMAGE_BRUSH_SVG("Icons/EditorIcons/chevron-down", Icon16x16));

	Set("Icons.Toolbox", new IMAGE_BRUSH(TEXT("Icons/ToolboxIcons/toolbox"), Icon16x16));

	// Categories
	Set("AvaInteractiveTools.Category_2D",       new IMAGE_BRUSH("Icons/ToolboxIcons/rectangle", Icon16x16));
	Set("AvaInteractiveTools.Category_3D",       new IMAGE_BRUSH("Icons/ToolboxIcons/cube", Icon16x16));
	Set("AvaInteractiveTools.Category_Actor",    new CORE_IMAGE_BRUSH(TEXT("Icons/SequencerIcons/icon_Sequencer_Move_24x"), Icon16x16));
	Set("AvaInteractiveTools.Category_Cloner",   new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/cloner", Icon16x16));
	Set("AvaInteractiveTools.Category_Effector", new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/effector", Icon16x16));
	Set("AvaInteractiveTools.Category_Layout",   new IMAGE_BRUSH("Icons/ToolboxIcons/layoutgrid", Icon16x16));

	// Actor Tools
	Set("AvaInteractiveTools.Tool_Actor_Null",   new CORE_IMAGE_BRUSH(TEXT("Icons/SequencerIcons/icon_Sequencer_Move_24x"), Icon16x16));
	Set("AvaInteractiveTools.Tool_Actor_Spline", new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/Toolbar_Spline", Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FAvaInteractiveToolsStyle::~FAvaInteractiveToolsStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
