// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FAvaInteractiveToolsStyle final : public FSlateStyleSet
{
public:
	static FAvaInteractiveToolsStyle& Get()
	{
		static FAvaInteractiveToolsStyle Instance;
		return Instance;
	}

	FAvaInteractiveToolsStyle();
	virtual ~FAvaInteractiveToolsStyle() override;
};
