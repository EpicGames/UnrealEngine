// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvalancheInteractiveToolsModule.h"
#include "AvaInteractiveToolsCommands.h"
#include "AvaInteractiveToolsDelegates.h"
#include "Builders/AvaInteractiveToolsActorToolBuilder.h"
#include "Interfaces/IPluginManager.h"
#include "IPlacementModeModule.h"
#include "Styling/SlateIconFinder.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "AvalancheInteractiveTools"

DEFINE_LOG_CATEGORY(LogAvaInteractiveTools);

namespace UE::AvaInteractiveTools::Private
{
	bool bInitialRegistration = false;
}

void FAvalancheInteractiveToolsModule::StartupModule()
{
	FAvaInteractiveToolsCommands::Register();

	if (IPluginManager::Get().GetLastCompletedLoadingPhase() >= ELoadingPhase::PostEngineInit)
	{
		OnPostEngineInit();
	}
	else
	{
		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FAvalancheInteractiveToolsModule::OnPostEngineInit);
	}
}

void FAvalancheInteractiveToolsModule::ShutdownModule()
{
	FAvaInteractiveToolsCommands::Unregister();
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
}

void FAvalancheInteractiveToolsModule::RegisterCategory(FName InCategoryName, TSharedPtr<FUICommandInfo> InCategoryCommand,
	int32 InPlacementModeSortPriority)
{
	if (!InCategoryCommand.IsValid())
	{
		return;
	}

	if (Categories.Contains(InCategoryName))
	{
		return;
	}

	Categories.Add(InCategoryName, InCategoryCommand);
	Tools.Add(InCategoryName, {});

	if (InPlacementModeSortPriority != NoPlacementCategory)
	{
		IPlacementModeModule& PlacementMode = IPlacementModeModule::Get();

		if (!PlacementMode.GetRegisteredPlacementCategory(InCategoryName))
		{
			static const FText LabelFormat = LOCTEXT("LabelFormat", "Motion Design {0}");

			const FPlacementCategoryInfo PlacementCategory = FPlacementCategoryInfo(
				FText::Format(LabelFormat, InCategoryCommand->GetLabel()),
				InCategoryCommand->GetIcon(),
				InCategoryName,
				InCategoryCommand->GetCommandName().ToString(),
				InPlacementModeSortPriority
			);

			PlacementMode.RegisterPlacementCategory(PlacementCategory);
		}
	}
}

void FAvalancheInteractiveToolsModule::RegisterTool(FName InCategory, FAvaInteractiveToolsToolParameters&& InToolParams)
{
	if (!Categories.Contains(InCategory))
	{
		return;
	}

	if (!Tools.Contains(InCategory))
	{
		Tools.Add(InCategory, {});
	}

	const bool bToolAdded = Tools[InCategory].ContainsByPredicate(
		[&InToolParams](const FAvaInteractiveToolsToolParameters& InTool)
		{
			return InTool.ToolIdentifier.Equals(InToolParams.ToolIdentifier);
		}
	);

	if (bToolAdded)
	{
		return;
	}

	Tools[InCategory].Add(MoveTemp(InToolParams));

	using namespace UE::AvaInteractiveTools::Private;

	if (!bInitialRegistration)
	{
		Tools[InCategory].StableSort([](const FAvaInteractiveToolsToolParameters& InA, const FAvaInteractiveToolsToolParameters& InB)
			{
				return InA.Priority < InB.Priority;
			});

		IPlacementModeModule::Get().RegenerateItemsForCategory(InCategory);
	}
}

const TMap<FName, TSharedPtr<FUICommandInfo>>& FAvalancheInteractiveToolsModule::GetCategories()
{
	return Categories;
}

const TArray<FAvaInteractiveToolsToolParameters>* FAvalancheInteractiveToolsModule::GetTools(FName InCategory)
{
	return Tools.Find(InCategory);
}

const FAvaInteractiveToolsToolParameters* FAvalancheInteractiveToolsModule::GetTool(const FString& InToolIdentifier)
{
	for (const TPair<FName, TArray<FAvaInteractiveToolsToolParameters>>& CategoryPair : Tools)
	{
		for (const FAvaInteractiveToolsToolParameters& ToolParams : CategoryPair.Value)
		{
			if (ToolParams.ToolIdentifier.Equals(InToolIdentifier))
			{
				return &ToolParams;
			}
		}
	}

	return nullptr;
}

FName FAvalancheInteractiveToolsModule::GetToolCategory(const FString& InToolIdentifier)
{
	for (const TPair<FName, TArray<FAvaInteractiveToolsToolParameters>>& CategoryPair : Tools)
	{
		for (const FAvaInteractiveToolsToolParameters& ToolParams : CategoryPair.Value)
		{
			if (ToolParams.ToolIdentifier.Equals(InToolIdentifier))
			{
				return CategoryPair.Key;
			}
		}
	}

	return NAME_None;
}

bool FAvalancheInteractiveToolsModule::HasActiveTool() const
{
	return ActiveToolIdentifier.IsSet();
}

void FAvalancheInteractiveToolsModule::AddReferencedObjects(FReferenceCollector& InCollector)
{
	for (TPair<FName, TArray<FAvaInteractiveToolsToolParameters>>& CategoryPair : Tools)
	{
		for (FAvaInteractiveToolsToolParameters& ToolParams : CategoryPair.Value)
		{
			if (ToolParams.ActorFactory.IsType<TObjectPtr<UActorFactory>>())
			{
				InCollector.AddReferencedObject<UActorFactory>(ToolParams.ActorFactory.Get<TObjectPtr<UActorFactory>>());
			}
		}
	}
}

FString FAvalancheInteractiveToolsModule::GetReferencerName() const
{
	static const FString ReferencerName = "AvaITFModule";
	return ReferencerName;
}

void FAvalancheInteractiveToolsModule::OnToolActivated(const FString& InToolIdentifier)
{
	ActiveToolIdentifier = InToolIdentifier;
	OnToolActivationDelegate.Broadcast(InToolIdentifier);
}

void FAvalancheInteractiveToolsModule::OnToolDeactivated()
{
	if (HasActiveTool())
	{
		OnToolDeactivationDelegate.Broadcast(ActiveToolIdentifier.GetValue());
		ActiveToolIdentifier.Reset();
	}
}

FString FAvalancheInteractiveToolsModule::GetActiveToolIdentifier() const
{
	return ActiveToolIdentifier.IsSet() ? ActiveToolIdentifier.GetValue() : FString();
}

void FAvalancheInteractiveToolsModule::OnPostEngineInit()
{
	using namespace UE::AvaInteractiveTools::Private;

	IPlacementModeModule& PlacementMode = IPlacementModeModule::Get();

	bInitialRegistration = true;
	BroadcastRegisterCategories();
	BroadcastRegisterTools();
	bInitialRegistration = false;

	for (TPair<FName, TArray<FAvaInteractiveToolsToolParameters>>& ToolPair : Tools)
	{
		ToolPair.Value.StableSort([](const FAvaInteractiveToolsToolParameters& InA, const FAvaInteractiveToolsToolParameters& InB)
			{
				return InA.Priority < InB.Priority;
			});
	}

	PlacementMode.OnPlacementModeCategoryRefreshed().AddRaw(
		this,
		&FAvalancheInteractiveToolsModule::OnPlacementCategoryRefreshed
	);

	for (const TPair<FName, TArray<FAvaInteractiveToolsToolParameters>>& Pair : Tools)
	{
		PlacementMode.RegenerateItemsForCategory(Pair.Key);
	}
}

void FAvalancheInteractiveToolsModule::BroadcastRegisterCategories()
{
	// Ensure that ours are first
	RegisterDefaultCategories();
	FAvaInteractiveToolsDelegates::GetRegisterCategoriesDelegate().Broadcast(this);
}

void FAvalancheInteractiveToolsModule::RegisterDefaultCategories()
{
	RegisterCategory(CategoryName2D, FAvaInteractiveToolsCommands::Get().Category_2D, 41);
	RegisterCategory(CategoryName3D, FAvaInteractiveToolsCommands::Get().Category_3D, 42);
	RegisterCategory(CategoryNameActor, FAvaInteractiveToolsCommands::Get().Category_Actor, 43);
	RegisterCategory(CategoryNameCloner, FAvaInteractiveToolsCommands::Get().Category_Cloner, 44);
	RegisterCategory(CategoryNameEffector, FAvaInteractiveToolsCommands::Get().Category_Effector, 45);
}

void FAvalancheInteractiveToolsModule::RegisterAutoRegisterTools()
{
	for (const UClass* const Class : TObjectRange<UClass>())
	{
		if (!Class->IsChildOf(UAvaInteractiveToolsToolBase::StaticClass())
			|| Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
		{
			continue;
		}

		UAvaInteractiveToolsToolBase* ToolCDO = Class->GetDefaultObject<UAvaInteractiveToolsToolBase>();

		if (!ToolCDO || !ToolCDO->ShouldAutoRegister())
		{
			continue;
		}

		ToolCDO->OnRegisterTool(this);

		UE_LOGF(LogAvaInteractiveTools, Log, "Tool %ls auto registered", *Class->GetName())
	}
}

void FAvalancheInteractiveToolsModule::BroadcastRegisterTools()
{
	RegisterAutoRegisterTools();
	FAvaInteractiveToolsDelegates::GetRegisterToolsDelegate().Broadcast(this);
}

void FAvalancheInteractiveToolsModule::OnPlacementCategoryRefreshed(FName InCategory)
{
	if (!Categories.Contains(InCategory))
	{
		return;
	}

	if (!Tools.Contains(InCategory))
	{
		return;
	}

	IPlacementModeModule& PlacementMode = IPlacementModeModule::Get();

	TArray<TSharedPtr<FPlaceableItem>> Items;
	PlacementMode.GetItemsForCategory(InCategory, Items);

	for (const FAvaInteractiveToolsToolParameters& Tool : Tools[InCategory])
	{
		const bool bHasActorClass = Tool.ActorFactory.IsType<TSubclassOf<AActor>>()
			&& Tool.ActorFactory.Get<TSubclassOf<AActor>>().Get() != nullptr;

		const bool bHasActorFactory = Tool.ActorFactory.IsType<TObjectPtr<UActorFactory>>()
			&& Tool.ActorFactory.Get<TObjectPtr<UActorFactory>>() != nullptr
			&& Tool.ActorFactory.Get<TObjectPtr<UActorFactory>>()->NewActorClass.Get() != nullptr;

		if (!bHasActorFactory && !bHasActorClass)
		{
			continue;
		}

		const bool bAlreadyRegistered = Items.ContainsByPredicate(
			[&Tool](const TSharedPtr<FPlaceableItem>& InItem)
			{
				return InItem.IsValid() && InItem->NativeName.Equals(Tool.ToolIdentifier);
			}
		);

		if (bAlreadyRegistered)
		{
			continue;
		}

		TSharedPtr<FPlaceableItem> PlaceableItem;

		if (bHasActorFactory)
		{
			UActorFactory* ActorFactory = Tool.ActorFactory.Get<TObjectPtr<UActorFactory>>();

			PlaceableItem = MakeShared<FPlaceableItem>(
				ActorFactory,
				FAssetData(ActorFactory->NewActorClass.GetDefaultObject()),
				Tool.Priority
			);
		}
		else
		{
			UClass* ActorClass = Tool.ActorFactory.Get<TSubclassOf<AActor>>();

			PlaceableItem = MakeShared<FPlaceableItem>(
				*ActorClass,
				FAssetData(ActorClass->GetDefaultObject()),
				NAME_None,
				NAME_None,
				TOptional<FLinearColor>(),
				Tool.Priority
			);
		}

		PlaceableItem->DisplayName = Tool.UICommand->GetLabel();
		PlaceableItem->NativeName = Tool.ToolIdentifier;

		const FName IconName(Tool.UICommand->GetBindingContext().ToString() + TEXT(".") + Tool.UICommand->GetCommandName().ToString());
		if (FSlateIconFinder::FindIcon(IconName).IsSet())
		{
			PlaceableItem->ClassThumbnailBrushOverride = IconName;
			PlaceableItem->bAlwaysUseGenericThumbnail = false;
		}
		else if (Tool.UICommand->GetIcon().IsSet())
		{
			PlaceableItem->ClassThumbnailBrushOverride = Tool.UICommand->GetIcon().GetStyleName();
			PlaceableItem->bAlwaysUseGenericThumbnail = false;
		}

		PlacementMode.RegisterPlaceableItem(InCategory, PlaceableItem.ToSharedRef());
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FAvalancheInteractiveToolsModule, AvalancheInteractiveTools)
