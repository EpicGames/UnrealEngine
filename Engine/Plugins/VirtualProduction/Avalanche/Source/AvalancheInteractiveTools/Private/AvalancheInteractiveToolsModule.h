// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvalancheInteractiveToolsModule.h"

class FUICommandInfo;

DECLARE_LOG_CATEGORY_EXTERN(LogAvaInteractiveTools, Log, All);

class FAvalancheInteractiveToolsModule : public IAvalancheInteractiveToolsModule, public FGCObject
{
public:
	static FAvalancheInteractiveToolsModule& Get()
	{
		static const FName ModuleName = TEXT("AvalancheInteractiveTools");
		return FModuleManager::LoadModuleChecked<FAvalancheInteractiveToolsModule>(ModuleName);
	}

	static FAvalancheInteractiveToolsModule* GetPtr()
	{
		static const FName ModuleName = TEXT("AvalancheInteractiveTools");
		return FModuleManager::GetModulePtr<FAvalancheInteractiveToolsModule>(ModuleName);
	}

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	//~ Begin IAvalancheInteractiveToolsModule
	virtual void RegisterCategory(FName InCategoryName, TSharedPtr<FUICommandInfo> InCategoryCommand, 
		int32 InPlacementModeSortPriority = NoPlacementCategory) override;
	virtual void RegisterTool(FName InCategory, FAvaInteractiveToolsToolParameters&& InToolParams) override;
	virtual const TMap<FName, TSharedPtr<FUICommandInfo>>& GetCategories() override;
	virtual const TArray<FAvaInteractiveToolsToolParameters>* GetTools(FName InCategory) override;
	virtual const FAvaInteractiveToolsToolParameters* GetTool(const FString& InToolIdentifier) override;
	virtual FName GetToolCategory(const FString& InToolIdentifier) override;
	virtual bool HasActiveTool() const override;
	//~ End IAvalancheInteractiveToolsModule

	// ~Begin FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	virtual FString GetReferencerName() const override;
	// ~End FGCObject Interface

	void OnToolActivated(const FString& InToolIdentifier);
	void OnToolDeactivated();
	FString GetActiveToolIdentifier() const;

	virtual FToolEvent::RegistrationType& OnToolActivation() override
	{
		return OnToolActivationDelegate;
	}

	virtual FToolEvent::RegistrationType& OnToolDeactivation() override
	{
		return OnToolDeactivationDelegate;
	}

private:
	TMap<FName, TSharedPtr<FUICommandInfo>> Categories;
	TMap<FName, TArray<FAvaInteractiveToolsToolParameters>> Tools;
	TOptional<FString> ActiveToolIdentifier;
	FToolEvent OnToolActivationDelegate;
	FToolEvent OnToolDeactivationDelegate;

	void OnPostEngineInit();

	void BroadcastRegisterCategories();
	void RegisterDefaultCategories();

	void RegisterAutoRegisterTools();
	void BroadcastRegisterTools();

	void OnPlacementCategoryRefreshed(FName InCategory);
};
