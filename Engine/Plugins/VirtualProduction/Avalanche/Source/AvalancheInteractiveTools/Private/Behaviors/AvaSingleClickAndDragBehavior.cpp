// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviors/AvaSingleClickAndDragBehavior.h"

UAvaSingleClickAndDragBehavior::UAvaSingleClickAndDragBehavior()
{
}

void UAvaSingleClickAndDragBehavior::Initialize(IAvaSingleClickAndDragBehaviorTarget* InTarget)
{
	check(InTarget != nullptr);
	Target = InTarget;
}

FInputCaptureRequest UAvaSingleClickAndDragBehavior::WantsCapture(const FInputDeviceState& InInput)
{
	bInClickDrag = false;	// should never be true here, but weird things can happen w/ focus

	if (IsPressed(InInput) && (ModifierCheckFunc == nullptr || ModifierCheckFunc(InInput)))
	{
		FInputRayHit HitResult = Target->CanBeginSingleClickAndDragSequence(GetDeviceRay(InInput));

		if (HitResult.bHit)
		{
			return FInputCaptureRequest::Begin(this, EInputCaptureSide::Any, HitResult.HitDepth);
		}
	}
	return FInputCaptureRequest::Ignore();
}

FInputCaptureUpdate UAvaSingleClickAndDragBehavior::BeginCapture(const FInputDeviceState& InInput, EInputCaptureSide InSide)
{
	Modifiers.UpdateModifiers(InInput, Target);
	OnClickPressInternal(InInput, InSide);
	bInClickDrag = true;
	bIsDragOperation = false;
	InitialMouseDownRay = GetDeviceRay(InInput);
	return FInputCaptureUpdate::Begin(this, EInputCaptureSide::Any);
}

FInputCaptureUpdate UAvaSingleClickAndDragBehavior::UpdateCapture(const FInputDeviceState& InInput, const FInputCaptureData& InData)
{
	// We have to check the device before going further because we get passed captures from 
	// keyboard for modifier key press/releases, and those don't have valid ray data.
	if ((GetSupportedDevices() & InInput.InputDevice) == EInputDevices::None)
	{
		return FInputCaptureUpdate::Continue();
	}

	if (!bSupportsDrag || !bIsDragOperation)
	{
		const float DistanceSquared = (InInput.Mouse.Position2D - InitialMouseDownRay.ScreenPosition).SizeSquared();
		const bool bBeyondClickDistance = DistanceSquared >= (DragStartDistance * DragStartDistance);

		if (bBeyondClickDistance && bSupportsDrag)
		{
			bIsDragOperation = true;
			OnDragStartedInternal(InInput, InData);
			return FInputCaptureUpdate::Continue();
		}
		else if (IsReleased(InInput))
		{
			if (bDistanceTestClick && bBeyondClickDistance)
			{
				return FInputCaptureUpdate::End();
			}

			OnClickReleaseInternal(InInput, InData);
			return FInputCaptureUpdate::End();
		}

		return FInputCaptureUpdate::Continue();
	}

	if (bUpdateModifiersDuringDrag)
	{
		Modifiers.UpdateModifiers(InInput, Target);
	}

	if (IsReleased(InInput))
	{
		OnClickReleaseInternal(InInput, InData);
		bInClickDrag = false;
		return FInputCaptureUpdate::End();
	}
	else
	{
		OnClickDragInternal(InInput, InData);
		return FInputCaptureUpdate::Continue();
	}
}

void UAvaSingleClickAndDragBehavior::ForceEndCapture(const FInputCaptureData& InData)
{
	Target->OnTerminateSingleClickAndDragSequence();
	bInClickDrag = false;
}

void UAvaSingleClickAndDragBehavior::OnClickPressInternal(const FInputDeviceState& InInput, EInputCaptureSide InSide)
{
	Target->OnClickPress(GetDeviceRay(InInput));
}

void UAvaSingleClickAndDragBehavior::OnDragStartedInternal(const FInputDeviceState& InInput, const FInputCaptureData& InData)
{
	Target->OnDragStart(GetDeviceRay(InInput));
}

void UAvaSingleClickAndDragBehavior::OnClickDragInternal(const FInputDeviceState& InInput, const FInputCaptureData& InData)
{
	Target->OnClickDrag(GetDeviceRay(InInput));
}

void UAvaSingleClickAndDragBehavior::OnClickReleaseInternal(const FInputDeviceState& InInput, const FInputCaptureData& InData)
{
	Target->OnClickRelease(GetDeviceRay(InInput), bIsDragOperation);
}
