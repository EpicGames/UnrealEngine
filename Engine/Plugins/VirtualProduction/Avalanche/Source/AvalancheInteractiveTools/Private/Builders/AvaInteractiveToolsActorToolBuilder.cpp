// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builders/AvaInteractiveToolsActorToolBuilder.h"
#include "InteractiveToolManager.h"
#include "Tools/AvaInteractiveToolsActorTool.h"
#include "Tools/UEdMode.h"

FAvaInteractiveToolsToolParameters UAvaInteractiveToolsActorToolBuilder::CreateToolParameters(FName InCategory,
	TSharedPtr<FUICommandInfo> InCommand, const FString& InIdentifier, int32 InPriority, TSubclassOf<AActor> InActorClass,
	TSubclassOf<UAvaInteractiveToolsActorTool> InToolClass)
{
	return {
		InCommand,
		InIdentifier,
		InPriority,
		FAvalancheInteractiveToolsCreateBuilder::CreateLambda(
			[InCategory, InCommandWeak = InCommand.ToWeakPtr(), InIdentifier, InPriority, InActorClass, InToolClass](UEdMode* InEdMode)
			{
				return CreateActorToolBuilder(InEdMode, InCategory, InCommandWeak.Pin(), InIdentifier, InPriority, InActorClass, InToolClass);
			})
	};
}

UAvaInteractiveToolsActorToolBuilder* UAvaInteractiveToolsActorToolBuilder::CreateActorToolBuilder(UEdMode* InEdMode, 
	FName InCategory, TSharedPtr<FUICommandInfo> InCommand, const FString& InIdentifier, int32 InPriority, TSubclassOf<AActor> InActorClass,
	TSubclassOf<UAvaInteractiveToolsActorTool> InToolClass)
{
	check(InEdMode);

	UClass* ActorClassLocal = InActorClass.Get();
	check(ActorClassLocal && !(ActorClassLocal->ClassFlags & EClassFlags::CLASS_Abstract));

	UAvaInteractiveToolsActorToolBuilder* NewToolBuilder = NewObject<UAvaInteractiveToolsActorToolBuilder>(InEdMode);
	NewToolBuilder->Category = InCategory;
	NewToolBuilder->Command = InCommand;
	NewToolBuilder->Identifier = InIdentifier;
	NewToolBuilder->Priority = InPriority;
	NewToolBuilder->ActorClass = InActorClass;
	NewToolBuilder->ToolClass = InToolClass;

	return NewToolBuilder;
}

UInteractiveTool* UAvaInteractiveToolsActorToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UAvaInteractiveToolsActorTool* ActorTool = NewObject<UAvaInteractiveToolsActorTool>(SceneState.ToolManager, ToolClass);
	ActorTool->Category = Category;
	ActorTool->Command = Command;
	ActorTool->Identifier = Identifier;
	ActorTool->Priority = Priority;
	ActorTool->ActorClass = ActorClass;

	return ActorTool;
}

UAvaInteractiveToolsActorToolBuilder::UAvaInteractiveToolsActorToolBuilder()
{
}
