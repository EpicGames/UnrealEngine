// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builders/AvaInteractiveToolsStaticMeshActorToolBuilder.h"
#include "Engine/StaticMesh.h"
#include "InteractiveToolManager.h"
#include "Tools/AvaInteractiveToolsStaticMeshActorTool.h"
#include "Tools/UEdMode.h"

FAvaInteractiveToolsToolParameters UAvaInteractiveToolsStaticMeshActorToolBuilder::CreateToolParameters(FName InCategory, 
	TSharedPtr<FUICommandInfo> InCommand, const FString& InIdentifier, int32 InPriority, UStaticMesh* InStaticMesh,
	TSubclassOf<UAvaInteractiveToolsStaticMeshActorTool> InToolClass)
{
	return {
		InCommand,
		InIdentifier,
		InPriority,
		FAvalancheInteractiveToolsCreateBuilder::CreateLambda(
			[InCategory, InCommandWeak = InCommand.ToWeakPtr(), InIdentifier, InPriority, InStaticMesh, InToolClass](UEdMode* InEdMode)
			{
				return CreateStaticMeshActorToolBuilder(InEdMode, InCategory, InCommandWeak.Pin(), InIdentifier, InPriority, InStaticMesh, InToolClass);
			})
	};
}

FAvaInteractiveToolsToolParameters UAvaInteractiveToolsStaticMeshActorToolBuilder::CreateToolParameters(FName InCategory, 
	TSharedPtr<FUICommandInfo> InCommand, const FString& InIdentifier, int32 InPriority,
	const TCHAR* InStaticMeshPath, TSubclassOf<UAvaInteractiveToolsStaticMeshActorTool> InToolClass)
{
	UStaticMesh* StaticMesh = LoadObject<UStaticMesh>(nullptr, InStaticMeshPath);

	return CreateToolParameters(InCategory, InCommand, InIdentifier, InPriority, StaticMesh, InToolClass);
}

UAvaInteractiveToolsStaticMeshActorToolBuilder* UAvaInteractiveToolsStaticMeshActorToolBuilder::CreateStaticMeshActorToolBuilder(
	UEdMode* InEdMode, FName InCategory, TSharedPtr<FUICommandInfo> InCommand, const FString& InIdentifier, int32 InPriority, 
	UStaticMesh* InStaticMesh, TSubclassOf<UAvaInteractiveToolsStaticMeshActorTool> InToolClass)
{
	check(InEdMode);
	check(InStaticMesh);

	UAvaInteractiveToolsStaticMeshActorToolBuilder* NewToolBuilder = NewObject<UAvaInteractiveToolsStaticMeshActorToolBuilder>(InEdMode);
	NewToolBuilder->Category = InCategory;
	NewToolBuilder->Command = InCommand;
	NewToolBuilder->Identifier = InIdentifier;
	NewToolBuilder->Priority = InPriority;
	NewToolBuilder->StaticMesh = InStaticMesh;
	NewToolBuilder->ToolClass = InToolClass;

	return NewToolBuilder;
}

UAvaInteractiveToolsStaticMeshActorToolBuilder* UAvaInteractiveToolsStaticMeshActorToolBuilder::CreateStaticMeshActorToolBuilder(
	UEdMode* InEdMode, FName InCategory, TSharedPtr<FUICommandInfo> InCommand, const FString& InIdentifier, int32 InPriority,
	const TCHAR* InStaticMeshPath, TSubclassOf<UAvaInteractiveToolsStaticMeshActorTool> InToolClass)
{
	UStaticMesh* StaticMesh = LoadObject<UStaticMesh>(nullptr, InStaticMeshPath);

	return CreateStaticMeshActorToolBuilder(InEdMode, InCategory, InCommand, InIdentifier, InPriority, StaticMesh, InToolClass);
}

UInteractiveTool* UAvaInteractiveToolsStaticMeshActorToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UAvaInteractiveToolsStaticMeshActorTool* StaticMeshActorTool = NewObject<UAvaInteractiveToolsStaticMeshActorTool>(SceneState.ToolManager);
	StaticMeshActorTool->Category = Category;
	StaticMeshActorTool->Command = Command;
	StaticMeshActorTool->Identifier = Identifier;
	StaticMeshActorTool->Priority = Priority;
	StaticMeshActorTool->StaticMesh = StaticMesh;

	return StaticMeshActorTool;
}

UAvaInteractiveToolsStaticMeshActorToolBuilder::UAvaInteractiveToolsStaticMeshActorToolBuilder()
{
}
