// Copyright Epic Games, Inc. All Rights Reserved.

#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "EdMode/AvaInteractiveToolsEdMode.h"
#include "InteractiveToolManager.h"
#include "Tools/AvaInteractiveToolsToolBase.h"
#include "Tools/UEdMode.h"

UAvaInteractiveToolsToolBuilder* UAvaInteractiveToolsToolBuilder::CreateToolBuilder(UEdMode* InEdMode, 
	TSubclassOf<UAvaInteractiveToolsToolBase> InToolClass)
{
	check(InEdMode);

	UClass* ToolClassLocal = InToolClass.Get();
	check(ToolClassLocal && !(ToolClassLocal->ClassFlags & EClassFlags::CLASS_Abstract));

	UAvaInteractiveToolsToolBuilder* NewToolBuilder = NewObject<UAvaInteractiveToolsToolBuilder>(InEdMode);
	NewToolBuilder->ToolClass = InToolClass;
	return NewToolBuilder;
}

UInteractiveTool* UAvaInteractiveToolsToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UAvaInteractiveToolsToolBase>(SceneState.ToolManager, ToolClass);
}

UAvaInteractiveToolsToolBuilder::UAvaInteractiveToolsToolBuilder()
{
}
