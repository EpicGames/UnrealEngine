// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaInteractiveToolsEdMode.h"

#include "AvaInteractiveToolsCommands.h"
#include "AvaInteractiveToolsEdModeToolkit.h"
#include "AvaInteractiveToolsSettings.h"
#include "AvaInteractiveToolsStyle.h"
#include "ContextObjectStore.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "IAvalancheInteractiveToolsModule.h"
#include "Selection.h"
#include "Tools/AvaInteractiveToolsToolBase.h"

#define LOCTEXT_NAMESPACE "AvaInteractiveToolsEdMode"

UAvaInteractiveToolsEdMode::UAvaInteractiveToolsEdMode()
{
	Info = FEditorModeInfo(IAvalancheInteractiveToolsModule::EM_AvaInteractiveToolsEdModeId,
		LOCTEXT("AvaInteractiveToolsEdModeName", "Motion Design"),
		FSlateIcon(FAvaInteractiveToolsStyle::Get().GetStyleSetName(), TEXT("Icons.Toolbox")),
		true);

	LastActiveTool = "";
}

namespace UE::AvaInteractiveToolsEditorMode::Private
{
	static TSet<FName> IncompatibleEdModes = {
		FBuiltinEditorModes::EM_None,
		FBuiltinEditorModes::EM_MeshPaint,
		FBuiltinEditorModes::EM_Landscape,
		FBuiltinEditorModes::EM_Foliage,
		FBuiltinEditorModes::EM_StreamingLevel,
		FBuiltinEditorModes::EM_Physics,
		FBuiltinEditorModes::EM_ActorPicker,
		FBuiltinEditorModes::EM_SceneDepthPicker
	};
}

bool UAvaInteractiveToolsEdMode::IsCompatibleWith(FEditorModeID OtherModeID) const
{
	return !UE::AvaInteractiveToolsEditorMode::Private::IncompatibleEdModes.Contains(OtherModeID);
}

void UAvaInteractiveToolsEdMode::BindCommands()
{
	Super::BindCommands();

	const FAvaInteractiveToolsCommands& ToolManagerCommands = FAvaInteractiveToolsCommands::Get();
	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();

	CommandList->MapAction(
		ToolManagerCommands.CancelActiveTool,
		FExecuteAction::CreateWeakLambda(this, [this]()
			{
				GetInteractiveToolsContext()->EndTool(EToolShutdownType::Cancel);
			}),
		FCanExecuteAction(),
		FGetActionCheckState(),
		FIsActionButtonVisible(),
		EUIActionRepeatMode::RepeatDisabled
	);

	CommandList->MapAction(
		ToolManagerCommands.ToggleViewportToolbar,
		FExecuteAction::CreateWeakLambda(this, [this]()
			{
				if (Toolkit.IsValid())
				{
					FAvaInteractiveToolsEdModeToolkit* AITToolkit = static_cast<FAvaInteractiveToolsEdModeToolkit*>(Toolkit.Get());
					AITToolkit->SetViewportToolbarVisibility(!AITToolkit->GetViewportToolbarVisibility());
				}
			}),
		FCanExecuteAction(),
		FGetActionCheckState(),
		FIsActionButtonVisible(),
		EUIActionRepeatMode::RepeatDisabled
	);
	
	CommandList->MapAction(
		ToolManagerCommands.OpenViewportToolbarSettings,
		FExecuteAction::CreateWeakLambda(this, [this]()
			{
				if (const UAvaInteractiveToolsSettings* Settings = UAvaInteractiveToolsSettings::Get())
				{
					Settings->OpenEditorSettingsWindow();
				}
			}),
		FCanExecuteAction(),
		FGetActionCheckState(),
		FIsActionButtonVisible(),
		EUIActionRepeatMode::RepeatDisabled
	);
}

void UAvaInteractiveToolsEdMode::Enter()
{
	UEdMode::Enter();

	UInteractiveToolManager* Manager = GetToolManager(EToolsContextScope::EdMode);

	if (UEditorInteractiveToolsContext* ToolContext = GetInteractiveToolsContext(EToolsContextScope::EdMode))
	{
		ToolContext->ContextObjectStore->AddContextObject(this);
	}

	const TMap<FName, TSharedPtr<FUICommandInfo>>& Categories = IAvalancheInteractiveToolsModule::Get().GetCategories();

	for (const TPair<FName, TSharedPtr<FUICommandInfo>>& Category : Categories)
	{
		if (Category.Value.IsValid())
		{
			TArray<TSharedPtr<FUICommandInfo>> CategoryCommands;
			const TArray<FAvaInteractiveToolsToolParameters>* ToolList = IAvalancheInteractiveToolsModule::Get().GetTools(Category.Key);

			if (ToolList)
			{
				for (const FAvaInteractiveToolsToolParameters& Tool : *ToolList)
				{
					if (Tool.UICommand.IsValid() && Tool.CreateBuilder.IsBound())
					{
						if (Manager)
						{
							Manager->UnregisterToolType(Tool.ToolIdentifier);
						}

						RegisterTool(Tool.UICommand, Tool.ToolIdentifier, Tool.CreateBuilder.Execute(this), EToolsContextScope::EdMode);
					}
				}
			}
		}
	}

	if (FEditorModeTools* ModeTools = GetModeManager())
	{
		if (USelection* ActorSelection = ModeTools->GetSelectedActors())
		{
			if (UTypedElementSelectionSet* ActorSelectionSet = ActorSelection->GetElementSelectionSet())
			{
				WeakActorSelectionSet = ActorSelectionSet;
				ActorSelectionSet->OnChanged().AddUObject(this, &UAvaInteractiveToolsEdMode::OnActorSelectionChange);
			}
		}
	}
}

void UAvaInteractiveToolsEdMode::CreateToolkit()
{
	Toolkit = MakeShared<FAvaInteractiveToolsEdModeToolkit>();
}

TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> UAvaInteractiveToolsEdMode::GetModeCommands() const
{
	TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> ToolCommands;

	const TMap<FName, TSharedPtr<FUICommandInfo>>& Categories = IAvalancheInteractiveToolsModule::Get().GetCategories();

	for (const TPair<FName, TSharedPtr<FUICommandInfo>>& Category : Categories)
	{
		if (Category.Value.IsValid())
		{
			TArray<TSharedPtr<FUICommandInfo>> CategoryCommands;
			const TArray<FAvaInteractiveToolsToolParameters>* ToolList = IAvalancheInteractiveToolsModule::Get().GetTools(Category.Key);

			if (ToolList)
			{
				for (const FAvaInteractiveToolsToolParameters& Tool : *ToolList)
				{
					if (Tool.UICommand.IsValid())
					{
						CategoryCommands.Add(Tool.UICommand);
					}
				}
			}

			if (CategoryCommands.IsEmpty() == false)
			{
				ToolCommands.Emplace(Category.Key, CategoryCommands);
			}
		}
	}

	return ToolCommands;
}

void UAvaInteractiveToolsEdMode::Exit()
{
	Super::Exit();

	// Try this first - SelectionSet from ModeTools can be invalid when loading between levels
	if (UTypedElementSelectionSet* StrongActorSelectionSet = WeakActorSelectionSet.Get())
	{
		StrongActorSelectionSet->OnChanged().RemoveAll(this);
		WeakActorSelectionSet.Reset();
	}
	else if (FEditorModeTools* ModeTools = GetModeManager())
	{
		if (USelection* ActorSelection = ModeTools->GetSelectedActors())
		{
			if (UTypedElementSelectionSet* ActorSelectionSet = ActorSelection->GetElementSelectionSet())
			{
				ActorSelectionSet->OnChanged().RemoveAll(this);
			}
		}
	}
}

void UAvaInteractiveToolsEdMode::OnToolPaletteChanged(FName InPaletteName)
{
	LastActiveTool = "";
}

void UAvaInteractiveToolsEdMode::OnToolSetup(UInteractiveTool* InTool)
{
	if (UInteractiveToolsContext* Context = GetInteractiveToolsContext(EToolsContextScope::EdMode))
	{
		LastActiveTool = Context->GetActiveToolName(EToolSide::Left);
	}
	else
	{
		LastActiveTool = "";
	}
}

void UAvaInteractiveToolsEdMode::OnToolShutdown(UInteractiveTool* InTool, EToolShutdownType InShutdownType)
{
	LastActiveTool = "";

	if (UEditorInteractiveToolsContext* Context = GetInteractiveToolsContext())
	{
		Context->EndTool(InShutdownType);
	}
}

void UAvaInteractiveToolsEdMode::OnToolActivateEnd()
{
	LastToolActivateTime = FApp::GetCurrentTime();
}

void UAvaInteractiveToolsEdMode::OnActorSelectionChange(const UTypedElementSelectionSet* InSelectionSet)
{
	UInteractiveToolManager* ToolManager = GetToolManager();

	if (!ToolManager)
	{
		return;
	}

	// Don't change with selection if we have an active tool
	if (ToolManager->GetActiveTool(EToolSide::Left))
	{
		return;
	}

	TSharedPtr<FModeToolkit> ToolkitPin = GetToolkit().Pin();

	if (!ToolkitPin)
	{
		return;
	}

	AActor* SelectedActor = InSelectionSet->GetTopSelectedObject<AActor>();
	UObject* DetailsObject = UAvaInteractiveToolsToolBase::GetDetailsObjectFromActor(SelectedActor);
	ToolkitPin->SetModeSettingsObject(DetailsObject);
}

#undef LOCTEXT_NAMESPACE
