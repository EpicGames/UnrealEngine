// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/LegacyEdModeWidgetHelpers.h"
#include "UObject/WeakObjectPtrFwd.h"
#include "AvaInteractiveToolsEdMode.generated.h"

class FAvaVisualizerBase;
enum class EToolShutdownType : uint8;

UCLASS()
class UAvaInteractiveToolsEdMode : public UBaseLegacyWidgetEdMode
{
	GENERATED_BODY()

public:
	UAvaInteractiveToolsEdMode();
	virtual ~UAvaInteractiveToolsEdMode() override = default;

	//~ Begin UEdMode interface
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override;
	virtual void BindCommands() override;
	virtual void Enter() override;
	virtual void CreateToolkit() override;
	virtual TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> GetModeCommands() const override;
	virtual void Exit() override;
	//~ End UEdMode interface

	void OnToolPaletteChanged(FName InPaletteName);
	void OnToolSetup(UInteractiveTool* InTool);
	void OnToolShutdown(UInteractiveTool* InTool, EToolShutdownType InShutdownType);
	FString GetLastActiveTool() const { return LastActiveTool; }
	double GetLastToolActivateTime() const { return LastToolActivateTime; }
	void OnToolActivateEnd();

protected:
	FString LastActiveTool;
	TWeakObjectPtr<UTypedElementSelectionSet> WeakActorSelectionSet;
	double LastToolActivateTime = -1.0;

	void OnActorSelectionChange(const UTypedElementSelectionSet* InSelectionSet);
};
