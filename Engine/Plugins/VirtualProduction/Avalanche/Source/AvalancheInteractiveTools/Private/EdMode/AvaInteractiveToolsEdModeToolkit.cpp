// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaInteractiveToolsEdModeToolkit.h"

#include "AvaInteractiveToolsCommands.h"
#include "AvaInteractiveToolsEdMode.h"
#include "AvaInteractiveToolsSettings.h"
#include "AvaInteractiveToolsStyle.h"
#include "AvalancheInteractiveToolsModule.h"
#include "IAvalancheInteractiveToolsModule.h"
#include "Toolkits/AssetEditorModeUILayer.h"
#include "Tools/UEdMode.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SAvaInteractiveToolsToolbarCategoryButton.h"

#define LOCTEXT_NAMESPACE "AvaInteractiveToolsEdModeToolkit"

FAvaInteractiveToolsEdModeToolkit::FAvaInteractiveToolsEdModeToolkit()
{
	bUsesToolkitBuilder = true;
}

FAvaInteractiveToolsEdModeToolkit::~FAvaInteractiveToolsEdModeToolkit()
{
	if (ToolkitHost.IsValid())
	{
		GetToolkitHost()->RemoveViewportOverlayWidget(ViewportToolbarWidget.ToSharedRef());
	}
}

void FAvaInteractiveToolsEdModeToolkit::SetViewportToolbarVisibility(bool bInShow)
{
	bViewportToolbarVisible = bInShow;
}

FName FAvaInteractiveToolsEdModeToolkit::GetToolkitFName() const
{
	static const FName ToolkitName("ModelingToolsEditorMode");
	return ToolkitName;
}

FText FAvaInteractiveToolsEdModeToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("DisplayName", "AvaInteractiveToolsEdMode Tool");
}

TSharedPtr<SWidget> FAvaInteractiveToolsEdModeToolkit::GetInlineContent() const
{
	checkf(ToolkitWidget.IsValid(), TEXT("Toolkit widget for interactive tools editor mode is invalid"))

	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.VAlign(VAlign_Fill)
		[
			ToolkitWidget.ToSharedRef()
		];
}

void FAvaInteractiveToolsEdModeToolkit::GetToolPaletteNames(TArray<FName>& InPaletteName) const
{
	const TMap<FName, TSharedPtr<FUICommandInfo>>& Categories = IAvalancheInteractiveToolsModule::Get().GetCategories();
	Categories.GetKeys(InPaletteName);
}

FText FAvaInteractiveToolsEdModeToolkit::GetToolPaletteDisplayName(FName InPaletteName) const
{
	const TMap<FName, TSharedPtr<FUICommandInfo>>& Categories = IAvalancheInteractiveToolsModule::Get().GetCategories();

	if (const TSharedPtr<FUICommandInfo>* CommandInfo = Categories.Find(InPaletteName))
	{
		return CommandInfo->Get()->GetLabel();
	}

	return FText::FromName(InPaletteName);
}

void FAvaInteractiveToolsEdModeToolkit::OnToolPaletteChanged(FName InPaletteName)
{
	if (UAvaInteractiveToolsEdMode* AvaInteractiveToolsEdMode = Cast<UAvaInteractiveToolsEdMode>(GetScriptableEditorMode().Get()))
	{
		AvaInteractiveToolsEdMode->OnToolPaletteChanged(InPaletteName);
	}
}

void FAvaInteractiveToolsEdModeToolkit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode)
{
	FModeToolkit::Init(InitToolkitHost, InOwningMode);

	MakeViewportOverlayToolbar();
	MakeToolkitPalettes();
}

void FAvaInteractiveToolsEdModeToolkit::OnToolStarted(UInteractiveToolManager* InManager, UInteractiveTool* InTool)
{
	// Nothing
}

void FAvaInteractiveToolsEdModeToolkit::OnToolEnded(UInteractiveToolManager* InManager, UInteractiveTool* InTool)
{
	// Nothing
}

void FAvaInteractiveToolsEdModeToolkit::InvokeUI()
{
	FModeToolkit::InvokeUI();

	const TSharedPtr<SWidget> InlineContentWidget = GetInlineContent();
	InlineContentHolder->SetContent(InlineContentWidget.ToSharedRef());
}

void FAvaInteractiveToolsEdModeToolkit::RequestModeUITabs()
{
	if (const TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin())
	{
		PrimaryTabInfo.OnSpawnTab = FOnSpawnTab::CreateSP(SharedThis(this), &FAvaInteractiveToolsEdModeToolkit::CreatePrimaryModePanel);
		PrimaryTabInfo.TabLabel = LOCTEXT("MotionDesignToolboxTab", "Motion Design");
		PrimaryTabInfo.TabTooltip = LOCTEXT("MotionDesignToolboxTabTooltipText", "Opens the Motion Design tab.");
		ModeUILayerPtr->SetModePanelInfo(UAssetEditorUISubsystem::TopLeftTabID, PrimaryTabInfo);

		if (!HasIntegratedToolPalettes() && !HasToolkitBuilder())
		{
			ToolbarInfo.OnSpawnTab = FOnSpawnTab::CreateSP(SharedThis(this), &FAvaInteractiveToolsEdModeToolkit::MakeModeToolbarTab);
			ToolbarInfo.TabLabel = LOCTEXT("MotionDesignToolbarTab", "Motion Design Toolbar");
			ToolbarInfo.TabTooltip = LOCTEXT("MotionDesignToolbarTabTooltipText", "Opens the toolbar for the Motion Design toolbox.");
			ModeUILayerPtr->SetModePanelInfo(UAssetEditorUISubsystem::VerticalToolbarID, ToolbarInfo);
		}
	}
}

void FAvaInteractiveToolsEdModeToolkit::MakeViewportOverlayToolbar()
{
	if (ViewportToolbarWidget.IsValid())
	{
		GetToolkitHost()->RemoveViewportOverlayWidget(ViewportToolbarWidget.ToSharedRef());
		ViewportToolbarWidget.Reset();
	}

	UAvaInteractiveToolsSettings* Settings = UAvaInteractiveToolsSettings::Get();

	if (!Settings)
	{
		return;
	}

	// When switching editor settings, update widget
	Settings->OnSettingChanged().RemoveAll(this);
	Settings->OnSettingChanged().AddSP(this, &FAvaInteractiveToolsEdModeToolkit::OnSettingsChanged);

	if (Settings->ViewportToolbarPosition == EAvaInteractiveToolsViewportToolbarPosition::None)
	{
		return;
	}

	// When switching viewport, overlay is reset
	GetToolkitHost()->OnActiveViewportChanged().RemoveAll(this);
	GetToolkitHost()->OnActiveViewportChanged().AddSP(this, &FAvaInteractiveToolsEdModeToolkit::OnViewportChanged);

	const TSharedRef<FUICommandList> ToolkitCommandList = GetToolkitCommands();

	auto SetupToolbar = [&ToolkitCommandList, &Settings](FToolBarBuilder& InToolbarBuilder)
	{
		InToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);
		InToolbarBuilder.SetStyle(&FAvaInteractiveToolsStyle::Get(), "ViewportToolbar");
		for (const TPair<FName, TSharedPtr<FUICommandInfo>>& Category : IAvalancheInteractiveToolsModule::Get().GetCategories())
		{
			if (!Category.Value.IsValid())
			{
				continue;
			}

			InToolbarBuilder.AddWidget(
				SNew(SAvaInteractiveToolsToolbarCategoryButton)
				.CommandList(ToolkitCommandList)
				.ShowLabel(Settings->bViewportToolbarLabelEnabled)
				.ToolCategory(Category.Key)
			);
		}

		InToolbarBuilder.AddSeparator();

		InToolbarBuilder.AddWidget(
			SNew(SAvaInteractiveToolsToolbarCategoryButton)
			.CommandList(ToolkitCommandList)
			.ShowLabel(Settings->bViewportToolbarLabelEnabled)
			.Command(FAvaInteractiveToolsCommands::Get().ToggleViewportToolbar)
		);

		InToolbarBuilder.AddWidget(
			SNew(SAvaInteractiveToolsToolbarCategoryButton)
			.CommandList(ToolkitCommandList)
			.ShowLabel(Settings->bViewportToolbarLabelEnabled)
			.Command(FAvaInteractiveToolsCommands::Get().OpenViewportToolbarSettings)
		);
	};

	TSharedPtr<SWidget> ToolbarWidget;
	EHorizontalAlignment HAlign;
	EVerticalAlignment VAlign;
	if (Settings->ViewportToolbarPosition == EAvaInteractiveToolsViewportToolbarPosition::Bottom
		|| Settings->ViewportToolbarPosition == EAvaInteractiveToolsViewportToolbarPosition::Top)
	{
		FSlimHorizontalToolBarBuilder ToolbarBuilder(
			ToolkitCommandList,
			FMultiBoxCustomization::None,
			TSharedPtr<FExtender>(),
			/** ForceSmallIcon */false);

		SetupToolbar(ToolbarBuilder);
		ToolbarWidget = ToolbarBuilder.MakeWidget();
		
		HAlign = HAlign_Center;
		VAlign = Settings->ViewportToolbarPosition == EAvaInteractiveToolsViewportToolbarPosition::Bottom ? VAlign_Bottom : VAlign_Top;
	}
	else
	{
		FVerticalToolBarBuilder ToolbarBuilder(
			ToolkitCommandList,
			FMultiBoxCustomization::None,
			TSharedPtr<FExtender>(),
			/** ForceSmallIcon */false);

		SetupToolbar(ToolbarBuilder);
		ToolbarWidget = ToolbarBuilder.MakeWidget();

		HAlign = Settings->ViewportToolbarPosition == EAvaInteractiveToolsViewportToolbarPosition::Left ? HAlign_Left : HAlign_Right;
		VAlign = VAlign_Center;
	}

	SAssignNew(ViewportToolbarWidget, SVerticalBox)
		+ SVerticalBox::Slot()
		.HAlign(HAlign)
		.VAlign(VAlign)
		.Padding(0.f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("EditorViewport.OverlayBrush"))
			.Padding(FMargin(3.0f, 6.0f, 3.f, 6.f))
			[
				ToolbarWidget.ToSharedRef()
			]
		];

	ViewportToolbarWidget->SetVisibility(TAttribute<EVisibility>::CreateLambda([this]()
	{
		// Only show when motion design mode is selected
		if (UAvaInteractiveToolsEdMode* AvaMode = Cast<UAvaInteractiveToolsEdMode>(GetScriptableEditorMode()))
		{
			// to avoid affecting other viewport overlay buttons
			return bViewportToolbarVisible ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
		}

		return EVisibility::Collapsed;
	}));

	GetToolkitHost()->AddViewportOverlayWidget(ViewportToolbarWidget.ToSharedRef());
}

void FAvaInteractiveToolsEdModeToolkit::MakeToolkitPalettes()
{
	const UAvaInteractiveToolsEdMode* AvaInteractiveToolsMode = CastChecked<UAvaInteractiveToolsEdMode>(GetScriptableEditorMode().Get());

	ModeDetailsView->SetIsPropertyVisibleDelegate(
		FIsPropertyVisible::CreateLambda(
			[](const FPropertyAndParent& InPropertyAndParent) -> bool
			{
				static const FString Material = FString(TEXT("Material"));
				static const FName Category = FName(TEXT("Category"));
				const FString CategoryMeta = InPropertyAndParent.Property.GetMetaData(Category);
				return CategoryMeta != Material;
			}
	));

	ToolkitSections = MakeShared<FToolkitSections>();

	// Show warning text when tool is active
	ToolkitSections->ToolWarningArea = SNew(STextBlock)
		.Text(this, &FAvaInteractiveToolsEdModeToolkit::GetToolWarningText)
		.AutoWrapText(true);
		
	ToolkitBuilder = MakeShared<FToolkitBuilder>(
		AvaInteractiveToolsMode->GetModeInfo().ToolbarCustomizationName,
		GetToolkitCommands(),
		ToolkitSections);

	const TMap<FName, TSharedPtr<FUICommandInfo>>& Categories = IAvalancheInteractiveToolsModule::Get().GetCategories();
	TSharedPtr<FUICommandInfo> FirstCategoryCommand = nullptr;

	// Group tools command by category in the palette
	for (const TPair<FName, TSharedPtr<FUICommandInfo>>& Category : Categories)
	{
		if (Category.Value.IsValid())
		{
			TArray<TSharedPtr<FUICommandInfo>> CategoryCommands;

			if (const TArray<FAvaInteractiveToolsToolParameters>* ToolList = IAvalancheInteractiveToolsModule::Get().GetTools(Category.Key))
			{
				for (const FAvaInteractiveToolsToolParameters& Tool : *ToolList)
				{
					if (Tool.UICommand.IsValid())
					{
						CategoryCommands.Add(Tool.UICommand);
					}
				}
			}

			if (CategoryCommands.IsEmpty() == false)
			{
				ToolkitBuilder->AddPalette(MakeShared<FToolPalette>(Category.Value.ToSharedRef(), CategoryCommands));

				if (FirstCategoryCommand.IsValid() == false)
				{
					FirstCategoryCommand = Category.Value;
				}
			}
		}
	}

	if (FirstCategoryCommand.IsValid())
	{
		ToolkitBuilder->SetActivePaletteOnLoad(FirstCategoryCommand.Get());
	}

	ToolkitBuilder->UpdateWidget();

	const TSharedPtr<SWidget> ToolkitGeneratedWidget = ToolkitBuilder->GenerateWidget();
	
	checkf(ToolkitGeneratedWidget.IsValid(), TEXT("Generated widget for interactive tools editor mode is invalid"))
	
	SAssignNew(ToolkitWidget, SBorder)
		.HAlign(HAlign_Fill)
		.Padding(0)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			ToolkitGeneratedWidget->AsShared()
		];
}

FText FAvaInteractiveToolsEdModeToolkit::GetToolWarningText() const
{
	static const FText ToolActive = LOCTEXT("ActiveToolWarning", "Tool Active.\n\nSelect the tool again to perform the default action (if supported).\n\nRight click or press escape to cancel.");
	static const FText ToolInactive = LOCTEXT("InactiveToolWarning", "Select a tool once to start drawing or double click to perform default action (if supported).");

	return FAvalancheInteractiveToolsModule::Get().HasActiveTool()
		? ToolActive
		: ToolInactive;
}

void FAvaInteractiveToolsEdModeToolkit::OnSettingsChanged(UObject* InSettings, FPropertyChangedEvent& InEvent)
{
	if (UAvaInteractiveToolsSettings::IsViewportToolbarProperty(InEvent.GetMemberPropertyName()))
	{
		MakeViewportOverlayToolbar();
	}
}

void FAvaInteractiveToolsEdModeToolkit::OnViewportChanged(TSharedPtr<IAssetViewport> InOldViewport, TSharedPtr<IAssetViewport> InNewViewport)
{
	if (!InNewViewport.IsValid())
	{
		return;
	}

	if (InOldViewport != InNewViewport)
	{
		MakeViewportOverlayToolbar();
	}
}

#undef LOCTEXT_NAMESPACE
