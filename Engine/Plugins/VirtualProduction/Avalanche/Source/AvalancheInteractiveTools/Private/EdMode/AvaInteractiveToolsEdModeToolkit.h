// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/BaseToolkit.h"

class SWidget;
class UObject;
struct FPropertyChangedEvent;

class FAvaInteractiveToolsEdModeToolkit : public FModeToolkit
{
public:
	FAvaInteractiveToolsEdModeToolkit();
	virtual ~FAvaInteractiveToolsEdModeToolkit() override;

	void SetViewportToolbarVisibility(bool bInShow);
	bool GetViewportToolbarVisibility() const
	{
		return bViewportToolbarVisible;
	}

protected:
	//~ Begin IToolkit
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual TSharedPtr<SWidget> GetInlineContent() const override;
	//~ End IToolkit

	//~ Begin FModeToolkit
	virtual void Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode) override;
	virtual void GetToolPaletteNames(TArray<FName>& PaletteNames) const override;
	virtual FText GetToolPaletteDisplayName(FName Palette) const override;
	virtual void OnToolPaletteChanged(FName PaletteName) override;
	virtual bool HasIntegratedToolPalettes() const override { return false; }
	virtual bool HasExclusiveToolPalettes() const override { return false; }
	virtual void OnToolStarted(UInteractiveToolManager* InManager, UInteractiveTool* InTool) override;
	virtual void OnToolEnded(UInteractiveToolManager* InManager, UInteractiveTool* InTool) override;
	virtual void InvokeUI() override;
	//~ End FModeToolkit

	//~ Begin FModeToolkit
	virtual void RequestModeUITabs() override;
	//~ End FModeToolkit

private:
	/** Creates an viewport overlay toolbar */
	void MakeViewportOverlayToolbar();

	void MakeToolkitPalettes();
	
	FText GetToolWarningText() const;

	void OnSettingsChanged(UObject* InSettings, FPropertyChangedEvent& InEvent);
	void OnViewportChanged(TSharedPtr<IAssetViewport> InOldViewport, TSharedPtr<IAssetViewport> InNewViewport);

	TSharedPtr<SWidget> ToolkitWidget;
	TSharedPtr<SWidget> ViewportToolbarWidget;
	bool bViewportToolbarVisible = true;
};
