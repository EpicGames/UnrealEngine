// Copyright Epic Games, Inc. All Rights Reserved.

#include "Planners/AvaInteractiveToolsToolViewportAreaPlanner.h"
#include "AvaInteractiveToolsSettings.h"
#include "AvaViewportUtils.h"
#include "AvaVisibleArea.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Framework/Application/SlateApplication.h"
#include "InteractiveToolManager.h"
#include "SceneView.h"
#include "Styling/StyleColors.h"
#include "Tools/AvaInteractiveToolsToolBase.h"
#include "ViewportClient/IAvaViewportClient.h"

void UAvaInteractiveToolsToolViewportAreaPlanner::SetStartPosition(const FVector2f& InStartPosition)
{
	StartPosition = InStartPosition;
}

void UAvaInteractiveToolsToolViewportAreaPlanner::SetEndPosition(const FVector2f& InEndPosition)
{
	EndPosition = InEndPosition;
}

FVector2f UAvaInteractiveToolsToolViewportAreaPlanner::GetCenterPosition() const
{
	return StartPosition * 0.5f + EndPosition * 0.5f;
}

FVector2f UAvaInteractiveToolsToolViewportAreaPlanner::GetTopLeftCorner() const
{
	return FVector2f(
		FMath::Min(StartPosition.X, EndPosition.X),
		FMath::Min(StartPosition.Y, EndPosition.Y)
	);
}

FVector2f UAvaInteractiveToolsToolViewportAreaPlanner::GetTopRightCorner() const
{
	return FVector2f(
		FMath::Max(StartPosition.X, EndPosition.X),
		FMath::Min(StartPosition.Y, EndPosition.Y)
	);
}

FVector2f UAvaInteractiveToolsToolViewportAreaPlanner::GetBottomLeftCorner() const
{
	return FVector2f(
		FMath::Min(StartPosition.X, EndPosition.X),
		FMath::Max(StartPosition.Y, EndPosition.Y)
	);
}

FVector2f UAvaInteractiveToolsToolViewportAreaPlanner::GetBottomRightCorner() const
{
	return FVector2f(
		FMath::Max(StartPosition.X, EndPosition.X),
		FMath::Max(StartPosition.Y, EndPosition.Y)
	);
}

FVector2f UAvaInteractiveToolsToolViewportAreaPlanner::GetSize() const
{
	return FVector2f(
		FMath::Abs(EndPosition.X - StartPosition.X),
		FMath::Abs(EndPosition.Y - StartPosition.Y)
	);
}

FVector UAvaInteractiveToolsToolViewportAreaPlanner::GetStartPositionWorld() const
{
	return ViewportToWorldPosition(StartPosition);
}

FVector UAvaInteractiveToolsToolViewportAreaPlanner::GetEndPositionWorld() const
{
	return ViewportToWorldPosition(EndPosition);
}

FVector UAvaInteractiveToolsToolViewportAreaPlanner::GetTopLeftCornerWorld() const
{
	return ViewportToWorldPosition(GetTopLeftCorner());
}

FVector UAvaInteractiveToolsToolViewportAreaPlanner::GetTopRightCornerWorld() const
{
	return ViewportToWorldPosition(GetTopRightCorner());
}

FVector UAvaInteractiveToolsToolViewportAreaPlanner::GetBottomLeftCornerWorld() const
{
	return ViewportToWorldPosition(GetBottomLeftCorner());
}

FVector UAvaInteractiveToolsToolViewportAreaPlanner::GetBottomRightCornerWorld() const
{
	return ViewportToWorldPosition(GetBottomRightCorner());
}

FVector2D UAvaInteractiveToolsToolViewportAreaPlanner::GetWorldSize() const
{
	if (!Tool)
	{
		return FVector2D::ZeroVector;
	}

	const float CameraDistance = GetDefault<UAvaInteractiveToolsSettings>()->CameraDistance;

	const FVector2f TopLeftCornerViewport = GetTopLeftCorner();
	const FVector2f TopRightCornerViewport = GetTopRightCorner();
	const FVector2f BottomLeftCornerViewport = GetBottomLeftCorner();

	UWorld* TempWorld;
	FVector TempVector;
	FRotator TempRotator;
	bool bValid = true;

	bValid &= Tool->ViewportPositionToWorldPositionAndOrientation(EAvaViewportStatus::Focused, TopLeftCornerViewport, CameraDistance, TempWorld, TempVector, TempRotator);
	const FVector TopLeftCornerWorld = TempVector;

	bValid &= Tool->ViewportPositionToWorldPositionAndOrientation(EAvaViewportStatus::Focused, TopRightCornerViewport, CameraDistance, TempWorld, TempVector, TempRotator);
	const FVector TopRightCornerWorld = TempVector;

	bValid &= Tool->ViewportPositionToWorldPositionAndOrientation(EAvaViewportStatus::Focused, BottomLeftCornerViewport, CameraDistance, TempWorld, TempVector, TempRotator);
	const FVector BottomLeftCornerWorld = TempVector;

	if (!bValid)
	{
		return FVector2D::ZeroVector;
	}

	FVector2D Size;
	Size.X = (TopRightCornerWorld - TopLeftCornerWorld).Size();
	Size.Y = (BottomLeftCornerWorld - TopLeftCornerWorld).Size();

	return Size;
}

void UAvaInteractiveToolsToolViewportAreaPlanner::Setup(UAvaInteractiveToolsToolBase* InTool)
{
	Super::Setup(InTool);

	bStartedAreaPlanning = false;
	StartPosition = FVector2f::ZeroVector;
	EndPosition = FVector2f::ZeroVector;
}

void UAvaInteractiveToolsToolViewportAreaPlanner::OnTick(float InDeltaTime)
{
	Super::OnTick(InDeltaTime);

	FVector2f MousePosition = GetConstrainedMousePosition();

	if (!bStartedAreaPlanning)
	{
		// To get the snap indicators to show up.
		SnapLocation(MousePosition);
	}
	else
	{
		UpdateEndPosition(MousePosition);

		if (Tool)
		{
			Tool->OnViewportPlannerUpdate();
		}
	}
}

void UAvaInteractiveToolsToolViewportAreaPlanner::DrawHUD(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI)
{
	Super::DrawHUD(InCanvas, InRenderAPI);

	if (!bStartedAreaPlanning)
	{
		return;
	}

	static const FLinearColor BoxColor = FStyleColors::AccentBlue.GetSpecifiedColor();
	static const FLinearColor SquareColor = FStyleColors::AccentGreen.GetSpecifiedColor();

	TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAvaViewportClient(GetViewport(EAvaViewportStatus::Focused));

	if (!AvaViewportClient.IsValid())
	{
		return;
	}

	const FAvaVisibleArea VisibleArea = AvaViewportClient->GetZoomedVisibleArea();

	if (!VisibleArea.IsValid())
	{
		return;
	}

	const FVector2f Offset = AvaViewportClient->GetViewportOffset();
	const FVector2f Size = GetSize();
	const bool bIsSquare = FMath::IsNearlyEqual(Size.X, Size.Y, 0.1);
	const float AppScale = FSlateApplication::Get().GetApplicationScale();
	const float DPIScale = VisibleArea.DPIScale;

	const FVector2f TopLeftCorner = (VisibleArea.GetVisiblePosition(GetTopLeftCorner()) * AppScale) + Offset;
	const FVector2f TopRightCorner = (VisibleArea.GetVisiblePosition(GetTopRightCorner()) * AppScale) + Offset;
	const FVector2f BottomLeftCorner = (VisibleArea.GetVisiblePosition(GetBottomLeftCorner()) * AppScale) + Offset;
	const FVector2f BottomRightCorner = (VisibleArea.GetVisiblePosition(GetBottomRightCorner()) * AppScale) + Offset;

	FCanvasLineItem TopLine(static_cast<FVector2D>(TopLeftCorner), static_cast<FVector2D>(TopRightCorner));
	TopLine.SetColor(bIsSquare ? SquareColor : BoxColor);

	FCanvasLineItem LeftLine(static_cast<FVector2D>(TopLeftCorner), static_cast<FVector2D>(BottomLeftCorner));
	LeftLine.SetColor(bIsSquare ? SquareColor : BoxColor);

	FCanvasLineItem BottomLine(static_cast<FVector2D>(BottomLeftCorner), static_cast<FVector2D>(BottomRightCorner));
	BottomLine.SetColor(bIsSquare ? SquareColor : BoxColor);

	FCanvasLineItem RightLine(static_cast<FVector2D>(TopRightCorner), static_cast<FVector2D>(BottomRightCorner));
	RightLine.SetColor(bIsSquare ? SquareColor : BoxColor);

	InCanvas->DrawItem(TopLine);
	InCanvas->DrawItem(LeftLine);
	InCanvas->DrawItem(BottomLine);
	InCanvas->DrawItem(RightLine);

	if (GEngine)
	{
		static const FSlateFontInfo FontInfo(FCoreStyle::GetDefaultFont(), 8);
		static const FLinearColor SizeBackgroundColor(0.f, 0.f, 0.f, 0.25f);
		static const FLinearColor SizeBorderColor = FStyleColors::AccentBlue.GetSpecifiedColor();
		static const FLinearColor SizeBorderColorSquare = FStyleColors::AccentGreen.GetSpecifiedColor();
		static const FLinearColor SizeTextColor = FStyleColors::AccentBlue.GetSpecifiedColor();
		static const FLinearColor SizeTextColorSquare = FStyleColors::AccentGreen.GetSpecifiedColor();

		const FIntRect CanvasSize = InRenderAPI->GetSceneView()->UnscaledViewRect;

		const FText SizeText = FText::Format(
			INVTEXT("{0} x {1}"),
			FText::AsNumber(FMath::FloorToInt(Size.X)),
			FText::AsNumber(FMath::FloorToInt(Size.Y))
		);

		FVector2f SizePosition = VisibleArea.GetVisiblePosition(GetBottomLeftCorner()) + FVector2f(2.f, 4.f);
		SizePosition.X = FMath::Clamp(SizePosition.X, 5.f, (CanvasSize.Width() / DPIScale) - 75.f); // roughly estimate max width at 70
		SizePosition.Y = FMath::Clamp(SizePosition.Y, 5.f, (CanvasSize.Height() / DPIScale) - 21.f); // Roughly estimate max height at 16
		SizePosition *= AppScale;
		SizePosition += Offset;

		FCanvasTextItem SizeTextItem(static_cast<FVector2D>(SizePosition + FVector2f(2.f, 2.f)), SizeText, FontInfo, bIsSquare ? SizeTextColorSquare : SizeTextColor);
		SizeTextItem.Font = GEngine->GetMediumFont();

		InCanvas->DrawItem(SizeTextItem);

		const FVector2D TextSize = (SizeTextItem.DrawnSize / DPIScale * AppScale) + FVector2D(4.f, 4.f);

		FCanvasTileItem SizeBackgroundItem(static_cast<FVector2D>(SizePosition), TextSize, SizeBackgroundColor);
		SizeBackgroundItem.BlendMode = ESimpleElementBlendMode::SE_BLEND_Translucent;

		InCanvas->DrawItem(SizeBackgroundItem);
		InCanvas->DrawItem(SizeTextItem); // Redraw on top of the background

		FCanvasBoxItem SizeBorderItem(static_cast<FVector2D>(SizePosition), TextSize);
		SizeBorderItem.LineThickness = 1.f;
		SizeBorderItem.SetColor(bIsSquare ? SizeBorderColorSquare : SizeBorderColor);

		InCanvas->DrawItem(SizeBorderItem);
	}
}

void UAvaInteractiveToolsToolViewportAreaPlanner::OnClicked(const FInputDeviceRay& InClickPos)
{
	Super::OnClicked(InClickPos);

	const FVector2f MousePosition = GetConstrainedMousePosition(
		static_cast<FVector2f>(InClickPos.ScreenPosition)
	);

	if (!bStartedAreaPlanning)
	{
		StartPosition = MousePosition;
		SnapLocation(StartPosition);

		EndPosition = StartPosition + FVector2f(MinDim, MinDim);
		bStartedAreaPlanning = true;
	}
	else
	{
		UpdateEndPosition(MousePosition);

		if (Tool)
		{
			Tool->OnViewportPlannerComplete();
		}
	}
}

void UAvaInteractiveToolsToolViewportAreaPlanner::UpdateEndPosition(const FVector2f& InNewPosition)
{
	const bool bFromCenter = FSlateApplication::Get().GetModifierKeys().IsAltDown();
	const bool bForceSquare = FSlateApplication::Get().GetModifierKeys().IsShiftDown();

	const FVector2f CenterPosition = StartPosition * 0.5f + EndPosition * 0.5f;

	if (bFromCenter)
	{
		StartPosition = CenterPosition;
	}

	FVector2f NewPosition;
	NewPosition.X = FMath::RoundToFloat(InNewPosition.X);
	NewPosition.Y = FMath::RoundToFloat(InNewPosition.Y);

	SnapLocation(NewPosition);

	for (int32 Component = 0; Component < 2; ++Component)
	{
		if (FMath::Abs(StartPosition[Component] - NewPosition[Component]) < MinDim)
		{
			if (StartPosition[Component] < NewPosition[Component])
			{
				EndPosition[Component] = StartPosition[Component] + MinDim;
			}
			else
			{
				EndPosition[Component] = StartPosition[Component] - MinDim;
			}
		}
		else
		{
			EndPosition[Component] = NewPosition[Component];
		}
	}

	// Force square
	if (bForceSquare)
	{
		const FVector2f Size = GetSize();
		const int32 ComponentToChange = Size.X > Size.Y ? 1 : 0;

		if (EndPosition[ComponentToChange] < StartPosition[ComponentToChange])
		{
			EndPosition[ComponentToChange] = StartPosition[ComponentToChange] - Size[1 - ComponentToChange];
		}
		else
		{
			EndPosition[ComponentToChange] = StartPosition[ComponentToChange] + Size[1 - ComponentToChange];
		}
	}

	if (bFromCenter)
	{
		StartPosition -= EndPosition - CenterPosition;
	}
}

FVector UAvaInteractiveToolsToolViewportAreaPlanner::ViewportToWorldPosition(const FVector2f& InViewportPosition) const
{
	if (!Tool)
	{
		return FVector::ZeroVector;
	}

	const float CameraDistance = GetDefault<UAvaInteractiveToolsSettings>()->CameraDistance;

	UWorld* TempWorld;
	FVector TempVector = FVector::ZeroVector;
	FRotator TempRotator;

	Tool->ViewportPositionToWorldPositionAndOrientation(EAvaViewportStatus::Focused, InViewportPosition, CameraDistance, TempWorld, TempVector, TempRotator);

	return TempVector;
}
