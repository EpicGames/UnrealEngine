// Copyright Epic Games, Inc. All Rights Reserved.

#include "Planners/AvaInteractiveToolsToolViewportPlanner.h"
#include "AvaViewportUtils.h"
#include "Interaction/AvaSnapOperation.h"
#include "InteractiveToolManager.h"
#include "InteractiveToolsContext.h"
#include "Math/Vector2D.h"
#include "Tools/AvaInteractiveToolsToolBase.h"
#include "UnrealClient.h"
#include "ViewportClient/IAvaViewportClient.h"

UAvaInteractiveToolsToolViewportPlanner::UAvaInteractiveToolsToolViewportPlanner()
{
	Tool = nullptr;
	bAttemptedToCreateSnapOperation = false;
}

void UAvaInteractiveToolsToolViewportPlanner::Setup(UAvaInteractiveToolsToolBase* InTool)
{
	Tool = InTool;
	bAttemptedToCreateSnapOperation = false;
}

void UAvaInteractiveToolsToolViewportPlanner::Shutdown(EToolShutdownType ShutdownType)
{
	SnapOperation.Reset();
}

FViewport* UAvaInteractiveToolsToolViewportPlanner::GetViewport(EAvaViewportStatus InViewportStatus) const
{
	if (Tool)
	{
		return Tool->GetViewport(InViewportStatus);
	}

	return nullptr;
}

FVector2f UAvaInteractiveToolsToolViewportPlanner::GetConstrainedMousePosition() const
{
	return GetConstrainedMousePosition(FVector2f::ZeroVector);
}

FVector2f UAvaInteractiveToolsToolViewportPlanner::GetConstrainedMousePosition(const FVector2f& InClickPos) const
{
	if (!Tool)
	{
		return InClickPos;
	}

	FViewport* Viewport = GetViewport(EAvaViewportStatus::Focused);

	if (!Viewport)
	{
		return InClickPos;
	}

	return GetConstrainedMousePosition(Viewport);
}

FVector2f UAvaInteractiveToolsToolViewportPlanner::GetConstrainedMousePosition(FViewport* InViewport) const
{
	if (TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAvaViewportClient(InViewport))
	{
		return AvaViewportClient->GetConstrainedZoomedViewportMousePosition();
	}

	return FVector2f::ZeroVector;
}

void UAvaInteractiveToolsToolViewportPlanner::StartSnapOperation()
{
	if (!bAttemptedToCreateSnapOperation && Tool)
	{
		bAttemptedToCreateSnapOperation = true;

		if (UInteractiveToolManager* Manager = Tool->GetToolManager())
		{
			if (IToolsContextQueriesAPI* ContextAPI = Manager->GetContextQueriesAPI())
			{
				if (FViewport* Viewport = ContextAPI->GetFocusedViewport())
				{
					if (TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAvaViewportClient(Viewport))
					{
						SnapOperation = AvaViewportClient->StartSnapOperation();

						if (SnapOperation.IsValid())
						{
							SnapOperation->GenerateActorSnapPoints({}, {});
							SnapOperation->FinaliseSnapPoints();
						}
					}
				}
			}
		}
	}
}

void UAvaInteractiveToolsToolViewportPlanner::SnapLocation(FVector2f& InOutLocation)
{
	if (!bAttemptedToCreateSnapOperation && Tool && !SnapOperation.IsValid())
	{
		StartSnapOperation();
	}

	if (SnapOperation.IsValid())
	{
		SnapOperation->SnapScreenLocation(InOutLocation);
	}
}
