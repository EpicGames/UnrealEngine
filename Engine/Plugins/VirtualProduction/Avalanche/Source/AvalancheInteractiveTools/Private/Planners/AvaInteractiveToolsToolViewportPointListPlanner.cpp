// Copyright Epic Games, Inc. All Rights Reserved.

#include "Planners/AvaInteractiveToolsToolViewportPointListPlanner.h"
#include "AvaViewportUtils.h"
#include "AvaVisibleArea.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Framework/Application/SlateApplication.h"
#include "InputState.h"
#include "SceneView.h"
#include "ToolContextInterfaces.h"
#include "Tools/AvaInteractiveToolsToolBase.h"
#include "ViewportClient/IAvaViewportClient.h"

void UAvaInteractiveToolsToolViewportPointListPlanner::OverrideCurrentViewportPosition(const FVector2f InPosition)
{
	CurrentViewportPosition = InPosition;
}

void UAvaInteractiveToolsToolViewportPointListPlanner::AddViewportPosition(const FVector2f& InViewportPosition)
{
	ViewportPositions.Add(InViewportPosition);
	LineStatus = EAvaInteractiveToolsToolViewportPointListPlannerLineStatus::Neutral;
}

void UAvaInteractiveToolsToolViewportPointListPlanner::SetLineStatus(EAvaInteractiveToolsToolViewportPointListPlannerLineStatus InLineStatus)
{
	LineStatus = InLineStatus;
}

void UAvaInteractiveToolsToolViewportPointListPlanner::Setup(UAvaInteractiveToolsToolBase* InTool)
{
	Super::Setup(InTool);

	CurrentViewportPosition = FVector2f::ZeroVector;
}

void UAvaInteractiveToolsToolViewportPointListPlanner::OnTick(float InDeltaTime)
{
	Super::OnTick(InDeltaTime);

	if (Tool)
	{
		CurrentViewportPosition = GetConstrainedMousePosition();
		SnapLocation(CurrentViewportPosition);

		Tool->OnViewportPlannerUpdate();
	}
}

void UAvaInteractiveToolsToolViewportPointListPlanner::DrawHUD(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI)
{
	Super::DrawHUD(InCanvas, InRenderAPI);

	if (ViewportPositions.IsEmpty())
	{
		return;
	}

	static const FLinearColor NeutralColor(0.5, 0.5, 1.0);
	static const FLinearColor AllowedColor(0.5, 1.0, 0.5);
	static const FLinearColor DisallowedColor(1.0, 0.5, 0.5);

	TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAvaViewportClient(GetViewport(EAvaViewportStatus::Focused));

	if (!AvaViewportClient.IsValid())
	{
		return;
	}

	const FAvaVisibleArea VisibleArea = AvaViewportClient->GetZoomedVisibleArea();
	const FVector2f Offset = AvaViewportClient->GetViewportOffset();
	const float AppScale = FSlateApplication::Get().GetApplicationScale();

	FVector2f FirstPoint;
	FVector2f SecondPoint;

	for (int32 PointIdx = 1; PointIdx < ViewportPositions.Num(); ++PointIdx)
	{
		FirstPoint = (VisibleArea.GetVisiblePosition(ViewportPositions[PointIdx]) * AppScale) + Offset;
		SecondPoint = (VisibleArea.GetVisiblePosition(ViewportPositions[PointIdx - 1]) * AppScale) + Offset;

		FCanvasLineItem LineItem(static_cast<FVector2D>(FirstPoint), static_cast<FVector2D>(SecondPoint));
		LineItem.SetColor(NeutralColor);

		InCanvas->DrawItem(LineItem);
	}

	FirstPoint = (VisibleArea.GetVisiblePosition(ViewportPositions.Last()) * AppScale) + Offset;
	SecondPoint = (VisibleArea.GetVisiblePosition(CurrentViewportPosition) * AppScale) + Offset;

	FCanvasLineItem LatestLineItem(static_cast<FVector2D>(FirstPoint), static_cast<FVector2D>(SecondPoint));

	switch (LineStatus)
	{
		default:
		case EAvaInteractiveToolsToolViewportPointListPlannerLineStatus::Neutral:
			LatestLineItem.SetColor(NeutralColor);
			break;

		case EAvaInteractiveToolsToolViewportPointListPlannerLineStatus::Allowed:
			LatestLineItem.SetColor(AllowedColor);
			break;

		case EAvaInteractiveToolsToolViewportPointListPlannerLineStatus::Disallowed:
			LatestLineItem.SetColor(DisallowedColor);
			break;
	}

	InCanvas->DrawItem(LatestLineItem);
}

void UAvaInteractiveToolsToolViewportPointListPlanner::OnClicked(const FInputDeviceRay& InClickPos)
{
	if (Tool)
	{
		FVector2f ScreenPosition = GetConstrainedMousePosition();
		SnapLocation(ScreenPosition);

		ViewportPositions.Add(ScreenPosition);

		Tool->OnViewportPlannerUpdate();
	}
}
