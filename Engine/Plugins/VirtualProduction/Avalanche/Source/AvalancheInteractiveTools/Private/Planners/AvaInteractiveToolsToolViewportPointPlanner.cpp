// Copyright Epic Games, Inc. All Rights Reserved.

#include "Planners/AvaInteractiveToolsToolViewportPointPlanner.h"
#include "InputState.h"
#include "Tools/AvaInteractiveToolsToolBase.h"

void UAvaInteractiveToolsToolViewportPointPlanner::SetViewportPosition(const FVector2f& InViewportPosition)
{
	ViewportPosition = InViewportPosition;
}

void UAvaInteractiveToolsToolViewportPointPlanner::Setup(UAvaInteractiveToolsToolBase* InTool)
{
	Super::Setup(InTool);

	ViewportPosition = FVector2f::ZeroVector;
}

void UAvaInteractiveToolsToolViewportPointPlanner::OnTick(float InDeltaTime)
{
	Super::OnTick(InDeltaTime);

	if (Tool)
	{
		ViewportPosition = GetConstrainedMousePosition();
		SnapLocation(ViewportPosition);

		Tool->OnViewportPlannerUpdate();
	}
}

void UAvaInteractiveToolsToolViewportPointPlanner::OnClicked(const FInputDeviceRay& InClickPos)
{
	if (Tool)
	{
		ViewportPosition = GetConstrainedMousePosition();
		SnapLocation(ViewportPosition);

		Tool->OnViewportPlannerComplete();
	}
}
