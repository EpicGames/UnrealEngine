// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaInteractiveToolsActorAreaToolBase.h"
#include "AvaViewportUtils.h"
#include "AvaVisibleArea.h"
#include "InteractiveToolManager.h"
#include "Planners/AvaInteractiveToolsToolViewportAreaPlanner.h"

UAvaInteractiveToolsActorAreaToolBase::UAvaInteractiveToolsActorAreaToolBase()
{
	ViewportPlannerClass = UAvaInteractiveToolsToolViewportAreaPlanner::StaticClass();
}

AActor* UAvaInteractiveToolsActorAreaToolBase::SpawnActor(TSubclassOf<AActor> InActorClass, EAvaViewportStatus InViewportStatus,
	const FVector2f& InViewportPosition, bool bInPreview, FString* InActorLabelOverride)
{
	AActor* NewActor = Super::SpawnActor(InActorClass, InViewportStatus, InViewportPosition, bInPreview, InActorLabelOverride);

	// When performing the default action, don't set scale.
	if (!bPerformingDefaultAction)
	{
		FBox ActorSizeBox = NewActor->CalculateComponentsBoundingBoxInLocalSpace(false, false);
		const_cast<UAvaInteractiveToolsActorAreaToolBase*>(this)->OriginalActorSize = ActorSizeBox.GetSize();
		SetActorScale(NewActor);
	}

	return NewActor;
}

void UAvaInteractiveToolsActorAreaToolBase::OnViewportPlannerUpdate()
{
	Super::OnViewportPlannerUpdate();

	if (UAvaInteractiveToolsToolViewportAreaPlanner* AreaPlanner = Cast<UAvaInteractiveToolsToolViewportAreaPlanner>(ViewportPlanner))
	{
		if (AreaPlanner->HasStarted())
		{
			if (!PreviewActor)
			{
				PreviewActor = SpawnActor(ActorClass, true);
			}

			if (PreviewActor)
			{
				const FVector2f ViewportPosition = AreaPlanner->GetCenterPosition();

				UWorld* World;
				FVector NewLocation;
				FRotator NewRotation;

				if (ViewportPositionToWorldPositionAndOrientation(EAvaViewportStatus::Focused, ViewportPosition, World, NewLocation, NewRotation))
				{
					PreviewActor->SetActorLocation(NewLocation);
					PreviewActor->SetActorRotation(NewRotation);
				}

				SetActorScale(PreviewActor);
			}
		}
	}
}

void UAvaInteractiveToolsActorAreaToolBase::OnViewportPlannerComplete()
{
	if (UAvaInteractiveToolsToolViewportAreaPlanner* AreaPlanner = Cast<UAvaInteractiveToolsToolViewportAreaPlanner>(ViewportPlanner))
	{
		SpawnedActor = SpawnActor(ActorClass, EAvaViewportStatus::Focused, AreaPlanner->GetCenterPosition(), /* Preview */ false);
	}

	Super::OnViewportPlannerComplete();
}

void UAvaInteractiveToolsActorAreaToolBase::SetActorScale(AActor* InActor) const
{
	if (!InActor)
	{
		return;
	}

	if (UAvaInteractiveToolsToolViewportAreaPlanner* AreaPlanner = Cast<UAvaInteractiveToolsToolViewportAreaPlanner>(ViewportPlanner))
	{
		if (AreaPlanner->HasStarted())
		{
			FVector2D WorldSize = AreaPlanner->GetWorldSize();

			FVector Scale = FVector::OneVector;
			Scale.X = 1.0f;
			Scale.Y = WorldSize.X / OriginalActorSize.Y;
			Scale.Z = WorldSize.Y / OriginalActorSize.Z;

			InActor->SetActorScale3D(Scale);
		}
	}
}
