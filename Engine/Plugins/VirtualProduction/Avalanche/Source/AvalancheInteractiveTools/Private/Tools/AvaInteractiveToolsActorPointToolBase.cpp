// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaInteractiveToolsActorPointToolBase.h"
#include "AvalancheInteractiveToolsModule.h"
#include "GameFramework/Actor.h"
#include "Planners/AvaInteractiveToolsToolViewportPointPlanner.h"
#include "UObject/ConstructorHelpers.h"

UAvaInteractiveToolsActorPointToolBase::UAvaInteractiveToolsActorPointToolBase()
{
	ViewportPlannerClass = UAvaInteractiveToolsToolViewportPointPlanner::StaticClass();
}

void UAvaInteractiveToolsActorPointToolBase::OnViewportPlannerUpdate()
{
	Super::OnViewportPlannerUpdate();

	if (UAvaInteractiveToolsToolViewportPointPlanner* PointPlanner = Cast<UAvaInteractiveToolsToolViewportPointPlanner>(ViewportPlanner))
	{
		if (!PreviewActor)
		{
 			PreviewActor = SpawnActor(ActorClass, /* Preview */ true);
		}

		const bool bUseIdentityLocation = UseIdentityLocation();
		const bool bUseIdentityRotation = UseIdentityRotation();

		if (PreviewActor && (!bUseIdentityLocation || !bUseIdentityRotation))
		{
			UWorld* World;
			FVector NewLocation;
			FRotator NewRotation;

			if (ViewportPositionToWorldPositionAndOrientation(EAvaViewportStatus::Focused, PointPlanner->GetViewportPosition(), World, NewLocation, NewRotation))
			{
				if (!bUseIdentityLocation)
				{
					PreviewActor->SetActorLocation(NewLocation);
				}

				if (!bUseIdentityRotation)
				{
					PreviewActor->SetActorRotation(NewRotation);
				}
			}
		}
	}
}

void UAvaInteractiveToolsActorPointToolBase::OnViewportPlannerComplete()
{
	if (UAvaInteractiveToolsToolViewportPointPlanner* PointPlanner = Cast<UAvaInteractiveToolsToolViewportPointPlanner>(ViewportPlanner))
	{
		SpawnedActor = SpawnActor(ActorClass, EAvaViewportStatus::Focused, PointPlanner->GetViewportPosition(), /* Preview */ false);
	}

	Super::OnViewportPlannerComplete();
}