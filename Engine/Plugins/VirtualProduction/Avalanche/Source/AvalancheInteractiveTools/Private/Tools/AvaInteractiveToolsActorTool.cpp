// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaInteractiveToolsActorTool.h"
#include "AvalancheInteractiveToolsModule.h"
#include "Builders/AvaInteractiveToolsActorToolBuilder.h"

UAvaInteractiveToolsActorTool::UAvaInteractiveToolsActorTool()
{
}

void UAvaInteractiveToolsActorTool::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	FAvaInteractiveToolsToolParameters ToolParameters = UAvaInteractiveToolsActorToolBuilder::CreateToolParameters(
	    Category,
	    Command,
	    Identifier,
	    Priority,
	    ActorClass,
	    GetClass()
    );
	
	InAITModule->RegisterTool(Category, MoveTemp(ToolParameters));
}