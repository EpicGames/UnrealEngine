// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaInteractiveToolsActorToolBase.h"

#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "GameFramework/Actor.h"

bool UAvaInteractiveToolsActorToolBase::OnBegin()
{
	if (ActorClass == nullptr)
	{
		return false;
	}

	return Super::OnBegin();
}

void UAvaInteractiveToolsActorToolBase::DefaultAction()
{
	if (OnBegin())
	{
		SpawnedActor = SpawnActor(ActorClass, /** Preview */false);

		OnComplete();
	}

	Super::DefaultAction();
}

bool UAvaInteractiveToolsActorToolBase::UseIdentityRotation() const
{
	return ConditionalIdentityRotation();
}

FAvaInteractiveToolsToolParameters UAvaInteractiveToolsActorToolBase::CreateDefaultToolParameters() const
{
	FAvaInteractiveToolsToolParameters Parameters;
	Parameters.ToolIdentifier = GetClass()->GetName();
	Parameters.CreateBuilder = FAvalancheInteractiveToolsCreateBuilder::CreateLambda(
		[this](UEdMode* InEdMode)
		{
			return UAvaInteractiveToolsToolBuilder::CreateToolBuilder(InEdMode, GetClass());
		});
	Parameters.ActorFactory.Set<TSubclassOf<AActor>>(ActorClass);
	return Parameters;
}
