// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaInteractiveToolsActorPointToolBase.h"
#include "AvaInteractiveToolsActorToolNull.generated.h"

UCLASS()
class UAvaInteractiveToolsActorToolNull : public UAvaInteractiveToolsActorPointToolBase
{
	GENERATED_BODY()

public:
	UAvaInteractiveToolsActorToolNull();

	//~ Begin UAvaInteractiveToolsToolBase
	virtual bool UseIdentityRotation() const override { return true; }
	virtual void OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule) override;
	//~ End UAvaInteractiveToolsToolBase
};
