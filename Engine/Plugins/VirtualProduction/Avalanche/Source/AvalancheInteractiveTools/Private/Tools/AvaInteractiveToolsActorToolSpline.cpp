// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaInteractiveToolsActorToolSpline.h"
#include "AvaInteractiveToolsCommands.h"
#include "AvalancheInteractiveToolsModule.h"
#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "Framework/AvaSplineActor.h"

UAvaInteractiveToolsActorToolSpline::UAvaInteractiveToolsActorToolSpline()
{
	ActorClass = AAvaSplineActor::StaticClass();
}

void UAvaInteractiveToolsActorToolSpline::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	FAvaInteractiveToolsToolParameters ToolParameters = CreateDefaultToolParameters();
	ToolParameters.UICommand = FAvaInteractiveToolsCommands::Get().Tool_Actor_Spline;
	ToolParameters.Priority = 5000;

	InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryNameActor, MoveTemp(ToolParameters));
}
