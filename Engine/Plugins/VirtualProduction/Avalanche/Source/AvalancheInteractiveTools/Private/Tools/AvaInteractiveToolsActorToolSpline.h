// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaInteractiveToolsActorPointToolBase.h"
#include "AvaInteractiveToolsActorToolSpline.generated.h"

UCLASS()
class UAvaInteractiveToolsActorToolSpline : public UAvaInteractiveToolsActorPointToolBase
{
	GENERATED_BODY()

public:
	UAvaInteractiveToolsActorToolSpline();

	//~ Begin UAvaInteractiveToolsToolBase
	virtual bool UseIdentityRotation() const override { return false; }
	virtual void OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule) override;
	//~ End UAvaInteractiveToolsToolBase
};
