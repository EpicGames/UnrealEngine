// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaInteractiveToolsStaticMeshActorTool.h"
#include "AvalancheInteractiveToolsModule.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMeshActor.h"
#include "Builders/AvaInteractiveToolsStaticMeshActorToolBuilder.h"

UAvaInteractiveToolsStaticMeshActorTool::UAvaInteractiveToolsStaticMeshActorTool()
{
	ActorClass = AStaticMeshActor::StaticClass();
}

void UAvaInteractiveToolsStaticMeshActorTool::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	FAvaInteractiveToolsToolParameters ToolParameters = UAvaInteractiveToolsStaticMeshActorToolBuilder::CreateToolParameters(
		Category,
		Command,
		Identifier,
		Priority,
		StaticMesh,
		GetClass()
	);

	InAITModule->RegisterTool(Category, MoveTemp(ToolParameters));
}

AActor* UAvaInteractiveToolsStaticMeshActorTool::SpawnActor(TSubclassOf<AActor> InActorClass, EAvaViewportStatus InViewportStatus,
	const FVector2f& InViewportPosition, bool bInPreview, FString* InActorLabelOverride)
{
	AActor* Actor = Super::SpawnActor(InActorClass, InViewportStatus, InViewportPosition, bInPreview, InActorLabelOverride);

	if (AStaticMeshActor* StaticMeshActor = Cast<AStaticMeshActor>(Actor))
	{
		StaticMeshActor->GetStaticMeshComponent()->SetStaticMesh(StaticMesh);
		StaticMeshActor->SetMobility(EComponentMobility::Type::Movable);
	}

	return Actor;
}
