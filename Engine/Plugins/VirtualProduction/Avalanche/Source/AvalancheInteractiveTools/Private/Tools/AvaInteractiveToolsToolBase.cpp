// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaInteractiveToolsToolBase.h"

#include "ActorFactories/ActorFactory.h"
#include "AvaInteractiveToolsSettings.h"
#include "AvalancheInteractiveToolsModule.h"
#include "AvaViewportUtils.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/SingleKeyCaptureBehavior.h"
#include "ContextObjectStore.h"
#include "Editor/EditorEngine.h"
#include "EditorViewportClient.h"
#include "EdMode/AvaInteractiveToolsEdMode.h"
#include "Engine/World.h"
#include "EngineAnalytics.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/InputSettings.h"
#include "IAvaInteractiveToolsModeDetailsObject.h"
#include "IAvaInteractiveToolsModeDetailsObjectProvider.h"
#include "InteractiveToolManager.h"
#include "InteractiveToolsContext.h"
#include "Planners/AvaInteractiveToolsToolViewportAreaPlanner.h"
#include "Planners/AvaInteractiveToolsToolViewportPlanner.h"
#include "Planners/AvaInteractiveToolsToolViewportPointPlanner.h"
#include "ToolContextInterfaces.h"
#include "Toolkits/BaseToolkit.h"
#include "UnrealClient.h"
#include "ViewportClient/IAvaViewportClient.h"

#define LOCTEXT_NAMESPACE "AvaInteractiveToolsToolBase"

UAvaInteractiveToolsRightClickBehavior::UAvaInteractiveToolsRightClickBehavior()
{
	HitTestOnRelease = false;
	SetUseRightMouseButton();
}

void UAvaInteractiveToolsRightClickBehavior::Clicked(const FInputDeviceState& Input, const FInputCaptureData& Data)
{
	Target->OnUpdateModifierState(UAvaInteractiveToolsToolBase::BID_Cancel, true);
}

UObject* UAvaInteractiveToolsToolBase::GetDetailsObjectFromActor(AActor* InActor)
{
	if (!InActor)
	{
		return nullptr;
	}

	if (InActor->Implements<UAvaInteractiveToolsModeDetailsObject>())
	{
		return InActor;
	}
	else if (InActor->Implements<UAvaInteractiveToolsModeDetailsObjectProvider>())
	{
		return IAvaInteractiveToolsModeDetailsObjectProvider::Execute_GetModeDetailsObject(InActor);
	}

	TArray<UActorComponent*> Components;
	InActor->GetComponents<UActorComponent>(Components);

	for (UActorComponent* Component : Components)
	{
		if (Component->Implements<UAvaInteractiveToolsModeDetailsObject>())
		{
			return Component;
		}
		else if (Component->Implements<UAvaInteractiveToolsModeDetailsObjectProvider>())
		{
			if (UObject* DetailsObject = IAvaInteractiveToolsModeDetailsObjectProvider::Execute_GetModeDetailsObject(Component))
			{
				return DetailsObject;
			}
		}
	}

	return nullptr;
}

void UAvaInteractiveToolsToolBase::Setup()
{
	Super::Setup();

	UInteractiveToolManager* ToolManager = GetToolManager();
	bool bReactivated = false;
	UAvaInteractiveToolsEdMode* AvaInteractiveToolsEdMode = nullptr;

	if (const UContextObjectStore* ContextStore = ToolManager->GetContextObjectStore())
	{
		AvaInteractiveToolsEdMode = Cast<UAvaInteractiveToolsEdMode>(ContextStore->FindContextByClass(UAvaInteractiveToolsEdMode::StaticClass()));

		if (AvaInteractiveToolsEdMode)
		{
			// I found double-clicking inconsistent at the default speed. 
			static const double ReactivateDelayTime = GetDefault<UInputSettings>()->DoubleClickTime + 0.1;

			const bool bIsDoubleClick = (FApp::GetCurrentTime() - AvaInteractiveToolsEdMode->GetLastToolActivateTime()) <= ReactivateDelayTime;
			const bool bIsSameTool = AvaInteractiveToolsEdMode->GetLastActiveTool() == ToolManager->GetActiveToolName(EToolSide::Left);

			if (bIsDoubleClick && bIsSameTool && SupportsDefaultAction())
			{
				bReactivated = true;
			}
		}
	}

	if (AvaInteractiveToolsEdMode)
	{
		AvaInteractiveToolsEdMode->OnToolSetup(this);
		Activate(AvaInteractiveToolsEdMode->GetLastActiveTool(), bReactivated);
		AvaInteractiveToolsEdMode->OnToolActivateEnd();
	}
}

void UAvaInteractiveToolsToolBase::Shutdown(EToolShutdownType ShutdownType)
{
	switch (ShutdownType)
	{
		case EToolShutdownType::Cancel:
			OnCancel();
			break;

		case EToolShutdownType::Accept:
		case EToolShutdownType::Completed:
			OnComplete();
			break;
	}

	if (ViewportPlanner)
	{
		ViewportPlanner->Shutdown(ShutdownType);
		ViewportPlanner = nullptr;
	}

	Super::Shutdown(ShutdownType);
}

void UAvaInteractiveToolsToolBase::DrawHUD(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI)
{
	Super::DrawHUD(InCanvas, InRenderAPI);

	if (ViewportPlanner)
	{
		ViewportPlanner->DrawHUD(InCanvas, InRenderAPI);
	}
}

void UAvaInteractiveToolsToolBase::Render(IToolsContextRenderAPI* InRenderAPI)
{
	Super::Render(InRenderAPI);

	if (ViewportPlanner)
	{
		ViewportPlanner->Render(InRenderAPI);
	}
}

void UAvaInteractiveToolsToolBase::OnTick(float InDeltaTime)
{
	Super::OnTick(InDeltaTime);

	if (ViewportPlanner)
	{
		ViewportPlanner->OnTick(InDeltaTime);
	}
}

bool UAvaInteractiveToolsToolBase::SupportsDefaultAction() const
{
	return ViewportPlannerClass == UAvaInteractiveToolsToolViewportPointPlanner::StaticClass()
		|| ViewportPlannerClass == UAvaInteractiveToolsToolViewportAreaPlanner::StaticClass();
}

void UAvaInteractiveToolsToolBase::DefaultAction()
{
	RequestShutdown(EToolShutdownType::Completed);
}

bool UAvaInteractiveToolsToolBase::ConditionalIdentityRotation() const
{
	switch (GetDefault<UAvaInteractiveToolsSettings>()->DefaultActionActorAlignment)
	{
		case EAvaInteractiveToolsDefaultActionAlignment::Camera:
			if (!bPerformingDefaultAction)
			{
				return false;
			}

			return !IsMotionDesignViewport();

		default:
		case EAvaInteractiveToolsDefaultActionAlignment::Axis:
			return bPerformingDefaultAction;
	}
}

AActor* UAvaInteractiveToolsToolBase::SpawnActor(TSubclassOf<AActor> InActorClass, bool bInPreview, FString* InActorLabelOverride)
{
	if (const IToolsContextQueriesAPI* ContextAPI = GetToolManager()->GetContextQueriesAPI())
	{
		if (FViewport* Viewport = ContextAPI->GetFocusedViewport())
		{
			if (FEditorViewportClient* ViewportClient = FAvaViewportUtils::GetAsEditorViewportClient(Viewport))
			{
				if (const TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(ViewportClient))
				{
					const FVector2f ViewportSize = AvaViewportClient->GetViewportSize();
					return SpawnActor(InActorClass, EAvaViewportStatus::Focused, ViewportSize * 0.5, bInPreview, InActorLabelOverride);
				}

				const FVector2f ViewportSize = Viewport->GetSizeXY();
				return SpawnActor(InActorClass, EAvaViewportStatus::Focused, ViewportSize * 0.5, bInPreview, InActorLabelOverride);
			}
		}
	}

	return nullptr;
}

AActor* UAvaInteractiveToolsToolBase::SpawnActor(TSubclassOf<AActor> InActorClass, EAvaViewportStatus InViewportStatus, 
	const FVector2f& InViewportPosition, bool bInPreview, FString* InActorLabelOverride)
{
	auto GetSpawnContext = [this, &InViewportPosition, &InViewportStatus](UWorld*& OutWorld, FVector& OutSpawnLocation, FRotator& OutSpawnRotation)->bool
	{
		bool bSuccess = false;

		if (const IToolsContextQueriesAPI* ContextAPI = GetToolManager()->GetContextQueriesAPI())
		{
			const bool bUseIdentityLocation = UseIdentityLocation();
			const bool bUseIdentityRotation = UseIdentityRotation();

			if (FViewport* Viewport = ContextAPI->GetFocusedViewport())
			{
				FVector CameraForward = FVector(1, 0, 0);

				if (const TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAvaViewportClient(Viewport))
				{
					CameraForward = AvaViewportClient->GetViewportViewTransform().TransformVectorNoScale(FVector::ForwardVector);
					CameraForward.Z = 0;
				}

				bSuccess = ViewportPositionToWorldPositionAndOrientation(
					InViewportStatus,
					InViewportPosition,
					OutWorld,
					OutSpawnLocation,
					OutSpawnRotation
				);

				if (!bSuccess || !IsValid(OutWorld))
				{
					return false;
				}

				if (bUseIdentityLocation)
				{
					OutSpawnLocation = FVector::ZeroVector;
				}

				if (bUseIdentityRotation)
				{
					if (CameraForward.Dot(FVector::ForwardVector) >= 0)
					{
						OutSpawnRotation = FRotator::ZeroRotator;
					}
					else
					{
						OutSpawnRotation = FRotator(0, 180, 0);
					}
				}
			}
		}

		return bSuccess;
	};
	
	UWorld* World = nullptr;
	FVector SpawnLocation = FVector::ZeroVector;
	FRotator SpawnRotation = FRotator::ZeroRotator;
	if (!GetSpawnContext(World, SpawnLocation, SpawnRotation))
	{
		return nullptr;
	}

	FActorSpawnParameters SpawnParams;
	SpawnParams.bNoFail = true;

	const bool bUseOverride = !bInPreview && InActorLabelOverride && !InActorLabelOverride->IsEmpty();

	if (bInPreview)
	{
		SpawnParams.bHideFromSceneOutliner = true;
		SpawnParams.bTemporaryEditorActor = true;
		SpawnParams.Name = FName(TEXT("AvaITFPreviewActor"));
		SpawnParams.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
		SpawnParams.ObjectFlags |= RF_Transient;
	}
	else
	{
		// No longer needed if we spawn final actor
		DestroyPreviewActor();

		if (bUseOverride)
		{
			SpawnParams.Name = FName(*InActorLabelOverride);
			SpawnParams.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
			SpawnParams.ObjectFlags |= RF_Transactional;
		}
	}

	UActorFactory* ActorFactory = nullptr;
	if (const FAvaInteractiveToolsToolParameters* ActiveToolParameters = GetActiveToolParameters())
	{
		if (ActiveToolParameters->ActorFactory.IsType<TObjectPtr<UActorFactory>>())
		{
			ActorFactory = ActiveToolParameters->ActorFactory.Get<TObjectPtr<UActorFactory>>();
		}
		else if (ActiveToolParameters->ActorFactory.IsType<TSubclassOf<AActor>>())
		{
			ActorFactory = GEditor->FindActorFactoryByClass(ActiveToolParameters->ActorFactory.Get<TSubclassOf<AActor>>());
		}
	}

	if (!ActorFactory)
	{
		ActorFactory = GEditor->FindActorFactoryForActorClass(InActorClass);
	}

	AActor* NewActor = nullptr;

	if (ActorFactory)
	{
		const FTransform SpawnTransform(SpawnRotation, SpawnLocation);
		NewActor = ActorFactory->CreateActor(InActorClass.Get(), World->PersistentLevel, SpawnTransform, SpawnParams);
	}

	if (!NewActor)
	{
		NewActor = World->SpawnActor<AActor>(InActorClass, SpawnLocation, SpawnRotation, SpawnParams);
	}

	checkf(!!NewActor, TEXT("Motion Design Interactive Tool : Failed to spawn actor of class %s"), *InActorClass->GetName())
	NewActor->bIsEditorPreviewActor = bInPreview;

	if (bInPreview)
	{
		// Cancel tool when preview actor is deleted in outliner to avoid spawning new one
		NewActor->OnDestroyed.AddUniqueDynamic(this, &UAvaInteractiveToolsToolBase::OnPreviewActorDestroyed);
	}

	if (bUseOverride)
	{
		FActorLabelUtilities::SetActorLabelUnique(NewActor, *InActorLabelOverride);
	}
	// Actor factory handles actor label
	else if (!ActorFactory)
	{
		FActorLabelUtilities::SetActorLabelUnique(NewActor, NewActor->GetDefaultActorLabel());
	}

	if (UObject* DetailsObject = GetDetailsObjectFromActor(NewActor))
	{
		SetToolkitSettingsObject(DetailsObject);
	}

	OnActorSpawned(NewActor);

	return NewActor;
}

void UAvaInteractiveToolsToolBase::SetToolkitSettingsObject(UObject* InObject) const
{
	if (const UInteractiveToolManager* ToolManager = GetToolManager())
	{
		if (const UContextObjectStore* ContextStore = ToolManager->GetContextObjectStore())
		{
			if (UAvaInteractiveToolsEdMode* AvaInteractiveToolsEdMode =
				Cast<UAvaInteractiveToolsEdMode>(ContextStore->FindContextByClass(UAvaInteractiveToolsEdMode::StaticClass())))
			{
				if (const TSharedPtr<FModeToolkit> Toolkit = AvaInteractiveToolsEdMode->GetToolkit().Pin())
				{
					Toolkit->SetModeSettingsObject(InObject);
				}
			}
		}
	}
}

bool UAvaInteractiveToolsToolBase::ViewportPositionToWorldPositionAndOrientation(EAvaViewportStatus InViewportStatus, 
	const FVector2f& InViewportPosition, UWorld*& OutWorld, FVector& OutPosition, FRotator& OutRotation) const
{
	const float CameraDistance = GetDefault<UAvaInteractiveToolsSettings>()->CameraDistance;

	return ViewportPositionToWorldPositionAndOrientation(InViewportStatus, InViewportPosition, CameraDistance, OutWorld, OutPosition, OutRotation);
}

bool UAvaInteractiveToolsToolBase::ViewportPositionToWorldPositionAndOrientation(EAvaViewportStatus InViewportStatus,
	const FVector2f& InViewportPosition, float InDistance, UWorld*& OutWorld, FVector& OutPosition, FRotator& OutRotation) const
{
	if (const IToolsContextQueriesAPI* ContextAPI = GetToolManager()->GetContextQueriesAPI())
	{
		if (UWorld* World = ContextAPI->GetCurrentEditingWorld())
		{
			if (const TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAvaViewportClient(GetViewport(InViewportStatus)))
			{
				if (const FEditorViewportClient* EditorViewportClient = AvaViewportClient->AsEditorViewportClient())
				{
					const FViewportCameraTransform ViewTransform = EditorViewportClient->GetViewTransform();

					OutWorld = World;
					OutRotation = ViewTransform.GetRotation();
					OutPosition = AvaViewportClient->ViewportPositionToWorldPosition(InViewportPosition, InDistance);

					return true;
				}
			}
		}
	}

	return false;
}

FViewport* UAvaInteractiveToolsToolBase::GetViewport(EAvaViewportStatus InViewportStatus) const
{
	if (UInteractiveToolManager* ToolManager = GetToolManager())
	{
		if (const IToolsContextQueriesAPI* ContextAPI = ToolManager->GetContextQueriesAPI())
		{
			FViewport* Viewport;

			switch (InViewportStatus)
			{
				case EAvaViewportStatus::Hovered:
					Viewport = ContextAPI->GetHoveredViewport();
					break;

				case EAvaViewportStatus::Focused:
					Viewport = ContextAPI->GetFocusedViewport();
					break;

				default:
					checkNoEntry();
					return nullptr;
			}

			if (FAvaViewportUtils::GetAsEditorViewportClient(Viewport))
			{
				return Viewport;
			}
		}
	}

	return nullptr;
}

void UAvaInteractiveToolsToolBase::OnViewportPlannerComplete()
{
	DestroyPreviewActor();

	RequestShutdown(EToolShutdownType::Completed);
}

void UAvaInteractiveToolsToolBase::OnActorSpawned(AActor* InActor)
{
	if (!InActor || InActor->bIsEditorPreviewActor)
	{
		return;
	}

	// Delay to select actor after setup completed
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateWeakLambda(InActor,
		[InActor](float InDelta)
		{
			if (GEditor)
			{
				GEditor->SelectNone(true, true);
				GEditor->SelectActor(InActor, true, true, false, true);
			}
			return false;
		}));
}

FInputRayHit UAvaInteractiveToolsToolBase::CanBeginSingleClickAndDragSequence(const FInputDeviceRay& InPressPos)
{
	// Always hits every place in the viewport
	return FInputRayHit(0);
}

void UAvaInteractiveToolsToolBase::OnClickPress(const FInputDeviceRay& InPressPos)
{
	// Nothing
}

void UAvaInteractiveToolsToolBase::OnDragStart(const FInputDeviceRay& InDragPos)
{
	if (LeftClickBehavior)
	{
		// Fake a click at the start position
		OnClickRelease(LeftClickBehavior->GetInitialMouseDownRay(), true);
	}
}

void UAvaInteractiveToolsToolBase::OnClickDrag(const FInputDeviceRay& InDragPos)
{
}

void UAvaInteractiveToolsToolBase::OnClickRelease(const FInputDeviceRay& InReleasePos, bool bInIsDragOperation)
{
	if (ViewportPlanner)
	{
		ViewportPlanner->OnClicked(InReleasePos);
	}
}

void UAvaInteractiveToolsToolBase::OnTerminateSingleClickAndDragSequence()
{
	RequestShutdown(EToolShutdownType::Cancel);
}

FInputRayHit UAvaInteractiveToolsToolBase::IsHitByClick(const FInputDeviceRay& InClickPos)
{
	return FInputRayHit(0);
}

void UAvaInteractiveToolsToolBase::OnClicked(const FInputDeviceRay& InClickPos)
{
	// Right click cancel
	RequestShutdown(EToolShutdownType::Cancel);
}

FString UAvaInteractiveToolsToolBase::GetActiveToolIdentifier() const
{
	return FAvalancheInteractiveToolsModule::Get().GetActiveToolIdentifier();
}

const FAvaInteractiveToolsToolParameters* UAvaInteractiveToolsToolBase::GetActiveToolParameters() const
{
	return FAvalancheInteractiveToolsModule::Get().GetTool(GetActiveToolIdentifier());
}

void UAvaInteractiveToolsToolBase::OnUpdateModifierState(int InModifierID, bool bInIsOn)
{
	switch (InModifierID)
	{
		case BID_Cancel:
			if (bInIsOn)
			{
				RequestShutdown(EToolShutdownType::Cancel);
			}

			break;
		default: ;
	}
}

bool UAvaInteractiveToolsToolBase::CanActivate(const FString& InToolIdentifier, bool bInReactivate) const
{
	if (!bInReactivate)
	{

		return true;
	}

	return SupportsDefaultAction();
}

void UAvaInteractiveToolsToolBase::Activate(const FString& InToolIdentifier, bool bInReactivate)
{
	if (!CanActivate(InToolIdentifier, bInReactivate))
	{
		RequestShutdown(EToolShutdownType::Cancel);
		return;
	}

	FAvalancheInteractiveToolsModule::Get().OnToolActivated(InToolIdentifier);
	
	const bool bSupportsDefaultAction = SupportsDefaultAction();
	const bool bForceDefaultAction = ShouldForceDefaultAction();

	if (!bForceDefaultAction && (!bInReactivate || !bSupportsDefaultAction))
	{
		OnActivate();

		if (!OnBegin())
		{
			RequestShutdown(EToolShutdownType::Cancel);
			return;
		}

		OnPostBegin();
	}
	else if (bSupportsDefaultAction)
	{
		bPerformingDefaultAction = true;
		DefaultAction();
	}
	else
	{
		if (bForceDefaultAction)
		{
			UE_LOGF(LogAvaInteractiveTools, Warning, "Tool forced into using the default action, but it does not support default action.");
		}

		RequestShutdown(EToolShutdownType::Cancel);
	}
}

void UAvaInteractiveToolsToolBase::OnActivate()
{
}

bool UAvaInteractiveToolsToolBase::OnBegin()
{
	if (ViewportPlannerClass == nullptr)
	{
		return false;
	}

	BeginTransaction();
	return true;
}

void UAvaInteractiveToolsToolBase::OnPostBegin()
{
	LeftClickBehavior = NewObject<UAvaSingleClickAndDragBehavior>(this);
	LeftClickBehavior->Initialize(this);
	LeftClickBehavior->bSupportsDrag = ViewportPlannerClass.Get() && ViewportPlannerClass->IsChildOf<UAvaInteractiveToolsToolViewportAreaPlanner>();
	AddInputBehavior(LeftClickBehavior);

	RightClickBehavior = NewObject<UAvaInteractiveToolsRightClickBehavior>(this);
	RightClickBehavior->Initialize(this);
	AddInputBehavior(RightClickBehavior);

	EscapeKeyBehavior = NewObject<USingleKeyCaptureBehavior>(this);
	EscapeKeyBehavior->Initialize(static_cast<IClickBehaviorTarget*>(this), BID_Cancel, EKeys::Escape);
	AddInputBehavior(EscapeKeyBehavior);

	ViewportPlanner = NewObject<UAvaInteractiveToolsToolViewportPlanner>(this, ViewportPlannerClass);
	ViewportPlanner->Setup(this);
}

void UAvaInteractiveToolsToolBase::OnCancel()
{
	CancelTransaction();

	DestroyPreviewActor();

	SetToolkitSettingsObject(nullptr);
}

void UAvaInteractiveToolsToolBase::OnComplete()
{
	EndTransaction();

	DestroyPreviewActor();

	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Reserve(2);
		Attributes.Emplace(TEXT("ToolClass"), GetClass()->GetName());
		if (SpawnedActor)
		{
			Attributes.Emplace(TEXT("ActorClass"), GetNameSafe(SpawnedActor->GetClass()));
		}
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MotionDesign.PlaceActor"), Attributes);
	}
}

void UAvaInteractiveToolsToolBase::BeginTransaction()
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("MotionDesignInteractiveToolsTool", "Motion Design Interactive Tools Box Tool"));
}

void UAvaInteractiveToolsToolBase::EndTransaction()
{
	GetToolManager()->EndUndoTransaction();
}

void UAvaInteractiveToolsToolBase::CancelTransaction()
{
	// Doesn't exist
	//GetToolManager()->CancelUndoTransaction();
	GetToolManager()->EndUndoTransaction();
}

void UAvaInteractiveToolsToolBase::RequestShutdown(EToolShutdownType InShutdownType)
{
	if (const UContextObjectStore* ContextStore = GetToolManager()->GetContextObjectStore())
	{
		if (UAvaInteractiveToolsEdMode* AvaInteractiveToolsEdMode = Cast<UAvaInteractiveToolsEdMode>(ContextStore->FindContextByClass(UAvaInteractiveToolsEdMode::StaticClass())))
		{
			AvaInteractiveToolsEdMode->OnToolShutdown(this, InShutdownType);
		}
	}

	if (ViewportPlanner)
	{
		ViewportPlanner->Shutdown(InShutdownType);
	}

	SetToolkitSettingsObject(nullptr);

	GetToolManager()->PostActiveToolShutdownRequest(this, InShutdownType);

	FAvalancheInteractiveToolsModule::Get().OnToolDeactivated();
}

bool UAvaInteractiveToolsToolBase::IsMotionDesignViewport() const
{
	if (const IToolsContextQueriesAPI* ContextAPI = GetToolManager()->GetContextQueriesAPI())
	{
		if (FViewport* Viewport = ContextAPI->GetFocusedViewport())
		{
			if (const TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAvaViewportClient(Viewport))
			{
				return AvaViewportClient->IsMotionDesignViewport();
			}
		}
	}

	return false;
}

bool UAvaInteractiveToolsToolBase::ShouldForceDefaultAction() const
{
	return FSlateApplication::Get().GetModifierKeys().IsAltDown();
}

void UAvaInteractiveToolsToolBase::DestroyPreviewActor()
{
	if (PreviewActor)
	{
		PreviewActor->OnDestroyed.RemoveAll(this);
		PreviewActor->Destroy();
		PreviewActor = nullptr;
	}
}

void UAvaInteractiveToolsToolBase::OnPreviewActorDestroyed(AActor* InDestroyedActor)
{
	RequestShutdown(EToolShutdownType::Cancel);
}

#undef LOCTEXT_NAMESPACE
