// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAvaInteractiveToolsToolbarCategoryButton.h"

#include "AvaInteractiveToolsSettings.h"
#include "Application/ThrottleManager.h"
#include "AvaInteractiveToolsStyle.h"
#include "Framework/Application/IMenu.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IAvalancheInteractiveToolsModule.h"
#include "Widgets/Images/SImage.h"

void SAvaInteractiveToolsToolbarCategoryButton::Construct(const FArguments& InArgs)
{
	CommandList = InArgs._CommandList;
	bShowLabel = InArgs._ShowLabel;
	ToolCategory = InArgs._ToolCategory;
	Command = InArgs._Command;

	check(CommandList.IsValid())

	if (!ToolCategory.IsNone())
	{
		IAvalancheInteractiveToolsModule& AITModule = IAvalancheInteractiveToolsModule::Get();
		check(AITModule.GetCategories().Contains(ToolCategory) && !AITModule.GetTools(ToolCategory)->IsEmpty())
		ActiveToolIdentifier = (*AITModule.GetTools(ToolCategory))[0].ToolIdentifier;
	}
	else
	{
		check(Command.IsValid())
	}

	CreateActiveCommandWidget();

	IAvalancheInteractiveToolsModule::Get().OnToolActivation().AddSP(this, &SAvaInteractiveToolsToolbarCategoryButton::OnToolActivated);
}

SAvaInteractiveToolsToolbarCategoryButton::~SAvaInteractiveToolsToolbarCategoryButton()
{
	IAvalancheInteractiveToolsModule::Get().OnToolActivation().RemoveAll(this);
}

FReply SAvaInteractiveToolsToolbarCategoryButton::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		if (!ToolCategory.IsNone())
		{
			ShowCommandsContextMenu();
		}

		return FReply::Handled();
	}

	return SCompoundWidget::OnMouseButtonDown(InGeometry, InMouseEvent);
}

void SAvaInteractiveToolsToolbarCategoryButton::CreateActiveCommandWidget()
{
	FVerticalToolBarBuilder CategoryToolbarBuilder(
			CommandList,
			FMultiBoxCustomization::None,
			TSharedPtr<FExtender>()
			, /* ForceSmallIcons */ true);

	CategoryToolbarBuilder.SetStyle(&FAvaInteractiveToolsStyle::Get(), "ViewportToolbar");
	CategoryToolbarBuilder.SetLabelVisibility(bShowLabel ? EVisibility::Visible : EVisibility::Collapsed);

	if (!ToolCategory.IsNone())
	{
		IAvalancheInteractiveToolsModule& AITModule = IAvalancheInteractiveToolsModule::Get();
		const FAvaInteractiveToolsToolParameters* Tool = AITModule.GetTool(ActiveToolIdentifier);

		if (Tool && Tool->UICommand.IsValid())
		{
			CategoryToolbarBuilder.AddToolBarButton(Tool->UICommand);
		}
	}
	else
	{
		CategoryToolbarBuilder.AddToolBarButton(Command);
	}

	ChildSlot
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		.ZOrder(1)
		.HAlign(HAlign_Right)
		.VAlign(bShowLabel ? VAlign_Top : VAlign_Bottom)
		.Padding(1.f)
		[
			SNew(SImage)
			.DesiredSizeOverride(FVector2D(8, 8))
			// Show dropdown chevron
			.Visibility(!ToolCategory.IsNone() ? EVisibility::HitTestInvisible : EVisibility::Collapsed)
			.Image(FAvaInteractiveToolsStyle::Get().GetBrush("AvaInteractiveTools.Dropdown"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
		+ SOverlay::Slot()
		.ZOrder(0)
		[
			SNew(SBox)
			.WidthOverride(bShowLabel ? 38.f : FOptionalSize())
			.HeightOverride(bShowLabel ? 38.f : FOptionalSize())
			[
				CategoryToolbarBuilder.MakeWidget()
			]
		]
	];
}

void SAvaInteractiveToolsToolbarCategoryButton::ShowCommandsContextMenu()
{
	HideCommandsContextMenu();

	// Needed otherwise cannot select entries in context toolbar below
	FSlateThrottleManager::Get().DisableThrottle(true);

	FVector2D MenuPosition = FSlateApplication::Get().GetLastCursorPos();

	if (UAvaInteractiveToolsSettings* ToolsSettings = UAvaInteractiveToolsSettings::Get())
	{
		const FGeometry& WidgetGeometry = GetCachedGeometry();

		FVector2D LocalPosition;
		if (ToolsSettings->ViewportToolbarPosition == EAvaInteractiveToolsViewportToolbarPosition::Bottom
			|| ToolsSettings->ViewportToolbarPosition == EAvaInteractiveToolsViewportToolbarPosition::Top)
		{
			LocalPosition = FVector2D(0, WidgetGeometry.GetLocalSize().Y);
		}
		else if (ToolsSettings->ViewportToolbarPosition == EAvaInteractiveToolsViewportToolbarPosition::Left
			|| ToolsSettings->ViewportToolbarPosition == EAvaInteractiveToolsViewportToolbarPosition::Right)
		{
			LocalPosition = FVector2D(WidgetGeometry.GetLocalSize().X, 0);
		}

        MenuPosition = WidgetGeometry.LocalToAbsolute(LocalPosition);
	}

	FMenuBuilder CategoryToolbarBuilder(/** CloseAfterSelection */true, CommandList);
	CategoryToolbarBuilder.SetSearchable(false);
	CategoryToolbarBuilder.SetCheckBoxStyle("TransparentCheckBox");

	IAvalancheInteractiveToolsModule& AITModule = IAvalancheInteractiveToolsModule::Get();
	for (const FAvaInteractiveToolsToolParameters& Tool : (*AITModule.GetTools(ToolCategory)))
	{
		if (Tool.UICommand.IsValid())
		{
			CategoryToolbarBuilder.AddMenuEntry(Tool.UICommand);
		}
	}

	ContextMenu = FSlateApplication::Get().PushMenu(
		FSlateApplication::Get().GetActiveTopLevelWindow().ToSharedRef(),
		FWidgetPath(),
		CategoryToolbarBuilder.MakeWidget(),
		MenuPosition,
		FPopupTransitionEffect(FPopupTransitionEffect::None)
	);

	ContextMenu->GetOnMenuDismissed().AddLambda([this](TSharedRef<IMenu>)
	{
		FSlateThrottleManager::Get().DisableThrottle(false);
		HideCommandsContextMenu();
	});
}

void SAvaInteractiveToolsToolbarCategoryButton::HideCommandsContextMenu()
{
	if (ContextMenu.IsValid())
	{
		ContextMenu->Dismiss();
		ContextMenu.Reset();
	}
}

void SAvaInteractiveToolsToolbarCategoryButton::OnToolActivated(const FString& InToolIdentifier)
{
	const FName ActiveToolCategory = IAvalancheInteractiveToolsModule::Get().GetToolCategory(InToolIdentifier);
	if (ActiveToolCategory.IsEqual(ToolCategory))
	{
		ActiveToolIdentifier = InToolIdentifier;
		CreateActiveCommandWidget();
		HideCommandsContextMenu();
	}
}