// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandInfo;
class FUICommandList;
class IMenu;

class SAvaInteractiveToolsToolbarCategoryButton : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaInteractiveToolsToolbarCategoryButton)
	{}
		/** Category to retrieve ITF commands from */
		SLATE_ARGUMENT(FName, ToolCategory)
		/** If not category is provided then a command must be provided */
		SLATE_ARGUMENT(TSharedPtr<FUICommandInfo>, Command)
		/** Command list to execute selected command */
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
		/** Show label under category button */
		SLATE_ARGUMENT(bool, ShowLabel)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SAvaInteractiveToolsToolbarCategoryButton() override;

protected:
	//~ Begin SButton
	virtual FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	//~ End SButton

	void CreateActiveCommandWidget();
	void ShowCommandsContextMenu();
	void HideCommandsContextMenu();

	void OnToolActivated(const FString& InToolIdentifier);

private:
	TSharedPtr<FUICommandInfo> Command;
	TSharedPtr<FUICommandList> CommandList;
	TSharedPtr<IMenu> ContextMenu;
	bool bShowLabel = false;
	FName ToolCategory = NAME_None;
	FString ActiveToolIdentifier;
};