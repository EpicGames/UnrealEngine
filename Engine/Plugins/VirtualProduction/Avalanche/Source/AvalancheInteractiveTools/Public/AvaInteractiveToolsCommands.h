// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FAvaInteractiveToolsCommands : public TCommands<FAvaInteractiveToolsCommands>
{
public:
	static AVALANCHEINTERACTIVETOOLS_API const FAvaInteractiveToolsCommands* GetExternal();

	FAvaInteractiveToolsCommands();

	//~ Begin TCommands
	virtual void RegisterCommands() override;
	//~ End TCommands

	void RegisterEdModeCommands();
	void RegisterCategoryCommands();
	void RegisterActorCommands();

	// Ed Mode
	TSharedPtr<FUICommandInfo> CancelActiveTool;
	TSharedPtr<FUICommandInfo> ToggleViewportToolbar;
	TSharedPtr<FUICommandInfo> OpenViewportToolbarSettings;

	// Categories
	TSharedPtr<FUICommandInfo> Category_2D;
	TSharedPtr<FUICommandInfo> Category_3D;
	TSharedPtr<FUICommandInfo> Category_Actor;
	TSharedPtr<FUICommandInfo> Category_Cloner;
	TSharedPtr<FUICommandInfo> Category_Effector;

	TSharedPtr<FUICommandInfo> Tool_Actor_Null;
	TSharedPtr<FUICommandInfo> Tool_Actor_Spline;
};
