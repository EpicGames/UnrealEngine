// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"

class IAvalancheInteractiveToolsModule;

DECLARE_MULTICAST_DELEGATE_OneParam(FAvaInteractiveToolsRegisterCategoriesDelegate, IAvalancheInteractiveToolsModule*)
DECLARE_MULTICAST_DELEGATE_OneParam(FAvaInteractiveToolsRegisterToolsDelegate, IAvalancheInteractiveToolsModule*);

class AVALANCHEINTERACTIVETOOLS_API FAvaInteractiveToolsDelegates
{
public:
	static FAvaInteractiveToolsRegisterToolsDelegate& GetRegisterToolsDelegate();

	static FAvaInteractiveToolsRegisterCategoriesDelegate& GetRegisterCategoriesDelegate();
};
