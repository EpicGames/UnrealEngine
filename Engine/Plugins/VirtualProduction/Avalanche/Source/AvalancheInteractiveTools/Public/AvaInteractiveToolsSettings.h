// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "AvaInteractiveToolsSettings.generated.h"

UENUM()
enum class EAvaInteractiveToolsDefaultActionAlignment : uint8
{
	Axis,
	Camera
};

UENUM()
enum class EAvaInteractiveToolsViewportToolbarPosition : uint8
{
	/** Disabled */
	None,
	/** Horizontal */
	Bottom,
	/** Horizontal */
	Top,
	/** Vertical */
	Left,
	/** Vertical */
	Right
};

UCLASS(Config = EditorPerProjectUserSettings, meta = (DisplayName = "Interactive Tools"))
class AVALANCHEINTERACTIVETOOLS_API UAvaInteractiveToolsSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	static bool IsViewportToolbarProperty(FName InPropertyName);
	
	static UAvaInteractiveToolsSettings* Get();

	UAvaInteractiveToolsSettings();

	/** Open these settings in the editor */
	void OpenEditorSettingsWindow() const;

	/** Checks if active mode supports viewport toolbar */
	bool IsViewportToolbarSupported() const;

	/** Set visibility of viewport toolbar */
	void SetViewportToolbarVisible(bool bInVisible) const;

	/** Get visibility of viewport toolbar*/
	bool GetViewportToolbarVisible() const;

	/** Distance from the camera at which actors are created. */
	UPROPERTY(Config, EditAnywhere, Category = "Motion Design")
	float CameraDistance = 500.f;

	UPROPERTY(Config, EditAnywhere, Category = "Motion Design")
	EAvaInteractiveToolsDefaultActionAlignment DefaultActionActorAlignment = EAvaInteractiveToolsDefaultActionAlignment::Axis;

	/** Position of the viewport overlay toolbar position */
	UPROPERTY(Config, EditAnywhere, Category = "Motion Design")
	EAvaInteractiveToolsViewportToolbarPosition ViewportToolbarPosition = EAvaInteractiveToolsViewportToolbarPosition::Bottom;

	/** Show viewport overlay toolbar label under each item */
	UPROPERTY(Config, EditAnywhere, Category = "Motion Design")
	bool bViewportToolbarLabelEnabled = false;
};
