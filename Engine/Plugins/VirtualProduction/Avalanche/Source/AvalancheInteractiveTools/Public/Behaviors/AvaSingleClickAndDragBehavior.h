// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseBehaviors/AnyButtonInputBehavior.h"
#include "BaseBehaviors/InputBehaviorModifierStates.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "InputBehavior.h"
#include "InputState.h"
#include "Math/Vector2D.h"
#include "Templates/Function.h"
#include "AvaSingleClickAndDragBehavior.generated.h"

class UObject;

/**
 * Functions required to apply standard "Click" and "Click-Drag" state machines to a target object.
 */
class IAvaSingleClickAndDragBehaviorTarget : public IModifierToggleBehaviorTarget
{
public:
	virtual ~IAvaSingleClickAndDragBehaviorTarget() {}

	/**
	 * Test if target can begin click-drag interaction at this point
	 * @param PressPos device position/ray at click point
	 * @return hit information at this point
	 */
	virtual FInputRayHit CanBeginSingleClickAndDragSequence(const FInputDeviceRay& InPressPos) = 0;

	/**
	 * Notify Target that click press ocurred
	 * @param PressPos device position/ray at click point
	 */
	virtual void OnClickPress(const FInputDeviceRay& InPressPos) = 0;

	/**
	 * Notify Target that the drag process has started
	 * @param DragPos device position/ray at move point
	 */
	virtual void OnDragStart(const FInputDeviceRay& InDragPos) = 0;

	/**
	 * Notify Target that input position has changed
	 * @param DragPos device position/ray at click point
	 */
	virtual void OnClickDrag(const FInputDeviceRay& InDragPos) = 0;

	/**
	 * Notify Target that click release occurred
	 * @param ReleasePos device position/ray at click point
	 */
	virtual void OnClickRelease(const FInputDeviceRay& InReleasePos, bool bInIsDragOperation) = 0;

	/**
	 * Notify Target that click-drag sequence has been explicitly terminated (eg by escape key)
	 */
	virtual void OnTerminateSingleClickAndDragSequence() = 0;
};


/**
 * UAvaSingleClickAndDragInputBehavior implements a combination of "button-click" and "button-click-drag"-style input behavior.
 * If the mouse is moved away from the original location, and update will occur and a drag operation will start. If the mouse
 * is released without moving too far away, a click event will occur. Once the drag has started, returning to the original
 * location will not produce a click event will not be produced, but the drag operation will continue.
 *
 * An IAvaSingleClickAndDragBehaviorTarget instance must be provided which is manipulated by this behavior.
 *
 * The state machine works as follows:
 *    1) on input-device-button-press, call Target::CanBeginSingleClickDragSequence to determine if capture should begin
 *    2) on input-device-move, call Target::OnClickDrag if drag mode has been started or Target::OnDragStart if it crosses that
 *	       threshold.
 *    3) on input-device-button-release, call Target::OnClickRelease
 *
 * If a ForceEndCapture occurs we call Target::OnTerminateDragClickSequence
 */
UCLASS()
class UAvaSingleClickAndDragBehavior : public UAnyButtonInputBehavior
{
	GENERATED_BODY()

public:
	UAvaSingleClickAndDragBehavior();

	/**
	 * The modifier set for this behavior
	 */
	FInputBehaviorModifierStates Modifiers;

	/**
	 * Initialize this behavior with the given Target
	 * @param Target implementor of hit-test and on-clicked functions
	 */
	virtual void Initialize(IAvaSingleClickAndDragBehaviorTarget* InTarget);

	/**
	 * WantsCapture() will only return capture request if this function returns true (or is null)
	 */
	TFunction<bool(const FInputDeviceState&)> ModifierCheckFunc = nullptr;

	//~ Begin UInputBehavior
	virtual FInputCaptureRequest WantsCapture(const FInputDeviceState& InInput) override;
	virtual FInputCaptureUpdate BeginCapture(const FInputDeviceState& InInput, EInputCaptureSide InSide) override;
	virtual FInputCaptureUpdate UpdateCapture(const FInputDeviceState& InInput, const FInputCaptureData& InData) override;
	virtual void ForceEndCapture(const FInputCaptureData& InData) override;
	//~ End UInputBehavior

	const FInputDeviceRay& GetInitialMouseDownRay() const { return InitialMouseDownRay; }

	/**
	 * If false, will never change into a drag event.
	 */
	UPROPERTY()
	bool bSupportsDrag = true;

	/**
	 * If drag is not supported, then check distance when releasing the button.
	 * The distance used is the drag start distance.
	 */
	UPROPERTY()
	bool bDistanceTestClick = true;

	/**
	 * If true, then we will update Modifier states in UpdateCapture(). This defaults to false because
	 * in most cases during a capture you don't want this, eg in a brush interaction, if you have a shift-to-smooth modifier,
	 * you don't want to toggle this on/off during a sculpting stroke.
	 */
	UPROPERTY()
	bool bUpdateModifiersDuringDrag = false;

	/** The distance the mouse has to travel to trigger a drag. */
	UPROPERTY()
	float DragStartDistance = 5.f;

protected:
	/** Click Target object */
	IAvaSingleClickAndDragBehaviorTarget* Target;

	/** set to true if we are in a capture */
	bool bInClickDrag = false;

	/** Set to true if this is a drag operation. */
	bool bIsDragOperation = false;

	/** The initial mouse down position. */
	FInputDeviceRay InitialMouseDownRay;

	/**
	 * Internal function that forwards click evens to Target::OnClickPress, you can customize behavior here
	 */
	virtual void OnClickPressInternal(const FInputDeviceState& InInput, EInputCaptureSide InSide);

	/**
	 * Internal function that forwards click evens to Target::OnDragStarted, you can customize behavior here
	 */
	virtual void OnDragStartedInternal(const FInputDeviceState& InInput, const FInputCaptureData& InData);

	/**
	 * Internal function that forwards click evens to Target::OnClickDrag, you can customize behavior here
	 */
	virtual void OnClickDragInternal(const FInputDeviceState& InInput, const FInputCaptureData& InData);

	/**
	 * Internal function that forwards click evens to Target::OnClickRelease, you can customize behavior here
	 */
	virtual void OnClickReleaseInternal(const FInputDeviceState& InInput, const FInputCaptureData& InData);
};
