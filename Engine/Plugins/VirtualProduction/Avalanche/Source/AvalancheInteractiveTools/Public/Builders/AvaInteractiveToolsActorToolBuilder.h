// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "Containers/UnrealString.h"
#include "IAvalancheInteractiveToolsModule.h"
#include "Templates/SubclassOf.h"
#include "Templates/SharedPointer.h"
#include "Tools/AvaInteractiveToolsActorTool.h"
#include "AvaInteractiveToolsActorToolBuilder.generated.h"

class AActor;
class FUICommandInfo;
class UEdMode;

UCLASS()
class AVALANCHEINTERACTIVETOOLS_API UAvaInteractiveToolsActorToolBuilder : public UAvaInteractiveToolsToolBuilder
{
	GENERATED_BODY()

public:
	static FAvaInteractiveToolsToolParameters CreateToolParameters(FName InCategory,
		TSharedPtr<FUICommandInfo> InCommand, const FString& InIdentifier, int32 InPriority, TSubclassOf<AActor> InActorClass,
		TSubclassOf<UAvaInteractiveToolsActorTool> InToolClass = UAvaInteractiveToolsActorTool::StaticClass());

	template<class InActorClass>
	static FAvaInteractiveToolsToolParameters CreateToolParameters(FName InCategory,
		TSharedPtr<FUICommandInfo> InCommand, const FString& InIdentifier, int32 InPriority,
		TSubclassOf<UAvaInteractiveToolsActorTool> InToolClass = UAvaInteractiveToolsActorTool::StaticClass())
	{
		return CreateToolParameters(InCategory, InCommand, InIdentifier, InPriority, InActorClass::StaticClass(), InToolClass);
	}

	static UAvaInteractiveToolsActorToolBuilder* CreateActorToolBuilder(UEdMode* InEdMode, FName InCategory,
		TSharedPtr<FUICommandInfo> InCommand, const FString& InIdentifier, int32 InPriority, TSubclassOf<AActor> InActorClass,
		TSubclassOf<UAvaInteractiveToolsActorTool> InToolClass = UAvaInteractiveToolsActorTool::StaticClass());

	template<class InActorClass>
	static UAvaInteractiveToolsActorToolBuilder* CreateActorToolBuilder(UEdMode* InEdMode, FName InCategory,
		TSharedPtr<FUICommandInfo> InCommand, const FString& InIdentifier, int32 InPriority,
		TSubclassOf<UAvaInteractiveToolsActorTool> InToolClass = UAvaInteractiveToolsActorTool::StaticClass())
	{
		return CreateActorToolBuilder(InEdMode, InCategory, InCommand, InIdentifier, InPriority, InActorClass::StaticClass(), InToolClass);
	}

	//~ Begin UInteractiveToolBuilder
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
	//~ End UInteractiveToolBuilder

private:
	FName Category;
	TSharedPtr<FUICommandInfo> Command;
	FString Identifier;
	int32 Priority;

	UPROPERTY()
	TSubclassOf<AActor> ActorClass;

	UPROPERTY()
	TSubclassOf<UAvaInteractiveToolsActorTool> ToolClass;

	UAvaInteractiveToolsActorToolBuilder();
};