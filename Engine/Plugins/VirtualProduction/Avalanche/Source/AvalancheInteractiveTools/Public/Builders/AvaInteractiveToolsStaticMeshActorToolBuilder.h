// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "Containers/UnrealString.h"
#include "IAvalancheInteractiveToolsModule.h"
#include "Templates/SubclassOf.h"
#include "Templates/SharedPointer.h"
#include "Tools/AvaInteractiveToolsStaticMeshActorTool.h"
#include "AvaInteractiveToolsStaticMeshActorToolBuilder.generated.h"

class FUICommandInfo;
class UEdMode;
class UStaticMesh;

UCLASS()
class AVALANCHEINTERACTIVETOOLS_API UAvaInteractiveToolsStaticMeshActorToolBuilder : public UAvaInteractiveToolsToolBuilder
{
	GENERATED_BODY()

public:
	static FAvaInteractiveToolsToolParameters CreateToolParameters(FName InCategory,
		TSharedPtr<FUICommandInfo> InCommand, const FString& InIdentifier, int32 InPriority, UStaticMesh* InStaticMesh,
		TSubclassOf<UAvaInteractiveToolsStaticMeshActorTool> InToolClass = UAvaInteractiveToolsStaticMeshActorTool::StaticClass());

	static FAvaInteractiveToolsToolParameters CreateToolParameters(FName InCategory,
		TSharedPtr<FUICommandInfo> InCommand, const FString& InIdentifier, int32 InPriority, const TCHAR* InStaticMeshPath,
		TSubclassOf<UAvaInteractiveToolsStaticMeshActorTool> InToolClass = UAvaInteractiveToolsStaticMeshActorTool::StaticClass());

	static UAvaInteractiveToolsStaticMeshActorToolBuilder* CreateStaticMeshActorToolBuilder(UEdMode* InEdMode, FName InCategory,
		TSharedPtr<FUICommandInfo> InCommand, const FString& InIdentifier, int32 InPriority, UStaticMesh* InStaticMesh,
		TSubclassOf<UAvaInteractiveToolsStaticMeshActorTool> InToolClass = UAvaInteractiveToolsStaticMeshActorTool::StaticClass());

	static UAvaInteractiveToolsStaticMeshActorToolBuilder* CreateStaticMeshActorToolBuilder(UEdMode* InEdMode, FName InCategory,
		TSharedPtr<FUICommandInfo> InCommand, const FString& InIdentifier, int32 InPriority, const TCHAR* InStaticMeshPath,
		TSubclassOf<UAvaInteractiveToolsStaticMeshActorTool> InToolClass = UAvaInteractiveToolsStaticMeshActorTool::StaticClass());

	//~ Begin UInteractiveToolBuilder
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
	//~ End UInteractiveToolBuilder

private:
	FName Category;
	TSharedPtr<FUICommandInfo> Command;
	FString Identifier;
	int32 Priority;

	UPROPERTY()
	TObjectPtr<UStaticMesh> StaticMesh;

	UAvaInteractiveToolsStaticMeshActorToolBuilder();
};