// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveToolBuilder.h"
#include "AvaInteractiveToolsToolBuilder.generated.h"

class UAvaInteractiveToolsToolBase;
class UEdMode;

UCLASS()
class AVALANCHEINTERACTIVETOOLS_API UAvaInteractiveToolsToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	static UAvaInteractiveToolsToolBuilder* CreateToolBuilder(UEdMode* InEdMode, TSubclassOf<UAvaInteractiveToolsToolBase> InToolClass);

	template<class InToolClass>
	static UAvaInteractiveToolsToolBuilder* CreateToolBuilder(UEdMode* InEdMode)
	{
		return CreateToolBuilder(InEdMode, InToolClass::StaticClass());
	}

	//~ Begin UInteractiveToolBuilder
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override { return true; }
	virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
	//~ End UInteractiveToolBuilder

protected:
	TSubclassOf<UAvaInteractiveToolsToolBase> ToolClass;

	UAvaInteractiveToolsToolBuilder();
};