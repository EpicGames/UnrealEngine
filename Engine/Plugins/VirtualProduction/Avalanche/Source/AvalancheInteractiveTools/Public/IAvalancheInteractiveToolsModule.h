// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Editor.h"
#include "Misc/Optional.h"
#include "Misc/TVariant.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"

class FName;
class FUICommandInfo;
class FText;
class IAvalancheInteractiveToolsModule;
class UEdMode;
class UInteractiveToolBuilder;
enum class EToolsContextScope;

DECLARE_DELEGATE_RetVal_OneParam(UInteractiveToolBuilder*, FAvalancheInteractiveToolsCreateBuilder, UEdMode*)

struct FAvaInteractiveToolsToolParameters
{
	TSharedPtr<FUICommandInfo> UICommand = nullptr;
	FString ToolIdentifier = "";
	int32 Priority = 0;
	FAvalancheInteractiveToolsCreateBuilder CreateBuilder;
	TVariant<TSubclassOf<AActor>, TObjectPtr<UActorFactory>> ActorFactory;
};

namespace UE::AvaInteractiveTools
{
	using FAvaInteractiveToolsCategoryToolMap = TMap<FName, TArray<FAvaInteractiveToolsToolParameters>>;
}

class IAvalancheInteractiveToolsModule : public IModuleInterface
{
public:
	static const inline FEditorModeID EM_AvaInteractiveToolsEdModeId = TEXT("EM_AvaInteractiveToolsEdModeId");
	static constexpr int32 NoPlacementCategory = -1;

	static const inline FName CategoryName2D = FName(TEXT("AvaTools2D"));
	static const inline FName CategoryName3D = FName(TEXT("AvaTools3D"));
	static const inline FName CategoryNameActor = FName(TEXT("AvaToolsActor"));
	static const inline FName CategoryNameCloner = FName(TEXT("AvaToolsCloner"));
	static const inline FName CategoryNameEffector = FName(TEXT("AvaToolsEffector"));

	static IAvalancheInteractiveToolsModule& Get()
	{
		static const FName ModuleName = TEXT("AvalancheInteractiveTools");
		return FModuleManager::LoadModuleChecked<IAvalancheInteractiveToolsModule>(ModuleName);
	}

	static IAvalancheInteractiveToolsModule* GetPtr()
	{
		static const FName ModuleName = TEXT("AvalancheInteractiveTools");
		return FModuleManager::GetModulePtr<IAvalancheInteractiveToolsModule>(ModuleName);
	}

	/** -1 on the sort priority skips the placement mode tab. */
	virtual void RegisterCategory(FName InCategoryName, TSharedPtr<FUICommandInfo> InCategoryCommand, 
		int32 InPlacementModeSortPriority = NoPlacementCategory) = 0;

	virtual const TMap<FName, TSharedPtr<FUICommandInfo>>& GetCategories() = 0;

	virtual void RegisterTool(FName InCategory, FAvaInteractiveToolsToolParameters&& InToolParams) = 0;

	virtual const TArray<FAvaInteractiveToolsToolParameters>* GetTools(FName InCategory) = 0;
	
	virtual const FAvaInteractiveToolsToolParameters* GetTool(const FString& InToolIdentifier) = 0;

	virtual FName GetToolCategory(const FString& InToolIdentifier) = 0;

	virtual bool HasActiveTool() const = 0;

	DECLARE_MULTICAST_DELEGATE_OneParam(FToolEvent, const FString& /** ToolIdentifier */)
	virtual FToolEvent::RegistrationType& OnToolActivation() = 0;
	virtual FToolEvent::RegistrationType& OnToolDeactivation() = 0;
};
