// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Planners/AvaInteractiveToolsToolViewportPlanner.h"
#include "AvaInteractiveToolsToolViewportAreaPlanner.generated.h"

UCLASS(BlueprintType, Blueprintable)
class AVALANCHEINTERACTIVETOOLS_API UAvaInteractiveToolsToolViewportAreaPlanner : public UAvaInteractiveToolsToolViewportPlanner
{
	GENERATED_BODY()

public:
	const FVector2f& GetStartPosition() const { return StartPosition; };
	void SetStartPosition(const FVector2f& InStartPosition);
	const FVector2f& GetEndPosition() const { return EndPosition; }
	void SetEndPosition(const FVector2f& InEndPosition);
	FVector2f GetCenterPosition() const;
	FVector2f GetTopLeftCorner() const;
	FVector2f GetTopRightCorner() const;
	FVector2f GetBottomLeftCorner() const;
	FVector2f GetBottomRightCorner() const;
	FVector2f GetSize() const;
	FVector GetStartPositionWorld() const;
	FVector GetEndPositionWorld() const;
	FVector GetTopLeftCornerWorld() const;
	FVector GetTopRightCornerWorld() const;
	FVector GetBottomLeftCornerWorld() const;
	FVector GetBottomRightCornerWorld() const;
	FVector2D GetWorldSize() const;

	FVector ViewportToWorldPosition(const FVector2f& InViewportPosition) const;

	//~ Begin UAvaInteractiveToolsToolViewportPlanner
	virtual void Setup(UAvaInteractiveToolsToolBase* InTool) override;
	virtual void OnTick(float InDeltaTime) override;
	virtual void DrawHUD(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI) override;
	virtual void OnClicked(const FInputDeviceRay& InClickPos) override;
	virtual bool HasStarted() const override { return bStartedAreaPlanning; }
	//~ End UAvaInteractiveToolsToolViewportPlanner

protected:
	bool bStartedAreaPlanning;
	FVector2f StartPosition;
	FVector2f EndPosition;

	void UpdateEndPosition(const FVector2f& InNewPosition);
};
