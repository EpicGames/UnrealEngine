// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Math/MathFwd.h"
#include "Math/IntRect.h"
#include "AvaInteractiveToolsToolViewportPlanner.generated.h"

class FAvaSnapOperation;
class FCanvas;
class FViewport;
class IToolsContextRenderAPI;
class UAvaInteractiveToolsToolBase;
enum class EAvaViewportStatus : uint8;
enum class EToolShutdownType : uint8;
struct FInputDeviceRay;

UCLASS(Abstract, BlueprintType, Blueprintable)
class AVALANCHEINTERACTIVETOOLS_API UAvaInteractiveToolsToolViewportPlanner : public UObject
{
	GENERATED_BODY()

public:
	UAvaInteractiveToolsToolViewportPlanner();
	virtual void Setup(UAvaInteractiveToolsToolBase* InTool);
	virtual void DrawHUD(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI) {}
	virtual void Render(IToolsContextRenderAPI* InRenderAPI) {}
	virtual void OnTick(float InDeltaTime) {}
	virtual void OnClicked(const FInputDeviceRay& InClickPos) {}
	virtual void Shutdown(EToolShutdownType ShutdownType);
	virtual bool HasStarted() const PURE_VIRTUAL(UAvaInteractiveToolsToolViewportPlanner::HasStarted, return false;)

	FViewport* GetViewport(EAvaViewportStatus InViewportStatus) const;

protected:
	// The minimum dimension for things
	static constexpr float MinDim = 5;

	UPROPERTY()
	TObjectPtr<UAvaInteractiveToolsToolBase> Tool;

	bool bAttemptedToCreateSnapOperation;
	TSharedPtr<FAvaSnapOperation> SnapOperation;

	FVector2f GetConstrainedMousePosition() const;
	FVector2f GetConstrainedMousePosition(const FVector2f& InClickPos) const;
	FVector2f GetConstrainedMousePosition(FViewport* InViewport) const;

	void StartSnapOperation();
	void SnapLocation(FVector2f& InOutLocation);
};
