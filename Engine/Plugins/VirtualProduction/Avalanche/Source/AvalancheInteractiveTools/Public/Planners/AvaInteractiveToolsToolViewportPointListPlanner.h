// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Planners/AvaInteractiveToolsToolViewportPlanner.h"
#include "AvaInteractiveToolsToolViewportPointListPlanner.generated.h"

UENUM(BlueprintType)
enum class EAvaInteractiveToolsToolViewportPointListPlannerLineStatus : uint8
{
	Neutral,
	Allowed,
	Disallowed
};

UCLASS(BlueprintType, Blueprintable)
class AVALANCHEINTERACTIVETOOLS_API UAvaInteractiveToolsToolViewportPointListPlanner : public UAvaInteractiveToolsToolViewportPlanner
{
	GENERATED_BODY()

public:
	const FVector2f& GetCurrentViewportPosition() const { return CurrentViewportPosition; }

	void OverrideCurrentViewportPosition(const FVector2f InPosition);

	const TArray<FVector2f>& GetViewportPositions() const { return ViewportPositions; }

	TArray<FVector2f>& GetViewportPositions() { return ViewportPositions; }

	void AddViewportPosition(const FVector2f& InViewportPosition);

	EAvaInteractiveToolsToolViewportPointListPlannerLineStatus GetLineStatus() const { return LineStatus; }

	void SetLineStatus(EAvaInteractiveToolsToolViewportPointListPlannerLineStatus InLineStatus);

	//~ Begin UAvaInteractiveToolsToolViewportPlanner
	virtual void Setup(UAvaInteractiveToolsToolBase* InTool) override;
	virtual void OnTick(float InDeltaTime) override;
	virtual void DrawHUD(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI) override;
	virtual void OnClicked(const FInputDeviceRay& InClickPos) override;
	virtual bool HasStarted() const override { return ViewportPositions.IsEmpty() == false; }
	//~ End UAvaInteractiveToolsToolViewportPlanner

protected:
	TArray<FVector2f> ViewportPositions;
	FVector2f CurrentViewportPosition;
	EAvaInteractiveToolsToolViewportPointListPlannerLineStatus LineStatus;
};
