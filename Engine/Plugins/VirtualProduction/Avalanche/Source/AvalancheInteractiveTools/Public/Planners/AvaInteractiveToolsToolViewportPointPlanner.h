// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Planners/AvaInteractiveToolsToolViewportPlanner.h"
#include "AvaInteractiveToolsToolViewportPointPlanner.generated.h"

UCLASS(BlueprintType, Blueprintable)
class AVALANCHEINTERACTIVETOOLS_API UAvaInteractiveToolsToolViewportPointPlanner : public UAvaInteractiveToolsToolViewportPlanner
{
	GENERATED_BODY()

public:
	const FVector2f& GetViewportPosition() const { return ViewportPosition; }
	void SetViewportPosition(const FVector2f& InViewportPosition);

	//~ Begin UAvaInteractiveToolsToolViewportPlanner
	virtual void Setup(UAvaInteractiveToolsToolBase* InTool) override;
	virtual void OnTick(float InDeltaTime) override;
	virtual void OnClicked(const FInputDeviceRay& InClickPos) override;
	virtual bool HasStarted() const override { return false; } // Start is also the end
	//~ End UAvaInteractiveToolsToolViewportPlanner

protected:
	FVector2f ViewportPosition;
};
