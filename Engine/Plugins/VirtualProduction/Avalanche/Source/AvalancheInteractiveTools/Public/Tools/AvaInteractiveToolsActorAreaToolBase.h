// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaInteractiveToolsActorToolBase.h"
#include "AvaInteractiveToolsActorAreaToolBase.generated.h"

UCLASS(Abstract)
class AVALANCHEINTERACTIVETOOLS_API UAvaInteractiveToolsActorAreaToolBase : public UAvaInteractiveToolsActorToolBase
{
	GENERATED_BODY()

	using UAvaInteractiveToolsToolBase::SpawnActor;

public:
	UAvaInteractiveToolsActorAreaToolBase();

	//~ Begin UAvaInteractiveToolsToolBase
	virtual void OnViewportPlannerUpdate() override;
	virtual void OnViewportPlannerComplete() override;
	//~ End UAvaInteractiveToolsToolBase

protected:
	FVector OriginalActorSize;

	void SetActorScale(AActor* InActor) const;

	//~ Begin UAvaInteractiveToolsToolBase
	virtual AActor* SpawnActor(TSubclassOf<AActor> InActorClass, EAvaViewportStatus InViewportStatus, 
		const FVector2f& InViewportPosition, bool bInPreview, FString* InActorLabelOverride = nullptr) override;
	//~ End UAvaInteractiveToolsToolBase
};
