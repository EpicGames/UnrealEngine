// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaInteractiveToolsActorToolBase.h"
#include "AvaInteractiveToolsActorPointToolBase.generated.h"

UCLASS(Abstract)
class AVALANCHEINTERACTIVETOOLS_API UAvaInteractiveToolsActorPointToolBase : public UAvaInteractiveToolsActorToolBase
{
	GENERATED_BODY()

public:
	UAvaInteractiveToolsActorPointToolBase();

	//~ Begin UAvaInteractiveToolsToolBase
	virtual void OnViewportPlannerUpdate() override;
	virtual void OnViewportPlannerComplete() override;
	//~ End UAvaInteractiveToolsToolBase
};
