// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaInteractiveToolsActorPointToolBase.h"
#include "Containers/UnrealString.h"
#include "Templates/SubclassOf.h"
#include "Templates/SharedPointer.h"
#include "AvaInteractiveToolsActorTool.generated.h"

class AActor;
class FUICommandInfo;

UCLASS()
class AVALANCHEINTERACTIVETOOLS_API UAvaInteractiveToolsActorTool : public UAvaInteractiveToolsActorPointToolBase
{
	GENERATED_BODY()

	friend class UAvaInteractiveToolsActorToolBuilder;

public:
	UAvaInteractiveToolsActorTool();

	//~ Begin UAvaInteractiveToolsToolBase
	virtual void OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule) override;
	//~ End UAvaInteractiveToolsToolBase

protected:
	FName Category;
	TSharedPtr<FUICommandInfo> Command;
	FString Identifier;
	int32 Priority = 0;
};
