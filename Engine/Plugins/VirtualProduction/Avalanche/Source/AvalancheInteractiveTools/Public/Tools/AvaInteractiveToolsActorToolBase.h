// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaInteractiveToolsToolBase.h"
#include "Templates/SubclassOf.h"
#include "AvaInteractiveToolsActorToolBase.generated.h"

UCLASS(Abstract)
class AVALANCHEINTERACTIVETOOLS_API UAvaInteractiveToolsActorToolBase : public UAvaInteractiveToolsToolBase
{
	GENERATED_BODY()

	friend class UAvaInteractiveToolsActorToolBuilder;

protected:
	UPROPERTY()
	TSubclassOf<AActor> ActorClass = nullptr;

	//~ Begin UAvaInteractiveToolsToolBase
	virtual bool OnBegin() override;
	virtual void DefaultAction() override;
	virtual bool UseIdentityLocation() const override { return bPerformingDefaultAction; }
	virtual bool UseIdentityRotation() const override;
	//~ End UAvaInteractiveToolsToolBase

	FAvaInteractiveToolsToolParameters CreateDefaultToolParameters() const;
};
