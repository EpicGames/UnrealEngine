// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaInteractiveToolsActorTool.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "AvaInteractiveToolsStaticMeshActorTool.generated.h"

class FUICommandInfo;
class UStaticMesh;

UCLASS()
class AVALANCHEINTERACTIVETOOLS_API UAvaInteractiveToolsStaticMeshActorTool : public UAvaInteractiveToolsActorTool
{
	GENERATED_BODY()

	friend class UAvaInteractiveToolsStaticMeshActorToolBuilder;

public:
	UAvaInteractiveToolsStaticMeshActorTool();

	//~ Begin UAvaInteractiveToolsToolBase
	virtual void OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule) override;
	//~ End UAvaInteractiveToolsToolBase

protected:
	UPROPERTY()
	TObjectPtr<UStaticMesh> StaticMesh;

	//~ Begin UAvaInteractiveToolsToolBase
	virtual AActor* SpawnActor(TSubclassOf<AActor> InActorClass, EAvaViewportStatus InViewportStatus,
		const FVector2f& InViewportPosition, bool bInPreview, FString* InActorLabelOverride = nullptr) override;
	//~ End UAvaInteractiveToolsToolBase
};
