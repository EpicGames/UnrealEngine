// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveTool.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "Behaviors/AvaSingleClickAndDragBehavior.h"
#include "IAvalancheInteractiveToolsModule.h"
#include "AvaInteractiveToolsToolBase.generated.h"

class AActor;
class FUICommandInfo;
class UAvaInteractiveToolsToolViewportPlanner;
class UEdMode;
class USingleClickInputBehavior;
class USingleKeyCaptureBehavior;
class UWorld;

UCLASS()
class UAvaInteractiveToolsRightClickBehavior : public USingleClickInputBehavior
{
	GENERATED_BODY()

public:
	UAvaInteractiveToolsRightClickBehavior();

	virtual void Clicked(const FInputDeviceState& Input, const FInputCaptureData& Data) override;
};

enum class EAvaViewportStatus : uint8
{
	Hovered,
	Focused
};

UCLASS(Abstract)
class AVALANCHEINTERACTIVETOOLS_API UAvaInteractiveToolsToolBase : public UInteractiveTool
#if CPP
	, public IAvaSingleClickAndDragBehaviorTarget
	, public IClickBehaviorTarget
#endif
{
	GENERATED_BODY()

public:
	/** Cancel Behavior ID */
	static constexpr int32 BID_Cancel = 1;

	/**
	 * Checks the given actor and its components against UAvaInteractiveToolsModeDetailsObject
	 * and UAvaInteractiveToolsModeDetailsObjectProvider.
	 */
	static UObject* GetDetailsObjectFromActor(AActor* InActor);

	virtual ~UAvaInteractiveToolsToolBase() override = default;

	//~ Begin UInteractiveTool
	virtual void Setup() override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;
	virtual void DrawHUD(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI) override;
	virtual void Render(IToolsContextRenderAPI* InRenderAPI) override;
	virtual void OnTick(float InDeltaTime) override;
	//~ End UInteractiveTool

	/** Returns true if this supports a default action. */
	virtual bool SupportsDefaultAction() const;

	FString GetActiveToolIdentifier() const;
	const FAvaInteractiveToolsToolParameters* GetActiveToolParameters() const;

	//~ Begin IModifierToggleBehaviorTarget
	virtual void OnUpdateModifierState(int InModifierID, bool bInIsOn) override;
	//~ End IModifierToggleBehaviorTarget

	//~ Begin IAvaSingleClickAndDragBehaviorTarget
	virtual FInputRayHit CanBeginSingleClickAndDragSequence(const FInputDeviceRay& InPressPos) override;
	virtual void OnClickPress(const FInputDeviceRay& InPressPos) override;
	virtual void OnDragStart(const FInputDeviceRay& InDragPos) override;
	virtual void OnClickDrag(const FInputDeviceRay& InDragPos) override;
	virtual void OnClickRelease(const FInputDeviceRay& InReleasePos, bool bInIsDragOperation) override;
	virtual void OnTerminateSingleClickAndDragSequence() override;
	//~ End IAvaSingleClickAndDragBehaviorTarget

	//~ Begin IClickBehaviorTarget
	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& InClickPos) override;
	virtual void OnClicked(const FInputDeviceRay& InClickPos) override;
	//~ End IClickBehaviorTarget

	AActor* GetSpawnedActor() const { return SpawnedActor; }

	virtual void OnViewportPlannerUpdate() {}
	virtual void OnViewportPlannerComplete();
	virtual void OnActorSpawned(AActor* InActor);

	/** Calls the below function with the default camera distance. */
	bool ViewportPositionToWorldPositionAndOrientation(EAvaViewportStatus InViewportStatus, const FVector2f& InViewportPosition,
		UWorld*& OutWorld, FVector& OutPosition, FRotator& OutRotation) const;

	/** Converts viewport position to camera plane transform at given distance. */
	bool ViewportPositionToWorldPositionAndOrientation(EAvaViewportStatus InViewportStatus, const FVector2f& InViewportPosition,
		float InDistance, UWorld*& OutWorld, FVector& OutPosition, FRotator& OutRotation) const;

	FViewport* GetViewport(EAvaViewportStatus InViewportStatus) const;

	bool ShouldAutoRegister() const
	{
		return bAutoRegister;
	}

	virtual void OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule) {}

protected:
	/** Helper to create an Actor Factory of a given class */
	template<typename InActorFactoryClass
		UE_REQUIRES(std::is_base_of_v<UActorFactory, InActorFactoryClass>)>
	static InActorFactoryClass* CreateActorFactory()
	{
		return NewObject<InActorFactoryClass>(GetTransientPackage(), InActorFactoryClass::StaticClass(), NAME_None, RF_Standalone);
	}

	UPROPERTY()
	TObjectPtr<UAvaSingleClickAndDragBehavior> LeftClickBehavior = nullptr;

	UPROPERTY()
	TObjectPtr<UAvaInteractiveToolsRightClickBehavior> RightClickBehavior = nullptr;

	UPROPERTY()
	TObjectPtr<USingleKeyCaptureBehavior> EscapeKeyBehavior = nullptr;

	UPROPERTY()
	TSubclassOf<UAvaInteractiveToolsToolViewportPlanner> ViewportPlannerClass = nullptr;

	UPROPERTY()
	TObjectPtr<UAvaInteractiveToolsToolViewportPlanner> ViewportPlanner = nullptr;

	UPROPERTY()
	TObjectPtr<AActor> PreviewActor = nullptr;

	UPROPERTY()
	TObjectPtr<AActor> SpawnedActor = nullptr;

	bool bPerformingDefaultAction = false;

	bool bAutoRegister = true;

	virtual bool CanActivate(const FString& InToolIdentifier, bool bInReactivate) const;
	virtual void Activate(const FString& InToolIdentifier, bool bInReactivate);

	/** Called when the tool is selected in the InteractiveTools tab. */
	virtual void OnActivate();

	/** Returns false if tool fails to initialise. */
	virtual bool OnBegin();

	/** Called if Begin() is successful. */
	virtual void OnPostBegin();

	/** Cancels the tool and shuts it down. */
	virtual void OnCancel();

	/** Completes the tool and shuts it down. */
	virtual void OnComplete();

	void BeginTransaction();

	void EndTransaction();

	void CancelTransaction();

	void RequestShutdown(EToolShutdownType InShutdownType);

	/** Creates a default actor in the world, without user input. */
	virtual void DefaultAction();

	/** Returns true if actors should be spawned at {0, 0, 0}. */
	virtual bool UseIdentityLocation() const { return false; }

	/** Returns true if actors should be spawned oriented to {0, 0, 0}. */
	virtual bool UseIdentityRotation() const { return true; }

	bool ConditionalIdentityRotation() const;

	/** Calls the other spawn actor with the centre of the last active viewport as the position. */
	AActor* SpawnActor(TSubclassOf<AActor> InActorClass, bool bInPreview, FString* InActorLabelOverride = nullptr);

	/** Spawns an actor of the given class at the correct distance from the camera in the given viewport position. */
	virtual AActor* SpawnActor(TSubclassOf<AActor> InActorClass, EAvaViewportStatus InViewportStatus, 
		const FVector2f& InViewportPosition, bool bInPreview, FString* InActorLabelOverride = nullptr);

	void SetToolkitSettingsObject(UObject* InObject) const;

	bool IsMotionDesignViewport() const;

	virtual bool ShouldForceDefaultAction() const;

	void DestroyPreviewActor();

	UFUNCTION()
	void OnPreviewActorDestroyed(AActor* InDestroyedActor);
};
