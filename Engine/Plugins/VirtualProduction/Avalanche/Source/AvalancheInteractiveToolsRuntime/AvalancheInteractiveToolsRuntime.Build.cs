// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheInteractiveToolsRuntime : ModuleRules
{
	public AvalancheInteractiveToolsRuntime(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
			}
		);

		ShortName = "AvIntToolsRT";
	}
}
