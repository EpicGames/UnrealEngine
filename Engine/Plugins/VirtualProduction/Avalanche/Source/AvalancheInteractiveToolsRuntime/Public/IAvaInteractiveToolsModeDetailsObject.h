// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IAvaInteractiveToolsModeDetailsObject.generated.h"

UINTERFACE(MinimalAPI)
class UAvaInteractiveToolsModeDetailsObject : public UInterface
{
	GENERATED_BODY()
};

class IAvaInteractiveToolsModeDetailsObject
{
	GENERATED_BODY()
};
