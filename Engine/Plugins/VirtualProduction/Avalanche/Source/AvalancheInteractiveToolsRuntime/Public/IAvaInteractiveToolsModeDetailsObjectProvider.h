// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IAvaInteractiveToolsModeDetailsObjectProvider.generated.h"

class UObject;

UINTERFACE(MinimalAPI)
class UAvaInteractiveToolsModeDetailsObjectProvider : public UInterface
{
	GENERATED_BODY()
};

class AVALANCHEINTERACTIVETOOLSRUNTIME_API IAvaInteractiveToolsModeDetailsObjectProvider
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintNativeEvent, CallInEditor, Category = "Motion Design Interactive Tools")
	UObject* GetModeDetailsObject() const;

	virtual UObject* GetModeDetailsObject_Implementation() const PURE_VIRTUAL(IAvaInteractiveToolsModeDetailsObjectProvider::GetModeDetailsObject, return nullptr;)
};
