// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheLevelViewport : ModuleRules
{
	public AvalancheLevelViewport(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicIncludePathModuleNames.AddRange(
			new string[]
			{
				"AvalancheInteractiveTools",
				"SceneOutliner",
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AvalancheViewport",
				"Core",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"UnrealEd"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"Avalanche",
				"AvalancheComponentVisualizers",
				"AvalancheEditorCore",
				"AvalancheFunctionalTest",
				"AvalancheInteractiveTools",
				"AvalancheViewport",
				"CinematicCamera",
				"CustomDetailsView",
				"EditorFramework",
				"FunctionalTesting",
				"InputCore",
				"LevelEditor",
				"Projects",
				"PropertyAnimatorCore",
				"PropertyEditor",
				"ToolMenus",
				"TypedElementRuntime"
			}
		);
	}
}
