// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaLevelViewportCommands.h"
#include "AvaLevelViewportStyle.h"

#define LOCTEXT_NAMESPACE "AvaLevelViewportCommands"

const FAvaLevelViewportCommands& FAvaLevelViewportCommands::GetExternal()
{
	if (!IsRegistered())
	{
		Register();
	}

	return Get();
}

const FAvaLevelViewportCommands& FAvaLevelViewportCommands::GetInternal()
{
	return Get();
}

FAvaLevelViewportCommands::FAvaLevelViewportCommands()
	: TCommands<FAvaLevelViewportCommands>(
		TEXT("AvaLevelViewport")
		, LOCTEXT("MotionDesignLevelViewport", "Motion Design Level Viewport")
		, NAME_None
		, FAvaLevelViewportStyle::Get().GetStyleSetName()
	)
{
}

void FAvaLevelViewportCommands::RegisterCommands()
{
	RegisterViewportCommands();
	RegisterCameraCommands();
	RegisterGridCommands();
	RegisterSnappingCommands();
	RegisterVirtualSizeCommands();
	RegisterGuideCommands();
	RegisterTransformCommands();
	RegisterAnimatorCommands();
}

void FAvaLevelViewportCommands::RegisterViewportCommands()
{
	UI_COMMAND(ToggleOverlay
		, "Viewport Overlay Visibility"
		, "Toggles the viewport overlay widget's visibility."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord(EKeys::O, EModifierKey::Shift));

	UI_COMMAND(ToggleBoundingBoxes
		, "Toggle Bounding Boxes"
		, "Toggles the display of selected actor bounding boxes."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord(EKeys::B));

	UI_COMMAND(ToggleIsolateActors
		, "Isolate Selected Actors"
		, "Changes the viewport and outliner to only show the selected actors. Also removes non-selected actors from snapping consideration. Changing the actor selection will not change which actors are isolated. Camera Preview Viewport Cameras are always visible (and their associated Canvas.)"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord(EKeys::Q, EModifierKey::Alt))

	UI_COMMAND(ToggleSafeFrames
		, "Toggle Safe Frames"
		, "Toggles view of an 80% view size safe frame."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord(EKeys::F, EModifierKey::Alt))

	UI_COMMAND(ToggleChildActorLock
		, "Child Actor Lock"
		, "Keeps child actors in place when moving parent actors."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord(EKeys::P, EModifierKey::Alt | EModifierKey::Shift))

	UI_COMMAND(ToggleShapeEditorOverlay
		, "Toggle Shape Editor Overlay"
		, "Adds a mini details panel to the viewport(s) allowing quick access to Motion Design properties."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord())

	UI_COMMAND(TogglePostProcessNone
		, "None"
		, "Switch off extra post process effects."
		, EUserInterfaceActionType::Check
		, FInputChord())

	UI_COMMAND(TogglePostProcessBackground
		, "Background"
		, "Switch to the background post process renderer."
		, EUserInterfaceActionType::Check
		, FInputChord())

	UI_COMMAND(TogglePostProcessChannelRed
		, "Red Channel"
		, "Switch to the red channel post process filter."
		, EUserInterfaceActionType::Check
		, FInputChord())

	UI_COMMAND(TogglePostProcessChannelGreen
		, "Green Channel"
		, "Switch to the green channel post process filter."
		, EUserInterfaceActionType::Check
		, FInputChord())

	UI_COMMAND(TogglePostProcessChannelBlue
		, "Blue Channel"
		, "Switch to the blue channel post process filter."
		, EUserInterfaceActionType::Check
		, FInputChord())

	UI_COMMAND(TogglePostProcessChannelAlpha
		, "Alpha Channel"
		, "Switch to the alpha channel post process filter."
		, EUserInterfaceActionType::Check
		, FInputChord())

	UI_COMMAND(TogglePostProcessCheckerboard
		, "Checkerboard"
		, "Switch to the checkerboard post process filter."
		, EUserInterfaceActionType::Check
		, FInputChord())

	UI_COMMAND(ToggleTextureOverlay
		, "Toggle Texture Overlay"
		, "Turns on and off the texture overlay."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord())
}

void FAvaLevelViewportCommands::RegisterCameraCommands()
{
	UI_COMMAND(CameraZoomInCenter
		, "Zoom Camera In"
		, "Zooms the preview viewport camera in, maintaining the center position."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::PageUp))

	UI_COMMAND(CameraZoomOutCenter
		, "Zoom Camera Out"
		, "Zooms the preview viewport camera out, maintaining the center position."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::PageDown))

	UI_COMMAND(CameraPanLeft
		, "Pan Camera Left"
		, "Pans the zoomed in preview viewport camera to the left. Will not exceed camera view."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Left))

	UI_COMMAND(CameraPanRight
		, "Pan Camera Right"
		, "Pans the zoomed in preview viewport camera to the right. Will not exceed camera view."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Right))

	UI_COMMAND(CameraPanUp
		, "Pan Camera Up"
		, "Pans the zoomed in preview viewport camera to the up. Will not exceed camera view."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Up))

	UI_COMMAND(CameraPanDown
		, "Pan Camera Down"
		, "Pans the zoomed in preview viewport camera to the down. Will not exceed camera view."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Down))

	UI_COMMAND(CameraFrameActor
		, "Zoom to Actor"
		, "Zooms and pans the preview viewport camera to frame an actor."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::F, EModifierKey::Shift))

	UI_COMMAND(CameraZoomReset
		, "Reset Camera Zoom"
		, "Resets the zoom and pan of the Motion Design Viewport camera."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Home))

	UI_COMMAND(CameraTransformReset
		, "Reset Camera Transform"
		, "Resets the location and rotation of the Motion Design Viewport camera."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Home, EModifierKey::Alt))

	UI_COMMAND(CameraTransformUndo
		, "Undo Viewport Camera Transform"
		, "Undoes the previous viewport camera move, if any."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::LeftBracket))

	UI_COMMAND(CameraTransformRedo
		, "Redo Viewport Camera Transform"
		, "Redoes the previous undone viewport camera move, if any."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::RightBracket))
}

void FAvaLevelViewportCommands::RegisterGridCommands()
{
	UI_COMMAND(ToggleGrid
		, "Toggle Grid"
		, "Toggles the grid on and off.\n\nNote: Grid will only show if Always On is enabled or a snap operation is ongoing."
		, EUserInterfaceActionType::Check
		, FInputChord(EKeys::Apostrophe, EModifierKey::Control));

	UI_COMMAND(ToggleGridAlwaysVisible
		, "Always Show Grid"
		, "Enables the grid even when it's not in use."
		, EUserInterfaceActionType::Check
		, FInputChord(EKeys::Slash, EModifierKey::Control));

	UI_COMMAND(IncreaseGridSize
		, "Increases Grid Size"
		, "Increases the grid size by 1 pixel."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Equals));

	UI_COMMAND(DecreaseGridSize
		, "Decreases Grid Size"
		, "Decreases the grid size by 1 pixel."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Hyphen));
}

void FAvaLevelViewportCommands::RegisterSnappingCommands()
{
	UI_COMMAND(ToggleSnapping
		, "Toggle Global Snap"
		, "Toggles all snapping."
		, EUserInterfaceActionType::Check
		, FInputChord(EKeys::Backslash, EModifierKey::Control));

	UI_COMMAND(ToggleGridSnapping
		, "Toggle Grid Snap"
		, "Toggles grid-based snapping."
		, EUserInterfaceActionType::Check
		, FInputChord(EKeys::Apostrophe, EModifierKey::Alt));

	UI_COMMAND(ToggleScreenSnapping
		, "Toggle Screen Snap"
		, "Toggles screen-based snapping (guides and edge of screen.)"
		, EUserInterfaceActionType::Check
		, FInputChord(EKeys::Semicolon, EModifierKey::Alt));

	UI_COMMAND(ToggleActorSnapping
		, "Toggle Actor Snap"
		, "Toggles actor-based snapping."
		, EUserInterfaceActionType::Check
		, FInputChord(EKeys::Backslash, EModifierKey::Alt));

}

void FAvaLevelViewportCommands::RegisterVirtualSizeCommands()
{
	UI_COMMAND(VirtualSizeDisable
		, "Remove Ruler Override"
		, "Removes the viewport's ruler override."
		, EUserInterfaceActionType::RadioButton
		, FInputChord());

	UI_COMMAND(VirtualSize1920x1080
		, "Ruler 1920x1080"
		, "Sets the viewport's ruler override to 1920x1080."
		, EUserInterfaceActionType::RadioButton
		, FInputChord());

	UI_COMMAND(VirtualSizeAspectRatioUnlocked
		, "Ruler Unlocked Aspect Ratio"
		, "Unlocks the viewport ruler override's aspect ratio."
		, EUserInterfaceActionType::RadioButton
		, FInputChord());

	UI_COMMAND(VirtualSizeAspectRatioLocked
		, "Ruler Locked Aspect Ratio"
		, "Locks the viewport ruler override's aspect ratio."
		, EUserInterfaceActionType::RadioButton
		, FInputChord());

	UI_COMMAND(VirtualSizeAspectRatioLockedToCamera
		, "Ruler Lock Aspect Ratio To Camera"
		, "Locks the viewport ruler override's aspect ratio to the active camera's aspect ratio."
		, EUserInterfaceActionType::RadioButton
		, FInputChord());
}

void FAvaLevelViewportCommands::RegisterGuideCommands()
{
	UI_COMMAND(ToggleGuides
		, "Toggle Guides"
		, "Toggles the overlay visibility."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord(EKeys::O, EModifierKey::Control));

	UI_COMMAND(AddGuideHorizontal
		, "Add Horizontal Guide"
		, "Adds a horizontal guide line to the viewport."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Hyphen, EModifierKey::Control));

	UI_COMMAND(AddGuideVertical
		, "Add Vertical Guide"
		, "Adds a vertical guide line to the viewport."
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Equals, EModifierKey::Control));

	UI_COMMAND(ToggleGuideEnabled
		, "Enabled"
		, "Can be toggled by shift-clicking the guide."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(ToggleGuideLocked
		, "Locked"
		, "Can be toggled by alt-clicking the guide."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(RemoveGuide
		, "Remove"
		, "Can be removed by double clicking the guide."
		, EUserInterfaceActionType::Button
		, FInputChord());
}

void FAvaLevelViewportCommands::RegisterTransformCommands()
{
	UI_COMMAND(ResetLocation
		, "Reset Location to 0,0,0"
		, "Resets the currently selected object's translation to 0,0,0."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord(EKeys::W, EModifierKey::Alt))

	UI_COMMAND(ResetRotation
		, "Reset Rotation to 0,0,0"
		, "Resets the currently selected object's rotation to 0,0,0."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord(EKeys::E, EModifierKey::Alt))

	UI_COMMAND(ResetScale
		, "Reset Scale to 1,1,1"
		, "Resets the currently selected object's scale to 1,1,1."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord(EKeys::R, EModifierKey::Alt))

	UI_COMMAND(ResetTransform
		, "Reset Transform to Default"
		, "Resets the currently selected object's transform to default."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord(EKeys::T, EModifierKey::Alt))
}

void FAvaLevelViewportCommands::RegisterAnimatorCommands()
{
	UI_COMMAND(DisableAnimators
	, "Disable Animators"
	, "Disable animators of the selected actors in the level"
	, EUserInterfaceActionType::Button
	, FInputChord(EKeys::M, EModifierKey::Control))

	UI_COMMAND(EnableAnimators
	, "Enable Animators"
	, "Enable animators of the selected actors in the level"
	, EUserInterfaceActionType::Button
	, FInputChord(EKeys::U, EModifierKey::Control))
}

#undef LOCTEXT_NAMESPACE
