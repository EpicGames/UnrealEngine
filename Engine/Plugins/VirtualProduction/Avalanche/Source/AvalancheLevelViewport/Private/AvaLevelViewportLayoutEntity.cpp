// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaLevelViewportLayoutEntity.h"
#include "Editor/UnrealEdEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "UnrealEdGlobals.h"
#include "SAvaLevelViewport.h"
#include "SAvaLevelViewportFrame.h"
#include "ViewportClient/AvaLevelViewportClient.h"

FAvaLevelViewportLayoutEntity::FAvaLevelViewportLayoutEntity(const FAssetEditorViewportConstructionArgs& InArgs, TSharedPtr<ILevelEditor> InLevelEditor)
{
	Viewport = SNew(SAvaLevelViewportFrame, InArgs, InLevelEditor);
}

FName FAvaLevelViewportLayoutEntity::GetStaticType()
{
	static const FName AvaLevelViewportName(TEXT("MotionDesign"));
	return AvaLevelViewportName;
}

TSharedPtr<SLevelViewport> FAvaLevelViewportLayoutEntity::AsLevelViewport() const
{
	return Viewport->GetViewportWidget();
}

TSharedRef<SWidget> FAvaLevelViewportLayoutEntity::AsWidget() const
{
	return Viewport.ToSharedRef();
}

void FAvaLevelViewportLayoutEntity::SetKeyboardFocus()
{
	FSlateApplication::Get().SetKeyboardFocus(GetViewportWidget());
}

void FAvaLevelViewportLayoutEntity::OnLayoutDestroyed()
{
	TSharedPtr<SLevelViewport> ViewportWidget = GetViewportWidget();
	if (ViewportWidget->IsPlayInEditorViewportActive() || GetLevelViewportClient().IsSimulateInEditorViewport())
	{
		GUnrealEd->EndPlayMap();
	}
}

void FAvaLevelViewportLayoutEntity::SaveConfig(const FString& InConfigSection)
{
	GetViewportWidget()->SaveConfig(InConfigSection);
}

FName FAvaLevelViewportLayoutEntity::GetType() const
{
	return GetStaticType();
}

void FAvaLevelViewportLayoutEntity::TakeHighResScreenShot() const
{
	GetLevelViewportClient().TakeHighResScreenShot();
}

FLevelEditorViewportClient& FAvaLevelViewportLayoutEntity::GetLevelViewportClient() const
{
	return GetViewportWidget()->GetLevelViewportClient();
}

bool FAvaLevelViewportLayoutEntity::IsPlayInEditorViewportActive() const
{
	return GetViewportWidget()->IsPlayInEditorViewportActive();
}

void FAvaLevelViewportLayoutEntity::RegisterGameViewportIfPIE()
{
	GetViewportWidget()->RegisterGameViewportIfPIE();
}

TSharedPtr<SLevelViewport> FAvaLevelViewportLayoutEntity::GetViewportWidget() const
{
	return Viewport->GetViewportWidget();
}
