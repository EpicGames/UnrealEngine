// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaLevelViewportModule.h"
#include "AvaLevelViewportCommands.h"
#include "AvaLevelViewportStyle.h"
#include "AvaViewportCameraHistory.h"
#include "AvaViewportUtils.h"
#include "Customizations/AvaLevelViewportCameraCustomization.h"
#include "Modules/ModuleManager.h"
#include "ToolMenus.h"
#include "ViewportClient/AvaLevelViewportClient.h"

DEFINE_LOG_CATEGORY(AvaLevelViewportLog);

namespace UE::AvaLevelViewport::Private
{
	TSharedPtr<IAvaViewportClient> GetAsAvaLevelViewportClient(FEditorViewportClient* InViewportClient)
	{
		if (FAvaLevelViewportClient::IsAvaLevelViewportClient(InViewportClient))
		{
			return static_cast<FAvaLevelViewportClient*>(InViewportClient)->AsShared();
		}

		return nullptr;
	}
}

void FAvaLevelViewportModule::StartupModule()
{
	FAvaLevelViewportCommands::Register();

	ViewportCameraHistory = MakeShared<FAvaViewportCameraHistory>();

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FAvaLevelViewportModule::RegisterMenus));

	AvaLevelViewportClientCasterDelegateHandle = FAvaViewportUtils::RegisterViewportClientCaster(
		&UE::AvaLevelViewport::Private::GetAsAvaLevelViewportClient
	);
}

void FAvaLevelViewportModule::ShutdownModule()
{
	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);

	ViewportCameraHistory.Reset();

	FAvaLevelViewportCommands::Unregister();

	FAvaLevelViewportCameraCustomization::Get()->Unregister();

	if (AvaLevelViewportClientCasterDelegateHandle.IsValid())
	{
		FAvaViewportUtils::UnregisterViewportClientCaster(AvaLevelViewportClientCasterDelegateHandle);
		AvaLevelViewportClientCasterDelegateHandle.Reset();
	}
}

void FAvaLevelViewportModule::RegisterMenus()
{
	// Owner will be used for cleanup in call to UToolMenus::UnregisterOwner
	FToolMenuOwnerScoped OwnerScoped(this);
	
	// Extension point for SAvaLevelViewportStatusBar
	{
		static FName StatusBarMenuName = UE::AvaLevelViewport::Internal::StatusBarMenuName;
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(StatusBarMenuName, NAME_None, EMultiBoxType::ToolBar, false);
		Menu->SetStyleSet(&FAvaLevelViewportStyle::Get());
		Menu->StyleName = "StatusBar";
	}

	FAvaLevelViewportCameraCustomization::Get()->Register();
}

IMPLEMENT_MODULE(FAvaLevelViewportModule, AvalancheLevelViewport)
