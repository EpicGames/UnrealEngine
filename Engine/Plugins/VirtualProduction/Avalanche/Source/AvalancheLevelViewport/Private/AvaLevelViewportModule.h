// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaViewportCameraHistory.h"
#include "Delegates/IDelegateInstance.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

DECLARE_LOG_CATEGORY_EXTERN(AvaLevelViewportLog, Log, All);

namespace UE::AvaLevelViewport::Internal
{
	static FName StatusBarMenuName = TEXT("AvalancheLevelViewport.StatusBar");
}

class FAvaLevelViewportModule : public IModuleInterface
{
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	void RegisterMenus();

	FDelegateHandle AvaLevelViewportClientCasterDelegateHandle;

	/** Handles viewport camera undo/redo */
	TSharedPtr<FAvaViewportCameraHistory> ViewportCameraHistory;
};
