// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaLevelViewportStyle.h"
#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Containers/StringFwd.h"
#include "Interfaces/IPluginManager.h"
#include "Layout/Margin.h"
#include "Math/MathFwd.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/ToolBarStyle.h"

FAvaLevelViewportStyle::FAvaLevelViewportStyle()
	: FSlateStyleSet(TEXT("AvaLevelViewport"))
{
	const FVector2f Icon16x16(16.0f, 16.0f);
	const FVector2f Icon20x20(20.0f, 20.0f);
	const FVector2f Icon22x22(22.0f, 22.0f);
	const FVector2f Icon25x25(25.0f, 25.0f);

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	SetContentRoot(Plugin->GetBaseDir() / TEXT("Resources"));

	// Grid Icons
	Set("Button.ToggleGrid", new IMAGE_BRUSH_SVG("Icons/PaletteIcons/GridToggle", Icon22x22));
	Set("Button.ToggleGridAlwaysVisible", new IMAGE_BRUSH_SVG("Icons/PaletteIcons/GridAlwaysVisibleToggle", Icon22x22));

	// Snap Icons
	Set("Button.ToggleSnap", new IMAGE_BRUSH_SVG("Icons/PaletteIcons/SnapToggle", Icon22x22));
	Set("Button.ToggleGrid", new IMAGE_BRUSH_SVG("Icons/PaletteIcons/SnapGridToggle", Icon22x22));
	Set("Button.ToggleScreen", new IMAGE_BRUSH_SVG("Icons/PaletteIcons/SnapScreenToggle", Icon22x22));
	Set("Button.ToggleActor", new IMAGE_BRUSH_SVG("Icons/PaletteIcons/SnapActorToggle", Icon22x22));

	// Viewport
	Set("Button.GameView", new IMAGE_BRUSH_SVG("Icons/EditorIcons/GameView", Icon25x25));
	Set("Button.Visualizers", new IMAGE_BRUSH_SVG("Icons/EditorIcons/Visualizers", Icon25x25));
	Set("Button.Guides", new IMAGE_BRUSH_SVG("Icons/EditorIcons/Guides", Icon25x25));
	Set("Button.Billboards", new IMAGE_BRUSH("Icons/EditorIcons/Billboards", Icon25x25));
	Set("Button.IsolateActors", new IMAGE_BRUSH("Icons/EditorIcons/IsolateActors", Icon25x25));
	Set("Button.BoundingBoxes", new IMAGE_BRUSH_SVG("Icons/EditorIcons/Mode_Bounding-Box", Icon25x25));
	Set("Button.SafeFrames", new IMAGE_BRUSH_SVG("Icons/EditorIcons/Mode_Broadcast-Safe", Icon25x25));
	Set("Button.KeyPreview", new IMAGE_BRUSH_SVG("Icons/EditorIcons/Mode_Key-Preview", Icon25x25));
	Set("Button.Snapshot", new IMAGE_BRUSH_SVG("Icons/EditorIcons/Mode_Snapshot", Icon25x25));
	Set("Button.WireframeMode", new IMAGE_BRUSH_SVG("Icons/EditorIcons/Mode_Wireframe", Icon25x25));
	Set("Button.Mask.Toggle", new IMAGE_BRUSH("Icons/EditorIcons/Mode_Mask", Icon25x25));

	// Editor
	Set("Button.SelectionLock", new IMAGE_BRUSH("Icons/EditorIcons/SelectionLock", Icon25x25));
	Set("Button.PivotMode", new IMAGE_BRUSH("Icons/EditorIcons/Pivot_Mode", Icon25x25));

	// Alignment
	Set("Icons.Alignment.Translation.TopLeft",     new IMAGE_BRUSH("Icons/DetailsPanelIcons/TopLeft",     Icon16x16));
	Set("Icons.Alignment.Translation.Top",         new IMAGE_BRUSH("Icons/DetailsPanelIcons/Top",         Icon16x16));
	Set("Icons.Alignment.Translation.TopRight",    new IMAGE_BRUSH("Icons/DetailsPanelIcons/TopRight",    Icon16x16));
	Set("Icons.Alignment.Translation.Left",        new IMAGE_BRUSH("Icons/DetailsPanelIcons/Left",        Icon16x16));
	Set("Icons.Alignment.Translation.Center",      new IMAGE_BRUSH("Icons/DetailsPanelIcons/Center",      Icon16x16));
	Set("Icons.Alignment.Translation.Right",       new IMAGE_BRUSH("Icons/DetailsPanelIcons/Right",       Icon16x16));
	Set("Icons.Alignment.Translation.BottomLeft",  new IMAGE_BRUSH("Icons/DetailsPanelIcons/BottomLeft",  Icon16x16));
	Set("Icons.Alignment.Translation.Bottom",      new IMAGE_BRUSH("Icons/DetailsPanelIcons/Bottom",      Icon16x16));
	Set("Icons.Alignment.Translation.BottomRight", new IMAGE_BRUSH("Icons/DetailsPanelIcons/BottomRight", Icon16x16));
	
	Set("Icons.Alignment.Translation.Back",        new IMAGE_BRUSH_SVG("Icons/DetailsPanelIcons/StackBack",   Icon22x22));
	Set("Icons.Alignment.Translation.Center_X",    new IMAGE_BRUSH_SVG("Icons/DetailsPanelIcons/StackCenter", Icon22x22));
	Set("Icons.Alignment.Translation.Front",       new IMAGE_BRUSH_SVG("Icons/DetailsPanelIcons/StackFront",  Icon22x22));

	Set("Icons.Alignment.Center_YZ",   new IMAGE_BRUSH_SVG("Icons/PaletteIcons/AlignCenterVertAndHoriz", Icon22x22));
	Set("Icons.Alignment.Left",        new IMAGE_BRUSH_SVG("Icons/PaletteIcons/AlignLeft",               Icon22x22));
	Set("Icons.Alignment.Center_Y",    new IMAGE_BRUSH_SVG("Icons/PaletteIcons/AlignCenterHoriz",        Icon22x22));
	Set("Icons.Alignment.Right",       new IMAGE_BRUSH_SVG("Icons/PaletteIcons/AlignRight",              Icon22x22));
	Set("Icons.Alignment.Top",         new IMAGE_BRUSH_SVG("Icons/PaletteIcons/AlignTop",                Icon22x22));
	Set("Icons.Alignment.Center_Z",    new IMAGE_BRUSH_SVG("Icons/PaletteIcons/AlignCenterVert",         Icon22x22));
	Set("Icons.Alignment.Bottom",      new IMAGE_BRUSH_SVG("Icons/PaletteIcons/AlignBottom",             Icon22x22));
	Set("Icons.Alignment.DistributeX", new IMAGE_BRUSH_SVG("Icons/PaletteIcons/DistributeDepth",         Icon22x22));
	Set("Icons.Alignment.DistributeY", new IMAGE_BRUSH_SVG("Icons/PaletteIcons/DistributeHorizontal",    Icon22x22));
	Set("Icons.Alignment.DistributeZ", new IMAGE_BRUSH_SVG("Icons/PaletteIcons/DistributeVertical",      Icon22x22));
	
	Set("Icons.Alignment.Rotation.Actor.Roll",   new IMAGE_BRUSH_SVG("Icons/PaletteIcons/RotationAlignActorRoll",   Icon22x22));
	Set("Icons.Alignment.Rotation.Actor.Pitch",  new IMAGE_BRUSH_SVG("Icons/PaletteIcons/RotationAlignActorPitch",  Icon22x22));
	Set("Icons.Alignment.Rotation.Actor.Yaw",    new IMAGE_BRUSH_SVG("Icons/PaletteIcons/RotationAlignActorYaw",    Icon22x22));
	Set("Icons.Alignment.Rotation.Actor.All",    new IMAGE_BRUSH_SVG("Icons/PaletteIcons/RotationAlignActorAll",    Icon22x22));
	Set("Icons.Alignment.Rotation.Camera.Roll",  new IMAGE_BRUSH_SVG("Icons/PaletteIcons/RotationAlignCameraRoll",  Icon22x22));
	Set("Icons.Alignment.Rotation.Camera.Pitch", new IMAGE_BRUSH_SVG("Icons/PaletteIcons/RotationAlignCameraPitch", Icon22x22));
	Set("Icons.Alignment.Rotation.Camera.Yaw",   new IMAGE_BRUSH_SVG("Icons/PaletteIcons/RotationAlignCameraYaw",   Icon22x22));
	Set("Icons.Alignment.Rotation.Camera.All",   new IMAGE_BRUSH_SVG("Icons/PaletteIcons/RotationAlignCameraAll",   Icon22x22));

	// Screen Icons
	Set("Icons.Screen.SizeToScreen",        new IMAGE_BRUSH_SVG("Icons/PaletteIcons/SizeToScreen",        Icon22x22));
	Set("Icons.Screen.SizeToScreenStretch", new IMAGE_BRUSH_SVG("Icons/PaletteIcons/SizeToScreenStretch", Icon22x22));
	Set("Icons.Screen.FitToScreen",         new IMAGE_BRUSH_SVG("Icons/PaletteIcons/FitToScreen",         Icon22x22));

	// Color picker icons
	Set("Icons.ColorPicker.SolidColors",    new IMAGE_BRUSH("Icons/EditorIcons/SolidColors",    Icon20x20));
	Set("Icons.ColorPicker.LinearGradient", new IMAGE_BRUSH("Icons/EditorIcons/LinearGradient", Icon20x20));

	// Post process icons
	Set("Icons.PostProcess.RGB",   new IMAGE_BRUSH("Icons/ViewportIcons/RGBSquare", Icon20x20));
	Set("Icons.PostProcess.Red",   new IMAGE_BRUSH("Icons/ViewportIcons/RedSquare",   Icon20x20));
	Set("Icons.PostProcess.Green", new IMAGE_BRUSH("Icons/ViewportIcons/GreenSquare", Icon20x20));
	Set("Icons.PostProcess.Blue",  new IMAGE_BRUSH("Icons/ViewportIcons/BlueSquare",  Icon20x20));
	Set("Icons.PostProcess.Alpha", new IMAGE_BRUSH("Icons/ViewportIcons/AlphaSquare", Icon20x20));

	// StatusBar ToolMenu
	{
    	FToolBarStyle StatusToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("AssetEditorToolbar");

    	StatusToolbarStyle.SetButtonPadding(       FMargin(0.0f, 0.0f));
    	StatusToolbarStyle.SetCheckBoxPadding(     FMargin(0.0f, 0.0f));
    	StatusToolbarStyle.SetComboButtonPadding(  FMargin(0.0f, 0.0f));
    	StatusToolbarStyle.SetIndentedBlockPadding(FMargin(0.0f, 0.0f));
    	StatusToolbarStyle.SetBlockPadding(        FMargin(0.0f, 0.0f));
    	StatusToolbarStyle.SetSeparatorPadding(    FMargin(0.0f, 0.0f));
    	StatusToolbarStyle.bShowLabels = false;
    	StatusToolbarStyle.SetBackground(FSlateColorBrush(FStyleColors::Transparent));
    	StatusToolbarStyle.SetBackgroundPadding(0);
    	StatusToolbarStyle.SetButtonPadding(0);
    	StatusToolbarStyle.SetCheckBoxPadding(0);
    	
    	Set("StatusBar", StatusToolbarStyle);
    }

	FCheckBoxStyle CheckBoxStyle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("DetailsView.SectionButton");
	CheckBoxStyle.Padding.Left = 10.f;
	CheckBoxStyle.Padding.Right = 10.f;

	Set("Avalanche.Alignment.Context", CheckBoxStyle);
	
	FButtonStyle ButtonStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button");
	ButtonStyle.NormalPadding.Left = 4.f;
	ButtonStyle.NormalPadding.Right = 4.f;
	ButtonStyle.PressedPadding = ButtonStyle.NormalPadding;

	Set("Avalanche.Alignment.Button", ButtonStyle);

	FButtonStyle GuidePresetMenuStyle = FAppStyle::GetWidgetStyle<FButtonStyle>("FlatButton");
	GuidePresetMenuStyle.SetNormalPadding(2.f);
	GuidePresetMenuStyle.SetPressedPadding(2.f);
	GuidePresetMenuStyle.SetDisabled(GuidePresetMenuStyle.Normal);
	GuidePresetMenuStyle.Hovered.TintColor = FSlateColor(EStyleColor::Highlight).GetSpecifiedColor();
	GuidePresetMenuStyle.SetPressed(GuidePresetMenuStyle.Hovered);

	Set("Avalanche.Menu.GuidePreset.Button", GuidePresetMenuStyle);

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FAvaLevelViewportStyle::~FAvaLevelViewportStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
