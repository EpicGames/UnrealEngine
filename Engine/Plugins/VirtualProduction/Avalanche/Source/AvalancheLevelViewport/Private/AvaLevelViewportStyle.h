// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FAvaLevelViewportStyle final : public FSlateStyleSet
{
public:
	static FAvaLevelViewportStyle& Get()
	{
		static FAvaLevelViewportStyle Instance;
		return Instance;
	}

	FAvaLevelViewportStyle();
	virtual ~FAvaLevelViewportStyle() override;
};
