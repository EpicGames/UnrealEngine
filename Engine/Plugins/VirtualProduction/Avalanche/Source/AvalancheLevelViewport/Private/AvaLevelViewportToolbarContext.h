// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "AvaLevelViewportToolbarContext.generated.h"

class SAvaLevelViewport;

UCLASS()
class UAvaLevelViewportToolbarContext : public UObject
{
	GENERATED_BODY()

public:
	UAvaLevelViewportToolbarContext() = default;

	void SetViewport(const TSharedPtr<SAvaLevelViewport>& InViewport) { ViewportWeak = InViewport; }

	TSharedPtr<SAvaLevelViewport> GetViewport() const { return ViewportWeak.Pin(); }

private:
	TWeakPtr<SAvaLevelViewport> ViewportWeak;
};
