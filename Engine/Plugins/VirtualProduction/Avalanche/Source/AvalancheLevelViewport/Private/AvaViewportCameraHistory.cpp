// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaViewportCameraHistory.h"

#include "AvaLevelViewportCommands.h"
#include "AvaLog.h"
#include "Framework/Notifications/NotificationManager.h"
#include "LevelEditor.h"
#include "LevelEditor/AvaLevelEditorUtils.h"
#include "Math/Transform.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "AvaViewportCameraHistory"

namespace UE::AvaEditor::Private
{
	static FText UndoRedoMessage = LOCTEXT("UndoRedoMessage", "Camera Movement");

	/** Modulus that correctly wraps negative values. */
	int32 WrapIndex(int32 InValue, const int32 InNum)
	{
		if (InNum == 0)
		{
			return 0;
		}
		
		const int32 Result = ((InValue %= InNum) < 0) ? InValue + InNum : InValue;
		return Result;
	}
}

FAvaViewportCameraHistory::FAvaViewportCameraHistory()
{
	FEditorDelegates::OnMapOpened.AddRaw(this, &FAvaViewportCameraHistory::MapOpened);
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FAvaViewportCameraHistory::PostEngineInit);
}

FAvaViewportCameraHistory::~FAvaViewportCameraHistory()
{
	FEditorDelegates::OnMapOpened.RemoveAll(this);
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	
	UnbindCommands();
}

void FAvaViewportCameraHistory::BindCommands()
{
	const FLevelEditorModule* const LevelEditorModule = FAvaLevelEditorUtils::GetLevelEditorModule();
	if (!LevelEditorModule)
	{
		return;
	}
	
	const FAvaLevelViewportCommands& AvaLevelViewportCommands = FAvaLevelViewportCommands::GetInternal();
	const TSharedRef<FUICommandList> LevelEditorActions = LevelEditorModule->GetGlobalLevelEditorActions();

	LevelEditorActions->MapAction(AvaLevelViewportCommands.CameraTransformUndo
		, FExecuteAction::CreateSP(this, &FAvaViewportCameraHistory::ExecuteCameraTransformUndo));

	LevelEditorActions->MapAction(AvaLevelViewportCommands.CameraTransformRedo
		, FExecuteAction::CreateSP(this, &FAvaViewportCameraHistory::ExecuteCameraTransformRedo));

	BindCameraTransformDelegates();
}

void FAvaViewportCameraHistory::UnbindCommands()
{
	UnbindCameraTransformDelegates();
	
	const FLevelEditorModule* const LevelEditorModule = FAvaLevelEditorUtils::GetLevelEditorModule();
	if (!LevelEditorModule)
	{
		return;
	}
	
	const FAvaLevelViewportCommands& AvaLevelViewportCommands = FAvaLevelViewportCommands::GetInternal();
	const TSharedRef<FUICommandList> LevelEditorActions = LevelEditorModule->GetGlobalLevelEditorActions();

	LevelEditorActions->UnmapAction(AvaLevelViewportCommands.CameraTransformUndo);
	LevelEditorActions->UnmapAction(AvaLevelViewportCommands.CameraTransformRedo);
}

void FAvaViewportCameraHistory::MapOpened(const FString& InMapName, bool bIsTemplate)
{
	Reset();
}

void FAvaViewportCameraHistory::PostEngineInit()
{
	BindCommands();
}

void FAvaViewportCameraHistory::Reset()
{
	CameraTransformHistory.Reset();
	CameraTransformHistory.SetNumUninitialized(UE::AvaEditor::Internal::CameraUndoHistoryCapacity, EAllowShrinking::No);
	CameraTransformHistoryIndex = INDEX_NONE;
	CameraTransformHistoryHeadIndex = 1;
}

void FAvaViewportCameraHistory::BindCameraTransformDelegates()
{
	check(GEditor);
	
	if (GEditor)
	{
		OnBeginCameraTransformHandle = GEditor->OnBeginCameraMovement().AddSP(this, &FAvaViewportCameraHistory::OnBeginCameraTransform);
		OnEndCameraTransformHandle = GEditor->OnEndCameraMovement().AddSP(this, &FAvaViewportCameraHistory::OnEndCameraTransform);
	}
}

void FAvaViewportCameraHistory::UnbindCameraTransformDelegates()
{
	if (GEditor)
	{
		GEditor->OnBeginCameraMovement().RemoveAll(this);
		GEditor->OnEndCameraMovement().RemoveAll(this);
	}
}

void FAvaViewportCameraHistory::OnBeginCameraTransform(UObject& InCameraObject)
{
	if (!::IsValid(&InCameraObject))
	{
		return;
	}

	if (AActor* CameraActor = Cast<AActor>(&InCameraObject))
	{
		// First move must be added
		if (CameraTransformHistoryIndex == INDEX_NONE)
		{
			CameraTransformHistoryIndex = 0;
			CameraTransformHistory[CameraTransformHistoryIndex] = { CameraActor, CameraActor->GetActorTransform() };
			CameraTransformHistoryHeadIndex = CameraTransformHistoryIndex + 1;
		}
	}
}

void FAvaViewportCameraHistory::OnEndCameraTransform(UObject& InCameraObject)
{
	if (!::IsValid(&InCameraObject))
	{
		return;
	}

	if (AActor* CameraActor = Cast<AActor>(&InCameraObject))
	{
		// New item replaces next
		int32 CameraTransformIndexToAdd = UE::AvaEditor::Private::WrapIndex(CameraTransformHistoryIndex + 1, CameraTransformHistory.Max());

		CameraTransformHistory[CameraTransformIndexToAdd] = { CameraActor, CameraActor->GetActorTransform() };
		
		CameraTransformHistoryIndex = CameraTransformIndexToAdd;
		CameraTransformHistoryHeadIndex = UE::AvaEditor::Private::WrapIndex(CameraTransformHistoryIndex + 1, CameraTransformHistory.Max());

		UE_LOGF(LogAva, Verbose, "Saved camera transform at %i, head %i, %ls", CameraTransformHistoryIndex, CameraTransformHistoryHeadIndex, *CameraActor->GetActorTransform().Rotator().ToString());
	}
}

void FAvaViewportCameraHistory::ExecuteCameraTransformUndo()
{
	const int32 CameraTransformIndexToRestore = UE::AvaEditor::Private::WrapIndex(CameraTransformHistoryIndex - 1, CameraTransformHistory.Max());
	if (CameraTransformIndexToRestore == CameraTransformHistoryHeadIndex)
	{
		return;
	}

	const TPair<TWeakObjectPtr<AActor>, FTransform> UndoPair = CameraTransformHistory[CameraTransformIndexToRestore];
	if (AActor* CameraActor = UndoPair.Key.Get())
	{
		CameraTransformHistoryIndex = CameraTransformIndexToRestore;
		CameraActor->SetActorTransform(UndoPair.Value);
		NotifyUndo();

		UE_LOGF(LogAva, Verbose, "Undo camera transform at %i, %ls", CameraTransformIndexToRestore, *CameraActor->GetActorTransform().Rotator().ToString());
	}
}

void FAvaViewportCameraHistory::ExecuteCameraTransformRedo()
{
	const int32 CameraTransformIndexToRestore = UE::AvaEditor::Private::WrapIndex(CameraTransformHistoryIndex + 1, CameraTransformHistory.Max());
	if (CameraTransformIndexToRestore == CameraTransformHistoryHeadIndex)
	{
		return;
	}

	const TPair<TWeakObjectPtr<AActor>, FTransform> RedoPair = CameraTransformHistory[CameraTransformIndexToRestore];	
	if (AActor* CameraActor = RedoPair.Key.Get())
	{
		CameraTransformHistoryIndex = CameraTransformIndexToRestore;
		CameraActor->SetActorTransform(RedoPair.Value);
		NotifyRedo();

		UE_LOGF(LogAva, Verbose, "Redo camera transform at %i, %ls", CameraTransformIndexToRestore, *CameraActor->GetActorTransform().Rotator().ToString());
	}
}

void FAvaViewportCameraHistory::NotifyUndo()
{
	const FText UndoMessage = NSLOCTEXT("UnrealEd", "UndoMessageFormat", "Undo: {0}");
	Notify(FText::Format(UndoMessage, UE::AvaEditor::Private::UndoRedoMessage));
}

void FAvaViewportCameraHistory::NotifyRedo()
{
	const FText RedoMessage = NSLOCTEXT("UnrealEd", "RedoMessageFormat", "Redo: {0}");
	Notify(FText::Format(RedoMessage, UE::AvaEditor::Private::UndoRedoMessage));
}

// @see: UEditorEngine::ShowUndoRedoNotification
void FAvaViewportCameraHistory::Notify(const FText& InText)
{
	// Add a new notification item only if the previous one has expired or is otherwise done fading out (CS_None). This way multiple undo/redo notifications do not pollute the notification window.
	if (!UndoRedoNotificationItem.IsValid() || UndoRedoNotificationItem->GetCompletionState() == SNotificationItem::CS_None)
	{
		FNotificationInfo Info(InText);
		Info.bUseLargeFont = false;
		Info.bUseSuccessFailIcons = false;

		UndoRedoNotificationItem = FSlateNotificationManager::Get().AddNotification(Info);
	}

	if (UndoRedoNotificationItem.IsValid())
	{
		// Update the text and completion state to reflect current info
		UndoRedoNotificationItem->SetText(InText);
		UndoRedoNotificationItem->SetCompletionState(SNotificationItem::CS_Success);

		// Restart the fade animation for the current undo/redo notification
		UndoRedoNotificationItem->ExpireAndFadeout();
	}
}

#undef LOCTEXT_NAMESPACE
