// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Delegates/IDelegateInstance.h"
#include "Math/MathFwd.h"
#include "Math/Transform.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

namespace UE::AvaEditor::Internal
{
	static constexpr int32 CameraUndoHistoryCapacity = 64;
}

class AActor;
class SNotificationItem;
class UObject;

/** Handles undo/redo for camera viewport movement. */
class FAvaViewportCameraHistory
	: public TSharedFromThis<FAvaViewportCameraHistory>
{
public:
	FAvaViewportCameraHistory();
	~FAvaViewportCameraHistory();

	void BindCommands();
	void UnbindCommands();

private:
	void MapOpened(const FString& InMapName, bool bIsTemplate);
	void PostEngineInit();

	void Reset();
	
	void BindCameraTransformDelegates();
	void UnbindCameraTransformDelegates();

	void OnBeginCameraTransform(UObject& InCameraObject);
	void OnEndCameraTransform(UObject& InCameraObject);

	void ExecuteCameraTransformUndo();
	void ExecuteCameraTransformRedo();

	void NotifyUndo();
	void NotifyRedo();
	void Notify(const FText& InText);

private:
	FDelegateHandle OnBeginCameraTransformHandle;
	FDelegateHandle OnEndCameraTransformHandle;

	/** The notification item to use for undo/redo */
	TSharedPtr<SNotificationItem> UndoRedoNotificationItem;

	// Stores previous camera transforms (as camera, transform pair) for undo/redo, if any
	TArray<TPair<TWeakObjectPtr<AActor>, FTransform>, TFixedAllocator<UE::AvaEditor::Internal::CameraUndoHistoryCapacity>> CameraTransformHistory;

	// Stores current position in the camera transform history, to allow redo of actions ahead of this position
	int32 CameraTransformHistoryIndex = INDEX_NONE;

	// Stores current position of the array "end", needed due to circular storage
	int32 CameraTransformHistoryHeadIndex = 1;
};
