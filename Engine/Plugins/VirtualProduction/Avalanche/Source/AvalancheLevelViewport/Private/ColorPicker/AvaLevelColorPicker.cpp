// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaLevelColorPicker.h"
#include "AvaDefs.h"
#include "ColorPicker/AvaViewportColorPickerActorClassRegistry.h"
#include "EditorModeManager.h"
#include "Selection.h"
#include "Selection/AvaEditorSelection.h"
#include "Toolkits/IToolkitHost.h"

FAvaLevelColorPicker::FAvaLevelColorPicker()
{
	OnSelectionChangedHandle = USelection::SelectionChangedEvent.AddRaw(this, &FAvaLevelColorPicker::UpdateFromSelection);
}

FAvaLevelColorPicker::~FAvaLevelColorPicker()
{
	USelection::SelectionChangedEvent.Remove(OnSelectionChangedHandle);
	OnSelectionChangedHandle.Reset();
}

void FAvaLevelColorPicker::SetToolkitHost(const TSharedRef<IToolkitHost>& InToolkitHost)
{
	ToolkitHostWeak = InToolkitHost;
	UpdateFromSelection(nullptr);
}

void FAvaLevelColorPicker::OnColorSelected(const FAvaColorChangeData& InNewColorData)
{
	LastColorData = InNewColorData;
	ApplyColorToSelections(InNewColorData);
}

void FAvaLevelColorPicker::ApplyColorToSelections(const FAvaColorChangeData& InNewColorData)
{
	TSharedPtr<IToolkitHost> ToolkitHost = ToolkitHostWeak.Pin();
	if (!ToolkitHost.IsValid())
	{
		return;
	}

	FAvaEditorSelection EditorSelection(ToolkitHost->GetEditorModeManager());
	if (!EditorSelection.IsValid())
	{
		return;
	}

	TArray<AActor*> SelectedActors = EditorSelection.GetSelectedObjects<AActor>();

	bool bColorApplied = false;

	for (AActor* Actor : SelectedActors)
	{
		if (FAvaViewportColorPickerActorClassRegistry::ApplyColorDataToActor(Actor, InNewColorData))
		{
			bColorApplied = true;
		}
	}

	if (bColorApplied)
	{
		GEditor->RedrawLevelEditingViewports();
	}
}

void FAvaLevelColorPicker::UpdateFromSelection(UObject* InSelection)
{
	TSharedPtr<IToolkitHost> ToolkitHost = ToolkitHostWeak.Pin();
	if (!ToolkitHost.IsValid())
	{
		return;
	}

	FAvaEditorSelection EditorSelection(ToolkitHost->GetEditorModeManager(), InSelection);
	if (!EditorSelection.IsValid())
	{
		return;
	}

	TArray<AActor*> SelectedActors = EditorSelection.GetSelectedObjects<AActor>();
	if (SelectedActors.Num() == 1)
	{
		const AActor* const ColorSource = SelectedActors[0];

		FAvaColorChangeData NewColorData;
		if (FAvaViewportColorPickerActorClassRegistry::GetColorDataFromActor(ColorSource, NewColorData))
		{
			LastColorData = NewColorData;
		}
	}
}
