// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaDefs.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"

class IToolkitHost;

class FAvaLevelColorPicker
{
public:
	FAvaLevelColorPicker();
	~FAvaLevelColorPicker();

	void SetToolkitHost(const TSharedRef<IToolkitHost>& InToolkitHost);

	void OnColorSelected(const FAvaColorChangeData& InNewColorData);

	const FAvaColorChangeData& GetLastColorData() const
	{
		return LastColorData;
	}

private:
	void ApplyColorToSelections(const FAvaColorChangeData& InNewColorData);

	void UpdateFromSelection(UObject* InSelection);

	TWeakPtr<IToolkitHost> ToolkitHostWeak;

	FAvaColorChangeData LastColorData;

	FDelegateHandle OnSelectionChangedHandle;
};
