// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorPicker/AvaViewportColorPickerActorClassRegistry.h"
#include "AvaDefs.h"
#include "ColorPicker/IAvaViewportColorPickerAdapter.h"
#include "Containers/Map.h"
#include "GameFramework/Actor.h"
#include "Templates/SharedPointer.h"

namespace UE::AvaLevelViewport::Private
{
	TMap<UClass*, TSharedRef<IAvaViewportColorPickerAdapter>> Adapters;

	TSharedPtr<IAvaViewportColorPickerAdapter> FindAdapterForClass(const AActor* InActor)
	{
		UClass* ActorClass = IsValid(InActor) ? InActor->GetClass() : nullptr;
		if (!ActorClass)
		{
			return nullptr;
		}

		for (UClass* Class = ActorClass; Class != AActor::StaticClass(); Class = Class->GetSuperClass())
		{
			if (const TSharedRef<IAvaViewportColorPickerAdapter>* Adapter = Adapters.Find(Class))
			{
				return *Adapter;
			}
		}

		return nullptr;
	}
}

void FAvaViewportColorPickerActorClassRegistry::RegisterClassAdapter(UClass* InClass, const TSharedRef<IAvaViewportColorPickerAdapter>& InAdapter)
{
	using namespace UE::AvaLevelViewport;
	Private::Adapters.Add(InClass, InAdapter);
}

bool FAvaViewportColorPickerActorClassRegistry::GetColorDataFromActor(const AActor* InActor, FAvaColorChangeData& OutColorData)
{
	using namespace UE::AvaLevelViewport;

	if (TSharedPtr<IAvaViewportColorPickerAdapter> Adapter = Private::FindAdapterForClass(InActor))
	{
		// Actor has already been checked for validity in FindAdapterForClass
		return Adapter->GetColorData(*InActor, OutColorData);
	}
	return false;
}

bool FAvaViewportColorPickerActorClassRegistry::ApplyColorDataToActor(AActor* InActor, const FAvaColorChangeData& InColorData)
{
	using namespace UE::AvaLevelViewport;

	if (TSharedPtr<IAvaViewportColorPickerAdapter> Adapter = Private::FindAdapterForClass(InActor))
	{
		// Actor has already been checked for validity in FindAdapterForClass
		Adapter->SetColorData(*InActor, InColorData);
		return true;
	}

	return false;
}
