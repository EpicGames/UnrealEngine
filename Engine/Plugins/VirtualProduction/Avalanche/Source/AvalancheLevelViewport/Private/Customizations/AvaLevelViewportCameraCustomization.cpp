// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/AvaLevelViewportCameraCustomization.h"

#include "SAvaLevelViewport.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ViewportToolbar/UnrealEdViewportToolbarContext.h"

namespace UE::AvalancheLevelViewport::Private
{
	constexpr const TCHAR* ToolbarMenuName = TEXT("LevelEditor.LevelViewportToolBar.Camera");
	constexpr const TCHAR* ToolbarSectionName = TEXT("VirtualViewport");
}

const TSharedRef<FAvaLevelViewportCameraCustomization>& FAvaLevelViewportCameraCustomization::Get()
{
	static TSharedRef<FAvaLevelViewportCameraCustomization> Instance = MakeShared<FAvaLevelViewportCameraCustomization>();
	return Instance;
}

void FAvaLevelViewportCameraCustomization::Register()
{
	using namespace UE::AvalancheLevelViewport::Private;

	UToolMenus* ToolMenus = UToolMenus::Get();

	if (!ToolMenus->IsMenuRegistered(ToolbarMenuName))
	{
		ToolMenus->RegisterMenu(ToolbarMenuName);
	}

	UToolMenu* Menu = ToolMenus->ExtendMenu(ToolbarMenuName);

	Menu->AddDynamicSection(
		ToolbarSectionName,
		FNewSectionConstructChoice(FNewToolMenuDelegate::CreateSP(this, &FAvaLevelViewportCameraCustomization::ExtendLevelViewportToolbar))
	);
}

void FAvaLevelViewportCameraCustomization::Unregister()
{
	using namespace UE::AvalancheLevelViewport::Private;

	if (!UObjectInitialized())
	{
		return;
	}

	UToolMenus* ToolMenus = UToolMenus::Get();

	UToolMenu* Menu = ToolMenus->ExtendMenu(ToolbarMenuName);
	Menu->RemoveSection(ToolbarSectionName);
}

void FAvaLevelViewportCameraCustomization::ExtendLevelViewportToolbar(UToolMenu* InToolMenu)
{
	UUnrealEdViewportToolbarContext* LevelViewportContext = InToolMenu->FindContext<UUnrealEdViewportToolbarContext>();

	if (!LevelViewportContext)
	{
		return;
	}

	TSharedPtr<SEditorViewport> ViewportWidget = LevelViewportContext->Viewport.Pin();

	if (!ViewportWidget.IsValid() || ViewportWidget->GetWidgetClass().GetWidgetType() != SAvaLevelViewport::StaticWidgetClass().GetWidgetType())
	{
		return;
	}

	TSharedPtr<SAvaLevelViewport> AvaViewportWidget = StaticCastSharedPtr<SAvaLevelViewport>(ViewportWidget);
	AvaViewportWidget->FillCameraMenu(InToolMenu, /* Include Cameras */ false);
}
