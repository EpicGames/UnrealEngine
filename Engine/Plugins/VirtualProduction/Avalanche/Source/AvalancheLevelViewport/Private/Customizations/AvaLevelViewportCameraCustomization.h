// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class UToolMenu;

class FAvaLevelViewportCameraCustomization : public TSharedFromThis<FAvaLevelViewportCameraCustomization>
{
public:
	static const TSharedRef<FAvaLevelViewportCameraCustomization>& Get();

	void Register();

	void Unregister();

protected:
	void ExtendLevelViewportToolbar(UToolMenu* InToolMenu);
};