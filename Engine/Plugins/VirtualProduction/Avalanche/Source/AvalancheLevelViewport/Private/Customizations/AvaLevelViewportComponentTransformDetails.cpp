// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/AvaLevelViewportComponentTransformDetails.h"
#include "Algo/Transform.h"
#include "Components/SceneComponent.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorModeManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "LevelEditorSubsystem.h"
#include "Math/UnitConversion.h"
#include "Modules/ModuleManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "ScopedTransaction.h"
#include "Settings/EditorProjectSettings.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "UObject/UnrealType.h"
#include "UnrealEdGlobals.h"
#include "ViewportClient/AvaLevelViewportClient.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/NumericUnitTypeInterface.inl"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SRotatorInputBox.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FAvaLevelViewportComponentTransformDetails"

class FAvaLevelViewportScopedSwitchWorldForObject
{
public:
	FAvaLevelViewportScopedSwitchWorldForObject(UObject* Object)
		: PrevWorld(nullptr)
	{
		bool bRequiresPlayWorld = false;
		if (GUnrealEd->PlayWorld && !GIsPlayInEditorWorld)
		{
			UPackage* ObjectPackage = Object->GetOutermost();
			bRequiresPlayWorld = ObjectPackage->HasAnyPackageFlags(PKG_PlayInEditor);
		}

		if (bRequiresPlayWorld)
		{
			PrevWorld = SetPlayInEditorWorld(GUnrealEd->PlayWorld);
		}
	}

	~FAvaLevelViewportScopedSwitchWorldForObject()
	{
		if (PrevWorld)
		{
			RestoreEditorWorld(PrevWorld);
		}
	}

private:
	UWorld* PrevWorld;
};

static USceneComponent* GetSceneComponentFromDetailsObject(UObject* InObject)
{
	AActor* Actor = Cast<AActor>(InObject);
	if (Actor)
	{
		return Actor->GetRootComponent();
	}

	return Cast<USceneComponent>(InObject);
}

bool FAvaLevelViewportComponentTransformDetails::bUseSelectionAsParent = true;
FOnUseSelectionAsParentChanged FAvaLevelViewportComponentTransformDetails::OnUseSelectionAsParentChangedEvent;

FAvaLevelViewportComponentTransformDetails::FAvaLevelViewportComponentTransformDetails(TSharedPtr<FAvaLevelViewportClient> InAvaViewportClient)
	: TNumericUnitTypeInterface(GetDefault<UEditorProjectAppearanceSettings>()->bDisplayUnitsOnComponentTransforms ? EUnit::Centimeters : EUnit::Unspecified)
	, SelectedActorInfo(AssetSelectionUtils::BuildSelectedActorInfo(TArray<AActor*>()))
	, SelectedObjects(TArray<TWeakObjectPtr<UObject>>())
	, NotifyHook(nullptr)
	, bPreserveScaleRatio(false)
	, bEditingRotationInUI(false)
	, bIsSliderTransaction(false)
	, AvaViewportClientWeak(InAvaViewportClient)
{
	OnUseSelectionAsParentChangedEvent.AddRaw(this, &FAvaLevelViewportComponentTransformDetails::OnUseSelectionAsParentChanged);

	GConfig->GetBool(TEXT("SelectionDetails"), TEXT("PreserveScaleRatio"), bPreserveScaleRatio, GEditorPerProjectIni);
	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FAvaLevelViewportComponentTransformDetails::OnObjectsReplaced);

	if (InAvaViewportClient.IsValid())
	{
		bCanUseSelectionAsParent = false;

		if (FEditorModeTools* const ModeTools = InAvaViewportClient->GetModeTools())
		{
			EditorSelectionSetWeak = ModeTools->GetEditorSelectionSet();
			if (EditorSelectionSetWeak.IsValid())
			{
				EditorSelectionSetWeak->OnChanged().AddRaw(this, &FAvaLevelViewportComponentTransformDetails::OnSelectionSetChanged);
			}
		}
	}
	else
	{
		bCanUseSelectionAsParent = false;
	}

	bLastUseSelectionAsParent = bUseSelectionAsParent;

	GeneratePropertyHandles();

	FCoreDelegates::OnEnginePreExit.AddRaw(this, &FAvaLevelViewportComponentTransformDetails::OnEnginePreExit);
}

FAvaLevelViewportComponentTransformDetails::~FAvaLevelViewportComponentTransformDetails()
{
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	OnUseSelectionAsParentChangedEvent.RemoveAll(this);

	if (UObjectInitialized() && EditorSelectionSetWeak.IsValid())
	{
		EditorSelectionSetWeak->OnChanged().RemoveAll(this);
	}

	FCoreDelegates::OnEnginePreExit.RemoveAll(this);

	PropertyHandles.Empty();
	TreeNodes.Empty();
	PropertyRowGenerator.Reset();
}

TSharedRef<SWidget> FAvaLevelViewportComponentTransformDetails::BuildTransformFieldLabel(EAvaLevelViewportTransformFieldType TransformField)
{
	FText Label;

	switch (TransformField)
	{
		case EAvaLevelViewportTransformFieldType::Rotation:
			Label = LOCTEXT("RotationLabel", "Rotation");
			break;
			
		case EAvaLevelViewportTransformFieldType::Scale:
			Label = LOCTEXT("ScaleLabel", "Scale");
			break;
			
		case EAvaLevelViewportTransformFieldType::Location:
		default:
			Label = LOCTEXT("LocationLabel", "Location");
			break;
	}

	FMenuBuilder MenuBuilder(true, nullptr, nullptr);

	FUIAction SetRelativeLocationAction
	(
		FExecuteAction::CreateSP(this, &FAvaLevelViewportComponentTransformDetails::OnSetAbsoluteTransform, TransformField, false),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FAvaLevelViewportComponentTransformDetails::IsAbsoluteTransformChecked, TransformField, false)
	);

	FUIAction SetWorldLocationAction
	(
		FExecuteAction::CreateSP(this, &FAvaLevelViewportComponentTransformDetails::OnSetAbsoluteTransform, TransformField, true),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FAvaLevelViewportComponentTransformDetails::IsAbsoluteTransformChecked, TransformField, true)
	);

	MenuBuilder.BeginSection(TEXT("TransformType"), FText::Format(LOCTEXT("TransformType", "{0} Type"), Label));

	MenuBuilder.AddMenuEntry
	(
		FText::Format(LOCTEXT("RelativeLabel", "Relative"), Label),
		FText::Format(LOCTEXT("RelativeLabel_ToolTip", "{0} is relative to its parent"), Label),
		FSlateIcon(),
		SetRelativeLocationAction,
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry
	(
		FText::Format(LOCTEXT("WorldLabel", "World"), Label),
		FText::Format(LOCTEXT("WorldLabel_ToolTip", "{0} is relative to the world"), Label),
		FSlateIcon(),
		SetWorldLocationAction,
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.EndSection();

	TSharedRef<SWidget> NameContent =
		SNew(SComboButton)
		.ContentPadding(0)
		.MenuContent()
		[
			MenuBuilder.MakeWidget()
		]
		.ButtonContent()
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 0.0f, 2.0f, 0.0f))
			.MinDesiredWidth(50.f)
			[
				SNew(STextBlock)
				.Text(this, &FAvaLevelViewportComponentTransformDetails::GetTransformFieldText, TransformField)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];

	if (TransformField == EAvaLevelViewportTransformFieldType::Scale)
	{
		NameContent =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				NameContent
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			[
				GetPreserveScaleRatioWidget()
			];
	}

	return NameContent;
}

FText FAvaLevelViewportComponentTransformDetails::GetTransformFieldText(EAvaLevelViewportTransformFieldType TransformField) const
{
	switch (TransformField)
	{
	case EAvaLevelViewportTransformFieldType::Location:
		return GetLocationText();
		
	case EAvaLevelViewportTransformFieldType::Rotation:
		return GetRotationText();
		
	case EAvaLevelViewportTransformFieldType::Scale:
		return GetScaleText();
		
	default:
		return FText::GetEmpty();
	}
}

bool FAvaLevelViewportComponentTransformDetails::OnCanCopy(EAvaLevelViewportTransformFieldType TransformField) const
{
	// We can only copy values if the whole field is set.  If multiple values are defined we do not copy since we are unable to determine the value
	switch (TransformField)
	{
		case EAvaLevelViewportTransformFieldType::Location:
			return CachedLocation.IsSet();

		case EAvaLevelViewportTransformFieldType::Rotation:
			return CachedRotation.IsSet();

		case EAvaLevelViewportTransformFieldType::Scale:
			return CachedScale.IsSet();

		default:
			return false;
	}
}

void FAvaLevelViewportComponentTransformDetails::OnCopy(EAvaLevelViewportTransformFieldType TransformField)
{
	CacheTransform();

	FString CopyStr;
	switch (TransformField)
	{
	case EAvaLevelViewportTransformFieldType::Location:
		CopyStr = FString::Printf(TEXT("(X=%f,Y=%f,Z=%f)"), CachedLocation.X.GetValue(), CachedLocation.Y.GetValue(), CachedLocation.Z.GetValue());
		break;
		
	case EAvaLevelViewportTransformFieldType::Rotation:
		CopyStr = FString::Printf(TEXT("(Pitch=%f,Yaw=%f,Roll=%f)"), CachedRotation.Y.GetValue(), CachedRotation.Z.GetValue(), CachedRotation.X.GetValue());
		break;
		
	case EAvaLevelViewportTransformFieldType::Scale:
		CopyStr = FString::Printf(TEXT("(X=%f,Y=%f,Z=%f)"), CachedScale.X.GetValue(), CachedScale.Y.GetValue(), CachedScale.Z.GetValue());
		break;
		
	default:
		break;
	}

	if (!CopyStr.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
	}
}

void FAvaLevelViewportComponentTransformDetails::OnPaste(EAvaLevelViewportTransformFieldType TransformField)
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	switch (TransformField)
	{
		case EAvaLevelViewportTransformFieldType::Location:
		{
			FVector Location;
			if (Location.InitFromString(PastedText))
			{
				FScopedTransaction Transaction(LOCTEXT("PasteLocation", "Paste Location"));
				OnBeginChange(EAvaLevelViewportTransformFieldType::Location);
				OnSetTransform(EAvaLevelViewportTransformFieldType::Location, EAxisList::All, Location, /* bMirror */ false, /* bCommitted */ true);
				OnEndChange(EAvaLevelViewportTransformFieldType::Location, EAxisList::All);
			}
		}
		break;
		
	case EAvaLevelViewportTransformFieldType::Rotation:
		{
			FRotator Rotation;
			PastedText.ReplaceInline(TEXT("Pitch="), TEXT("P="));
			PastedText.ReplaceInline(TEXT("Yaw="), TEXT("Y="));
			PastedText.ReplaceInline(TEXT("Roll="), TEXT("R="));
			if (Rotation.InitFromString(PastedText))
			{
				FScopedTransaction Transaction(LOCTEXT("PasteRotation", "Paste Rotation"));
				OnBeginChange(EAvaLevelViewportTransformFieldType::Rotation);
				OnSetTransform(EAvaLevelViewportTransformFieldType::Rotation, EAxisList::All, Rotation.Euler(), /* bMirror */ false, /* bCommitted */ true);
				OnEndChange(EAvaLevelViewportTransformFieldType::Rotation, EAxisList::All);
			}
		}
		break;
		
	case EAvaLevelViewportTransformFieldType::Scale:
		{
			FVector Scale;
			if (Scale.InitFromString(PastedText))
			{
				FScopedTransaction Transaction(LOCTEXT("PasteScale", "Paste Scale"));
				OnBeginChange(EAvaLevelViewportTransformFieldType::Scale);
				OnSetTransform(EAvaLevelViewportTransformFieldType::Scale, EAxisList::All, Scale, /* bMirror */ false, /* bCommitted */ true);
				OnEndChange(EAvaLevelViewportTransformFieldType::Scale, EAxisList::All);
			}
		}
		break;
		
	default:
		break;
	}
}

FUIAction FAvaLevelViewportComponentTransformDetails::CreateCopyAction(EAvaLevelViewportTransformFieldType TransformField) const
{
	FAvaLevelViewportComponentTransformDetails* const MutableThis = const_cast<FAvaLevelViewportComponentTransformDetails*>(this);
	return FUIAction(FExecuteAction::CreateSP(MutableThis, &FAvaLevelViewportComponentTransformDetails::OnCopy, TransformField)
		, FCanExecuteAction::CreateSP(this, &FAvaLevelViewportComponentTransformDetails::OnCanCopy, TransformField));
}

FUIAction FAvaLevelViewportComponentTransformDetails::CreatePasteAction(EAvaLevelViewportTransformFieldType TransformField) const
{
	FAvaLevelViewportComponentTransformDetails* const MutableThis = const_cast<FAvaLevelViewportComponentTransformDetails*>(this);
	return FUIAction(FExecuteAction::CreateSP(MutableThis, &FAvaLevelViewportComponentTransformDetails::OnPaste, TransformField));
}

TSharedPtr<SWidget> FAvaLevelViewportComponentTransformDetails::GetTransformBody()
{
	FSlateFontInfo FontInfo = IDetailLayoutBuilder::GetDetailFont();

	return SNew(SNumericVectorInputBox<FVector::FReal>)
		.X(this, &FAvaLevelViewportComponentTransformDetails::GetLocationX)
		.Y(this, &FAvaLevelViewportComponentTransformDetails::GetLocationY)
		.Z(this, &FAvaLevelViewportComponentTransformDetails::GetLocationZ)
		.bColorAxisLabels(true)
		.IsEnabled(this, &FAvaLevelViewportComponentTransformDetails::GetIsEnabled)
		.OnXChanged(this, &FAvaLevelViewportComponentTransformDetails::OnSetTransformAxis, ETextCommit::Default, EAvaLevelViewportTransformFieldType::Location, EAxisList::X, /* bCommitted */ false)
		.OnYChanged(this, &FAvaLevelViewportComponentTransformDetails::OnSetTransformAxis, ETextCommit::Default, EAvaLevelViewportTransformFieldType::Location, EAxisList::Y, /* bCommitted */ false)
		.OnZChanged(this, &FAvaLevelViewportComponentTransformDetails::OnSetTransformAxis, ETextCommit::Default, EAvaLevelViewportTransformFieldType::Location, EAxisList::Z, /* bCommitted */ false)
		.OnXCommitted(this, &FAvaLevelViewportComponentTransformDetails::OnSetTransformAxis, EAvaLevelViewportTransformFieldType::Location, EAxisList::X, /* bCommitted */ true)
		.OnYCommitted(this, &FAvaLevelViewportComponentTransformDetails::OnSetTransformAxis, EAvaLevelViewportTransformFieldType::Location, EAxisList::Y, /* bCommitted */ true)
		.OnZCommitted(this, &FAvaLevelViewportComponentTransformDetails::OnSetTransformAxis, EAvaLevelViewportTransformFieldType::Location, EAxisList::Z, /* bCommitted */ true)
		.Font(FontInfo)
		.AllowSpin(true)
		.SpinDelta(1)
		.OnBeginSliderMovement(this, &FAvaLevelViewportComponentTransformDetails::OnBeginLocationSlider)
		.OnEndSliderMovement(this, &FAvaLevelViewportComponentTransformDetails::OnEndLocationSlider);
}

TSharedPtr<SWidget> FAvaLevelViewportComponentTransformDetails::GetRotationBody()
{
	FSlateFontInfo FontInfo = IDetailLayoutBuilder::GetDetailFont();

	return SNew(SRotatorInputBox)
		.AllowSpin(SelectedObjects.Num() == 1)
		.Roll(this, &FAvaLevelViewportComponentTransformDetails::GetRotationX)
		.Pitch(this, &FAvaLevelViewportComponentTransformDetails::GetRotationY)
		.Yaw(this, &FAvaLevelViewportComponentTransformDetails::GetRotationZ)
		.bColorAxisLabels(true)
		.IsEnabled(this, &FAvaLevelViewportComponentTransformDetails::GetIsEnabled)
		.OnRollChanged(this, &FAvaLevelViewportComponentTransformDetails::OnSetTransformAxisFloat, ETextCommit::Default, EAvaLevelViewportTransformFieldType::Rotation, EAxisList::X, /* bCommitted */ false)
		.OnPitchChanged(this, &FAvaLevelViewportComponentTransformDetails::OnSetTransformAxisFloat, ETextCommit::Default, EAvaLevelViewportTransformFieldType::Rotation, EAxisList::Y, /* bCommitted */ false)
		.OnYawChanged(this, &FAvaLevelViewportComponentTransformDetails::OnSetTransformAxisFloat, ETextCommit::Default, EAvaLevelViewportTransformFieldType::Rotation, EAxisList::Z, /* bCommitted */ false)
		.OnRollCommitted(this, &FAvaLevelViewportComponentTransformDetails::OnSetTransformAxisFloat, EAvaLevelViewportTransformFieldType::Rotation, EAxisList::X, /* bCommitted */ true)
		.OnPitchCommitted(this, &FAvaLevelViewportComponentTransformDetails::OnSetTransformAxisFloat, EAvaLevelViewportTransformFieldType::Rotation, EAxisList::Y, /* bCommitted */ true)
		.OnYawCommitted(this, &FAvaLevelViewportComponentTransformDetails::OnSetTransformAxisFloat, EAvaLevelViewportTransformFieldType::Rotation, EAxisList::Z, /* bCommitted */ true)
		.Font(FontInfo)
		.AllowSpin(true)
		.OnYawBeginSliderMovement(this, &FAvaLevelViewportComponentTransformDetails::OnBeginRotationSlider)
		.OnPitchBeginSliderMovement(this, &FAvaLevelViewportComponentTransformDetails::OnBeginRotationSlider)
		.OnRollBeginSliderMovement(this, &FAvaLevelViewportComponentTransformDetails::OnBeginRotationSlider)
		.OnYawEndSliderMovement(this, &FAvaLevelViewportComponentTransformDetails::OnEndRotationSlider)
		.OnPitchEndSliderMovement(this, &FAvaLevelViewportComponentTransformDetails::OnEndRotationSlider)
		.OnRollEndSliderMovement(this, &FAvaLevelViewportComponentTransformDetails::OnEndRotationSlider)
		.MinSliderValue(-360.f)
		.MaxSliderValue(360.f);
}

TSharedPtr<SWidget> FAvaLevelViewportComponentTransformDetails::GetScaleBody()
{
	FSlateFontInfo FontInfo = IDetailLayoutBuilder::GetDetailFont();

	return SNew(SNumericVectorInputBox<FVector::FReal>)
		.X(this, &FAvaLevelViewportComponentTransformDetails::GetScaleX)
		.Y(this, &FAvaLevelViewportComponentTransformDetails::GetScaleY)
		.Z(this, &FAvaLevelViewportComponentTransformDetails::GetScaleZ)
		.bColorAxisLabels(true)
		.IsEnabled(this, &FAvaLevelViewportComponentTransformDetails::GetIsEnabled)
		.OnXChanged(this, &FAvaLevelViewportComponentTransformDetails::OnSetTransformAxis, ETextCommit::Default, EAvaLevelViewportTransformFieldType::Scale, EAxisList::X, /* bCommitted */ false)
		.OnYChanged(this, &FAvaLevelViewportComponentTransformDetails::OnSetTransformAxis, ETextCommit::Default, EAvaLevelViewportTransformFieldType::Scale, EAxisList::Y, /* bCommitted */ false)
		.OnZChanged(this, &FAvaLevelViewportComponentTransformDetails::OnSetTransformAxis, ETextCommit::Default, EAvaLevelViewportTransformFieldType::Scale, EAxisList::Z, /* bCommitted */ false)
		.OnXCommitted(this, &FAvaLevelViewportComponentTransformDetails::OnSetTransformAxis, EAvaLevelViewportTransformFieldType::Scale, EAxisList::X, /* bCommitted */ true)
		.OnYCommitted(this, &FAvaLevelViewportComponentTransformDetails::OnSetTransformAxis, EAvaLevelViewportTransformFieldType::Scale, EAxisList::Y, /* bCommitted */ true)
		.OnZCommitted(this, &FAvaLevelViewportComponentTransformDetails::OnSetTransformAxis, EAvaLevelViewportTransformFieldType::Scale, EAxisList::Z, /* bCommitted */ true)
		.ContextMenuExtenderX(this, &FAvaLevelViewportComponentTransformDetails::ExtendXScaleContextMenu)
		.ContextMenuExtenderY(this, &FAvaLevelViewportComponentTransformDetails::ExtendYScaleContextMenu)
		.ContextMenuExtenderZ(this, &FAvaLevelViewportComponentTransformDetails::ExtendZScaleContextMenu)
		.Font(FontInfo)
		.AllowSpin(true)
		.SpinDelta(0.0025f)
		.OnBeginSliderMovement(this, &FAvaLevelViewportComponentTransformDetails::OnBeginScaleSlider)
		.OnEndSliderMovement(this, &FAvaLevelViewportComponentTransformDetails::OnEndScaleSlider);
}

TSharedPtr<SWidget> FAvaLevelViewportComponentTransformDetails::GetOptionsHeader() 
{
	FText Tooltip = LOCTEXT("UseSelectionAsParentTooltip", "Transforms selected actors as if the selection were the common attach parent.");

	return SNew(SCheckBox)
		.IsChecked(this, &FAvaLevelViewportComponentTransformDetails::IsUseSelectionAsParentChecked)
		.OnCheckStateChanged(this, &FAvaLevelViewportComponentTransformDetails::OnUseSelectionAsParentChanged)
		.ToolTipText(Tooltip);
}

TSharedPtr<SWidget> FAvaLevelViewportComponentTransformDetails::GetOptionsBody()
{
	FText Tooltip = LOCTEXT("UseSelectionAsParentTooltip", "Transforms selected actors as if the selection were the common attach parent.");
	FSlateFontInfo FontInfo = IDetailLayoutBuilder::GetDetailFont();

	return SNew(STextBlock)
		.Text(LOCTEXT("UseSelectionAsParent", "Use Selection As Parent"))
		.Font(FontInfo)
		.ToolTipText(Tooltip);
}

TSharedRef<SWidget> FAvaLevelViewportComponentTransformDetails::GetPreserveScaleRatioWidget()
{
	// Add a checkbox to toggle between preserving the ratio of x,y,z components of scale when a value is entered
	return SNew(SCheckBox)
		.IsChecked(this, &FAvaLevelViewportComponentTransformDetails::IsPreserveScaleRatioChecked)
		.IsEnabled(this, &FAvaLevelViewportComponentTransformDetails::GetIsEnabled)
		.OnCheckStateChanged(this, &FAvaLevelViewportComponentTransformDetails::OnPreserveScaleRatioToggled)
		.Style(FAppStyle::Get(), "TransparentCheckBox")
		.ToolTipText(LOCTEXT("PreserveScaleToolTip", "When locked, all axis values scale together so the object maintains its proportions in all directions."))
		[
			SNew(SImage)
			.Image(this, &FAvaLevelViewportComponentTransformDetails::GetPreserveScaleRatioImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
		];
}

void FAvaLevelViewportComponentTransformDetails::SetTransform(EAvaLevelViewportTransformFieldType TransformField, const FVector& NewValue)
{
	FScopedTransaction Transaction(LOCTEXT("SetTransform", "Set Transform"));
	OnBeginChange(TransformField);
	OnSetTransform(TransformField, EAxisList::XYZ, NewValue, /* bMirror */ false, /* bCommitted */ true);
	OnEndChange(TransformField, EAxisList::XYZ);
}

void FAvaLevelViewportComponentTransformDetails::OnSelectionChanged(const TArray<UObject*>& InSelectedObjects)
{
	SelectedObjects.Empty();
	TArray<AActor*> ActorArray;

	for (UObject* Object : InSelectedObjects)
	{
		SelectedObjects.Add(TWeakObjectPtr<UObject>(Object));

		if (AActor* Actor = Cast<AActor>(Object))
		{
			ActorArray.Add(Actor);
		}
	}

	SelectedActorInfo = AssetSelectionUtils::BuildSelectedActorInfo(ActorArray);

	if (SelectedActorInfo.NumSelected == 0)
	{
		CachedLocation.X.Reset();
		CachedLocation.Y.Reset();
		CachedLocation.Z.Reset();

		CachedRotation.X.Reset();
		CachedRotation.Y.Reset();
		CachedRotation.Z.Reset();

		CachedScale.X.Reset();
		CachedScale.Y.Reset();
		CachedScale.Z.Reset();

		bCanUseSelectionAsParent = false;
	}
	else if (SelectedActorInfo.NumSelected == 1)
	{
		bCanUseSelectionAsParent = false;
	}
	else
	{
		bCanUseSelectionAsParent = false;
	}

	TArray<UObject*> RootComponents;
	RootComponents.Reserve(ActorArray.Num());

	for (AActor* Actor : ActorArray)
	{
		RootComponents.Add(Actor->GetRootComponent());
	}

	UpdatePropertyHandlesObjects(RootComponents);

	CacheTransform();
}

void FAvaLevelViewportComponentTransformDetails::OnSelectionSetChanged(const UTypedElementSelectionSet* InSelectionSet)
{
	OnSelectionChanged(InSelectionSet->GetSelectedObjects());
}

void FAvaLevelViewportComponentTransformDetails::Tick(float DeltaTime)
{
	if (bLastUseSelectionAsParent != bCanUseSelectionAsParent)
	{
		bLastUseSelectionAsParent = bCanUseSelectionAsParent;

		CachedLocation.X.Reset();
		CachedLocation.Y.Reset();
		CachedLocation.Z.Reset();

		CachedRotation.X.Reset();
		CachedRotation.Y.Reset();
		CachedRotation.Z.Reset();

		CachedScale.X.Reset();
		CachedScale.Y.Reset();
		CachedScale.Z.Reset();
	}

	CacheTransform();

	if (!FixedDisplayUnits.IsSet())
	{
		CacheCommonLocationUnits();
	}
}

void FAvaLevelViewportComponentTransformDetails::CacheCommonLocationUnits()
{
	float LargestValue = 0.f;
	if (CachedLocation.X.IsSet() && CachedLocation.X.GetValue() > LargestValue)
	{
		LargestValue = CachedLocation.X.GetValue();
	}
	if (CachedLocation.Y.IsSet() && CachedLocation.Y.GetValue() > LargestValue)
	{
		LargestValue = CachedLocation.Y.GetValue();
	}
	if (CachedLocation.Z.IsSet() && CachedLocation.Z.GetValue() > LargestValue)
	{
		LargestValue = CachedLocation.Z.GetValue();
	}

	SetupFixedDisplay(LargestValue);
}

void FAvaLevelViewportComponentTransformDetails::GeneratePropertyHandles()
{
	static const FName LocationName = "Location";
	static const FName RotationName = "Rotation";
	static const FName ScaleName = "Scale";
	static const FName CategoryName = "TransformCommon";

	FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
	PropertyRowGenerator = PropertyEditor.CreatePropertyRowGenerator(FPropertyRowGeneratorArgs());
	PropertyRowGenerator->SetObjects({USceneComponent::StaticClass()->GetDefaultObject()});

	TreeNodes.SetNum(3);
	PropertyHandles.SetNum(3);

	for (const TSharedRef<IDetailTreeNode>& CategoryNode : PropertyRowGenerator->GetRootTreeNodes())
	{
		if (CategoryNode->GetNodeName() != CategoryName)
		{
			continue;
		}

		TArray<TSharedRef<IDetailTreeNode>> ChildNodes;
		CategoryNode->GetChildren(ChildNodes);

		for (const TSharedRef<IDetailTreeNode>& ChildNode : ChildNodes)
		{
			if (ChildNode->GetNodeType() != EDetailNodeType::Item)
			{
				continue;
			}

			if (ChildNode->GetNodeName() == LocationName)
			{
				TreeNodes[0] = ChildNode;
				PropertyHandles[0] = ChildNode->CreatePropertyHandle();
			}
			else if (ChildNode->GetNodeName() == RotationName)
			{
				TreeNodes[1] = ChildNode;
				PropertyHandles[1] = ChildNode->CreatePropertyHandle();
			}
			else if (ChildNode->GetNodeName() == ScaleName)
			{
				TreeNodes[2] = ChildNode;
				PropertyHandles[2] = ChildNode->CreatePropertyHandle();
			}
		}

		break;
	}

	for (const TSharedPtr<IPropertyHandle>& PropertyHandle : PropertyHandles)
	{
		ensure(PropertyHandle.IsValid() && PropertyHandle->IsValidHandle());
	}
}

void FAvaLevelViewportComponentTransformDetails::UpdatePropertyHandlesObjects(TArray<UObject*> NewSceneComponents)
{
	// Cached the old handles objects.
	CachedHandlesObjects.Reset(NewSceneComponents.Num());
	Algo::Transform(NewSceneComponents, CachedHandlesObjects, [](UObject* Obj) { return TWeakObjectPtr<UObject>(Obj); });

	// If the new outer objects list is empty, the property handle is destroyed.
	if (NewSceneComponents.IsEmpty())
	{
		NewSceneComponents = {USceneComponent::StaticClass()->GetDefaultObject()};
	}

	for (TSharedPtr<IPropertyHandle>& Handle : PropertyHandles)
	{
		if (Handle)
		{
			Handle->ReplaceOuterObjects(NewSceneComponents);
		}
	}
}

bool FAvaLevelViewportComponentTransformDetails::GetIsEnabled() const
{
	return SelectedActorInfo.NumSelected > 0;
}

const FSlateBrush* FAvaLevelViewportComponentTransformDetails::GetPreserveScaleRatioImage() const
{
	return bPreserveScaleRatio
		? FAppStyle::GetBrush(TEXT("Icons.Lock"))
		: FAppStyle::GetBrush(TEXT("Icons.Unlock")) ;
}

ECheckBoxState FAvaLevelViewportComponentTransformDetails::IsPreserveScaleRatioChecked() const
{
	return bPreserveScaleRatio
		? ECheckBoxState::Checked
		: ECheckBoxState::Unchecked;
}

void FAvaLevelViewportComponentTransformDetails::OnPreserveScaleRatioToggled(ECheckBoxState NewState)
{
	bPreserveScaleRatio = (NewState == ECheckBoxState::Checked);
	GConfig->SetBool(TEXT("SelectionDetails"), TEXT("PreserveScaleRatio"), bPreserveScaleRatio, GEditorPerProjectIni);
}

FText FAvaLevelViewportComponentTransformDetails::GetLocationText() const
{
	return bAbsoluteLocation
		? LOCTEXT("AbsoluteLocation", "Absolute Location")
		: LOCTEXT("Location", "Location");
}

FText FAvaLevelViewportComponentTransformDetails::GetRotationText() const
{
	return bAbsoluteRotation
		? LOCTEXT("AbsoluteRotation", "Absolute Rotation")
		: LOCTEXT("Rotation", "Rotation");
}

FText FAvaLevelViewportComponentTransformDetails::GetScaleText() const
{
	return bAbsoluteScale
		? LOCTEXT("AbsoluteScale", "Absolute Scale")
		: LOCTEXT("Scale", "Scale");
}

void FAvaLevelViewportComponentTransformDetails::OnSetAbsoluteTransform(EAvaLevelViewportTransformFieldType TransformField, bool bAbsoluteEnabled)
{
	FBoolProperty* AbsoluteProperty = nullptr;
	FText TransactionText;

	switch (TransformField)
	{
		case EAvaLevelViewportTransformFieldType::Location:
			AbsoluteProperty = FindFProperty<FBoolProperty>(USceneComponent::StaticClass(), USceneComponent::GetAbsoluteLocationPropertyName());
			TransactionText = LOCTEXT("ToggleAbsoluteLocation", "Toggle Absolute Location");
			break;
		
		case EAvaLevelViewportTransformFieldType::Rotation:
			AbsoluteProperty = FindFProperty<FBoolProperty>(USceneComponent::StaticClass(), USceneComponent::GetAbsoluteRotationPropertyName());
			TransactionText = LOCTEXT("ToggleAbsoluteRotation", "Toggle Absolute Rotation");
			break;
		
		case EAvaLevelViewportTransformFieldType::Scale:
			AbsoluteProperty = FindFProperty<FBoolProperty>(USceneComponent::StaticClass(), USceneComponent::GetAbsoluteScalePropertyName());
			TransactionText = LOCTEXT("ToggleAbsoluteScale", "Toggle Absolute Scale");
			break;
		
		default:
			return;
	}

	bool bBeganTransaction = false;
	TArray<UObject*> ModifiedObjectsLocal;
	for (int32 ObjectIndex = 0; ObjectIndex < SelectedObjects.Num(); ++ObjectIndex)
	{
		TWeakObjectPtr<UObject> ObjectPtr = SelectedObjects[ObjectIndex];
		if (ObjectPtr.IsValid())
		{
			UObject* Object = ObjectPtr.Get();
			USceneComponent* SceneComponent = GetSceneComponentFromDetailsObject(Object);
			if (SceneComponent)
			{
				bool bOldValue = TransformField == EAvaLevelViewportTransformFieldType::Location ? SceneComponent->IsUsingAbsoluteLocation() : (TransformField == EAvaLevelViewportTransformFieldType::Rotation ? SceneComponent->IsUsingAbsoluteRotation() : SceneComponent->IsUsingAbsoluteScale());

				if (bOldValue == bAbsoluteEnabled)
				{
					// Already the desired value
					continue;
				}

				if (!bBeganTransaction)
				{
					// NOTE: One transaction per change, not per actor
					GEditor->BeginTransaction(TransactionText);
					bBeganTransaction = true;
				}

				FAvaLevelViewportScopedSwitchWorldForObject WorldSwitcher(Object);

				if (SceneComponent->HasAnyFlags(RF_DefaultSubObject))
				{
					// Default subobjects must be included in any undo/redo operations
					SceneComponent->SetFlags(RF_Transactional);
				}

				SceneComponent->PreEditChange(AbsoluteProperty);

				if (NotifyHook)
				{
					NotifyHook->NotifyPreChange(AbsoluteProperty);
				}

				switch (TransformField)
				{
				case EAvaLevelViewportTransformFieldType::Location:
					SceneComponent->SetUsingAbsoluteLocation(bAbsoluteEnabled);

					// Update RelativeLocation to maintain/stabilize position when switching between relative and world.
					if (SceneComponent->GetAttachParent())
					{
						if (SceneComponent->IsUsingAbsoluteLocation())
						{
							SceneComponent->SetRelativeLocation_Direct(SceneComponent->GetComponentTransform().GetTranslation());
						}
						else
						{
							FTransform ParentToWorld = SceneComponent->GetAttachParent()->GetSocketTransform(SceneComponent->GetAttachSocketName());
							FTransform RelativeTM = SceneComponent->GetComponentTransform().GetRelativeTransform(ParentToWorld);
							SceneComponent->SetRelativeLocation_Direct(RelativeTM.GetTranslation());
						}
					}
					break;
					
				case EAvaLevelViewportTransformFieldType::Rotation:
					SceneComponent->SetUsingAbsoluteRotation(bAbsoluteEnabled);
					break;
					
				case EAvaLevelViewportTransformFieldType::Scale:
					SceneComponent->SetUsingAbsoluteScale(bAbsoluteEnabled);
					break;
				}

				ModifiedObjectsLocal.Add(Object);
			}
		}
	}

	if (bBeganTransaction)
	{
		FPropertyChangedEvent PropertyChangedEvent(AbsoluteProperty, EPropertyChangeType::ValueSet, MakeArrayView(ModifiedObjectsLocal));

		for (UObject* Object : ModifiedObjectsLocal)
		{
			USceneComponent* SceneComponent = GetSceneComponentFromDetailsObject(Object);

			if (SceneComponent)
			{
				SceneComponent->PostEditChangeProperty(PropertyChangedEvent);
			}
		}

		if (NotifyHook)
		{
			NotifyHook->NotifyPostChange(PropertyChangedEvent, AbsoluteProperty);
		}

		GEditor->EndTransaction();

		GUnrealEd->RedrawLevelEditingViewports();
	}
}

bool FAvaLevelViewportComponentTransformDetails::IsAbsoluteTransformChecked(EAvaLevelViewportTransformFieldType TransformField, bool bAbsoluteEnabled) const
{
	switch (TransformField)
	{
	case EAvaLevelViewportTransformFieldType::Location:
		return bAbsoluteLocation == bAbsoluteEnabled;
		
	case EAvaLevelViewportTransformFieldType::Rotation:
		return bAbsoluteRotation == bAbsoluteEnabled;
		
	case EAvaLevelViewportTransformFieldType::Scale:
		return bAbsoluteScale == bAbsoluteEnabled;
		
	default:
		return false;
	}
}

struct FGetRootComponentArchetype
{
	static USceneComponent* Get(UObject* Object)
	{
		USceneComponent* const RootComponent = Object
			? GetSceneComponentFromDetailsObject(Object)
			: nullptr;
		
		return RootComponent
			? Cast<USceneComponent>(RootComponent->GetArchetype())
			: nullptr;
	}
};

bool FAvaLevelViewportComponentTransformDetails::GetLocationResetVisibility() const
{
	const USceneComponent* Archetype = FGetRootComponentArchetype::Get(SelectedObjects[0].Get());
	
	const FVector Data = Archetype
		? Archetype->GetRelativeLocation()
		: FVector::ZeroVector;

	// unset means multiple differing values, so show "Reset to Default" in that case
	return CachedLocation.IsSet()
			&& CachedLocation.X.GetValue() == Data.X
			&& CachedLocation.Y.GetValue() == Data.Y
			&& CachedLocation.Z.GetValue() == Data.Z
		? false
		: true;
}

void FAvaLevelViewportComponentTransformDetails::OnLocationResetClicked()
{
	const FText TransactionName = LOCTEXT("ResetLocation", "Reset Location");
	FScopedTransaction Transaction(TransactionName);

	const USceneComponent* Archetype = FGetRootComponentArchetype::Get(SelectedObjects[0].Get());
	const FVector Data = Archetype ? Archetype->GetRelativeLocation() : FVector::ZeroVector;

	OnBeginChange(EAvaLevelViewportTransformFieldType::Location);
	OnSetTransform(EAvaLevelViewportTransformFieldType::Location, EAxisList::All, Data, /* bMirror */ false, /* bCommitted */ true);
	OnEndChange(EAvaLevelViewportTransformFieldType::Location, EAxisList::All);
}

bool FAvaLevelViewportComponentTransformDetails::GetRotationResetVisibility() const
{
	const USceneComponent* Archetype = FGetRootComponentArchetype::Get(SelectedObjects[0].Get());
	const FVector Data = Archetype ? Archetype->GetRelativeRotation().Euler() : FVector::ZeroVector;

	// unset means multiple differing values, so show "Reset to Default" in that case
	return CachedRotation.IsSet()
			&& CachedRotation.X.GetValue() == Data.X
			&& CachedRotation.Y.GetValue() == Data.Y
			&& CachedRotation.Z.GetValue() == Data.Z
		? false
		: true;
}

void FAvaLevelViewportComponentTransformDetails::OnRotationResetClicked()
{
	const FText TransactionName = LOCTEXT("ResetRotation", "Reset Rotation");
	FScopedTransaction Transaction(TransactionName);

	const USceneComponent* Archetype = FGetRootComponentArchetype::Get(SelectedObjects[0].Get());
	const FVector Data = Archetype ? Archetype->GetRelativeRotation().Euler() : FVector::ZeroVector;

	OnBeginChange(EAvaLevelViewportTransformFieldType::Rotation);
	OnSetTransform(EAvaLevelViewportTransformFieldType::Rotation, EAxisList::All, Data, /* bMirror */ false, /* bCommitted */ true);
	OnEndChange(EAvaLevelViewportTransformFieldType::Rotation, EAxisList::All);
}

bool FAvaLevelViewportComponentTransformDetails::GetScaleResetVisibility() const
{
	const USceneComponent* Archetype = FGetRootComponentArchetype::Get(SelectedObjects[0].Get());
	const FVector Data = Archetype ? Archetype->GetRelativeScale3D() : FVector(1.0f);

	// unset means multiple differing values, so show "Reset to Default" in that case
	return CachedScale.IsSet()
			&& CachedScale.X.GetValue() == Data.X
			&& CachedScale.Y.GetValue() == Data.Y
			&& CachedScale.Z.GetValue() == Data.Z
		? false
		: true;
}

void FAvaLevelViewportComponentTransformDetails::OnScaleResetClicked()
{
	const FText TransactionName = LOCTEXT("ResetScale", "Reset Scale");
	FScopedTransaction Transaction(TransactionName);

	const USceneComponent* Archetype = FGetRootComponentArchetype::Get(SelectedObjects[0].Get());
	const FVector Data = Archetype ? Archetype->GetRelativeScale3D() : FVector(1.0f);

	OnBeginChange(EAvaLevelViewportTransformFieldType::Scale);
	OnSetTransform(EAvaLevelViewportTransformFieldType::Scale, EAxisList::All, Data, /* bMirror */ false, /* bCommitted */ true);
	OnEndChange(EAvaLevelViewportTransformFieldType::Scale, EAxisList::All);
}

void FAvaLevelViewportComponentTransformDetails::ExtendXScaleContextMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("ScaleOperations", LOCTEXT("ScaleOperations", "Scale Operations"));
	MenuBuilder.AddMenuEntry(
		FText::Format(LOCTEXT("MirrorValue", "Mirror {0} Axis"), AxisDisplayInfo::GetAxisDisplayName(EAxisList::Forward)),
		FText::Format(LOCTEXT("MirrorValue_Tooltip", "Mirror scale value on the {0} axis"), AxisDisplayInfo::GetAxisDisplayName(EAxisList::Forward)),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &FAvaLevelViewportComponentTransformDetails::OnXScaleMirrored), FCanExecuteAction())
	);
	MenuBuilder.EndSection();
}

void FAvaLevelViewportComponentTransformDetails::ExtendYScaleContextMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("ScaleOperations", LOCTEXT("ScaleOperations", "Scale Operations"));
	MenuBuilder.AddMenuEntry(
		FText::Format(LOCTEXT("MirrorValue", "Mirror {0} Axis"), AxisDisplayInfo::GetAxisDisplayName(EAxisList::Left)),
		FText::Format(LOCTEXT("MirrorValue_Tooltip", "Mirror scale value on the {0} axis"), AxisDisplayInfo::GetAxisDisplayName(EAxisList::Left)),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &FAvaLevelViewportComponentTransformDetails::OnYScaleMirrored), FCanExecuteAction())
	);
	MenuBuilder.EndSection();
}

void FAvaLevelViewportComponentTransformDetails::ExtendZScaleContextMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("ScaleOperations", LOCTEXT("ScaleOperations", "Scale Operations"));
	MenuBuilder.AddMenuEntry(
		FText::Format(LOCTEXT("MirrorValue", "Mirror {0} Axis"), AxisDisplayInfo::GetAxisDisplayName(EAxisList::Up)),
		FText::Format(LOCTEXT("MirrorValue_Tooltip", "Mirror scale value on the {0} axis"), AxisDisplayInfo::GetAxisDisplayName(EAxisList::Up)),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &FAvaLevelViewportComponentTransformDetails::OnZScaleMirrored), FCanExecuteAction())
	);
	MenuBuilder.EndSection();
}

void FAvaLevelViewportComponentTransformDetails::OnXScaleMirrored()
{
	FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Mouse);
	FScopedTransaction Transaction(FText::Format(LOCTEXT("MirrorScaleTransaction", "Scale - Mirror {0} Axis"), AxisDisplayInfo::GetAxisDisplayName(EAxisList::Forward)));
	OnBeginChange(EAvaLevelViewportTransformFieldType::Scale);
	OnSetTransform(EAvaLevelViewportTransformFieldType::Scale, EAxisList::X, FVector(1.0f), /* bMirror */ true, /* bCommitted */ true);
	OnEndChange(EAvaLevelViewportTransformFieldType::Scale, EAxisList::X);
}

void FAvaLevelViewportComponentTransformDetails::OnYScaleMirrored()
{
	FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Mouse);
	FScopedTransaction Transaction(FText::Format(LOCTEXT("MirrorScaleTransaction", "Scale - Mirror {0} Axis"), AxisDisplayInfo::GetAxisDisplayName(EAxisList::Left)));
	OnBeginChange(EAvaLevelViewportTransformFieldType::Scale);
	OnSetTransform(EAvaLevelViewportTransformFieldType::Scale, EAxisList::Y, FVector(1.0f), /* bMirror */ true, /* bCommitted */ true);
	OnEndChange(EAvaLevelViewportTransformFieldType::Scale, EAxisList::Y);
}

void FAvaLevelViewportComponentTransformDetails::OnZScaleMirrored()
{
	FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Mouse);
	FScopedTransaction Transaction(FText::Format(LOCTEXT("MirrorScaleTransaction", "Scale - Mirror {0} Axis"), AxisDisplayInfo::GetAxisDisplayName(EAxisList::Up)));
	OnBeginChange(EAvaLevelViewportTransformFieldType::Scale);
	OnSetTransform(EAvaLevelViewportTransformFieldType::Scale, EAxisList::Z, FVector(1.0f), /* bMirror */ true, /* bCommitted */ true);
	OnEndChange(EAvaLevelViewportTransformFieldType::Scale, EAxisList::Z);
}

void FAvaLevelViewportComponentTransformDetails::CacheTransform()
{
	if (bCanUseSelectionAsParent && bUseSelectionAsParent)
	{
		// @TODO: Get selection accumulated transform stuff

		// Reset transforms
		FVector PivotLocation = FVector::ZeroVector; // BlueprintEditorSP->GetAvaModeTools()->PivotLocation;
		CachedLocation.X = PivotLocation.X;
		CachedLocation.Y = PivotLocation.Y;
		CachedLocation.Z = PivotLocation.Z;

		FRotator SelectionRotation = FRotator::ZeroRotator; // BlueprintEditorSP->GetAvaModeTools()->GetAccumulatedTransform().Rotator();
		CachedRotation.X = SelectionRotation.Roll;
		CachedRotation.Y = SelectionRotation.Pitch;
		CachedRotation.Z = SelectionRotation.Yaw;

		FVector SelectionScale = FVector::ZeroVector; // BlueprintEditorSP->GetAvaModeTools()->GetAccumulatedTransform().GetScale3D();
		CachedScale.X = SelectionScale.X;
		CachedScale.Y = SelectionScale.Y;
		CachedScale.Z = SelectionScale.Z;
	}

	FVector CurLoc = FVector::ZeroVector;
	FRotator CurRot = FRotator::ZeroRotator;
	FVector CurScale = FVector::ZeroVector;
	bool bFirstObject = true;
	bool bSkipFirst = false;

	if (SelectedObjects.Num() == 2)
	{
		AActor* First = Cast<AActor>(SelectedObjects[0].Get());
		UActorComponent* Second = Cast<UActorComponent>(SelectedObjects[1].Get());

		if (First && Second && Second->GetOwner() == First)
		{
			bSkipFirst = true;
		}
	}

	for (int32 ObjectIndex = 0; ObjectIndex < SelectedObjects.Num(); ++ObjectIndex)
	{
		if (ObjectIndex == 0 && bSkipFirst)
		{
			continue;
		}

		TWeakObjectPtr<UObject> ObjectPtr = SelectedObjects[ObjectIndex];

		if (ObjectPtr.IsValid())
		{
			UObject* Object = ObjectPtr.Get();
			USceneComponent* SceneComponent = GetSceneComponentFromDetailsObject(Object);

			if (bCanUseSelectionAsParent && bUseSelectionAsParent)
			{
				bAbsoluteLocation = SceneComponent ? SceneComponent->IsUsingAbsoluteLocation() : false;
				bAbsoluteScale = SceneComponent ? SceneComponent->IsUsingAbsoluteScale() : false;
				bAbsoluteRotation = SceneComponent ? SceneComponent->IsUsingAbsoluteRotation() : false;
				return;
			}

			if (!SceneComponent)
			{
				continue;
			}

			FVector Loc = SceneComponent->GetRelativeLocation();
			FRotator* FoundRotator = ObjectToRelativeRotationMap.Find(SceneComponent);
			FRotator Rot = (bEditingRotationInUI && !Object->IsTemplate() && FoundRotator) ? *FoundRotator : SceneComponent->GetRelativeRotation();
			FVector Scale = SceneComponent->GetRelativeScale3D();

			if (bFirstObject)
			{
				// Cache the current values from the first actor to see if any values differ among other actors
				CurLoc = Loc;
				CurRot = Rot;
				CurScale = Scale;

				CachedLocation.Set(Loc);
				CachedRotation.Set(Rot);
				CachedScale.Set(Scale);

				bAbsoluteLocation = SceneComponent->IsUsingAbsoluteLocation();
				bAbsoluteScale = SceneComponent->IsUsingAbsoluteScale();
				bAbsoluteRotation = SceneComponent->IsUsingAbsoluteRotation();

				bFirstObject = false;
			}
			else if (CurLoc != Loc || CurRot != Rot || CurScale != Scale)
			{
				// Check which values differ and unset the different values
				CachedLocation.X = Loc.X == CurLoc.X && CachedLocation.X.IsSet() ? Loc.X : TOptional<FVector::FReal>();
				CachedLocation.Y = Loc.Y == CurLoc.Y && CachedLocation.Y.IsSet() ? Loc.Y : TOptional<FVector::FReal>();
				CachedLocation.Z = Loc.Z == CurLoc.Z && CachedLocation.Z.IsSet() ? Loc.Z : TOptional<FVector::FReal>();

				CachedRotation.X = Rot.Roll == CurRot.Roll && CachedRotation.X.IsSet() ? Rot.Roll : TOptional<float>();
				CachedRotation.Y = Rot.Pitch == CurRot.Pitch && CachedRotation.Y.IsSet() ? Rot.Pitch : TOptional<float>();
				CachedRotation.Z = Rot.Yaw == CurRot.Yaw && CachedRotation.Z.IsSet() ? Rot.Yaw : TOptional<float>();

				CachedScale.X = Scale.X == CurScale.X && CachedScale.X.IsSet() ? Scale.X : TOptional<FVector::FReal>();
				CachedScale.Y = Scale.Y == CurScale.Y && CachedScale.Y.IsSet() ? Scale.Y : TOptional<FVector::FReal>();
				CachedScale.Z = Scale.Z == CurScale.Z && CachedScale.Z.IsSet() ? Scale.Z : TOptional<FVector::FReal>();

				// If all values are unset all values are different and we can stop looking
				const bool bAllValuesDiffer = !CachedLocation.IsSet() && !CachedRotation.IsSet() && !CachedScale.IsSet();

				if (bAllValuesDiffer)
				{
					break;
				}
			}
		}
	}
}

FVector FAvaLevelViewportComponentTransformDetails::GetAxisFilteredVector(EAxisList::Type Axis, const FVector& NewValue, const FVector& OldValue)
{
	return FVector((Axis & EAxisList::X) ? NewValue.X : OldValue.X,
		(Axis & EAxisList::Y) ? NewValue.Y : OldValue.Y,
		(Axis & EAxisList::Z) ? NewValue.Z : OldValue.Z);
}

void FAvaLevelViewportComponentTransformDetails::OnBeginChange(EAvaLevelViewportTransformFieldType TransformField)
{
	ModifiedObjects.Empty();
	OriginalLocations.Empty();
	OriginalRotations.Empty();
	OriginalScales.Empty();

	FProperty* ValueProperty = nullptr;

	switch (TransformField)
	{
		case EAvaLevelViewportTransformFieldType::Location:
			ValueProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeLocationPropertyName());
			break;

		case EAvaLevelViewportTransformFieldType::Rotation:
			ValueProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeRotationPropertyName());
			break;

		case EAvaLevelViewportTransformFieldType::Scale:
			ValueProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeScale3DPropertyName());
			break;

		default:
			return;
	}

	for (int32 ObjectIndex = 0; ObjectIndex < SelectedObjects.Num(); ++ObjectIndex)
	{
		TWeakObjectPtr<UObject> ObjectPtr = SelectedObjects[ObjectIndex];

		if (ObjectPtr.IsValid())
		{
			UObject* Object = ObjectPtr.Get();

			AActor* Actor = Cast<AActor>(Object);

			if (Actor)
			{
				Actor->Modify();
				ModifiedObjects.Add(Actor);

				if (USceneComponent* RootComponent = Actor->GetRootComponent())
				{
					RootComponent->Modify();
					ModifiedObjects.Add(RootComponent);
				}

				continue;
			}

			USceneComponent* SceneComponent = GetSceneComponentFromDetailsObject(Object);

			if (SceneComponent)
			{
				if (SceneComponent->HasAnyFlags(RF_DefaultSubObject))
				{
					// Default subobjects must be included in any undo/redo operations
					SceneComponent->SetFlags(RF_Transactional);
				}

				SceneComponent->Modify();
				SceneComponent->PreEditChange(ValueProperty);

				if (NotifyHook)
				{
					NotifyHook->NotifyPreChange(ValueProperty);
				}

				ModifiedObjects.Add(SceneComponent);

				switch (TransformField)
				{
					case EAvaLevelViewportTransformFieldType::Location:
						OriginalLocations.Emplace(SceneComponent, SceneComponent->GetRelativeLocation());
						break;

					case EAvaLevelViewportTransformFieldType::Rotation:
						OriginalRotations.Emplace(SceneComponent, SceneComponent->GetRelativeRotation());
						break;

					case EAvaLevelViewportTransformFieldType::Scale:
						OriginalScales.Emplace(SceneComponent, SceneComponent->GetRelativeScale3D());
						break;

					default:
						break;
				}
			}

			GEditor->BroadcastBeginObjectMovement(*SceneComponent);
			AActor* EditedActor = SceneComponent->GetOwner();

			if (EditedActor && EditedActor->GetRootComponent() == SceneComponent)
			{
				GEditor->BroadcastBeginObjectMovement(*EditedActor);
			}
		}
	}
}

void FAvaLevelViewportComponentTransformDetails::OnEndChange(EAvaLevelViewportTransformFieldType TransformField, EAxisList::Type Axis)
{
	FProperty* ValueProperty = nullptr;
	FProperty* AxisProperty = nullptr;

	switch (TransformField)
	{
		case EAvaLevelViewportTransformFieldType::Location:
			ValueProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeLocationPropertyName());

			// Only set axis property for single axis set
			if (Axis == EAxisList::X)
			{
				AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FVector>::Get(), GET_MEMBER_NAME_CHECKED(FVector, X));
				check(AxisProperty != nullptr);
			}
			else if (Axis == EAxisList::Y)
			{
				AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FVector>::Get(), GET_MEMBER_NAME_CHECKED(FVector, Y));
				check(AxisProperty != nullptr);
			}
			else if (Axis == EAxisList::Z)
			{
				AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FVector>::Get(), GET_MEMBER_NAME_CHECKED(FVector, Z));
				check(AxisProperty != nullptr);
			}
			break;

		case EAvaLevelViewportTransformFieldType::Rotation:
			ValueProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeRotationPropertyName());

			// Only set axis property for single axis set
			if (Axis == EAxisList::X)
			{
				AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FRotator>::Get(), GET_MEMBER_NAME_CHECKED(FRotator, Roll));
				check(AxisProperty != nullptr);
			}
			else if (Axis == EAxisList::Y)
			{
				AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FRotator>::Get(), GET_MEMBER_NAME_CHECKED(FRotator, Pitch));
				check(AxisProperty != nullptr);
			}
			else if (Axis == EAxisList::Z)
			{
				AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FRotator>::Get(), GET_MEMBER_NAME_CHECKED(FRotator, Yaw));
				check(AxisProperty != nullptr);
			}
			break;

		case EAvaLevelViewportTransformFieldType::Scale:
			ValueProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), USceneComponent::GetRelativeScale3DPropertyName());

			// If keep scale is set, don't set axis property
			if (!bPreserveScaleRatio)
			{
				if (Axis == EAxisList::X)
				{
					AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FVector>::Get(), GET_MEMBER_NAME_CHECKED(FVector, X));
					check(AxisProperty != nullptr);
				}
				else if (Axis == EAxisList::Y)
				{
					AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FVector>::Get(), GET_MEMBER_NAME_CHECKED(FVector, Y));
					check(AxisProperty != nullptr);
				}
				else if (Axis == EAxisList::Z)
				{
					AxisProperty = FindFProperty<FDoubleProperty>(TBaseStructure<FVector>::Get(), GET_MEMBER_NAME_CHECKED(FVector, Z));
					check(AxisProperty != nullptr);
				}
			}
			break;

		default:
			return;
	}

	TArray<UObject*> ArrayViewArray;
	Algo::Transform(ModifiedObjects, ArrayViewArray, [](const TWeakObjectPtr<UObject>& Ptr) { return Ptr.Get(); });

	FPropertyChangedEvent PropertyChangedEvent(ValueProperty, EPropertyChangeType::ValueSet, MakeArrayView(ArrayViewArray));
	FEditPropertyChain PropertyChain;

	if (AxisProperty)
	{
		PropertyChain.AddHead(AxisProperty);
	}
	
	PropertyChain.AddHead(ValueProperty);
	
	FPropertyChangedChainEvent PropertyChangedChainEvent(PropertyChain, PropertyChangedEvent);

	for (int32 ObjectIndex = 0; ObjectIndex < ModifiedObjects.Num(); ++ObjectIndex)
	{
		TWeakObjectPtr<UObject> ObjectPtr = ModifiedObjects[ObjectIndex];

		if (ObjectPtr.IsValid())
		{
			UObject* Object = ObjectPtr.Get();

			AActor* Actor = Cast<AActor>(Object);

			if (Actor)
			{
				Actor->PostEditChangeChainProperty(PropertyChangedChainEvent);
				Actor->PostEditMove(true);
				continue;
			}

			USceneComponent* SceneComponent = GetSceneComponentFromDetailsObject(Object);
			USceneComponent* OldSceneComponent = SceneComponent;

			if (SceneComponent)
			{
				AActor* EditedActor = SceneComponent->GetOwner();
				FString SceneComponentPath = SceneComponent->GetPathName(EditedActor);

				// This can invalidate OldSceneComponent
				OldSceneComponent->PostEditChangeChainProperty(PropertyChangedChainEvent);

				// Make sure we've got the latest one!
				SceneComponent = FindObject<USceneComponent>(EditedActor, *SceneComponentPath);

				if (SceneComponent)
				{
					// Broadcast when the actor is done moving
					GEditor->BroadcastEndObjectMovement(*SceneComponent);

					if (EditedActor && EditedActor->GetRootComponent() == SceneComponent)
					{
						GEditor->BroadcastEndObjectMovement(*EditedActor);
					}
				}
			}
		}
	}

	if (NotifyHook)
	{
		NotifyHook->NotifyPostChange(PropertyChangedEvent, ValueProperty);
	}
}

void FAvaLevelViewportComponentTransformDetails::OnSetTransform(EAvaLevelViewportTransformFieldType TransformField, EAxisList::Type Axis, FVector NewValue, bool bMirror, bool bCommitted)
{
	if (bCanUseSelectionAsParent && bUseSelectionAsParent)
	{
		OnSetTransformSelectionParent(TransformField, Axis, NewValue, bMirror, bCommitted);
		return;
	}

	for (int32 ObjectIndex = 0; ObjectIndex < SelectedObjects.Num(); ++ObjectIndex)
	{
		TWeakObjectPtr<UObject> ObjectPtr = SelectedObjects[ObjectIndex];

		if (ObjectPtr.IsValid())
		{
			UObject* Object = ObjectPtr.Get();
			USceneComponent* SceneComponent = GetSceneComponentFromDetailsObject(Object);

			if (SceneComponent)
			{
				AActor* EditedActor = SceneComponent->GetOwner();

				FVector OldComponentValue;
				FVector NewComponentValue;

				switch (TransformField)
				{
					case EAvaLevelViewportTransformFieldType::Location:
						OldComponentValue = SceneComponent->GetRelativeLocation();
						break;

					case EAvaLevelViewportTransformFieldType::Rotation:
						// Pull from the actual component or from the cache
						OldComponentValue = SceneComponent->GetRelativeRotation().Euler();
						if (bEditingRotationInUI && ObjectToRelativeRotationMap.Find(SceneComponent))
						{
							OldComponentValue = ObjectToRelativeRotationMap.Find(SceneComponent)->Euler();
						}
						break;

					case EAvaLevelViewportTransformFieldType::Scale:
						OldComponentValue = SceneComponent->GetRelativeScale3D();
						break;
				}

				// Set the incoming value
				if (bMirror)
				{
					NewComponentValue = GetAxisFilteredVector(Axis, -OldComponentValue, OldComponentValue);
				}
				else
				{
					NewComponentValue = GetAxisFilteredVector(Axis, NewValue, OldComponentValue);
				}

				// If we're committing during a slider transaction then we need to force it, in order that PostEditChangeChainProperty be called.
				// Note: this will even happen if the slider hasn't changed the value.
				bool bDoChange = bCommitted && bIsSliderTransaction;
				bDoChange = bDoChange || (OldComponentValue != NewComponentValue);

				if (bDoChange)
				{
					FAvaLevelViewportScopedSwitchWorldForObject WorldSwitcher(Object);

					switch (TransformField)
					{
						case EAvaLevelViewportTransformFieldType::Location:
							{
								ObjectToRelativeRotationMap.FindOrAdd(SceneComponent) = SceneComponent->GetRelativeRotation();
								SceneComponent->SetRelativeLocation(NewComponentValue);

								// Also forcibly set it as the cache may have changed it slightly
								SceneComponent->SetRelativeLocation_Direct(NewComponentValue);
								break;
							}
						case EAvaLevelViewportTransformFieldType::Rotation:
							{
								FRotator NewRotation = FRotator::MakeFromEuler(NewComponentValue);
								ObjectToRelativeRotationMap.FindOrAdd(SceneComponent) = NewRotation;
								SceneComponent->SetRelativeRotationExact(NewRotation);
								break;
							}
						case EAvaLevelViewportTransformFieldType::Scale:
							{
								if ((!bCanUseSelectionAsParent || !bUseSelectionAsParent) && bPreserveScaleRatio)
								{
									// If we set a single axis, scale the others
									float Ratio = 0.0f;

									switch (Axis)
									{
										case EAxisList::X:
											if (bIsSliderTransaction)
											{
												Ratio = SliderScaleRatio.X == 0.0f ? SliderScaleRatio.Y : (SliderScaleRatio.Y / SliderScaleRatio.X);
												NewComponentValue.Y = NewComponentValue.X * Ratio;

												Ratio = SliderScaleRatio.X == 0.0f ? SliderScaleRatio.Z : (SliderScaleRatio.Z / SliderScaleRatio.X);
												NewComponentValue.Z = NewComponentValue.X * Ratio;
											}
											else
											{
												Ratio = OldComponentValue.X == 0.0f ? NewComponentValue.Z : NewComponentValue.X / OldComponentValue.X;
												NewComponentValue.Y *= Ratio;
												NewComponentValue.Z *= Ratio;
											}
										break;
										case EAxisList::Y:
											if (bIsSliderTransaction)
											{
												Ratio = SliderScaleRatio.Y == 0.0f ? SliderScaleRatio.X : (SliderScaleRatio.X / SliderScaleRatio.Y);
												NewComponentValue.X = NewComponentValue.Y * Ratio;

												Ratio = SliderScaleRatio.Y == 0.0f ? SliderScaleRatio.Z : (SliderScaleRatio.Z / SliderScaleRatio.Y);
												NewComponentValue.Z = NewComponentValue.Y * Ratio;
											}
											else
											{
												Ratio = OldComponentValue.Y == 0.0f ? NewComponentValue.Z : NewComponentValue.Y / OldComponentValue.Y;
												NewComponentValue.X *= Ratio;
												NewComponentValue.Z *= Ratio;
											}
										break;
										case EAxisList::Z:
											if (bIsSliderTransaction)
											{
												Ratio = SliderScaleRatio.Z == 0.0f ? SliderScaleRatio.X : (SliderScaleRatio.X / SliderScaleRatio.Z);
												NewComponentValue.X = NewComponentValue.Z * Ratio;

												Ratio = SliderScaleRatio.Z == 0.0f ? SliderScaleRatio.Y : (SliderScaleRatio.Y / SliderScaleRatio.Z);
												NewComponentValue.Y = NewComponentValue.Z * Ratio;
											}
											else
											{
												Ratio = OldComponentValue.Z == 0.0f ? NewComponentValue.Z : NewComponentValue.Z / OldComponentValue.Z;
												NewComponentValue.X *= Ratio;
												NewComponentValue.Y *= Ratio;
											}
										break;
										default:
											// Do nothing, this set multiple axis at once
											break;
									}
								}

								SceneComponent->SetRelativeScale3D(NewComponentValue);
								break;
							}
					}

					SceneComponent->Modify();
				}
			}
		}
	}

	CacheTransform();

	if (TransformField == EAvaLevelViewportTransformFieldType::Location)
	{
		if (TSharedPtr<FAvaLevelViewportClient> ViewportClient = AvaViewportClientWeak.Pin())
		{
			if (FEditorModeTools* ModeTools = ViewportClient->GetModeTools())
			{
				FVector CurrentPivot = ModeTools->PivotLocation;

				if (Axis & EAxisList::X)
				{
					CurrentPivot.X = NewValue.X;
				}

				if (Axis & EAxisList::Y)
				{
					CurrentPivot.Y = NewValue.Y;
				}

				if (Axis & EAxisList::Z)
				{
					CurrentPivot.Z = NewValue.Z;
				}

				ModeTools->SetPivotLocation(CurrentPivot, false);
			}
		}
	}

	if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
	{
		LevelEditorSubsystem->EditorInvalidateViewports();
	}
}

void FAvaLevelViewportComponentTransformDetails::OnSetTransformSelectionParent(EAvaLevelViewportTransformFieldType TransformField, EAxisList::Type Axis, FVector NewValue, bool bMirror, bool bCommitted)
{
	FVector OldSelectionValue;
	FVector NewSelectionValue;
	FTransform OldAccumulatedTransform;
	FTransform NewAccumulatedTransform;

	switch (TransformField)
	{
		case EAvaLevelViewportTransformFieldType::Location:
			OldSelectionValue.X = CachedLocation.X.Get(0.f);
			OldSelectionValue.Y = CachedLocation.Y.Get(0.f);
			OldSelectionValue.Z = CachedLocation.Z.Get(0.f);
			break;

		case EAvaLevelViewportTransformFieldType::Rotation:
			OldSelectionValue.X = CachedRotation.X.Get(0.f);
			OldSelectionValue.Y = CachedRotation.Y.Get(0.f);
			OldSelectionValue.Z = CachedRotation.Z.Get(0.f);
			break;

		case EAvaLevelViewportTransformFieldType::Scale:
			OldSelectionValue.X = CachedScale.X.Get(0.f);
			OldSelectionValue.Y = CachedScale.Y.Get(0.f);
			OldSelectionValue.Z = CachedScale.Z.Get(0.f);
			break;
	}

	if (!bMirror)
	{
		NewSelectionValue = GetAxisFilteredVector(Axis, NewValue, OldSelectionValue);
	}

	else
	{
		NewSelectionValue = GetAxisFilteredVector(Axis, -OldSelectionValue, OldSelectionValue);
	}

	if (TransformField == EAvaLevelViewportTransformFieldType::Scale && bPreserveScaleRatio)
	{
		// If we set a single axis, scale the others
		float Ratio = 0.0f;

		switch (Axis)
		{
			case EAxisList::X:
				if (bIsSliderTransaction)
				{
					Ratio = SliderScaleRatio.X == 0.0f ? SliderScaleRatio.Y : (SliderScaleRatio.Y / SliderScaleRatio.X);
					NewSelectionValue.Y = NewSelectionValue.X * Ratio;

					Ratio = SliderScaleRatio.X == 0.0f ? SliderScaleRatio.Z : (SliderScaleRatio.Z / SliderScaleRatio.X);
					NewSelectionValue.Z = NewSelectionValue.X * Ratio;
				}
				else
				{
					Ratio = OldSelectionValue.X == 0.0f ? NewSelectionValue.Z : NewSelectionValue.X / OldSelectionValue.X;
					NewSelectionValue.Y *= Ratio;
					NewSelectionValue.Z *= Ratio;
				}
			break;
			case EAxisList::Y:
				if (bIsSliderTransaction)
				{
					Ratio = SliderScaleRatio.Y == 0.0f ? SliderScaleRatio.X : (SliderScaleRatio.X / SliderScaleRatio.Y);
					NewSelectionValue.X = NewSelectionValue.Y * Ratio;

					Ratio = SliderScaleRatio.Y == 0.0f ? SliderScaleRatio.Z : (SliderScaleRatio.Z / SliderScaleRatio.Y);
					NewSelectionValue.Z = NewSelectionValue.Y * Ratio;
				}
				else
				{
					Ratio = OldSelectionValue.Y == 0.0f ? NewSelectionValue.Z : NewSelectionValue.Y / OldSelectionValue.Y;
					NewSelectionValue.X *= Ratio;
					NewSelectionValue.Z *= Ratio;
				}
			break;
			case EAxisList::Z:
				if (bIsSliderTransaction)
				{
					Ratio = SliderScaleRatio.Z == 0.0f ? SliderScaleRatio.X : (SliderScaleRatio.X / SliderScaleRatio.Z);
					NewSelectionValue.X = NewSelectionValue.Z * Ratio;

					Ratio = SliderScaleRatio.Z == 0.0f ? SliderScaleRatio.Y : (SliderScaleRatio.Y / SliderScaleRatio.Z);
					NewSelectionValue.Y = NewSelectionValue.Z * Ratio;
				}
				else
				{
					Ratio = OldSelectionValue.Z == 0.0f ? NewSelectionValue.Z : NewSelectionValue.Z / OldSelectionValue.Z;
					NewSelectionValue.X *= Ratio;
					NewSelectionValue.Y *= Ratio;
				}
			break;
			default:
				// Do nothing, this set multiple axis at once
				break;
		}
	}

	bool bTranslationChanged = false;
	bool bRotationChanged = false;
	bool bScaleChanged = false;

	// @TODO Accumulated transform
	FVector SelectionCenter = FVector::ZeroVector; // BlueprintEditorSP->GetAvaModeTools()->GetSelectionTransform().GetLocation();
	OldAccumulatedTransform = FTransform::Identity; // BlueprintEditorSP->GetAvaModeTools()->GetAccumulatedTransform();
	NewAccumulatedTransform = OldAccumulatedTransform;

 	switch (TransformField)
	{
		case EAvaLevelViewportTransformFieldType::Location:
			{
				CachedLocation.X = NewSelectionValue.X;
				CachedLocation.Y = NewSelectionValue.Y;
				CachedLocation.Z = NewSelectionValue.Z;
				NewAccumulatedTransform.SetLocation(NewSelectionValue);
				bTranslationChanged = true;
				break;
			}
		case EAvaLevelViewportTransformFieldType::Rotation:
			{
				CachedRotation.X = NewSelectionValue.X;
				CachedRotation.Y = NewSelectionValue.Y;
				CachedRotation.Z = NewSelectionValue.Z;
				NewAccumulatedTransform.SetRotation(FRotator(NewSelectionValue.Y, NewSelectionValue.Z, NewSelectionValue.X).Quaternion());
				bRotationChanged = true;
				break;
			}
		case EAvaLevelViewportTransformFieldType::Scale:
			{
				CachedScale.X = NewSelectionValue.X;
				CachedScale.Y = NewSelectionValue.Y;
				CachedScale.Z = NewSelectionValue.Z;
				NewAccumulatedTransform.SetScale3D(NewSelectionValue);
				bScaleChanged = true;
				break;
			}
	}

	for (int32 ObjectIndex = 0; ObjectIndex < SelectedObjects.Num(); ++ObjectIndex)
	{
		TWeakObjectPtr<UObject> ObjectPtr = SelectedObjects[ObjectIndex];

		if (ObjectPtr.IsValid())
		{
			UObject* Object = ObjectPtr.Get();
			USceneComponent* SceneComponent = GetSceneComponentFromDetailsObject(Object);

			if (SceneComponent)
			{
				AActor* EditedActor = SceneComponent->GetOwner();

				// If we're committing during a slider transaction then we need to force it, in order that PostEditChangeChainProperty be called.
				// Note: this will even happen if the slider hasn't changed the value.
				bool bDoChange = bCommitted && bIsSliderTransaction;
				bDoChange = bDoChange || (OldSelectionValue != NewSelectionValue);

				if (EditedActor && bDoChange)
				{
					// @TODO Accumulated transform
					const FTransform* InitialActorTransform = nullptr; // BlueprintEditorSP->GetAvaModeTools()->GetSelectionInitialActorTransforms().Find(EditedActor);

					if (!InitialActorTransform)
						continue;

					FTransform OldSceneTransform = SceneComponent->GetComponentTransform();
					FTransform NewSceneTransform = (*InitialActorTransform) * NewAccumulatedTransform;
					SceneComponent->SetWorldTransform(NewSceneTransform);
					SceneComponent->Modify();
					ObjectToRelativeRotationMap.FindOrAdd(SceneComponent) = SceneComponent->GetRelativeRotation();
				}
			}
		}
	}

	// @TODO Accumulated transform
	// BlueprintEditorSP->GetAvaModeTools()->SetAccumulatedTransform(NewAccumulatedTransform);

	CacheTransform();
	
	if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
	{
		LevelEditorSubsystem->EditorInvalidateViewports();
	}
}

void FAvaLevelViewportComponentTransformDetails::OnSetTransformAxis(FVector::FReal NewValue, ETextCommit::Type CommitInfo, EAvaLevelViewportTransformFieldType TransformField, EAxisList::Type Axis, bool bCommitted)
{
	FVector NewVector = GetAxisFilteredVector(Axis, FVector(NewValue), FVector::ZeroVector);

	OnSetTransform(TransformField, Axis, NewVector, /* bMirror */ false, bCommitted);
}

void FAvaLevelViewportComponentTransformDetails::OnSetTransformAxisFloat(float NewValue, ETextCommit::Type CommitInfo, EAvaLevelViewportTransformFieldType TransformField, EAxisList::Type Axis, bool bCommitted)
{
	FVector NewVector = GetAxisFilteredVector(Axis, FVector(NewValue), FVector::ZeroVector);

	OnSetTransform(EAvaLevelViewportTransformFieldType::Rotation, Axis, NewVector, /* bMirror */ false, bCommitted);
}

void FAvaLevelViewportComponentTransformDetails::BeginSliderTransaction(FText ActorTransaction, FText ComponentTransaction) const
{
	bool bBeganTransaction = false;
	for (TWeakObjectPtr<UObject> ObjectPtr : SelectedObjects)
	{
		if (ObjectPtr.IsValid())
		{
			UObject* Object = ObjectPtr.Get();

			// Start a new transaction when a slider begins to change
			// We'll end it when the slider is released
			// NOTE: One transaction per change, not per actor
			if (!bBeganTransaction)
			{
				if (Object->IsA<AActor>())
				{
					GEditor->BeginTransaction(ActorTransaction);
				}
				else
				{
					GEditor->BeginTransaction(ComponentTransaction);
				}

				bBeganTransaction = true;
			}

			USceneComponent* SceneComponent = GetSceneComponentFromDetailsObject(Object);
			if (SceneComponent)
			{
				FAvaLevelViewportScopedSwitchWorldForObject WorldSwitcher(Object);

				if (SceneComponent->HasAnyFlags(RF_DefaultSubObject))
				{
					// Default subobjects must be included in any undo/redo operations
					SceneComponent->SetFlags(RF_Transactional);
				}

				// Call modify but not PreEdit, we don't do the proper "Edit" until it's committed
				SceneComponent->Modify();
			}
		}
	}

	// Just in case we couldn't start a new transaction for some reason
	if (!bBeganTransaction)
	{
		GEditor->BeginTransaction(ActorTransaction);
	}
}

void FAvaLevelViewportComponentTransformDetails::OnBeginRotationSlider()
{
	FText ActorTransaction = LOCTEXT("OnSetRotation", "Set Rotation");
	FText ComponentTransaction = LOCTEXT("OnSetRotation_ComponentDirect", "Modify Component(s)");
	BeginSliderTransaction(ActorTransaction, ComponentTransaction);

	bEditingRotationInUI = true;
	bIsSliderTransaction = true;

	for (TWeakObjectPtr<UObject> ObjectPtr : SelectedObjects)
	{
		if (ObjectPtr.IsValid())
		{
			UObject* Object = ObjectPtr.Get();

			USceneComponent* SceneComponent = GetSceneComponentFromDetailsObject(Object);
			if (SceneComponent)
			{
				FAvaLevelViewportScopedSwitchWorldForObject WorldSwitcher(Object);

				// Add/update cached rotation value prior to slider interaction
				ObjectToRelativeRotationMap.FindOrAdd(SceneComponent) = SceneComponent->GetRelativeRotation();
			}
		}
	}

	OnBeginChange(EAvaLevelViewportTransformFieldType::Rotation);
}

void FAvaLevelViewportComponentTransformDetails::OnEndRotationSlider(float NewValue)
{
	OnEndChange(EAvaLevelViewportTransformFieldType::Rotation, EAxisList::None);

	// Commit gets called right before this, only need to end the transaction
	bEditingRotationInUI = false;
	bIsSliderTransaction = false;
	GEditor->EndTransaction();
}

void FAvaLevelViewportComponentTransformDetails::OnBeginLocationSlider()
{
	bIsSliderTransaction = true;
	FText ActorTransaction = LOCTEXT("OnSetLocation", "Set Location");
	FText ComponentTransaction = LOCTEXT("OnSetLocation_ComponentDirect", "Modify Component Location");
	BeginSliderTransaction(ActorTransaction, ComponentTransaction);

	OnBeginChange(EAvaLevelViewportTransformFieldType::Location);
}

void FAvaLevelViewportComponentTransformDetails::OnEndLocationSlider(FVector::FReal NewValue)
{
	OnEndChange(EAvaLevelViewportTransformFieldType::Location, EAxisList::All);

	bIsSliderTransaction = false;
	GEditor->EndTransaction();
}

void FAvaLevelViewportComponentTransformDetails::OnBeginScaleSlider()
{
	// Assumption: slider isn't usable if multiple objects are selected
	SliderScaleRatio.X = CachedScale.X.GetValue();
	SliderScaleRatio.Y = CachedScale.Y.GetValue();
	SliderScaleRatio.Z = CachedScale.Z.GetValue();

	bIsSliderTransaction = true;
	FText ActorTransaction = LOCTEXT("OnSetScale", "Set Scale");
	FText ComponentTransaction = LOCTEXT("OnSetScale_ComponentDirect", "Modify Component Scale");
	BeginSliderTransaction(ActorTransaction, ComponentTransaction);

	OnBeginChange(EAvaLevelViewportTransformFieldType::Scale);
}

void FAvaLevelViewportComponentTransformDetails::OnEndScaleSlider(FVector::FReal NewValue)
{
	OnEndChange(EAvaLevelViewportTransformFieldType::Scale, EAxisList::All);

	bIsSliderTransaction = false;
	GEditor->EndTransaction();
}

void FAvaLevelViewportComponentTransformDetails::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	TArray<UObject*> NewSceneComponents;
	for (const TWeakObjectPtr<UObject> Obj : CachedHandlesObjects)
	{
		if (UObject* Replacement = ReplacementMap.FindRef(Obj.GetEvenIfUnreachable()))
		{
			NewSceneComponents.Add(Replacement);
		}
	}

	if (NewSceneComponents.Num())
	{
		UpdatePropertyHandlesObjects(NewSceneComponents);
	}
}

void FAvaLevelViewportComponentTransformDetails::OnUseSelectionAsParentChanged(ECheckBoxState NewState)
{
	OnUseSelectionAsParentChangedEvent.Broadcast(NewState == ECheckBoxState::Checked);
}

void FAvaLevelViewportComponentTransformDetails::OnUseSelectionAsParentChanged(bool bNewValue)
{
	bUseSelectionAsParent = bNewValue;
	bLastUseSelectionAsParent = bUseSelectionAsParent;

	CachedLocation.X.Reset();
	CachedLocation.Y.Reset();
	CachedLocation.Z.Reset();

	CachedRotation.X.Reset();
	CachedRotation.Y.Reset();
	CachedRotation.Z.Reset();

	CachedScale.X.Reset();
	CachedScale.Y.Reset();
	CachedScale.Z.Reset();

	CacheTransform();

	if (TransformBox.IsValid())
	{
		TransformBox->SetContent(GetTransformBody().ToSharedRef());
	}

	if (RotationBox.IsValid())
	{
		RotationBox->SetContent(GetRotationBody().ToSharedRef());
	}

	if (ScaleBox.IsValid())
	{
		ScaleBox->SetContent(GetScaleBody().ToSharedRef());
	}
}

void FAvaLevelViewportComponentTransformDetails::OnEnginePreExit()
{
	PropertyHandles.Empty();
	TreeNodes.Empty();
	PropertyRowGenerator.Reset();
}

#undef LOCTEXT_NAMESPACE
