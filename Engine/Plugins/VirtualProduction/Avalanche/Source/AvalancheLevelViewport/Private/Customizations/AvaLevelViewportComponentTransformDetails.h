// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetSelection.h"
#include "Framework/Commands/UIAction.h"
#include "IDetailCustomNodeBuilder.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/Input/NumericTypeInterface.h"

class FAvaLevelViewportClient;
class FDetailWidgetRow;
class FMenuBuilder;
class FNotifyHook;
class IDetailCategoryBuilder;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class IDetailTreeNode;
class IPropertyHandle;
class IPropertyRowGenerator;
class SBox;
class SWidget;
class UTypedElementSelectionSet;
struct FSlateBrush;

enum class EAvaLevelViewportTransformFieldType : uint8
{
	Location,
	Rotation,
	Scale,
	All
};

DECLARE_EVENT_OneParam(FAvaLevelViewportComponentTransformDetails, FOnUseSelectionAsParentChanged, bool);

/**
 * Manages the Transform section of a details view
 */
class FAvaLevelViewportComponentTransformDetails : public TSharedFromThis<FAvaLevelViewportComponentTransformDetails>,
	public IDetailCustomNodeBuilder, public TNumericUnitTypeInterface<FVector::FReal>
{
public:
	FAvaLevelViewportComponentTransformDetails(TSharedPtr<FAvaLevelViewportClient> InAvaViewportClient);
	virtual ~FAvaLevelViewportComponentTransformDetails() override;

	static bool IsUseSelectionAsParent() { return bUseSelectionAsParent; }

	/**
	 * Caches the representation of the actor transform for the user input boxes
	 */
	void CacheTransform();

	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override {}
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override {}
	virtual bool RequiresTick() const override { return true; }
	virtual FName GetName() const override { return "Transform"; }
	virtual bool InitiallyCollapsed() const override { return false; }
	virtual void Tick(float DeltaTime) override;
	virtual void SetOnRebuildChildren(FSimpleDelegate OnRebuildChildren) override {}

	void OnSelectionChanged(const TArray<UObject*>& InSelectedObjects);
	void OnSelectionSetChanged(const UTypedElementSelectionSet* InSelectionSet);

	TSharedPtr<SWidget> GetTransformBody();
	TSharedPtr<SWidget> GetRotationBody();
	TSharedPtr<SWidget> GetScaleBody();
	TSharedPtr<SWidget> GetOptionsHeader();
	TSharedPtr<SWidget> GetOptionsBody();
	TSharedRef<SWidget> GetPreserveScaleRatioWidget();

	int32 GetSelectedActorCount() const { return SelectedActorInfo.NumSelected; }

	/** Returns the cached list of selected objects. */
	TConstArrayView<TWeakObjectPtr<UObject>> GetSelectedObjects() const { return SelectedObjects; }

	void SetTransform(EAvaLevelViewportTransformFieldType TransformField, const FVector& NewValue);

	static FOnUseSelectionAsParentChanged OnUseSelectionAsParentChangedEvent;

	/** Called when the "Reset to Default" button for the location has been clicked */
	void OnLocationResetClicked();

	/** Called when the "Reset to Default" button for the rotation has been clicked */
	void OnRotationResetClicked();

	/** Called when the "Reset to Default" button for the scale has been clicked */
	void OnScaleResetClicked();

	TConstArrayView<TSharedPtr<IPropertyHandle>> GetPropertyHandles() const { return PropertyHandles; }

private:

	/** @return Whether the transform details panel should be enabled (editable) or not (read-only / greyed out) */
	bool GetIsEnabled() const;

	/** Sets a vector based on two source vectors and an axis list */
	FVector GetAxisFilteredVector(EAxisList::Type Axis, const FVector& NewValue, const FVector& OldValue);

	/* Called when a change starts (direct ui commit or slider begin) */
	void OnBeginChange(EAvaLevelViewportTransformFieldType TransformField);

	/* Called when a change ends (direct ui commit or slider end) */
	void OnEndChange(EAvaLevelViewportTransformFieldType TransformField, EAxisList::Type Axis);

	/**
	 * Sets the selected object(s) axis to passed in value
	 *
	 * @param TransformField	The field (location/rotation/scale) to modify
	 * @param Axis				Bitfield of which axis to set, can be multiple
	 * @param NewValue			The new vector values, it only uses the ones with specified axis
	 * @param bMirror			If true, set the value to it's inverse instead of using NewValue
	 * @param bCommitted		True if the value was committed, false is the value comes from the slider
	 */
	void OnSetTransform(EAvaLevelViewportTransformFieldType TransformField, EAxisList::Type Axis, FVector NewValue, bool bMirror, bool bCommitted);

	// Transform components based the selection transform, not the component transform.
	void OnSetTransformSelectionParent(EAvaLevelViewportTransformFieldType TransformField, EAxisList::Type Axis, FVector NewValue, bool bMirror, bool bCommitted);

	/**
	 * Sets a single axis value, called from UI
	 *
	 * @param TransformField	The field (location/rotation/scale) to modify
	 * @param NewValue			The new translation value
	 * @param CommitInfo		Whether or not this was committed from pressing enter or losing focus
	 * @param Axis				Bitfield of which axis to set, can be multiple
	 * @param bCommitted		true if the value was committed, false is the value comes from the slider
	 */
	void OnSetTransformAxis(FVector::FReal NewValue, ETextCommit::Type CommitInfo, EAvaLevelViewportTransformFieldType TransformField, EAxisList::Type Axis, bool bCommitted);

	/**
	 * Sets a single axis float value, called from UI
	 *
	 * @param TransformField	The field (location/rotation/scale) to modify
	 * @param NewValue			The new translation value
	 * @param CommitInfo		Whether or not this was committed from pressing enter or losing focus
	 * @param Axis				Bitfield of which axis to set, can be multiple
	 * @param bCommitted		true if the value was committed, false is the value comes from the slider
	 */
	void OnSetTransformAxisFloat(float NewValue, ETextCommit::Type CommitInfo, EAvaLevelViewportTransformFieldType TransformField, EAxisList::Type Axis, bool bCommitted);

	/**
	 * Helper to begin a new transaction for a slider interaction.
	 * @param ActorTransaction		The name to give the transaction when changing an actor transform.
	 * @param ComponentTransaction	The name to give the transaction when directly editing a component transform.
	 */
	void BeginSliderTransaction(FText ActorTransaction, FText ComponentTransaction) const;

	/** Called when the one of the axis sliders for object rotation begins to change for the first time */
	void OnBeginRotationSlider();

	/** Called when the one of the axis sliders for object rotation is released */
	void OnEndRotationSlider(float NewValue);

	/** Called when one of the axis sliders for object location begins to change */
	void OnBeginLocationSlider();

	/** Called when one of the axis sliders for object location is released */
	void OnEndLocationSlider(FVector::FReal NewValue);

	/** Called when one of the axis sliders for object scale begins to change */
	void OnBeginScaleSlider();

	/** Called when one of the axis sliders for object scale is released */
	void OnEndScaleSlider(FVector::FReal NewValue);

	/** @return Icon to use in the preserve scale ratio check box */
	const FSlateBrush* GetPreserveScaleRatioImage() const;

	/** @return The state of the preserve scale ratio checkbox */
	ECheckBoxState IsPreserveScaleRatioChecked() const;

	/**
	 * Called when the preserve scale ratio checkbox is toggled
	 */
	void OnPreserveScaleRatioToggled(ECheckBoxState NewState);

	/**
	 * Builds a transform field label
	 *
	 * @param TransformField The field to build the label for
	 * @return The label widget
	 */
	TSharedRef<SWidget> BuildTransformFieldLabel(EAvaLevelViewportTransformFieldType TransformField);

	/**
	 * Gets display text for a transform field
	 *
	 * @param TransformField	The field to get the text for
	 * @return The text label for TransformField
	 */
	FText GetTransformFieldText(EAvaLevelViewportTransformFieldType TransformField) const;

	/**
	 * @return the text to display for translation
	 */
	FText GetLocationText() const;

	/**
	 * @return the text to display for rotation
	 */
	FText GetRotationText() const;

	/**
	 * @return the text to display for scale
	 */
	FText GetScaleText() const;

	/**
	 * Sets relative transform on the specified field
	 *
	 * @param TransformField the field that should be set to relative
	 * @param bAbsoluteEnabled true to absolute, false for relative 
	 */
	void OnSetAbsoluteTransform(EAvaLevelViewportTransformFieldType TransformField, bool bAbsoluteEnabled);

	/** @return true if Absolute flag of transform type matches passed in bCheckAbsolute*/
	bool IsAbsoluteTransformChecked(EAvaLevelViewportTransformFieldType TransformField, bool bAbsoluteEnabled=true) const;

	/** @return true of copy is enabled for the specified field */
	bool OnCanCopy(EAvaLevelViewportTransformFieldType TransformField) const;

	/**
	 * Copies the specified transform field to the clipboard
	 */
	void OnCopy(EAvaLevelViewportTransformFieldType TransformField);

	/**
	 * Pastes the specified transform field from the clipboard
	 */
	void OnPaste(EAvaLevelViewportTransformFieldType TransformField);

	/**
	 * Creates a UI action for copying a specified transform field
	 */
	FUIAction CreateCopyAction(EAvaLevelViewportTransformFieldType TransformField) const;

	/**
	 * Creates a UI action for pasting a specified transform field
	 */
	FUIAction CreatePasteAction(EAvaLevelViewportTransformFieldType TransformField) const;

	/** Extend the context menu for the X component */
	void ExtendXScaleContextMenu(FMenuBuilder& MenuBuilder);
	/** Extend the context menu for the Y component */
	void ExtendYScaleContextMenu(FMenuBuilder& MenuBuilder);
	/** Extend the context menu for the Z component */
	void ExtendZScaleContextMenu(FMenuBuilder& MenuBuilder);

	/** Called when the X axis scale is mirrored */
	void OnXScaleMirrored();
	/** Called when the Y axis scale is mirrored */
	void OnYScaleMirrored();
	/** Called when the Z axis scale is mirrored */
	void OnZScaleMirrored();

	/** @return The X component of location */
	TOptional<FVector::FReal> GetLocationX() const { return CachedLocation.X; }
	/** @return The Y component of location */
	TOptional<FVector::FReal> GetLocationY() const { return CachedLocation.Y; }
	/** @return The Z component of location */
	TOptional<FVector::FReal> GetLocationZ() const { return CachedLocation.Z; }
	/** @return The visibility of the "Reset to Default" button for the location component */
	bool GetLocationResetVisibility() const;

	/** @return The X component of rotation */
	TOptional<float> GetRotationX() const { return CachedRotation.X; }
	/** @return The Y component of rotation */
	TOptional<float> GetRotationY() const { return CachedRotation.Y; }
	/** @return The Z component of rotation */
	TOptional<float> GetRotationZ() const { return CachedRotation.Z; }
	/** @return The visibility of the "Reset to Default" button for the rotation component */
	bool GetRotationResetVisibility() const;

	/** @return The X component of scale */
	TOptional<FVector::FReal> GetScaleX() const { return CachedScale.X; }
	/** @return The Y component of scale */
	TOptional<FVector::FReal> GetScaleY() const { return CachedScale.Y; }
	/** @return The Z component of scale */
	TOptional<FVector::FReal> GetScaleZ() const { return CachedScale.Z; }
	/** @return The visibility of the "Reset to Default" button for the scale component */
	bool GetScaleResetVisibility() const;

	/** Cache a single unit to display all location components in */
	void CacheCommonLocationUnits();

	/** Generate a property handles from a the scene component properties. */
	void GeneratePropertyHandles();

	/** Update the outer objects of the property handles generated from this transform. */
	void UpdatePropertyHandlesObjects(TArray<UObject*> NewSceneComponents);

	/** Delegate called when the OnObjectsReplaced is broadcast by the engine. */
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);

	ECheckBoxState IsUseSelectionAsParentChecked() const { return bUseSelectionAsParent ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; }
	void OnUseSelectionAsParentChanged(ECheckBoxState NewState);

	void OnUseSelectionAsParentChanged(bool bNewValue);

	void OnEnginePreExit();

private:
	/** A vector where it may optionally be unset */
	template <typename NumericType>
	struct FOptionalVector
	{
		/**
		 * Sets the value from an FVector
		 */
		void Set(const FVector& InVec)
		{
			X = InVec.X;
			Y = InVec.Y;
			Z = InVec.Z;
		}

		/**
		 * Sets the value from an FRotator
		 */
		void Set(const FRotator& InRot)
		{
			X = InRot.Roll;
			Y = InRot.Pitch;
			Z = InRot.Yaw;
		}

		/** @return Whether or not the value is set */
		bool IsSet() const
		{
			// The vector is set if all values are set
			return X.IsSet() && Y.IsSet() && Z.IsSet();
		}

		TOptional<NumericType> X;
		TOptional<NumericType> Y;
		TOptional<NumericType> Z;
	};

	FSelectedActorInfo SelectedActorInfo;

	/** Copy of selected actor references in the details view */
	TArray< TWeakObjectPtr<UObject> > SelectedObjects;

	/** Cache translation value of the selected set */
	FOptionalVector<FVector::FReal> CachedLocation;

	/** Cache rotation value of the selected set */
	FOptionalVector<float> CachedRotation;

	/** Cache scale value of the selected set */
	FOptionalVector<FVector::FReal> CachedScale;

	/** Notify hook to use */
	FNotifyHook* NotifyHook;

	/** Mapping from object to relative rotation values which are not affected by Quat->Rotator conversions during transform calculations */
	TMap< UObject*, FRotator > ObjectToRelativeRotationMap;

	/** Whether or not we are in absolute translation mode */
	bool bAbsoluteLocation;

	/** Whether or not we are in absolute rotation mode */
	bool bAbsoluteRotation;

	/** Whether or not we are in absolute scale mode */
	bool bAbsoluteScale;

	/** Whether or not to preserve scale ratios */
	bool bPreserveScaleRatio;

	/** Scale ratio to use when we are using the sliders with bPreserveScaleRatio set. */
	FVector SliderScaleRatio;

	/** Flag to indicate we are currently editing the rotation in the UI, so we should rely on the cached value in objectToRelativeRotationMap, not the value from the object */
	bool bEditingRotationInUI;

	/** Flag to indicate we are currently performing a slider transaction */
	bool bIsSliderTransaction;

	/** Bitmask to indicate which fields should be hidden (if any) */
	uint8 DisplayedFieldType;

	/** Used to generated property handles. */
	TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;

	/** Tree nodes for the property handles. */
	TArray<TSharedPtr<IDetailTreeNode>> TreeNodes;

	/** Holds this transform's property handles. */
	TArray<TSharedPtr<IPropertyHandle>> PropertyHandles;

	/** Holds the property handles' outer objects. Used to update the handles' objects when the actor construction script runs. */
	TArray<TWeakObjectPtr<UObject>> CachedHandlesObjects;

	// Link to the viewport client for which this is displaying details
	TWeakPtr<FAvaLevelViewportClient> AvaViewportClientWeak;

	// If true, use the multi-select as an ad-hoc parent
	static bool bUseSelectionAsParent;
	bool bLastUseSelectionAsParent;

	// If true, we can use the multi-select as a parent
	bool bCanUseSelectionAsParent;

	TSharedPtr<SBox> TransformBox;
	TSharedPtr<SBox> RotationBox;
	TSharedPtr<SBox> ScaleBox;
	TArray<TWeakObjectPtr<UObject>> ModifiedObjects;
	TMap<USceneComponent*, FVector> OriginalLocations;
	TMap<USceneComponent*, FRotator> OriginalRotations;
	TMap<USceneComponent*, FVector> OriginalScales;

	/** Editor Selection Set used to listen to Selection Changes */
	TWeakObjectPtr<UTypedElementSelectionSet> EditorSelectionSetWeak;
};
