// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaLevelViewport.h"
#include "AvaLevelViewportCommands.h"
#include "AvaViewportDataSubsystem.h"
#include "AvaViewportPostProcessManager.h"
#include "AvaViewportSettings.h"
#include "AvaViewportVirtualSizeEnums.h"
#include "Bounds/AvaBoundsProviderSubsystem.h"
#include "Camera/CameraActor.h"
#include "CineCameraComponent.h"
#include "Editor/TransBuffer.h"
#include "EditorViewportCommands.h"
#include "Engine/Texture.h"
#include "Interaction/AvaCameraZoomController.h"
#include "Math/IntPoint.h"
#include "Math/Vector2D.h"
#include "SAvaLevelViewportFrame.h"
#include "ScopedTransaction.h"
#include "Subsystems/PanelExtensionSubsystem.h"
#include "Types/ISlateMetaData.h"
#include "ViewportClient/AvaLevelViewportClient.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SAvaLevelViewportCameraBounds.h"
#include "Widgets/SAvaLevelViewportGuide.h"
#include "Widgets/SAvaLevelViewportTextureOverlay.h"
#include "Widgets/SAvaLevelViewportPixelGrid.h"
#include "Widgets/SAvaLevelViewportSafeFrames.h"
#include "Widgets/SAvaLevelViewportScreenGrid.h"
#include "Widgets/SAvaLevelViewportSnapIndicators.h"
#include "Widgets/SCanvas.h"

#define LOCTEXT_NAMESPACE "SAvaLevelViewport"

namespace UE::AvaLevelViewport::Private
{
	static const FText GuideTransactionTitle = LOCTEXT("UpdateGuides", "Update Guides");
}

void SAvaLevelViewport::PrivateRegisterAttributes(struct FSlateAttributeDescriptor::FInitializer&)
{
}

SAvaLevelViewport::~SAvaLevelViewport()
{
	GetMutableDefault<UAvaViewportSettings>()->OnChange.RemoveAll(this);
}

void SAvaLevelViewport::Construct(const FArguments& InArgs, const FAssetEditorViewportConstructionArgs& InViewportArgs)
{
	ViewportFrameWeak = InArgs._ViewportFrame;
	VirtualSizeAspectRatio = 0.f;
	VirtualSizeAspectRatioState = EAvaViewportVirtualSizeAspectRatioState::LockedToCamera;

	TSharedPtr<FAvaLevelViewportClient> ViewportClient = InArgs._ViewportFrame->GetViewportClient();

	if (ViewportClient.IsValid())
	{
		ViewportClient->SetViewportWidget(SharedThis(this));
	}

	Super::Construct(Super::FArguments()
			.ParentLevelEditor(InArgs._ParentLevelEditor)
			.LevelEditorViewportClient(ViewportClient)
		, InViewportArgs);

	GetMutableDefault<UAvaViewportSettings>()->OnChange.AddSP(SharedThis(this), &SAvaLevelViewport::OnSettingsChanged);
}

int32 SAvaLevelViewport::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const_cast<SAvaLevelViewport*>(this)->CheckVirtualSizeCameraUpdateSettings();

	return Super::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

void SAvaLevelViewport::BindCommands()
{
	Super::BindCommands();

	const FEditorViewportCommands& EditorViewportCommands = FEditorViewportCommands::Get();
	FUICommandList& CommandListRef = *CommandList;

	FUIAction InvalidAction;
	InvalidAction.IsActionVisibleDelegate.BindLambda([](){ return false; });

	// Hide View Entries for this camera-driven Viewport
	CommandListRef.MapAction(EditorViewportCommands.Perspective, InvalidAction);
	CommandListRef.MapAction(EditorViewportCommands.Front, InvalidAction);
	CommandListRef.MapAction(EditorViewportCommands.Left, InvalidAction);
	CommandListRef.MapAction(EditorViewportCommands.Top, InvalidAction);
	CommandListRef.MapAction(EditorViewportCommands.Back, InvalidAction);
	CommandListRef.MapAction(EditorViewportCommands.Right, InvalidAction);
	CommandListRef.MapAction(EditorViewportCommands.Bottom, InvalidAction);

	const FAvaLevelViewportCommands& AvaLevelViewportCommands = FAvaLevelViewportCommands::GetInternal();

	// Viewport
	CommandListRef.MapAction(
		AvaLevelViewportCommands.ToggleOverlay,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteToggleOverlay),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanToggleOverlay)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.ToggleSafeFrames,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteToggleSafeFrames),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanToggleSafeFrames)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.ToggleChildActorLock,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteToggleChildActorLock),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAvaLevelViewport::IsChildActorLockEnabled)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.TogglePostProcessNone,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteTogglePostProcessType, EAvaViewportPostProcessType::None),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanTogglePostProcessType, EAvaViewportPostProcessType::None),
		FIsActionChecked::CreateSP(this, &SAvaLevelViewport::IsPostProcessTypeEnabled, EAvaViewportPostProcessType::None)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.TogglePostProcessBackground,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteTogglePostProcessType, EAvaViewportPostProcessType::Background),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanTogglePostProcessType, EAvaViewportPostProcessType::Background),
		FIsActionChecked::CreateSP(this, &SAvaLevelViewport::IsPostProcessTypeEnabled, EAvaViewportPostProcessType::Background)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.TogglePostProcessChannelRed,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteTogglePostProcessType, EAvaViewportPostProcessType::RedChannel),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanTogglePostProcessType, EAvaViewportPostProcessType::RedChannel),
		FIsActionChecked::CreateSP(this, &SAvaLevelViewport::IsPostProcessTypeEnabled, EAvaViewportPostProcessType::RedChannel)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.TogglePostProcessChannelGreen,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteTogglePostProcessType, EAvaViewportPostProcessType::GreenChannel),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanTogglePostProcessType, EAvaViewportPostProcessType::GreenChannel),
		FIsActionChecked::CreateSP(this, &SAvaLevelViewport::IsPostProcessTypeEnabled, EAvaViewportPostProcessType::GreenChannel)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.TogglePostProcessChannelBlue,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteTogglePostProcessType, EAvaViewportPostProcessType::BlueChannel),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanTogglePostProcessType, EAvaViewportPostProcessType::BlueChannel),
		FIsActionChecked::CreateSP(this, &SAvaLevelViewport::IsPostProcessTypeEnabled, EAvaViewportPostProcessType::BlueChannel)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.TogglePostProcessChannelAlpha,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteTogglePostProcessType, EAvaViewportPostProcessType::AlphaChannel),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanTogglePostProcessType, EAvaViewportPostProcessType::AlphaChannel),
		FIsActionChecked::CreateSP(this, &SAvaLevelViewport::IsPostProcessTypeEnabled, EAvaViewportPostProcessType::AlphaChannel)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.TogglePostProcessCheckerboard,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteTogglePostProcessType, EAvaViewportPostProcessType::Checkerboard),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanTogglePostProcessType, EAvaViewportPostProcessType::Checkerboard),
		FIsActionChecked::CreateSP(this, &SAvaLevelViewport::IsPostProcessTypeEnabled, EAvaViewportPostProcessType::Checkerboard)
	);
	CommandListRef.MapAction(
		AvaLevelViewportCommands.ToggleTextureOverlay,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteToggleTextureOverlay),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanToggleTextureOverlay)
	);

	// Grid
	CommandListRef.MapAction(
		AvaLevelViewportCommands.ToggleGrid,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteToggleGrid),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanToggleGrid)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.ToggleGridAlwaysVisible,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteToggleGridAlwaysVisible),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanToggleGridAlwaysVisible),
		FIsActionChecked::CreateSP(this, &SAvaLevelViewport::IsGridAlwaysVisible)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.IncreaseGridSize,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteIncreaseGridSize),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanIncreaseGridSize)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.DecreaseGridSize,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteDecreaseGridSize),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanDecreaseGridSize)
	);

	// Snapping
	CommandListRef.MapAction(
		AvaLevelViewportCommands.ToggleSnapping,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteToggleSnapping),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanToggleSnapping)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.ToggleGridSnapping,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteToggleGridSnapping),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanToggleGridSnapping),
		FIsActionChecked::CreateSP(this, &SAvaLevelViewport::IsGridSnappingEnabled)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.ToggleScreenSnapping,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteToggleScreenSnapping),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanToggleScreenSnapping),
		FIsActionChecked::CreateSP(this, &SAvaLevelViewport::IsScreenSnappingEnabled)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.ToggleActorSnapping,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteToggleActorSnapping),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanToggleActorSnapping),
		FIsActionChecked::CreateSP(this, &SAvaLevelViewport::IsActorSnappingEnabled)
	);

	// Guides
	CommandListRef.MapAction(
		AvaLevelViewportCommands.ToggleGuides,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteToggleGuides),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanToggleGuides)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.AddGuideHorizontal,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteAddHorizontalGuide),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanAddHorizontalGuide)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.AddGuideVertical,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteAddVerticalGuide),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanAddVerticalGuide)
	);

	// Virtual sizes
	CommandListRef.MapAction(
		AvaLevelViewportCommands.VirtualSizeDisable,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::SetVirtualSize, FIntPoint::ZeroValue),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAvaLevelViewport::IsVirtualSizeActive, FIntPoint::ZeroValue)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.VirtualSize1920x1080,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::SetVirtualSize, FIntPoint(1920, 1080)),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAvaLevelViewport::IsVirtualSizeActive, FIntPoint(1920, 1080))
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.VirtualSizeAspectRatioUnlocked,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::SetVirtualSizeAspectRatioState, EAvaViewportVirtualSizeAspectRatioState::Unlocked),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAvaLevelViewport::IsUsingVirtualSizeAspectRatioState, EAvaViewportVirtualSizeAspectRatioState::Unlocked)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.VirtualSizeAspectRatioLocked,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::SetVirtualSizeAspectRatioState, EAvaViewportVirtualSizeAspectRatioState::Locked),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAvaLevelViewport::IsUsingVirtualSizeAspectRatioState, EAvaViewportVirtualSizeAspectRatioState::Locked)
	);

	CommandListRef.MapAction(
		AvaLevelViewportCommands.VirtualSizeAspectRatioLockedToCamera,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::SetVirtualSizeAspectRatioState, EAvaViewportVirtualSizeAspectRatioState::LockedToCamera),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAvaLevelViewport::IsUsingVirtualSizeAspectRatioState, EAvaViewportVirtualSizeAspectRatioState::LockedToCamera)
	);

	// Camera controls
	CommandListRef.MapAction(
		AvaLevelViewportCommands.CameraTransformReset,
		FExecuteAction::CreateSP(this, &SAvaLevelViewport::ResetPilotedCameraTransform),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanResetPilotedCameraTransform)
	);

	if (TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient())
	{
		if (TSharedPtr<FAvaCameraZoomController> ZoomController = ViewportClient->GetZoomController())
		{
			TSharedRef<FAvaCameraZoomController> ZoomControllerRef = ZoomController.ToSharedRef();

			CommandListRef.MapAction(
				AvaLevelViewportCommands.CameraZoomInCenter,
				FExecuteAction::CreateSP(ZoomControllerRef, &FAvaCameraZoomController::ZoomIn)
			);

			CommandListRef.MapAction(
				AvaLevelViewportCommands.CameraZoomOutCenter,
				FExecuteAction::CreateSP(ZoomControllerRef, &FAvaCameraZoomController::ZoomOut)
			);

			CommandListRef.MapAction(
				AvaLevelViewportCommands.CameraPanLeft,
				FExecuteAction::CreateSP(ZoomControllerRef, &FAvaCameraZoomController::PanLeft)
			);

			CommandListRef.MapAction(
				AvaLevelViewportCommands.CameraPanRight,
				FExecuteAction::CreateSP(ZoomControllerRef, &FAvaCameraZoomController::PanRight)
			);

			CommandListRef.MapAction(
				AvaLevelViewportCommands.CameraPanUp,
				FExecuteAction::CreateSP(ZoomControllerRef, &FAvaCameraZoomController::PanUp)
			);

			CommandListRef.MapAction(
				AvaLevelViewportCommands.CameraPanDown,
				FExecuteAction::CreateSP(ZoomControllerRef, &FAvaCameraZoomController::PanDown)
			);

			CommandListRef.MapAction(
				AvaLevelViewportCommands.CameraFrameActor,
				FExecuteAction::CreateSP(ZoomControllerRef, &FAvaCameraZoomController::FrameActor)
			);

			CommandListRef.MapAction(
				AvaLevelViewportCommands.CameraZoomReset,
				FExecuteAction::CreateSP(ZoomControllerRef, &FAvaCameraZoomController::Reset)
			);
		}
	}
}

void SAvaLevelViewport::PopulateViewportOverlays(TSharedRef<SOverlay> Overlay)
{
	Super::PopulateViewportOverlays(Overlay);

	if (TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient())
	{
		// Add in reverse order so they replace each other as the first widget.

		// -1 / INDEX_NONE has special treatment.
		Overlay->AddSlot(-2)
		[
			SAssignNew(GuideCanvas, SCanvas)
		];

		Overlay->AddSlot(-2)
		[
			SAssignNew(SnapIndicators, SAvaLevelViewportSnapIndicators, ViewportClient.ToSharedRef())
		];

		Overlay->AddSlot(-2)
		[
			SAssignNew(SafeFrames, SAvaLevelViewportSafeFrames, ViewportClient.ToSharedRef())
		];

		Overlay->AddSlot(-2)
		[
			SAssignNew(ScreenGrid, SAvaLevelViewportScreenGrid, ViewportClient.ToSharedRef())
		];

		Overlay->AddSlot(-2)
		[
			SAssignNew(PixelGrid, SAvaLevelViewportPixelGrid, ViewportClient.ToSharedRef())
		];

		Overlay->AddSlot(-2)
		[
			SAssignNew(TextureOverlay, SAvaLevelViewportTextureOverlay, ViewportClient.ToSharedRef())
		];

		Overlay->AddSlot(-2)
		[
			SAssignNew(CameraBounds, SAvaLevelViewportCameraBounds, ViewportClient.ToSharedRef())
		];
	}

	ApplySettings(GetDefault<UAvaViewportSettings>());
	LoadGuides();
	LoadVirtualSizeSettings();
}

TSharedPtr<SAvaLevelViewportFrame> SAvaLevelViewport::GetAvaLevelViewportFrame() const
{
	return ViewportFrameWeak.Pin();
}

TSharedPtr<FAvaLevelViewportClient> SAvaLevelViewport::GetAvaLevelViewportClient() const
{
	if (TSharedPtr<SAvaLevelViewportFrame> ViewportFrame = ViewportFrameWeak.Pin())
	{
		return ViewportFrame->GetViewportClient();
	}

	return nullptr;
}

TSharedPtr<SAvaLevelViewportGuide> SAvaLevelViewport::AddGuide(EOrientation InOrientation, float InOffsetFraction, bool bInSaveGuides)
{
	if (!GuideCanvas.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<SAvaLevelViewportFrame> ViewportFrame = ViewportFrameWeak.Pin();

	if (!ViewportFrame.IsValid())
	{
		return nullptr;
	}

	TSharedPtr<SAvaLevelViewportGuide> NewGuide = SNew(SAvaLevelViewportGuide, ViewportFrame, Guides.Num())
		.InitialState(EAvaViewportGuideState::Enabled)
		.OffsetFraction(InOffsetFraction)
		.Orientation(InOrientation);

	NewGuide->SetVisibility(TAttribute<EVisibility>::CreateSP(NewGuide.Get(), &SAvaLevelViewportGuide::GetGuideVisibility));

	Guides.Add(NewGuide);

	if (bInSaveGuides)
	{
		SaveGuides();
	}

	GuideCanvas->AddSlot()
		.Position(NewGuide.Get(), &SAvaLevelViewportGuide::GetPosition)
		.Size(NewGuide.Get(), &SAvaLevelViewportGuide::GetSize)
		[
			NewGuide.ToSharedRef()
		];

	return NewGuide;
}

bool SAvaLevelViewport::RemoveGuide(TSharedPtr<SAvaLevelViewportGuide> InGuidetoRemove)
{
	if (!GuideCanvas.IsValid())
	{
		return false;
	}

	int32 GuideIndex = Guides.Find(InGuidetoRemove);

	if (GuideIndex == INDEX_NONE)
	{
		return false;
	}

	Guides.RemoveAt(GuideIndex);

	for (int ShiftIndex = GuideIndex; ShiftIndex < Guides.Num(); ++ShiftIndex)
	{
		Guides[ShiftIndex]->SetIndex(ShiftIndex);
	}

	if (FChildren* Children = GuideCanvas->GetChildren())
	{
		if (GuideIndex < Children->Num())
		{
			TSharedRef<SWidget> FoundWidget = Children->GetChildAt(GuideIndex);
			check(FoundWidget == InGuidetoRemove); // Just ensure we're keeping track of indices properly.
			GuideCanvas->RemoveSlot(FoundWidget);
		}
	}

	SaveGuides();

	return true;
}

void SAvaLevelViewport::LoadGuides()
{
	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return;
	}

	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(ViewportClient->GetViewportWorld());

	if (!DataSubsystem)
	{
		return;
	}

	FAvaViewportData* Data = DataSubsystem->GetData();

	if (!Data)
	{
		return;
	}

	constexpr bool bSaveGuides = false;

	for (const FAvaViewportGuideInfo& GuideInfo : Data->GuideData)
	{
		TSharedPtr<SAvaLevelViewportGuide> NewGuide = AddGuide(GuideInfo.Orientation, GuideInfo.OffsetFraction, bSaveGuides);

		if (NewGuide.IsValid())
		{
			NewGuide->SetState(GuideInfo.State, bSaveGuides);
		}
	}
}

void SAvaLevelViewport::ReloadGuides()
{
	GuideCanvas->ClearChildren();
	Guides.Empty();
	LoadGuides();
}

void SAvaLevelViewport::LoadPostProcessInfo()
{
	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return;
	}

	if (!ViewportClient->GetPostProcessManager().IsValid())
	{
		return;
	}

	ViewportClient->GetPostProcessManager()->LoadPostProcessInfo();
}

void SAvaLevelViewport::SaveGuides()
{
	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return;
	}

	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(ViewportClient->GetViewportWorld());

	if (!DataSubsystem)
	{
		return;
	}

	FAvaViewportData* Data = DataSubsystem->GetData();

	if (!Data)
	{
		return;
	}

	FScopedTransaction SaveTransaction(UE::AvaLevelViewport::Private::GuideTransactionTitle, !GIsTransacting);
	DataSubsystem->ModifyDataSource();

	TArray<FAvaViewportGuideInfo> GuideInfos;
	GuideInfos.Reserve(Guides.Num());

	for (const TSharedPtr<SAvaLevelViewportGuide>& Guide : Guides)
	{
		if (Guide.IsValid())
		{
			GuideInfos.Add(Guide->GetGuideInfo());
		}
	}

	Data->GuideData = GuideInfos;
}

void SAvaLevelViewport::PostUndo(bool bInSuccess)
{
	ReloadGuides();
	LoadVirtualSizeSettings();
}

void SAvaLevelViewport::PostRedo(bool bInSuccess)
{
	ReloadGuides();
	LoadVirtualSizeSettings();
}

void SAvaLevelViewport::ActivateCamera(TWeakObjectPtr<ACameraActor> InCamera)
{
	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return;
	}

	ViewportClient->SetViewTarget(InCamera);

	const FAvaLevelViewportCommands& AvaLevelViewportCommands = FAvaLevelViewportCommands::GetInternal();
	ViewportClient->GetZoomController()->Reset();

	if (VirtualSizeAspectRatioState == EAvaViewportVirtualSizeAspectRatioState::LockedToCamera)
	{
		UpdateVirtualSizeSettings();
	}
}

bool SAvaLevelViewport::IsCameraActive(TWeakObjectPtr<ACameraActor> InCamera) const
{
	if (TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient())
	{
		return ViewportClient->GetViewTarget() == InCamera;
	}

	return false;
}

void SAvaLevelViewport::OnSettingsChanged(const UAvaViewportSettings* InSettings, FName InPropertyChanged)
{
	ApplySettings(InSettings);
}

void SAvaLevelViewport::ApplySettings(const UAvaViewportSettings* InSettings)
{
	if (!IsValid(InSettings))
	{
		return;
	}

	if (PixelGrid.IsValid())
	{
		PixelGrid->SetVisibility(InSettings->bEnableViewportOverlay && InSettings->bPixelGridEnabled ? EVisibility::HitTestInvisible : EVisibility::Collapsed);
	}

	if (ScreenGrid.IsValid())
	{
		ScreenGrid->SetVisibility(InSettings->bEnableViewportOverlay && InSettings->bGridEnabled ? EVisibility::HitTestInvisible : EVisibility::Collapsed);
	}

	if (SafeFrames.IsValid())
	{
		SafeFrames->SetVisibility(InSettings->bEnableViewportOverlay && InSettings->bSafeFramesEnabled ? EVisibility::HitTestInvisible : EVisibility::Collapsed);
	}

	if (GuideCanvas.IsValid())
	{
		GuideCanvas->SetVisibility(InSettings->bEnableViewportOverlay && InSettings->bGuidesEnabled ? EVisibility::Visible : EVisibility::Collapsed);
	}

	if (SnapIndicators.IsValid())
	{
		SnapIndicators->SetVisibility(InSettings->bEnableViewportOverlay && InSettings->bSnapIndicatorsEnabled ? EVisibility::HitTestInvisible : EVisibility::Collapsed);
	}

	if (TextureOverlay.IsValid())
	{
		TextureOverlay->SetVisibility(InSettings->bEnableTextureOverlay ? EVisibility::HitTestInvisible : EVisibility::Collapsed);
	}

	if (CameraBounds.IsValid())
	{
		CameraBounds->SetVisibility(InSettings->bEnableViewportOverlay ? EVisibility::HitTestInvisible : EVisibility::Collapsed);
	}
}

void SAvaLevelViewport::LoadVirtualSizeSettings()
{
	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return;
	}

	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(ViewportClient->GetViewportWorld());

	if (!DataSubsystem)
	{
		return;
	}

	FAvaViewportData* Data = DataSubsystem->GetData();

	if (!Data)
	{
		return;
	}

	const FIntPoint ProviderVirtualSize = Data->VirtualSize;

	VirtualSizeAspectRatioState = Data->VirtualSizeAspectRatioState;

	if (ProviderVirtualSize.X > 0 && ProviderVirtualSize.Y > 0)
	{
		VirtualSizeAspectRatio = static_cast<float>(ProviderVirtualSize.X) / static_cast<float>(ProviderVirtualSize.Y);
		SetVirtualSize(ProviderVirtualSize);
	}
}

FIntPoint SAvaLevelViewport::GetVirtualSize() const
{
	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return FIntPoint::ZeroValue;
	}

	return ViewportClient->GetVirtualViewportSize();
}

bool SAvaLevelViewport::IsVirtualSizeActive(FIntPoint InVirtualSize) const
{
	const FIntPoint CurrentSize = GetVirtualSize();

	if ((InVirtualSize.X <= 0 || InVirtualSize.Y <= 0)
		&& (CurrentSize.X <= 0 || CurrentSize.Y <= 0))
	{
		return true;
	}

	return InVirtualSize == CurrentSize;
}

void SAvaLevelViewport::SetVirtualSize(FIntPoint InVirtualSize)
{
	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return;
	}

	switch (VirtualSizeAspectRatioState)
	{
		default:
		case EAvaViewportVirtualSizeAspectRatioState::Unlocked:
			// Do nothing
			break;

		case EAvaViewportVirtualSizeAspectRatioState::Locked:
		{
			const float AspectRatio = GetVirtualSizeAspectRatio();

			if (AspectRatio <= 0)
			{
				const FIntPoint VirtualSize = GetVirtualSize();

				if (VirtualSize.X > 0 && VirtualSize.Y > 0)
				{
					VirtualSizeAspectRatio = static_cast<float>(VirtualSize.X) / static_cast<float>(VirtualSize.Y);
				}
				else
				{
					VirtualSizeAspectRatio = 1.f;
				}
			}
		}
			// Falls through

		case EAvaViewportVirtualSizeAspectRatioState::LockedToCamera:
			ApplyVirtualSizeSettings(InVirtualSize);
			break;
	}

	ViewportClient->SetVirtualViewportSize(InVirtualSize);

	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(ViewportClient->GetViewportWorld());

	if (!DataSubsystem)
	{
		return;
	}

	FAvaViewportData* Data = DataSubsystem->GetData();

	if (!Data)
	{
		return;
	}

	if (Data->VirtualSize == InVirtualSize)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("SetVirtualSize", "Set Virtual Size"));
	DataSubsystem->ModifyDataSource();
	Data->VirtualSize = InVirtualSize;
}

float SAvaLevelViewport::GetVirtualSizeAspectRatio() const
{
	switch (VirtualSizeAspectRatioState)
	{
		case EAvaViewportVirtualSizeAspectRatioState::Unlocked:
		{
			const FIntPoint VirtualSize = GetVirtualSize();

			if (VirtualSize.X > 0 && VirtualSize.Y > 0)
			{
				return static_cast<float>(VirtualSize.X) / static_cast<float>(VirtualSize.Y);
			}
			break;
		}

		case EAvaViewportVirtualSizeAspectRatioState::Locked:
			if (!FMath::IsNearlyZero(VirtualSizeAspectRatio) && VirtualSizeAspectRatio > 0)
			{
				return VirtualSizeAspectRatio;
			}
			break;

		case EAvaViewportVirtualSizeAspectRatioState::LockedToCamera:
			if (TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient())
			{
				if (UCameraComponent* CameraComponent = ViewportClient->GetCameraComponentViewTarget())
				{
					if (UCineCameraComponent* CineComponent = Cast<UCineCameraComponent>(CameraComponent))
					{
						return CineComponent->Filmback.SensorAspectRatio;
					}
					else
					{
						return CameraComponent->AspectRatio;
					}
				}
			}
			break;

		default:
			// Do nothing
			break;
	}

	return 0.f;
}

bool SAvaLevelViewport::IsUsingVirtualSizeAspectRatioState(EAvaViewportVirtualSizeAspectRatioState InAspectRatioState) const
{
	return VirtualSizeAspectRatioState == InAspectRatioState;
}

void SAvaLevelViewport::SetVirtualSizeAspectRatioState(EAvaViewportVirtualSizeAspectRatioState InAspectRatioState)
{
	if (VirtualSizeAspectRatioState == InAspectRatioState)
	{
		return;
	}

	VirtualSizeAspectRatioState = InAspectRatioState;

	switch (InAspectRatioState)
	{
		default:
		case EAvaViewportVirtualSizeAspectRatioState::Unlocked:
			// Do nothing
			break;

		case EAvaViewportVirtualSizeAspectRatioState::Locked:
		{
			const FIntPoint VirtualSize = GetVirtualSize();

			// Store our aspect ratio for later
			if (VirtualSize.X > 0 && VirtualSize.Y > 0)
			{
				VirtualSizeAspectRatio = static_cast<float>(VirtualSize.X) / static_cast<float>(VirtualSize.Y);
			}
			else
			{
				VirtualSizeAspectRatio = 0.f;
			}
			break;
		}

		case EAvaViewportVirtualSizeAspectRatioState::LockedToCamera:
		{
			// Apply the settings from the camera
			UpdateVirtualSizeSettings();
			break;
		}

	}

	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return;
	}

	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(ViewportClient->GetViewportWorld());

	if (!DataSubsystem)
	{
		return;
	}

	FAvaViewportData* Data = DataSubsystem->GetData();

	if (!Data)
	{
		return;
	}

	if (Data->VirtualSizeAspectRatioState == InAspectRatioState)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("SetVirtualSizeAspectRatioState", "Set Custom Virtual Size Aspect Ratio State"));
	DataSubsystem->ModifyDataSource();
	Data->VirtualSizeAspectRatioState = InAspectRatioState;
}

ECheckBoxState SAvaLevelViewport::GetTextureOverlayStretchEnabledCheckBoxState() const
{
	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		return AvaViewportSettings->bTextureOverlayStretch
			? ECheckBoxState::Checked
			: ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Undetermined;
}

void SAvaLevelViewport::OnTextureOverlayStretchEnabledCheckBoxChanged(ECheckBoxState InState)
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->bTextureOverlayStretch = (InState == ECheckBoxState::Checked);
		AvaViewportSettings->BroadcastSettingChanged(GET_MEMBER_NAME_CHECKED(UAvaViewportSettings, bTextureOverlayStretch));
		AvaViewportSettings->SaveConfig();
	}
}

int32 SAvaLevelViewport::GetVirtualSizeX() const
{
	return GetVirtualSize().X;
}

int32 SAvaLevelViewport::GetVirtualSizeY() const
{
	return GetVirtualSize().Y;
}

bool SAvaLevelViewport::ApplyVirtualSizeSettings(FIntPoint& InOutVirtualSize)
{
	const float AspectRatio = GetVirtualSizeAspectRatio();

	if (AspectRatio <= 0)
	{
		return false;
	}

	const int32 NewVirtualSizeY = FMath::Clamp(FMath::RoundToInt(static_cast<float>(InOutVirtualSize.X) / AspectRatio), 1, 1000000);

	if (InOutVirtualSize.Y != NewVirtualSizeY)
	{
		InOutVirtualSize.Y = NewVirtualSizeY;
		return true;
	}

	return false;
}

void SAvaLevelViewport::UpdateVirtualSizeSettings()
{
	FIntPoint VirtualSize = GetVirtualSize();

	if (ApplyVirtualSizeSettings(VirtualSize))
	{
		SetVirtualSize(VirtualSize);
	}
}

void SAvaLevelViewport::CheckVirtualSizeCameraUpdateSettings()
{
	if (VirtualSizeAspectRatioState != EAvaViewportVirtualSizeAspectRatioState::LockedToCamera)
	{
		return;
	}

	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return;
	}

	UCameraComponent* CameraComponent = ViewportClient->GetCameraComponentViewTarget();

	if (!CameraComponent)
	{
		return;
	}

	if (UCineCameraComponent* CineComponent = Cast<UCineCameraComponent>(CameraComponent))
	{
		VirtualSizeAspectRatio = CineComponent->Filmback.SensorAspectRatio;
	}
	else
	{
		VirtualSizeAspectRatio = CameraComponent->AspectRatio;
	}

	UpdateVirtualSizeSettings();
}

bool SAvaLevelViewport::HasGuides() const
{
	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(this);

	if (!DataSubsystem)
	{
		return false;
	}

	FAvaViewportData* Data = DataSubsystem->GetData();

	return Data && Data->GuideData.Num() > 0;
}

#undef LOCTEXT_NAMESPACE
