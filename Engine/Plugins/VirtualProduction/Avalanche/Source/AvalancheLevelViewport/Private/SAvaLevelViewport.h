// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SLevelViewport.h"
#include "Containers/Array.h"
#include "EditorUndoClient.h"
#include "Math/MathFwd.h"
#include "Subsystems/PanelExtensionSubsystem.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrFwd.h"
#include "Widgets/SAvaLevelViewportTextureOverlay.h"

class ACameraActor;
class FAvaLevelViewportClient;
class FName;
class FScopedTransaction;
class ILevelEditor;
class SAvaLevelViewportCameraBounds;
class SAvaLevelViewportFrame;
class SAvaLevelViewportGuide;
class SAvaLevelViewportPixelGrid;
class SAvaLevelViewportSafeFrames;
class SAvaLevelViewportScreenGrid;
class SAvaLevelViewportSnapIndicators;
class SAvaLevelViewportTextureOverlay;
class SCanvas;
class SOverlay;
class SWidget;
class UAvaViewportSettings;
class UToolMenu;
enum class EAvaViewportPostProcessType : uint8;
enum class EAvaViewportVirtualSizeAspectRatioState : uint8;
enum class ECheckBoxState : uint8;
struct FAssetEditorViewportConstructionArgs;
struct FSlateIcon;
struct FToolMenuSection;

class SAvaLevelViewport : public SLevelViewport, public FSelfRegisteringEditorUndoClient
{
	friend class SAvaLevelViewportStatusBarButtons;

public:
	SLATE_DECLARE_WIDGET(SAvaLevelViewport, SLevelViewport)

	SLATE_BEGIN_ARGS(SAvaLevelViewport) {}
		SLATE_ARGUMENT(TWeakPtr<ILevelEditor>, ParentLevelEditor)
		SLATE_ARGUMENT(TSharedPtr<SAvaLevelViewportFrame>, ViewportFrame)
	SLATE_END_ARGS()

	virtual ~SAvaLevelViewport() override;

	void Construct(const FArguments& InArgs, const FAssetEditorViewportConstructionArgs& InViewportArgs);

	//~ Begin SWidget
	/** Tick is private in SLevelViewport ?! */
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
		FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	//~ End SWidget

	//~ Begin SEditorViewport
	virtual void BindCommands() override;
	virtual FName RegisterViewportToolbar() const override;
	virtual void PopulateViewportOverlays(TSharedRef<SOverlay> Overlay) override;
	//~ End SEditorViewport

	TSharedPtr<SAvaLevelViewportFrame> GetAvaLevelViewportFrame() const;
	TSharedPtr<FAvaLevelViewportClient> GetAvaLevelViewportClient() const;

	TSharedPtr<SAvaLevelViewportPixelGrid> GetPixelGrid() const { return PixelGrid; }
	TSharedPtr<SAvaLevelViewportScreenGrid> GetScreenGrid() const { return ScreenGrid; }
	TSharedPtr<SAvaLevelViewportSnapIndicators> GetSnapIndicators() const { return SnapIndicators; }

	TConstArrayView<TSharedPtr<SAvaLevelViewportGuide>> GetGuides() const { return Guides; }
	TSharedPtr<SAvaLevelViewportGuide> AddGuide(EOrientation InOrientation, float InOffsetFraction, bool bInSaveGuides = true); /** 0 = left/top. 1 = right/bottom/ */
	bool RemoveGuide(TSharedPtr<SAvaLevelViewportGuide> InGuidetoRemove);
	void SaveGuides();

	bool CanToggleOverlay() const;
	void ExecuteToggleOverlay();

	bool CanToggleSafeFrames() const;
	void ExecuteToggleSafeFrames();

	bool CanToggleBoundingBox() const;
	void ExecuteToggleBoundingBox();

	bool CanToggleGrid() const;
	void ExecuteToggleGrid();

	bool CanToggleGridAlwaysVisible() const;
	bool IsGridAlwaysVisible() const;
	void ExecuteToggleGridAlwaysVisible();

	bool CanIncreaseGridSize() const;
	void ExecuteIncreaseGridSize();

	bool CanDecreaseGridSize() const;
	void ExecuteDecreaseGridSize();

	bool CanChangeGridSize() const;
	void ExecuteSetGridSize(int32 InNewSize, bool bInCommit);

	bool CanToggleSnapping() const;
	void ExecuteToggleSnapping();

	bool CanToggleGridSnapping() const;
	bool IsGridSnappingEnabled() const;
	void ExecuteToggleGridSnapping();

	bool CanToggleScreenSnapping() const;
	bool IsScreenSnappingEnabled() const;
	void ExecuteToggleScreenSnapping();

	bool CanToggleActorSnapping() const;
	bool IsActorSnappingEnabled() const;
	void ExecuteToggleActorSnapping();

	bool CanToggleGuides() const;
	void ExecuteToggleGuides();

	bool CanAddHorizontalGuide() const;
	void ExecuteAddHorizontalGuide();

	bool CanAddVerticalGuide() const;
	void ExecuteAddVerticalGuide();

	void ExecuteToggleChildActorLock();

	bool IsPostProcessTypeEnabled(EAvaViewportPostProcessType InPostProcessType) const;
	bool CanTogglePostProcessType(EAvaViewportPostProcessType InPostProcessType) const;
	void ExecuteTogglePostProcessType(EAvaViewportPostProcessType InPostProcessType);

	bool CanToggleTextureOverlay() const;
	void ExecuteToggleTextureOverlay();
	
	FIntPoint GetVirtualSize() const;
	bool IsVirtualSizeActive(FIntPoint InVirtualSize) const;
	void SetVirtualSize(FIntPoint InVirtualSize);
	float GetVirtualSizeAspectRatio() const;

	bool IsUsingVirtualSizeAspectRatioState(EAvaViewportVirtualSizeAspectRatioState InAspectRatioState) const;
	void SetVirtualSizeAspectRatioState(EAvaViewportVirtualSizeAspectRatioState InAspectRatioState);

	void LoadGuides();
	void ReloadGuides();

	void LoadPostProcessInfo();

	void ActivateCamera(TWeakObjectPtr<ACameraActor> InCamera);
	bool IsCameraActive(TWeakObjectPtr<ACameraActor> InCamera) const;

	bool CanResetPilotedCameraTransform() const;
	void ResetPilotedCameraTransform();

	void FillCameraMenu(UToolMenu* InMenu, bool bInIncludeCamerasAndViewports);

	//~ Begin FSelfRegisteringEditorUndoClient
	virtual void PostUndo(bool bInSuccess);
	virtual void PostRedo(bool bInSuccess);
	//~ End FSelfRegisteringEditorUndoClient

private:
	TWeakPtr<SAvaLevelViewportFrame> ViewportFrameWeak;
	TSharedPtr<SAvaLevelViewportCameraBounds> CameraBounds;
	TSharedPtr<SAvaLevelViewportTextureOverlay> TextureOverlay;
	TSharedPtr<SAvaLevelViewportPixelGrid> PixelGrid;
	TSharedPtr<SAvaLevelViewportScreenGrid> ScreenGrid;
	TSharedPtr<SAvaLevelViewportSafeFrames> SafeFrames;
	TSharedPtr<SCanvas> GuideCanvas;
	TArray<TSharedPtr<SAvaLevelViewportGuide>> Guides;
	TSharedPtr<SAvaLevelViewportSnapIndicators> SnapIndicators;
	TSharedPtr<SWidget> BackgroundTextureSelector;
	TSharedPtr<SWidget> PostProcessOpacitySlider;
	TSharedPtr<FScopedTransaction> PostProcessInfoTransaction;

	float VirtualSizeAspectRatio;
	EAvaViewportVirtualSizeAspectRatioState VirtualSizeAspectRatioState;
	TSharedPtr<FScopedTransaction> VirtualSizeComponentSliderTransaction;

	FSlateIcon GetCameraIcon() const;

	TSharedRef<SWidget> GenerateCameraMenu();
	bool CanChangeCameraInCameraMenu() const;

	void AddCameraEntries(UToolMenu* InMenu, TArray<TWeakObjectPtr<ACameraActor>> InCameraActors);
	void AddCameraEntries(FToolMenuSection& InSection, TArray<TWeakObjectPtr<ACameraActor>> InCameraActors);

	void AddViewportEntries(UToolMenu* InMenu);
	void AddViewportEntries(FToolMenuSection& InSection);

	void AddVirtualSizeMenuEntries(UToolMenu* InMenu);
	void AddVirtualSizeDefaultEntries(FToolMenuSection& InSection);
	void AddVirtualSizeSizeSettings(FToolMenuSection& InSection);

	void AddGuidePresetMenuEntries(UToolMenu* InMenu);
	void AddGuidePresetCurrentMenu(UToolMenu* InMenu);
	void AddGuidePresetSavedMenu(UToolMenu* InMenu);

	void AddCameraZoomMenuEntries(UToolMenu* InMenu);

	static void AddVisualizerEntries(UToolMenu* InMenu);

	void OnSettingsChanged(const UAvaViewportSettings* InSettings, FName InPropertyChanged);
	void ApplySettings(const UAvaViewportSettings* InSettings);

	void LoadVirtualSizeSettings();

	bool IsChildActorLockEnabled() const;

	void BeginPostProcessInfoTransaction();
	void EndPostProcessInfoTransaction();

	FString GetBackgroundTextureObjectPath() const;
	void OnBackgroundTextureChanged(const FAssetData& InAssetData);

	float GetBackgroundOpacity() const;
	void OnBackgroundOpacitySliderBegin();
	void OnBackgroundOpacitySliderEnd(float InValue);
	void OnBackgroundOpacityChanged(float InValue);
	void OnBackgroundOpacityCommitted(float InValue, ETextCommit::Type InCommitType);

	FString GetTextureOverlayTextureObjectPath() const;
	void OnTextureOverlayTextureChanged(const FAssetData& InAssetData);

	float GetTextureOverlayOpacity() const;
	void OnTextureOverlayOpacitySliderEnd(float InValue);
	void OnTextureOverlayOpacityChanged(float InValue);
	void OnTextureOverlayOpacityCommitted(float InValue, ETextCommit::Type InCommitType);

	ECheckBoxState GetTextureOverlayStretchEnabledCheckBoxState() const;
	void OnTextureOverlayStretchEnabledCheckBoxChanged(ECheckBoxState InState);

	int32 GetVirtualSizeX() const;
	int32 GetVirtualSizeY() const;
	void OnVirtualSizeComponentSliderCommitted(int32 InNewDimension, ETextCommit::Type InCommitType, EAxis::Type InAxis);
	void OnVirtualSizeAspectRatioCommitted(float InNewAspectRatio, ETextCommit::Type InCommitType);
	void OnVirtualSizeSliderBegin();
	void OnVirtualSizeComponentSliderEnd(int32 InNewSize);
	void OnVirtualSizeAspectRatioSliderEnd(float InNewAspectRatio);
	bool IsVirtualSizeAspectRatioEnabled() const;

	bool ApplyVirtualSizeSettings(FIntPoint& InOutVirtualSize);
	void UpdateVirtualSizeSettings();

	void CheckVirtualSizeCameraUpdateSettings();

	bool HasGuides() const;

	bool CanSaveAsGuidePreset() const;

	bool CanSaveAsGuidePreset(const FToolMenuContext& InContext) const;
	void ExecuteSaveAsGuidePreset(const FToolMenuContext& InContext);

	bool CanSaveGuidePreset(const FToolMenuContext & InContext) const;
	void ExecuteSaveGuidePreset(const FToolMenuContext& InContext);

	bool CanReloadGuidePreset(const FToolMenuContext& InContext) const;
	void ExecuteReloadGuidePreset(const FToolMenuContext& InContext);

	FReply ExecuteLoadGuidePreset(FString InPresetName);

	FReply ExecuteReplaceGuidePreset(FString InPresetName);

	FReply ExecuteRemoveGuidePreset(FString InPresetName);
};
