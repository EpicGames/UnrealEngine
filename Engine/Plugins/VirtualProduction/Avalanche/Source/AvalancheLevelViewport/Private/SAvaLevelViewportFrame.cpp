// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaLevelViewportFrame.h"
#include "AvaViewportUtils.h"
#include "AvaVisibleArea.h"
#include "Framework/Application/SlateApplication.h"
#include "Math/Vector2D.h"
#include "SAvaLevelViewport.h"
#include "ViewportClient/AvaLevelViewportClient.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "Widgets/SAvaLevelViewportRuler.h"
#include "Widgets/SAvaLevelViewportStatusBar.h"

namespace UE::AvaLevelViewport::Private
{
	static constexpr float RulerSize = 20.f;
	static constexpr float StatusBarSize = 26.f;
}

FAvaLevelViewportGuideFrameAndWidget::FAvaLevelViewportGuideFrameAndWidget(
	TWeakPtr<SAvaLevelViewportFrame> InViewportFrameWeak)
{
	ViewportFrame = InViewportFrameWeak.Pin();

	if (ViewportFrame.IsValid())
	{
		ViewportWidget = ViewportFrame->GetViewportWidget();
		bIsValid = ViewportWidget.IsValid();
	}
}

FAvaLevelViewportGuideFrameAndClient::FAvaLevelViewportGuideFrameAndClient(
	TWeakPtr<SAvaLevelViewportFrame> InViewportFrameWeak)
{
	ViewportFrame = InViewportFrameWeak.Pin();

	if (ViewportFrame.IsValid())
	{
		ViewportClient = ViewportFrame->GetViewportClient();
		bIsValid = ViewportClient.IsValid();
	}
}

FAvaLevelViewportGuideFrameClientAndWidget::FAvaLevelViewportGuideFrameClientAndWidget(
	TWeakPtr<SAvaLevelViewportFrame> InViewportFrameWeak)
{
	ViewportFrame = InViewportFrameWeak.Pin();

	if (ViewportFrame.IsValid())
	{
		ViewportClient = ViewportFrame->GetViewportClient();
		ViewportWidget = ViewportFrame->GetViewportWidget();
		bIsValid = ViewportClient.IsValid() && ViewportWidget.IsValid();
	}
}

void SAvaLevelViewportFrame::Construct(const FArguments& InArgs, const FAssetEditorViewportConstructionArgs& InViewportArgs,
	TSharedPtr<ILevelEditor> InLevelEditor)
{
	using namespace UE::AvaLevelViewport::Private;

	ViewportClient = MakeShared<FAvaLevelViewportClient>();
	ViewportClient->Init();

	DPIScale = 1.f;

	ChildSlot
	[
		// Border added to hide background artifacts when in immersive mode.
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(0)
		[
			SNew(SConstraintCanvas)

			// Horizontal/Top ruler
			+ SConstraintCanvas::Slot()
			.Anchors(FAnchors(0, 0, 1, 0))
			.Offset(FMargin(RulerSize + 1, 0, 0, RulerSize))
			.Alignment(FVector2f(0, 0))
			[
				SAssignNew(HorizontalRuler, SAvaLevelViewportRuler, SharedThis(this))
				.Orientation(EOrientation::Orient_Horizontal)
			]

			// Vertical/Side ruler
			+ SConstraintCanvas::Slot()
			.Anchors(FAnchors(0, 0, 0, 1))
			.Offset(FMargin(0, RulerSize + 1, RulerSize, StatusBarSize + 1))
			.Alignment(FVector2f(0, 0))
			[
				SAssignNew(VerticalRuler, SAvaLevelViewportRuler, SharedThis(this))
				.Orientation(EOrientation::Orient_Vertical)
			]

			// Viewport
			+ SConstraintCanvas::Slot()
			.Anchors(FAnchors(0, 0, 1, 1))
			.Offset(FMargin(RulerSize + 1, RulerSize + 1, 0, StatusBarSize + 1))
			.Alignment(FVector2f(0, 0))
			[
				SAssignNew(ViewportWidget, SAvaLevelViewport, InViewportArgs)
				.ViewportFrame(SharedThis(this))
				.ParentLevelEditor(InLevelEditor)
			]

			// Status Bar
			+ SConstraintCanvas::Slot()
			.Anchors(FAnchors(0, 1, 1, 1))
			.Offset(FMargin(0, -StatusBarSize, 0, StatusBarSize))
			.Alignment(FVector2f(0, 0))
			[
				SNew(SAvaLevelViewportStatusBar, SharedThis(this))
			]
		]
	];
}

void SAvaLevelViewportFrame::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime,
	const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	DPIScale = 1.f;
	const FVector2f AbsoluteSize = AllottedGeometry.GetAbsoluteSize();
	const FVector2f LocalSize = AllottedGeometry.GetLocalSize();

	if (FAvaViewportUtils::IsValidViewportSize(AbsoluteSize)
		&& FAvaViewportUtils::IsValidViewportSize(LocalSize))
	{
		DPIScale = AbsoluteSize.X / LocalSize.X;
	}

	if (ViewportWidget.IsValid())
	{
		FVector2f MousePosition = FSlateApplication::Get().GetCursorPos();

		FVector2f ViewportPosition = ViewportWidget->GetTickSpaceGeometry().GetAbsolutePosition();
		FVector2f UnitScale = FVector2f::UnitVector;

		if (ViewportClient.IsValid())
		{
			const FAvaVisibleArea& VisibleArea = ViewportClient->GetVirtualZoomedVisibleArea();
			const float VisibleAreaFraction = VisibleArea.GetVisibleAreaFraction();
			const FVector2f WidgetLocalSize = ViewportWidget->GetTickSpaceGeometry().GetLocalSize();
			const FVector2f ViewportSize = ViewportClient->GetCachedViewportSize();
			const FVector2f ViewportOffset = (WidgetLocalSize - ViewportSize) * 0.5f;
			const FVector2f VirtualScale = ViewportClient->GetVirtualViewportScale();
			
			UnitScale = VisibleAreaFraction * VirtualScale;
			ViewportPosition -= VisibleArea.Offset / VisibleAreaFraction * DPIScale / VirtualScale;
			ViewportPosition += ViewportOffset * DPIScale;
		}

		if (TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(ViewportWidget.ToSharedRef()))
		{
			const FVector2f WindowPosition = Window->GetPositionInScreen();
			MousePosition -= WindowPosition;
			ViewportPosition -= WindowPosition;
		}

		if (HorizontalRuler.IsValid())
		{
			HorizontalRuler->SetCursor(static_cast<FVector2D>(MousePosition));
			HorizontalRuler->SetRuling(static_cast<FVector2D>(ViewportPosition), UnitScale.X);
		}

		if (VerticalRuler.IsValid())
		{
			VerticalRuler->SetCursor(static_cast<FVector2D>(MousePosition));
			VerticalRuler->SetRuling(static_cast<FVector2D>(ViewportPosition), UnitScale.Y);
		}
	}
}

FMargin SAvaLevelViewportFrame::GetHorizontalRulerOffset() const
{
	using namespace UE::AvaLevelViewport::Private;

	FMargin Offset = {RulerSize + 1, 0, 0, RulerSize};

	if (ViewportWidget.IsValid() && ViewportClient.IsValid())
	{
		const FVector2f LocalSize = ViewportWidget->GetTickSpaceGeometry().GetLocalSize();
		const FVector2f ViewportSize = ViewportClient->GetCachedViewportSize();
		const FVector2f ViewportOffset = (LocalSize - ViewportSize) * 0.5f;
		Offset.Left += FMath::RoundToDouble(ViewportOffset.X);
		Offset.Right += FMath::RoundToDouble(ViewportOffset.X);
	}

	return Offset;
}

FMargin SAvaLevelViewportFrame::GetVerticalRulerOffset() const
{
	using namespace UE::AvaLevelViewport::Private;

	FMargin Offset = {0, RulerSize + 1, RulerSize, StatusBarSize + 1};

	if (ViewportWidget.IsValid() && ViewportClient.IsValid())
	{
		const FVector2f LocalSize = ViewportWidget->GetTickSpaceGeometry().GetLocalSize();
		const FVector2f ViewportSize = ViewportClient->GetCachedViewportSize();
		const FVector2f ViewportOffset = (LocalSize - ViewportSize) * 0.5f;
		Offset.Top += ViewportOffset.Y;
		Offset.Bottom += FMath::RoundToDouble(ViewportOffset.Y);
	}

	return Offset;
}
