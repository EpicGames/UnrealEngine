// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelEditorViewport.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FAvaLevelViewportClient;
class ILevelEditor;
class SAvaLevelViewport;
class SAvaLevelViewportFrame;
class SAvaLevelViewportRuler;
struct FAssetEditorViewportConstructionArgs;

struct FAvaLevelViewportGuideFrameAndWidget
{
	TSharedPtr<SAvaLevelViewportFrame> ViewportFrame;
	TSharedPtr<SAvaLevelViewport> ViewportWidget;

	FAvaLevelViewportGuideFrameAndWidget(TWeakPtr<SAvaLevelViewportFrame> InViewportFrameWeak);

	bool IsValid() const { return bIsValid; }

protected:
	bool bIsValid = false;
};

struct FAvaLevelViewportGuideFrameAndClient
{
	TSharedPtr<SAvaLevelViewportFrame> ViewportFrame;
	TSharedPtr<FAvaLevelViewportClient> ViewportClient;

	FAvaLevelViewportGuideFrameAndClient(TWeakPtr<SAvaLevelViewportFrame> InViewportFrameWeak);

	bool IsValid() const { return bIsValid; }

protected:
	bool bIsValid = false;
};

struct FAvaLevelViewportGuideFrameClientAndWidget
{
	TSharedPtr<SAvaLevelViewportFrame> ViewportFrame;
	TSharedPtr<FAvaLevelViewportClient> ViewportClient;
	TSharedPtr<SAvaLevelViewport> ViewportWidget;

	FAvaLevelViewportGuideFrameClientAndWidget(TWeakPtr<SAvaLevelViewportFrame> InViewportFrameWeak);

	bool IsValid() const { return bIsValid; }

protected:
	bool bIsValid = false;
};

class SAvaLevelViewportFrame : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaLevelViewportFrame) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const FAssetEditorViewportConstructionArgs& InViewportArgs
		, TSharedPtr<ILevelEditor> InLevelEditor);

	TSharedPtr<SAvaLevelViewport> GetViewportWidget() const { return ViewportWidget; }

	TSharedPtr<FAvaLevelViewportClient> GetViewportClient() const { return ViewportClient; }

	float GetDPIScale() const { return DPIScale; }

	//~ Begin SWidget
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

private:
	TSharedPtr<SAvaLevelViewport> ViewportWidget;
	TSharedPtr<FAvaLevelViewportClient> ViewportClient;
	TSharedPtr<SAvaLevelViewportRuler> HorizontalRuler;
	TSharedPtr<SAvaLevelViewportRuler> VerticalRuler;
	float DPIScale;

	FMargin GetHorizontalRulerOffset() const;
	FMargin GetVerticalRulerOffset() const;
};
