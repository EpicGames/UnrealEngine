// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaLevelViewport.h"
#include "AvaLevelViewportCommands.h"
#include "AvaLevelViewportStyle.h"
#include "AvaLevelViewportToolbarContext.h"
#include "AvaViewportDataSubsystem.h"
#include "AvaViewportGuideInfo.h"
#include "AvaViewportVirtualSizeEnums.h"
#include "Camera/CameraActor.h"
#include "Engine/Texture.h"
#include "EngineUtils.h"
#include "IAvaComponentVisualizersSettings.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "LevelEditor.h"
#include "LevelEditor/AvaLevelEditorUtils.h"
#include "Misc/MessageDialog.h"
#include "ScopedTransaction.h"
#include "SEditorViewportToolBarButton.h"
#include "SEditorViewportToolBarMenu.h"
#include "SortHelper.h"
#include "SViewportToolBar.h"
#include "ToolMenus.h"
#include "Editor/LevelEditor/Private/ViewportToolbar/LevelViewportContext.h"
#include "ViewportClient/AvaLevelViewportClient.h"
#include "Widgets/DataTypes/AvaUserInputDialogDataTypeText.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/SAvaUserInputDialog.h"

#define LOCTEXT_NAMESPACE "SAvaLevelViewport"

namespace UE::AvaLevelViewport::Private
{
	static constexpr uint32 MaxCameraEntries = 10;
	static constexpr uint32 MaxViewportEntries = 4;
	static constexpr int32 VirtualSizeComponentMin = 1;
	static constexpr int32 VirtualSizeComponentMax = 10000;
	static constexpr float VirtualSizeAspectRatioMin = 0.0001f;
	static constexpr float VirtualSizeAspectRatioMax = 10.f;

	FSlotBase* FindChildSlotWithTag(const FName& InTag, const TSharedRef<SWidget>& InParentWidget)
	{
		if (FChildren* const Children = InParentWidget->GetChildren())
		{
			const int32 ChildrenNum = Children->Num();
			for (int32 Index = 0; Index < ChildrenNum; ++Index)
			{
				const TSharedRef<SWidget> ChildWidget = Children->GetChildAt(Index);

				const TSharedPtr<FTagMetaData> TagMetaData = ChildWidget->GetMetaData<FTagMetaData>();
				if (TagMetaData.IsValid() && TagMetaData->Tag == InTag)
				{
					return const_cast<FSlotBase*>(&Children->GetSlotAt(Index));
				}

				if (FSlotBase* const FoundSlot = FindChildSlotWithTag(InTag, ChildWidget))
				{
					return FoundSlot;
				}
			}
		}
		return nullptr;
	}
}

FName SAvaLevelViewport::RegisterViewportToolbar() const
{
	{
		// Extend the settings menu with custom visualizer entries
		const FName ParentSettingsName = "LevelEditor.ViewportToolbar.Settings";
		const FName SettingsName = "Ava.ViewportToolbar.Settings";
		
		if (!UToolMenus::Get()->FindMenu(ParentSettingsName))
		{
			UToolMenus::Get()->ExtendMenu(ParentSettingsName);
		}
		
		if (!UToolMenus::Get()->IsMenuRegistered(SettingsName))
		{
			AddVisualizerEntries(UToolMenus::Get()->RegisterMenu(SettingsName, ParentSettingsName));
		}	
	}
	
	const FName ViewportToolbarName = "Ava.ViewportToolbar";
	
	if (!UToolMenus::Get()->IsMenuRegistered(ViewportToolbarName))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(ViewportToolbarName, Super::RegisterViewportToolbar(), EMultiBoxType::SlimHorizontalToolBar);
		Menu->StyleName = "ViewportToolbar";
		Menu->bSeparateSections = false;
		
		{
			FToolMenuSection& LeftSection = Menu->AddSection("Left");
			FToolMenuEntry& ChildActorLock = LeftSection.AddMenuEntry(FAvaLevelViewportCommands::GetInternal().ToggleChildActorLock);
			ChildActorLock.InsertPosition.Position = EToolMenuInsertType::Last;
			ChildActorLock.Icon = FSlateIcon(FAvaLevelViewportStyle::Get().GetStyleSetName(), "Button.PivotMode");
			ChildActorLock.ToolBarData.LabelOverride = FText::GetEmpty();
		}
		
		{
			FToolMenuSection& RightSection = Menu->AddSection("Right");
			RightSection.Alignment = EToolMenuSectionAlign::Last;
		
			// Override the "Camera" submenu
			RightSection.AddDynamicEntry("DynamicCamera", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
			{
				TSharedPtr<SAvaLevelViewport> Viewport = StaticCastSharedPtr<SAvaLevelViewport>(ULevelViewportContext::GetLevelViewport(Section));
				if (!Viewport)
				{
					return;
				}
			
				FToolMenuEntry& CameraMenu = Section.AddSubMenu(
					"Camera",
					LOCTEXT("CameraMenuLabel", "Camera"),
					FText::GetEmpty(),
					FOnGetContent::CreateSP(Viewport.Get(), &SAvaLevelViewport::GenerateCameraMenu)
				);
				CameraMenu.Icon.Bind(Viewport.Get(), &SAvaLevelViewport::GetCameraIcon);
			}));
		}	
	}
	
	return ViewportToolbarName;
}

FSlateIcon SAvaLevelViewport::GetCameraIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.CameraComponent");
}

TSharedRef<SWidget> SAvaLevelViewport::GenerateCameraMenu()
{
	UToolMenus* const ToolMenus = UToolMenus::Get();
	check(ToolMenus);

	static const FName MenuName("AvaLevelEditor.ViewportToolBar.Camera");
	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* const Menu = ToolMenus->RegisterMenu(MenuName);
		Menu->AddDynamicSection(
			"DynamicSection", 
			FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				if (UAvaLevelViewportToolbarContext* const Context = InMenu->FindContext<UAvaLevelViewportToolbarContext>())
				{
					if (TSharedPtr<SAvaLevelViewport> Viewport = Context->GetViewport())
					{
						Viewport->FillCameraMenu(InMenu, /* Include cameras */ true);
					}
				}
			})
		);
	}

	OnFloatingButtonClicked();

	UAvaLevelViewportToolbarContext* Context = NewObject<UAvaLevelViewportToolbarContext>();
	Context->SetViewport(SharedThis(this));

	FToolMenuContext MenuContext(GetCommandList(), TSharedPtr<FExtender>(), Context);
	return ToolMenus->GenerateWidget(MenuName, MenuContext);
}

void SAvaLevelViewport::FillCameraMenu(UToolMenu* InMenu, bool bInIncludeCamerasAndViewports)
{
	UWorld* const World = GetWorld();
	if (!World)
	{
		return;
	}

	const FAvaLevelViewportCommands& LevelViewportCommands = FAvaLevelViewportCommands::GetInternal();

	if (bInIncludeCamerasAndViewports)
	{
		TArray<TWeakObjectPtr<ACameraActor>> CameraActors;
		for (ACameraActor* const CameraActor : TActorRange<ACameraActor>(World))
		{
			CameraActors.Add(CameraActor);
		}

		static const FText CamerasLabel = LOCTEXT("CameraActors", "Placed Cameras");

		if (CameraActors.Num() > UE::AvaLevelViewport::Private::MaxCameraEntries)
		{
			FToolMenuSection& Section = InMenu->AddSection("CameraActors");
			Section.AddSubMenu("CameraActors"
				, CamerasLabel
				, LOCTEXT("CameraActorsToolTip", "Scene Cameras")
				, FNewToolMenuDelegate::CreateSP(this, &SAvaLevelViewport::AddCameraEntries, CameraActors));
		}
		else
		{
			FToolMenuSection& Section = InMenu->AddSection("CameraActors", CamerasLabel);
			AddCameraEntries(Section, CameraActors);
		}
	}

	FToolMenuSection& VirtualSection = InMenu->AddSection("VirtualViewport", LOCTEXT("VirtualViewport", "Virtual Viewport"));

	static const FText ViewportVirtualSizeLabel = LOCTEXT("ViewportVirtualSize", "Ruler Override");
	static const FText ViewportVirtualSizeToolTip = LOCTEXT("ViewportVirtualSizeToolTip", "Override the size of the viewport rulers.\n\nThe ruler size does not effect the output render resolution. It is a design aide only.");

	VirtualSection.AddSubMenu(
		"VirtualViewportSize",
		ViewportVirtualSizeLabel,
		ViewportVirtualSizeToolTip,
		FNewToolMenuDelegate::CreateSP(this, &SAvaLevelViewport::AddVirtualSizeMenuEntries)
	);

	static const FText ViewportGuidePresetsLabel = LOCTEXT("ViewportGuidePresets", "Guide Presets");
	static const FText ViewportGuidePresetsToolTip = LOCTEXT("ViewportGuidePresetsToolTip", "Save and load guide presets.");

	VirtualSection.AddSubMenu(
		"GuidePresets",
		ViewportGuidePresetsLabel,
		ViewportGuidePresetsToolTip,
		FNewToolMenuDelegate::CreateSP(this, &SAvaLevelViewport::AddGuidePresetMenuEntries)
	);

	static const FText CameraZoomLabel = LOCTEXT("ViewportCameraZoom", "Camera Zoom");

	VirtualSection.AddSubMenu(
		"ViewportCameraZoom",
		CameraZoomLabel,
		FText(),
		FNewToolMenuDelegate::CreateSP(this, &SAvaLevelViewport::AddCameraZoomMenuEntries)
	);

	VirtualSection.AddMenuEntry(
		"CameraResetTransform", 
		LevelViewportCommands.CameraTransformReset, 
		LOCTEXT("CameraResetTransform", "Reset Transform")
	);

	if (!bInIncludeCamerasAndViewports)
	{
		return;
	}

	int32 ViewportTypeCount = 0;

	if (FLevelEditorModule* LevelEditorModule = FAvaLevelEditorUtils::GetLevelEditorModule())
	{
		LevelEditorModule->IterateViewportTypes(
			[&ViewportTypeCount](FName, const FViewportTypeDefinition&)
			{
				++ViewportTypeCount;
			}
		);
	}

	static const FText ViewportTypesLabel = LOCTEXT("ViewportTypes", "Viewport Type");

	if (ViewportTypeCount > UE::AvaLevelViewport::Private::MaxViewportEntries)
	{
		FToolMenuSection& Section = InMenu->AddSection("ViewportTypes");
		Section.AddSubMenu("ViewportTypes"
			, ViewportTypesLabel
			, FText()
			, FNewToolMenuDelegate::CreateSP(this, &SAvaLevelViewport::AddViewportEntries));
	}
	else
	{
		FToolMenuSection& Section = InMenu->AddSection("ViewportTypes", ViewportTypesLabel);
		AddViewportEntries(Section);
	}
}

bool SAvaLevelViewport::CanChangeCameraInCameraMenu() const
{
	if (TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient())
	{
		if (ViewportClient->GetCinematicViewTarget())
		{
			return false;
		}
	}

	return true;
}

void SAvaLevelViewport::AddCameraEntries(UToolMenu* InMenu, TArray<TWeakObjectPtr<ACameraActor>> InCameraActors)
{
	AddCameraEntries(InMenu->AddSection(TEXT("CameraSection")), MoveTemp(InCameraActors));
}

void SAvaLevelViewport::AddCameraEntries(FToolMenuSection& InSection, TArray<TWeakObjectPtr<ACameraActor>> InCameraActors)
{
	InCameraActors.StableSort(
		[](const TWeakObjectPtr<ACameraActor>& Left, const TWeakObjectPtr<ACameraActor>& Right)
		{
			SceneOutliner::FNumericStringWrapper LeftWrapper(FString(Left->GetActorLabel()));
			SceneOutliner::FNumericStringWrapper RightWrapper(FString(Right->GetActorLabel()));
			return LeftWrapper < RightWrapper;
		}
	);

	const TSharedRef<SAvaLevelViewport> This = SharedThis(this);

	const FSlateIcon CameraIcon = GetCameraIcon();

	for (const TWeakObjectPtr<ACameraActor>& CameraActor : InCameraActors)
	{
		if (!CameraActor.IsValid())
		{
			continue;
		}

		const FText ActorDisplayName = FText::FromString(CameraActor->GetActorLabel());

		const FUIAction CameraAction(FExecuteAction::CreateSP(This, &SAvaLevelViewport::ActivateCamera, CameraActor)
			, FCanExecuteAction::CreateSP(This, &SAvaLevelViewport::CanChangeCameraInCameraMenu)
			, FIsActionChecked::CreateSP(This, &SAvaLevelViewport::IsCameraActive, CameraActor));

		InSection.AddMenuEntry(NAME_None
			, ActorDisplayName
			, FText::Format(LOCTEXT("ActivateCameraActorToolTip", "Activate {0}"), ActorDisplayName)
			, CameraIcon
			, CameraAction
			, EUserInterfaceActionType::RadioButton);
	}
}

bool SAvaLevelViewport::CanResetPilotedCameraTransform() const
{
	if (TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient())
	{
		if (ViewportClient->GetCinematicViewTarget())
		{
			return false;
		}

		return !!ViewportClient->GetViewTarget();
	}

	return false;
}

void SAvaLevelViewport::ResetPilotedCameraTransform()
{
	if (TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient())
	{
		if (AActor* Actor = ViewportClient->GetViewTarget())
		{
			Actor->SetActorLocationAndRotation(
				FVector(-500.0, 0.0, 0.0),
				FRotator::ZeroRotator
			);
		}
	}
}

void SAvaLevelViewport::AddViewportEntries(UToolMenu* InMenu)
{
	AddViewportEntries(InMenu->AddSection(TEXT("ViewportSection")));
}

void SAvaLevelViewport::AddViewportEntries(FToolMenuSection& InSection)
{
	if (FLevelEditorModule* LevelEditorModule = FAvaLevelEditorUtils::GetLevelEditorModule())
	{
		LevelEditorModule->IterateViewportTypes(
			[&InSection](FName InViewportTypeName, const FViewportTypeDefinition& InDefinition)
			{
				if (InDefinition.ActivationCommand.IsValid())
				{
					InSection.AddMenuEntry(*FString::Printf(TEXT("ViewportType_%s")
						, *InViewportTypeName.ToString())
						, InDefinition.ActivationCommand);
				}
			}
		);
	}
}

void SAvaLevelViewport::AddVirtualSizeMenuEntries(UToolMenu* InMenu)
{
	AddVirtualSizeDefaultEntries(InMenu->AddSection(TEXT("VirtualSizeDefaults"), LOCTEXT("VirtualSizeDefaults", "Defaults")));
	AddVirtualSizeSizeSettings(InMenu->AddSection(TEXT("VirtualSizeSection"), LOCTEXT("VirtualSize", "Settings")));
}

void SAvaLevelViewport::AddVirtualSizeDefaultEntries(FToolMenuSection& InSection)
{
	const FAvaLevelViewportCommands& LevelViewportCommands = FAvaLevelViewportCommands::GetInternal();

	InSection.AddMenuEntry("NoSize", LevelViewportCommands.VirtualSizeDisable, INVTEXT("-"));
	InSection.AddMenuEntry("1920x1080", LevelViewportCommands.VirtualSize1920x1080, INVTEXT("1920 x 1080"));
}

void SAvaLevelViewport::AddVirtualSizeSizeSettings(FToolMenuSection& InSection)
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportCommands& LevelViewportCommands = FAvaLevelViewportCommands::GetInternal();

	InSection.AddMenuEntry("UseUnlockedAspectRatio", LevelViewportCommands.VirtualSizeAspectRatioUnlocked, LOCTEXT("VirtualSizeUnlockedAspectRatio", "Free Aspect Ratio"));
	InSection.AddMenuEntry("UseLockedAspectRatio", LevelViewportCommands.VirtualSizeAspectRatioLocked, LOCTEXT("VirtualSizeLockedAspectRatio", "Locked Aspect Ratio"));
	InSection.AddMenuEntry("UseLockedToCameraAspectRatio", LevelViewportCommands.VirtualSizeAspectRatioLockedToCamera, LOCTEXT("VirtualSizeLockedToCameraAspectRatio", "Use Camera Aspect Ratio"));

	TSharedRef<SWidget> ResolutionWidget = SNew(SHorizontalBox)
			
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			SNew(SSpinBox<int32>)
			.Value(this, &SAvaLevelViewport::GetVirtualSizeX)
			.Delta(1)
			.MinValue(VirtualSizeComponentMin)
			.MinSliderValue(VirtualSizeComponentMin)
			.MaxValue(VirtualSizeComponentMax)
			.MaxSliderValue(VirtualSizeComponentMax)
			.EnableSlider(true)
			.OnValueChanged(this, &SAvaLevelViewport::OnVirtualSizeComponentSliderCommitted, ETextCommit::Default, EAxis::X)
			.OnValueCommitted(this, &SAvaLevelViewport::OnVirtualSizeComponentSliderCommitted, EAxis::X)
			.MinDesiredWidth(50.f)
			.Justification(ETextJustify::InvariantRight)
			.OnBeginSliderMovement(this, &SAvaLevelViewport::OnVirtualSizeSliderBegin)
			.OnEndSliderMovement(this, &SAvaLevelViewport::OnVirtualSizeComponentSliderEnd)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.f, 0.f, 5.f, 0.f)
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			SNew(STextBlock)
			.Text(INVTEXT("x"))
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			SNew(SSpinBox<int32>)
			.Value(this, &SAvaLevelViewport::GetVirtualSizeY)
			.Delta(1)
			.MinValue(VirtualSizeComponentMin)
			.MinSliderValue(VirtualSizeComponentMin)
			.MaxValue(VirtualSizeComponentMax)
			.MaxSliderValue(VirtualSizeComponentMax)
			.EnableSlider(true)
			.OnValueChanged(this, &SAvaLevelViewport::OnVirtualSizeComponentSliderCommitted, ETextCommit::Default, EAxis::Y)
			.OnValueCommitted(this, &SAvaLevelViewport::OnVirtualSizeComponentSliderCommitted, EAxis::Y)
			.MinDesiredWidth(50.f)
			.Justification(ETextJustify::InvariantRight)
			.OnBeginSliderMovement(this, &SAvaLevelViewport::OnVirtualSizeSliderBegin)
			.OnEndSliderMovement(this, &SAvaLevelViewport::OnVirtualSizeComponentSliderEnd)
		];

	InSection.AddEntry(FToolMenuEntry::InitWidget(
		"CustomResolution",
		SNew(SBox)
		.Padding(3.f)
		[
			ResolutionWidget
		],
		LOCTEXT("VirtualSizeCustomResolution", "Resolution"),
		true
	));

	if (VirtualSizeAspectRatioState == EAvaViewportVirtualSizeAspectRatioState::LockedToCamera)
	{
		bool bIsLockedToCamera = false;

		if (TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient())
		{
			bIsLockedToCamera = ViewportClient->GetActorLock().HasValidLockedActor()
				|| ViewportClient->GetCinematicActorLock().HasValidLockedActor();
		}

		if (!bIsLockedToCamera)
		{
			InSection.AddEntry(FToolMenuEntry::InitWidget(
				"NoCameraWarning",
				SNew(SBox)
					.HAlign(EHorizontalAlignment::HAlign_Center)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(3.f)
						[
							SNew(SImage)
								.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
								.DesiredSizeOverride(FVector2D(16, 16))
								.ColorAndOpacity(FSlateColor(EStyleColor::AccentRed))
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1.f)
						.Padding(3.f)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("VirtualSizeCustomNoCameraWarning", "Warning! No active camera."))
						]
					],
				FText::GetEmpty()
			));
		}
	}

	InSection.AddEntry(FToolMenuEntry::InitWidget(
		"AspectRatio",
		SNew(SBox)
		.HAlign(EHorizontalAlignment::HAlign_Center)
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("VirtualSizeAspectRatio", "Aspect Ratio:"))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				.Padding(3.f)
				[
					SNew(SSpinBox<float>)
					.Value(this, &SAvaLevelViewport::GetVirtualSizeAspectRatio)
					.MinValue(VirtualSizeAspectRatioMin)
					.MinSliderValue(VirtualSizeAspectRatioMin)
					.MaxValue(VirtualSizeAspectRatioMax)
					.MaxSliderValue(VirtualSizeAspectRatioMax)
					.EnableSlider(true)
					.OnValueChanged(this, &SAvaLevelViewport::OnVirtualSizeAspectRatioCommitted, ETextCommit::Default)
					.OnValueCommitted(this, &SAvaLevelViewport::OnVirtualSizeAspectRatioCommitted)
					.MinDesiredWidth(50.f)
					.Justification(ETextJustify::InvariantRight)
					.OnBeginSliderMovement(this, &SAvaLevelViewport::OnVirtualSizeSliderBegin)
					.OnEndSliderMovement(this, &SAvaLevelViewport::OnVirtualSizeAspectRatioSliderEnd)
					.IsEnabled(this, &SAvaLevelViewport::IsVirtualSizeAspectRatioEnabled)
				]
		],
		FText::GetEmpty()
	));
}

void SAvaLevelViewport::AddGuidePresetMenuEntries(UToolMenu* InMenu)
{
	InMenu->AddDynamicSection(
		"Current",
		FNewSectionConstructChoice(FNewToolMenuDelegate::CreateSP(this, &SAvaLevelViewport::AddGuidePresetCurrentMenu))
	);

	InMenu->AddDynamicSection(
		"Saved",
		FNewSectionConstructChoice(FNewToolMenuDelegate::CreateSP(this, &SAvaLevelViewport::AddGuidePresetSavedMenu))
	);
}

void SAvaLevelViewport::AddGuidePresetCurrentMenu(UToolMenu* InMenu)
{
	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(this);

	if (!DataSubsystem)
	{
		return;
	}

	FAvaViewportData* Data = DataSubsystem->GetData();

	if (!Data)
	{
		return;
	}

	FToolMenuSection& ActiveSection = InMenu->AddSection(
		"Active",
		LOCTEXT("GuidePresetsActive", "Active")
	);

	if (!DataSubsystem->GetGuidePresetProvider().GetLastAccessedGuidePresetName().IsEmpty())
	{
		const FText ActiveEntryDescription = FText::Format(
			LOCTEXT("ActiveGuidePresetNameFormat", "Active: {0}"),
			FText::FromString(DataSubsystem->GetGuidePresetProvider().GetLastAccessedGuidePresetName())
		);

		TSharedRef<SHorizontalBox> ActivePresetWidget = SNew(SHorizontalBox)
			.ToolTipText(LOCTEXT("GuidePresetActivePreset", "This is the last saved or loaded preset."))
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(26.f, 0.f, 0.f, 0.f)
			[
				SNew(SImage)
				.Image(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Layout").GetIcon())
				.DesiredSizeOverride(FVector2D(14.0, 14.f))
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(10.f, 0.f, 8.f, 0.f)
			[
				SNew(STextBlock)
				.Text(ActiveEntryDescription)
			];

		ActiveSection.AddEntry(FToolMenuEntry::InitWidget(
			"PresetName",
			ActivePresetWidget,
			FText::GetEmpty()
		));

		FToolUIAction SaveLastAction;
		SaveLastAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanSaveGuidePreset);
		SaveLastAction.ExecuteAction = FToolMenuExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteSaveGuidePreset);

		ActiveSection.AddMenuEntry(
			"SaveLast",
			LOCTEXT("SaveLastGuidePreset", "Save"),
			LOCTEXT("SaveLastGuidePresetToolTip", "Save the current guides to the last saved or loaded preset. The guides already in the preset will be replaced."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.SaveAsset"),
			FToolUIActionChoice(SaveLastAction)
		);
	}

	// This could be a command, but we're going to need individual actions for each load preset action anyway.
	FToolUIAction SaveAction;
	SaveAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanSaveAsGuidePreset);
	SaveAction.ExecuteAction = FToolMenuExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteSaveAsGuidePreset);

	ActiveSection.AddMenuEntry(
		"Save",
		LOCTEXT("SaveGuidePreset", "Save As"),
		LOCTEXT("SaveGuidePresetToolTip", "Save the current guides as a new preset."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.SaveAssetAs"),
		FToolUIActionChoice(SaveAction)
	);

	if (!DataSubsystem->GetGuidePresetProvider().GetLastAccessedGuidePresetName().IsEmpty())
	{
		FToolUIAction ReloadAction;
		ReloadAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateSP(this, &SAvaLevelViewport::CanReloadGuidePreset);
		ReloadAction.ExecuteAction = FToolMenuExecuteAction::CreateSP(this, &SAvaLevelViewport::ExecuteReloadGuidePreset);

		ActiveSection.AddMenuEntry(
			"LoadLast",
			LOCTEXT("LoadLastGuidePreset", "Reload"),
			LOCTEXT("LoadLastGuidePresetToolTip", "Reload the current guides from the last saved or loaded preset. The active guides will be replaced."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Redo"),
			FToolUIActionChoice(ReloadAction)
		);
	}
}

void SAvaLevelViewport::AddGuidePresetSavedMenu(UToolMenu* InMenu)
{
	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(this);

	if (!DataSubsystem)
	{
		return;
	}

	TArray<FString> PresetNames = DataSubsystem->GetGuidePresetProvider().GetGuidePresetNames();

	if (PresetNames.IsEmpty())
	{
		return;
	}

	FToolMenuSection& LoadSection = InMenu->AddSection(
		"Saved",
		LOCTEXT("SavedGuidePresets", "Saved")
	);

	static const FName LoadName = "Load";

	int32 LoadNameIndex = 1;

	for (const FString& PresetName : PresetNames)
	{
		const FButtonStyle* GuidePresetMenuStyle = &FAvaLevelViewportStyle::Get().GetWidgetStyle<FButtonStyle>("Avalanche.Menu.GuidePreset.Button");

		TSharedRef<SHorizontalBox> RowWidget = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			//.Padding(6.f, 0.f, 0.f, 0.f)
			[
				SNew(SButton)
				.ButtonStyle(GuidePresetMenuStyle)
				.OnClicked(this, &SAvaLevelViewport::ExecuteLoadGuidePreset, PresetName)
				.ToolTipText(LOCTEXT("LoadGuidePresetToolTip", "Load guides from this preset. The active guides will be replaced."))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(EVerticalAlignment::VAlign_Center)
					.Padding(20.f, 0.f, 0.f, 0.f)
					[
						SNew(SImage)
						.Image(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Layout").GetIcon())
						.DesiredSizeOverride(FVector2D(14.0, 14.f))
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.f)
					.HAlign(EHorizontalAlignment::HAlign_Left)
					.VAlign(EVerticalAlignment::VAlign_Center)
					.Padding(10.f, 0.f, 0.f, 0.f)
					[
						SNew(STextBlock)
						.Text(FText::FromString(PresetName))
					]
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(6.f, 0.f, 0.f, 0.f)
			[
				SNew(SButton)
				.ButtonStyle(GuidePresetMenuStyle)
				.OnClicked(this, &SAvaLevelViewport::ExecuteReplaceGuidePreset, PresetName)
				.ToolTipText(LOCTEXT("ReplaceGuidePresetTooltip", "Save the current guides to this preset. The guides already in the preset will be replaced."))
				.IsEnabled(this, &SAvaLevelViewport::CanSaveAsGuidePreset)
				[
					SNew(SImage)
					.Image(FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.SaveAsset").GetIcon())
					.DesiredSizeOverride(FVector2D(14.0, 14.f))
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(6.f, 0.f, 6.f, 0.f)
			[
				SNew(SButton)
				.ButtonStyle(GuidePresetMenuStyle)
				.OnClicked(this, &SAvaLevelViewport::ExecuteRemoveGuidePreset, PresetName)
				.ToolTipText(LOCTEXT("RemoveGuidePresetTooltip", "Remove this guide preset."))
				[
					SNew(SImage)
					.Image(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete").GetIcon())
					.DesiredSizeOverride(FVector2D(14.0, 14.f))
				]
			];

		LoadSection.AddEntry(
			FToolMenuEntry::InitWidget(
				FName(LoadName, LoadNameIndex++),
				RowWidget,
				FText::GetEmpty()
			)
		);
	}
}

void SAvaLevelViewport::AddCameraZoomMenuEntries(UToolMenu* InMenu)
{
	const FAvaLevelViewportCommands& LevelViewportCommands = FAvaLevelViewportCommands::GetInternal();

	FToolMenuSection& PanSection = InMenu->AddSection(TEXT("CameraPanSection"), LOCTEXT("CameraPanSection", "Pan"));
	PanSection.AddMenuEntry("CameraPanLeft",       LevelViewportCommands.CameraPanLeft,       LOCTEXT("CameraPanLeft",       "Pan Left"));
	PanSection.AddMenuEntry("CameraPanRight",      LevelViewportCommands.CameraPanRight,      LOCTEXT("CameraPanRight",      "Pan Right"));
	PanSection.AddMenuEntry("CameraPanUp",         LevelViewportCommands.CameraPanUp,         LOCTEXT("CameraPanUp",         "Pan Up"));
	PanSection.AddMenuEntry("CameraPanDown",       LevelViewportCommands.CameraPanDown,       LOCTEXT("CameraPanDown",       "Pan Down"));

	FToolMenuSection& ZoomSection = InMenu->AddSection(TEXT("CameraZoomSection"), LOCTEXT("CameraZoomSection", "Zoom"));
	ZoomSection.AddMenuEntry("CameraZoomInCenter",  LevelViewportCommands.CameraZoomInCenter,  LOCTEXT("CameraZoomInCenter",  "Zoom In"));
	ZoomSection.AddMenuEntry("CameraZoomOutCenter", LevelViewportCommands.CameraZoomOutCenter, LOCTEXT("CameraZoomOutCenter", "Zoom Out"));

	FToolMenuSection& OtherSection = InMenu->AddSection(TEXT("CameraOtherSection"), LOCTEXT("CameraOtherSection", "Other"));
	OtherSection.AddMenuEntry("CameraFrameActor", LevelViewportCommands.CameraFrameActor, LOCTEXT("CameraFrameActor", "Frame Actor"));
	OtherSection.AddMenuEntry("CameraResetZoom",  LevelViewportCommands.CameraZoomReset,  LOCTEXT("CameraResetZoom",  "Reset Zoom"));
}

void SAvaLevelViewport::AddVisualizerEntries(UToolMenu* InMenu)
{
	FToolMenuSection& Section = InMenu->FindOrAddSection("Settings");

	TSharedRef<SWidget> IconSizeWidget = SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.WidthOverride(100.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
				.Padding(FMargin(1.0f))
				[
					SNew(SSpinBox<int32>)
					.Style(&FAppStyle::Get(), "Menu.SpinBox")
					.ToolTipText(LOCTEXT("VisualizerIconSizeTooltip", "Screen size guide for Motion Design component visualizer icons."))
					.MinValue(1)
					.MinSliderValue(1)
					.MaxValue(32)
					.MaxSliderValue(32)
					.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
					.Value_Lambda(
						[]() -> int32
						{
							if (IAvalancheComponentVisualizersModule* CompVisModule = IAvalancheComponentVisualizersModule::GetIfLoaded())
							{
								if (IAvaComponentVisualizersSettings* CompVizSettings = CompVisModule->GetSettings())
								{
									return static_cast<int32>(CompVizSettings->GetSpriteSize());
								}
							}

							return 10;
						})
					.OnValueChanged_Lambda(
						[](int32 InNewValue)
						{
							if (IAvalancheComponentVisualizersModule* CompVisModule = IAvalancheComponentVisualizersModule::GetIfLoaded())
							{
								if (IAvaComponentVisualizersSettings* CompVizSettings = CompVisModule->GetSettings())
								{
									CompVizSettings->SetSpriteSize(static_cast<float>(InNewValue));
									
									if (FLevelEditorModule* LevelEditorModule = FAvaLevelEditorUtils::GetLevelEditorModule())
									{
										LevelEditorModule->BroadcastRedrawViewports(false);
									}
								}
							}
						})
					.OnValueCommitted_Lambda(
						[](int32 InNewValue, ETextCommit::Type InCommittype)
						{
							if (IAvalancheComponentVisualizersModule* CompVisModule = IAvalancheComponentVisualizersModule::GetIfLoaded())
							{
								if (IAvaComponentVisualizersSettings* CompVizSettings = CompVisModule->GetSettings())
								{
									CompVizSettings->SetSpriteSize(static_cast<float>(InNewValue));
									CompVizSettings->SaveSettings();

									if (FLevelEditorModule* LevelEditorModule = FAvaLevelEditorUtils::GetLevelEditorModule())
									{
										LevelEditorModule->BroadcastRedrawViewports(false);
									}
								}
							}
						}
					)
				]
			]
		];

	Section.AddEntry(FToolMenuEntry::InitWidget(
		"IconSizeWidget",
		IconSizeWidget,
		LOCTEXT("VisualizerIconSize", "Visualizer Icon Size")
	));
}

bool SAvaLevelViewport::IsChildActorLockEnabled() const
{
	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return false;
	}

	return ViewportClient->AreChildActorsLocked();
}

void SAvaLevelViewport::OnVirtualSizeComponentSliderCommitted(int32 InNewDimension, ETextCommit::Type InCommitType, EAxis::Type InAxis)
{
	using namespace UE::AvaLevelViewport::Private;

	InNewDimension = FMath::Clamp(InNewDimension, VirtualSizeComponentMin, VirtualSizeComponentMax);

	FIntPoint LocalCustomVirtualSize = GetVirtualSize();

	switch (InAxis)
	{
		case EAxis::X:
			if (LocalCustomVirtualSize.X == InNewDimension)
			{
				return;
			}

			LocalCustomVirtualSize.X = InNewDimension;
			break;

		case EAxis::Y:
			if (LocalCustomVirtualSize.Y == InNewDimension)
			{
				return;
			}

			LocalCustomVirtualSize.Y = InNewDimension;
			break;

		default:
			return;
	}

	switch (VirtualSizeAspectRatioState)
	{
		default:
		case EAvaViewportVirtualSizeAspectRatioState::Unlocked:
			if (LocalCustomVirtualSize.X > 0 && LocalCustomVirtualSize.Y > 0)
			{
				VirtualSizeAspectRatio = static_cast<float>(LocalCustomVirtualSize.X) / static_cast<float>(LocalCustomVirtualSize.Y);
			}
			else
			{
				VirtualSizeAspectRatio = 0.f;
			}
			break;

		case EAvaViewportVirtualSizeAspectRatioState::Locked:
		case EAvaViewportVirtualSizeAspectRatioState::LockedToCamera:
		{
			const float AspectRatio = GetVirtualSizeAspectRatio();

			if (AspectRatio > 0)
			{
				switch (InAxis)
				{
					case EAxis::X:
						LocalCustomVirtualSize.Y = FMath::RoundToInt(static_cast<float>(LocalCustomVirtualSize.X) / AspectRatio);
						break;

					case EAxis::Y:
						LocalCustomVirtualSize.X = FMath::RoundToInt(static_cast<float>(LocalCustomVirtualSize.Y) * AspectRatio);
						break;
				}
			}
			break;
		}
	}

	SetVirtualSize(LocalCustomVirtualSize);
}

void SAvaLevelViewport::OnVirtualSizeAspectRatioCommitted(float InNewAspectRatio, ETextCommit::Type InCommitType)
{
	if (VirtualSizeAspectRatioState == EAvaViewportVirtualSizeAspectRatioState::LockedToCamera)
	{
		return;
	}

	using namespace UE::AvaLevelViewport::Private;

	InNewAspectRatio = FMath::Clamp(InNewAspectRatio, VirtualSizeAspectRatioMin, VirtualSizeAspectRatioMax);

	VirtualSizeAspectRatio = InNewAspectRatio;

	FIntPoint VirtualSize = GetVirtualSize();

	if (VirtualSize.X > 0)
	{
		VirtualSize.Y = FMath::Clamp(FMath::RoundToInt(static_cast<float>(VirtualSize.X) / InNewAspectRatio), 1, 1000000);
		SetVirtualSize(VirtualSize);
	}
}

void SAvaLevelViewport::OnVirtualSizeSliderBegin()
{
	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return;
	}

	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(ViewportClient->GetViewportWorld());

	if (!DataSubsystem)
	{
		return;
	}

	FAvaViewportData* Data = DataSubsystem->GetData();

	if (!Data)
	{
		return;
	}

	VirtualSizeComponentSliderTransaction = MakeShared<FScopedTransaction>(
		LOCTEXT("SetCustomVirtualSize", "Set Custom Virtual Size")
	);
	DataSubsystem->ModifyDataSource();
}

void SAvaLevelViewport::OnVirtualSizeComponentSliderEnd(int32 InNewSize)
{
	VirtualSizeComponentSliderTransaction.Reset();
}

void SAvaLevelViewport::OnVirtualSizeAspectRatioSliderEnd(float InNewAspectRatio)
{
	VirtualSizeComponentSliderTransaction.Reset();
}

bool SAvaLevelViewport::IsVirtualSizeAspectRatioEnabled() const
{
	return VirtualSizeAspectRatioState != EAvaViewportVirtualSizeAspectRatioState::LockedToCamera;
}

bool SAvaLevelViewport::CanSaveAsGuidePreset() const
{
	return HasGuides();
}

bool SAvaLevelViewport::CanSaveAsGuidePreset(const FToolMenuContext& InContext) const
{
	return HasGuides();
}

void SAvaLevelViewport::ExecuteSaveAsGuidePreset(const FToolMenuContext& InContext)
{
	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(this);

	if (!DataSubsystem)
	{
		return;
	}

	FAvaViewportData* ViewportData = DataSubsystem->GetData();

	if (!ViewportData)
	{
		return;
	}

	FAvaUserInputDialogTextData::FParams Params;
	Params.InitialValue = LOCTEXT("NewPreset", "NewPreset");
	Params.MaxLength = 30;

	TSharedRef<FAvaUserInputDialogTextData> TextInput = MakeShared<FAvaUserInputDialogTextData>(Params);

	const bool bAccepted = SAvaUserInputDialog::CreateModalDialog(
		TextInput,
		SharedThis(this),
		LOCTEXT("NewPresetName", "New Preset Name")
	);

	if (!bAccepted)
	{
		return;
	}

	const FString PresetName = TextInput->GetValue().ToString();

	if (PresetName.IsEmpty())
	{
		return;
	}

	TArray<FAvaViewportGuideInfo> GuidesTemp;

	if (DataSubsystem->GetGuidePresetProvider().LoadGuidePreset(PresetName, GuidesTemp, FVector2f(1.f, 1.f)))
	{
		static const FText ReplaceGuidePresetFormat = LOCTEXT("AlreadyExistsGuidePresetFormat", "Guide preset already exists. Are you sure you want to replace:\n\n{0}");

		const EAppReturnType::Type Response = FMessageDialog::Open(
			EAppMsgType::OkCancel,
			FText::Format(ReplaceGuidePresetFormat, FText::FromString(PresetName)),
			LOCTEXT("ConfirmReplace", "Confirm Replace")
		);

		if (Response != EAppReturnType::Ok)
		{
			return;
		}
	}

	DataSubsystem->GetGuidePresetProvider().SaveGuidePreset(PresetName, ViewportData->GuideData, GetVirtualSize());
}

bool SAvaLevelViewport::CanSaveGuidePreset(const FToolMenuContext& InContext) const
{
	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(this);

	if (!DataSubsystem)
	{
		return false;
	}

	if (DataSubsystem->GetGuidePresetProvider().GetLastAccessedGuidePresetName().IsEmpty())
	{
		return false;
	}

	FAvaViewportData* Data = DataSubsystem->GetData();

	if (!Data)
	{
		return false;
	}

	return Data->GuideData.Num() > 0;
}

void SAvaLevelViewport::ExecuteSaveGuidePreset(const FToolMenuContext& InContext)
{
	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(this);

	if (!DataSubsystem)
	{
		return;
	}

	FAvaViewportData* ViewportData = DataSubsystem->GetData();

	if (!ViewportData)
	{
		return;
	}

	DataSubsystem->GetGuidePresetProvider().SaveGuidePreset(
		DataSubsystem->GetGuidePresetProvider().GetLastAccessedGuidePresetName(),
		ViewportData->GuideData,
		GetVirtualSize()
	);
}

bool SAvaLevelViewport::CanReloadGuidePreset(const FToolMenuContext& InContext) const
{
	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(this);

	if (!DataSubsystem)
	{
		return false;
	}

	return !DataSubsystem->GetGuidePresetProvider().GetLastAccessedGuidePresetName().IsEmpty();
}

void SAvaLevelViewport::ExecuteReloadGuidePreset(const FToolMenuContext& InContext)
{
	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(this);

	if (!DataSubsystem)
	{
		return;
	}

	FAvaViewportData* ViewportData = DataSubsystem->GetData();

	if (!ViewportData)
	{
		return;
	}

	const bool bSuccess = DataSubsystem->GetGuidePresetProvider().LoadGuidePreset(
		DataSubsystem->GetGuidePresetProvider().GetLastAccessedGuidePresetName(),
		ViewportData->GuideData,
		GetVirtualSize()
	);

	if (bSuccess)
	{
		ReloadGuides();
	}
}

FReply SAvaLevelViewport::ExecuteLoadGuidePreset(FString InPresetName)
{
	FSlateApplication::Get().DismissAllMenus();

	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(this);

	if (!DataSubsystem)
	{
		return FReply::Handled();
	}

	FAvaViewportData* ViewportData = DataSubsystem->GetData();

	if (!ViewportData)
	{
		return FReply::Handled();
	}

	const bool bSuccess = DataSubsystem->GetGuidePresetProvider().LoadGuidePreset(
		InPresetName,
		ViewportData->GuideData,
		GetVirtualSize()
	);

	if (bSuccess)
	{
		ReloadGuides();
	}

	return FReply::Handled();
}

FReply SAvaLevelViewport::ExecuteReplaceGuidePreset(FString InPresetName)
{
	FSlateApplication::Get().DismissAllMenus();

	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(this);

	if (!DataSubsystem)
	{
		return FReply::Handled();
	}

	FAvaViewportData* ViewportData = DataSubsystem->GetData();

	if (!ViewportData)
	{
		return FReply::Handled();
	}

	static const FText ReplaceGuidePresetFormat = LOCTEXT("ReplaceGuidePresetFormat", "Are you sure you want to replace guide preset:\n\n{0}");

	const EAppReturnType::Type Response = FMessageDialog::Open(
		EAppMsgType::OkCancel,
		FText::Format(ReplaceGuidePresetFormat, FText::FromString(InPresetName)),
		LOCTEXT("ConfirmReplace", "Confirm Replace")
	);

	if (Response == EAppReturnType::Ok)
	{
		DataSubsystem->GetGuidePresetProvider().SaveGuidePreset(
			InPresetName,
			ViewportData->GuideData,
			GetVirtualSize()
		);
	}

	return FReply::Handled();
}

FReply SAvaLevelViewport::ExecuteRemoveGuidePreset(FString InPresetName)
{
	FSlateApplication::Get().DismissAllMenus();

	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(this);

	if (!DataSubsystem)
	{
		return FReply::Handled();
	}

	static const FText RemoveGuidePresetFormat = LOCTEXT("RemoveGuidePresetFormat", "Are you sure you want to remove guide preset:\n\n{0}");

	const EAppReturnType::Type Response = FMessageDialog::Open(
		EAppMsgType::OkCancel,
		FText::Format(RemoveGuidePresetFormat, FText::FromString(InPresetName)),
		LOCTEXT("ConfirmRemoval", "Confirm Removal")
	);

	if (Response == EAppReturnType::Ok)
	{
		DataSubsystem->GetGuidePresetProvider().RemoveGuidePreset(InPresetName);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
