// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaLevelViewportModule.h"
#include "AvaViewportDataSubsystem.h"
#include "SAvaLevelViewport.h"
#include "AvaViewportPostProcessManager.h"
#include "AvaViewportSettings.h"
#include "Engine/Texture.h"
#include "ScopedTransaction.h"
#include "ViewportClient/AvaLevelViewportClient.h"
#include "Visualizers/IAvaViewportBoundingBoxVisualizer.h"
#include "Visualizers/IAvaViewportPostProcessVisualizer.h"

#define LOCTEXT_NAMESPACE "SAvaLevelViewport"

void SAvaLevelViewport::ExecuteToggleChildActorLock()
{
	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return;
	}

	ViewportClient->SetChildActorsLocked(!ViewportClient->AreChildActorsLocked());
}

bool SAvaLevelViewport::IsPostProcessTypeEnabled(EAvaViewportPostProcessType InPostProcessType) const
{
	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid() || !ViewportClient->GetPostProcessManager().IsValid())
	{
		// Make sure none is toggled on if there's an error.
		return (InPostProcessType == EAvaViewportPostProcessType::None);
	}

	return ViewportClient->GetPostProcessManager()->GetType() == InPostProcessType;
}

bool SAvaLevelViewport::CanTogglePostProcessType(EAvaViewportPostProcessType InPostProcessType) const
{
	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return false;
	}

	if (!ViewportClient->GetPostProcessManager().IsValid())
	{
		return false;
	}

	if (InPostProcessType == EAvaViewportPostProcessType::None)
	{
		return true;
	}

	if (IsPostProcessTypeEnabled(InPostProcessType))
	{
		return true;
	}

	if (TSharedPtr<IAvaViewportPostProcessVisualizer> Visualizer = ViewportClient->GetPostProcessManager()->GetVisualizer(InPostProcessType))
	{
		return Visualizer->CanActivate(/* bInSilent */ true);
	}

	return false;
}

void SAvaLevelViewport::ExecuteTogglePostProcessType(EAvaViewportPostProcessType InPostProcessType)
{
	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		UE_LOGF(AvaLevelViewportLog, Warning, "SAvaLevelViewport::ExecuteTogglePostProcessType: Invalid viewport client.");
		return;
	}

	if (!ViewportClient->GetPostProcessManager().IsValid())
	{
		UE_LOGF(AvaLevelViewportLog, Warning, "SAvaLevelViewport::ExecuteTogglePostProcessType: Missing post process manager.");
		return;
	}

	BeginPostProcessInfoTransaction();

	ViewportClient->GetPostProcessManager()->SetType(InPostProcessType);
	ViewportClient->Invalidate();

	EndPostProcessInfoTransaction();
}

bool SAvaLevelViewport::CanToggleOverlay() const
{
	return true;
}

void SAvaLevelViewport::ExecuteToggleOverlay()
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->bEnableViewportOverlay = !AvaViewportSettings->bEnableViewportOverlay;
		AvaViewportSettings->SaveConfig();
		ApplySettings(AvaViewportSettings);
	}
}

bool SAvaLevelViewport::CanToggleSafeFrames() const
{
	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		return AvaViewportSettings->bEnableViewportOverlay;
	}

	return false;
}

void SAvaLevelViewport::ExecuteToggleSafeFrames()
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->bSafeFramesEnabled = !AvaViewportSettings->bSafeFramesEnabled;
		AvaViewportSettings->SaveConfig();
		ApplySettings(AvaViewportSettings);
	}
}

bool SAvaLevelViewport::CanToggleBoundingBox() const
{
	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return false;
	}

	return ViewportClient->GetBoundingBoxVisualizer()->GetOptimizationState() != EAvaViewportBoundingBoxOptimizationState::RenderNothing;
}

void SAvaLevelViewport::ExecuteToggleBoundingBox()
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->bEnableBoundingBoxes = !AvaViewportSettings->bEnableBoundingBoxes;
		AvaViewportSettings->SaveConfig();
		ApplySettings(AvaViewportSettings);
	}
}

bool SAvaLevelViewport::CanToggleGrid() const
{
	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		return AvaViewportSettings->bEnableViewportOverlay;
	}

	return false;
}

void SAvaLevelViewport::ExecuteToggleGrid()
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->bGridEnabled = !AvaViewportSettings->bGridEnabled;
		AvaViewportSettings->SaveConfig();
		ApplySettings(AvaViewportSettings);
	}
}

bool SAvaLevelViewport::CanToggleGridAlwaysVisible() const
{
	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		return AvaViewportSettings->bEnableViewportOverlay && AvaViewportSettings->bGridEnabled;
	}

	return false;
}

bool SAvaLevelViewport::IsGridAlwaysVisible() const
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		return AvaViewportSettings->bGridAlwaysVisible;
	}

	return false;
}

void SAvaLevelViewport::ExecuteToggleGridAlwaysVisible()
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->bGridAlwaysVisible = !AvaViewportSettings->bGridAlwaysVisible;
		AvaViewportSettings->SaveConfig();
	}
}

bool SAvaLevelViewport::CanIncreaseGridSize() const
{
	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		return AvaViewportSettings->bEnableViewportOverlay && AvaViewportSettings->bGridEnabled;
	}

	return false;
}

void SAvaLevelViewport::ExecuteIncreaseGridSize()
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->GridSize = FMath::Min(AvaViewportSettings->GridSize + 1, 256);
		AvaViewportSettings->SaveConfig();
	}
}

bool SAvaLevelViewport::CanDecreaseGridSize() const
{
	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		return AvaViewportSettings->bEnableViewportOverlay && AvaViewportSettings->bGridEnabled;
	}

	return false;
}

void SAvaLevelViewport::ExecuteDecreaseGridSize()
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->GridSize = FMath::Max(AvaViewportSettings->GridSize - 1, 1);
		AvaViewportSettings->SaveConfig();
	}
}

bool SAvaLevelViewport::CanChangeGridSize() const
{
	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		return AvaViewportSettings->bEnableViewportOverlay && AvaViewportSettings->bGridEnabled;
	}

	return false;
}

void SAvaLevelViewport::ExecuteSetGridSize(int32 InNewSize, bool bInCommit)
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->GridSize = FMath::Clamp(InNewSize, 1, 256);

		if (bInCommit)
		{
			AvaViewportSettings->SaveConfig();
		}
	}
}

bool SAvaLevelViewport::CanToggleSnapping() const
{
	return true;
}

void SAvaLevelViewport::ExecuteToggleSnapping()
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->SetSnapState(AvaViewportSettings->GetSnapState() ^ EAvaViewportSnapState::Global);
		AvaViewportSettings->SaveConfig();
	}
}

bool SAvaLevelViewport::CanToggleGridSnapping() const
{
	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		return EnumHasAllFlags(AvaViewportSettings->GetSnapState(), EAvaViewportSnapState::Global);
	}

	return false;
}

bool SAvaLevelViewport::IsGridSnappingEnabled() const
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		return AvaViewportSettings->HasSnapState(EAvaViewportSnapState::Grid);
	}

	return false;
}

void SAvaLevelViewport::ExecuteToggleGridSnapping()
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->SetSnapState(AvaViewportSettings->GetSnapState() ^ EAvaViewportSnapState::Grid);
		AvaViewportSettings->SaveConfig();
	}
}

bool SAvaLevelViewport::CanToggleScreenSnapping() const
{
	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		return EnumHasAllFlags(AvaViewportSettings->GetSnapState(), EAvaViewportSnapState::Global);
	}

	return false;
}

bool SAvaLevelViewport::IsScreenSnappingEnabled() const
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		return AvaViewportSettings->HasSnapState(EAvaViewportSnapState::Screen);
	}

	return false;
}

void SAvaLevelViewport::ExecuteToggleScreenSnapping()
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->SetSnapState(AvaViewportSettings->GetSnapState() ^ EAvaViewportSnapState::Screen);
		AvaViewportSettings->SaveConfig();
	}
}

bool SAvaLevelViewport::CanToggleActorSnapping() const
{
	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		return EnumHasAllFlags(AvaViewportSettings->GetSnapState(), EAvaViewportSnapState::Global);
	}

	return false;
}

bool SAvaLevelViewport::IsActorSnappingEnabled() const
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		return AvaViewportSettings->HasSnapState(EAvaViewportSnapState::Actor);
	}

	return false;
}

void SAvaLevelViewport::ExecuteToggleActorSnapping()
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->SetSnapState(AvaViewportSettings->GetSnapState() ^ EAvaViewportSnapState::Actor);
		AvaViewportSettings->SaveConfig();
	}
}

bool SAvaLevelViewport::CanToggleGuides() const
{
	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		return AvaViewportSettings->bEnableViewportOverlay;
	}

	return false;
}

void SAvaLevelViewport::ExecuteToggleGuides()
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->bGuidesEnabled = !AvaViewportSettings->bGuidesEnabled;
		AvaViewportSettings->SaveConfig();
		ApplySettings(AvaViewportSettings);
	}
}

bool SAvaLevelViewport::CanAddHorizontalGuide() const
{
	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		return AvaViewportSettings->bEnableViewportOverlay && AvaViewportSettings->bGuidesEnabled;
	}

	return false;
}

void SAvaLevelViewport::ExecuteAddHorizontalGuide()
{
	AddGuide(EOrientation::Orient_Horizontal, 0.5f);
}

bool SAvaLevelViewport::CanAddVerticalGuide() const
{
	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		return AvaViewportSettings->bEnableViewportOverlay && AvaViewportSettings->bGuidesEnabled;
	}

	return false;
}

void SAvaLevelViewport::ExecuteAddVerticalGuide()
{
	AddGuide(EOrientation::Orient_Vertical, 0.5f);
}

FString SAvaLevelViewport::GetBackgroundTextureObjectPath() const
{
	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return "";
	}

	if (!ViewportClient->GetPostProcessManager().IsValid())
	{
		return "";
	}

	FAvaViewportPostProcessInfo* PostProcessInfo = ViewportClient->GetPostProcessManager()->GetPostProcessInfo();

	if (!PostProcessInfo)
	{
		return "";
	}

	return PostProcessInfo->Texture.ToString();
}

void SAvaLevelViewport::OnBackgroundTextureChanged(const FAssetData& InAssetData)
{
	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return;
	}

	if (!ViewportClient->GetPostProcessManager().IsValid())
	{
		return;
	}

	FAvaViewportPostProcessInfo* PostProcessInfo = ViewportClient->GetPostProcessManager()->GetPostProcessInfo();

	if (!PostProcessInfo)
	{
		return;
	}

	BeginPostProcessInfoTransaction();

	PostProcessInfo->Texture = Cast<UTexture>(InAssetData.GetAsset());
	ViewportClient->GetPostProcessManager()->LoadPostProcessInfo();
	ViewportClient->Invalidate();

	EndPostProcessInfoTransaction();
}

float SAvaLevelViewport::GetBackgroundOpacity() const
{
	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return 1.f;
	}

	if (!ViewportClient->GetPostProcessManager().IsValid())
	{
		return 1.f;
	}

	FAvaViewportPostProcessInfo* PostProcessInfo = ViewportClient->GetPostProcessManager()->GetPostProcessInfo();

	return ViewportClient->GetPostProcessManager()->GetOpacity();
}

void SAvaLevelViewport::BeginPostProcessInfoTransaction()
{
	if (PostProcessInfoTransaction.IsValid())
	{
		return;
	}

	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return;
	}

	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(ViewportClient->GetViewportWorld());

	if (!DataSubsystem)
	{
		return;
	}

	FAvaViewportData* Data = DataSubsystem->GetData();

	if (!Data)
	{
		return;
	}

	PostProcessInfoTransaction = MakeShared<FScopedTransaction>(LOCTEXT("PostProcessSettingsChange", "Post Process Settings Change"));
	DataSubsystem->ModifyDataSource();
}

void SAvaLevelViewport::EndPostProcessInfoTransaction()
{
	PostProcessInfoTransaction.Reset();
}

void SAvaLevelViewport::OnBackgroundOpacitySliderBegin()
{
	BeginPostProcessInfoTransaction();
}

void SAvaLevelViewport::OnBackgroundOpacitySliderEnd(float InValue)
{
	EndPostProcessInfoTransaction();
}

void SAvaLevelViewport::OnBackgroundOpacityChanged(float InValue)
{
	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return;
	}

	if (!ViewportClient->GetPostProcessManager().IsValid())
	{
		return;
	}

	ViewportClient->GetPostProcessManager()->SetOpacity(InValue);
	ViewportClient->Invalidate();
}

void SAvaLevelViewport::OnBackgroundOpacityCommitted(float InValue, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnCleared)
	{
		return;
	}

	TSharedPtr<FAvaLevelViewportClient> ViewportClient = GetAvaLevelViewportClient();

	if (!ViewportClient.IsValid())
	{
		return;
	}

	if (!ViewportClient->GetPostProcessManager().IsValid())
	{
		return;
	}

	if (InCommitType == ETextCommit::OnEnter)
	{
		BeginPostProcessInfoTransaction();
	}

	ViewportClient->GetPostProcessManager()->SetOpacity(InValue);
	ViewportClient->Invalidate();

	if (InCommitType == ETextCommit::OnEnter)
	{
		EndPostProcessInfoTransaction();
	}
}

FString SAvaLevelViewport::GetTextureOverlayTextureObjectPath() const
{
	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		return AvaViewportSettings->TextureOverlayTexture.ToString();
	}

	return "";
}

void SAvaLevelViewport::OnTextureOverlayTextureChanged(const FAssetData& InAssetData)
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->TextureOverlayTexture = Cast<UTexture>(InAssetData.GetAsset());
		AvaViewportSettings->BroadcastSettingChanged(GET_MEMBER_NAME_CHECKED(UAvaViewportSettings, TextureOverlayTexture));
		AvaViewportSettings->SaveConfig();
	}
}

float SAvaLevelViewport::GetTextureOverlayOpacity() const
{
	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		return AvaViewportSettings->TextureOverlayOpacity;
	}

	return 0.f;
}

void SAvaLevelViewport::OnTextureOverlayOpacitySliderEnd(float InValue)
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->TextureOverlayOpacity = InValue;
		AvaViewportSettings->BroadcastSettingChanged(GET_MEMBER_NAME_CHECKED(UAvaViewportSettings, TextureOverlayOpacity));
		AvaViewportSettings->SaveConfig();
	}
}

void SAvaLevelViewport::OnTextureOverlayOpacityChanged(float InValue)
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->TextureOverlayOpacity = InValue;
		AvaViewportSettings->BroadcastSettingChanged(GET_MEMBER_NAME_CHECKED(UAvaViewportSettings, TextureOverlayOpacity));
	}
}

void SAvaLevelViewport::OnTextureOverlayOpacityCommitted(float InValue, ETextCommit::Type InCommitType)
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->TextureOverlayOpacity = InValue;
		AvaViewportSettings->BroadcastSettingChanged(GET_MEMBER_NAME_CHECKED(UAvaViewportSettings, TextureOverlayOpacity));
		AvaViewportSettings->SaveConfig();
	}
}

bool SAvaLevelViewport::CanToggleTextureOverlay() const
{
	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		return AvaViewportSettings->bEnableViewportOverlay;
	}

	return false;
}

void SAvaLevelViewport::ExecuteToggleTextureOverlay()
{
	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->bEnableTextureOverlay = !AvaViewportSettings->bEnableTextureOverlay;
		AvaViewportSettings->SaveConfig();
		ApplySettings(AvaViewportSettings);
	}
}

#undef LOCTEXT_NAMESPACE
