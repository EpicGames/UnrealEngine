// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaLevelViewportClient.h"
#include "AvaActorUtils.h"
#include "AvaViewportDataSubsystem.h"
#include "AvaViewportPostProcessManager.h"
#include "AvaViewportUtils.h"
#include "AvaVisibleArea.h"
#include "Bounds/AvaBoundsProviderSubsystem.h"
#include "Camera/CameraActor.h"
#include "CanvasTypes.h"
#include "Editor/UnrealEdEngine.h"
#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "FunctionalTestBase.h"
#include "GameFramework/Actor.h"
#include "Interaction/AvaCameraZoomController.h"
#include "Interaction/AvaChildTransformLockOperation.h"
#include "Interaction/AvaDragOperation.h"
#include "Interaction/AvaIsolateActorsOperation.h"
#include "Interaction/AvaSnapOperation.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "MouseDeltaTracker.h"
#include "FunctionalTestingModule.h"
#include "SAvaLevelViewport.h"
#include "SceneView.h"
#include "ScreenshotFunctionalTestAvalanche.h"
#include "Selection.h"
#include "Selection/AvaSelectionProviderSubsystem.h"
#include "SLevelViewport.h"
#include "UnrealEdGlobals.h"
#include "UObject/UObjectIterator.h"
#include "Visualizers/IAvaViewportBoundingBoxVisualizer.h"

#define LOCTEXT_NAMESPACE "AvaLevelViewportClient"

namespace UE::AvaLevelViewport::Private
{
	static TSet<FEditorViewportClient*> ViewportClientRegistry;

	class SEditorViewportExposeViewport : public SEditorViewport
	{
	public:
		TSharedPtr<SViewport> GetViewportWidget() const { return ViewportWidget; }
	};
}

bool FAvaLevelViewportClient::IsAvaLevelViewportClient(const FEditorViewportClient* InViewportClient)
{
	return InViewportClient && UE::AvaLevelViewport::Private::ViewportClientRegistry.Contains(InViewportClient);
}

FAvaLevelViewportClient::FAvaLevelViewportClient()
	: Super(nullptr)
	, bIsZoomedSceneView(true)
	, bLockChildActorsOnDrag(false)
{
	bDrawAxes     = false;
	bIsRealtime   = true;
	bDisableInput = false;
	ViewportType  = LVT_Perspective;
	SetAllowCinematicControl(true);
	BoundingBoxVisualizer = FAvaViewportBoundingBoxVisualizerProvider::CreateVisualizer();
	bUseControllingActorViewInfo = true;

	UE::AvaLevelViewport::Private::ViewportClientRegistry.Add(this);
}

FAvaLevelViewportClient::~FAvaLevelViewportClient()
{
	UE::AvaLevelViewport::Private::ViewportClientRegistry.Remove(this);

	if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor")))
	{
		LevelEditorModule->OnMapChanged().RemoveAll(this);
	}
}

void FAvaLevelViewportClient::Init()
{
	ZoomController = MakeShared<FAvaCameraZoomController>(SharedThis(this), 90.f);
	PostProcessManager = MakeShared<FAvaViewportPostProcessManager>(SharedThis(this));
	IsolateActorsOperation = MakeShared<FAvaIsolateActorsOperation>(SharedThis(this));

	PostProcessManager->LoadPostProcessInfo();

	LoadActiveCameraComponent();

	if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor")))
	{
		LevelEditorModule->OnMapChanged().AddSP(this, &FAvaLevelViewportClient::OnMapChanged);
	}
}

bool FAvaLevelViewportClient::HasCachedViewportSize() const
{
	return FAvaViewportUtils::IsValidViewportSize(ViewportGeometry.WidgetSize);
}

FVector2f FAvaLevelViewportClient::GetCachedViewportSize() const
{
	return ViewportGeometry.CameraBounds.GetSize();
}

FVector2f FAvaLevelViewportClient::GetCachedViewportOffset() const
{
	return ViewportGeometry.CameraBounds.Min;
}

bool FAvaLevelViewportClient::HasVirtualViewportSize() const
{
	return ViewportGeometry.VirtualSize.X > 0 && ViewportGeometry.VirtualSize.Y > 0;
}

FIntPoint FAvaLevelViewportClient::GetVirtualViewportSize() const
{
	if (!FAvaViewportUtils::IsValidViewportSize(ViewportGeometry.VirtualSize))
	{
		const FVector2f WidgetSize = GetViewportSize();

		FIntPoint VirtualSize;
		VirtualSize.X = FMath::RoundToInt(WidgetSize.X);
		VirtualSize.Y = FMath::RoundToInt(WidgetSize.Y);

		return VirtualSize;
	}

	return ViewportGeometry.VirtualSize;
}

FVector2f FAvaLevelViewportClient::GetViewportOffset() const
{
	return GetCachedViewportOffset();
}

FVector2f FAvaLevelViewportClient::GetViewportWidgetSize() const
{
	return ViewportGeometry.WidgetSize;
}

float FAvaLevelViewportClient::GetViewportDPIScale() const
{
	return ViewportGeometry.WidgetDPIScale;
}

void FAvaLevelViewportClient::SetVirtualViewportSize(const FIntPoint& InVirtualSize)
{
	ViewportGeometry.VirtualSize = InVirtualSize;
}

void FAvaLevelViewportClient::SetViewportWidget(TSharedPtr<SAvaLevelViewport> InLevelViewport)
{
	EditorViewportWidget = InLevelViewport;
	AvaLevelViewportWeak = InLevelViewport;
}

UCameraComponent* FAvaLevelViewportClient::GetCameraComponentViewTarget() const
{
	if (GetViewTarget() || GetCinematicViewTarget())
	{
		return ActiveCameraComponentWeak.Get();
	}

	return nullptr;
}

AActor* FAvaLevelViewportClient::GetCinematicViewTarget() const
{
	if (bLockedCameraView)
	{
		return GetCinematicActorLock().GetLockedActor();
	}

	return nullptr;
}

AActor* FAvaLevelViewportClient::GetViewTarget() const
{
	if (bLockedCameraView)
	{
		return GetActorLock().GetLockedActor();
	}

	return nullptr;
}

void FAvaLevelViewportClient::SetViewTarget(TWeakObjectPtr<AActor> InViewTarget)
{
	UpdateActiveCameraComponent(InViewTarget.Get());
	SetCinematicActorLock(nullptr);

	UCameraComponent* CameraComponent = ActiveCameraComponentWeak.Get();
	AActor* ViewTarget = CameraComponent ? CameraComponent->GetOwner() : nullptr;
	SetActorLock(ViewTarget);
}

void FAvaLevelViewportClient::SetCinematicViewTarget(AActor* InCinematicViewTarget)
{
	UpdateActiveCameraComponent(InCinematicViewTarget);
	SetActorLock(nullptr);

	AActor* ViewTarget = ActiveCameraComponentWeak.IsValid() ? ActiveCameraComponentWeak->GetOwner() : nullptr;

	SetCinematicActorLock(ViewTarget);
}

void FAvaLevelViewportClient::OnCameraCut(AActor* InTarget, bool bInJumpCut)
{
	if (!IsValid(InTarget))
	{
		return;
	}

	SetCinematicViewTarget(InTarget);

	if (bInJumpCut)
	{
		SetIsCameraCut();
	}
}

FAvaVisibleArea FAvaLevelViewportClient::GetVisibleArea() const
{
	return ZoomController->GetCachedVisibleArea();
}

FAvaVisibleArea FAvaLevelViewportClient::GetZoomedVisibleArea() const
{
	return ZoomController->GetCachedZoomedVisibleArea();
}

FAvaVisibleArea FAvaLevelViewportClient::GetVirtualVisibleArea() const
{
	FAvaVisibleArea VisibleArea = GetVisibleArea();

	if (HasVirtualViewportSize() && HasCachedViewportSize())
	{
		const FVector2f VirtualRatio = GetVirtualViewportScale();
		VisibleArea.AbsoluteSize *= VirtualRatio;
		VisibleArea.VisibleSize *= VirtualRatio;
		VisibleArea.Offset *= VirtualRatio;

		// Just in case they are different, give an estimated average.
		VisibleArea.DPIScale *= VirtualRatio.X * 0.5f + VirtualRatio.Y * 0.5f;
	}

	return VisibleArea;
}

FAvaVisibleArea FAvaLevelViewportClient::GetVirtualZoomedVisibleArea() const
{
	FAvaVisibleArea VisibleArea = GetZoomedVisibleArea();

	if (HasVirtualViewportSize() && HasCachedViewportSize())
	{
		const FVector2f VirtualRatio = GetVirtualViewportScale();

		VisibleArea.AbsoluteSize *= VirtualRatio;
		VisibleArea.VisibleSize *= VirtualRatio;
		VisibleArea.Offset *= VirtualRatio;

		// Just in case they are different, give an estimated average.
		VisibleArea.DPIScale *= VirtualRatio.X * 0.5f + VirtualRatio.Y * 0.5f;
	}

	return VisibleArea;
}

FVector2f FAvaLevelViewportClient::GetUnconstrainedViewportMousePosition() const
{
	const FVector2f AbsoluteMousePosition = FSlateApplication::Get().GetCursorPos();

	TSharedPtr<SEditorViewport> ViewportWidget = GetEditorViewportWidget();

	if (!ViewportWidget.IsValid())
	{
		return AbsoluteMousePosition;
	}

	TSharedPtr<SViewport> InnerViewportWidget = static_cast<UE::AvaLevelViewport::Private::SEditorViewportExposeViewport*>(ViewportWidget.Get())->GetViewportWidget();

	if (!InnerViewportWidget.IsValid())
	{
		return AbsoluteMousePosition;
	}

    return (AbsoluteMousePosition - InnerViewportWidget->GetTickSpaceGeometry().GetAbsolutePosition());
}

FVector2f FAvaLevelViewportClient::GetConstrainedViewportMousePosition() const
{
	const FVector2f AbsoluteMousePosition = FSlateApplication::Get().GetCursorPos();

	TSharedPtr<SEditorViewport> ViewportWidget = GetEditorViewportWidget();

	if (!ViewportWidget.IsValid())
	{
		return AbsoluteMousePosition;
	}

	TSharedPtr<SViewport> InnerViewportWidget = static_cast<UE::AvaLevelViewport::Private::SEditorViewportExposeViewport*>(ViewportWidget.Get())->GetViewportWidget();

	if (!InnerViewportWidget.IsValid())
	{
		return AbsoluteMousePosition;
	}

	const FVector2f ViewportAbsolutePosition = InnerViewportWidget->GetTickSpaceGeometry().GetAbsolutePosition();

	return (AbsoluteMousePosition
			- ViewportAbsolutePosition
			- ViewportGeometry.CameraBounds.Min * ViewportGeometry.WidgetDPIScale);
}

FVector2f FAvaLevelViewportClient::GetUnconstrainedZoomedViewportMousePosition() const
{
	const FVector2f ViewportMousePosition = GetUnconstrainedViewportMousePosition();
	const FAvaVisibleArea& VisibleArea = GetZoomedVisibleArea();

	return VisibleArea.GetDPIScaledAbsolutePosition(ViewportMousePosition);
}

FVector2f FAvaLevelViewportClient::GetConstrainedZoomedViewportMousePosition() const
{
	const FVector2f ViewportMousePosition = GetConstrainedViewportMousePosition();
	const FAvaVisibleArea& VisibleArea = GetZoomedVisibleArea();

	return VisibleArea.GetDPIScaledAbsolutePosition(ViewportMousePosition);
}

TSharedPtr<FAvaSnapOperation> FAvaLevelViewportClient::StartSnapOperation()
{
	if (TSharedPtr<FAvaSnapOperation> SnapOperation = SnapOperationWeak.Pin())
	{
		EndSnapOperation(SnapOperation.Get());
	}

	TSharedRef<FAvaSnapOperation> SnapOperation = MakeShared<FAvaSnapOperation>(this);
	SnapOperationWeak = SnapOperation;

	return SnapOperation;
}

bool FAvaLevelViewportClient::EndSnapOperation(FAvaSnapOperation* InSnapOperation)
{
	if (TSharedPtr<FAvaSnapOperation> SnapOperation = SnapOperationWeak.Pin())
	{
		if (!InSnapOperation || SnapOperation.Get() == InSnapOperation)
		{
			SnapOperationWeak.Reset();
			return true;
		}
	}

	return false;
}

void FAvaLevelViewportClient::OnActorSelectionChanged()
{
	BoundingBoxVisualizer->ResetOptimizationState();
}

ELevelViewportType FAvaLevelViewportClient::GetViewportType() const
{
	return ControllingActorViewInfo.ProjectionMode == ECameraProjectionMode::Perspective
		? LVT_Perspective
		: LVT_OrthoFreelook;
}

void FAvaLevelViewportClient::Draw(const FSceneView* InView, FPrimitiveDrawInterface* InPDI)
{
	FLevelEditorViewportClient::Draw(InView, InPDI);

	// There is no event triggered for unpiloting a camera, so we need to check.
	CheckActiveCameraComponent();

	if (InPDI)
	{
		if (UWorld* ViewportClientWorld = GetWorld())
		{
			if (UAvaSelectionProviderSubsystem* SelectionProvider = ViewportClientWorld->GetSubsystem<UAvaSelectionProviderSubsystem>())
			{
				if (UAvaBoundsProviderSubsystem* BoundsProvider = ViewportClientWorld->GetSubsystem<UAvaBoundsProviderSubsystem>())
				{
					BoundingBoxVisualizer->Draw(
						*SelectionProvider,
						*BoundsProvider,
						*InPDI
					);
				}
			}
		}
	}
}

void FAvaLevelViewportClient::DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas)
{
	ViewportGeometry.WidgetDPIScale = FMath::IsNearlyZero(Canvas.GetDPIScale()) ? 1.f : Canvas.GetDPIScale()
		* FSlateApplication::Get().GetApplicationScale();

	const FIntPoint CanvasSize = Canvas.GetViewRect().Size();

	if (FAvaViewportUtils::IsValidViewportSize(CanvasSize))
	{
		ViewportGeometry.WidgetSize = static_cast<FVector2f>(CanvasSize);
		ViewportGeometry.WidgetSize /= ViewportGeometry.WidgetDPIScale;

		ViewportGeometry.CameraBounds.Min = FVector2f::ZeroVector;
		ViewportGeometry.CameraBounds.Max = ViewportGeometry.WidgetSize;

		AspectRatio = ViewportGeometry.WidgetSize.X / ViewportGeometry.WidgetSize.Y;

		if (UCameraComponent* CachedCameraComponent = ActiveCameraComponentWeak.Get())
		{
			const float DesiredAspectRatio = CachedCameraComponent->AspectRatio;

			if (!FMath::IsNearlyEqual(AspectRatio, DesiredAspectRatio))
			{
				if (AspectRatio > DesiredAspectRatio)
				{
					const float DesiredWidth = ViewportGeometry.WidgetSize.Y * DesiredAspectRatio;
					const float Slack = (ViewportGeometry.WidgetSize.X - DesiredWidth) * 0.5f;
					ViewportGeometry.CameraBounds.Min.X += Slack;
					ViewportGeometry.CameraBounds.Max.X -= Slack;
				}
				else
				{
					const float DesiredHeight = ViewportGeometry.WidgetSize.X / DesiredAspectRatio;
					const float Slack = (ViewportGeometry.WidgetSize.Y - DesiredHeight) * 0.5f;
					ViewportGeometry.CameraBounds.Min.Y += Slack;
					ViewportGeometry.CameraBounds.Max.Y -= Slack;
				}
			}
		}
	}

	ZoomController->UpdateVisibleAreas();

	Super::DrawCanvas(InViewport, View, Canvas);
}

FMatrix FAvaLevelViewportClient::CalcViewRotationMatrixForControllingActorView(const FRotator& InViewRotation) const
{
	return CalcViewRotationMatrix(InViewRotation);
}

bool FAvaLevelViewportClient::ProcessScreenShots(FViewport* InViewport)
{
	bool bIsRunningAvaScreenshotTest = false;

	for (AScreenshotFunctionalTestAvalanche* Test : TObjectRange<AScreenshotFunctionalTestAvalanche>())
	{
		if (!Test->IsTemplate() && Test->IsRunning())
		{
			return false;
		}
	}

	/*
	if (IFunctionalTestingModule::Get().IsRunning())
	{
		return false;
	}
	*/

	return FEditorViewportClient::ProcessScreenShots(InViewport);
}

FSceneView* FAvaLevelViewportClient::CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex)
{
	UCameraComponent* CachedCameraComponent = nullptr;
	bool bHasViewTarget = false;

	if (bLockedCameraView)
	{
		CachedCameraComponent = ActiveCameraComponentWeak.Get();

		if (CachedCameraComponent)
		{
			if (AActor* LockedCinematicViewTarget = GetCinematicViewTarget())
			{
				if (CachedCameraComponent->GetOwner() == LockedCinematicViewTarget)
				{
					bHasViewTarget = true;
				}
				else
				{
					SetCinematicViewTarget(LockedCinematicViewTarget);
					CachedCameraComponent = ActiveCameraComponentWeak.Get();
					bHasViewTarget = !!CachedCameraComponent;
				}				
			}
			else if (AActor* LockedViewTarget = GetViewTarget())
			{
				if (CachedCameraComponent->GetOwner() == LockedViewTarget)
				{
					bHasViewTarget = true;
				}
				else
				{
					SetViewTarget(LockedViewTarget);
					CachedCameraComponent = ActiveCameraComponentWeak.Get();
					bHasViewTarget = !!CachedCameraComponent;
				}				
			}

			if (bHasViewTarget)
			{
				ControllingActorViewInfo.Location = CachedCameraComponent->GetComponentLocation();
				ControllingActorViewInfo.Rotation = CachedCameraComponent->GetComponentRotation();
			}
		}
	}

	const bool bIsZoomPossible = ZoomController->CanZoom();

	if (!bIsZoomPossible)
	{
		ViewFOV = ZoomController->GetFallbackFOV();
		ControllingActorViewInfo.OffCenterProjectionOffset = FVector2D::ZeroVector;
	}
	else
	{
		if (bIsZoomedSceneView)
		{
			ViewFOV = ZoomController->GetFOV();
			ControllingActorViewInfo.OffCenterProjectionOffset = static_cast<FVector2D>(ZoomController->GetCameraProjectionOffset());
		}
		else if (CachedCameraComponent)
		{
			ViewFOV = CachedCameraComponent->FieldOfView;
			ControllingActorViewInfo.OffCenterProjectionOffset = FVector2D::ZeroVector;
		}
		else
		{
			ViewFOV = ZoomController->GetFallbackFOV();
			ControllingActorViewInfo.OffCenterProjectionOffset = FVector2D::ZeroVector;
		}
	}

	// If the camera bounds are offset on X it means we need to increase the apparent horizontal fov to compensate.
	if (!FMath::IsNearlyZero(ViewportGeometry.CameraBounds.Min.X))
	{
		const float FocalLength = ViewportGeometry.CameraBounds.GetSize().X * 0.5f / FMath::Tan(FMath::DegreesToRadians(ViewFOV * 0.5f));

		if (!FMath::IsNearlyZero(FocalLength))
		{
			ViewFOV = 2.f * FMath::RadiansToDegrees(FMath::Atan(ViewportGeometry.WidgetSize.X * 0.5f / FocalLength));
		}
	}

	bUseControllingActorViewInfo = bIsZoomPossible;

	ControllingActorViewInfo.FOV = ViewFOV;
	ControllingActorViewInfo.AspectRatio = AspectRatio;
	ControllingActorViewInfo.bConstrainAspectRatio = false;

	FSceneView* const SceneView = Super::CalcSceneView(ViewFamily, StereoViewIndex);

	SceneView->FOV = ViewFOV;
	SceneView->DesiredFOV = ViewFOV;

	IsolateActorsOperation->AddIsolatedActorPrimitives(SceneView);

	PostProcessManager->UpdateSceneView(SceneView);

	return SceneView;
}

void FAvaLevelViewportClient::TrackingStarted(const FInputEventState& InInputState, bool bIsDraggingWidget, bool bNudge)
{
	const bool bIsLeftMouseButtonPressed = InInputState.IsLeftMouseButtonPressed();
	const bool bIsMiddleMouseButtonPressed = InInputState.IsMiddleMouseButtonPressed();
	const bool bIsRightMouseButtonPressed = InInputState.IsRightMouseButtonPressed();
	const bool bIsAltModifierPressed = InInputState.IsAltButtonPressed();

	if (!bIsDraggingWidget && bIsMiddleMouseButtonPressed && !bIsLeftMouseButtonPressed && !bIsRightMouseButtonPressed
		&& !bIsAltModifierPressed)
	{
		ZoomController->StartPanning();
		return;
	}

	if (!bDuplicateActorsInProgress && !bOnlyMovedPivot && bIsDraggingWidget && !bIsAltModifierPressed)
	{
		if (GetWidgetMode() == UE::Widget::WM_Translate)
		{
			const bool bHasVisualizer = GUnrealEd ? GUnrealEd->ComponentVisManager.IsActive() : false;

			if (!bHasVisualizer)
			{
				DragOperation = MakeShared<FAvaDragOperation>(SharedThis(this), bLockChildActorsOnDrag);

				// Check that the operation correctly init'd, not if the sharedptr is valid (-> vs .)
				if (!DragOperation->IsValid())
				{
					DragOperation.Reset();
				}
			}
		}

		if (bLockChildActorsOnDrag)
		{
			ChildTransformLockOperation = MakeShared<FAvaChildTransformLockOperation>(SharedThis(this));
		}
	}

	Super::TrackingStarted(InInputState, bIsDraggingWidget, bNudge);
}

void FAvaLevelViewportClient::TrackingStopped()
{
	ZoomController->EndPanning();

	Super::TrackingStopped();

	if (ChildTransformLockOperation.IsValid())
	{
		ChildTransformLockOperation->Restore();
		ChildTransformLockOperation.Reset();
	}

	DragOperation.Reset();
}

bool FAvaLevelViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
{
	const bool bIsZoomPossible = ZoomController->CanZoom();

	if (bIsZoomPossible)
	{
		if (EventArgs.Key == EKeys::MouseScrollUp)
		{
			if (EventArgs.Event == IE_Pressed)
			{
				ZoomController->ZoomInCursor();
			}

			return true;
		}

		if (EventArgs.Key == EKeys::MouseScrollDown)
		{
			if (EventArgs.Event == IE_Pressed)
			{
				ZoomController->ZoomOutCursor();
			}

			return true;
		}

		if (!bDraggingByHandle && EventArgs.Key == EKeys::MiddleMouseButton)
		{
			if (EventArgs.Event == EInputEvent::IE_Pressed)
			{
				ZoomController->StartPanning();
			}
			else if (EventArgs.Event == EInputEvent::IE_Released)
			{
				ZoomController->EndPanning();
			}
		}
	}

	return Super::InputKey(EventArgs);
}

void FAvaLevelViewportClient::UpdateMouseDelta()
{
	if (ZoomController->IsPanning())
	{
		const FVector DragDelta = MouseDeltaTracker->GetDelta();
		MouseDeltaTracker->ReduceBy(DragDelta);

		// There's some disconnect between movement distance of the mouse and how much it pans... and I'm not sure what it is.
		ZoomController->PanAdjustZoomed(FVector2f(-DragDelta.X, DragDelta.Y) * 2.f);
	}
	else
	{
		const FVector DragDelta = MouseDeltaTracker->GetDelta();

		if (DragDelta.IsNearlyZero())
		{
			return;
		}

		if (DragOperation.IsValid())
		{
			DragOperation->PreMouseUpdate();
		}

		Super::UpdateMouseDelta();

		if (DragOperation.IsValid())
		{
			DragOperation->PostMouseUpdate();
		}

		if (ChildTransformLockOperation.IsValid())
		{
			ChildTransformLockOperation->Restore();
		}
	}
}

EMouseCursor::Type FAvaLevelViewportClient::GetCursor(FViewport* InViewport, int32 InX, int32 InY)
{
	if (ZoomController->IsPanning())
	{
		return EMouseCursor::GrabHandClosed;
	}

	return FLevelEditorViewportClient::GetCursor(InViewport, InX, InY);
}

FTransform FAvaLevelViewportClient::GetViewportViewTransform() const
{
	// Uses CalcSceneView which, for some reason, isn't const.
	return FAvaViewportUtils::GetViewportViewTransform(const_cast<FAvaLevelViewportClient*>(this));
}

float FAvaLevelViewportClient::GetZoomedFOV() const
{
	return ZoomController->GetFOV();
}

float FAvaLevelViewportClient::GetUnZoomedFOV() const
{
	return ZoomController->GetDefaultFOV();
}

FVector2f FAvaLevelViewportClient::GetViewportSize() const
{
	return GetCachedViewportSize();
}

FSceneView* FAvaLevelViewportClient::CalcNonZoomedSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex)
{
	TGuardValue Guard(bIsZoomedSceneView, false);
	return CalcSceneView(ViewFamily, StereoViewIndex);
}

void FAvaLevelViewportClient::LoadActiveCameraComponent()
{
	if (UWorld* ViewportWorld = GetWorld())
	{
		if (UAvaViewportDataSubsystem* ViewportDataSubsystem = ViewportWorld->GetSubsystem<UAvaViewportDataSubsystem>())
		{
			if (FAvaViewportData* ViewportData = ViewportDataSubsystem->GetData())
			{
				if (AActor* Actor = ViewportData->PilotedCamera.Get())
				{
					SetViewTarget(Actor);
				}
			}
		}
	}
}

void FAvaLevelViewportClient::CheckActiveCameraComponent()
{
	if (AActor* ActorViewTarget = GetViewTarget())
	{
		UpdateActiveCameraComponent(ActorViewTarget);
	}
	else if (AActor* CinematicViewTarget = GetCinematicViewTarget())
	{
		UpdateActiveCameraComponent(CinematicViewTarget);
	}
	else
	{
		UpdateActiveCameraComponent(nullptr);
	}
}

UCameraComponent* FAvaLevelViewportClient::UpdateActiveCameraComponent(AActor* InViewTarget)
{
	if (!IsValid(InViewTarget))
	{
		SetActiveCameraComponent(nullptr);
		return nullptr;
	}

	if (UCameraComponent* ActiveCameraComponent = ActiveCameraComponentWeak.Get())
	{
		if (ActiveCameraComponent->GetOwner() == InViewTarget && ActiveCameraComponent->IsActive())
		{
			SetActiveCameraComponent(ActiveCameraComponent);
			return ActiveCameraComponent;
		}

		ActiveCameraComponentWeak.Reset();
	}

	TArray<UCameraComponent*> CameraComponents;
	InViewTarget->GetComponents<UCameraComponent>(CameraComponents, false);

	for (UCameraComponent* CameraComponent : CameraComponents)
	{
		if (IsValid(CameraComponent) && CameraComponent->IsActive())
		{
			SetActiveCameraComponent(CameraComponent);
			return CameraComponent;
		}
	}

	SetActiveCameraComponent(nullptr);
	return nullptr;
}

void FAvaLevelViewportClient::SetActiveCameraComponent(UCameraComponent* InCameraComponent)
{
	AActor* Actor = IsValid(InCameraComponent) ? InCameraComponent->GetOwner() : nullptr;

	if (IsValid(Actor))
	{
		bLockedCameraView = true;
		ActiveCameraComponentWeak = InCameraComponent;
	}
	else
	{
		bLockedCameraView = false;
		ActiveCameraComponentWeak.Reset();
	}

	if (UWorld* ViewportWorld = GetWorld())
	{
		if (UAvaViewportDataSubsystem* ViewportDataSubsystem = ViewportWorld->GetSubsystem<UAvaViewportDataSubsystem>())
		{
			if (FAvaViewportData* ViewportData = ViewportDataSubsystem->GetData())
			{
				AActor* CurrentActor = ViewportData->PilotedCamera.Get();

				if (Actor != CurrentActor)
				{
					ViewportDataSubsystem->ModifyDataSource();
					ViewportData->PilotedCamera = Actor;
				}
			}
		}
	}
}

void FAvaLevelViewportClient::OnMapChanged(UWorld* InWorld, EMapChangeType InChangeType)
{
	switch (InChangeType)
	{
		case EMapChangeType::NewMap:
		case EMapChangeType::LoadMap:
		{
			if (PostProcessManager.IsValid())
			{
				PostProcessManager->LoadPostProcessInfo();
			}
			LoadActiveCameraComponent();
			break;
		}
		case EMapChangeType::TearDownWorld:
		{
			SetActorLock(nullptr);
			break;
		}
		default:
			// Do nothing
			break;
	}
}

#undef LOCTEXT_NAMESPACE
