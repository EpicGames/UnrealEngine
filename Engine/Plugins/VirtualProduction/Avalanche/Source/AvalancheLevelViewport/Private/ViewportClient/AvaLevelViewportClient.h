// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "AvaViewportGeometry.h"
#include "LevelEditorViewport.h"
#include "ViewportClient/AvaViewportClientUtilityProvider.h"
#include "Math/MathFwd.h"
#include "Math/Vector2D.h"

class AActor;
class FAvaCameraZoomController;
class FAvaChildTransformLockOperation;
class FAvaDragOperation;
class FAvaIsolateActorsOperation;
class FAvaSnapOperation;
class FAvaViewportPostProcessManager;
class FEditorViewportClient;
class IAvaBoundsProviderInterface;
class IAvaViewportBoundingBoxVisualizer;
class SAvaLevelViewport;
class UCameraComponent;
struct FAvaSnapPoint;
struct FAvaVisibleArea;

UE_AVA_TYPE_EXTERNAL(FLevelEditorViewportClient);

class FAvaLevelViewportClient : public FLevelEditorViewportClient, public FAvaViewportClientUtilityProvider, public TSharedFromThis<FAvaLevelViewportClient>
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaLevelViewportClient, FLevelEditorViewportClient, FAvaViewportClientUtilityProvider)

	static bool IsAvaLevelViewportClient(const FEditorViewportClient* InViewportClient);

	FAvaLevelViewportClient();
	virtual ~FAvaLevelViewportClient() override;

	void Init();

	const FAvaViewportGeometry& GetViewportGeometry() const { return ViewportGeometry; }

	bool HasCachedViewportSize() const;
	FVector2f GetCachedViewportSize() const;
	FVector2f GetCachedViewportOffset() const;

	bool HasVirtualViewportSize() const;
	void SetVirtualViewportSize(const FIntPoint& InVirtualSize);
	// Set method is below

	void SetViewportWidget(TSharedPtr<SAvaLevelViewport> InLevelViewport);

	virtual FSceneView* CalcNonZoomedSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex = INDEX_NONE);

	bool AreChildActorsLocked() const { return bLockChildActorsOnDrag; }
	void SetChildActorsLocked(bool bInLockChildActors) { bLockChildActorsOnDrag = bInLockChildActors; }

	TSharedRef<IAvaViewportBoundingBoxVisualizer> GetBoundingBoxVisualizer() const { return BoundingBoxVisualizer.ToSharedRef(); }

	TSharedRef<FAvaIsolateActorsOperation> GetIsolateActorsOperation() const { return IsolateActorsOperation.ToSharedRef(); }

	AActor* GetCinematicViewTarget() const;

	//~ Begin FEditorViewportClient
	virtual ELevelViewportType GetViewportType() const override;
	virtual void SetViewportType(ELevelViewportType InViewportType) override {}
	virtual void Draw(const FSceneView* InView, FPrimitiveDrawInterface* InPDI) override;
	virtual void DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas) override;
	virtual FSceneView* CalcSceneView(FSceneViewFamily* ViewFamily, const int32 StereoViewIndex = INDEX_NONE) override;
	virtual void TrackingStarted(const FInputEventState& InInputState, bool bIsDraggingWidget, bool bNudge) override;
	virtual void TrackingStopped() override;
	virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	virtual void UpdateMouseDelta() override;
	virtual EMouseCursor::Type GetCursor(FViewport* InViewport, int32 InX, int32 InY) override;
	virtual FMatrix CalcViewRotationMatrixForControllingActorView(const FRotator& InViewRotation) const override;
	virtual bool ProcessScreenShots(FViewport* InViewport) override;
	//~ End FEditorViewportClient

	//~ Begin IAvaViewportWorldCoordinateConverter
	virtual FVector2f GetViewportSize() const override;
	virtual FTransform GetViewportViewTransform() const override;
	//~ End IAvaViewportWorldCoordinateConverter

	//~ Begin IAvaViewportClient
	virtual bool IsMotionDesignViewport() const override { return true; }
	virtual const FEditorViewportClient* AsEditorViewportClient() const override { return this; }
	virtual bool SupportsZoom() const override { return true; }
	virtual float GetZoomedFOV() const override;
	virtual float GetUnZoomedFOV() const override;
	virtual FIntPoint GetVirtualViewportSize() const override;
	virtual FVector2f GetViewportOffset() const override;
	virtual FVector2f GetViewportWidgetSize() const override;
	virtual float GetViewportDPIScale() const override;
	virtual FAvaVisibleArea GetVisibleArea() const override;
	virtual FAvaVisibleArea GetZoomedVisibleArea() const override;
	virtual FAvaVisibleArea GetVirtualVisibleArea() const override;
	virtual FAvaVisibleArea GetVirtualZoomedVisibleArea() const override;
	virtual FVector2f GetUnconstrainedViewportMousePosition() const override;
	virtual FVector2f GetConstrainedViewportMousePosition() const override;
	virtual FVector2f GetUnconstrainedZoomedViewportMousePosition() const override;
	virtual FVector2f GetConstrainedZoomedViewportMousePosition() const override;
	virtual TSharedPtr<FAvaSnapOperation> GetSnapOperation() const override { return SnapOperationWeak.Pin(); };
	virtual TSharedPtr<FAvaSnapOperation> StartSnapOperation() override;
	virtual bool EndSnapOperation(FAvaSnapOperation* InSnapOperation) override;
	virtual void OnActorSelectionChanged() override;
	virtual TSharedPtr<FAvaCameraZoomController> GetZoomController() const override { return ZoomController; }
	virtual UCameraComponent* GetCameraComponentViewTarget() const override;
	virtual AActor* GetViewTarget() const override;
	virtual void SetViewTarget(TWeakObjectPtr<AActor> InViewTarget) override;
	virtual void OnCameraCut(AActor* InTarget, bool bInJumpCut) override;
	virtual UWorld* GetViewportWorld() const { return GetWorld(); }
	virtual TSharedPtr<FAvaViewportPostProcessManager> GetPostProcessManager() const override { return PostProcessManager; }
	//~ End IAvaViewportClient

protected:
	TWeakPtr<SAvaLevelViewport> AvaLevelViewportWeak;

	FAvaViewportGeometry ViewportGeometry;

	TSharedPtr<FAvaCameraZoomController> ZoomController;

	TSharedPtr<IAvaViewportBoundingBoxVisualizer> BoundingBoxVisualizer;

	TWeakObjectPtr<UCameraComponent> ActiveCameraComponentWeak;

	TWeakPtr<FAvaSnapOperation> SnapOperationWeak;

	TSharedPtr<FAvaDragOperation> DragOperation;

	TSharedPtr<FAvaChildTransformLockOperation> ChildTransformLockOperation;

	TSharedPtr<FAvaViewportPostProcessManager> PostProcessManager;

	TSharedPtr<FAvaIsolateActorsOperation> IsolateActorsOperation;

	bool bIsZoomedSceneView;

	bool bLockChildActorsOnDrag;

	void SetCinematicViewTarget(AActor* InCinematicViewTarget);

	void LoadActiveCameraComponent();

	void CheckActiveCameraComponent();

	UCameraComponent* UpdateActiveCameraComponent(AActor* InViewTarget);

	void SetActiveCameraComponent(UCameraComponent* InCameraComponent);

	void OnMapChanged(UWorld* InWorld, EMapChangeType InChangeType);
};
