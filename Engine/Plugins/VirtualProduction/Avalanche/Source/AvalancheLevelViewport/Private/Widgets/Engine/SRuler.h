// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Layout/Clipping.h"
#include "Math/Vector2D.h"
#include "Misc/Optional.h"
#include "Types/SlateEnums.h"
#include "Types/WidgetMouseEventsDelegate.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
struct FGeometry;
struct FPointerEvent;

namespace UE::AvaLevelViewport::Private
{

class SRuler : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SRuler)
		: _Orientation(Orient_Horizontal)
		, _OnMouseButtonDown()
	{
		_Clipping = EWidgetClipping::ClipToBounds;
	}

	SLATE_ARGUMENT(EOrientation, Orientation)

	SLATE_EVENT(FPointerEventHandler, OnMouseButtonDown)

	SLATE_END_ARGS()


	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs );

	/**
	* Determines the optimal spacing between tick marks in the slider for a given pixel density
	* Increments until a minimum amount of slate units specified by MinTick is reached
	*
	* @param InPixelsPerInput	The density of pixels between each input
	* @param MinTick			The minimum slate units per tick allowed
	* @param MinTickSpacing	The minimum tick spacing in time units allowed
	* @return the optimal spacing in time units
	*/
	float DetermineOptimalSpacing(float InPixelsPerInput, uint32 MinTick, float MinTickSpacing) const;

	/**
	 * Sets the rules to use when rendering the ruler.
	 *
	 * @param AbsoluteOrigin The origin of the ruler in absolute window space.
	 * @param SlateToUnitScale The scaler applied to slate units to get the same number 
	 * of units currently being displayed by the rule marks.
	 */
	void SetRuling(FVector2D AbsoluteOrigin, float SlateToUnitScale);

	/**
	 * Sets the cursor position in window absolute space.
	 *
	 * @param AbsoluteCursor The position of the cursor in absolute window space.
	 */
	void SetCursor(TOptional<FVector2D> AbsoluteCursor);

protected:
	// SWidget interface
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

private:
	// forward declared as class members to prevent name collision with similar types defined in other units
	struct FDrawTickArgs;
	struct FScrubRangeToScreen;

	/** Draws the ticks on the track */
	int32 DrawTicks(FSlateWindowElementList& OutDrawElements, const FScrubRangeToScreen& RangeToScreen, FDrawTickArgs& InArgs) const;

private:
	/** The orientation of the ruler */
	EOrientation Orientation;

	/** The absolute origin of the document being measured.  The 0 on the ruler will start here. */
	FVector2D AbsoluteOrigin;

	/** The absolute position of the cursor so that little lines can be drawn showing the cursors position on the ruler. */
	TOptional<FVector2D> AbsoluteCursor;

	/** The current conversion from Slate Pixel To Unit Size. */
	float SlateToUnitScale;

	/** The public event we expose when the mouse button is pressed onto the ruler. */
	FPointerEventHandler MouseButtonDownHandler;
};

}
