// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaEyeDropperButton.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#define LOCTEXT_NAMESPACE "SAvaEyeDroppperButton"

void SAvaEyeDropperButton::Construct(const FArguments& InArgs)
{
	OnValueChanged = InArgs._OnValueChanged;
	OnBegin = InArgs._OnBegin;
	OnComplete = InArgs._OnComplete;
	DisplayGamma = InArgs._DisplayGamma;

	// This is a button containing an dropper image and text that tells the user to hit Esc.
	// Their visibility and colors are changed according to whether dropper mode is active or not.
	SButton::Construct(
		SButton::FArguments()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.ContentPadding(1.0f)
		.OnClicked(this, &SAvaEyeDropperButton::OnClicked)
		[
			SAssignNew(ButtonContentSwitcher, SWidgetSwitcher)
			+ SWidgetSwitcher::Slot()
			.Padding(FMargin(1.0f, 0.0f))
			[
				SNew(SImage)
				.Image(FCoreStyle::Get().GetBrush("ColorPicker.EyeDropper"))
				.ToolTipText(LOCTEXT("EyeDropperButton_ToolTip", "Activates the eye-dropper for selecting a colored pixel from any window."))
				.ColorAndOpacity(this, &SAvaEyeDropperButton::GetDropperImageColor)
			]
			+ SWidgetSwitcher::Slot()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("EscapeCue", "Esc"))
				.ToolTipText(LOCTEXT("EyeDropperEscapeCue_ToolTip", "Hit Escape key to stop the eye dropper"))
				.Visibility(this, &SAvaEyeDropperButton::GetEscapeTextVisibility)
			]
		]
	);

	ButtonContentSwitcher->SetActiveWidgetIndex(0);
}

void SAvaEyeDropperButton::ResetDropperModeStates()
{
	bWasClickActivated = false;
	bWasLeft = false;
	bWasReEntered = false;
	ButtonContentSwitcher->SetActiveWidgetIndex(0);
}

void SAvaEyeDropperButton::OnPreTick(float InDeltaTime)
{
	if (bWasClickActivated)
	{
		if (HasMouseCapture() && bWasLeft && !bWasReEntered)
		{
			const FVector2f CurrentCursorPosition = FSlateApplication::Get().GetCursorPos();
			if (CurrentCursorPosition != LastCursorPosition)
			{
				// In dropper mode and outside the button - sample the pixel color and push it to the client
				// Convert the display gamma into a ratio of gamma from the default gamma.
				float Gamma = DisplayGamma.Get(2.2f) / 2.2f;
				FLinearColor ScreenColor = FPlatformApplicationMisc::GetScreenPixelColor(static_cast<FVector2D>(CurrentCursorPosition), Gamma);
				OnValueChanged.ExecuteIfBound(ScreenColor);
			}

			LastCursorPosition = CurrentCursorPosition;
		}

		return;
	}

	LastCursorPosition.Reset();
	FSlateApplication::Get().OnPreTick().RemoveAll(this);
}

FReply SAvaEyeDropperButton::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Clicking ANY mouse button when the dropper isn't active resets the active dropper states ready to activate
	if (!HasMouseCapture())
	{
		ResetDropperModeStates();
	}

	return SButton::OnMouseButtonDown(MyGeometry, MouseEvent);
}

FReply SAvaEyeDropperButton::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// If a mouse click is completing and the dropper is active ALWAYS deactivate
	bool bDeactivating = false;
	if (bWasClickActivated)
	{
		bDeactivating = true;
	}

	// bWasClicked is reset here because if it is set during SButton::OnMouseButtonUp()
	// then the button was 'clicked' according to the usual rules. We might want to capture the
	// mouse when the button is clicked but can't do it in the Clicked callback.
	bWasClicked = false;
	FReply Reply = SButton::OnMouseButtonUp(MyGeometry, MouseEvent);

	// Switching dropper mode off?
	if (bDeactivating)
	{
		ResetDropperModeStates();

		Reply.ReleaseMouseCapture();

		const bool bCancelled = false;
		OnComplete.ExecuteIfBound(bCancelled);
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && bWasClicked)
	{
		// A normal LMB mouse click on the button occurred
		// Set the initial dropper mode state and capture the mouse
		bWasClickActivated = true;
		bWasLeft = false;
		bWasReEntered = false;
		ButtonContentSwitcher->SetActiveWidgetIndex(1);

		OnBegin.ExecuteIfBound();

		Reply.CaptureMouse(this->AsShared());

		FSlateApplication::Get().OnPreTick().AddSP(this, &SAvaEyeDropperButton::OnPreTick);
	}
	bWasClicked = false;

	return Reply;
}

FReply SAvaEyeDropperButton::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// If the mouse is captured and bWasClickActivated is set then we are in dropper mode
	if (HasMouseCapture() && bWasClickActivated)
	{
		if (IsMouseOver(MyGeometry, MouseEvent))
		{
			if (bWasLeft)
			{
				// Mouse is over the button having left it once
				bWasReEntered = true;
			}
		}
		else
		{
			// Mouse is outside the button
			bWasLeft = true;
			bWasReEntered = false;
		}
	}

	return SButton::OnMouseMove(MyGeometry, MouseEvent);
}

void SAvaEyeDropperButton::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	if (bWasClickActivated && bWasLeft && !bWasReEntered)
	{
		// We can't just wipe the dropper states when we lose mouse capture, since we will briefly lose mouse capture when the
		// dropper is selected. We need to ensure that we haven't just clicked the widget before resetting states.
		ResetDropperModeStates();

		const bool bCancelled = false;
		OnComplete.ExecuteIfBound(bCancelled);
	}
}

FReply SAvaEyeDropperButton::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// Escape key when in dropper mode
	if (InKeyEvent.GetKey() == EKeys::Escape &&
		HasMouseCapture() &&
		bWasClickActivated)
	{
		ResetDropperModeStates();

		// This is needed to switch the dropper cursor off immediately so the user can see the Esc key worked
		FSlateApplication::Get().QueryCursor();

		FReply ReleaseReply = FReply::Handled().ReleaseMouseCapture();

		const bool bCancelled = true;
		OnComplete.ExecuteIfBound(bCancelled);

		return ReleaseReply;
	}

	return FReply::Unhandled();
}

FCursorReply SAvaEyeDropperButton::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	// Cursor is changed to the dropper when dropper mode is active and the states are correct
	if (HasMouseCapture() &&
		bWasClickActivated && bWasLeft && !bWasReEntered)
	{
		return FCursorReply::Cursor(EMouseCursor::EyeDropper);
	}

	return SButton::OnCursorQuery(MyGeometry, CursorEvent);
}

FReply SAvaEyeDropperButton::OnClicked()
{
	// Log clicks so that OnMouseButtonUp() can post process clicks
	bWasClicked = true;
	return FReply::Handled();
}

EVisibility SAvaEyeDropperButton::GetEscapeTextVisibility() const
{
	// Show the Esc key message in the button when dropper mode is active
	if (HasMouseCapture() && bWasClickActivated)
	{
		return EVisibility::Visible;
	}
	return EVisibility::Hidden;
}

FSlateColor SAvaEyeDropperButton::GetDropperImageColor() const
{
	// Make the dropper image in the button pale when dropper mode is active
	if (HasMouseCapture() && bWasClickActivated)
	{
		return FLinearColor(0.3f, 0.3f, 0.3f, 1.0f);
	}
	return FSlateColor::UseForeground();
}

bool SAvaEyeDropperButton::IsMouseOver(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) const
{
	return MyGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition());
}

#undef LOCTEXT_NAMESPACE
