// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Attribute.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Widgets/Input/SButton.h"

class SWidgetSwitcher;

/**
 * Class for placing a color picker eye-dropper button.
 * A self contained until that only needs client code to set the display gamma and listen
 * for the OnValueChanged events. It toggles the dropper when clicked.
 * When active it captures the mouse, shows a dropper cursor and samples the pixel color constantly.
 * It is stopped normally by hitting the ESC key.
 */

/**
 * Unashamedly stolen from private engine code
 */
class SAvaEyeDropperButton : public SButton
{
public:
	DECLARE_DELEGATE_OneParam(FOnDropperComplete, bool)

	SLATE_BEGIN_ARGS( SAvaEyeDropperButton )
		: _OnValueChanged()
		, _OnBegin()
		, _OnComplete()
		, _DisplayGamma()
		{}

		/** Invoked when a new value is selected by the dropper */
		SLATE_EVENT(FOnLinearColorValueChanged, OnValueChanged)

		/** Invoked when the dropper goes from inactive to active */
		SLATE_EVENT(FSimpleDelegate, OnBegin)

		/** Invoked when the dropper goes from active to inactive */
		SLATE_EVENT(FOnDropperComplete, OnComplete)

		/** Sets the display Gamma setting - used to correct colors sampled from the screen */
		SLATE_ATTRIBUTE(float, DisplayGamma)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;

private:

	FReply OnClicked();

	EVisibility GetEscapeTextVisibility() const;

	FSlateColor GetDropperImageColor() const;

	bool IsMouseOver(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) const;

	void OnPreTick(float DeltaTime);
	void ResetDropperModeStates();

private:

	TSharedPtr<SWidgetSwitcher> ButtonContentSwitcher;

	/** Invoked when a new value is selected by the dropper */
	FOnLinearColorValueChanged OnValueChanged;

	/** Invoked when the dropper goes from inactive to active */
	FSimpleDelegate OnBegin;

	/** Invoked when the dropper goes from active to inactive - can be used to commit colors by the owning picker */
	FOnDropperComplete OnComplete;

	/** Sets the display Gamma setting - used to correct colors sampled from the screen */
	TAttribute<float> DisplayGamma;

	/** Previous Tick's cursor position */
	TOptional<FVector2f> LastCursorPosition;

	// Dropper states
	bool bWasClicked = false;
	bool bWasClickActivated = false;
	bool bWasLeft = false;
	bool bWasReEntered = false;
};
