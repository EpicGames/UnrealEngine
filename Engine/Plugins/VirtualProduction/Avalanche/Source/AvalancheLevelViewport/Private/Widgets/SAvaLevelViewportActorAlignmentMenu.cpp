// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAvaLevelViewportActorAlignmentMenu.h"
#include "AvaDefs.h"
#include "AvaLevelViewportStyle.h"
#include "AvaScreenAlignmentEnums.h"
#include "AvaScreenAlignmentUtils.h"
#include "AvaViewportUtils.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "Styling/CoreStyle.h"
#include "Styling/StyleColors.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/IToolkitHost.h"
#include "ViewportClient/IAvaViewportClient.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaLevelViewportActorAlignmentMenu"

namespace UE::AvaLevelViewport::Private
{
	// Transient setting. And that's fine.
	EAvaAlignmentContext ActorAlignContextType = EAvaAlignmentContext::SelectedActors;

	ECheckBoxState GetAlignButtonState(EAvaAlignmentContext InContextType)
	{
		return InContextType == ActorAlignContextType
			? ECheckBoxState::Checked
			: ECheckBoxState::Unchecked;
	}

	void OnAlignButtonClicked(ECheckBoxState InCheckBoxState, EAvaAlignmentContext InContextType)
	{
		ActorAlignContextType = InContextType == EAvaAlignmentContext::Screen && InCheckBoxState == ECheckBoxState::Checked
			? EAvaAlignmentContext::Screen
			: EAvaAlignmentContext::SelectedActors;
	}

	/** Walk up Child Actor's parent hierarchy and see if PossibleParentActor is there anywhere. */
	bool IsParentOf(const AActor* PossibleParentActor, const AActor* ChildActor)
	{
		if (PossibleParentActor == ChildActor)
		{
			return false;
		}

		if (PossibleParentActor == nullptr || ChildActor == nullptr)
		{
			return false;
		}

		AActor* Parent = ChildActor->GetAttachParentActor();

		if (!Parent)
		{
			return false;
		}

		if (Parent == PossibleParentActor)
		{
			return true;
		}

		return IsParentOf(PossibleParentActor, Parent);
	}

	/** Returns an array of weak pointers to the currently selected actors. */
	TArray<TWeakObjectPtr<const AActor>> GetSelectedActors(FEditorViewportClient* InViewportClient)
	{
		if (!InViewportClient)
		{
			return {};
		}

		FEditorModeTools* ModeTools = InViewportClient->GetModeTools();

		if (!ModeTools)
		{
			return {};
		}

		USelection* ActorSelection = ModeTools->GetSelectedActors();

		if (!ActorSelection)
		{
			return {};
		}

		TArray<AActor*> SelectedActors;
		ActorSelection->GetSelectedObjects<AActor>(SelectedActors);

		TArray<TWeakObjectPtr<const AActor>> SelectedActorWeak;
		Algo::Transform(SelectedActors, SelectedActorWeak, [](AActor* InActor) { return InActor; });

		return SelectedActorWeak;
	}

	bool IgnoreSelectedChildren()
	{
		return FSlateApplication::Get().GetModifierKeys().IsControlDown();
	}

	EAvaActorDistributionMode GetDistributionMode()
	{
		return FSlateApplication::Get().GetModifierKeys().IsShiftDown()
			? EAvaActorDistributionMode::EdgeDistance
			: EAvaActorDistributionMode::CenterDistance;
	}

	EAvaAlignmentSizeMode GetSizeMode()
	{
		return FSlateApplication::Get().GetModifierKeys().IsAltDown()
			? EAvaAlignmentSizeMode::SelfAndChildren
			: EAvaAlignmentSizeMode::Self;
	}

	/**
	 * Sorts actors into parents first. Then children. Then the children of those children. Etc.
	 * Optionally removes the children instead.
	 */
	TArray<AActor*> SortActors(const TArray<AActor*>& InActors, bool bInRemoveChildren)
	{
		const int32 NumActors = InActors.Num();

		TSet<AActor*> SelectedActorProcessedSet;
		SelectedActorProcessedSet.Reserve(NumActors);

		TArray<AActor*> SelectedActorsSorted;
		SelectedActorsSorted.Reserve(NumActors);

		while (SelectedActorProcessedSet.Num() < NumActors)
		{
			for (int32 Index = 0; Index < NumActors; ++Index)
			{
				if (SelectedActorProcessedSet.Contains(InActors[Index]))
				{
					continue;
				}

				bool bFoundSelectedParent = false;

				for (int32 SubIndex = Index + 1; SubIndex < NumActors; ++SubIndex)
				{
					if (SelectedActorProcessedSet.Contains(InActors[SubIndex]))
					{
						continue;
					}

					if (IsParentOf(InActors[SubIndex], InActors[Index]))
					{
						bFoundSelectedParent = true;
						break;
					}
				}

				if (bFoundSelectedParent && bInRemoveChildren)
				{
					if (bInRemoveChildren)
					{
						SelectedActorProcessedSet.Add(InActors[Index]);
					}
				}
				else
				{
					InActors[Index]->Modify();
					SelectedActorsSorted.Add(InActors[Index]);
					SelectedActorProcessedSet.Add(InActors[Index]);
				}
			}
		}

		return SelectedActorsSorted;
	}

	struct FDismissMenus
	{
		~FDismissMenus()
		{
			FSlateApplication::Get().DismissAllMenus();
		}
	};
}

TSharedRef<SAvaLevelViewportActorAlignmentMenu> SAvaLevelViewportActorAlignmentMenu::CreateMenu(const TSharedRef<IToolkitHost>& InToolkitHost)
{
	return SNew(SAvaLevelViewportActorAlignmentMenu, InToolkitHost);
}

void SAvaLevelViewportActorAlignmentMenu::PrivateRegisterAttributes(struct FSlateAttributeDescriptor::FInitializer&)
{
}

void SAvaLevelViewportActorAlignmentMenu::Construct(const FArguments& Args, const TSharedRef<IToolkitHost>& InToolkitHost)
{
	ToolkitHostWeak = InToolkitHost;

	FSlateFontInfo Font = FCoreStyle::GetDefaultFontStyle("Regular", 10);
	SGridPanel::Layer ForegroundLayer(0);

 	TSharedRef<SGridPanel> Grid = SNew(SGridPanel);

	static constexpr int32 Button1 = 0;
	static constexpr int32 Button2 = 1;
	static constexpr int32 Button3 = 2;
	static constexpr int32 Button4 = 3;

	enum ECellPosition
	{
		First,
		Middle,
		Last
	};

	auto CreateButtonCell = [](TSharedRef<SWidget>&& InButton, ECellPosition InPosition = ECellPosition::Middle)
		{
			return SNew(SBox)
				.Padding(3.f, 3.f, 3.f, 3.f)
				.WidthOverride(34.f)
				.HeightOverride(29.f)
				[
					InButton
				];
		};

	ChildSlot
	[
		SNew(SBox)
		.Padding(FMargin(3.f, 8.f, 3.f, 3.f))
		[
			Grid
		]
	];

	int32 Row = 0;

	// Context
	using namespace UE::AvaLevelViewport::Private;

	Grid->AddSlot(Button1, Row, ForegroundLayer)
		.ColumnSpan(4)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.Padding(0.f, -1.f, 0.f, 1.f)
			[
				SNew(SCheckBox)
				.Style(FAvaLevelViewportStyle::Get(), "Avalanche.Alignment.Context")
				.OnCheckStateChanged_Static(&OnAlignButtonClicked, EAvaAlignmentContext::SelectedActors)
				.IsChecked_Static(&GetAlignButtonState, EAvaAlignmentContext::SelectedActors)
				[
					SNew(STextBlock)
					.Font(Font)
					.Text(LOCTEXT("Actors", "Actors"))
				]
			]
			
			+ SHorizontalBox::Slot()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.Padding(0.f, -1.f, 0.f, 1.f)
			[
				SNew(SCheckBox)
				.Style(FAvaLevelViewportStyle::Get(), "Avalanche.Alignment.Context")
				.OnCheckStateChanged_Static(&OnAlignButtonClicked, EAvaAlignmentContext::Screen)
				.IsChecked_Static(&GetAlignButtonState, EAvaAlignmentContext::Screen)
				[
					SNew(STextBlock)
					.Font(Font)
					.Text(LOCTEXT("Screen", "Screen"))
				]
			]
		];

	++Row;

	// Horizontal
	Grid->AddSlot(Button1, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetLocationAlignmentButton(EAvaHorizontalAlignment::Left, static_cast<EAvaVerticalAlignment>(NO_VALUE), 
					static_cast<EAvaDepthAlignment>(NO_VALUE), "Icons.Alignment.Left", 
					LOCTEXT("AlignSelectedActorsLeft", "Align Selected Actors Left (to Screen or Actors)")),
				ECellPosition::First
			)
		];

	Grid->AddSlot(Button2, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetLocationAlignmentButton(EAvaHorizontalAlignment::Center, static_cast<EAvaVerticalAlignment>(NO_VALUE),
					static_cast<EAvaDepthAlignment>(NO_VALUE), "Icons.Alignment.Center_Y",
					LOCTEXT("AlignSelectedActorsHCenter", "Align Selected Actors Center (Horizontally) (to Screen or Actors)"))
			)
		];

	Grid->AddSlot(Button3, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetLocationAlignmentButton(EAvaHorizontalAlignment::Right, static_cast<EAvaVerticalAlignment>(NO_VALUE),
					static_cast<EAvaDepthAlignment>(NO_VALUE), "Icons.Alignment.Right",
					LOCTEXT("AlignSelectedActorsRight", "Align Selected Actors Right (to Screen or Actors)"))
			)
		];

	// Horiz + Vertical
	Grid->AddSlot(Button4, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetLocationAlignmentButton(EAvaHorizontalAlignment::Center, EAvaVerticalAlignment::Center,
					static_cast<EAvaDepthAlignment>(NO_VALUE), "Icons.Alignment.Center_YZ",
					LOCTEXT("AlignSelectedActorsHVCenter", "Align Selected Actors Center (Horizontally and Vertically) (to Screen or Actors)")),
				ECellPosition::Last
			)
		];

	++Row;

	// Vertical
	Grid->AddSlot(Button1, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetLocationAlignmentButton(static_cast<EAvaHorizontalAlignment>(NO_VALUE), EAvaVerticalAlignment::Top, 
					static_cast<EAvaDepthAlignment>(NO_VALUE), "Icons.Alignment.Top", 
					LOCTEXT("AlignSelectedActorsTop", "Align Selected Actors Top (to Screen or Actors)")),
				ECellPosition::First
			)
		];

	Grid->AddSlot(Button2, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetLocationAlignmentButton(static_cast<EAvaHorizontalAlignment>(NO_VALUE), EAvaVerticalAlignment::Center,
					static_cast<EAvaDepthAlignment>(NO_VALUE), "Icons.Alignment.Center_Z",
					LOCTEXT("AlignSelectedActorsVCenter", "Align Selected Actors Center (Vertically) (to Screen or Actors)"))
			)
		];

	Grid->AddSlot(Button3, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetLocationAlignmentButton(static_cast<EAvaHorizontalAlignment>(NO_VALUE), EAvaVerticalAlignment::Bottom,
					static_cast<EAvaDepthAlignment>(NO_VALUE), "Icons.Alignment.Bottom",
					LOCTEXT("AlignSelectedActorsBottom", "Align Selected Actors Bottom (to Screen or Actors)"))
			)
		];

	++Row;

	// Depth
	Grid->AddSlot(Button1, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetLocationAlignmentButton(static_cast<EAvaHorizontalAlignment>(NO_VALUE), static_cast<EAvaVerticalAlignment>(NO_VALUE), 
					EAvaDepthAlignment::Front, "Icons.Alignment.Translation.Front", 
					LOCTEXT("AlignSelectedActorsFront", "Align Selected Actors Front (to Actors Only)")),
				ECellPosition::First
			)
		];

	Grid->AddSlot(Button2, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetLocationAlignmentButton(static_cast<EAvaHorizontalAlignment>(NO_VALUE), static_cast<EAvaVerticalAlignment>(NO_VALUE), 
					EAvaDepthAlignment::Center, "Icons.Alignment.Translation.Center_X", 
					LOCTEXT("AlignSelectedActorsDCenter", "Align Selected Actors Center (Depth) (to Actors Only)"))
			)
		];

	Grid->AddSlot(Button3, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetLocationAlignmentButton(static_cast<EAvaHorizontalAlignment>(NO_VALUE), static_cast<EAvaVerticalAlignment>(NO_VALUE),
					EAvaDepthAlignment::Back, "Icons.Alignment.Translation.Back",
					LOCTEXT("AlignSelectedActorsBack", "Align Selected Actors Back (to Actors Only)"))
			)
		];

	++Row;

	// Distribute
	Grid->AddSlot(Button1, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetDistributionAlignmentButton(EAvaScreenAxis::Horizontal, "Icons.Alignment.DistributeY",
					LOCTEXT("AlignSelectedActorsDistributeY", "Distribute Selected Actors Left to Right (based on Screen or Actors)")),
				ECellPosition::First
			)
		];

	Grid->AddSlot(Button2, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetDistributionAlignmentButton(EAvaScreenAxis::Vertical, "Icons.Alignment.DistributeZ",
					LOCTEXT("AlignSelectedActorsDistributeZ", "Distribute Selected Actors Top to Bottom (based on Screen or Actors)"))
			)
		];

	Grid->AddSlot(Button3, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetDistributionAlignmentButton(EAvaScreenAxis::Depth, "Icons.Alignment.DistributeX",
					LOCTEXT("AlignSelectedActorsDistributeX", "Distribute Selected Actors Front to Back (based on Screen or Actors)"))
			)
		];

	++Row;

	// Actor Rotation
	Grid->AddSlot(Button1, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetRotationAlignmentButton(EAvaRotationAxis::Roll, "Icons.Alignment.Rotation.Actor.Roll",
					LOCTEXT("AlignSelectedActorsToRoll", "Align Roll of Selected Actors")),
				ECellPosition::First
			)
		];

	Grid->AddSlot(Button2, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetRotationAlignmentButton(EAvaRotationAxis::Pitch, "Icons.Alignment.Rotation.Actor.Pitch",
					LOCTEXT("AlignSelectedActorsToPitch", "Align Pitch of Selected Actors"))
			)
		];

	Grid->AddSlot(Button3, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetRotationAlignmentButton(EAvaRotationAxis::Yaw, "Icons.Alignment.Rotation.Actor.Yaw",
					LOCTEXT("AlignSelectedActorsToYaw", "Align Yaw of Selected Actors"))
			)
		];

	Grid->AddSlot(Button4, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetRotationAlignmentButton(EAvaRotationAxis::All, "Icons.Alignment.Rotation.Actor.All",
					LOCTEXT("AlignSelectedActorsToRollYawPitch", "Align Roll, Pitch and Yaw  of Selected Actors")),
				ECellPosition::Last
			)
		];

	++Row;

	// Camera Rotation
	Grid->AddSlot(Button1, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetCameraRotationAlignmentButton(EAvaRotationAxis::Roll, "Icons.Alignment.Rotation.Camera.Roll",
					LOCTEXT("AlignSelectedActorsToRollCamera", "Align Roll of Selected Actors to face the Camera")),
				ECellPosition::First
			)
		];

	Grid->AddSlot(Button2, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetCameraRotationAlignmentButton(EAvaRotationAxis::Pitch, "Icons.Alignment.Rotation.Camera.Pitch",
					LOCTEXT("AlignSelectedActorsToPitchCamera", "Align Pitch of Selected Actors to face the Camera"))
			)
		];

	Grid->AddSlot(Button3, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetCameraRotationAlignmentButton(EAvaRotationAxis::Yaw, "Icons.Alignment.Rotation.Camera.Yaw",
					LOCTEXT("AlignSelectedActorsToYawCamera", "Align Yaw of Selected Actors to face the Camera"))
			)
		];

	Grid->AddSlot(Button4, Row, ForegroundLayer)
		[
			CreateButtonCell(
				GetCameraRotationAlignmentButton(EAvaRotationAxis::All, "Icons.Alignment.Rotation.Camera.All",
					LOCTEXT("AlignSelectedActorsToRollYawPitchCamera", "Align Roll, Pitch and Yaw of Selected Actors to face the Camera")),
				ECellPosition::Last
			)
		];

	++Row;

	// Screen size
	Grid->AddSlot(Button1, Row, ForegroundLayer)
		[
			CreateButtonCell(
				SNew(SButton)
				.ButtonStyle(&FAvaLevelViewportStyle::Get().GetWidgetStyle<FButtonStyle>("Avalanche.Alignment.Button"))
				.ContentPadding(FMargin(2.f))
				.OnClicked(this, &SAvaLevelViewportActorAlignmentMenu::OnSizeToScreenClicked, false)
				.ToolTipText(LOCTEXT("SizeToScreen", "Size the Currently Selected Actors to the Screen."))
				[
					SNew(SImage)
					.Image(FAvaLevelViewportStyle::Get().GetBrush("Icons.Screen.SizeToScreen"))
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
				],
				ECellPosition::First
			)
		];

	Grid->AddSlot(Button2, Row, ForegroundLayer)
		[
			CreateButtonCell(
				SNew(SButton)
				.ButtonStyle(&FAvaLevelViewportStyle::Get().GetWidgetStyle<FButtonStyle>("Avalanche.Alignment.Button"))
				.ContentPadding(FMargin(2.f))
				.OnClicked(this, &SAvaLevelViewportActorAlignmentMenu::OnSizeToScreenClicked, true)
				.ToolTipText(LOCTEXT("SizeToScreenStretch", "Stretch the Currently Selected Actors to the Screen."))
				[
					SNew(SImage)
					.Image(FAvaLevelViewportStyle::Get().GetBrush("Icons.Screen.SizeToScreenStretch"))
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
				]
			)
		];

	Grid->AddSlot(Button3, Row, ForegroundLayer)
		[
			CreateButtonCell(
				SNew(SButton)
				.ButtonStyle(&FAvaLevelViewportStyle::Get().GetWidgetStyle<FButtonStyle>("Avalanche.Alignment.Button"))
				.ContentPadding(FMargin(2.f))
				.OnClicked(this, &SAvaLevelViewportActorAlignmentMenu::OnFitToScreenClicked)
				.ToolTipText(LOCTEXT("FitToScreen", "Stretches the Currently Selected Actors to the Screen and centers it.\n\n- Shift: If an object is rotated beyond 45 degrees with respect to the camera, align to nearest axis."))
				[
					SNew(SImage)
					.Image(FAvaLevelViewportStyle::Get().GetBrush("Icons.Screen.FitToScreen"))
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
				]
			)
		];
}

TSharedRef<SButton> SAvaLevelViewportActorAlignmentMenu::GetLocationAlignmentButton(EAvaHorizontalAlignment Horiz, EAvaVerticalAlignment Vert, EAvaDepthAlignment Depth, FName Image, FText ToolTip)
{
	static const FText ToolTipFormat = LOCTEXT("AlignmentButtonToolTip", "{0}\n\n- Control: If an actor and its parent are both selected, do not align the child actor.\n- Alt: When considering the size of an actor, include its child actors.");

	return SNew(SButton)
		.ButtonStyle(&FAvaLevelViewportStyle::Get().GetWidgetStyle<FButtonStyle>("Avalanche.Alignment.Button"))
		.ContentPadding(FMargin(2.f))
		.OnClicked(this, &SAvaLevelViewportActorAlignmentMenu::OnLocationAlignmentButtonClicked, Horiz, Vert, Depth)
		.ToolTipText(FText::Format(ToolTipFormat, ToolTip))
		[
			SNew(SImage)
			.Image(FAvaLevelViewportStyle::Get().GetBrush(Image))
			.DesiredSizeOverride(FVector2D(16.f, 16.f))
		];
}

TSharedRef<SButton> SAvaLevelViewportActorAlignmentMenu::GetDistributionAlignmentButton(EAvaScreenAxis InScreenAxis, FName Image, FText ToolTip)
{
	static const FText ToolTipFormat = LOCTEXT("DistributionButtonToolTip", "{0}\n\n- Control: If an actor and its parent are both selected, do not align the child actor.\n- Alt: When considering the size of an actor, include its child actors.\n- Shift: Distribute actors at regular offsets.\n- No Shift: Distribute space between actors.");

	return SNew(SButton)
		.ButtonStyle(&FAvaLevelViewportStyle::Get().GetWidgetStyle<FButtonStyle>("Avalanche.Alignment.Button"))
		.ContentPadding(FMargin(2.f))
		.OnClicked(this, &SAvaLevelViewportActorAlignmentMenu::OnDistributeAlignmentButtonClicked, InScreenAxis)
		.ToolTipText(FText::Format(ToolTipFormat, ToolTip))
		[
			SNew(SImage)
			.Image(FAvaLevelViewportStyle::Get().GetBrush(Image))
			.DesiredSizeOverride(FVector2D(16.f, 16.f))
		];
}
TSharedRef<SButton> SAvaLevelViewportActorAlignmentMenu::GetRotationAlignmentButton(EAvaRotationAxis InAxisList, FName Image, FText ToolTip)
{
	static const FText ToolTipFormat = LOCTEXT("RotationAlignmentButtonToolTip", "{0}\n\n- Shift: If an object is rotated beyond 45 degrees, align to nearest axis.\n- Alt: Rotate the actor an extra 180 degrees on the given axis.");

	return SNew(SButton)
		.ButtonStyle(&FAvaLevelViewportStyle::Get().GetWidgetStyle<FButtonStyle>("Avalanche.Alignment.Button"))
		.ContentPadding(FMargin(2.f))
		.OnClicked(this, &SAvaLevelViewportActorAlignmentMenu::OnRotationAlignmentButtonClicked, InAxisList)
		.ToolTipText(FText::Format(ToolTipFormat, ToolTip))
		[
			SNew(SImage)
			.Image(FAvaLevelViewportStyle::Get().GetBrush(Image))
			.DesiredSizeOverride(FVector2D(16.f, 16.f))
		];
}

TSharedRef<SButton> SAvaLevelViewportActorAlignmentMenu::GetCameraRotationAlignmentButton(EAvaRotationAxis InAxisList, FName Image, FText ToolTip)
{
	static const FText ToolTipFormat = LOCTEXT("RotationAlignmentButtonToolTip", "{0}\n\n- Shift: If an object is rotated beyond 45 degrees, align to nearest axis.\n- Alt: Rotate the actor an extra 180 degrees on the given axis.");

	return SNew(SButton)
		.ButtonStyle(&FAvaLevelViewportStyle::Get().GetWidgetStyle<FButtonStyle>("Avalanche.Alignment.Button"))
		.ContentPadding(FMargin(2.f))
		.OnClicked(this, &SAvaLevelViewportActorAlignmentMenu::OnCameraRotationAlignmentButtonClicked, InAxisList)
		.ToolTipText(FText::Format(ToolTipFormat, ToolTip))
		[
			SNew(SImage)
			.Image(FAvaLevelViewportStyle::Get().GetBrush(Image))
			.DesiredSizeOverride(FVector2D(16.f, 16.f))
		];
}

FReply SAvaLevelViewportActorAlignmentMenu::OnSizeToScreenClicked(bool bInStretchToFit)
{
	using namespace UE::AvaLevelViewport::Private;

	FDismissMenus DismissMenus;

	TSharedPtr<IToolkitHost> ToolkitHost = ToolkitHostWeak.Pin();

	if (!ToolkitHost.IsValid())
	{
		return FReply::Handled();
	}

	FEditorViewportClient* EditorViewportClient = ToolkitHost->GetEditorModeManager().GetFocusedViewportClient();

	if (!EditorViewportClient)
	{
		return FReply::Handled();
	}

	TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient);

	if (!AvaViewportClient.IsValid())
	{
		return FReply::Handled();
	}

	FScopedTransaction Transaction(LOCTEXT("SizeToScreenTransaction", "Size Actor to Screen"));

	TArray<TWeakObjectPtr<const AActor>> SelectedActorsWeak = GetSelectedActors(EditorViewportClient);

	for (const TWeakObjectPtr<const AActor>& SelectedActorWeak : SelectedActorsWeak)
	{
		if (AActor* SelectedActor = const_cast<AActor*>(SelectedActorWeak.Get()))
		{
			if (USceneComponent* RootComponent = SelectedActor->GetRootComponent())
			{
				SelectedActor->Modify();
				RootComponent->Modify();

				FAvaScreenAlignmentUtils::SizeActorToScreen(
					AvaViewportClient.ToSharedRef(),
					*SelectedActor,
					bInStretchToFit
				);
			}
		}
	}

	return FReply::Handled();
}

FReply SAvaLevelViewportActorAlignmentMenu::OnFitToScreenClicked()
{
	using namespace UE::AvaLevelViewport::Private;

	FDismissMenus DismissMenus;

	TSharedPtr<IToolkitHost> ToolkitHost = ToolkitHostWeak.Pin();

	if (!ToolkitHost.IsValid())
	{
		return FReply::Handled();
	}

	FEditorViewportClient* EditorViewportClient = ToolkitHost->GetEditorModeManager().GetFocusedViewportClient();

	if (!EditorViewportClient)
	{
		return FReply::Handled();
	}

	TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient);

	if (!AvaViewportClient.IsValid())
	{
		return FReply::Handled();
	}

	FScopedTransaction Transaction(LOCTEXT("FitToScreenTransaction", "Fit Actor to Screen"));

	const bool bUseNearestAxis = FSlateApplication::Get().GetModifierKeys().IsShiftDown();

	TArray<TWeakObjectPtr<const AActor>> SelectedActorsWeak = GetSelectedActors(EditorViewportClient);

	for (const TWeakObjectPtr<const AActor>& SelectedActorWeak : SelectedActorsWeak)
	{
		if (AActor* SelectedActor = const_cast<AActor*>(SelectedActorWeak.Get()))
		{
			if (USceneComponent* RootComponent = SelectedActor->GetRootComponent())
			{
				SelectedActor->Modify();
				RootComponent->Modify();

				FAvaScreenAlignmentUtils::FitActorToScreen(
					AvaViewportClient.ToSharedRef(),
					*SelectedActor,
					/* bSizeToFit */ true,
					bUseNearestAxis
				);
			}
		}
	}

	return FReply::Handled();
}

FReply SAvaLevelViewportActorAlignmentMenu::OnRotationAlignmentButtonClicked(EAvaRotationAxis InAxisList)
{
	using namespace UE::AvaLevelViewport::Private;

	FDismissMenus DismissMenus;

	TSharedPtr<IToolkitHost> ToolkitHost = ToolkitHostWeak.Pin();

	if (!ToolkitHost.IsValid())
	{
		return FReply::Handled();
	}

	FEditorViewportClient* EditorViewportClient = ToolkitHost->GetEditorModeManager().GetFocusedViewportClient();

	if (!EditorViewportClient)
	{
		return FReply::Handled();
	}

	TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient);

	if (!AvaViewportClient.IsValid())
	{
		return FReply::Handled();
	}

	FScopedTransaction Transaction(LOCTEXT("AlignRotationToActor", "Align Rotation to Actor"));

	TArray<TWeakObjectPtr<const AActor>> SelectedActorsWeak = GetSelectedActors(EditorViewportClient);
	AActor* FirstActor = nullptr;
	FRotator Rotation;

	const bool bUseNearestAxis = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
	const bool bBackwards = FSlateApplication::Get().GetModifierKeys().IsAltDown();

	for (const TWeakObjectPtr<const AActor>& SelectedActorWeak : SelectedActorsWeak)
	{
		if (AActor* SelectedActor = const_cast<AActor*>(SelectedActorWeak.Get()))
		{
			if (!FirstActor)
			{
				FirstActor = const_cast<AActor*>(SelectedActor);
				Rotation = FirstActor->GetActorRotation();
			}
			else if (USceneComponent* RootComponent = SelectedActor->GetRootComponent())
			{
				SelectedActor->Modify();
				RootComponent->Modify();

				FAvaScreenAlignmentUtils::AlignActorRotationAxis(
					AvaViewportClient.ToSharedRef(),
					*const_cast<AActor*>(SelectedActor),
					InAxisList,
					Rotation,
					bUseNearestAxis,
					bBackwards
				);
			}
		}
	}

	return FReply::Handled();
}

FReply SAvaLevelViewportActorAlignmentMenu::OnCameraRotationAlignmentButtonClicked(EAvaRotationAxis InAxisList)
{
	using namespace UE::AvaLevelViewport::Private;

	FDismissMenus DismissMenus;

	TSharedPtr<IToolkitHost> ToolkitHost = ToolkitHostWeak.Pin();

	if (!ToolkitHost.IsValid())
	{
		return FReply::Handled();
	}

	FEditorViewportClient* EditorViewportClient = ToolkitHost->GetEditorModeManager().GetFocusedViewportClient();

	if (!EditorViewportClient)
	{
		return FReply::Handled();
	}

	TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient);

	if (!AvaViewportClient.IsValid())
	{
		return FReply::Handled();
	}

	FScopedTransaction Transaction(LOCTEXT("AlignRotationToCamera", "Align Rotation to Camera"));

	TArray<TWeakObjectPtr<const AActor>> SelectedActorsWeak = GetSelectedActors(EditorViewportClient);

	TArray<AActor*> SelectedActors;
	SelectedActors.Reserve(SelectedActorsWeak.Num());

	for (const TWeakObjectPtr<const AActor>& SelectedActorWeak : SelectedActorsWeak)
	{
		if (AActor* SelectedActor = const_cast<AActor*>(SelectedActorWeak.Get()))
		{
			if (USceneComponent* RootComponent = SelectedActor->GetRootComponent())
			{
				SelectedActor->Modify();
				RootComponent->Modify();

				SelectedActors.Add(SelectedActor);
			}
		}
	}

	const bool bUseNearestAxis = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
	const bool bBackwards = FSlateApplication::Get().GetModifierKeys().IsAltDown();

	if (!SelectedActors.IsEmpty())
	{
		FAvaScreenAlignmentUtils::AlignActorsCameraRotationAxis(
			AvaViewportClient.ToSharedRef(),
			SelectedActors,
			InAxisList,
			bUseNearestAxis,
			bBackwards
		);
	}

	return FReply::Handled();
}

FReply SAvaLevelViewportActorAlignmentMenu::OnDistributeAlignmentButtonClicked(EAvaScreenAxis InScreenAxis)
{
	using namespace UE::AvaLevelViewport::Private;

	FDismissMenus DismissMenus;

	TSharedPtr<IToolkitHost> ToolkitHost = ToolkitHostWeak.Pin();

	if (!ToolkitHost.IsValid())
	{
		return FReply::Handled();
	}

	FEditorViewportClient* EditorViewportClient = ToolkitHost->GetEditorModeManager().GetFocusedViewportClient();

	if (!EditorViewportClient)
	{
		return FReply::Handled();
	}

	TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient);

	if (!AvaViewportClient.IsValid())
	{
		return FReply::Handled();
	}

	FScopedTransaction Transaction(LOCTEXT("DistributeActors", "Distribute Actors"));

	TArray<TWeakObjectPtr<const AActor>> SelectedActorsWeak = GetSelectedActors(EditorViewportClient);

	TArray<AActor*> SelectedActors;
	SelectedActors.Reserve(SelectedActorsWeak.Num());

	for (const TWeakObjectPtr<const AActor>& SelectedActorWeak : SelectedActorsWeak)
	{
		if (AActor* SelectedActor = const_cast<AActor*>(SelectedActorWeak.Get()))
		{
			if (USceneComponent* RootComponent = SelectedActor->GetRootComponent())
			{
				SelectedActors.Add(SelectedActor);
			}
		}
	}

	SelectedActors = SortActors(SelectedActors, IgnoreSelectedChildren());

	switch (InScreenAxis)
	{
		case EAvaScreenAxis::Horizontal:
			FAvaScreenAlignmentUtils::DistributeActorsHorizontal(AvaViewportClient.ToSharedRef(), SelectedActors,
				GetSizeMode(), GetDistributionMode(), ActorAlignContextType);
			break;

		case EAvaScreenAxis::Vertical:
			FAvaScreenAlignmentUtils::DistributeActorsVertical(AvaViewportClient.ToSharedRef(), SelectedActors,
				GetSizeMode(), GetDistributionMode(), ActorAlignContextType);
			break;

		case EAvaScreenAxis::Depth:
			FAvaScreenAlignmentUtils::DistributeActorsDepth(AvaViewportClient.ToSharedRef(), SelectedActors,
				GetSizeMode(), GetDistributionMode());
			break;

		default:
			// Do nothing
			break;
	}

	return FReply::Handled();
}

FReply SAvaLevelViewportActorAlignmentMenu::OnLocationAlignmentButtonClicked(EAvaHorizontalAlignment InHoriz, EAvaVerticalAlignment InVert,
	EAvaDepthAlignment InDepth)
{
	using namespace UE::AvaLevelViewport::Private;

	FDismissMenus DismissMenus;

	TSharedPtr<IToolkitHost> ToolkitHost = ToolkitHostWeak.Pin();

	if (!ToolkitHost.IsValid())
	{
		return FReply::Handled();
	}

	FEditorViewportClient* EditorViewportClient = ToolkitHost->GetEditorModeManager().GetFocusedViewportClient();

	if (!EditorViewportClient)
	{
		return FReply::Handled();
	}

	TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient);

	if (!AvaViewportClient.IsValid())
	{
		return FReply::Handled();
	}

	FScopedTransaction Transaction(LOCTEXT("AlignActors", "Align Actors"));

	TArray<TWeakObjectPtr<const AActor>> SelectedActorsWeak = GetSelectedActors(EditorViewportClient);

	TArray<AActor*> SelectedActors;
	SelectedActors.Reserve(SelectedActorsWeak.Num());

	for (const TWeakObjectPtr<const AActor>& SelectedActorWeak : SelectedActorsWeak)
	{
		if (AActor* SelectedActor = const_cast<AActor*>(SelectedActorWeak.Get()))
		{
			if (USceneComponent* RootComponent = SelectedActor->GetRootComponent())
			{
				SelectedActors.Add(SelectedActor);
			}
		}
	}

	SelectedActors = SortActors(SelectedActors, IgnoreSelectedChildren());

	if (InHoriz != static_cast<EAvaHorizontalAlignment>(NO_VALUE))
	{
		FAvaScreenAlignmentUtils::AlignActorsHorizontal(AvaViewportClient.ToSharedRef(), SelectedActors,
			InHoriz, GetSizeMode(), ActorAlignContextType);
	}

	if (InVert != static_cast<EAvaVerticalAlignment>(NO_VALUE))
	{
		FAvaScreenAlignmentUtils::AlignActorsVertical(AvaViewportClient.ToSharedRef(), SelectedActors,
			InVert, GetSizeMode(), ActorAlignContextType);
	}

	if (InDepth != static_cast<EAvaDepthAlignment>(NO_VALUE))
	{
		FAvaScreenAlignmentUtils::AlignActorsDepth(AvaViewportClient.ToSharedRef(), SelectedActors,
			InDepth, GetSizeMode());
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
