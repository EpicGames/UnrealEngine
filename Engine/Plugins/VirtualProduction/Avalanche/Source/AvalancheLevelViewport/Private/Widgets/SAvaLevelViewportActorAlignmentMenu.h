// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Templates/SharedPointer.h"

class FName;
class FText;
class IToolkitHost;
class SButton;
enum class EAvaDepthAlignment : uint8;
enum class EAvaHorizontalAlignment : uint8;
enum class EAvaRotationAxis : uint8;
enum class EAvaScreenAxis : uint8;
enum class EAvaVerticalAlignment : uint8;

class SAvaLevelViewportActorAlignmentMenu : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SAvaLevelViewportActorAlignmentMenu, SCompoundWidget)

	SLATE_BEGIN_ARGS(SAvaLevelViewportActorAlignmentMenu) {}
	SLATE_END_ARGS()

public:
	static TSharedRef<SAvaLevelViewportActorAlignmentMenu> CreateMenu(const TSharedRef<IToolkitHost>& InToolkitHost);

	void Construct(const FArguments& Args, const TSharedRef<IToolkitHost>& InToolkitHost);

protected:
	TWeakPtr<IToolkitHost> ToolkitHostWeak;

	static const inline uint8 NO_VALUE = 255;

	TSharedRef<SButton> GetLocationAlignmentButton(EAvaHorizontalAlignment InHoriz, EAvaVerticalAlignment InVert, EAvaDepthAlignment InDepth, 
		FName InImage, FText InToolTip);

	TSharedRef<SButton> GetDistributionAlignmentButton(EAvaScreenAxis InScreenAxis, FName InImage, FText InToolTip);

	TSharedRef<SButton> GetRotationAlignmentButton(EAvaRotationAxis InAxisList, FName InImage, FText InToolTip);

	TSharedRef<SButton> GetCameraRotationAlignmentButton(EAvaRotationAxis InAxisList, FName InImage, FText InToolTip);

	FReply OnSizeToScreenClicked(bool bInStretchToFit);

	FReply OnFitToScreenClicked();

	FReply OnRotationAlignmentButtonClicked(EAvaRotationAxis InAxisList);

	FReply OnCameraRotationAlignmentButtonClicked(EAvaRotationAxis InAxisList);

	FReply OnDistributeAlignmentButtonClicked(EAvaScreenAxis InScreenAxis);

	FReply OnLocationAlignmentButtonClicked(EAvaHorizontalAlignment InHoriz, EAvaVerticalAlignment InVert, EAvaDepthAlignment InDepth);
};
