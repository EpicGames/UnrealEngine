// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAvaLevelViewportActorColorMenu.h"
#include "AvaLevelViewportModule.h"
#include "AvaLevelViewportStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "ColorPicker/AvaViewportColorPickerActorClassRegistry.h"
#include "CoreGlobals.h"
#include "EditorModeManager.h"
#include "Engine/Engine.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Input/DragAndDrop.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Misc/ConfigCacheIni.h"
#include "Selection.h"
#include "SlateMaterialBrush.h"
#include "Styling/StyleColors.h"
#include "Toolkits/IToolkitHost.h"
#include "UObject/Package.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SAvaEyeDropperButton.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SAvaLevelViewportActorColorMenu"

int32 SAvaLevelViewportActorColorMenu::ActiveThemeIndex = 0;

class FAvaColorPickerPaletteDragDropOperation : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FAvaColorPickerPaletteDragDropOperation, FDragDropOperation)

	FAvaColorPickerPaletteDragDropOperation(TSharedPtr<SAvaLevelViewportActorColorMenu> InColorPickerPtr)
	{
		ColorPickerPtr = InColorPickerPtr;
	}

	virtual ~FAvaColorPickerPaletteDragDropOperation() override
	{
		const TSharedPtr<SAvaLevelViewportActorColorMenu> ColorPicker = ColorPickerPtr.Pin();

		if (ColorPicker.IsValid())
		{
			ColorPicker->EndColorSelection();
		}
	}

	virtual void OnDragged(const FDragDropEvent& DragDropEvent) override
	{
		const TSharedPtr<SAvaLevelViewportActorColorMenu> ColorPicker = ColorPickerPtr.Pin();

		if (ColorPicker.IsValid())
		{
			const FVector2f AbsoluteMousePosition = FSlateApplication::Get().GetCursorPos();
			const FVector2f LocalMouseLocation = ColorPicker->GetTickSpaceGeometry().AbsoluteToLocal(AbsoluteMousePosition);
			ColorPicker->UpdateColorWheel(LocalMouseLocation);
		}
	}

protected:
	TWeakPtr<SAvaLevelViewportActorColorMenu> ColorPickerPtr;
};

void SAvaLevelViewportActorColorMenu::OnColorStyleChanged()
{
	BroadcastColorChange();
}

void SAvaLevelViewportActorColorMenu::BroadcastColorChange()
{
	ColorPicker.OnColorSelected({ColorStyle, ActiveColor, InactiveColor, bIsUnlit});
}

void SAvaLevelViewportActorColorMenu::SetActiveColorRGB_Direct(FLinearColor InRGB)
{
	Red = InRGB.R;
	Green = InRGB.G;
	Blue = InRGB.B;

	ActiveColor.R = Red;
	ActiveColor.G = Green;
	ActiveColor.B = Blue;

	CopyRGBtoHSV();

	ApplyActiveColor();
}

void SAvaLevelViewportActorColorMenu::SetActiveColorHSV_Direct(FLinearColor InHSV)
{
	Hue = InHSV.R / 360.f;
	Saturation = InHSV.G;
	Value = InHSV.B;

	CopyHSVtoRGB();

	ApplyActiveColor();
}

void SAvaLevelViewportActorColorMenu::SetInactiveColorRGB_Direct(FLinearColor InRGB)
{
	InactiveColor = InRGB;
}

void SAvaLevelViewportActorColorMenu::SetInactiveColorHSV_Direct(FLinearColor InHSV)
{
	InactiveColor = InHSV.HSVToLinearRGB();
}

void SAvaLevelViewportActorColorMenu::SetColorStyle_Direct(EAvaColorStyle InStyle)
{
	ColorStyle = InStyle;

	if (ColorStyleSwitcher.IsValid())
	{
		switch (InStyle)
		{
			case EAvaColorStyle::LinearGradient:
				ColorStyleSwitcher->SetActiveWidgetIndex(1);
				break;

			case EAvaColorStyle::Solid:
			default:
				ColorStyleSwitcher->SetActiveWidgetIndex(0);
				break;
		}
	}
}

FSlateColor SAvaLevelViewportActorColorMenu::GetUnlitButtonColor() const
{
	return bIsUnlit ? EStyleColor::Foreground : EStyleColor::AccentBlue;
}

TSharedRef<SAvaLevelViewportActorColorMenu> SAvaLevelViewportActorColorMenu::CreateMenu(const TSharedRef<IToolkitHost>& InToolkitHost)
{
	return SNew(SAvaLevelViewportActorColorMenu, InToolkitHost);
}

void SAvaLevelViewportActorColorMenu::PrivateRegisterAttributes(struct FSlateAttributeDescriptor::FInitializer&)
{
}

void SAvaLevelViewportActorColorMenu::Construct(const FArguments& InArgs, const TSharedRef<IToolkitHost>& InToolkitHost)
{
	ColorPicker.SetToolkitHost(InToolkitHost);

	HueMaterial = LoadObject<UMaterial>(nullptr, TEXT("Material'/Avalanche/ColorPickerHueMaterial.ColorPickerHueMaterial'"));
	SatValueMaterial = LoadObject<UMaterial>(nullptr, TEXT("Material'/Avalanche/ColorPickerBrightnessSaturdationMaterial.ColorPickerBrightnessSaturdationMaterial'"));
	SelectedValueImage = LoadObject<UTexture2D>(nullptr, TEXT("Texture2D'/Avalanche/EditorResources/ColorSelectionIcon.ColorSelectionIcon'"));

	if (!HueMaterial || !SatValueMaterial)
	{
		return;
	}

	Red = 1.f;
	Green = 0.f;
	Blue = 0.f;
	Hue = 0.f;
	Saturation = 1.f;
	Value = 1.f;

	ActiveColor = FLinearColor::White;
	InactiveColor = FLinearColor::White;
	ColorStyle = EAvaColorStyle::Solid;

	SatValueMaterialDynamic = UMaterialInstanceDynamic::Create(SatValueMaterial.Get(), GetTransientPackage());

	HueBrush = MakeShared<FSlateMaterialBrush>(*HueMaterial.Get(), FVector2f(200.f, 200.f));
	SatValueBrush = MakeShared<FSlateMaterialBrush>(*SatValueMaterialDynamic.Get(), FVector2f(80.f, 80.f));
	SelectedValueBrush = MakeShared<FSlateImageBrush>(SelectedValueImage.Get(), FVector2f(32.f, 32.f));

	const ISlateStyle& AvaLevelViewportStyle = FAvaLevelViewportStyle::Get();

	ChildSlot
	[
		SNew(SBox)
		.Padding(FMargin(3.f, 3.f, 3.f, 10.f))
		[
			SNew(SConstraintCanvas)
			+ SConstraintCanvas::Slot()
			.Alignment(FVector2D(0.f))
			.Anchors(FAnchors(0.f))
			.Offset(FMargin(0.f, 0.f, 200.f, 200.f))
			[
				SNew(SImage)
				.Image(HueBrush.Get())
			]
			+ SConstraintCanvas::Slot()
			.Alignment(FVector2D(0.f))
			.Anchors(FAnchors(0.f))
			.Offset(FMargin(60.f, 60.f, 80.f, 80.f))
			[
				SNew(SImage)
				.Image(SatValueBrush.Get())
			]
			+ SConstraintCanvas::Slot()
			.Alignment(FVector2D(0.f))
			.Anchors(FAnchors(0.f))
			.Offset(FMargin(84.f, 0.f, 16.f, 16.f))
			.Expose(HueSlot)
			[
				SNew(SImage)
				.Image(SelectedValueBrush.Get())
			]
			+ SConstraintCanvas::Slot()
			.Alignment(FVector2D(0.f))
			.Anchors(FAnchors(0.f))
			.Offset(FMargin(124.f, 44.f, 16.f, 16.f))
			.Expose(SatValueSlot)
			[
				SNew(SImage)
				.Image(SelectedValueBrush.Get())
			]
			+ SConstraintCanvas::Slot()
			.Alignment(FVector2D(0.f))
			.Anchors(FAnchors(0.f))
			.Offset(FMargin(0.f, 155.f, 15.f, 15.f))
			[
				SAssignNew(ColorStyleSwitcher, SWidgetSwitcher)
				+ SWidgetSwitcher::Slot()
				[
					SNew(SImage)
					.Image(AvaLevelViewportStyle.GetBrush("Icons.ColorPicker.SolidColors"))
					.OnMouseButtonDown(this, &SAvaLevelViewportActorColorMenu::OnColorSwitchedMouseDown)
				]
				+ SWidgetSwitcher::Slot()
				[
					SNew(SImage)
					.Image(AvaLevelViewportStyle.GetBrush("Icons.ColorPicker.LinearGradient"))
					.OnMouseButtonDown(this, &SAvaLevelViewportActorColorMenu::OnColorSwitchedMouseDown)
				]
			]
			+ SConstraintCanvas::Slot()
			.Alignment(FVector2D(0.f))
			.Anchors(FAnchors(0.f))
			.Offset(FMargin(0.f, 180.f, 30.f, 20.f))
			[
				SNew(SColorBlock)
				.Color(this, &SAvaLevelViewportActorColorMenu::GetActiveColor)
				.OnMouseButtonDown(this, &SAvaLevelViewportActorColorMenu::OnActiveColorMouseDown)
			]
			+ SConstraintCanvas::Slot()
			.Alignment(FVector2D(0.f))
			.Anchors(FAnchors(0.f))
			.Offset(FMargin(30.f, 187.f, 20.f, 13.f))
			[
				SNew(SColorBlock)
				.Color(this, &SAvaLevelViewportActorColorMenu::GetInactiveColor)
				.OnMouseButtonDown(this, &SAvaLevelViewportActorColorMenu::OnInactiveColorMouseDown)
			]
			+ SConstraintCanvas::Slot()
			.Alignment(FVector2D(0.f))
			.Anchors(FAnchors(0.f))
			.Offset(FMargin(0.f, 3.f, 43.f, 18.f))
			[
				SNew(SAvaEyeDropperButton)
				.OnValueChanged(this, &SAvaLevelViewportActorColorMenu::OnDropperValueChanged)
				.OnComplete(this, &SAvaLevelViewportActorColorMenu::OnDropperComplete)
				.DisplayGamma_UObject(GEngine, &UEngine::GetDisplayGamma)
			]
			+ SConstraintCanvas::Slot()
			.Alignment(FVector2D(0.f))
			.Anchors(FAnchors(0.f))
			.Offset(FMargin(160.f, 4.f, 43.f, 18.f))
			[
				SNew(SButton)
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText(LOCTEXT("ToggleBetweenRBGAndHSV", "Toggle between RGB and HSV"))
				.OnClicked(this, &SAvaLevelViewportActorColorMenu::OnColorFormatToggleClicked)
				[
					SAssignNew(ColorFormatButtonText, STextBlock)
					.Text(LOCTEXT("RGB", "RGB"))
					.RenderTransform(FSlateRenderTransform(FVector2f(0.f, -2.f)))
				]
			]
			+ SConstraintCanvas::Slot()
			.Alignment(FVector2D(0.f))
			.Anchors(FAnchors(0.f))
			.Offset(FMargin(176.f, 24.f, 26.f, 20.f))
			[
				SNew(SButton)
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText(LOCTEXT("UnlitToolTip", "Change Unlit/Lit"))
				.OnClicked(this, &SAvaLevelViewportActorColorMenu::OnUnlitButtonClicked)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Level.LightingScenarioIcon16x"))
					.ColorAndOpacity(this, &SAvaLevelViewportActorColorMenu::GetUnlitButtonColor)
				]
			]
			+ SConstraintCanvas::Slot()
			.Alignment(FVector2D(0.f))
			.Anchors(FAnchors(0.f))
			.Offset(FMargin(160.f, 180.f, 43.f, 18.f))
			[
				SAssignNew(ThemesButton, SButton)
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText(LOCTEXT("ThemeOptionToolTip", "Change themes"))
				.OnClicked(this, &SAvaLevelViewportActorColorMenu::OnChangeThemeButtonClicked)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("ColorPicker.ColorThemes"))
				]
			]
			+ SConstraintCanvas::Slot()
			.Alignment(FVector2D(0.f))
			.Anchors(FAnchors(0.f))
			.Offset(FMargin(210.f, 0.f, 66.f, 77.f))
			[
				SAssignNew(ColorTextInputWidgetBox, SBox)
				[
					GetRGBEntryBox()
				]
			]
			+ SConstraintCanvas::Slot()
			.Alignment(FVector2D(0.f))
			.Anchors(FAnchors(0.f))
			.Offset(FMargin(205.f, 84.f, 72.f, 111.f))
			[
				SAssignNew(ColorPalette, SGridPanel)
			]
		]
	];

	const FAvaColorChangeData& LastColorData = ColorPicker.GetLastColorData();
	SetColorStyle_Direct(LastColorData.ColorStyle);
	SetActiveColorRGB_Direct(LastColorData.PrimaryColor);
	SetInactiveColorRGB_Direct(LastColorData.SecondaryColor);

	UpdateColorPalette();
}

const FVector2f SAvaLevelViewportActorColorMenu::WheelMiddle = FVector2f(100.f, 100.f);

FReply SAvaLevelViewportActorColorMenu::OnMouseButtonDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	const FVector2f LocalMouseLocation = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());

	if (IsOverSaturationValueBlock(LocalMouseLocation))
	{
		bEditingSaturationValue = true;
	}
	else if (IsOverColorWheel(LocalMouseLocation))
	{
		bEditingHue = true;
	}

	if (bEditingSaturationValue || bEditingHue)
	{
		UpdateColorWheel(LocalMouseLocation);
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	}

	return Super::OnMouseButtonDown(InMyGeometry, InMouseEvent);
}

FReply SAvaLevelViewportActorColorMenu::OnMouseButtonUp(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (bEditingSaturationValue || bEditingHue)
	{
		EndColorSelection();
	}

	return Super::OnMouseButtonUp(InMyGeometry, InMouseEvent);
}

FReply SAvaLevelViewportActorColorMenu::OnMouseMove(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (!bEditingHue && !bEditingSaturationValue)
	{
		return Super::OnMouseMove(InMyGeometry, InMouseEvent);
	}

	const FVector2f LocalMouseLocation = InMyGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());
	UpdateColorWheel(LocalMouseLocation);

	return FReply::Handled();
}

FReply SAvaLevelViewportActorColorMenu::OnDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	const TSharedRef<FAvaColorPickerPaletteDragDropOperation> Operation = MakeShared<FAvaColorPickerPaletteDragDropOperation>(SharedThis(this));
	return FReply::Handled().BeginDragDrop(Operation);
}

void SAvaLevelViewportActorColorMenu::EndColorSelection()
{
	bEditingHue = false;
	bEditingSaturationValue = false;

	AddColorToTheme(ActiveColor);
}

void SAvaLevelViewportActorColorMenu::SetActiveColorRGB(FLinearColor InRGB)
{
	SetActiveColorRGB_Direct(InRGB);
	BroadcastColorChange();
}

void SAvaLevelViewportActorColorMenu::SetActiveColorHSV(FLinearColor InHSV)
{
	SetActiveColorHSV_Direct(InHSV);
	BroadcastColorChange();
}

void SAvaLevelViewportActorColorMenu::SetInactiveColorRGB(FLinearColor InRGB)
{
	SetInactiveColorRGB_Direct(InRGB);
	AddColorToTheme(InactiveColor);
	BroadcastColorChange();
}

void SAvaLevelViewportActorColorMenu::SetInactiveColorHSV(FLinearColor InHSV)
{
	SetInactiveColorHSV_Direct(InHSV);
	AddColorToTheme(InactiveColor);
	BroadcastColorChange();
}

void SAvaLevelViewportActorColorMenu::SwitchActiveColor()
{
	const FLinearColor OldInactiveColor = InactiveColor;
	InactiveColor = ActiveColor;
	SetActiveColorRGB(OldInactiveColor);
}

FAvaColorTheme* SAvaLevelViewportActorColorMenu::FindTheme(const FString& InName, bool bInAddIfNotFound)
{
	for (int32 ThemeIdx = 0; ThemeIdx < ColorThemes.Num(); ++ThemeIdx)
	{
		if (ColorThemes[ThemeIdx].GetName() == InName)
		{
			return &ColorThemes[ThemeIdx];
		}
	}

	if (bInAddIfNotFound)
	{
		const FAvaColorTheme Theme = FAvaColorTheme(InName);
		ColorThemes.Add(Theme);
		return &ColorThemes.Last();
	}

	return nullptr;
}

FAvaColorTheme* SAvaLevelViewportActorColorMenu::GetTheme(int32 InThemeIndex)
{
	if (ColorThemes.IsValidIndex(InThemeIndex))
	{
		return &ColorThemes[InThemeIndex];
	}

	return nullptr;
}

FString SAvaLevelViewportActorColorMenu::GetReferencerName() const
{
	return TEXT("SAvaLevelViewportActorColorMenu");
}

void SAvaLevelViewportActorColorMenu::AddReferencedObjects(FReferenceCollector& InCollector)
{
	InCollector.AddReferencedObject(HueMaterial);
	InCollector.AddReferencedObject(SatValueMaterial);
	InCollector.AddReferencedObject(SatValueMaterialDynamic);
	InCollector.AddReferencedObject(SelectedValueImage);
}

void SAvaLevelViewportActorColorMenu::ApplyActiveColor()
{
	if (IsValid(SatValueMaterialDynamic))
	{
		FLinearColor HSVColor(Hue * 360.0f, 1.f, 1.f);
		HSVColor = HSVColor.HSVToLinearRGB();
		SatValueMaterialDynamic->SetVectorParameterValue("BaseColor", FVector4(HSVColor.R, HSVColor.G, HSVColor.B, 1.f));
	}

	if (HueSlot)
	{
		const float HueRadians = Hue * PI * 2.f;
		float X = WheelMiddle.X + FMath::Sin(HueRadians) * 80.f;
		float Y = WheelMiddle.Y - FMath::Cos(HueRadians) * 80.f;

		X -= HueSlot->GetOffset().Right / 2.f;
		Y -= HueSlot->GetOffset().Bottom / 2.f;

		HueSlot->SetOffset(FMargin(X, Y, HueSlot->GetOffset().Right, HueSlot->GetOffset().Bottom));
	}

	if (SatValueSlot)
	{
		float X = 60.f + (Saturation * 80.f);
		float Y = 140.f - (Value * 80.f);

		X -= SatValueSlot->GetOffset().Right / 2.f;
		Y -= SatValueSlot->GetOffset().Bottom / 2.f;

		SatValueSlot->SetOffset(FMargin(X, Y, SatValueSlot->GetOffset().Right, SatValueSlot->GetOffset().Bottom));
	}
}

TArray<FAvaColorTheme> SAvaLevelViewportActorColorMenu::ColorThemes = TArray<FAvaColorTheme>();

void SAvaLevelViewportActorColorMenu::LoadColorThemesFromIni()
{
	if (FPaths::FileExists(GEditorPerProjectIni))
	{
		bool bThemesRemaining = true;
		int32 ThemeID = 0;
		while (bThemesRemaining)
		{
			const FString ThemeName = GConfig->GetStr(TEXT("ColorThemes"), *FString::Printf(TEXT("Theme%i"), ThemeID), GEditorPerProjectIni);
			if (!ThemeName.IsEmpty())
			{
				FAvaColorTheme* ColorTheme = SAvaLevelViewportActorColorMenu::FindTheme(ThemeName, true);
				check(ColorTheme);
				bool bColorsRemaining = true;
				int32 ColorID = 0;
				while (bColorsRemaining)
				{
					const FString ColorString = GConfig->GetStr(TEXT("ColorThemes"), *FString::Printf(TEXT("Theme%iColor%i"), ThemeID, ColorID), GEditorPerProjectIni);
					if (!ColorString.IsEmpty())
					{
						// Add the color if it hasn't already
						FLinearColor Color;
						Color.InitFromString(ColorString);
						if (ColorTheme->FindApproxColor(Color) == INDEX_NONE)
						{
							const FString LabelString = GConfig->GetStr(TEXT("ColorThemes"), *FString::Printf(TEXT("Theme%iLabel%i"), ThemeID, ColorID), GEditorPerProjectIni);
							FText Label = FText::GetEmpty();
							if (!LabelString.IsEmpty())
							{
								Label = FText::FromString(LabelString);
							}
							FAvaColorInfo NewColor = {Color,Label};
							ColorTheme->GetColors().Insert(NewColor, 0);
						}
						++ColorID;
					}
					else
					{
						bColorsRemaining = false;
					}
				}
				++ThemeID;
			}
			else
			{
				bThemesRemaining = false;
			}
		}
	}

	if (ColorThemes.Num() == 0)
	{
		FAvaColorTheme* NewTheme = FindTheme("Default", true);
		NewTheme->GetColors().Add({FLinearColor::White, LOCTEXT("WhiteColor", "White")});
		NewTheme->GetColors().Add({FLinearColor::Black, LOCTEXT("BlackColor", "Black")});
		NewTheme->GetColors().Add({FLinearColor::Red, LOCTEXT("RedColor", "Red")});
		NewTheme->GetColors().Add({FLinearColor::Green, LOCTEXT("GreenColor", "Green")});
		NewTheme->GetColors().Add({FLinearColor::Blue, LOCTEXT("BlueColor", "Blue")});
		SaveColorThemesToIni();
	}

	FAvaColorTheme* EditorAccentColors = FindTheme("Editor Accents", true);
	EditorAccentColors->GetColors().Empty();
	EditorAccentColors->GetColors().Add({FStyleColors::AccentBlue.GetSpecifiedColor(),   LOCTEXT("AccentBlue","Accent Blue")});
	EditorAccentColors->GetColors().Add({FStyleColors::AccentPurple.GetSpecifiedColor(), LOCTEXT("AccentPurple","Accent Purple")});
	EditorAccentColors->GetColors().Add({FStyleColors::AccentPink.GetSpecifiedColor(),   LOCTEXT("AccentPink","Accent Pink")});
	EditorAccentColors->GetColors().Add({FStyleColors::AccentRed.GetSpecifiedColor(),    LOCTEXT("AccentRed","Accent Red")});
	EditorAccentColors->GetColors().Add({FStyleColors::AccentOrange.GetSpecifiedColor(), LOCTEXT("AccentOrange","Accent Orange")});
	EditorAccentColors->GetColors().Add({FStyleColors::AccentYellow.GetSpecifiedColor(), LOCTEXT("AccentYellow","Accent Yellow")});
	EditorAccentColors->GetColors().Add({FStyleColors::AccentGreen.GetSpecifiedColor(),  LOCTEXT("AccentGreen","Accent Green")});
	EditorAccentColors->GetColors().Add({FStyleColors::AccentBrown.GetSpecifiedColor(),  LOCTEXT("AccentBrown","Accent Brown")});
	EditorAccentColors->GetColors().Add({FStyleColors::AccentBlack.GetSpecifiedColor(),  LOCTEXT("AccentBlack","Accent Black")});
	EditorAccentColors->GetColors().Add({FStyleColors::AccentGray.GetSpecifiedColor(),   LOCTEXT("AccentGray","Accent Gray")});
	EditorAccentColors->GetColors().Add({FStyleColors::AccentWhite.GetSpecifiedColor(),  LOCTEXT("AccentWhite","Accent White")});
	EditorAccentColors->GetColors().Add({FStyleColors::AccentFolder.GetSpecifiedColor(), LOCTEXT("AccentFolder","Accent Folder")});
	SaveColorThemesToIni();
}

void SAvaLevelViewportActorColorMenu::SaveColorThemesToIni()
{
	if (FPaths::FileExists(GEditorPerProjectIni))
	{
		GConfig->EmptySection(TEXT("ColorThemes"), GEditorPerProjectIni);
		for (int32 ThemeIndex = 0; ThemeIndex < ColorThemes.Num(); ++ThemeIndex)
		{
			FAvaColorTheme& Theme = ColorThemes[ThemeIndex];
			GConfig->SetString(TEXT("ColorThemes"), *FString::Printf(TEXT("Theme%i"), ThemeIndex), *Theme.GetName(), GEditorPerProjectIni);

			const TArray<FAvaColorInfo>& Colors = Theme.GetColors();
			for (int32 ColorIndex = 0; ColorIndex < Colors.Num(); ++ColorIndex)
			{
				const FLinearColor& Color = Colors[ColorIndex].Color;
				const FText& Label = Colors[ColorIndex].Label;
				GConfig->SetString(TEXT("ColorThemes"), *FString::Printf(TEXT("Theme%iColor%i"), ThemeIndex, ColorIndex), *Color.ToString(), GEditorPerProjectIni);
				GConfig->SetString(TEXT("ColorThemes"), *FString::Printf(TEXT("Theme%iLabel%i"), ThemeIndex, ColorIndex), *Label.ToString(), GEditorPerProjectIni);
			}
		}
	}
}

void SAvaLevelViewportActorColorMenu::UpdateColorPalette()
{
	ColorPalette->ClearChildren();

	if (ColorThemes.Num() == 0)
	{
		LoadColorThemesFromIni();
	}

	if (ActiveThemeIndex == INDEX_NONE)
	{
		ActiveThemeIndex = 0;
	}

	FAvaColorTheme* Theme = GetTheme(ActiveThemeIndex);

	if (!Theme)
	{
		ActiveThemeIndex = 0;
		Theme = GetTheme(ActiveThemeIndex);

		if (!Theme)
		{
			return;
		}
	}

	TArray<FAvaColorInfo>& ThemeColors = Theme->GetColors();

	if (ThemeColors.Num() == 0)
	{
		return;
	}

	int32 Row = 0;

	// starting from last: most recent color is shown at the top
	int32 ColorIdx = ThemeColors.Num() - 1;

	do
	{
		for (int32 Col = 0; Col < ColorsPerPaletteRow; ++Col)
		{
			ColorPalette->AddSlot(Col, Row)
				[
					SNew(SBox)
					.WidthOverride(24.f)
					.HeightOverride(16.f)
					.Padding(FMargin(1.f))
					[
						SNew(SColorBlock)
						.OnMouseButtonDown(this, &SAvaLevelViewportActorColorMenu::OnColorPaletteMouseDown, ColorIdx)
						.Color(ThemeColors[ColorIdx].Color)
						.ToolTipText(ThemeColors[ColorIdx].Label)
					]
				];

			--ColorIdx;

			if (ColorIdx < 0)
			{
				break;
			}
		}

		++Row;

		if (ColorIdx < 0)
		{
			break;
		}
	} while (true);
}

void SAvaLevelViewportActorColorMenu::AddColorToTheme(const FLinearColor& InNewColor)
{
	FAvaColorTheme* Theme = SAvaLevelViewportActorColorMenu::GetTheme(ActiveThemeIndex);

	if (!Theme)
	{
		return;
	}

	static constexpr float VarianceForSameColor = 0.001f;
	TArray<FAvaColorInfo>& ThemeColors = Theme->GetColors();

	for (int32 ColorIdx = 0; ColorIdx < ThemeColors.Num(); ++ColorIdx)
	{
		float TotalVariance = 0.f;

		TotalVariance += FMath::Abs(ThemeColors[ColorIdx].Color.R - InNewColor.R);
		TotalVariance += FMath::Abs(ThemeColors[ColorIdx].Color.G - InNewColor.G);
		TotalVariance += FMath::Abs(ThemeColors[ColorIdx].Color.B - InNewColor.B);

		if (TotalVariance < VarianceForSameColor)
		{
			// If we're almost identical to a previously set colour, set that colour instead.
			if (InNewColor == ActiveColor)
			{
				SetActiveColorRGB(ThemeColors[ColorIdx].Color);
			}
			else if (InNewColor == InactiveColor)
			{
				InactiveColor = ThemeColors[ColorIdx].Color;
			}
			return;
		}
	}

	if (ThemeColors.Num() >= MaxColorsForAutoAdd)
	{
		ThemeColors.RemoveAt(0);
	}

	const FAvaColorInfo NewColorInfo = {InNewColor, FText::GetEmpty()};
	ThemeColors.Add(NewColorInfo);

	SaveColorThemesToIni();
	UpdateColorPalette();
}

TSharedRef<SWidget> SAvaLevelViewportActorColorMenu::GetRGBEntryBox()
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		[
			SNew(SNumericEntryBox<float>)
			.MinValue(0)
			.MaxValue(1)
			.MinSliderValue(0)
			.MaxSliderValue(1)
			.Justification(ETextJustify::Right)
			.Delta(0.01f)
			.AllowSpin(true)
			.Value(this, &SAvaLevelViewportActorColorMenu::GetRed)
			.OnValueChanged(this, &SAvaLevelViewportActorColorMenu::OnRedSliderValueChanged)
			.OnValueCommitted(this, &SAvaLevelViewportActorColorMenu::OnRedValueChanged)
			.Label()
			[
				SNew(STextBlock)
				.MinDesiredWidth(10.f)
				.Margin(FMargin(0.f, 5.f, 0.f, 0.f))
				.Text(LOCTEXT("Red", "R"))
			]
		]
		+ SVerticalBox::Slot()
		[
			SNew(SNumericEntryBox<float>)
			.MinValue(0)
			.MaxValue(1)
			.MinSliderValue(0)
			.MaxSliderValue(1)
			.Justification(ETextJustify::Right)
			.Delta(0.01f)
			.AllowSpin(true)
			.Value(this, &SAvaLevelViewportActorColorMenu::GetGreen)
			.OnValueChanged(this, &SAvaLevelViewportActorColorMenu::OnGreenSliderValueChanged)
			.OnValueCommitted(this, &SAvaLevelViewportActorColorMenu::OnGreenValueChanged)
			.Label()
			[
				SNew(STextBlock)
				.MinDesiredWidth(10.f)
				.Margin(FMargin(0.f, 5.f, 0.f, 0.f))
				.Text(LOCTEXT("Green", "G"))
			]
		]
		+ SVerticalBox::Slot()
		[
			SNew(SNumericEntryBox<float>)
			.MinValue(0)
			.MaxValue(1)
			.MinSliderValue(0)
			.MaxSliderValue(1)
			.Justification(ETextJustify::Right)
			.Delta(0.01f)
			.AllowSpin(true)
			.Value(this, &SAvaLevelViewportActorColorMenu::GetBlue)
			.OnValueChanged(this, &SAvaLevelViewportActorColorMenu::OnBlueSliderValueChanged)
			.OnValueCommitted(this, &SAvaLevelViewportActorColorMenu::OnBlueValueChanged)
			.Label()
			[
				SNew(STextBlock)
				.MinDesiredWidth(10.f)
				.Margin(FMargin(0.f, 5.f, 0.f, 0.f))
				.Text(LOCTEXT("Blue", "B"))
			]
		];
}

TSharedRef<SWidget> SAvaLevelViewportActorColorMenu::GetHSLEntryBox()
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		[
			SNew(SNumericEntryBox<float>)
			.MinValue(0)
			.MaxValue(1)
			.MinSliderValue(0)
			.MaxSliderValue(1)
			.Justification(ETextJustify::Right)
			.Delta(0.01f)
			.AllowSpin(true)
			.Value(this, &SAvaLevelViewportActorColorMenu::GetHue)
			.OnValueChanged(this, &SAvaLevelViewportActorColorMenu::OnHueSliderValueChanged)
			.OnValueCommitted(this, &SAvaLevelViewportActorColorMenu::OnHueValueChanged)
			.Label()
			[
				SNew(STextBlock)
				.MinDesiredWidth(10.f)
				.Margin(FMargin(0.f, 5.f, 0.f, 0.f))
				.Text(LOCTEXT("Hue", "H"))
			]
		]
		+ SVerticalBox::Slot()
		[
			SNew(SNumericEntryBox<float>)
			.MinValue(0)
			.MaxValue(1)
			.MinSliderValue(0)
			.MaxSliderValue(1)
			.Justification(ETextJustify::Right)
			.Delta(0.01f)
			.AllowSpin(true)
			.Value(this, &SAvaLevelViewportActorColorMenu::GetSaturation)
			.OnValueChanged(this, &SAvaLevelViewportActorColorMenu::OnSaturationSliderValueChanged)
			.OnValueCommitted(this, &SAvaLevelViewportActorColorMenu::OnSaturationValueChanged)
			.Label()
			[
				SNew(STextBlock)
				.MinDesiredWidth(10.f)
				.Margin(FMargin(0.f, 5.f, 0.f, 0.f))
				.Text(LOCTEXT("Saturation", "S"))
			]
		]
		+ SVerticalBox::Slot()
		[
			SNew(SNumericEntryBox<float>)
			.MinValue(0)
			.MaxValue(1)
			.MinSliderValue(0)
			.MaxSliderValue(1)
			.Justification(ETextJustify::Right)
			.Delta(0.01f)
			.AllowSpin(true)
			.Value(this, &SAvaLevelViewportActorColorMenu::GetValue)
			.OnValueChanged(this, &SAvaLevelViewportActorColorMenu::OnValueSliderValueChanged)
			.OnValueCommitted(this, &SAvaLevelViewportActorColorMenu::OnValueValueChanged)
			.Label()
			[
				SNew(STextBlock)
				.MinDesiredWidth(10.f)
				.Margin(FMargin(0.f, 5.f, 0.f, 0.f))
				.Text(LOCTEXT("Value", "V"))
			]
		];
}

bool SAvaLevelViewportActorColorMenu::IsOverSaturationValueBlock(const FVector2f& InLocalMouseLocation) const
{
	if (InLocalMouseLocation.X >= 55.f && InLocalMouseLocation.X < 145.f
		&& InLocalMouseLocation.Y >= 55.f && InLocalMouseLocation.Y < 145.f)
	{
		return true;
	}

	return false;
}

bool SAvaLevelViewportActorColorMenu::IsOverColorWheel(const FVector2f& InLocalMouseLocation) const
{
	const float Distance = (InLocalMouseLocation - WheelMiddle).Size();

	if (Distance >= 62.f && Distance < 98.f)
	{
		return true;
	}

	return false;
}

bool SAvaLevelViewportActorColorMenu::IsOverColorSelector(const FVector2f& InLocalMouseLocation) const
{
	if (InLocalMouseLocation.X >= 0.f && InLocalMouseLocation.X < 200.f
		&& InLocalMouseLocation.Y >= 0.f && InLocalMouseLocation.Y < 200.f)
	{
		return true;
	}

	return false;
}

void SAvaLevelViewportActorColorMenu::UpdateColorWheel(const FVector2f& InLocalMouseLocation)
{
	if (bEditingSaturationValue)
	{
		SetSaturationValue(FMath::Clamp((InLocalMouseLocation.X - 60.f), 0.f, 80.f) / 80.f, false);
		SetValueValue(FMath::Clamp((140.f - InLocalMouseLocation.Y), 0.f, 80.f) / 80.f, false);
	}
	else if (bEditingHue)
	{
		float Angle = FMath::Atan2(InLocalMouseLocation.X - WheelMiddle.X, WheelMiddle.Y - InLocalMouseLocation.Y) / PI;

		if (Angle < 0.f)
		{
			Angle += 2.f;
		}

		if (Saturation == 0.f || Value == 0.f)
		{
			SetSaturationValue(1.0f, false);
			SetValueValue(1.0f, false);
		}

		SetHueValue(Angle * 0.5f, false);
	}
}

FReply SAvaLevelViewportActorColorMenu::OnColorFormatToggleClicked()
{
	if (bIsRGB)
	{
		ColorTextInputWidgetBox->SetContent(GetHSLEntryBox());
		ColorFormatButtonText->SetText(LOCTEXT("HSV", "HSV"));
		bIsRGB = false;
	}
	else
	{
		ColorTextInputWidgetBox->SetContent(GetRGBEntryBox());
		ColorFormatButtonText->SetText(LOCTEXT("RGB", "RGB"));
		bIsRGB = true;
	}

	return FReply::Handled();
}

FReply SAvaLevelViewportActorColorMenu::OnActiveColorMouseDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	const FString ToCopy = FString::Printf(TEXT("(R=%f,G=%f,B=%f,A=%f)"), ActiveColor.R, ActiveColor.G, ActiveColor.B, ActiveColor.A);
	FPlatformApplicationMisc::ClipboardCopy(*ToCopy);

	return FReply::Handled();
}

FReply SAvaLevelViewportActorColorMenu::OnInactiveColorMouseDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	SwitchActiveColor();

	return FReply::Handled();
}

FReply SAvaLevelViewportActorColorMenu::OnColorPaletteMouseDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent, int32 InColorIdx)
{
	FAvaColorTheme* Theme = SAvaLevelViewportActorColorMenu::GetTheme(ActiveThemeIndex);

	if (!Theme)
	{
		return FReply::Handled();
	}

	TArray<FAvaColorInfo>& ThemeColors = Theme->GetColors();

	if (!ThemeColors.IsValidIndex(InColorIdx))
	{
		return FReply::Handled();
	}

	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		const bool bDeleteColor = FSlateApplication::Get().GetModifierKeys().IsControlDown();

		if (bDeleteColor)
		{
			ThemeColors.RemoveAt(InColorIdx);
			SaveColorThemesToIni();
			UpdateColorPalette();
		}
		else
		{
			SetActiveColorRGB(ThemeColors[InColorIdx].Color);

			const FString ToCopy = FString::Printf(TEXT("(R=%f,G=%f,B=%f,A=%f)"),
				ThemeColors[InColorIdx].Color.R, ThemeColors[InColorIdx].Color.G, ThemeColors[InColorIdx].Color.B, ThemeColors[InColorIdx].Color.A);
			FPlatformApplicationMisc::ClipboardCopy(*ToCopy);
		}
	}
	else if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		if (ColorPalette.IsValid())
		{
			const TSharedPtr<SWidget> ColorWidget = ColorPalette->GetChildren()->GetChildAt(InColorIdx);

			if (ColorWidget.IsValid())
			{
				FSlateApplication::Get().PushMenu(
					ColorWidget.ToSharedRef(),
					InMouseEvent.GetEventPath() ? *InMouseEvent.GetEventPath() : FWidgetPath(),
					BuildRenameColorMenu(InColorIdx),
					InMouseEvent.GetScreenSpacePosition() - FVector2f(100.f, 0.f),
					FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
				);
			}
		}
	}

	return FReply::Handled();
}

FReply SAvaLevelViewportActorColorMenu::OnChangeThemeButtonClicked()
{
	FSlateApplication::Get().PushMenu(
		ThemesButton.ToSharedRef(),
		FWidgetPath(),
		BuildThemesMenu(),
		FSlateApplication::Get().GetCursorPos() - FVector2f(100.f, 0.f),
		FPopupTransitionEffect(FPopupTransitionEffect::ComboButton)
	);

	return FReply::Handled();
}

FReply SAvaLevelViewportActorColorMenu::OnUnlitButtonClicked()
{
	bIsUnlit = !bIsUnlit;
	BroadcastColorChange();

	return FReply::Handled();
}

void SAvaLevelViewportActorColorMenu::OnRedSliderValueChanged(float InNewValue)
{
	SetRedValue(InNewValue, false);
}

void SAvaLevelViewportActorColorMenu::OnGreenSliderValueChanged(float InNewValue)
{
	SetGreenValue(InNewValue, false);
}

void SAvaLevelViewportActorColorMenu::OnBlueSliderValueChanged(float InNewValue)
{
	SetBlueValue(InNewValue, false);
}

void SAvaLevelViewportActorColorMenu::OnHueSliderValueChanged(float InNewValue)
{
	SetHueValue(InNewValue, false);
}

void SAvaLevelViewportActorColorMenu::OnSaturationSliderValueChanged(float InNewValue)
{
	SetSaturationValue(InNewValue, false);
}

void SAvaLevelViewportActorColorMenu::OnValueSliderValueChanged(float InNewValue)
{
	SetValueValue(InNewValue, false);
}

void SAvaLevelViewportActorColorMenu::OnRedValueChanged(float InNewValue, ETextCommit::Type InCommitType)
{
	SetRedValue(InNewValue, InCommitType == ETextCommit::OnEnter);
}

void SAvaLevelViewportActorColorMenu::OnGreenValueChanged(float InNewValue, ETextCommit::Type InCommitType)
{
	SetGreenValue(InNewValue, InCommitType == ETextCommit::OnEnter);
}

void SAvaLevelViewportActorColorMenu::OnBlueValueChanged(float InNewValue, ETextCommit::Type InCommitType)
{
	SetBlueValue(InNewValue, InCommitType == ETextCommit::OnEnter);
}

void SAvaLevelViewportActorColorMenu::OnHueValueChanged(float InNewValue, ETextCommit::Type InCommitType)
{
	SetHueValue(InNewValue, InCommitType == ETextCommit::OnEnter);
}

void SAvaLevelViewportActorColorMenu::OnSaturationValueChanged(float InNewValue, ETextCommit::Type InCommitType)
{
	SetSaturationValue(InNewValue, InCommitType == ETextCommit::OnEnter);
}

void SAvaLevelViewportActorColorMenu::OnValueValueChanged(float InNewValue, ETextCommit::Type InCommitType)
{
	SetValueValue(InNewValue, InCommitType == ETextCommit::OnEnter);
}

TSharedRef<SWidget> SAvaLevelViewportActorColorMenu::BuildRenameColorMenu(int32 InColorIdx)
{
	FAvaColorTheme* Theme = SAvaLevelViewportActorColorMenu::GetTheme(ActiveThemeIndex);

	if (!Theme)
	{
		return SNullWidget::NullWidget;
	}

	TArray<FAvaColorInfo>& ThemeColors = Theme->GetColors();

	if (!ThemeColors.IsValidIndex(InColorIdx))
	{
		return SNullWidget::NullWidget;
	}

	if (!ColorPalette.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	const TSharedPtr<SBox> ColorBoxWidget = StaticCastSharedRef<SBox>(ColorPalette->GetChildren()->GetChildAt(InColorIdx));

	if (!ColorBoxWidget.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<SWidget> ColorWidget = ColorBoxWidget->GetChildren()->GetChildAt(0);

	if (!ColorWidget.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	FLinearColor CurrentColor = ThemeColors[InColorIdx].Color;

	constexpr bool bInShouldCloseWindowAfterMenuSelection = false;

	FMenuBuilder ColorMenuBuilder(bInShouldCloseWindowAfterMenuSelection, nullptr);
	{
		ColorMenuBuilder.BeginSection("ColorOptions", LOCTEXT("ColorOptions", "Color Options"));

		ColorMenuBuilder.AddEditableText(LOCTEXT("NameLabel", "Name"), LOCTEXT("NameTooltip", "The name of the color."),
			FSlateIcon(), ThemeColors[InColorIdx].Label,
			FOnTextCommitted::CreateLambda([Theme, ColorWidget, InColorIdx, CurrentColor](const FText& NewLabel, ETextCommit::Type CommitType)
				{
					TArray<FAvaColorInfo>& ThemeColors = Theme->GetColors();

					if (!ThemeColors.IsValidIndex(InColorIdx))
					{
						return;
					}

					// In case the underlying array changed somehow during the menu operation
					if (ThemeColors[InColorIdx].Color != CurrentColor)
					{
						return;
					}

					ThemeColors[InColorIdx].Label = NewLabel;
					SaveColorThemesToIni();

					ColorWidget->SetToolTipText(NewLabel);
				}));

		ColorMenuBuilder.AddMenuEntry(LOCTEXT("RemovePaletteColorLabel", "Remove"), LOCTEXT("RemovePaletteColorTooltip", "Remove this color from the palette."), FSlateIcon(),
			FUIAction(FExecuteAction::CreateSPLambda(this, [this, Theme, InColorIdx, CurrentColor]()
				{
					TArray<FAvaColorInfo>& ThemeColors = Theme->GetColors();

					if (!ThemeColors.IsValidIndex(InColorIdx))
					{
						return;
					}

					// In case the underlying array changed somehow during the menu operation
					if (ThemeColors[InColorIdx].Color != CurrentColor)
					{
						return;
					}

					ThemeColors.RemoveAt(InColorIdx);
					SaveColorThemesToIni();

					UpdateColorPalette();

					FSlateApplication::Get().DismissAllMenus();
				})));

		ColorMenuBuilder.EndSection();
	}

	return ColorMenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SAvaLevelViewportActorColorMenu::BuildThemesMenu()
{
	/*
	 * Causes the action to not execute if this is true. The widget (from the SP Lambda) is destroyed
	 * prior to the action executing. Using FSlateApplication to dismiss the menu manually.
	 */
	constexpr bool bInShouldCloseWindowAfterMenuSelection = false;
	constexpr bool bInOpenSubMenuOnClick = false;

	FMenuBuilder ThemeMenuBuilder(bInShouldCloseWindowAfterMenuSelection, nullptr);
	{
		static FText RemoveThemeLabelText = LOCTEXT("RemoveThemeLabel", "Remove");
		static FText RemoveThemeTooltipText = LOCTEXT("RemoveThemeTooltip", "Remove this theme.");

		if (ColorThemes.IsValidIndex(ActiveThemeIndex))
		{
			ThemeMenuBuilder.BeginSection("CurrentTheme", LOCTEXT("CurrentTheme", "Current Theme"));

			ThemeMenuBuilder.AddWidget(
				SNew(SEditableTextBox)
				.Text(FText::FromString(ColorThemes[ActiveThemeIndex].GetName()))
				.AllowContextMenu(false)
				.SelectAllTextWhenFocused(true)
				.RevertTextOnEscape(true)
				.MinDesiredWidth(100.f)
				.IsReadOnly(false)
				.ClearKeyboardFocusOnCommit(true)
				.OnTextCommitted(FOnTextCommitted::CreateSPLambda(
					this,
					[this](const FText& NewName, ETextCommit::Type CommitType)
					{
						if (CommitType == ETextCommit::OnEnter)
						{
							if (!ColorThemes.IsValidIndex(ActiveThemeIndex))
							{
								return;
							}

							const FString NewNameString = NewName.ToString();

							for (int32 Index = 0; Index < ColorThemes.Num(); ++Index)
							{
									if (Index != ActiveThemeIndex && ColorThemes[Index].GetName().Equals(NewNameString, ESearchCase::IgnoreCase))
								{
									UE_LOGF(AvaLevelViewportLog, Error, "Theme with that name already exists.");
									return;
								}
							}

							ColorThemes[ActiveThemeIndex].SetName(NewNameString);
							SaveColorThemesToIni();
						}
					})
				),
				LOCTEXT("ThemeNameLabel", "Name")
			);

			ThemeMenuBuilder.AddMenuEntry(RemoveThemeLabelText, RemoveThemeTooltipText, FSlateIcon(),
				FUIAction(FExecuteAction::CreateSPLambda(this, [this]()
					{
						if (!ColorThemes.IsValidIndex(ActiveThemeIndex))
						{
							return;
						}

						ColorThemes.RemoveAt(ActiveThemeIndex);

						if (ActiveThemeIndex > 0)
						{
							--ActiveThemeIndex;
						}

						SaveColorThemesToIni();
						UpdateColorPalette();

						FSlateApplication::Get().DismissAllMenus();
					}),
					FCanExecuteAction::CreateSPLambda(this, [this]() { return ColorThemes.Num() > 1; })
				)
			);

			ThemeMenuBuilder.EndSection();
		}

		ThemeMenuBuilder.BeginSection("SelectTheme", LOCTEXT("SelectTheme", "Select Theme"));

		for (int32 ThemeIdx = 0; ThemeIdx < ColorThemes.Num(); ++ThemeIdx)
		{
			ThemeMenuBuilder.AddSubMenu(
				FText::FromString(ColorThemes[ThemeIdx].GetName()),
				ActiveThemeIndex == ThemeIdx
					? LOCTEXT("CurrentThemeTooltip", "This is the current theme.")
					: LOCTEXT("ChangeToTheme", "Change to this theme."),
				FNewMenuDelegate::CreateSPLambda(
					this, 
					[this, ThemeIdx](FMenuBuilder& SubMenuBuilder)
					{
						SubMenuBuilder.BeginSection("ThemeOptions", LOCTEXT("Options", "Options"));

						SubMenuBuilder.AddMenuEntry(RemoveThemeLabelText, RemoveThemeTooltipText, FSlateIcon(),
							FUIAction(FExecuteAction::CreateSPLambda(this, [this, ThemeIdx]()
								{
									if (!ColorThemes.IsValidIndex(ThemeIdx))
									{
										return;
									}

									ColorThemes.RemoveAt(ThemeIdx);

									if (ActiveThemeIndex == ThemeIdx && ActiveThemeIndex > 0)
									{
										--ActiveThemeIndex;
									}

									SaveColorThemesToIni();
									UpdateColorPalette();

									FSlateApplication::Get().DismissAllMenus();
								}),
								FCanExecuteAction::CreateSPLambda(this, [this]() { return ColorThemes.Num() > 1; }))
						);

						SubMenuBuilder.EndSection();

						if (ColorThemes.IsValidIndex(ThemeIdx) && ColorThemes[ThemeIdx].GetColors().Num() > 0)
						{
							SubMenuBuilder.BeginSection("Colors", LOCTEXT("Colors", "Colors"));

							for (int32 ColorIdx = 0; ColorIdx < ColorThemes[ThemeIdx].GetColors().Num(); ++ColorIdx)
							{
								SubMenuBuilder.AddSubMenu(
									FUIAction(FExecuteAction::CreateSPLambda(this, [this, ThemeIdx, ColorIdx]()
										{
											if (!ColorThemes.IsValidIndex(ThemeIdx))
											{
												return;
											}

											if (!ColorThemes[ThemeIdx].GetColors().IsValidIndex(ColorIdx))
											{
												return;
											}

											SetActiveColorRGB(ColorThemes[ThemeIdx].GetColors()[ColorIdx].Color);

											FSlateApplication::Get().DismissAllMenus();
										})
									),
									SNew(SHorizontalBox)
										+ SHorizontalBox::Slot()
										.AutoWidth()
										[
											SNew(SBox)
											.Padding(FMargin(1.f))
											.WidthOverride(30.f)
											[
												SNew(SColorBlock)
												.Color(ColorThemes[ThemeIdx].GetColors()[ColorIdx].Color)
											]
										]
										+ SHorizontalBox::Slot()
										.AutoWidth()
										.VAlign(EVerticalAlignment::VAlign_Center)
										.Padding(FMargin(5.f, 0.5f, 4.f, 0.f))
										[
											SNew(STextBlock)
												.Text(ColorThemes[ThemeIdx].GetColors()[ColorIdx].Label)
										],
									FNewMenuDelegate::CreateSPLambda(this, [this, ThemeIdx, ColorIdx](FMenuBuilder& SubMenuBuilder)
										{
											SubMenuBuilder.AddMenuEntry(LOCTEXT("RemoveColorLabel", "Remove"), LOCTEXT("RemoveColorTooltip", "Remove this color."), FSlateIcon(),
												FUIAction(FExecuteAction::CreateSPLambda(this, [this, ThemeIdx, ColorIdx]()
													{
														if (!ColorThemes.IsValidIndex(ThemeIdx))
														{
															return;
														}

														if (!ColorThemes[ThemeIdx].GetColors().IsValidIndex(ColorIdx))
														{
															return;
														}

														ColorThemes[ThemeIdx].GetColors().RemoveAt(ColorIdx);

														SaveColorThemesToIni();

														FSlateApplication::Get().DismissAllMenus();
													})
												));
										})
									);
							}

							SubMenuBuilder.EndSection();
						}
					}
				),
				FUIAction(
					FExecuteAction::CreateSPLambda(this, [this, ThemeIdx]()
						{
							if (!ColorThemes.IsValidIndex(ThemeIdx))
							{
								return;
							}

							ActiveThemeIndex = ThemeIdx;
							UpdateColorPalette();

							FSlateApplication::Get().DismissAllMenus();
						}),
					FCanExecuteAction::CreateSPLambda(this, [this, ThemeIdx]()
						{
							return ActiveThemeIndex != ThemeIdx;
						}),
					FGetActionCheckState::CreateSPLambda(this, [this, ThemeIdx]()
						{
							return ActiveThemeIndex == ThemeIdx ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						})
				),
				FName(ColorThemes[ThemeIdx].GetName()),
				EUserInterfaceActionType::RadioButton,
				bInOpenSubMenuOnClick,
				FSlateIcon(),
				bInShouldCloseWindowAfterMenuSelection
			);
		}

		ThemeMenuBuilder.EndSection();

		ThemeMenuBuilder.BeginSection("NewTheme", LOCTEXT("NewThemeSectionHeading", "New Theme"));

		ThemeMenuBuilder.AddWidget(
			SNew(SEditableTextBox)
			.Text(LOCTEXT("NewTheme", "New Theme"))
			.AllowContextMenu(false)
			.SelectAllTextWhenFocused(true)
			.RevertTextOnEscape(true)
			.MinDesiredWidth(100.f)
			.IsReadOnly(false)
			.ClearKeyboardFocusOnCommit(true)
			.OnTextCommitted(FOnTextCommitted::CreateSPLambda(this,
				[this](const FText& NewName, ETextCommit::Type CommitType)
				{
					if (CommitType == ETextCommit::OnEnter)
					{
						const FString NewNameString = NewName.ToString();

						for (int32 Index = 0; Index < ColorThemes.Num(); ++Index)
						{
							if (ColorThemes[Index].GetName().Equals(NewNameString, ESearchCase::IgnoreCase))
							{
								UE_LOGF(AvaLevelViewportLog, Error, "Theme with that name already exists.");
								return;
							}
						}

						ActiveThemeIndex = ColorThemes.Add(FAvaColorTheme(NewNameString));
						SaveColorThemesToIni();
						UpdateColorPalette();
					}
				})
			),
			LOCTEXT("NewThemeLabel", "Name")
		);

		ThemeMenuBuilder.EndSection();
	}

	return ThemeMenuBuilder.MakeWidget();
}

void SAvaLevelViewportActorColorMenu::OnDropperValueChanged(FLinearColor InNewColor)
{
	SetActiveColorRGB(InNewColor);
}

void SAvaLevelViewportActorColorMenu::OnDropperComplete(bool bCancelled)
{
	if (!bCancelled)
	{
		AddColorToTheme(ActiveColor);
		const FString ToCopy = FString::Printf(TEXT("(R=%f,G=%f,B=%f,A=%f)"), ActiveColor.R, ActiveColor.G, ActiveColor.B, ActiveColor.A);
		FPlatformApplicationMisc::ClipboardCopy(*ToCopy);
	}
}

void SAvaLevelViewportActorColorMenu::CopyRGBtoHSV()
{
	const FLinearColor HSLColor = ActiveColor.LinearRGBToHSV();
	Hue = HSLColor.R / 360.f;
	Saturation = HSLColor.G;
	Value = HSLColor.B;
}

void SAvaLevelViewportActorColorMenu::CopyHSVtoRGB()
{
	const FLinearColor HSVColor(Hue * 360.0f, Saturation, Value);
	ActiveColor = HSVColor.HSVToLinearRGB();

	Red = ActiveColor.R;
	Green = ActiveColor.G;
	Blue = ActiveColor.B;

	ActiveColor.R = Red;
	ActiveColor.G = Green;
	ActiveColor.B = Blue;
}

FReply SAvaLevelViewportActorColorMenu::OnColorSwitchedMouseDown(const FGeometry& InMyGeometry, const FPointerEvent& InPointerEvent)
{
	switch (ColorStyle)
	{
		case EAvaColorStyle::Solid:
			SetColorStyle(EAvaColorStyle::LinearGradient);
			break;

		case EAvaColorStyle::LinearGradient:
			SetColorStyle(EAvaColorStyle::Solid);
			break;

		default:
			return FReply::Unhandled();
	}

	return FReply::Handled();
}

void SAvaLevelViewportActorColorMenu::SetRedValue(float InNewValue, bool bInAddToTheme)
{
	Red = InNewValue;
	ActiveColor.R = InNewValue;

	CopyRGBtoHSV();
	ApplyActiveColor();
	BroadcastColorChange();

	if (bInAddToTheme)
	{
		AddColorToTheme(ActiveColor);
	}
}

void SAvaLevelViewportActorColorMenu::SetGreenValue(float InNewValue, bool bInAddToTheme)
{
	Green = InNewValue;
	ActiveColor.G = InNewValue;

	CopyRGBtoHSV();
	ApplyActiveColor();
	BroadcastColorChange();

	if (bInAddToTheme)
	{
		AddColorToTheme(ActiveColor);
	}
}

void SAvaLevelViewportActorColorMenu::SetBlueValue(float InNewValue, bool bInAddToTheme)
{
	Blue = InNewValue;
	ActiveColor.B = InNewValue;

	CopyRGBtoHSV();
	ApplyActiveColor();
	BroadcastColorChange();

	if (bInAddToTheme)
	{
		AddColorToTheme(ActiveColor);
	}
}

void SAvaLevelViewportActorColorMenu::SetHueValue(float InNewValue, bool bInAddToTheme)
{
	Hue = InNewValue;

	CopyHSVtoRGB();
	ApplyActiveColor();
	BroadcastColorChange();

	if (bInAddToTheme)
	{
		AddColorToTheme(ActiveColor);
	}
}

void SAvaLevelViewportActorColorMenu::SetSaturationValue(float InNewValue, bool InbAddToTheme)
{
	Saturation = InNewValue;

	CopyHSVtoRGB();
	ApplyActiveColor();
	BroadcastColorChange();

	if (InbAddToTheme)
	{
		AddColorToTheme(ActiveColor);
	}
}

void SAvaLevelViewportActorColorMenu::SetValueValue(float InNewValue, bool InbAddToTheme)
{
	Value = InNewValue;

	CopyHSVtoRGB();
	ApplyActiveColor();
	BroadcastColorChange();

	if (InbAddToTheme)
	{
		AddColorToTheme(ActiveColor);
	}
}

void SAvaLevelViewportActorColorMenu::SetColorStyle(EAvaColorStyle InNewStyle)
{
	if (ColorStyle == InNewStyle || ColorStyle == EAvaColorStyle::None)
	{
		return;
	}

	SetColorStyle_Direct(InNewStyle);
	OnColorStyleChanged();
}

#undef LOCTEXT_NAMESPACE
