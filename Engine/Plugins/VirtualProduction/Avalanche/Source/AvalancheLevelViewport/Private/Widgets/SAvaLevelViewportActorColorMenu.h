// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "AvaDefs.h"
#include "ColorPicker/AvaLevelColorPicker.h"
#include "Materials/Material.h"
#include "Templates/SharedPointer.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/Layout/SConstraintCanvas.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

class IAvaEditor;
class IToolkitHost;
class SAvaLevelViewportActorColorMenu;
class SBox;
class SButton;
class SColorBlock;
class SGridPanel;
class STextBlock;
class UMaterialInstanceDynamic;
struct FSlateImageBrush;
struct FSlateMaterialBrush;

struct FAvaColorInfo
{
	FLinearColor Color;
	FText Label;
};

struct FAvaColorTheme
{
	FAvaColorTheme()
		:FAvaColorTheme("Theme")
	{
	}

	FAvaColorTheme(const FString& InName)
	{
		Name = InName;
	}

	const FString& GetName() const { return Name; }
	void SetName(const FString& InNewName) { Name = InNewName; }

	TArray<FAvaColorInfo>& GetColors()
	{
		return Colors;
	}

	int32 FindApproxColor(const FLinearColor& InColor, float InTolerance = KINDA_SMALL_NUMBER) const
	{
		for (int32 ColorIndex = 0; ColorIndex < Colors.Num(); ++ColorIndex)
		{
			FLinearColor ApproxColor = Colors[ColorIndex].Color;
			if (ApproxColor.Equals(InColor, InTolerance))
			{
				return ColorIndex;
			}
		}

		return INDEX_NONE;
	}

protected:
	FString Name;
	TArray<FAvaColorInfo> Colors;
};

class SAvaLevelViewportActorColorMenu : public SCompoundWidget, public FGCObject
{
	SLATE_DECLARE_WIDGET(SAvaLevelViewportActorColorMenu, SCompoundWidget)

	SLATE_BEGIN_ARGS(SAvaLevelViewportActorColorMenu) {}
	SLATE_END_ARGS()

public:
	static TSharedRef<SAvaLevelViewportActorColorMenu> CreateMenu(const TSharedRef<IToolkitHost>& InToolkitHost);

	static FAvaColorTheme* FindTheme(const FString& InName, bool bInAddIfNotFound = false);
	static FAvaColorTheme* GetTheme(int32 InThemeIndex);

	void Construct(const FArguments& InArgs, const TSharedRef<IToolkitHost>& InToolkitHost);

	virtual FReply OnMouseButtonDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	void UpdateColorWheel(const FVector2f& InLocalMouseLocation);
	void EndColorSelection();

	FLinearColor GetActiveColor() const { return ActiveColor; }
	FLinearColor GetInactiveColor() const { return InactiveColor; }
	EAvaColorStyle GetColorStyle() const { return ColorStyle; }

	void SetActiveColorRGB(FLinearColor InRGB);
	void SetActiveColorHSV(FLinearColor InHSV);

	void SetInactiveColorRGB(FLinearColor InRGB);
	void SetInactiveColorHSV(FLinearColor InHSV);

	void SetColorStyle(EAvaColorStyle InNewStyle);

	void SwitchActiveColor();

	void SetRedValue(float InNewValue, bool bInAddToTheme);
	void SetGreenValue(float InNewValue, bool bInAddToTheme);
	void SetBlueValue(float InNewValue, bool bInAddToTheme);
	void SetHueValue(float InNewValue, bool bInAddToTheme);
	void SetSaturationValue(float InNewValue, bool InbAddToTheme);
	void SetValueValue(float InNewValue, bool InbAddToTheme);

private:
	static const FVector2f WheelMiddle;
	static const inline int32 MaxColorsForAutoAdd = 21;
	static constexpr int32 ColorsPerPaletteRow = 3;

	static int32 ActiveThemeIndex;
	static TArray<FAvaColorTheme> ColorThemes;

	static void LoadColorThemesFromIni();
	static void SaveColorThemesToIni();

	FAvaLevelColorPicker ColorPicker;

	TObjectPtr<UMaterial> HueMaterial;
	TObjectPtr<UMaterial> SatValueMaterial;
	TObjectPtr<UTexture2D> SelectedValueImage;
	TObjectPtr<UMaterialInstanceDynamic> SatValueMaterialDynamic;

	TSharedPtr<FSlateMaterialBrush> HueBrush;
	TSharedPtr<FSlateMaterialBrush> SatValueBrush;
	TSharedPtr<FSlateImageBrush> SelectedValueBrush;

	float Red;
	float Green;
	float Blue;

	float Hue;
	float Saturation;
	float Value;

	FLinearColor ActiveColor;
	FLinearColor InactiveColor;

	EAvaColorStyle ColorStyle = EAvaColorStyle::Solid;
	bool bIsUnlit = true;

	bool bIsRGB = true;
	TSharedPtr<STextBlock> ColorFormatButtonText;
	TSharedPtr<SBox> ColorTextInputWidgetBox;

	SConstraintCanvas::FSlot* HueSlot = nullptr;
	SConstraintCanvas::FSlot* SatValueSlot = nullptr;

	TSharedPtr<SGridPanel> ColorPalette;
	TSharedPtr<SButton> ThemesButton;
	TSharedPtr<SWidgetSwitcher> ColorStyleSwitcher;

	bool bEditingHue;
	bool bEditingSaturationValue;

	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FGCObject

	void ApplyActiveColor();
	void UpdateColorPalette();
	void AddColorToTheme(const FLinearColor& InNewColor);

	TSharedRef<SWidget> GetRGBEntryBox();
	TSharedRef<SWidget> GetHSLEntryBox();

	bool IsOverSaturationValueBlock(const FVector2f& InLocalMouseLocation) const;
	bool IsOverColorWheel(const FVector2f& InLocalMouseLocation) const;
	bool IsOverColorSelector(const FVector2f& InLocalMouseLocation) const;

	FReply OnColorFormatToggleClicked();
	FReply OnActiveColorMouseDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);
	FReply OnInactiveColorMouseDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);
	FReply OnColorPaletteMouseDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent, int32 InColorIdx);
	FReply OnChangeThemeButtonClicked();
	FReply OnUnlitButtonClicked();

	void OnRedSliderValueChanged(float InNewValue);
	void OnGreenSliderValueChanged(float InNewValue);
	void OnBlueSliderValueChanged(float InNewValue);
	void OnHueSliderValueChanged(float InNewValue);
	void OnSaturationSliderValueChanged(float InNewValue);
	void OnValueSliderValueChanged(float InNewValue);

	void OnRedValueChanged(float InNewValue, ETextCommit::Type InCommitType);
	void OnGreenValueChanged(float InNewValue, ETextCommit::Type InCommitType);
	void OnBlueValueChanged(float InNewValue, ETextCommit::Type InCommitType);
	void OnHueValueChanged(float InNewValue, ETextCommit::Type InCommitType);
	void OnSaturationValueChanged(float InNewValue, ETextCommit::Type InCommitType);
	void OnValueValueChanged(float InNewValue, ETextCommit::Type InCommitType);

	TSharedRef<SWidget> BuildRenameColorMenu(int32 InColorIdx);
	TSharedRef<SWidget> BuildThemesMenu();

	void OnDropperValueChanged(FLinearColor InNewColor);
	void OnDropperComplete(bool bInSuccess);

	TOptional<float> GetRed() const { return Red; }
	TOptional<float> GetGreen() const { return Green; }
	TOptional<float> GetBlue() const { return Blue; }
	TOptional<float> GetHue() const { return Hue; }
	TOptional<float> GetSaturation() const { return Saturation; }
	TOptional<float> GetValue() const { return Value; }

	FSlateColor GetUnlitButtonColor() const;

	void CopyRGBtoHSV();
	void CopyHSVtoRGB();

	FReply OnColorSwitchedMouseDown(const FGeometry& InMyGeometry, const FPointerEvent& InPointerEvent);

	void OnColorStyleChanged();
	void BroadcastColorChange();

	void SetActiveColorRGB_Direct(FLinearColor InRGB);
	void SetActiveColorHSV_Direct(FLinearColor InHSV);

	void SetInactiveColorRGB_Direct(FLinearColor InRGB);
	void SetInactiveColorHSV_Direct(FLinearColor InHSV);

	void SetColorStyle_Direct(EAvaColorStyle InStyle);
};
