// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAvaLevelViewportCameraBounds.h"
#include "AvaViewportSettings.h"
#include "AvaVisibleArea.h"
#include "Editor/EditorEngine.h"
#include "Interaction/AvaSnapOperation.h"
#include "ViewportClient/AvaLevelViewportClient.h"

void SAvaLevelViewportCameraBounds::Construct(const FArguments& InArgs, TSharedRef<FAvaLevelViewportClient> InAvaLevelViewportClient)
{
	AvaLevelViewportClientWeak = InAvaLevelViewportClient;
	SetVisibility(EVisibility::HitTestInvisible);
}

int32 SAvaLevelViewportCameraBounds::OnPaint(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry,
	const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle,
	bool bInParentEnabled) const
{
	InLayerId = SCompoundWidget::OnPaint(InPaintArgs, InAllottedGeometry, InMyCullingRect, OutDrawElements, InLayerId, InWidgetStyle, bInParentEnabled);

	if (UAvaViewportSettings const* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		if (!GEditor->PlayWorld)
		{
			DrawCameraBounds(InPaintArgs, InAllottedGeometry, InMyCullingRect, OutDrawElements, InLayerId,
				AvaViewportSettings->CameraBoundsShadeColor);
		}
	}

	return InLayerId;
}

void SAvaLevelViewportCameraBounds::DrawCameraBounds(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry,
	const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32& InOutLayerId, const FLinearColor& InQuadColor) const
{
	TSharedPtr<FAvaLevelViewportClient> AvaLevelViewportClient = AvaLevelViewportClientWeak.Pin();

	if (!AvaLevelViewportClient.IsValid())
	{
		return;
	}

	++InOutLayerId;

	const FVector2f Offset = AvaLevelViewportClient->GetCachedViewportOffset();
	const FAvaVisibleArea& VisibleArea = AvaLevelViewportClient->GetZoomedVisibleArea();

	if (!VisibleArea.IsValid())
	{
		return;
	}

	const FVector2f QuadTopTopLeft = VisibleArea.GetVisiblePosition(-VisibleArea.AbsoluteSize) - Offset;
	const FVector2f QuadTopBottomRight = VisibleArea.GetVisiblePosition({VisibleArea.AbsoluteSize.X * 2.f, 0.f}) + FVector2f(Offset.X, 0.f);

	const FVector2f QuadLeftTopLeft = VisibleArea.GetVisiblePosition({-VisibleArea.AbsoluteSize.X, 0.f}) - FVector2f(Offset.X, 0.f);
	const FVector2f QuadLeftBottomRight = VisibleArea.GetVisiblePosition({0.f, VisibleArea.AbsoluteSize.Y});

	const FVector2f QuadRightTopLeft = VisibleArea.GetVisiblePosition({VisibleArea.AbsoluteSize.X, 0.f});
	const FVector2f QuadRightBottomRight = VisibleArea.GetVisiblePosition({VisibleArea.AbsoluteSize.X * 2.f, VisibleArea.AbsoluteSize.Y}) + FVector2f(Offset.X, 0.f);

	const FVector2f QuadBottomTopLeft = VisibleArea.GetVisiblePosition({-VisibleArea.AbsoluteSize.X, VisibleArea.AbsoluteSize.Y}) - FVector2f(Offset.X, 0.f);
	const FVector2f QuadBottomBottomRight = VisibleArea.GetVisiblePosition({VisibleArea.AbsoluteSize.X * 2.f, VisibleArea.AbsoluteSize.Y * 2.f}) + Offset;

	const FPaintGeometry TopRect = InAllottedGeometry.ToPaintGeometry(
		QuadTopBottomRight - QuadTopTopLeft,
		FSlateLayoutTransform(QuadTopTopLeft + Offset)
	);

	const FPaintGeometry LeftRect = InAllottedGeometry.ToPaintGeometry(
		QuadLeftBottomRight - QuadLeftTopLeft,
		FSlateLayoutTransform(QuadLeftTopLeft + Offset)
	);

	const FPaintGeometry RightRect = InAllottedGeometry.ToPaintGeometry(
		QuadRightBottomRight - QuadRightTopLeft,
		FSlateLayoutTransform(QuadRightTopLeft + Offset)
	);

	const FPaintGeometry BottomRect = InAllottedGeometry.ToPaintGeometry(
		QuadBottomBottomRight - QuadBottomTopLeft,
		FSlateLayoutTransform(QuadBottomTopLeft + Offset)
	);

	static const FSlateBrush* White = FAppStyle::Get().GetBrush("Brushes.White");

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		InOutLayerId,
		TopRect,
		White,
		ESlateDrawEffect::NoPixelSnapping,
		InQuadColor
	);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		InOutLayerId,
		LeftRect,
		White,
		ESlateDrawEffect::NoPixelSnapping,
		InQuadColor
	);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		InOutLayerId,
		RightRect,
		White,
		ESlateDrawEffect::NoPixelSnapping,
		InQuadColor
	);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		InOutLayerId,
		BottomRect,
		White,
		ESlateDrawEffect::NoPixelSnapping,
		InQuadColor
	);
}
