// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaLevelViewportGuide.h"
#include "AvaLevelViewportCommands.h"
#include "AvaViewportSettings.h"
#include "AvaVisibleArea.h"
#include "Framework/Commands/UICommandList.h"
#include "IAvalancheInteractiveToolsModule.h"
#include "Interaction/AvaSnapOperation.h"
#include "SAvaLevelViewport.h"
#include "SAvaLevelViewportFrame.h"
#include "Styling/SlateTypes.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ViewportClient/AvaLevelViewportClient.h"
#include "Widgets/Input/SSpinBox.h"

#define LOCTEXT_NAMESPACE "AvaLevelViewportGuide"

namespace UE::AvaViewport::Private
{
	constexpr float Padding = 2.f;
	const FTimespan DoubleClickMaxDelay = FTimespan::FromMilliseconds(850);
	constexpr float ClickMaxDistance = 4.f;
	constexpr float MinimumViewportEdgeDistance = 5.f;
	const FName GuideMenuName = "AvaViewportGuideMenu";

	void AddGuidePositionWidgetEntry(FToolMenuSection& InSection)
	{
		UAvaLevelViewportGuideContext* GuideContext = InSection.Context.FindContext<UAvaLevelViewportGuideContext>();

		if (!GuideContext)
		{
			return;
		}

		TSharedPtr<SAvaLevelViewportGuide> Guide = GuideContext->GuideWeak.Pin();

		if (!Guide.IsValid())
		{
			return;
		}

		TSharedPtr<SAvaLevelViewportFrame> ViewportFrame = Guide->GetViewportFrame();

		if (!ViewportFrame.IsValid())
		{
			return;
		}

		const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrame);

		if (!FrameAndClient.IsValid())
		{
			return;
		}

		const FVector2f ViewportSize = FrameAndClient.ViewportClient->GetCachedViewportSize();
		float MinValue = MinimumViewportEdgeDistance;;
		float MaxValue;

		switch (Guide->GetGuideInfo().Orientation)
		{
			case Orient_Horizontal:
				MaxValue = ViewportSize.Y - MinimumViewportEdgeDistance;
				break;

			case Orient_Vertical:
				MaxValue = ViewportSize.X - MinimumViewportEdgeDistance;
				break;

			default:
				// Not possible
				return;
		}

		float Offset = FMath::Clamp(Guide->GetOffset(), MinValue, MaxValue);

		TWeakObjectPtr<UAvaLevelViewportGuideContext> GuideContextWeak(GuideContext);

		auto SaveGuides = [GuideContextWeak]()
			{
				if (UAvaLevelViewportGuideContext* GuideContext = GuideContextWeak.Get())
				{
					if (TSharedPtr<SAvaLevelViewportGuide> Guide = GuideContext->GuideWeak.Pin())
					{
						if (TSharedPtr<SAvaLevelViewportFrame> ViewportFrame = Guide->GetViewportFrame())
						{
							const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrame);

							if (FrameAndWidget.IsValid())
							{
								FrameAndWidget.ViewportWidget->SaveGuides();
							}
						}
					}
				}
			};

		auto SetOffset = [GuideContextWeak](float InValue)
			{
				if (UAvaLevelViewportGuideContext* GuideContext = GuideContextWeak.Get())
				{
					if (TSharedPtr<SAvaLevelViewportGuide> Guide = GuideContext->GuideWeak.Pin())
					{
						if (!Guide->GetGuideInfo().bLocked)
						{
							Guide->SetOffset(InValue);
						}
					}
				}
			};

		auto GetLockedState = [GuideContextWeak]()
			{
				if (UAvaLevelViewportGuideContext* GuideContext = GuideContextWeak.Get())
				{
					if (TSharedPtr<SAvaLevelViewportGuide> Guide = GuideContext->GuideWeak.Pin())
					{
						return Guide->GetGuideInfo().bLocked;
					}
				}

				return true;
			};

		TSharedRef<SSpinBox<int32>> PositionWidget = SNew(SSpinBox<int32>)
			.Value(Offset)
			.IsEnabled(TAttribute<bool>::CreateLambda(
				[GetLockedState]()
				{
					return !GetLockedState();
				}))
			.EnableSlider(true)
			.MinValue(MinValue)
			.MaxValue(MaxValue)
			.MinSliderValue(MinValue)
			.MaxSliderValue(MaxValue)
			.MinDesiredWidth(50.f)
			.OnEndSliderMovement(SSpinBox<int32>::FOnValueChanged::CreateLambda(
				[SetOffset, SaveGuides](int32 InValue)
				{
					SetOffset(InValue);
					SaveGuides();
				}))
			.OnValueChanged(SSpinBox<int32>::FOnValueChanged::CreateLambda(
				[SetOffset](int32 InValue)
				{
					SetOffset(InValue);
				}))
			.OnValueCommitted(SSpinBox<int32>::FOnValueCommitted::CreateLambda(
				[SetOffset, SaveGuides](int32 InValue, ETextCommit::Type InCommitType)
				{
					SetOffset(InValue);
					SaveGuides();
				}))
			.PreventThrottling(false);

		InSection.AddEntry(FToolMenuEntry::InitWidget(
			"PositionWidget", 
			PositionWidget, 
			FText::GetEmpty()
		));
	}

	void CreateGuideMenu(UToolMenu* InToolMenu)
	{
		UAvaLevelViewportGuideContext* Context = InToolMenu->FindContext<UAvaLevelViewportGuideContext>();

		if (!Context)
		{
			return;
		}

		TSharedPtr<SAvaLevelViewportGuide> Guide = Context->GuideWeak.Pin();

		if (!Guide.IsValid())
		{
			return;
		}

		const FAvaLevelViewportCommands& ViewportCommands = FAvaLevelViewportCommands::GetInternal();

		FToolMenuSection& Section = InToolMenu->AddSection("Guide", LOCTEXT("GuideOptions", "Guide Options"));
		Section.AddMenuEntryWithCommandList(ViewportCommands.ToggleGuideEnabled, Guide->GetGetCommandList());
		Section.AddMenuEntryWithCommandList(ViewportCommands.ToggleGuideLocked, Guide->GetGetCommandList());
		Section.AddMenuEntryWithCommandList(ViewportCommands.RemoveGuide, Guide->GetGetCommandList());

		Section.AddDynamicEntry(
			"Position",
			FNewToolMenuSectionDelegate::CreateStatic(&AddGuidePositionWidgetEntry)
		);
	}

	void RegisterGuideMenu()
	{
		if (UToolMenus::Get()->IsMenuRegistered(GuideMenuName))
		{
			return;
		}

		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(GuideMenuName);
		Menu->AddDynamicSection("Guide", FNewSectionConstructChoice(FNewToolMenuDelegate::CreateStatic(&CreateGuideMenu)));
	}
}

FAvaLevelViewportGuideDragDropOperation::FAvaLevelViewportGuideDragDropOperation(TSharedPtr<SAvaLevelViewportGuide> InGuide)
{
	GuideWeak = InGuide;

	if (InGuide.IsValid())
	{
		if (TSharedPtr<SAvaLevelViewportFrame> ViewportFrame = InGuide->GetViewportFrame())
		{
			if (TSharedPtr<FAvaLevelViewportClient> ViewportClient = ViewportFrame->GetViewportClient())
			{
				if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
				{
					if (EnumHasAnyFlags(AvaViewportSettings->GetSnapState(), EAvaViewportSnapState::Global))
					{
						SnapStateGuard = MakeShared<TGuardValue<int32>>(
							AvaViewportSettings->SnapState,
							AvaViewportSettings->SnapState & ~static_cast<int32>(EAvaViewportSnapState::Screen)
						);

						SnapOperation = ViewportClient->StartSnapOperation();

						if (SnapOperation.IsValid())
						{
							SnapOperation->GenerateActorSnapPoints({}, {});
							SnapOperation->FinaliseSnapPoints();
						}
					}
				}
			}
		}
	}
}

FAvaLevelViewportGuideDragDropOperation::~FAvaLevelViewportGuideDragDropOperation()
{
	TSharedPtr<SAvaLevelViewportGuide> Guide = GuideWeak.Pin();

	if (Guide.IsValid())
	{
		Guide->DragEnd();
	}
}

void FAvaLevelViewportGuideDragDropOperation::OnDragged(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<SAvaLevelViewportGuide> Guide = GuideWeak.Pin();

	if (Guide.IsValid())
	{
		Guide->DragUpdate();
	}
}

void SAvaLevelViewportGuide::Construct(const FArguments& InArgs, TSharedPtr<SAvaLevelViewportFrame> InViewportFrame, int32 InIndex)
{
	check(InViewportFrame.IsValid());

	ViewportFrameWeak = InViewportFrame;
	Index = InIndex;
	Info.Orientation = InArgs._Orientation;
	Info.OffsetFraction = InArgs._OffsetFraction;
	Info.State = InArgs._InitialState;
	bBeingDragged = false;
	bDragRemove = false;
	LastMouseDownLocation = FVector2f::ZeroVector;
	LastClickTime = FDateTime(0);
	LastClickLocation = FVector2f::ZeroVector;

	using namespace UE::AvaViewport::Private;

	if (Info.Orientation == Orient_Horizontal)
	{
		SBox::Construct(SBox::FArguments().Padding(FMargin(0.f, Padding, 0.f, Padding)));
	}
	else
	{
		SBox::Construct(SBox::FArguments().Padding(FMargin(Padding, 0.f, Padding, 0.f)));
	}

	SetOnMouseButtonDown(FPointerEventHandler::CreateSP(this, &SAvaLevelViewportGuide::OnMouseButtonDown));
	SetOnMouseButtonUp(FPointerEventHandler::CreateSP(this, &SAvaLevelViewportGuide::OnMouseButtonUp));

	SetCursor(TAttribute<TOptional<EMouseCursor::Type>>::Create(TAttribute<TOptional<EMouseCursor::Type>>::FGetter::CreateSP(
		this, &SAvaLevelViewportGuide::GetCursor)));

	BindCommands();
}

FReply SAvaLevelViewportGuide::OnMouseButtonDown(const FGeometry& Geometry, const FPointerEvent& PointerEvent)
{
	if (IAvalancheInteractiveToolsModule::Get().HasActiveTool())
	{
		return FReply::Unhandled();
	}

	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (!FrameAndClient.IsValid() || FrameAndClient.ViewportClient->IsInGameView())
	{
		return FReply::Unhandled();
	}

	if (PointerEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (PointerEvent.GetModifierKeys().IsShiftDown())
		{
			if (Info.State == EAvaViewportGuideState::Disabled)
			{
				Info.State = EAvaViewportGuideState::Enabled;
			}
			else
			{
				Info.State = EAvaViewportGuideState::Disabled;
			}

			return FReply::Handled();
		}

		if (PointerEvent.GetModifierKeys().IsAltDown())
		{
			Info.bLocked = !Info.bLocked;
			UpdateGuideData();

			return FReply::Handled();
		}

		if (!Info.bLocked)
		{
			LastMouseDownLocation = PointerEvent.GetLastScreenSpacePosition();
			DragStart();

			return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
		}

		return FReply::Handled();
	}

	if (PointerEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		OpenRightClickMenu();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SAvaLevelViewportGuide::OnMouseButtonUp(const FGeometry& Geometry, const FPointerEvent& PointerEvent)
{
	if (IAvalancheInteractiveToolsModule::Get().HasActiveTool())
	{
		return FReply::Unhandled();
	}

	const FAvaLevelViewportGuideFrameClientAndWidget FrameClientAndWidget(ViewportFrameWeak);

	if (!FrameClientAndWidget.IsValid() || FrameClientAndWidget.ViewportClient->IsInGameView())
	{
		return FReply::Unhandled();
	}

	DragEnd();

	const FDateTime CurrentTime = FDateTime::Now();

	using namespace UE::AvaViewport::Private;

	if ((PointerEvent.GetScreenSpacePosition() - LastMouseDownLocation).Size() < ClickMaxDistance)
	{
		if ((CurrentTime - LastClickTime) < DoubleClickMaxDelay &&
			(PointerEvent.GetScreenSpacePosition() - LastClickLocation).Size() < ClickMaxDistance)
		{
			FrameClientAndWidget.ViewportWidget->RemoveGuide(SharedThis(this));
		}
		else
		{
			LastClickTime     = CurrentTime;
			LastClickLocation = PointerEvent.GetScreenSpacePosition();
		}
	}

	return FReply::Handled();
}

FReply SAvaLevelViewportGuide::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (IAvalancheInteractiveToolsModule::Get().HasActiveTool())
	{
		DragEnd();
		return FReply::Unhandled();
	}

	if (bBeingDragged && DragUpdate())
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SAvaLevelViewportGuide::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedRef<FAvaLevelViewportGuideDragDropOperation> Operation = MakeShared<FAvaLevelViewportGuideDragDropOperation>(SharedThis(this));
	return FReply::Handled().BeginDragDrop(Operation);
}

int32 SAvaLevelViewportGuide::OnPaint(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry,
	const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId,
	const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const
{
	InLayerId = SBox::OnPaint(InPaintArgs, InAllottedGeometry, InMyCullingRect, OutDrawElements, InLayerId, InWidgetStyle, bInParentEnabled);

	const EVisibility GuideVisibility = GetGuideVisibility();

	if (GuideVisibility == EVisibility::Collapsed || GuideVisibility == EVisibility::Hidden)
	{
		return InLayerId;
	}

	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (!FrameAndClient.IsValid() || FrameAndClient.ViewportClient->IsInGameView())
	{
		return InLayerId;
	}

	UAvaViewportSettings const* AvaViewportSettings = GetDefault<UAvaViewportSettings>();

	if (!AvaViewportSettings)
	{
		return InLayerId;
	}

	const FLinearColor Color = GetColor();
	const FVector2f LocalSize = InAllottedGeometry.GetLocalSize();

	TArray<FVector2f> LinePoints;
	LinePoints.SetNumUninitialized(2);

	switch (Info.Orientation)
	{
		case Orient_Horizontal:
		{
			const float LocalOffset = LocalSize.Y * 0.5f;
			LinePoints[0] = {0, LocalOffset};
			LinePoints[1] = {LocalSize.X, LocalOffset};
			break;
		}

		case Orient_Vertical:
		{
			const float LocalOffset = LocalSize.X * 0.5f;
			LinePoints[0] = {LocalOffset, 0};
			LinePoints[1] = {LocalOffset, LocalSize.Y};
			break;
		}

		default:
			// Not possible
			return InLayerId;
	}

	++InLayerId;

	FSlateDrawElement::MakeLines(
		OutDrawElements,
		InLayerId,
		InAllottedGeometry.ToPaintGeometry(),
		LinePoints,
		ESlateDrawEffect::NoPixelSnapping,
		Color,
		false,
		AvaViewportSettings->GuideThickness
	);

	return InLayerId;
}

void SAvaLevelViewportGuide::DragStart()
{
	bBeingDragged = true;
	bDragRemove   = false;
}

bool SAvaLevelViewportGuide::DragUpdate()
{
	if (!bBeingDragged)
	{
		return false;
	}

	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (!FrameAndClient.IsValid())
	{
		return false;
	}

	FVector2f MousePosition = FrameAndClient.ViewportClient->GetConstrainedZoomedViewportMousePosition();
	FVector2f NewSlotPosition = FVector2f::ZeroVector;

	if (TSharedPtr<FAvaSnapOperation> SnapOperation = FrameAndClient.ViewportClient->GetSnapOperation())
	{
		SnapOperation->SnapScreenLocation(MousePosition, Info.Orientation == EOrientation::Orient_Vertical, Info.Orientation == EOrientation::Orient_Horizontal);
	}

	using namespace UE::AvaViewport::Private;

	switch (Info.Orientation)
	{
		case Orient_Horizontal:
			NewSlotPosition.Y = MousePosition.Y;

			bDragRemove = NewSlotPosition.Y < 0 || NewSlotPosition.Y >= FrameAndClient.ViewportClient->GetCachedViewportSize().Y;

			NewSlotPosition.Y = FMath::Clamp(
					NewSlotPosition.Y,
					MinimumViewportEdgeDistance,
					FrameAndClient.ViewportClient->GetCachedViewportSize().Y - MinimumViewportEdgeDistance
				);

			Info.OffsetFraction = NewSlotPosition.Y / FrameAndClient.ViewportClient->GetCachedViewportSize().Y;
			break;

		case Orient_Vertical:
			NewSlotPosition.X = MousePosition.X;

			bDragRemove = NewSlotPosition.X < 0 || NewSlotPosition.X >= FrameAndClient.ViewportClient->GetCachedViewportSize().X;

			NewSlotPosition.X = FMath::Clamp(
					NewSlotPosition.X,
					MinimumViewportEdgeDistance,
					FrameAndClient.ViewportClient->GetCachedViewportSize().X - MinimumViewportEdgeDistance
				);
			Info.OffsetFraction = NewSlotPosition.X / FrameAndClient.ViewportClient->GetCachedViewportSize().X;
			break;

		default:
			break;
	}

	return true;
}

void SAvaLevelViewportGuide::DragEnd()
{
	bBeingDragged = false;

	if (bDragRemove)
	{
		const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

		if (FrameAndWidget.IsValid())
		{
			FrameAndWidget.ViewportWidget->RemoveGuide(SharedThis(this));
		}

		bDragRemove = false;
	}

	// We only need to update the guide data if we end a drag.
	UpdateGuideData();
}

TSharedPtr<SAvaLevelViewportFrame> SAvaLevelViewportGuide::GetViewportFrame() const
{
	return ViewportFrameWeak.Pin();
}

TSharedPtr<FUICommandList> SAvaLevelViewportGuide::GetGetCommandList()
{
	return CommandList.ToSharedRef();
}

float SAvaLevelViewportGuide::GetOffset() const
{
	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (!FrameAndClient.IsValid())
	{
		return TNumericLimits<float>::Max();
	}

	const FVector2f ViewportSize = FrameAndClient.ViewportClient->GetCachedViewportSize();

	float Offset = Info.Orientation == Orient_Horizontal ? ViewportSize.Y : ViewportSize.X;
	Offset *= Info.OffsetFraction;

	return Offset;
}

bool SAvaLevelViewportGuide::SetOffset(float InOffset)
{
	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (!FrameAndClient.IsValid())
	{
		return false;
	}

	const FVector2f ViewportSize = FrameAndClient.ViewportClient->GetCachedViewportSize();
	float NewOffsetFraction;

	using namespace UE::AvaViewport::Private;

	switch (Info.Orientation)
	{
		case Orient_Horizontal:
			InOffset = FMath::Clamp(
				InOffset,
				MinimumViewportEdgeDistance,
				ViewportSize.Y - MinimumViewportEdgeDistance
			);

			NewOffsetFraction = InOffset / ViewportSize.Y;
			break;

		case Orient_Vertical:
			InOffset = FMath::Clamp(
				InOffset,
				MinimumViewportEdgeDistance,
				ViewportSize.X - MinimumViewportEdgeDistance
			);

			NewOffsetFraction = InOffset / ViewportSize.X;
			break;

		default:
			// Not possible
			return false;
	}

	if (FMath::IsNearlyEqual(Info.OffsetFraction, NewOffsetFraction))
	{
		return false;
	}

	Info.OffsetFraction = NewOffsetFraction;

	return true;
}

FVector2D SAvaLevelViewportGuide::GetSize() const
{
	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (!FrameAndClient.IsValid())
	{
		return FVector2D::ZeroVector;
	}

	const FVector2f ViewportSize = FrameAndClient.ViewportClient->GetCachedViewportSize();

	using namespace UE::AvaViewport::Private;

	switch (Info.Orientation)
	{
		case Orient_Horizontal:
			return FVector2D(ViewportSize.X, 1.0 + Padding * 2.0);

		case Orient_Vertical:
			return FVector2D(1.0 + Padding * 2.0, ViewportSize.Y);

		default:
			// Not possible
			return FVector2D::ZeroVector;
	}
}

FVector2D SAvaLevelViewportGuide::GetPosition() const
{
	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (!FrameAndClient.IsValid())
	{
		return FVector2D::ZeroVector;
	}

	const FAvaVisibleArea VisibleArea = FrameAndClient.ViewportClient->GetZoomedVisibleArea();
	const FVector2f Offset = FrameAndClient.ViewportClient->GetCachedViewportOffset();
	FVector2f GuidePosition;

	using namespace UE::AvaViewport::Private;

	if (Info.Orientation == Orient_Horizontal)
	{
		GuidePosition = VisibleArea.GetVisiblePosition(FVector2f(0.f, GetOffset()));
		GuidePosition.X = 0.f;
		GuidePosition.Y -= Padding;
	}
	else
	{
		GuidePosition = VisibleArea.GetVisiblePosition(FVector2f(GetOffset(), 0.f));
		GuidePosition.Y = 0.f;
		GuidePosition.X -= Padding;
	}

	return static_cast<FVector2D>(GuidePosition + Offset);
}

FLinearColor SAvaLevelViewportGuide::GetColor() const
{
	UAvaViewportSettings const* AvaViewportSettings = GetDefault<UAvaViewportSettings>();

	if (!AvaViewportSettings)
	{
		return FLinearColor::Black;
	}

	if (bBeingDragged)
	{
		return AvaViewportSettings->DraggedGuideColor;
	}

	switch (Info.State)
	{
		case EAvaViewportGuideState::Disabled:
			return Info.bLocked
				? AvaViewportSettings->DisabledLockedGuideColor
				: AvaViewportSettings->DisabledGuideColor;

		case EAvaViewportGuideState::Enabled:
			return Info.bLocked
				? AvaViewportSettings->EnabledLockedGuideColor
				: AvaViewportSettings->EnabledGuideColor;

		case EAvaViewportGuideState::SnappedTo:
			return AvaViewportSettings->SnappedToGuideColor;

		default:
			return AvaViewportSettings->EnabledGuideColor;
	}
}

EVisibility SAvaLevelViewportGuide::GetGuideVisibility() const
{
	if (bDragRemove)
	{
		return EVisibility::Hidden;
	}

	const float Position = GetOffset();

	if (Position < 0)
	{
		return EVisibility::Collapsed;
	}

	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (!FrameAndClient.IsValid())
	{
		return EVisibility::Collapsed;
	}

	const FAvaVisibleArea VisibleArea = FrameAndClient.ViewportClient->GetZoomedVisibleArea();

	if (Info.Orientation == Orient_Horizontal)
	{
		if (Position < VisibleArea.Offset.Y || Position >= VisibleArea.Offset.Y + VisibleArea.VisibleSize.Y)
		{
			return EVisibility::Collapsed;
		}
	}
	else
	{
		if (Position < VisibleArea.Offset.X || Position >= VisibleArea.Offset.X + VisibleArea.VisibleSize.X)
		{
			return EVisibility::Collapsed;
		}
	}

	return EVisibility::Visible;
}

TOptional<EMouseCursor::Type> SAvaLevelViewportGuide::GetCursor() const
{
	if (IAvalancheInteractiveToolsModule::Get().HasActiveTool())
	{
		return EMouseCursor::Crosshairs;
	}

	if (Info.bLocked)
	{
		return EMouseCursor::Default;
	}

	switch (Info.Orientation)
	{
		case Orient_Horizontal:
			return EMouseCursor::ResizeUpDown;

		case Orient_Vertical:
			return EMouseCursor::ResizeLeftRight;

		default:
			return EMouseCursor::Crosshairs;
	}
}

void SAvaLevelViewportGuide::UpdateGuideData() const
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (!FrameAndWidget.IsValid())
	{
		return;
	}

	FrameAndWidget.ViewportWidget->SaveGuides();
}

void SAvaLevelViewportGuide::OpenRightClickMenu()
{
	using namespace UE::AvaViewport::Private;

	RegisterGuideMenu();

	TSharedRef<SAvaLevelViewportGuide> This = SharedThis(this);

	UAvaLevelViewportGuideContext* GuideContext = NewObject<UAvaLevelViewportGuideContext>(GetTransientPackage());
	GuideContext->GuideWeak = This;

	FToolMenuContext MenuContext(GuideContext);

	FSlateApplication::Get().PushMenu(
		This,
		FWidgetPath(),
		UToolMenus::Get()->GenerateWidget(GuideMenuName, GuideContext),
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect::ContextMenu
	);
}

void SAvaLevelViewportGuide::BindCommands()
{
	const FAvaLevelViewportCommands& ViewportCommands = FAvaLevelViewportCommands::GetInternal();

	CommandList = MakeShared<FUICommandList>();

	CommandList->MapAction(
		ViewportCommands.ToggleGuideEnabled,
		FExecuteAction::CreateSP(this, &SAvaLevelViewportGuide::ToggleEnabled),
		FCanExecuteAction::CreateSP(this, &SAvaLevelViewportGuide::CanToggleEnable),
		FGetActionCheckState::CreateSP(this, &SAvaLevelViewportGuide::GetEnabledCheckState)
	);

	CommandList->MapAction(
		ViewportCommands.ToggleGuideLocked,
		FExecuteAction::CreateSP(this, &SAvaLevelViewportGuide::ToggleLocked),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &SAvaLevelViewportGuide::GetLockedCheckState)
	);

	CommandList->MapAction(
		ViewportCommands.RemoveGuide,
		FExecuteAction::CreateSP(this, &SAvaLevelViewportGuide::RemoveGuide)
	);
}

ECheckBoxState SAvaLevelViewportGuide::GetEnabledCheckState() const
{
	return Info.State == EAvaViewportGuideState::Disabled
		? ECheckBoxState::Unchecked
		: ECheckBoxState::Checked;
}

bool SAvaLevelViewportGuide::CanToggleEnable() const
{
	return Info.State != EAvaViewportGuideState::SnappedTo;
}

void SAvaLevelViewportGuide::ToggleEnabled()
{
	switch (Info.State)
	{
		default:
		case EAvaViewportGuideState::SnappedTo:
			// Do nothing
			break;

		case EAvaViewportGuideState::Disabled:
			SetState(EAvaViewportGuideState::Enabled);
			break;

		case EAvaViewportGuideState::Enabled:
			SetState(EAvaViewportGuideState::Disabled);
			break;
	}
}

void SAvaLevelViewportGuide::SetState(EAvaViewportGuideState InNewState, bool bInSaveGuides)
{
	Info.State = InNewState;

	if (!bInSaveGuides)
	{
		return;
	}

	TSharedPtr<SAvaLevelViewportFrame> ViewportFrame = GetViewportFrame();

	if (!ViewportFrame.IsValid())
	{
		return;
	}

	TSharedPtr<SAvaLevelViewport> ViewportWidget = ViewportFrame->GetViewportWidget();

	if (!ViewportWidget.IsValid())
	{
		return;
	}

	ViewportWidget->SaveGuides();
}

ECheckBoxState SAvaLevelViewportGuide::GetLockedCheckState() const
{
	return Info.bLocked
		? ECheckBoxState::Checked
		: ECheckBoxState::Unchecked;
}

void SAvaLevelViewportGuide::ToggleLocked()
{
	SetLocked(!Info.bLocked);
}

void SAvaLevelViewportGuide::RemoveGuide()
{
	const FAvaLevelViewportGuideFrameClientAndWidget FrameClientAndWidget(ViewportFrameWeak);

	if (!FrameClientAndWidget.IsValid())
	{
		return;
	}

	FrameClientAndWidget.ViewportWidget->RemoveGuide(SharedThis(this));
}

#undef LOCTEXT_NAMESPACE
