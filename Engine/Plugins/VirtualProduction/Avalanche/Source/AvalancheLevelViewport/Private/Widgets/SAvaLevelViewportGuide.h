// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaViewportGuideInfo.h"
#include "Input/DragAndDrop.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Layout/SBox.h"
#include "SAvaLevelViewportGuide.generated.h"

class FAvaSnapOperation;
class FUICommandList;
class SAvaLevelViewportFrame;
class SAvaLevelViewportGuide;
enum class ECheckBoxState : uint8;

UCLASS()
class UAvaLevelViewportGuideContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<SAvaLevelViewportGuide> GuideWeak;
};

class FAvaLevelViewportGuideDragDropOperation : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FAvaLevelViewportGuideDragDropOperation, FDragDropOperation)

	FAvaLevelViewportGuideDragDropOperation(TSharedPtr<SAvaLevelViewportGuide> InGuide);
	virtual ~FAvaLevelViewportGuideDragDropOperation() override;

	//~ Begin FDragDropOperation
	virtual void OnDragged(const FDragDropEvent& DragDropEvent) override;
	//~ End FDragDropOperation

	TSharedPtr<FAvaSnapOperation> GetSnapOperation() const { return SnapOperation; }

protected:
	TWeakPtr<SAvaLevelViewportGuide> GuideWeak;
	TSharedPtr<FAvaSnapOperation> SnapOperation;
	TSharedPtr<TGuardValue<int32>> SnapStateGuard;
};

class SAvaLevelViewportGuide : public SBox
{
public:
	SLATE_BEGIN_ARGS(SAvaLevelViewportGuide)
		: _Orientation(EOrientation::Orient_Horizontal)
		, _OffsetFraction(0.5f)
		, _InitialState(EAvaViewportGuideState::Enabled)
		{}
		SLATE_ARGUMENT(EOrientation, Orientation)
		SLATE_ARGUMENT(float, OffsetFraction)
		SLATE_ARGUMENT(EAvaViewportGuideState, InitialState)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SAvaLevelViewportFrame> InViewportFrame, int32 InIndex);

	const FAvaViewportGuideInfo& GetGuideInfo() const { return Info; }
	void SetState(EAvaViewportGuideState InNewState, bool bInSaveGuides = true);
	void SetLocked(bool bInLocked) { Info.bLocked = bInLocked; }
	bool IsBeingDragged() const { return bBeingDragged; }
	EVisibility GetGuideVisibility() const;
	float GetOffset() const;
	bool SetOffset(float InOffset);
	FVector2D GetSize() const;
	FVector2D GetPosition() const;
	void SetIndex(int32 InNewIndex) { Index = InNewIndex; }

	void DragStart();
	bool DragUpdate(); // Returns true if successfully updated
	void DragEnd();

	TSharedPtr<SAvaLevelViewportFrame> GetViewportFrame() const;

	TSharedPtr<FUICommandList> GetGetCommandList();

	//~ Begin SWidget
	virtual FReply OnMouseButtonDown(const FGeometry& Geometry, const FPointerEvent& PointerEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& Geometry, const FPointerEvent& PointerEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual int32 OnPaint(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect,
		FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const override;
	//~ End SWidget

protected:
	TWeakPtr<SAvaLevelViewportFrame> ViewportFrameWeak;
	int32 Index;
	FAvaViewportGuideInfo Info;
	bool bBeingDragged;
	bool bDragRemove;
	FVector2f LastMouseDownLocation;
	FDateTime LastClickTime;
	FVector2f LastClickLocation;
	TSharedPtr<FUICommandList> CommandList;

	FLinearColor GetColor() const;

	//~ Begin SWidget
	virtual TOptional<EMouseCursor::Type> GetCursor() const override;
	//~ End SWidget

	void UpdateGuideData() const;

	void OpenRightClickMenu();

	void BindCommands();

	ECheckBoxState GetEnabledCheckState() const;
	bool CanToggleEnable() const;
	void ToggleEnabled();

	ECheckBoxState GetLockedCheckState() const;
	void ToggleLocked();

	void RemoveGuide();
};
