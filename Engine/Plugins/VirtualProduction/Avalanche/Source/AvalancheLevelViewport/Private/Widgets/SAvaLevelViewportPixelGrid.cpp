// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAvaLevelViewportPixelGrid.h"
#include "AvaViewportSettings.h"
#include "AvaVisibleArea.h"
#include "ViewportClient/AvaLevelViewportClient.h"

void SAvaLevelViewportPixelGrid::Construct(const FArguments& InArgs, TSharedRef<FAvaLevelViewportClient> InAvaLevelViewportClient)
{
	AvaLevelViewportClientWeak = InAvaLevelViewportClient;
	SetVisibility(EVisibility::HitTestInvisible);
}

int32 SAvaLevelViewportPixelGrid::OnPaint(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry,
	const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle,
	bool bInParentEnabled) const
{
	InLayerId = SCompoundWidget::OnPaint(InPaintArgs, InAllottedGeometry, InMyCullingRect, OutDrawElements, InLayerId, InWidgetStyle, bInParentEnabled);

	TSharedPtr<FAvaLevelViewportClient> ViewportClient = AvaLevelViewportClientWeak.Pin();

	if (!ViewportClient.IsValid() || ViewportClient->IsInGameView())
	{
		return InLayerId;
	}

	if (UAvaViewportSettings const* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		DrawGrid(InPaintArgs, InAllottedGeometry, InMyCullingRect, OutDrawElements, InLayerId, AvaViewportSettings->PixelGridColor, 1.f);
	}

	return InLayerId;
}

void SAvaLevelViewportPixelGrid::DrawGrid(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry,
	const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32& InOutLayerId,
	const FLinearColor& InGridColor, const float InGridThickness) const
{
	TSharedPtr<FAvaLevelViewportClient> AvaLevelViewportClient = AvaLevelViewportClientWeak.Pin();

	if (!AvaLevelViewportClient.IsValid())
	{
		return;
	}

	const FAvaVisibleArea VisibleArea = AvaLevelViewportClient->GetZoomedVisibleArea();
	const FVector2f LocalSize = InAllottedGeometry.GetLocalSize();
	const FVector2f Offset = (LocalSize - VisibleArea.AbsoluteSize) * 0.5f;
	const float GridSeparation = 1.f / VisibleArea.GetVisibleAreaFraction();

	if (GridSeparation < 6)
	{
		return;
	}

	++InOutLayerId;

	auto DrawLine = [&InAllottedGeometry, &OutDrawElements, InOutLayerId, &InGridColor, InGridThickness, &VisibleArea](const FVector2f& InStart, const FVector2f& InEnd)
	{
		const TArray<FVector2f> LinePoints = {InStart, InEnd};

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			InOutLayerId,
			InAllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::NoPixelSnapping,
			InGridColor,
			false,
			InGridThickness
		);
	};

	FVector2f TopLeftCornerVisible = VisibleArea.GetVisiblePosition({
		FMath::Floor(VisibleArea.Offset.X + 1.f),
		FMath::Floor(VisibleArea.Offset.Y + 1.f)
	});

	// Draw top horizontal lines.
	for (float GridVerticalPosition = TopLeftCornerVisible.Y + Offset.Y; GridVerticalPosition > 0; GridVerticalPosition -= GridSeparation)
	{
		if (GridVerticalPosition >= LocalSize.Y)
		{
			continue;
		}

		DrawLine(
			{0, GridVerticalPosition},
			{LocalSize.X, GridVerticalPosition}
		);
	}

	// Draw bottom horizontal lines.
	for (float GridVerticalPosition = TopLeftCornerVisible.Y + Offset.Y + GridSeparation; GridVerticalPosition < LocalSize.Y; GridVerticalPosition += GridSeparation)
	{
		if (GridVerticalPosition <= 0)
		{
			continue;
		}

		DrawLine(
			{0, GridVerticalPosition},
			{LocalSize.X, GridVerticalPosition}
		);
	}

	// Draw left vertical lines.
	for (float GridHorizontalPosition = TopLeftCornerVisible.X + Offset.X; GridHorizontalPosition > 0; GridHorizontalPosition -= GridSeparation)
	{
		if (GridHorizontalPosition >= LocalSize.X)
		{
			continue;
		}

		DrawLine(
			{GridHorizontalPosition, 0},
			{GridHorizontalPosition, LocalSize.Y}
		);
	}

	// Draw bottom horizontal lines.
	for (float GridHorizontalPosition = TopLeftCornerVisible.X + Offset.X + GridSeparation; GridHorizontalPosition < LocalSize.X; GridHorizontalPosition += GridSeparation)
	{
		if (GridHorizontalPosition <= 0)
		{
			continue;
		}

		DrawLine(
			{GridHorizontalPosition, 0},
			{GridHorizontalPosition, LocalSize.Y}
		);
	}
}
