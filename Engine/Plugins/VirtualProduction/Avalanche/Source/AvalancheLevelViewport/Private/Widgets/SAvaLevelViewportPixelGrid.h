// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FAvaLevelViewportClient;

class SAvaLevelViewportPixelGrid : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaLevelViewportPixelGrid)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FAvaLevelViewportClient> InAvaLevelViewportClient);

	//~ Begin SWidget
	virtual int32 OnPaint(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect,
		FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const override;
	//~ End SWidget

protected:
	TWeakPtr<FAvaLevelViewportClient> AvaLevelViewportClientWeak;

	void DrawGrid(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect,
		FSlateWindowElementList& OutDrawElements, int32& InOutLayerId, const FLinearColor& InGridColor, const float InGridThickness) const;
};
