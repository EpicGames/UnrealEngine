// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAvaLevelViewportRuler.h"

#include "AvaViewportSettings.h"
#include "AvaVisibleArea.h"
#include "SAvaLevelViewport.h"
#include "SAvaLevelViewportFrame.h"
#include "Styling/StyleColors.h"
#include "ViewportClient/AvaLevelViewportClient.h"
#include "Widgets/SAvaLevelViewportGuide.h"

void SAvaLevelViewportRuler::Construct(const FArguments& InArgs, TSharedPtr<SAvaLevelViewportFrame> InViewportFrame)
{
	ViewportFrameWeak = InViewportFrame;
	AccessibleOrientation = InArgs._Orientation;

	SRuler::Construct(
		SRuler::FArguments()
			.Orientation(AccessibleOrientation)
	);
}

FReply SAvaLevelViewportRuler::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		if (AvaViewportSettings->bEnableViewportOverlay && AvaViewportSettings->bGuidesEnabled 
			&& MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
		}
	}

	return FReply::Unhandled();
}

FReply SAvaLevelViewportRuler::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Handled();
}

FReply SAvaLevelViewportRuler::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		return FReply::Unhandled();
	}

	TSharedPtr<SAvaLevelViewportFrame> ViewportFrame = ViewportFrameWeak.Pin();

	if (!ViewportFrame.IsValid())
	{
		return FReply::Unhandled();
	}

	TSharedPtr<SAvaLevelViewport> ViewportWidget = ViewportFrame->GetViewportWidget();

	if (!ViewportWidget.IsValid())
	{
		return FReply::Unhandled();
	}

	TSharedPtr<SAvaLevelViewportGuide> NewGuide = ViewportWidget->AddGuide(AccessibleOrientation, 0.f);
	NewGuide->DragStart();
	NewGuide->DragUpdate();

	TSharedRef<FAvaLevelViewportGuideDragDropOperation> Operation = MakeShared<FAvaLevelViewportGuideDragDropOperation>(NewGuide);
	return FReply::Handled().BeginDragDrop(Operation);
}

int32 SAvaLevelViewportRuler::OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect, 
	FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const
{
	if (InAllottedGeometry.Size.X < 0 || InAllottedGeometry.Size.Y < 0)
	{
		return InLayerId;
	}

	InLayerId = SRuler::OnPaint(InArgs, InAllottedGeometry, InMyCullingRect, OutDrawElements, InLayerId, InWidgetStyle, bInParentEnabled);

	if (TSharedPtr<SAvaLevelViewportFrame> ViewportFrame = ViewportFrameWeak.Pin())
	{
		const FAvaVisibleArea& VisibleArea = ViewportFrame->GetViewportClient()->GetZoomedVisibleArea();

		if (VisibleArea.IsValid() && VisibleArea.IsZoomedView())
		{
			static const FLinearColor BarColor = FStyleColors::AccentBlue.GetSpecifiedColor();
			static constexpr float BarThickness = 2.f;

			TArray<FVector2f> LinePoints;
			LinePoints.Reserve(2);

			switch (AccessibleOrientation)
			{
				case EOrientation::Orient_Horizontal:
				{
					const float TotalWidth = InAllottedGeometry.GetLocalSize().X;
					const float Width = FMath::RoundToFloat(FMath::Max(1.f, TotalWidth * VisibleArea.GetVisibleAreaFraction() - BarThickness));
					const float StartX = FMath::RoundToFloat(VisibleArea.Offset.X / VisibleArea.AbsoluteSize.X * TotalWidth - BarThickness * 0.5f);
					const float StartY = FMath::RoundToFloat(BarThickness * 0.5f);
					LinePoints.Add({StartX, StartY});
					LinePoints.Add({StartX + Width, StartY});
					break;
				}

				case EOrientation::Orient_Vertical:
				{
					const float TotalHeight = InAllottedGeometry.GetLocalSize().Y;
					const float Height = FMath::RoundToFloat(FMath::Max(1.f, TotalHeight * VisibleArea.GetVisibleAreaFraction() - BarThickness));
					const float StartY = FMath::RoundToFloat(VisibleArea.Offset.Y / VisibleArea.AbsoluteSize.Y * TotalHeight - BarThickness * 0.5f);
					const float StartX = FMath::RoundToFloat(BarThickness * 0.5f);
					LinePoints.Add({StartX, StartY});
					LinePoints.Add({StartX, StartY + Height});
					break;
				}

				default:
					return InLayerId;
			}

			++InLayerId;

			FSlateDrawElement::MakeLines(
				OutDrawElements,
				InLayerId,
				InAllottedGeometry.ToPaintGeometry(),
				LinePoints,
				ESlateDrawEffect::None,
				BarColor,
				false,
				BarThickness
			);
		}
	}

	return InLayerId;
}
