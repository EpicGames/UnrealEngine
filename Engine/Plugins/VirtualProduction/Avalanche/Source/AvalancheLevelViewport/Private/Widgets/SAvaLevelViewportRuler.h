// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Engine/SRuler.h"

class SAvaLevelViewportFrame;

class SAvaLevelViewportRuler : public UE::AvaLevelViewport::Private::SRuler
{
public:
	SLATE_BEGIN_ARGS(SAvaLevelViewportRuler)
		: _Orientation(Orient_Horizontal)
	{
		_Clipping = EWidgetClipping::ClipToBounds;
	}
	SLATE_ARGUMENT(EOrientation, Orientation)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SAvaLevelViewportFrame> InViewportFrame);

	//~ Begin SWidget
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual int32 OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect,
		FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const override;
	//~ End SWidget

protected:
	TWeakPtr<SAvaLevelViewportFrame> ViewportFrameWeak;

	/** This is private in the parent class. */
	EOrientation AccessibleOrientation;
};
