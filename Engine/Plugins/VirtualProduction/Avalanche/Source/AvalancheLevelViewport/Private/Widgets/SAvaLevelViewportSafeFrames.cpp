// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAvaLevelViewportSafeFrames.h"
#include "AvaViewportSettings.h"
#include "AvaVisibleArea.h"
#include "ViewportClient/AvaLevelViewportClient.h"

void SAvaLevelViewportSafeFrames::Construct(const FArguments& InArgs, TSharedRef<FAvaLevelViewportClient> InAvaLevelViewportClient)
{
	AvaLevelViewportClientWeak = InAvaLevelViewportClient;
	SetVisibility(EVisibility::Collapsed);
}

int32 SAvaLevelViewportSafeFrames::OnPaint(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry,
	const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle,
	bool bInParentEnabled) const
{
	InLayerId = SCompoundWidget::OnPaint(InPaintArgs, InAllottedGeometry, InMyCullingRect, OutDrawElements, InLayerId, InWidgetStyle, bInParentEnabled);

	if (UAvaViewportSettings const* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		if (AvaViewportSettings->SafeFrames.IsEmpty() == false)
		{
			if (TSharedPtr<FAvaLevelViewportClient> AvaLevelViewportClient = AvaLevelViewportClientWeak.Pin())
			{
				if (!AvaLevelViewportClient->IsInGameView())
				{
					++InLayerId;

					for (const FAvaLevelViewportSafeFrame& SafeFrame : AvaViewportSettings->SafeFrames)
					{
						DrawSafeFrame(InPaintArgs, InAllottedGeometry, InMyCullingRect, OutDrawElements, InLayerId,
							SafeFrame.ScreenPercentage, SafeFrame.Color, SafeFrame.Thickness);
					}
				}
			}
		}
	}

	return InLayerId;
}

void SAvaLevelViewportSafeFrames::DrawSafeFrame(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry, 
	const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32& InOutLayerId, float InScreenPercentage,
	const FLinearColor& InLineColor, const float InLineThickness) const
{
	if (InScreenPercentage <= 0 || InScreenPercentage > 300)
	{
		return;
	}

	TSharedPtr<FAvaLevelViewportClient> AvaLevelViewportClient = AvaLevelViewportClientWeak.Pin();

	if (!AvaLevelViewportClient.IsValid())
	{
		return;
	}

	const FAvaVisibleArea& VisibleArea = AvaLevelViewportClient->GetZoomedVisibleArea();

	if (!VisibleArea.IsValid())
	{
		return;
	}

	const FVector2f Offset = AvaLevelViewportClient->GetCachedViewportOffset();
	const FVector2f Border = VisibleArea.AbsoluteSize * (100.f - InScreenPercentage) * 0.01f * 0.5f;

	const TArray<FVector2f> LinePoints = {
		VisibleArea.GetVisiblePosition(Border) + Offset,
		VisibleArea.GetVisiblePosition(FVector2f(VisibleArea.AbsoluteSize.X - Border.X, Border.Y)) + Offset,
		VisibleArea.GetVisiblePosition(VisibleArea.AbsoluteSize - Border) + Offset,
		VisibleArea.GetVisiblePosition(FVector2f(Border.X, VisibleArea.AbsoluteSize.Y - Border.Y)) + Offset,
		VisibleArea.GetVisiblePosition(Border) + Offset
	};

	FSlateDrawElement::MakeLines(
		OutDrawElements,
		InOutLayerId,
		InAllottedGeometry.ToPaintGeometry(),
		LinePoints,
		ESlateDrawEffect::NoPixelSnapping,
		InLineColor,
		false,
		InLineThickness
	);
}
