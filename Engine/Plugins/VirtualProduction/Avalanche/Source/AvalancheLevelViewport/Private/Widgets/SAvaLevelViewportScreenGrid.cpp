// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAvaLevelViewportScreenGrid.h"
#include "AvaViewportSettings.h"
#include "AvaVisibleArea.h"
#include "ViewportClient/AvaLevelViewportClient.h"

void SAvaLevelViewportScreenGrid::Construct(const FArguments& InArgs, TSharedRef<FAvaLevelViewportClient> InAvaLevelViewportClient)
{
	AvaLevelViewportClientWeak = InAvaLevelViewportClient;
	SetVisibility(EVisibility::HitTestInvisible);
}

int32 SAvaLevelViewportScreenGrid::OnPaint(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry, 
	const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, 
	bool bInParentEnabled) const
{
	InLayerId = SCompoundWidget::OnPaint(InPaintArgs, InAllottedGeometry, InMyCullingRect, OutDrawElements, InLayerId, InWidgetStyle, bInParentEnabled);

	if (UAvaViewportSettings const* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		if (TSharedPtr<FAvaLevelViewportClient> AvaLevelViewportClient = AvaLevelViewportClientWeak.Pin())
		{
			if (!AvaLevelViewportClient->IsInGameView()
				&& (AvaViewportSettings->bGridAlwaysVisible || AvaLevelViewportClient->GetSnapOperation().IsValid()))
			{
				if (AvaViewportSettings->GridSize >= 1)
				{
					DrawGrid(InPaintArgs, InAllottedGeometry, InMyCullingRect, OutDrawElements, InLayerId,
						AvaViewportSettings->GridSize, AvaViewportSettings->GridColor, AvaViewportSettings->GridThickness);
				}
			}
		}
	}

	return InLayerId;
}

void SAvaLevelViewportScreenGrid::DrawGrid(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry,
	const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32& InOutLayerId, float InGridSizeSeparation,
	const FLinearColor& InGridColor, const float InGridThickness) const
{
	TSharedPtr<FAvaLevelViewportClient> AvaLevelViewportClient = AvaLevelViewportClientWeak.Pin();

	if (!AvaLevelViewportClient.IsValid())
	{
		return;
	}

	++InOutLayerId;

	const FAvaVisibleArea VisibleArea = AvaLevelViewportClient->GetZoomedVisibleArea();
	const FVector2f LocalSize = InAllottedGeometry.GetLocalSize();
	const FVector2f Offset = (LocalSize - VisibleArea.AbsoluteSize) * 0.5f;
	const float GridSeparation = InGridSizeSeparation / VisibleArea.GetVisibleAreaFraction();

	const FVector2f MidPointAbsolute = VisibleArea.AbsoluteSize * 0.5f;
	const FVector2f MidPointVisible = VisibleArea.GetVisiblePosition(MidPointAbsolute);

	auto DrawLine = [&InAllottedGeometry, &OutDrawElements, InOutLayerId, &InGridColor, InGridThickness](const FVector2f& InStart, const FVector2f& InEnd)
	{
		const TArray<FVector2f> LinePoints = {
			InStart,
			InEnd
		};

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			InOutLayerId,
			InAllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::NoPixelSnapping,
			InGridColor,
			false,
			InGridThickness
		);
	};

	// Draw top horizontal lines.
	for (float GridVerticalPosition = MidPointVisible.Y + Offset.Y; GridVerticalPosition > 0; GridVerticalPosition -= GridSeparation)
	{
		if (GridVerticalPosition >= LocalSize.Y)
		{
			continue;
		}

		DrawLine(
			{0, GridVerticalPosition},
			{LocalSize.X, GridVerticalPosition}
		);
	}

	// Draw bottom horizontal lines.
	for (float GridVerticalPosition = MidPointVisible.Y + Offset.Y + GridSeparation; GridVerticalPosition < LocalSize.Y; GridVerticalPosition += GridSeparation)
	{
		if (GridVerticalPosition <= 0)
		{
			continue;
		}

		DrawLine(
			{0, GridVerticalPosition},
			{LocalSize.X, GridVerticalPosition}
		);
	}

	// Draw left vertical lines.
	for (float GridHorizontalPosition = MidPointVisible.X + Offset.X; GridHorizontalPosition > 0; GridHorizontalPosition -= GridSeparation)
	{
		if (GridHorizontalPosition >= LocalSize.X)
		{
			continue;
		}

		DrawLine(
			{GridHorizontalPosition, 0},
			{GridHorizontalPosition, LocalSize.Y}
		);
	}

	// Draw bottom horizontal lines.
	for (float GridHorizontalPosition = MidPointVisible.X + Offset.X + GridSeparation; GridHorizontalPosition < LocalSize.X; GridHorizontalPosition += GridSeparation)
	{
		if (GridHorizontalPosition <= 0)
		{
			continue;
		}

		DrawLine(
			{GridHorizontalPosition, 0},
			{GridHorizontalPosition, LocalSize.Y}
		);
	}

	if (GridSeparation > 4)
	{
		DrawLine(
			FVector2f(MidPointVisible.X, MidPointVisible.Y + GridSeparation) + Offset,
			FVector2f(MidPointVisible.X + GridSeparation, MidPointVisible.Y) + Offset
		);

		DrawLine(
			FVector2f(MidPointVisible.X, MidPointVisible.Y - GridSeparation) + Offset,
			FVector2f(MidPointVisible.X + GridSeparation, MidPointVisible.Y) + Offset
		);

		DrawLine(
			FVector2f(MidPointVisible.X, MidPointVisible.Y + GridSeparation) + Offset,
			FVector2f(MidPointVisible.X - GridSeparation, MidPointVisible.Y) + Offset
		);

		DrawLine(
			FVector2f(MidPointVisible.X, MidPointVisible.Y - GridSeparation) + Offset,
			FVector2f(MidPointVisible.X - GridSeparation, MidPointVisible.Y) + Offset
		);
	}
}
