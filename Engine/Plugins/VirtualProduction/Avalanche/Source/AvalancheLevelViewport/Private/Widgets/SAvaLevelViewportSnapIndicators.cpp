// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAvaLevelViewportSnapIndicators.h"
#include "AvaViewportSettings.h"
#include "AvaVisibleArea.h"
#include "Interaction/AvaSnapOperation.h"
#include "ViewportClient/AvaLevelViewportClient.h"

void SAvaLevelViewportSnapIndicators::Construct(const FArguments& InArgs, TSharedRef<FAvaLevelViewportClient> InAvaLevelViewportClient)
{
	AvaLevelViewportClientWeak = InAvaLevelViewportClient;
	SetVisibility(EVisibility::HitTestInvisible);
}

int32 SAvaLevelViewportSnapIndicators::OnPaint(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry,
	const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle,
	bool bInParentEnabled) const
{
	InLayerId = SCompoundWidget::OnPaint(InPaintArgs, InAllottedGeometry, InMyCullingRect, OutDrawElements, InLayerId, InWidgetStyle, bInParentEnabled);

	if (UAvaViewportSettings const* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		if (TSharedPtr<FAvaLevelViewportClient> AvaLevelViewportClient = AvaLevelViewportClientWeak.Pin())
		{
			if (!AvaLevelViewportClient->IsInGameView())
			{
				if (TSharedPtr<FAvaSnapOperation> SnapOperation = AvaLevelViewportClient->GetSnapOperation())
				{
					if (SnapOperation->WasSnappedTo())
					{
						++InLayerId;

						if (SnapOperation->WasSnappedToX())
						{
							DrawIndicator(InPaintArgs, InAllottedGeometry, InMyCullingRect, OutDrawElements, InLayerId,
								SnapOperation->GetSnappedToLocation().X, Orient_Horizontal,
								AvaViewportSettings->SnapIndicatorColor, AvaViewportSettings->SnapIndicatorThickness);
						}

						if (SnapOperation->WasSnappedToY())
						{
							DrawIndicator(InPaintArgs, InAllottedGeometry, InMyCullingRect, OutDrawElements, InLayerId,
								SnapOperation->GetSnappedToLocation().Y, Orient_Vertical,
								AvaViewportSettings->SnapIndicatorColor, AvaViewportSettings->SnapIndicatorThickness);
						}
					}
				}
			}
		}
	}

	return InLayerId;
}

void SAvaLevelViewportSnapIndicators::DrawIndicator(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect,
	FSlateWindowElementList& OutDrawElements, int32& InOutLayerId, float InPosition, EOrientation InLineOrientation,
	const FLinearColor& InLineColor, const float InLineThickness) const
{
	TSharedPtr<FAvaLevelViewportClient> AvaLevelViewportClient = AvaLevelViewportClientWeak.Pin();

	if (!AvaLevelViewportClient.IsValid())
	{
		return;
	}

	++InOutLayerId;

	const FAvaVisibleArea& VisibleArea = AvaLevelViewportClient->GetZoomedVisibleArea();

	if (!VisibleArea.IsValid())
	{
		return;
	}

	const FVector2f Offset = AvaLevelViewportClient->GetCachedViewportOffset();
	const FVector2f LocalSize = InAllottedGeometry.GetLocalSize();

	TArray<FVector2f> LinePoints;
	LinePoints.SetNumUninitialized(2);

	switch (InLineOrientation)
	{
		case Orient_Horizontal:
			LinePoints[0] = VisibleArea.GetVisiblePosition({InPosition, 0}) + Offset;
			LinePoints[1] = VisibleArea.GetVisiblePosition({InPosition, VisibleArea.AbsoluteSize.Y}) + Offset;
			LinePoints[0].Y = 0;
			LinePoints[1].Y = LocalSize.Y;
			break;

		case Orient_Vertical:
			LinePoints[0] = VisibleArea.GetVisiblePosition({0, InPosition}) + Offset;
			LinePoints[1] = VisibleArea.GetVisiblePosition({VisibleArea.AbsoluteSize.X, InPosition}) + Offset;
			LinePoints[0].X = 0;
			LinePoints[1].X = LocalSize.X;
			break;

		default:
			// Not possible
			return;
	}

	FSlateDrawElement::MakeLines(
		OutDrawElements,
		InOutLayerId,
		InAllottedGeometry.ToPaintGeometry(),
		LinePoints,
		ESlateDrawEffect::NoPixelSnapping,
		InLineColor,
		false,
		InLineThickness
	);
}
