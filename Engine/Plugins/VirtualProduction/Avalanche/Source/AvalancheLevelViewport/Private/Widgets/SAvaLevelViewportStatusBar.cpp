// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAvaLevelViewportStatusBar.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SAvaLevelViewportStatusBarButtons.h"
#include "Widgets/SAvaLevelViewportStatusBarTransformSettings.h"
#include "Widgets/SBoxPanel.h"

void SAvaLevelViewportStatusBar::Construct(const FArguments& InArgs, TSharedPtr<SAvaLevelViewportFrame> InViewportFrame)
{
	ViewportFrameWeak = InViewportFrame;
	
	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.MinDesiredWidth(250.f)
			[
				SNew(SAvaLevelViewportStatusBarTransformSettings, InViewportFrame)
			]
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			SNew(SBox)
			.HAlign(EHorizontalAlignment::HAlign_Right)
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				SNew(SAvaLevelViewportStatusBarButtons, InViewportFrame)
			]
		]
	];
}
