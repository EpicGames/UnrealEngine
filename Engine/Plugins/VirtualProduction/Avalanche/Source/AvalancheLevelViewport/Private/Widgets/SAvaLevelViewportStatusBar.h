// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaEditorWidgetUtils.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/SlateDelegates.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SAvaMultiComboButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SOverlay.h"

class FUICommandInfo;
class SAvaLevelViewportFrame;

namespace UE::AvaLevelViewport::Private
{
	namespace ViewportStatusBarButton
	{
		const FSlateColor ActiveColor = FSlateColor(FLinearColor(0.0f, 0.45f, 0.9f, 1.0f));
		const FSlateColor EnabledColor = FSlateColor(FStyleColors::AccentGray.GetSpecifiedColor());
		const FSlateColor DisabledColor = FSlateColor(FLinearColor(0.5f, 0.1f, 0.1f, 1.0f));
		const FMargin Padding = FMargin(0.f, 1.f, 0.f, 2.f);

		static const FButtonStyle* Style = &FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton");
		static const FMargin Margin = FMargin(1.0f, 2.0f, 1.0f, 2.0f);
		static const FVector2D ImageSize = FVector2D(16.0f, 16.0f);

		template<typename InWidgetType>
		TSharedRef<SButton> MakeButton(
			InWidgetType* InWidget,
			TSharedPtr<FUICommandInfo> InCommandInfo,
			const FSlateBrush* InBrush,
			FOnClicked::TMethodPtr<InWidgetType> InCallback,
			TAttribute<bool>::FGetter::TConstMethodPtr<InWidgetType> InEnabledFunc,
			TAttribute<FSlateColor>::FGetter::TConstMethodPtr<InWidgetType> InColorFunc)
		{
			return SNew(SButton)
				.ButtonStyle(Style)
				.ContentPadding(Margin)
				.OnClicked(InWidget, InCallback)
				.IsEnabled(InWidget, InEnabledFunc)
				.ToolTipText_Static(&FAvaEditorWidgetUtils::AddKeybindToTooltip, InCommandInfo->GetLabel(), InCommandInfo)
				[
					SNew(SImage)
					.Image(InBrush)
					.ColorAndOpacity(InWidget, InColorFunc)
					.DesiredSizeOverride(ImageSize)
				];
		}

		template<typename InGetType, typename InBrushType, typename InColorType>
		TSharedRef<SComboButton> MakeMenuButton(
			const FText& InTooltip,
			InGetType InGetMenuContent,
			InBrushType InBrush,
			InColorType InColor)
		{
			return SNew(SComboButton)
				.ButtonStyle(Style)
				.ContentPadding(Margin)
				.ToolTipText(InTooltip)
				.HasDownArrow(false)
				.OnGetMenuContent(InGetMenuContent)
				.ButtonContent()
				[
					SNew(SImage)
					.Image(InBrush)
					.ColorAndOpacity(InColor)
					.DesiredSizeOverride(ImageSize)
				];
		}

		template<typename InGetType, typename InBrushType, typename InColorType>
		TSharedRef<SAvaMultiComboButton> MakeMultiMenuButton(
			const FText& InTooltip,
			InGetType InGetMenuContent,
			InBrushType InBrush,
			InColorType InColor,
			FOnClicked InClicked)
		{
			return SNew(SAvaMultiComboButton)
				.ButtonStyle(Style)
				.ContentPadding(Margin)
				.ToolTipText(InTooltip)
				.HasDownArrow(false)
				.OnGetMenuContent(InGetMenuContent)
				.OnButtonClicked(InClicked)
				.ButtonContent()
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					.Padding(0.f, 0.f, ImageSize.X - 1.f, 0.f)
					[
						SNew(SImage)
						.Image(InBrush)
						.ColorAndOpacity(InColor)
						.DesiredSizeOverride(ImageSize)
					]
					+ SOverlay::Slot()
					.Padding(ImageSize.X - 1.f, 0.f, 0.f, 0.f)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.ChevronDown"))
						.DesiredSizeOverride(ImageSize)
						.ColorAndOpacity(FSlateColor(FLinearColor(0.5f, 0.5f, 0.5f, 1.f)))
					]
				];
		}
	}
}

class SAvaLevelViewportStatusBar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaLevelViewportStatusBar) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SAvaLevelViewportFrame> InViewportFrame);

	TSharedPtr<SAvaLevelViewportFrame> GetViewportFrameWeak() const { return ViewportFrameWeak.Pin(); }

protected:
	TWeakPtr<SAvaLevelViewportFrame> ViewportFrameWeak;
};
