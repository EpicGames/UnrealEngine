// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAvaLevelViewportStatusBarButtons.h"

#include "AvaEditorWidgetUtils.h"
#include "AvaInteractiveToolsCommands.h"
#include "AvaInteractiveToolsSettings.h"
#include "AvaLevelViewportCommands.h"
#include "AvaLevelViewportModule.h"
#include "AvaLevelViewportStyle.h"
#include "AvaViewportPostProcessManager.h"
#include "AvaViewportSettings.h"
#include "CoreGlobals.h"
#include "EditorSupportDelegates.h"
#include "Engine/Texture.h"
#include "IAvalancheInteractiveToolsModule.h"
#include "Interaction/AvaIsolateActorsOperation.h"
#include "LevelEditor.h"
#include "LevelViewportActions.h"
#include "PropertyCustomizationHelpers.h"
#include "SAvaLevelViewport.h"
#include "SAvaLevelViewportFrame.h"
#include "SAvaViewportInfo.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/PropertyAnimatorCoreSubsystem.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ViewportClient/AvaLevelViewportClient.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SAvaLevelViewportActorAlignmentMenu.h"
#include "Widgets/SAvaLevelViewportActorColorMenu.h"
#include "Widgets/SAvaLevelViewportStatusBar.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "SAvaLevelViewportStatusBarButtons"

namespace UE::Ava::LevelViewportStatusBarButtons::Private
{
	static const FName AvaLevelViewportStyleName = FAvaLevelViewportStyle::Get().GetStyleSetName();
	static const FName AppStyleSetName = FAppStyle::Get().GetStyleSetName();
	static const FSlateIcon RGBChannelIcon = FSlateIcon(AvaLevelViewportStyleName, "Icons.PostProcess.RGB");
	static const FSlateIcon BackgroundIcon = FSlateIcon(AppStyleSetName, "Icons.Role");
	static const FSlateIcon RedChannelIcon = FSlateIcon(AvaLevelViewportStyleName, "Icons.PostProcess.Red");
	static const FSlateIcon GreenChannelIcon = FSlateIcon(AvaLevelViewportStyleName, "Icons.PostProcess.Green");
	static const FSlateIcon BlueChannelIcon = FSlateIcon(AvaLevelViewportStyleName, "Icons.PostProcess.Blue");
	static const FSlateIcon AlphaChannelIcon = FSlateIcon(AvaLevelViewportStyleName, "Icons.PostProcess.Alpha");
	static const FSlateIcon CheckerboardIcon = FSlateIcon(AppStyleSetName, "Checker");

	static void TogglePostProcess(const TWeakPtr<SAvaLevelViewportFrame>& InViewportFrameWeak, EAvaViewportPostProcessType InPostProcessType)
	{
		const FAvaLevelViewportGuideFrameAndClient FrameAndClient(InViewportFrameWeak);

		if (FrameAndClient.IsValid())
		{
			if (const TSharedPtr<FAvaViewportPostProcessManager> PostProcessManager = FrameAndClient.ViewportClient->GetPostProcessManager())
			{
				// None should always apply None.
				if (InPostProcessType == EAvaViewportPostProcessType::None || PostProcessManager->GetType() == InPostProcessType)
				{
					PostProcessManager->SetType(EAvaViewportPostProcessType::None);
				}
				else
				{
					PostProcessManager->SetType(InPostProcessType);
				}
			}
			else
			{
				UE_LOGF(AvaLevelViewportLog, Warning, "TogglePostProcess: Unable to find post process manager.");
			}
		}
		else
		{
			UE_LOGF(AvaLevelViewportLog, Warning, "TogglePostProcess: Invalid viewport frame/client.");
		}
	}
}

void SAvaLevelViewportStatusBarButtons::Construct(const FArguments& InArgs, TSharedPtr<SAvaLevelViewportFrame> InViewportFrame)
{
	ViewportFrameWeak = InViewportFrame;

	constexpr float Padding = 5.f;

	TSharedPtr<SHorizontalBox> ActorButtons;
	TSharedPtr<SHorizontalBox> ViewportButtons;

	TSharedPtr<SWidget> StatusBarWidget = nullptr;
	{
		static FName MenuName = UE::AvaLevelViewport::Internal::StatusBarMenuName;
		UToolMenu* Menu = UToolMenus::Get()->FindMenu(MenuName);
		StatusBarWidget = UToolMenus::Get()->GenerateWidget(Menu);
	}

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0, Padding, Padding, Padding)
		[
			SNew(SSeparator)
			.Orientation(EOrientation::Orient_Vertical)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0, 0, Padding, 0)
		[
			StatusBarWidget.ToSharedRef()
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0, Padding, Padding, Padding)
		[
			SNew(SSeparator)
			.Orientation(EOrientation::Orient_Vertical)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0, 0, Padding, 0)
		[
			SAssignNew(ActorButtons, SHorizontalBox)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0, Padding, Padding, Padding)
		[
			SNew(SSeparator)
			.Orientation(EOrientation::Orient_Vertical)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0, 0, Padding, 0)
		[
			SAssignNew(ViewportButtons, SHorizontalBox)
		]
	];

	CreateContextMenuWigets();

	PopulateActorButtons(ActorButtons);
	PopulateViewportButtons(ViewportButtons);
}

void SAvaLevelViewportStatusBarButtons::CreateContextMenuWigets()
{
	FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (!FrameAndWidget.IsValid())
	{
		return;
	}

	SAvaLevelViewport* LevelViewport = FrameAndWidget.ViewportWidget.Get();

	if (!PostProcessOpacitySlider.IsValid())
	{
		PostProcessOpacitySlider = SNew(SSpinBox<float>)
			.ClearKeyboardFocusOnCommit(true)
			.MaxFractionalDigits(3)
			.MinDesiredWidth(50.f)
			.OnBeginSliderMovement(LevelViewport, &SAvaLevelViewport::OnBackgroundOpacitySliderBegin)
			.OnEndSliderMovement(LevelViewport, &SAvaLevelViewport::OnBackgroundOpacitySliderEnd)
			.OnValueCommitted(LevelViewport, &SAvaLevelViewport::OnBackgroundOpacityCommitted)
			.OnValueChanged(LevelViewport, &SAvaLevelViewport::OnBackgroundOpacityCommitted, ETextCommit::Default)
			.Value(LevelViewport, &SAvaLevelViewport::GetBackgroundOpacity)
			.MinValue(0.f)
			.MinSliderValue(0.f)
			.MaxValue(1.f)
			.MaxSliderValue(1.f);
	}

	if (!BackgroundTextureSelector.IsValid())
	{
		BackgroundTextureSelector = SNew(SObjectPropertyEntryBox)
			.AllowClear(true)
			.AllowedClass(UTexture::StaticClass())
			.DisplayBrowse(true)
			.DisplayThumbnail(true)
			.DisplayCompactSize(true)
			.DisplayUseSelected(true)
			.ThumbnailPool(UThumbnailManager::Get().GetSharedThumbnailPool())
			.EnableContentPicker(true)
			.ObjectPath(LevelViewport, &SAvaLevelViewport::GetBackgroundTextureObjectPath)
			.OnObjectChanged(LevelViewport, &SAvaLevelViewport::OnBackgroundTextureChanged)
			.OnShouldSetAsset(FOnShouldSetAsset::CreateLambda([](const FAssetData& InAssetData) { return false; }));
	}

	if (!GridSizeSlider.IsValid())
	{
		GridSizeSlider = SNew(SBox)
			.Padding(10.f, 0.f, 0.f, 0.f)
			[
				SNew(SSpinBox<int32>)
				.Justification(ETextJustify::Center)
				.Style(&FAppStyle::Get(), "Menu.SpinBox")
				.Font(FAppStyle::GetFontStyle("TinyText"))
				.MinValue(1)
				.MaxValue(256)
				.Value_Lambda([]() { return GetDefault<UAvaViewportSettings>()->GridSize; })
				.IsEnabled(this, &SAvaLevelViewportStatusBarButtons::CanChangeGridSize)
				.OnValueChanged(this, &SAvaLevelViewportStatusBarButtons::OnGridSizeChanged)
				.OnValueCommitted(this, &SAvaLevelViewportStatusBarButtons::OnGridSizeCommitted)
			];
	}

	if (!TextureOverlayOpacitySlider.IsValid())
	{
		TextureOverlayOpacitySlider = SNew(SSpinBox<float>)
			.ClearKeyboardFocusOnCommit(true)
			.MaxFractionalDigits(3)
			.MinDesiredWidth(50.f)
			.OnEndSliderMovement(LevelViewport, &SAvaLevelViewport::OnTextureOverlayOpacitySliderEnd)
			.OnValueCommitted(LevelViewport, &SAvaLevelViewport::OnTextureOverlayOpacityCommitted)
			.OnValueChanged(LevelViewport, &SAvaLevelViewport::OnTextureOverlayOpacityChanged)
			.Value(LevelViewport, &SAvaLevelViewport::GetTextureOverlayOpacity)
			.MinValue(0.f)
			.MinSliderValue(0.f)
			.MaxValue(1.f)
			.MaxSliderValue(1.f);
	}

	if (!TextureOverlayTextureSelector.IsValid())
	{
		TextureOverlayTextureSelector = SNew(SObjectPropertyEntryBox)
			.AllowClear(true)
			.AllowedClass(UTexture::StaticClass())
			.DisplayBrowse(true)
			.DisplayThumbnail(true)
			.DisplayCompactSize(true)
			.DisplayUseSelected(true)
			.ThumbnailPool(UThumbnailManager::Get().GetSharedThumbnailPool())
			.EnableContentPicker(true)
			.ObjectPath(LevelViewport, &SAvaLevelViewport::GetTextureOverlayTextureObjectPath)
			.OnObjectChanged(LevelViewport, &SAvaLevelViewport::OnTextureOverlayTextureChanged)
			.OnShouldSetAsset(FOnShouldSetAsset::CreateLambda([](const FAssetData& InAssetData) { return false; }));
	}

	if (!TextureOverlayStretchCheckBox.IsValid())
	{
		TextureOverlayStretchCheckBox = SNew(SCheckBox)
			.IsChecked(LevelViewport, &SAvaLevelViewport::GetTextureOverlayStretchEnabledCheckBoxState)
			.OnCheckStateChanged(LevelViewport, &SAvaLevelViewport::OnTextureOverlayStretchEnabledCheckBoxChanged);
	}
}

void SAvaLevelViewportStatusBarButtons::PopulateActorButtons(TSharedPtr<SHorizontalBox> InContainer)
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportCommands& CommandsRef = FAvaLevelViewportCommands::GetInternal();

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			ViewportStatusBarButton::MakeMenuButton(
				LOCTEXT("ActorColor", "Actor Color"),
				FOnGetContent::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetActorColorMenuContent),
				FAppStyle::Get().GetBrush(TEXT("ColorPicker.Mode")),
				FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f))
			)
		];

	TSharedRef<SComboButton> AlignmentButton = ViewportStatusBarButton::MakeMenuButton(
		LOCTEXT("ActorAlign", "Align Actors"),
		FOnGetContent::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetActorAlignmentMenuContent),
		FAvaLevelViewportStyle::Get().GetBrush(TEXT("Icons.Alignment.Center_Y")),
		TAttribute<FSlateColor>::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetActorAlignmentColor)
	);

	AlignmentButton->SetEnabled(TAttribute<bool>::CreateSP(
		this,
		&SAvaLevelViewportStatusBarButtons::GetActorAlignmentEnabled
	));

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			AlignmentButton
		];

	const FSlateBrush* AnimatorBrush = FSlateIconFinder::FindIconBrushForClass(UPropertyAnimatorCoreBase::StaticClass());

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			ViewportStatusBarButton::MakeButton(
				this,
				CommandsRef.EnableAnimators,
				AnimatorBrush,
				&SAvaLevelViewportStatusBarButtons::EnableAnimators,
				&SAvaLevelViewportStatusBarButtons::GetAnimatorButtonEnabled,
				&SAvaLevelViewportStatusBarButtons::GetAnimatorButtonUnmuteColor
			)
		];

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			ViewportStatusBarButton::MakeButton(
				this,
				CommandsRef.DisableAnimators,
				AnimatorBrush,
				&SAvaLevelViewportStatusBarButtons::DisableAnimators,
				&SAvaLevelViewportStatusBarButtons::GetAnimatorButtonEnabled,
				&SAvaLevelViewportStatusBarButtons::GetAnimatorButtonMuteColor
			)
		];
}

void SAvaLevelViewportStatusBarButtons::PopulateViewportButtons(TSharedPtr<SHorizontalBox> InContainer)
{
	using namespace UE::AvaLevelViewport::Private;
	using namespace UE::Ava::LevelViewportStatusBarButtons::Private;

	const FAvaLevelViewportCommands& CommandsRef = FAvaLevelViewportCommands::GetInternal();
	const FLevelViewportCommands& ViewportActionsRef = FLevelViewportCommands::Get();
	const FAvaInteractiveToolsCommands* IFCommands = FAvaInteractiveToolsCommands::GetExternal();

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			ViewportStatusBarButton::MakeButton(
				this,
				IFCommands->ToggleViewportToolbar,
				IFCommands->ToggleViewportToolbar->GetIcon().GetIcon(),
				&SAvaLevelViewportStatusBarButtons::ToggleViewportToolbar,
				&SAvaLevelViewportStatusBarButtons::GetToggleViewportToolbarEnabled,
				&SAvaLevelViewportStatusBarButtons::GetToggleViewportToolbarColor
			)
		];

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			ViewportStatusBarButton::MakeButton(
				this,
				ViewportActionsRef.ToggleGameView,
				FAvaLevelViewportStyle::Get().GetBrush(TEXT("Button.GameView")),
				&SAvaLevelViewportStatusBarButtons::ToggleGameView,
				&SAvaLevelViewportStatusBarButtons::GetToggleGameViewEnabled,
				&SAvaLevelViewportStatusBarButtons::GetToggleGameViewColor
			)
		];

	TSharedRef<SComboButton> PostProcessButton = ViewportStatusBarButton::MakeMenuButton(
		LOCTEXT("PostProcessEffects", "Post Process Effects"),
		FOnGetContent::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetPostProcessMenuContent),
		RGBChannelIcon.GetIcon(),
		FLinearColor::White
	);

	PostProcessButton->SetEnabled(TAttribute<bool>::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetPostProcessEnabled));

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			PostProcessButton
		];

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			ViewportStatusBarButton::MakeButton(
				this,
				CommandsRef.ToggleIsolateActors,
				FAvaLevelViewportStyle::Get().GetBrush(TEXT("Button.IsolateActors")),
				&SAvaLevelViewportStatusBarButtons::ToggleIsolateActors,
				&SAvaLevelViewportStatusBarButtons::GetToggleIsolateActorsEnabled,
				&SAvaLevelViewportStatusBarButtons::GetToggleIsolateActorsColor
			)
		];

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			ViewportStatusBarButton::MakeButton(
				this,
				CommandsRef.ToggleBoundingBoxes,
				FAvaLevelViewportStyle::Get().GetBrush(TEXT("Button.BoundingBoxes")),
				&SAvaLevelViewportStatusBarButtons::ToggleBoundingBoxes,
				&SAvaLevelViewportStatusBarButtons::GetToggleBoundingBoxesEnabled,
				&SAvaLevelViewportStatusBarButtons::GetToggleBoundingBoxesColor
			)
		];

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(5.f)
		[
			SNew(SSeparator)
			.Orientation(EOrientation::Orient_Vertical)
		];

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			ViewportStatusBarButton::MakeButton(
				this,
				CommandsRef.ToggleOverlay,
				FAppStyle::GetBrush("Icons.Visible"),
				&SAvaLevelViewportStatusBarButtons::ToggleOverlay,
				&SAvaLevelViewportStatusBarButtons::GetToggleOverlayEnabled,
				&SAvaLevelViewportStatusBarButtons::GetToggleOverlayColor
			)
		];

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			ViewportStatusBarButton::MakeButton(
				this,
				CommandsRef.ToggleSafeFrames,
				FAvaLevelViewportStyle::Get().GetBrush(TEXT("Button.SafeFrames")),
				&SAvaLevelViewportStatusBarButtons::ToggleSafeFrames,
				&SAvaLevelViewportStatusBarButtons::GetToggleSafeFramesEnabled,
				&SAvaLevelViewportStatusBarButtons::GetToggleSafeFramesColor
			)
		];

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			ViewportStatusBarButton::MakeButton(
				this,
				CommandsRef.ToggleGuides,
				FAvaLevelViewportStyle::Get().GetBrush(TEXT("Button.Guides")),
				&SAvaLevelViewportStatusBarButtons::ToggleGuides,
				&SAvaLevelViewportStatusBarButtons::GetToggleGuidesEnabled,
				&SAvaLevelViewportStatusBarButtons::GetToggleGuidesColor
			)
		];

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			ViewportStatusBarButton::MakeButton(
				this,
				CommandsRef.ToggleShapeEditorOverlay,
				FAppStyle::Get().GetBrush(TEXT("Icons.Filter")),
				&SAvaLevelViewportStatusBarButtons::ToggleShapeEditorOverlay,
				&SAvaLevelViewportStatusBarButtons::GetToggleShapeEditorOverlayEnabled,
				&SAvaLevelViewportStatusBarButtons::GetToggleShapeEditorOverlayColor
			)
		];

	TSharedRef<SAvaMultiComboButton> TextureOverlayButton = ViewportStatusBarButton::MakeMultiMenuButton(
		CommandsRef.ToggleTextureOverlay->GetDescription(),
		FOnGetContent::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetTextureOverlayMenuContent),
		FAppStyle::Get().GetBrush(TEXT("GenericCommands.Paste")),
		TAttribute<FSlateColor>::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetTextureOverlayColor),
		FOnClicked::CreateSP(this, &SAvaLevelViewportStatusBarButtons::ToggleTextureOverlay)
	);

	TextureOverlayButton->SetEnabled(TAttribute<bool>::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetTextureOverlayEnabled));

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			TextureOverlayButton
		];

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(5.f)
		[
			SNew(SSeparator)
			.Orientation(EOrientation::Orient_Vertical)
		];

	TSharedRef<SAvaMultiComboButton> GridButton = ViewportStatusBarButton::MakeMultiMenuButton(
		CommandsRef.ToggleGrid->GetDescription(),
		FOnGetContent::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetGridMenuContent),
		FAvaLevelViewportStyle::Get().GetBrush(TEXT("Button.ToggleGrid")),
		TAttribute<FSlateColor>::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetToggleGridColor),
		FOnClicked::CreateSP(this, &SAvaLevelViewportStatusBarButtons::ToggleGrid)
	);

	GridButton->SetEnabled(TAttribute<bool>::CreateSP(
		this,
		&SAvaLevelViewportStatusBarButtons::GetToggleGridEnabled
	));

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			GridButton
		];

	TSharedRef<SAvaMultiComboButton> SnapButton = ViewportStatusBarButton::MakeMultiMenuButton(
		CommandsRef.ToggleSnapping->GetDescription(),
		FOnGetContent::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetSnappingMenuContent),
		FAvaLevelViewportStyle::Get().GetBrush(TEXT("Button.ToggleSnap")),
		TAttribute<FSlateColor>::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetToggleSnapColor),
		FOnClicked::CreateSP(this, &SAvaLevelViewportStatusBarButtons::ToggleSnap)
	);

	SnapButton->SetEnabled(TAttribute<bool>::CreateSP(
		this,
		&SAvaLevelViewportStatusBarButtons::GetToggleSnapEnabled
	));

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			SnapButton
		];

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			ViewportStatusBarButton::MakeButton(
				this,
				ViewportActionsRef.HighResScreenshot,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelViewport.HighResScreenshot").GetIcon(),
				&SAvaLevelViewportStatusBarButtons::HighResScreenshot,
				&SAvaLevelViewportStatusBarButtons::GetHighResScreenshotEnabled,
				&SAvaLevelViewportStatusBarButtons::GetHighResScreenshotColor
			)
		];

	TSharedRef<SComboButton> ViewportInfoButton = ViewportStatusBarButton::MakeMenuButton(
		LOCTEXT("ViewportInfomation", "Viewport Information"),
		FOnGetContent::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetViewportInfoWidget),
		FAppStyle::Get().GetBrush(TEXT("Icons.AutoFilter")),
		TAttribute<FSlateColor>::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetViewportInfoColor)
	);

	ViewportInfoButton->SetEnabled(TAttribute<bool>::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetViewportInfoEnabled));

	InContainer->AddSlot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			ViewportInfoButton
		];
}

bool SAvaLevelViewportStatusBarButtons::GetPostProcessEnabled() const
{
	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (FrameAndClient.IsValid())
	{
		return FrameAndClient.ViewportClient->GetPostProcessManager().IsValid();
	}

	return false;
}

TSharedRef<SWidget> SAvaLevelViewportStatusBarButtons::GetPostProcessMenuContent()
{
	UToolMenus* Menus = UToolMenus::Get();

	check(Menus);

	static const FName BackgroundMenuName = TEXT("AvaLevelViewport.StatusBar.PostProcess.Background");

	UToolMenu* ContextMenu = Menus->FindMenu(BackgroundMenuName);

	if (!ContextMenu)
	{
		ContextMenu = Menus->RegisterMenu(BackgroundMenuName, NAME_None, EMultiBoxType::Menu);

		if (!ContextMenu)
		{
			return SNullWidget::NullWidget;
		}
	}

	using namespace UE::Ava::LevelViewportStatusBarButtons::Private;

	const FAvaLevelViewportCommands& CommandsRef = FAvaLevelViewportCommands::GetInternal();

	FToolMenuSection& EffectsSection = ContextMenu->FindOrAddSection("Effects", LOCTEXT("Effects", "Effects"));

	FToolUIAction RGBAction;
	RGBAction.ExecuteAction = FToolMenuExecuteAction::CreateSP(this, &SAvaLevelViewportStatusBarButtons::TogglePostProcessMenu, EAvaViewportPostProcessType::None);
	RGBAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetPostProcessActiveMenu, EAvaViewportPostProcessType::None);
	RGBAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetPostProcessEnabledMenu);

	EffectsSection.AddEntry(FToolMenuEntry::InitMenuEntry(
		"RGB",
		LOCTEXT("RGB", "RGB"),
		CommandsRef.TogglePostProcessNone->GetDescription(),
		RGBChannelIcon,
		FToolUIActionChoice(RGBAction),
		EUserInterfaceActionType::Check
	));

	FToolUIAction BackgroundAction;
	BackgroundAction.ExecuteAction = FToolMenuExecuteAction::CreateSP(this, &SAvaLevelViewportStatusBarButtons::TogglePostProcessMenu, EAvaViewportPostProcessType::Background);
	BackgroundAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetPostProcessActiveMenu, EAvaViewportPostProcessType::Background);
	BackgroundAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetPostProcessEnabledMenu);

	EffectsSection.AddEntry(FToolMenuEntry::InitMenuEntry(
		"Background", 
		LOCTEXT("Background", "Background"), 
		CommandsRef.TogglePostProcessBackground->GetDescription(),
		BackgroundIcon,
		FToolUIActionChoice(BackgroundAction),
		EUserInterfaceActionType::Check
	));

	FToolUIAction RedAction;
	RedAction.ExecuteAction = FToolMenuExecuteAction::CreateSP(this, &SAvaLevelViewportStatusBarButtons::TogglePostProcessMenu, EAvaViewportPostProcessType::RedChannel);
	RedAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetPostProcessActiveMenu, EAvaViewportPostProcessType::RedChannel);
	RedAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetPostProcessEnabledMenu);

	EffectsSection.AddEntry(FToolMenuEntry::InitMenuEntry(
		"Red",
		LOCTEXT("Red", "Red"),
		CommandsRef.TogglePostProcessChannelRed->GetDescription(),
		RedChannelIcon,
		FToolUIActionChoice(RedAction),
		EUserInterfaceActionType::Check
	));

	FToolUIAction GreenAction;
	GreenAction.ExecuteAction = FToolMenuExecuteAction::CreateSP(this, &SAvaLevelViewportStatusBarButtons::TogglePostProcessMenu, EAvaViewportPostProcessType::GreenChannel);
	GreenAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetPostProcessActiveMenu, EAvaViewportPostProcessType::GreenChannel);
	GreenAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetPostProcessEnabledMenu);

	EffectsSection.AddEntry(FToolMenuEntry::InitMenuEntry(
		"Green",
		LOCTEXT("Green", "Green"),
		CommandsRef.TogglePostProcessChannelGreen->GetDescription(),
		GreenChannelIcon,
		FToolUIActionChoice(GreenAction),
		EUserInterfaceActionType::Check
	));

	FToolUIAction BlueAction;
	BlueAction.ExecuteAction = FToolMenuExecuteAction::CreateSP(this, &SAvaLevelViewportStatusBarButtons::TogglePostProcessMenu, EAvaViewportPostProcessType::BlueChannel);
	BlueAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetPostProcessActiveMenu, EAvaViewportPostProcessType::BlueChannel);
	BlueAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetPostProcessEnabledMenu);

	EffectsSection.AddEntry(FToolMenuEntry::InitMenuEntry(
		"Blue",
		LOCTEXT("Blue", "Blue"),
		CommandsRef.TogglePostProcessChannelBlue->GetDescription(),
		BlueChannelIcon,
		FToolUIActionChoice(BlueAction),
		EUserInterfaceActionType::Check
	));

	FToolUIAction AlphaAction;
	AlphaAction.ExecuteAction = FToolMenuExecuteAction::CreateSP(this, &SAvaLevelViewportStatusBarButtons::TogglePostProcessMenu, EAvaViewportPostProcessType::AlphaChannel);
	AlphaAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetPostProcessActiveMenu, EAvaViewportPostProcessType::AlphaChannel);
	AlphaAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetPostProcessEnabledMenu);

	EffectsSection.AddEntry(FToolMenuEntry::InitMenuEntry(
		"Alpha",
		LOCTEXT("Alpha", "Alpha"),
		CommandsRef.TogglePostProcessChannelAlpha->GetDescription(),
		AlphaChannelIcon,
		FToolUIActionChoice(AlphaAction),
		EUserInterfaceActionType::Check
	));

	FToolUIAction CheckerboardAction;
	CheckerboardAction.ExecuteAction = FToolMenuExecuteAction::CreateSP(this, &SAvaLevelViewportStatusBarButtons::TogglePostProcessMenu, EAvaViewportPostProcessType::Checkerboard);
	CheckerboardAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetPostProcessActiveMenu, EAvaViewportPostProcessType::Checkerboard);
	CheckerboardAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateSP(this, &SAvaLevelViewportStatusBarButtons::GetPostProcessEnabledMenu);

	EffectsSection.AddEntry(FToolMenuEntry::InitMenuEntry(
		"Checkerboard",
		LOCTEXT("Checkerboard", "Checkerboard"),
		CommandsRef.TogglePostProcessCheckerboard->GetDescription(),
		CheckerboardIcon,
		FToolUIActionChoice(CheckerboardAction),
		EUserInterfaceActionType::Check
	));

	FToolMenuSection& OptionsSection = ContextMenu->AddSection("Options", LOCTEXT("Options", "Options"));

	if (PostProcessOpacitySlider.IsValid())
	{
		OptionsSection.AddEntry(FToolMenuEntry::InitWidget(
			"PostProcessOpacity",
			PostProcessOpacitySlider.ToSharedRef(),
			LOCTEXT("PostProcessOpacity", "Opacity"),
			true
		));
	}

	if (BackgroundTextureSelector.IsValid())
	{
		OptionsSection.AddEntry(FToolMenuEntry::InitWidget(
			"PostProcessTexture",
			BackgroundTextureSelector.ToSharedRef(),
			LOCTEXT("PostProcessTexture", "Texture"),
			true
		));
	}

	return Menus->GenerateWidget(ContextMenu);
}

bool SAvaLevelViewportStatusBarButtons::GetPostProcessEnabledMenu(const FToolMenuContext& InContext) const
{
	return GetPostProcessEnabled();
}

ECheckBoxState SAvaLevelViewportStatusBarButtons::GetPostProcessActiveMenu(const FToolMenuContext& InContext, EAvaViewportPostProcessType InPostProcessType) const
{
	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (FrameAndClient.IsValid())
	{
		if (const TSharedPtr<FAvaViewportPostProcessManager> PostProcessManager = FrameAndClient.ViewportClient->GetPostProcessManager())
		{
			if (PostProcessManager->GetType() == InPostProcessType)
			{
				return ECheckBoxState::Checked;
			}
		}
	}

	return ECheckBoxState::Unchecked;
}

void SAvaLevelViewportStatusBarButtons::TogglePostProcessMenu(const FToolMenuContext& InContext, EAvaViewportPostProcessType InPostProcessType)
{
	FScopedTransaction Transaction(LOCTEXT("ChangeViewportPostProcess", "Change Viewport Post Process"));
	UE::Ava::LevelViewportStatusBarButtons::Private::TogglePostProcess(ViewportFrameWeak, InPostProcessType);
}

FSlateColor SAvaLevelViewportStatusBarButtons::GetToggleViewportToolbarColor() const
{
	if (const UAvaInteractiveToolsSettings* ITSettings = UAvaInteractiveToolsSettings::Get())
	{
		if (ITSettings->IsViewportToolbarSupported())
		{
			const bool bIsVisible = ITSettings->GetViewportToolbarVisible();
			return bIsVisible
				? UE::AvaLevelViewport::Private::ViewportStatusBarButton::ActiveColor
				: UE::AvaLevelViewport::Private::ViewportStatusBarButton::EnabledColor;
		}
	}
	
	return UE::AvaLevelViewport::Private::ViewportStatusBarButton::DisabledColor;
}

bool SAvaLevelViewportStatusBarButtons::GetToggleViewportToolbarEnabled() const
{
	if (const UAvaInteractiveToolsSettings* ITSettings = UAvaInteractiveToolsSettings::Get())
	{
		return ITSettings->IsViewportToolbarSupported();
	}

	return false;
}

FReply SAvaLevelViewportStatusBarButtons::ToggleViewportToolbar()
{
	if (const UAvaInteractiveToolsSettings* ITSettings = UAvaInteractiveToolsSettings::Get())
	{
		const bool bIsVisible = ITSettings->GetViewportToolbarVisible();
		ITSettings->SetViewportToolbarVisible(!bIsVisible);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FSlateColor SAvaLevelViewportStatusBarButtons::GetHighResScreenshotColor() const
{
	return FSlateColor(EStyleColor::Foreground);
}

bool SAvaLevelViewportStatusBarButtons::GetHighResScreenshotEnabled() const
{
	if (GIsHighResScreenshot)
	{
		return false;
	}

	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (FrameAndClient.IsValid())
	{
		return !!FrameAndClient.ViewportClient->Viewport;
	}

	return false;
}

FReply SAvaLevelViewportStatusBarButtons::HighResScreenshot()
{
	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (FrameAndClient.IsValid() && !!FrameAndClient.ViewportClient->Viewport)
	{
		GScreenshotResolutionX = 0;
		GScreenshotResolutionY = 0;

		FrameAndClient.ViewportClient->Viewport->TakeHighResScreenShot();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FSlateColor SAvaLevelViewportStatusBarButtons::GetActorAlignmentColor() const
{
	static const FSlateColor Active = FSlateColor(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f));
	static const FSlateColor Inactive = FSlateColor(FLinearColor(0.3f, 0.3f, 0.3f, 1.0f));

	if (GetActorAlignmentEnabled())
	{
		return Active;
	}

	return Inactive;
}

bool SAvaLevelViewportStatusBarButtons::GetActorAlignmentEnabled() const
{
	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (FrameAndClient.IsValid())
	{
		if (FEditorModeTools* ModeTools = FrameAndClient.ViewportClient->GetModeTools())
		{
			if (USelection* ActorSelection = ModeTools->GetSelectedActors())
			{
				return ActorSelection->Num() > 0;
			}
		}
	}

	return false;
}

TSharedRef<SWidget> SAvaLevelViewportStatusBarButtons::GetActorAlignmentMenuContent() const
{
	if (!GetActorAlignmentEnabled())
	{
		return SNullWidget::NullWidget;
	}

	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (!FrameAndWidget.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<ILevelEditor> LevelEditor = FrameAndWidget.ViewportWidget->GetParentLevelEditor().Pin();

	if (!LevelEditor.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	return SAvaLevelViewportActorAlignmentMenu::CreateMenu(LevelEditor.ToSharedRef());
}

TSharedRef<SWidget> SAvaLevelViewportStatusBarButtons::GetActorColorMenuContent() const
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (!FrameAndWidget.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<ILevelEditor> LevelEditor = FrameAndWidget.ViewportWidget->GetParentLevelEditor().Pin();

	if (!LevelEditor.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	return SAvaLevelViewportActorColorMenu::CreateMenu(LevelEditor.ToSharedRef());
}

FSlateColor SAvaLevelViewportStatusBarButtons::GetAnimatorButtonMuteColor() const
{
	return UE::AvaLevelViewport::Private::ViewportStatusBarButton::EnabledColor;
}

FSlateColor SAvaLevelViewportStatusBarButtons::GetAnimatorButtonUnmuteColor() const
{
	return UE::AvaLevelViewport::Private::ViewportStatusBarButton::ActiveColor;
}

bool SAvaLevelViewportStatusBarButtons::GetAnimatorButtonEnabled() const
{
	return true;
}

FReply SAvaLevelViewportStatusBarButtons::EnableAnimators()
{
	const TSharedPtr<SAvaLevelViewportFrame> ViewportFrame = ViewportFrameWeak.Pin();

	if (!ViewportFrame.IsValid())
	{
		return FReply::Handled();
	}

	const TSharedPtr<FAvaLevelViewportClient> ViewportClient = ViewportFrame->GetViewportClient();

	if (!ViewportClient.IsValid())
	{
		return FReply::Handled();
	}

	const FEditorModeTools* ModeTools = ViewportClient->GetModeTools();

	if (!ModeTools)
	{
		return FReply::Handled();
	}

	UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get();
	const UWorld* World = ModeTools->GetWorld();
	const UTypedElementSelectionSet* SelectionSet = ModeTools->GetEditorSelectionSet();

	if (!World || !SelectionSet || !AnimatorSubsystem)
	{
		return FReply::Handled();
	}

	const TSet<AActor*> SelectedActors(SelectionSet->GetSelectedObjects<AActor>());

	if (SelectedActors.IsEmpty())
	{
		AnimatorSubsystem->SetLevelAnimatorsEnabled(World, /** Enabled */true, /** Transact */true);
	}
	else
	{
		AnimatorSubsystem->SetActorAnimatorsEnabled(SelectedActors, /** Enabled */true, /** Transact */true);
	}

	return FReply::Handled();
}

FReply SAvaLevelViewportStatusBarButtons::DisableAnimators()
{
	const TSharedPtr<SAvaLevelViewportFrame> ViewportFrame = ViewportFrameWeak.Pin();

	if (!ViewportFrame.IsValid())
	{
		return FReply::Handled();
	}

	const TSharedPtr<FAvaLevelViewportClient> ViewportClient = ViewportFrame->GetViewportClient();

	if (!ViewportClient.IsValid())
	{
		return FReply::Handled();
	}

	const FEditorModeTools* ModeTools = ViewportClient->GetModeTools();

	if (!ModeTools)
	{
		return FReply::Handled();
	}

	UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get();
	const UWorld* World = ModeTools->GetWorld();
	const UTypedElementSelectionSet* SelectionSet = ModeTools->GetEditorSelectionSet();

	if (!World || !SelectionSet || !AnimatorSubsystem)
	{
		return FReply::Handled();
	}

	const TSet<AActor*> SelectedActors(SelectionSet->GetSelectedObjects<AActor>());

	if (SelectedActors.IsEmpty())
	{
		AnimatorSubsystem->SetLevelAnimatorsEnabled(World, /** Enabled */false, /** Transact */true);
	}
	else
	{
		AnimatorSubsystem->SetActorAnimatorsEnabled(SelectedActors, /** Enabled */false, /** Transact */true);
	}

	return FReply::Handled();
}

FSlateColor SAvaLevelViewportStatusBarButtons::GetToggleSnapColor() const
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid() && FrameAndWidget.ViewportWidget->CanToggleSnapping())
	{
		if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
		{
			return EnumHasAllFlags(AvaViewportSettings->GetSnapState(), EAvaViewportSnapState::Global)
				? ViewportStatusBarButton::ActiveColor
				: ViewportStatusBarButton::EnabledColor;
		}
	}

	return ViewportStatusBarButton::DisabledColor;
}

bool SAvaLevelViewportStatusBarButtons::GetToggleSnapEnabled() const
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid())
	{
		return FrameAndWidget.ViewportWidget->CanToggleSnapping();
	}

	return false;
}

FReply SAvaLevelViewportStatusBarButtons::ToggleSnap()
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid() && FrameAndWidget.ViewportWidget->CanToggleSnapping())
	{
		FrameAndWidget.ViewportWidget->ExecuteToggleSnapping();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

TSharedRef<SWidget> SAvaLevelViewportStatusBarButtons::GetSnappingMenuContent() const
{
	UToolMenus* Menus = UToolMenus::Get();

	check(Menus);

	static const FName SnapMenuName = TEXT("AvaLevelViewport.StatusBar.Snapping");

	UToolMenu* ContextMenu = Menus->FindMenu(SnapMenuName);

	if (!ContextMenu)
	{
		ContextMenu = Menus->RegisterMenu(SnapMenuName, NAME_None, EMultiBoxType::Menu);

		if (!ContextMenu)
		{
			return SNullWidget::NullWidget;
		}
	}

	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (!FrameAndWidget.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	FToolMenuSection& SnapToSection = ContextMenu->FindOrAddSection("SnapTo", LOCTEXT("SnapTo", "Snap To"));

	const FAvaLevelViewportCommands& CommandsRef = FAvaLevelViewportCommands::GetInternal();

	SnapToSection.AddEntry(FToolMenuEntry::InitMenuEntryWithCommandList(
		CommandsRef.ToggleGridSnapping,
		FrameAndWidget.ViewportWidget->GetCommandList(),
		LOCTEXT("GridSnapping", "Grid")
	));

	SnapToSection.AddEntry(FToolMenuEntry::InitMenuEntryWithCommandList(
		CommandsRef.ToggleScreenSnapping,
		FrameAndWidget.ViewportWidget->GetCommandList(),
		LOCTEXT("ScreenSnapping", "Screen & Guide")
	));

	SnapToSection.AddEntry(FToolMenuEntry::InitMenuEntryWithCommandList(
		CommandsRef.ToggleActorSnapping,
		FrameAndWidget.ViewportWidget->GetCommandList(),
		LOCTEXT("ActorSnapping", "Actor")
	));

	return Menus->GenerateWidget(ContextMenu);
}

FSlateColor SAvaLevelViewportStatusBarButtons::GetToggleShapeEditorOverlayColor() const
{
	using namespace UE::AvaLevelViewport::Private;

	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		if (AvaViewportSettings->bEnableViewportOverlay)
		{
			return AvaViewportSettings->bEnableShapesEditorOverlay ? ViewportStatusBarButton::ActiveColor : ViewportStatusBarButton::EnabledColor;
		}
	}

	return ViewportStatusBarButton::DisabledColor;
}

bool SAvaLevelViewportStatusBarButtons::GetToggleShapeEditorOverlayEnabled() const
{
	if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		return AvaViewportSettings->bEnableViewportOverlay;
	}

	return false;
}

FReply SAvaLevelViewportStatusBarButtons::ToggleShapeEditorOverlay()
{
	using namespace UE::AvaLevelViewport::Private;

	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->bEnableShapesEditorOverlay = !AvaViewportSettings->bEnableShapesEditorOverlay;
		AvaViewportSettings->SaveConfig();
		AvaViewportSettings->BroadcastSettingChanged(GET_MEMBER_NAME_CHECKED(UAvaViewportSettings, bEnableShapesEditorOverlay));

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FSlateColor SAvaLevelViewportStatusBarButtons::GetToggleGuidesColor() const
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid() && FrameAndWidget.ViewportWidget->CanToggleGuides())
	{
		if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
		{
			return AvaViewportSettings->bGuidesEnabled ? ViewportStatusBarButton::ActiveColor : ViewportStatusBarButton::EnabledColor;
		}
	}

	return ViewportStatusBarButton::DisabledColor;
}

bool SAvaLevelViewportStatusBarButtons::GetToggleGuidesEnabled() const
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid())
	{
		return FrameAndWidget.ViewportWidget->CanToggleGuides();
	}

	return false;
}

FReply SAvaLevelViewportStatusBarButtons::ToggleGuides()
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid() && FrameAndWidget.ViewportWidget->CanToggleGuides())
	{
		FrameAndWidget.ViewportWidget->ExecuteToggleGuides();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FSlateColor SAvaLevelViewportStatusBarButtons::GetToggleGridColor() const
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid() && FrameAndWidget.ViewportWidget->CanToggleGrid())
	{
		if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
		{
			return AvaViewportSettings->bGridEnabled ? ViewportStatusBarButton::ActiveColor : ViewportStatusBarButton::EnabledColor;
		}
	}

	return ViewportStatusBarButton::DisabledColor;
}

bool SAvaLevelViewportStatusBarButtons::GetToggleGridEnabled() const
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid())
	{
		return FrameAndWidget.ViewportWidget->CanToggleGrid();
	}

	return false;
}

FReply SAvaLevelViewportStatusBarButtons::ToggleGrid()
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid() && FrameAndWidget.ViewportWidget->CanToggleGrid())
	{
		FrameAndWidget.ViewportWidget->ExecuteToggleGrid();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

TSharedRef<SWidget> SAvaLevelViewportStatusBarButtons::GetGridMenuContent() const
{
	UToolMenus* Menus = UToolMenus::Get();

	check(Menus);

	static const FName GridMenuName = TEXT("AvaLevelViewport.StatusBar.Grid");

	UToolMenu* ContextMenu = Menus->FindMenu(GridMenuName);

	if (!ContextMenu)
	{
		ContextMenu = Menus->RegisterMenu(GridMenuName, NAME_None, EMultiBoxType::Menu);

		if (!ContextMenu)
		{
			return SNullWidget::NullWidget;
		}
	}

	FToolMenuSection& GridSection = ContextMenu->FindOrAddSection("Grid", LOCTEXT("Grid", "Grid"));

	const FAvaLevelViewportCommands& CommandsRef = FAvaLevelViewportCommands::GetInternal();

	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid())
	{
		GridSection.AddEntry(FToolMenuEntry::InitMenuEntryWithCommandList(
			CommandsRef.ToggleGridAlwaysVisible,
			FrameAndWidget.ViewportWidget->GetCommandList(),
			LOCTEXT("AlwaysShowGrid", "Always On")
		));
	}

	if (GridSizeSlider.IsValid())
	{
		GridSection.AddEntry(FToolMenuEntry::InitWidget(
			"GridSize",
			GridSizeSlider.ToSharedRef(),
			LOCTEXT("GridSize", "Size"),
			true
		));
	}

	return Menus->GenerateWidget(ContextMenu);
}

void SAvaLevelViewportStatusBarButtons::OnGridSizeChanged(int32 InNewValue) const
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid() && FrameAndWidget.ViewportWidget->CanChangeGridSize())
	{
		FrameAndWidget.ViewportWidget->ExecuteSetGridSize(InNewValue, false);
	}
}

void SAvaLevelViewportStatusBarButtons::OnGridSizeCommitted(int32 InNewValue, ETextCommit::Type InCommitType) const
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid() && FrameAndWidget.ViewportWidget->CanChangeGridSize())
	{
		FrameAndWidget.ViewportWidget->ExecuteSetGridSize(InNewValue, true);
	}
}

bool SAvaLevelViewportStatusBarButtons::GetViewportInfoEnabled() const
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (FrameAndClient.IsValid())
	{
		if (FEditorModeTools* ModeTools = FrameAndClient.ViewportClient->GetModeTools())
		{
			return ModeTools->GetToolkitHost().IsValid();
		}
	}

	return false;
}

FSlateColor SAvaLevelViewportStatusBarButtons::GetViewportInfoColor() const
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (FrameAndClient.IsValid())
	{
		if (FEditorModeTools* ModeTools = FrameAndClient.ViewportClient->GetModeTools())
		{
			if (ModeTools->GetToolkitHost().IsValid())
			{
				return ViewportStatusBarButton::EnabledColor;
			}
		}
	}

	return ViewportStatusBarButton::DisabledColor;
}

TSharedRef<SWidget> SAvaLevelViewportStatusBarButtons::GetViewportInfoWidget() const
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (FrameAndClient.IsValid())
	{
		if (FEditorModeTools* ModeTools = FrameAndClient.ViewportClient->GetModeTools())
		{
			if (TSharedPtr<IToolkitHost> ToolkitHost = ModeTools->GetToolkitHost())
			{
				return SAvaViewportInfo::CreateInstance(ToolkitHost.ToSharedRef());
			}
		}
	}

	return SNullWidget::NullWidget;
}

FSlateColor SAvaLevelViewportStatusBarButtons::GetTextureOverlayColor() const
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid() && FrameAndWidget.ViewportWidget->CanToggleTextureOverlay())
	{
		if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
		{
			return AvaViewportSettings->bEnableTextureOverlay ? ViewportStatusBarButton::ActiveColor : ViewportStatusBarButton::EnabledColor;
		}
	}

	return ViewportStatusBarButton::DisabledColor;
}

bool SAvaLevelViewportStatusBarButtons::GetTextureOverlayEnabled() const
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid())
	{
		return FrameAndWidget.ViewportWidget->CanToggleTextureOverlay();
	}

	return false;
}

TSharedRef<SWidget> SAvaLevelViewportStatusBarButtons::GetTextureOverlayMenuContent()
{
	UToolMenus* Menus = UToolMenus::Get();

	check(Menus);

	static const FName TextureOverlayMenuName = TEXT("AvaLevelViewport.StatusBar.TextureOverlay");

	UToolMenu* ContextMenu = Menus->FindMenu(TextureOverlayMenuName);

	if (!ContextMenu)
	{
		using namespace UE::Ava::LevelViewportStatusBarButtons::Private;

		ContextMenu = Menus->RegisterMenu(TextureOverlayMenuName, NAME_None, EMultiBoxType::Menu);

		if (!ContextMenu)
		{
			return SNullWidget::NullWidget;
		}
	}

	FToolMenuSection& OptionsSection = ContextMenu->FindOrAddSection("TextureOverlay", LOCTEXT("TextureOverlay", "Texture Overlay"));

	if (TextureOverlayOpacitySlider.IsValid())
	{
		OptionsSection.AddEntry(FToolMenuEntry::InitWidget(
			"TextureOverlayOpacity",
			TextureOverlayOpacitySlider.ToSharedRef(),
			LOCTEXT("TextureOverlayOpacity", "Opacity"),
			true
		));
	}

	if (TextureOverlayTextureSelector.IsValid())
	{
		OptionsSection.AddEntry(FToolMenuEntry::InitWidget(
			"TextureOverlayTexture",
			TextureOverlayTextureSelector.ToSharedRef(),
			LOCTEXT("TextureOverlayTexture", "Texture"),
			true
		));
	}

	if (TextureOverlayStretchCheckBox.IsValid())
	{
		OptionsSection.AddEntry(FToolMenuEntry::InitWidget(
			"TextureOverlayStretch",
			TextureOverlayStretchCheckBox.ToSharedRef(),
			LOCTEXT("TextureOverlayStretch", "Stretch Texture"),
			true
		));
	}

	return Menus->GenerateWidget(ContextMenu);
}

FReply SAvaLevelViewportStatusBarButtons::ToggleTextureOverlay()
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid() && FrameAndWidget.ViewportWidget->CanToggleSnapping())
	{
		FrameAndWidget.ViewportWidget->ExecuteToggleTextureOverlay();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

bool SAvaLevelViewportStatusBarButtons::CanChangeGridSize() const
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid())
	{
		return FrameAndWidget.ViewportWidget->CanChangeGridSize();
	}

	return false;
}

FSlateColor SAvaLevelViewportStatusBarButtons::GetToggleOverlayColor() const
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid() && FrameAndWidget.ViewportWidget->CanToggleOverlay())
	{
		if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
		{
			return AvaViewportSettings->bEnableViewportOverlay ? ViewportStatusBarButton::ActiveColor : ViewportStatusBarButton::EnabledColor;
		}
	}

	return ViewportStatusBarButton::DisabledColor;
}

bool SAvaLevelViewportStatusBarButtons::GetToggleOverlayEnabled() const
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid())
	{
		return FrameAndWidget.ViewportWidget->CanToggleOverlay();
	}

	return false;
}

FReply SAvaLevelViewportStatusBarButtons::ToggleOverlay()
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid() && FrameAndWidget.ViewportWidget->CanToggleOverlay())
	{
		FrameAndWidget.ViewportWidget->ExecuteToggleOverlay();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FSlateColor SAvaLevelViewportStatusBarButtons::GetToggleBoundingBoxesColor() const
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid() && FrameAndWidget.ViewportWidget->CanToggleBoundingBox())
	{
		if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
		{
			return AvaViewportSettings->bEnableBoundingBoxes ? ViewportStatusBarButton::ActiveColor : ViewportStatusBarButton::EnabledColor;
		}
	}

	return ViewportStatusBarButton::DisabledColor;
}

bool SAvaLevelViewportStatusBarButtons::GetToggleBoundingBoxesEnabled() const
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid())
	{
		return FrameAndWidget.ViewportWidget->CanToggleBoundingBox();
	}

	return false;
}

FReply SAvaLevelViewportStatusBarButtons::ToggleBoundingBoxes()
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid())
	{
		FrameAndWidget.ViewportWidget->ExecuteToggleBoundingBox();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FSlateColor SAvaLevelViewportStatusBarButtons::GetToggleIsolateActorsColor() const
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (FrameAndClient.IsValid() && FrameAndClient.ViewportClient->GetIsolateActorsOperation()->CanToggleIsolateActors())
	{
		return FrameAndClient.ViewportClient->GetIsolateActorsOperation()->IsIsolatingActors() ? ViewportStatusBarButton::ActiveColor : ViewportStatusBarButton::EnabledColor;
	}

	return ViewportStatusBarButton::DisabledColor;
}

bool SAvaLevelViewportStatusBarButtons::GetToggleIsolateActorsEnabled() const
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (FrameAndClient.IsValid())
	{
		return FrameAndClient.ViewportClient->GetIsolateActorsOperation()->CanToggleIsolateActors();
	}

	return false;
}

FReply SAvaLevelViewportStatusBarButtons::ToggleIsolateActors()
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (FrameAndClient.IsValid())
	{
		FrameAndClient.ViewportClient->GetIsolateActorsOperation()->ToggleIsolateActors();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FSlateColor SAvaLevelViewportStatusBarButtons::GetToggleSafeFramesColor() const
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid() && FrameAndWidget.ViewportWidget->CanToggleSafeFrames())
	{
		if (!FrameAndWidget.ViewportWidget->CanToggleSafeFrames())
		{
			return ViewportStatusBarButton::DisabledColor;
		}

		if (const UAvaViewportSettings* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
		{
			return AvaViewportSettings->bSafeFramesEnabled ? ViewportStatusBarButton::ActiveColor : ViewportStatusBarButton::EnabledColor;
		}
	}

	return ViewportStatusBarButton::DisabledColor;
}

bool SAvaLevelViewportStatusBarButtons::GetToggleSafeFramesEnabled() const
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid())
	{
		return FrameAndWidget.ViewportWidget->CanToggleSafeFrames();
	}

	return false;
}

FReply SAvaLevelViewportStatusBarButtons::ToggleSafeFrames()
{
	const FAvaLevelViewportGuideFrameAndWidget FrameAndWidget(ViewportFrameWeak);

	if (FrameAndWidget.IsValid() && FrameAndWidget.ViewportWidget->CanToggleSafeFrames())
	{
		FrameAndWidget.ViewportWidget->ExecuteToggleSafeFrames();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FSlateColor SAvaLevelViewportStatusBarButtons::GetToggleGameViewColor() const
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (FrameAndClient.IsValid())
	{
		return FrameAndClient.ViewportClient->IsInGameView() ? ViewportStatusBarButton::ActiveColor : ViewportStatusBarButton::EnabledColor;
	}

	return ViewportStatusBarButton::DisabledColor;
}

bool SAvaLevelViewportStatusBarButtons::GetToggleGameViewEnabled() const
{
	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	return FrameAndClient.IsValid();
}

FReply SAvaLevelViewportStatusBarButtons::ToggleGameView()
{
	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (FrameAndClient.IsValid())
	{
		const bool bNewGameModeValue = !FrameAndClient.ViewportClient->IsInGameView();

		FrameAndClient.ViewportClient->SetGameView(bNewGameModeValue);

		if (!bNewGameModeValue)
		{
			FrameAndClient.ViewportClient->ShowWidget(true);
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
