// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class SAvaLevelViewportFrame;
class SHorizontalBox;
enum class EAvaViewportPostProcessType : uint8;
enum class ECheckBoxState : uint8;
struct FToolMenuContext;

class SAvaLevelViewportStatusBarButtons : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaLevelViewportStatusBarButtons) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SAvaLevelViewportFrame> InViewportFrame);

	TSharedPtr<SAvaLevelViewportFrame> GetViewportFrameWeak() const { return ViewportFrameWeak.Pin(); }

protected:
	TWeakPtr<SAvaLevelViewportFrame> ViewportFrameWeak;

	TSharedPtr<SWidget> BackgroundTextureSelector;

	TSharedPtr<SWidget> PostProcessOpacitySlider;

	TSharedPtr<SWidget> GridSizeSlider;

	TSharedPtr<SWidget> TextureOverlayTextureSelector;

	TSharedPtr<SWidget> TextureOverlayOpacitySlider;

	TSharedPtr<SWidget> TextureOverlayStretchCheckBox;

	void CreateContextMenuWigets();

	void PopulateActorButtons(TSharedPtr<SHorizontalBox> InContainer);
	void PopulateViewportButtons(TSharedPtr<SHorizontalBox> InContainer);

	// Actor
	FSlateColor GetActorAlignmentColor() const;
	bool GetActorAlignmentEnabled() const;
	TSharedRef<SWidget> GetActorAlignmentMenuContent() const;
	TSharedRef<SWidget> GetActorColorMenuContent() const;

	FSlateColor GetAnimatorButtonMuteColor() const;
	FSlateColor GetAnimatorButtonUnmuteColor() const;
	bool GetAnimatorButtonEnabled() const;
	FReply EnableAnimators();
	FReply DisableAnimators();

	// Post Process
	bool GetPostProcessEnabled() const;
	TSharedRef<SWidget> GetPostProcessMenuContent();

	bool GetPostProcessEnabledMenu(const FToolMenuContext& InContext) const;
	ECheckBoxState GetPostProcessActiveMenu(const FToolMenuContext& InContext, EAvaViewportPostProcessType InPostProcessType) const;
	void TogglePostProcessMenu(const FToolMenuContext& InContext, EAvaViewportPostProcessType InPostProcessType);

	// Viewport
	FSlateColor GetToggleViewportToolbarColor() const;
	bool GetToggleViewportToolbarEnabled() const;
	FReply ToggleViewportToolbar();
	
	FSlateColor GetHighResScreenshotColor() const;
	bool GetHighResScreenshotEnabled() const;
	FReply HighResScreenshot();

	FSlateColor GetToggleGuidesColor() const;
	bool GetToggleGuidesEnabled() const;
	FReply ToggleGuides();

	FSlateColor GetToggleOverlayColor() const;
	bool GetToggleOverlayEnabled() const;
	FReply ToggleOverlay();

	FSlateColor GetToggleBoundingBoxesColor() const;
	bool GetToggleBoundingBoxesEnabled() const;
	FReply ToggleBoundingBoxes();

	FSlateColor GetToggleIsolateActorsColor() const;
	bool GetToggleIsolateActorsEnabled() const;
	FReply ToggleIsolateActors();

	FSlateColor GetToggleSafeFramesColor() const;
	bool GetToggleSafeFramesEnabled() const;
	FReply ToggleSafeFrames();

	FSlateColor GetToggleGameViewColor() const;
	bool GetToggleGameViewEnabled() const;
	FReply ToggleGameView();

	FSlateColor GetToggleSnapColor() const;
	bool GetToggleSnapEnabled() const;
	FReply ToggleSnap();

	TSharedRef<SWidget> GetSnappingMenuContent() const;

	FSlateColor GetToggleShapeEditorOverlayColor() const;
	bool GetToggleShapeEditorOverlayEnabled() const;
	FReply ToggleShapeEditorOverlay();

	FSlateColor GetToggleGridColor() const;
	bool GetToggleGridEnabled() const;
	FReply ToggleGrid();

	TSharedRef<SWidget> GetGridMenuContent() const;

	bool CanChangeGridSize() const;
	void OnGridSizeChanged(int32 InNewValue) const;
	void OnGridSizeCommitted(int32 InNewValue, ETextCommit::Type InCommitType) const;

	bool GetViewportInfoEnabled() const;
	FSlateColor GetViewportInfoColor() const;
	TSharedRef<SWidget> GetViewportInfoWidget() const;

	FSlateColor GetTextureOverlayColor() const;
	bool GetTextureOverlayEnabled() const;
	TSharedRef<SWidget> GetTextureOverlayMenuContent();
	FReply ToggleTextureOverlay();
};
