// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAvaLevelViewportStatusBarTransformSettings.h"
#include "AvaLevelViewportCommands.h"
#include "CustomDetailsViewSequencer.h"
#include "Customizations/AvaLevelViewportComponentTransformDetails.h"
#include "IDetailKeyframeHandler.h"
#include "LevelEditor/AvaLevelEditorUtils.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SAvaLevelViewportFrame.h"
#include "ViewportClient/AvaLevelViewportClient.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SAvaLevelViewportStatusBar.h"

#define LOCTEXT_NAMESPACE "SAvaLevelViewportStatusBarTransformSettings"

namespace UE::AvaLevelViewport::Private
{
	TSharedPtr<IDetailKeyframeHandler> GetKeyframeHandler()
	{
		static FName PropertyEditorModuleName("PropertyEditor");
		FPropertyEditorModule& PropertyEditor = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditorModuleName);

		TSharedPtr<IDetailsView> DetailView = nullptr;

		for (const FName& DetailViewId : FAvaLevelEditorUtils::GetDetailsViewNames())
		{
			if (const TSharedPtr<IDetailsView> DetailViewIter = PropertyEditor.FindDetailView(DetailViewId))
			{
				DetailView = DetailViewIter;
				break;
			}
		}

		if (!DetailView.IsValid())
		{
			return nullptr;
		}

		return DetailView->GetKeyframeHandler();
	}
}

void SAvaLevelViewportStatusBarTransformSettings::Construct(const FArguments& InArgs, TSharedPtr<SAvaLevelViewportFrame> InViewportFrame)
{
	ViewportFrameWeak = InViewportFrame;
	TransformDetails = MakeShared<FAvaLevelViewportComponentTransformDetails>(InViewportFrame.IsValid() ? InViewportFrame->GetViewportClient() : nullptr);
	SetCanTick(true);

	using namespace UE::AvaLevelViewport::Private;

	auto CreateKeyframeButton = [this](int32 InIndex)
		{
			return SNew(SBox)
				.WidthOverride(ViewportStatusBarButton::ImageSize.X + 6.f)
				[
					SNew(SButton)
					.ButtonStyle(ViewportStatusBarButton::Style)
					.ContentPadding(ViewportStatusBarButton::Margin)
					.OnClicked(this, &SAvaLevelViewportStatusBarTransformSettings::OnKeyframeClicked, InIndex)
					.IsEnabled(this, &SAvaLevelViewportStatusBarTransformSettings::IsKeyframeEnabled, InIndex)
					.ToolTipText(this, &SAvaLevelViewportStatusBarTransformSettings::GetKeyframeToolTip, InIndex)
					[
						SNew(SImage)
							.Image(this, &SAvaLevelViewportStatusBarTransformSettings::GetKeyframeIcon, InIndex)
							.DesiredSizeOverride(ViewportStatusBarButton::ImageSize)
					]
				];
		};

	ChildSlot
	[
		SNew(SBox)
		.WidthOverride(280.f)
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		[
			SAssignNew(Switcher, SWidgetSwitcher)		

			+ SWidgetSwitcher::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					CreateLocationWidgets()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					CreateKeyframeButton(0)
				]
			]

			+ SWidgetSwitcher::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					CreateRotationWidgets()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					CreateKeyframeButton(1)
				]
			]

			+ SWidgetSwitcher::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.FillWidth(1.f)
				[
					CreateScaleWidgets()
				]
				+ SHorizontalBox::Slot()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.AutoWidth()
				[
					CreateKeyframeButton(2)
				]
			]
		]
	];

	Switcher->SetActiveWidgetIndex(0);
}

TSharedRef<SHorizontalBox> SAvaLevelViewportStatusBarTransformSettings::CreateLocationWidgets()
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportCommands& CommandsRef = FAvaLevelViewportCommands::GetInternal();

	return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			ViewportStatusBarButton::MakeButton(
				this,
				CommandsRef.ResetLocation,
				FAppStyle::GetBrush("EditorViewport.TranslateMode"),
				&SAvaLevelViewportStatusBarTransformSettings::OnResetLocationClicked,
				&SAvaLevelViewportStatusBarTransformSettings::IsResetLocationEnabled,
				&SAvaLevelViewportStatusBarTransformSettings::GetResetLocationColor
			)
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.VAlign(VAlign_Center)
		[
			TransformDetails->GetTransformBody().ToSharedRef()
		]

		// "Spacer" for the scale button
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.Padding(ViewportStatusBarButton::Padding)
			.WidthOverride(18.f)
			.HeightOverride(16.f)
		];
}

TSharedRef<SHorizontalBox> SAvaLevelViewportStatusBarTransformSettings::CreateRotationWidgets()
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportCommands& CommandsRef = FAvaLevelViewportCommands::GetInternal();

	return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			ViewportStatusBarButton::MakeButton(
				this,
				CommandsRef.ResetLocation,
				FAppStyle::GetBrush("EditorViewport.RotateMode"),
				&SAvaLevelViewportStatusBarTransformSettings::OnResetRotationClicked,
				&SAvaLevelViewportStatusBarTransformSettings::IsResetRotationEnabled,
				&SAvaLevelViewportStatusBarTransformSettings::GetResetRotationColor
			)
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			TransformDetails->GetRotationBody().ToSharedRef()
		]

		// "Spacer" for the scale button
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.Padding(ViewportStatusBarButton::Padding)
			.WidthOverride(18.f)
			.HeightOverride(16.f)
		];
}

TSharedRef<SHorizontalBox> SAvaLevelViewportStatusBarTransformSettings::CreateScaleWidgets()
{
	using namespace UE::AvaLevelViewport::Private;

	const FAvaLevelViewportCommands& CommandsRef = FAvaLevelViewportCommands::GetInternal();

	return SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(ViewportStatusBarButton::Padding)
		[
			ViewportStatusBarButton::MakeButton(
				this,
				CommandsRef.ResetScale,
				FAppStyle::GetBrush("EditorViewport.ScaleMode"),
				&SAvaLevelViewportStatusBarTransformSettings::OnResetScaleClicked,
				&SAvaLevelViewportStatusBarTransformSettings::IsResetScaleEnabled,
				&SAvaLevelViewportStatusBarTransformSettings::GetResetScaleColor
			)
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			TransformDetails->GetScaleBody().ToSharedRef()
		]

		// Taken from 
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.Padding(ViewportStatusBarButton::Padding)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.WidthOverride(18.f)
			.HeightOverride(16.f)
			[
				TransformDetails->GetPreserveScaleRatioWidget()
			]
		];
}

void SAvaLevelViewportStatusBarTransformSettings::CreateKeyframeButtons()
{
	if (KeyframeButtons.Num() == 3)
	{
		return;
	}

	using namespace UE::AvaLevelViewport::Private;

	TConstArrayView<TSharedPtr<IPropertyHandle>> PropertyHandles = TransformDetails->GetPropertyHandles();

	if (PropertyHandles.Num() != 3)
	{
		return;
	}

	KeyframeButtons.Reserve(3);

	static const FCustomDetailsViewSequencerUtils::FGetKeyframeHandlerDelegate KeyframeHandlerDelegate =
		FCustomDetailsViewSequencerUtils::FGetKeyframeHandlerDelegate::CreateStatic(&GetKeyframeHandler);

	FCustomDetailsViewSequencerUtils::CreateSequencerExtensionButton(KeyframeHandlerDelegate, PropertyHandles[0], KeyframeButtons);
	FCustomDetailsViewSequencerUtils::CreateSequencerExtensionButton(KeyframeHandlerDelegate, PropertyHandles[1], KeyframeButtons);
	FCustomDetailsViewSequencerUtils::CreateSequencerExtensionButton(KeyframeHandlerDelegate, PropertyHandles[2], KeyframeButtons);
}

void SAvaLevelViewportStatusBarTransformSettings::UpdateKeyframeButtons()
{
	if (!TransformDetails.IsValid())
	{
		KeyframeButtons.Reset();
		return;
	}

	TConstArrayView<TWeakObjectPtr<UObject>> SelectedObjects = TransformDetails->GetSelectedObjects();

	if (SelectedObjects.Num() != 1)
	{
		KeyframeButtons.Reset();
		return;
	}

	UObject* SelectedObject = SelectedObjects[0].Get();

	if (!IsValid(SelectedObject) || SelectedObject == USceneComponent::StaticClass()->GetDefaultObject())
	{
		KeyframeButtons.Reset();
		return;
	}

	if (KeyframeButtons.Num() == 3)
	{
		return;
	}

	CreateKeyframeButtons();
}

bool SAvaLevelViewportStatusBarTransformSettings::IsResetLocationEnabled() const
{
	return TransformDetails.IsValid() && TransformDetails->GetSelectedActorCount() > 0;
}

FSlateColor SAvaLevelViewportStatusBarTransformSettings::GetResetLocationColor() const
{
	using namespace UE::AvaLevelViewport::Private;
	return IsResetLocationEnabled() ? ViewportStatusBarButton::EnabledColor : ViewportStatusBarButton::DisabledColor;
}

FReply SAvaLevelViewportStatusBarTransformSettings::OnResetLocationClicked()
{
	if (TransformDetails.IsValid())
	{
		TransformDetails->OnLocationResetClicked();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

bool SAvaLevelViewportStatusBarTransformSettings::IsResetRotationEnabled() const
{
	return TransformDetails.IsValid() && TransformDetails->GetSelectedActorCount() > 0;
}

FSlateColor SAvaLevelViewportStatusBarTransformSettings::GetResetRotationColor() const
{
	using namespace UE::AvaLevelViewport::Private;
	return IsResetRotationEnabled() ? ViewportStatusBarButton::EnabledColor : ViewportStatusBarButton::DisabledColor;
}

FReply SAvaLevelViewportStatusBarTransformSettings::OnResetRotationClicked()
{
	if (TransformDetails.IsValid())
	{
		TransformDetails->OnRotationResetClicked();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

bool SAvaLevelViewportStatusBarTransformSettings::IsResetScaleEnabled() const
{
	return TransformDetails.IsValid() && TransformDetails->GetSelectedActorCount() > 0;
}

FSlateColor SAvaLevelViewportStatusBarTransformSettings::GetResetScaleColor() const
{
	using namespace UE::AvaLevelViewport::Private;
	return IsResetScaleEnabled() ? ViewportStatusBarButton::EnabledColor : ViewportStatusBarButton::DisabledColor;
}

FReply SAvaLevelViewportStatusBarTransformSettings::OnResetScaleClicked()
{
	if (TransformDetails.IsValid())
	{
		TransformDetails->OnScaleResetClicked();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

bool SAvaLevelViewportStatusBarTransformSettings::IsKeyframeEnabled(int32 InButtonIndex) const
{
	if (!KeyframeButtons.IsValidIndex(InButtonIndex))
	{
		return false;
	}

	return KeyframeButtons[InButtonIndex].UIAction.CanExecute();
}

const FSlateBrush* SAvaLevelViewportStatusBarTransformSettings::GetKeyframeIcon(int32 InButtonIndex) const
{
	if (!KeyframeButtons.IsValidIndex(InButtonIndex))
	{
		return nullptr;
	}

	if (!KeyframeButtons[InButtonIndex].Icon.IsSet())
	{
		return nullptr;
	}

	return KeyframeButtons[InButtonIndex].Icon.Get().GetIcon();
}

FText SAvaLevelViewportStatusBarTransformSettings::GetKeyframeToolTip(int32 InButtonIndex) const
{
	if (!KeyframeButtons.IsValidIndex(InButtonIndex))
	{
		return FText::GetEmpty();
	}

	if (!KeyframeButtons[InButtonIndex].Icon.IsSet())
	{
		return FText::GetEmpty();
	}

	return KeyframeButtons[InButtonIndex].ToolTip.Get();
}

FReply SAvaLevelViewportStatusBarTransformSettings::OnKeyframeClicked(int32 InButtonIndex)
{
	if (!KeyframeButtons.IsValidIndex(InButtonIndex))
	{
		return FReply::Handled();
	}

	if (!KeyframeButtons[InButtonIndex].UIAction.CanExecute())
	{
		return FReply::Handled();
	}

	KeyframeButtons[InButtonIndex].UIAction.Execute();

	return FReply::Handled();
}

void SAvaLevelViewportStatusBarTransformSettings::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	const FAvaLevelViewportGuideFrameAndClient FrameAndClient(ViewportFrameWeak);

	if (FrameAndClient.IsValid())
	{
		UE::Widget::EWidgetMode Mode = FrameAndClient.ViewportClient->GetWidgetMode();

		if (Mode == UE::Widget::WM_None)
		{
			Switcher->SetActiveWidgetIndex(0);
		}
		else if (EnumHasAnyFlags(Mode, UE::Widget::WM_Rotate))
		{
			Switcher->SetActiveWidgetIndex(1);
		}
		else if (EnumHasAnyFlags(Mode, UE::Widget::WM_Scale))
		{
			Switcher->SetActiveWidgetIndex(2);
		}
		else
		{
			Switcher->SetActiveWidgetIndex(0);
		}
	}

	if (TransformDetails.IsValid())
	{
		TransformDetails->CacheTransform();
	}

	UpdateKeyframeButtons();
}

#undef LOCTEXT_NAMESPACE
