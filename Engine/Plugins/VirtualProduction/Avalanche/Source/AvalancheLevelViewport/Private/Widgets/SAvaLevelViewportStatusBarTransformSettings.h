// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Templates/SharedPointer.h"
#include "PropertyEditorDelegates.h"

class FAvaLevelViewportComponentTransformDetails;
class SAvaLevelViewportFrame;
class SHorizontalBox;
class SWidgetSwitcher;

class SAvaLevelViewportStatusBarTransformSettings : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaLevelViewportStatusBarTransformSettings) {}
	SLATE_END_ARGS()
	void Construct(const FArguments& InArgs, TSharedPtr<SAvaLevelViewportFrame> InViewportFrame);

	TSharedPtr<SAvaLevelViewportFrame> GetViewportFrameWeak() const { return ViewportFrameWeak.Pin(); }

	//~ Begin SWidget
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

protected:
	TWeakPtr<SAvaLevelViewportFrame> ViewportFrameWeak;
	TSharedPtr<FAvaLevelViewportComponentTransformDetails> TransformDetails;
	TSharedPtr<SWidgetSwitcher> Switcher;
	TArray<FPropertyRowExtensionButton> KeyframeButtons;

	TSharedRef<SHorizontalBox> CreateLocationWidgets();
	TSharedRef<SHorizontalBox> CreateRotationWidgets();
	TSharedRef<SHorizontalBox> CreateScaleWidgets();

	void CreateKeyframeButtons();
	void UpdateKeyframeButtons();

	bool IsResetLocationEnabled() const;
	FSlateColor GetResetLocationColor() const;
	FReply OnResetLocationClicked();

	bool IsResetRotationEnabled() const;
	FSlateColor GetResetRotationColor() const;
	FReply OnResetRotationClicked();

	bool IsResetScaleEnabled() const;
	FSlateColor GetResetScaleColor() const;
	FReply OnResetScaleClicked();

	bool IsKeyframeEnabled(int32 InButtonIndex) const;
	const FSlateBrush* GetKeyframeIcon(int32 InButtonIndex) const;
	FText GetKeyframeToolTip(int32 InButtonIndex) const;
	FReply OnKeyframeClicked(int32 InButtonIndex);
};