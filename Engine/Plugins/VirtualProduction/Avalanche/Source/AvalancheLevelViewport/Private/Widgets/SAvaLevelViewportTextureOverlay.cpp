// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAvaLevelViewportTextureOverlay.h"
#include "AvaViewportSettings.h"
#include "AvaViewportUtils.h"
#include "AvaVisibleArea.h"
#include "Engine/Texture.h"
#include "TextureCompiler.h"
#include "ViewportClient/AvaLevelViewportClient.h"

SAvaLevelViewportTextureOverlay::SAvaLevelViewportTextureOverlay()
	: TextureBrush((UObject*)nullptr, FVector2f::ZeroVector)
{
}

void SAvaLevelViewportTextureOverlay::Construct(const FArguments& InArgs, TSharedRef<FAvaLevelViewportClient> InAvaLevelViewportClient)
{
	AvaLevelViewportClientWeak = InAvaLevelViewportClient;
	SetVisibility(EVisibility::HitTestInvisible);
	TextureStrong.Reset(nullptr);

	if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		AvaViewportSettings->OnChange.AddSP(this, &SAvaLevelViewportTextureOverlay::OnAvaViewportSettingsChanged);
		SetTexture(AvaViewportSettings->TextureOverlayTexture.LoadSynchronous());
		SetOpacity(AvaViewportSettings->TextureOverlayOpacity);
	}
}

SAvaLevelViewportTextureOverlay::~SAvaLevelViewportTextureOverlay()
{
	if (UObjectInitialized())
	{
		if (UAvaViewportSettings* AvaViewportSettings = GetMutableDefault<UAvaViewportSettings>())
		{
			AvaViewportSettings->OnChange.RemoveAll(this);
		}		
	}
}

int32 SAvaLevelViewportTextureOverlay::OnPaint(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry,
	const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle,
	bool bInParentEnabled) const
{
	InLayerId = SCompoundWidget::OnPaint(InPaintArgs, InAllottedGeometry, InMyCullingRect, OutDrawElements, 
		InLayerId, InWidgetStyle, bInParentEnabled);

	if (UAvaViewportSettings const* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		if (AvaViewportSettings->bEnableTextureOverlay)
		{
			UTexture* Texture = TextureStrong.Get();

			if (IsValid(Texture))
			{
				if (TSharedPtr<FAvaLevelViewportClient> AvaLevelViewportClient = AvaLevelViewportClientWeak.Pin())
				{
					DrawOverlay(InPaintArgs, InAllottedGeometry, InMyCullingRect, OutDrawElements, InLayerId,
						Texture, AvaViewportSettings->TextureOverlayOpacity, AvaViewportSettings->bTextureOverlayStretch);
				}
			}
		}
	}

	return InLayerId;
}

void SAvaLevelViewportTextureOverlay::DrawOverlay(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry,
	const FSlateRect& InMyCullingRect, FSlateWindowElementList& OutDrawElements, int32& InOutLayerId, 
	UTexture* InTexture, float InOpacity, bool bInStretch) const
{
	TSharedPtr<FAvaLevelViewportClient> AvaLevelViewportClient = AvaLevelViewportClientWeak.Pin();

	if (!AvaLevelViewportClient.IsValid())
	{
		return;
	}

	const FAvaVisibleArea& VisibleArea = AvaLevelViewportClient->GetZoomedVisibleArea();

	if (!VisibleArea.IsValid())
	{
		return;
	}

	++InOutLayerId;

	const FVector2f Offset = AvaLevelViewportClient->GetCachedViewportOffset();
	FVector2f DrawSize = VisibleArea.AbsoluteSize;

	if (!bInStretch)
	{
		const FVector2f TextureSize = TextureBrush.GetImageSize();

		if (!TextureSize.Equals(DrawSize))
		{
			const float VisibleAreaAspectRatio = DrawSize.X / DrawSize.Y;
			const float TextureAspectRatio = TextureSize.X / TextureSize.Y;

			if (VisibleAreaAspectRatio > TextureAspectRatio)
			{
				DrawSize.X *= TextureAspectRatio / VisibleAreaAspectRatio;
			}
			else if (VisibleAreaAspectRatio < TextureAspectRatio)
			{
				DrawSize.Y *= VisibleAreaAspectRatio / TextureAspectRatio;
			}
		}
	}

	const FVector2f TextureOffset = (VisibleArea.AbsoluteSize - DrawSize) * 0.5f;
	const FVector2f TopLeft = VisibleArea.GetVisiblePosition(TextureOffset);
	const FVector2f BottomRight = VisibleArea.GetVisiblePosition(VisibleArea.AbsoluteSize - TextureOffset);

	const FPaintGeometry TextureGeometry = InAllottedGeometry.ToPaintGeometry(
		BottomRight - TopLeft,
		FSlateLayoutTransform(TopLeft + Offset)
	);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		InOutLayerId,
		TextureGeometry,
		&TextureBrush,
		ESlateDrawEffect::NoGamma,
		TextureBrush.TintColor.GetSpecifiedColor()
	);
}

void SAvaLevelViewportTextureOverlay::OnAvaViewportSettingsChanged(const UAvaViewportSettings* InSettings, FName InSetting)
{
	static const FName OverlayTextureName = GET_MEMBER_NAME_CHECKED(UAvaViewportSettings, TextureOverlayTexture);
	static const FName OverlayOpacity = GET_MEMBER_NAME_CHECKED(UAvaViewportSettings, TextureOverlayOpacity);

	if (InSetting == OverlayTextureName)
	{
		SetTexture(InSettings->TextureOverlayTexture.LoadSynchronous());		
	}
	else if (InSetting == OverlayOpacity)
	{
		SetOpacity(InSettings->TextureOverlayOpacity);
	}
}

void SAvaLevelViewportTextureOverlay::SetTexture(UTexture* InTexture)
{
	TextureStrong.Reset(InTexture);
	TextureBrush.SetResourceObject(InTexture);

	if (InTexture)
	{
		FTextureCompilingManager::Get().FinishCompilation({InTexture});
		TextureBrush.SetImageSize(FVector2f((float)InTexture->GetSurfaceWidth(), (float)InTexture->GetSurfaceHeight()));
	}
}

void SAvaLevelViewportTextureOverlay::SetOpacity(float InOpacity)
{
	TextureBrush.TintColor = FSlateColor(FLinearColor(1.f, 1.f, 1.f, FMath::Clamp(InOpacity, 0.f, 1.f)));
}
