// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Brushes/SlateImageBrush.h"
#include "Templates/SharedPointer.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/SCompoundWidget.h"

class FAvaLevelViewportClient;
class UAvaViewportSettings;
class UTexture;

class SAvaLevelViewportTextureOverlay: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaLevelViewportTextureOverlay)
		{}
	SLATE_END_ARGS()

	SAvaLevelViewportTextureOverlay();
	virtual ~SAvaLevelViewportTextureOverlay() override;

	void Construct(const FArguments& InArgs, TSharedRef<FAvaLevelViewportClient> InAvaLevelViewportClient);

	//~ Begin SWidget
	virtual int32 OnPaint(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect,
		FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const override;
	//~ End SWidget

protected:
	TWeakPtr<FAvaLevelViewportClient> AvaLevelViewportClientWeak;
	TStrongObjectPtr<UTexture> TextureStrong;
	FSlateImageBrush TextureBrush;

	void DrawOverlay(const FPaintArgs& InPaintArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InMyCullingRect,
		FSlateWindowElementList& OutDrawElements, int32& InOutLayerId, UTexture* InTexture, float InOpacity, bool bInStretch) const;

	void OnAvaViewportSettingsChanged(const UAvaViewportSettings* InSettings, FName InSetting);

	void SetTexture(UTexture* InTexture);

	void SetOpacity(float InOpacity);
};
