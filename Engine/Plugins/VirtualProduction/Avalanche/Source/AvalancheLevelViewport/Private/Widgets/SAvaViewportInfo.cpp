// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaViewportInfo.h"

#include "AvaViewportUtils.h"
#include "AvaVisibleArea.h"
#include "EditorModeManager.h"
#include "Styling/StyleColors.h"
#include "Toolkits/IToolkitHost.h"
#include "ViewportClient/IAvaViewportClient.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaViewportInfo"

namespace UE::AvaLevelViewport::Private
{
	const FText& GetEmptyXYDisplayText()
	{
		static const FText EmptyXY = LOCTEXT("NoXY", "-");
		return EmptyXY;
	}

	FText MakeXYDisplayText_By(float InX, float InY)
	{
		return FText::Format(LOCTEXT("XYDash", "{0} x {1}"), FMath::RoundToInt(InX), FMath::RoundToInt(InY));
	}

	FText MakeXYDisplayText_Comma(float InX, float InY)
	{
		return FText::Format(LOCTEXT("XYComma", "{0}, {1}"), FMath::RoundToInt(InX), FMath::RoundToInt(InY));
	}
}

TSharedRef<SAvaViewportInfo> SAvaViewportInfo::CreateInstance(const TSharedRef<IToolkitHost>& InToolkitHost)
{
	return SNew(SAvaViewportInfo, InToolkitHost);
}

void SAvaViewportInfo::PrivateRegisterAttributes(struct FSlateAttributeDescriptor::FInitializer&)
{
}

void SAvaViewportInfo::Construct(const FArguments& Args, const TSharedRef<IToolkitHost>& InToolkitHost)
{
	ToolkitHostWeak = InToolkitHost;

	ChildSlot
	[
		SNew(SBox)
		.Padding(FMargin(3.f, 3.f, 3.f, 10.f))
		[
			SNew(SGridPanel)
			+SGridPanel::Slot(1, 0)
			.Padding(FMargin(3.f))
			.HAlign(EHorizontalAlignment::HAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Viewport","Viewport"))
			]
			+SGridPanel::Slot(2, 0)
			.Padding(FMargin(3.f))
			.HAlign(EHorizontalAlignment::HAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Virtual","Virtual"))
			]
			+SGridPanel::Slot(0, 1)
			.Padding(FMargin(3.f))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Size","Size:"))
			]
			+SGridPanel::Slot(1, 1)
			.Padding(FMargin(5.f, 3.f, 5.f, 3.f))
			.HAlign(EHorizontalAlignment::HAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SAvaViewportInfo::GetViewportSize)
				.ColorAndOpacity(FStyleColors::AccentBlue.GetSpecifiedColor())
			]
			+SGridPanel::Slot(2, 1)
			.Padding(FMargin(5.f, 3.f, 5.f, 3.f))
			.HAlign(EHorizontalAlignment::HAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SAvaViewportInfo::GetVirtualViewportSize)
				.ColorAndOpacity(FStyleColors::AccentGreen.GetSpecifiedColor())
			]
			+SGridPanel::Slot(0, 2)
			.Padding(FMargin(3.f))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("VisibleArea","Visible Area:"))
			]
			+SGridPanel::Slot(1, 2)
			.Padding(FMargin(5.f, 3.f, 5.f, 3.f))
			.HAlign(EHorizontalAlignment::HAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SAvaViewportInfo::GetViewportVisibleAreaSize)
				.ColorAndOpacity(FStyleColors::AccentBlue.GetSpecifiedColor())
			]
			+SGridPanel::Slot(2, 2)
			.Padding(FMargin(5.f, 3.f, 5.f, 3.f))
			.HAlign(EHorizontalAlignment::HAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SAvaViewportInfo::GetCanvasVisibleAreaSize)
				.ColorAndOpacity(FStyleColors::AccentGreen.GetSpecifiedColor())
			]
			+SGridPanel::Slot(0, 3)
			.Padding(FMargin(3.f))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CenterOffset","Center Offset:"))
			]
			+SGridPanel::Slot(1, 3)
			.Padding(FMargin(5.f, 3.f, 5.f, 3.f))
			.HAlign(EHorizontalAlignment::HAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SAvaViewportInfo::GetViewportZoomOffset)
				.ColorAndOpacity(FStyleColors::AccentBlue.GetSpecifiedColor())
			]
			+SGridPanel::Slot(2, 3)
			.Padding(FMargin(5.f, 3.f, 5.f, 3.f))
			.HAlign(EHorizontalAlignment::HAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SAvaViewportInfo::GetCanvasZoomOffset)
				.ColorAndOpacity(FStyleColors::AccentGreen.GetSpecifiedColor())
			]
			+SGridPanel::Slot(0, 4)
			.Padding(FMargin(3.f))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Cursor","Cursor:"))
			]
			+SGridPanel::Slot(1, 4)
			.Padding(FMargin(5.f, 3.f, 5.f, 3.f))
			.HAlign(EHorizontalAlignment::HAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SAvaViewportInfo::GetMouseLocation)
				.ColorAndOpacity(FStyleColors::AccentBlue.GetSpecifiedColor())
			]
			+SGridPanel::Slot(2, 4)
			.Padding(FMargin(5.f, 3.f, 5.f, 3.f))
			.HAlign(EHorizontalAlignment::HAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SAvaViewportInfo::GetVirtualMouseLocation)
				.ColorAndOpacity(FStyleColors::AccentGreen.GetSpecifiedColor())
			]
			+SGridPanel::Slot(0, 5)
			.Padding(FMargin(3.f))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Zoom","Zoom:"))
			]
			+SGridPanel::Slot(1, 5)
			.Padding(FMargin(3.f))
			.HAlign(EHorizontalAlignment::HAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SAvaViewportInfo::GetZoomLevel)
			]
		]
	];
}

FVector2f SAvaViewportInfo::GetViewportSizeForActiveViewport() const
{
	static const FVector2f Invalid = FVector2f::ZeroVector;

	TSharedPtr<IToolkitHost> ToolkitHost = ToolkitHostWeak.Pin();

	if (!ToolkitHost.IsValid())
	{
		return Invalid;
	}

	FEditorViewportClient* EditorViewportClient = ToolkitHost->GetEditorModeManager().GetFocusedViewportClient();

	if (TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient))
	{
		return AvaViewportClient->GetViewportSize();
	}

	return Invalid;
}

FIntPoint SAvaViewportInfo::GetVirtualSizeForActiveViewport() const
{
	static const FIntPoint Invalid = FIntPoint::ZeroValue;

	TSharedPtr<IToolkitHost> ToolkitHost = ToolkitHostWeak.Pin();

	if (!ToolkitHost.IsValid())
	{
		return Invalid;
	}

	FEditorViewportClient* EditorViewportClient = ToolkitHost->GetEditorModeManager().GetFocusedViewportClient();

	if (TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient))
	{
		return AvaViewportClient->GetVirtualViewportSize();
	}

	return Invalid;
}

FAvaVisibleArea SAvaViewportInfo::GetVisibleAreaForActiveViewport() const
{
	static const FAvaVisibleArea Invalid = FAvaVisibleArea();

	TSharedPtr<IToolkitHost> ToolkitHost = ToolkitHostWeak.Pin();

	if (!ToolkitHost.IsValid())
	{
		return Invalid;
	}

	FEditorViewportClient* EditorViewportClient = ToolkitHost->GetEditorModeManager().GetFocusedViewportClient();

	if (TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient))
	{
		return AvaViewportClient->GetZoomedVisibleArea();
	}

	return Invalid;
}

FVector2f SAvaViewportInfo::GetMouseLocationOnViewport() const
{
	static const FVector2f Invalid = FVector2f(-1, -1);

	TSharedPtr<IToolkitHost> ToolkitHost = ToolkitHostWeak.Pin();

	if (!ToolkitHost.IsValid())
	{
		return Invalid;
	}

	FEditorViewportClient* EditorViewportClient = ToolkitHost->GetEditorModeManager().GetFocusedViewportClient();

	if (TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient))
	{
		return AvaViewportClient->GetConstrainedViewportMousePosition();
	}

	return Invalid;
}

FText SAvaViewportInfo::GetViewportSize() const
{
	const FVector2f& ViewportSize = GetViewportSizeForActiveViewport();

	if (FAvaViewportUtils::IsValidViewportSize(ViewportSize))
	{
		return UE::AvaLevelViewport::Private::MakeXYDisplayText_By(FMath::RoundToInt(ViewportSize.X), FMath::RoundToInt(ViewportSize.Y));
	}

	return UE::AvaLevelViewport::Private::GetEmptyXYDisplayText();
}

FText SAvaViewportInfo::GetVirtualViewportSize() const
{
	const FIntPoint VirtualSize = GetVirtualSizeForActiveViewport();

	if (FAvaViewportUtils::IsValidViewportSize(VirtualSize))
	{
		return UE::AvaLevelViewport::Private::MakeXYDisplayText_By(VirtualSize.X, VirtualSize.Y);
	}

	return UE::AvaLevelViewport::Private::GetEmptyXYDisplayText();
}

FText SAvaViewportInfo::GetViewportVisibleAreaSize() const
{
	const FAvaVisibleArea VisibleArea = GetVisibleAreaForActiveViewport();

	if (VisibleArea.IsValid() && VisibleArea.IsZoomedView())
	{
		return UE::AvaLevelViewport::Private::MakeXYDisplayText_By(VisibleArea.VisibleSize.X, VisibleArea.VisibleSize.Y);
	}

	return UE::AvaLevelViewport::Private::GetEmptyXYDisplayText();
}

FText SAvaViewportInfo::GetCanvasVisibleAreaSize() const
{
	const FAvaVisibleArea VisibleArea = GetVisibleAreaForActiveViewport();

	if (VisibleArea.IsValid() && VisibleArea.IsZoomedView())
	{
		const FIntPoint CanvasSize = GetVirtualSizeForActiveViewport();

		if (FAvaViewportUtils::IsValidViewportSize(CanvasSize))
		{
			const FVector2f VisibleSize = VisibleArea.VisibleSize * static_cast<float>(CanvasSize.X) / VisibleArea.AbsoluteSize.X;

			return UE::AvaLevelViewport::Private::MakeXYDisplayText_By(VisibleSize.X, VisibleSize.Y);
		}
	}

	return UE::AvaLevelViewport::Private::GetEmptyXYDisplayText();
}

FText SAvaViewportInfo::GetViewportZoomOffset() const
{
	const FAvaVisibleArea VisibleArea = GetVisibleAreaForActiveViewport();

	if (VisibleArea.IsValid() && VisibleArea.IsOffset())
	{
		return UE::AvaLevelViewport::Private::MakeXYDisplayText_Comma(VisibleArea.Offset.X, VisibleArea.Offset.Y);
	}

	return UE::AvaLevelViewport::Private::GetEmptyXYDisplayText();
}

FText SAvaViewportInfo::GetCanvasZoomOffset() const
{
	const FAvaVisibleArea VisibleArea = GetVisibleAreaForActiveViewport();

	if (VisibleArea.IsValid() && VisibleArea.IsOffset())
	{
		const FIntPoint CanvasSize = GetVirtualSizeForActiveViewport();

		if (FAvaViewportUtils::IsValidViewportSize(CanvasSize))
		{
			const FVector2f CanvasOffset = VisibleArea.Offset * static_cast<float>(CanvasSize.X) / VisibleArea.AbsoluteSize.X;

			return UE::AvaLevelViewport::Private::MakeXYDisplayText_Comma(CanvasOffset.X, CanvasOffset.Y);
		}
	}

	return UE::AvaLevelViewport::Private::GetEmptyXYDisplayText();
}

FText SAvaViewportInfo::GetMouseLocation() const
{
	const FVector2f MouseLocation = GetMouseLocationOnViewport();

	// -1 is the invalid value
	if (MouseLocation.X < 0 || MouseLocation.Y < 0)
	{
		return UE::AvaLevelViewport::Private::GetEmptyXYDisplayText();
	}

	return UE::AvaLevelViewport::Private::MakeXYDisplayText_Comma(MouseLocation.X, MouseLocation.Y);
}

FText SAvaViewportInfo::GetVirtualMouseLocation() const
{
	static const FText& Default = UE::AvaLevelViewport::Private::GetEmptyXYDisplayText();

	FVector2f MouseLocation = GetMouseLocationOnViewport();

	// -1 is the invalid value
	if (MouseLocation.X < 0 || MouseLocation.Y < 0)
	{
		return Default;
	}

	const FVector2f ViewportSize = GetViewportSizeForActiveViewport();

	if (!FAvaViewportUtils::IsValidViewportSize(ViewportSize))
	{
		return Default;
	}

	const FIntPoint CanvasSize = GetVirtualSizeForActiveViewport();

	if (!FAvaViewportUtils::IsValidViewportSize(CanvasSize))
	{
		return Default;
	}

	MouseLocation *= static_cast<float>(CanvasSize.X) / ViewportSize.X;

	return UE::AvaLevelViewport::Private::MakeXYDisplayText_Comma(MouseLocation.X, MouseLocation.Y);
}

FText SAvaViewportInfo::GetZoomLevel() const
{
	const FAvaVisibleArea VisibleArea = GetVisibleAreaForActiveViewport();

	if (VisibleArea.IsValid())
	{
		const int32 Percentage = FMath::RoundToInt(100.f / VisibleArea.GetVisibleAreaFraction());

		return FText::Format(
			LOCTEXT("PercentFormat", "{0}%"),
			FText::AsNumber(Percentage)
		);
	}

	return UE::AvaLevelViewport::Private::GetEmptyXYDisplayText();
}

#undef LOCTEXT_NAMESPACE
