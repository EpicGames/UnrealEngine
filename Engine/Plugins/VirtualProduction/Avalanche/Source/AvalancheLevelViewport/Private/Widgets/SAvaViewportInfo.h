// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Math/MathFwd.h"
#include "Templates/SharedPointer.h"

class FText;
class IToolkitHost;
struct FAvaVisibleArea;

class SAvaViewportInfo : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SAvaViewportInfo, SCompoundWidget)
	
	SLATE_BEGIN_ARGS(SAvaViewportInfo) {}
	SLATE_END_ARGS()

public:
	static TSharedRef<SAvaViewportInfo> CreateInstance(const TSharedRef<IToolkitHost>& InToolkitHost);

	void Construct(const FArguments& Args, const TSharedRef<IToolkitHost>& InToolkitHost);

protected:
	TWeakPtr<IToolkitHost> ToolkitHostWeak;

	FVector2f GetViewportSizeForActiveViewport() const;
	FIntPoint GetVirtualSizeForActiveViewport() const;
	FAvaVisibleArea GetVisibleAreaForActiveViewport() const;
	FVector2f GetMouseLocationOnViewport() const;

	FText GetViewportSize() const;
	FText GetVirtualViewportSize() const;
	FText GetViewportVisibleAreaSize() const;
	FText GetCanvasVisibleAreaSize() const;
	FText GetViewportZoomOffset() const;
	FText GetCanvasZoomOffset() const;
	FText GetMouseLocation() const;
	FText GetVirtualMouseLocation() const;
	FText GetZoomLevel() const;
};
