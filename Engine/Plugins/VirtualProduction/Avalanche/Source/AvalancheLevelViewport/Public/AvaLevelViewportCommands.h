// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FAvaLevelViewportCommands : public TCommands<FAvaLevelViewportCommands>
{
public:
	AVALANCHELEVELVIEWPORT_API static const FAvaLevelViewportCommands& GetExternal();

	static const FAvaLevelViewportCommands& GetInternal();

	FAvaLevelViewportCommands();

	//~ Begin TCommands
	virtual void RegisterCommands() override;
	//~ End TCommands

	// Viewport
	TSharedPtr<FUICommandInfo> ToggleOverlay;
	TSharedPtr<FUICommandInfo> ToggleBoundingBoxes;
	TSharedPtr<FUICommandInfo> ToggleIsolateActors;
	TSharedPtr<FUICommandInfo> ToggleSafeFrames;
	TSharedPtr<FUICommandInfo> ToggleChildActorLock;
	TSharedPtr<FUICommandInfo> ToggleShapeEditorOverlay;
	TSharedPtr<FUICommandInfo> TogglePostProcessNone;
	TSharedPtr<FUICommandInfo> TogglePostProcessBackground;
	TSharedPtr<FUICommandInfo> TogglePostProcessChannelRed;
	TSharedPtr<FUICommandInfo> TogglePostProcessChannelGreen;
	TSharedPtr<FUICommandInfo> TogglePostProcessChannelBlue;
	TSharedPtr<FUICommandInfo> TogglePostProcessChannelAlpha;
	TSharedPtr<FUICommandInfo> TogglePostProcessCheckerboard;
	TSharedPtr<FUICommandInfo> ToggleTextureOverlay;

	// Camera
	TSharedPtr<FUICommandInfo> CameraZoomInCenter;
	TSharedPtr<FUICommandInfo> CameraZoomOutCenter;
	TSharedPtr<FUICommandInfo> CameraPanLeft;
	TSharedPtr<FUICommandInfo> CameraPanRight;
	TSharedPtr<FUICommandInfo> CameraPanUp;
	TSharedPtr<FUICommandInfo> CameraPanDown;
	TSharedPtr<FUICommandInfo> CameraFrameActor;
	TSharedPtr<FUICommandInfo> CameraZoomReset;
	TSharedPtr<FUICommandInfo> CameraTransformReset;
	TSharedPtr<FUICommandInfo> CameraTransformUndo;
	TSharedPtr<FUICommandInfo> CameraTransformRedo;

	// Grid
	TSharedPtr<FUICommandInfo> ToggleGrid;
	TSharedPtr<FUICommandInfo> ToggleGridAlwaysVisible;
	TSharedPtr<FUICommandInfo> IncreaseGridSize;
	TSharedPtr<FUICommandInfo> DecreaseGridSize;

	// Snapping
	TSharedPtr<FUICommandInfo> ToggleSnapping;
	TSharedPtr<FUICommandInfo> ToggleGridSnapping;
	TSharedPtr<FUICommandInfo> ToggleScreenSnapping;
	TSharedPtr<FUICommandInfo> ToggleActorSnapping;

	// Virtual size
	TSharedPtr<FUICommandInfo> VirtualSizeDisable;
	TSharedPtr<FUICommandInfo> VirtualSize1920x1080;
	TSharedPtr<FUICommandInfo> VirtualSizeAspectRatioUnlocked;
	TSharedPtr<FUICommandInfo> VirtualSizeAspectRatioLocked;
	TSharedPtr<FUICommandInfo> VirtualSizeAspectRatioLockedToCamera;

	// Guides
	TSharedPtr<FUICommandInfo> ToggleGuides;
	TSharedPtr<FUICommandInfo> AddGuideHorizontal;
	TSharedPtr<FUICommandInfo> AddGuideVertical;
	TSharedPtr<FUICommandInfo> ToggleGuideEnabled;
	TSharedPtr<FUICommandInfo> ToggleGuideLocked;
	TSharedPtr<FUICommandInfo> RemoveGuide;

	// Transform
	TSharedPtr<FUICommandInfo> ResetLocation;
	TSharedPtr<FUICommandInfo> ResetRotation;
	TSharedPtr<FUICommandInfo> ResetScale;
	TSharedPtr<FUICommandInfo> ResetTransform;

	// Animator
	TSharedPtr<FUICommandInfo> DisableAnimators;
	TSharedPtr<FUICommandInfo> EnableAnimators;

private:
	// Make this unavailable to the public
	using TCommands<FAvaLevelViewportCommands>::Get;

	void RegisterViewportCommands();
	void RegisterCameraCommands();
	void RegisterGridCommands();
	void RegisterSnappingCommands();
	void RegisterVirtualSizeCommands();
	void RegisterGuideCommands();
	void RegisterTransformCommands();
	void RegisterAnimatorCommands();
};
