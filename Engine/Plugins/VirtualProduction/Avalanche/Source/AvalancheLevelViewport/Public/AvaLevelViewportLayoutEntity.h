// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelViewportLayout.h"
#include "Templates/SharedPointer.h"

class ILevelEditor;
class SAvaLevelViewportFrame;
struct FAssetEditorViewportConstructionArgs;

class AVALANCHELEVELVIEWPORT_API FAvaLevelViewportLayoutEntity : public ILevelViewportLayoutEntity
{
public:
	FAvaLevelViewportLayoutEntity(const FAssetEditorViewportConstructionArgs& InArgs, TSharedPtr<ILevelEditor> InLevelEditor);

	static FName GetStaticType();

	//~ Begin ILevelViewportLayoutEntity
	virtual TSharedPtr<SLevelViewport> AsLevelViewport() const override;
	virtual TSharedRef<SWidget> AsWidget() const override;
	virtual void SetKeyboardFocus() override;
	virtual void OnLayoutDestroyed() override;
	virtual void SaveConfig(const FString& InConfigSection) override;
	virtual FName GetType() const override;
	virtual void TakeHighResScreenShot() const override;
	virtual FLevelEditorViewportClient& GetLevelViewportClient() const override;
	virtual bool IsPlayInEditorViewportActive() const override;
	virtual void RegisterGameViewportIfPIE() override;
	//~ End ILevelViewportLayoutEntity

private:
	TSharedPtr<SLevelViewport> GetViewportWidget() const;

	TSharedPtr<SAvaLevelViewportFrame> Viewport;
};
