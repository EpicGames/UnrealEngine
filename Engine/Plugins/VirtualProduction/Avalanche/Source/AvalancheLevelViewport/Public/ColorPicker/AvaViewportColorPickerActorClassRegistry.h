// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaViewportColorPickerAdapter.h"
#include "Concepts/StaticClassProvider.h"
#include "Templates/SharedPointer.h"

class AActor;
class UClass;
struct FAvaColorChangeData;

struct FAvaViewportColorPickerActorClassRegistry
{
	template<typename InClassType
		UE_REQUIRES(TModels_V<CStaticClassProvider, InClassType>)>
	static void RegisterDefaultClassAdapter()
	{
		RegisterClassAdapter<InClassType, TAvaViewportColorPickerActorAdapter<InClassType>>();
	}

	template<typename InClassType, typename InAdapterType, typename... InArgTypes
		UE_REQUIRES(TModels_V<CStaticClassProvider, InClassType> && std::is_base_of_v<IAvaViewportColorPickerAdapter, InAdapterType>)>
	static void RegisterClassAdapter(InArgTypes&&... InArgs)
	{
		RegisterClassAdapter(InClassType::StaticClass(), MakeShared<InAdapterType>(Forward<InArgTypes>(InArgs)...));
	}

	AVALANCHELEVELVIEWPORT_API static void RegisterClassAdapter(UClass* InClass, const TSharedRef<IAvaViewportColorPickerAdapter>& InAdapter);

	AVALANCHELEVELVIEWPORT_API static bool GetColorDataFromActor(const AActor* InActor, FAvaColorChangeData& OutColorData);

	static bool ApplyColorDataToActor(AActor* InActor, const FAvaColorChangeData& InColorData);
};
