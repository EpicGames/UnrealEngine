// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaDefs.h"
#include "IAvaViewportColorPickerAdapter.h"

/**
 * Describes a type that implements:
 * - GetColorData const member function that returns FAvaColorChangeData
 * - SetColorData that accepts a const FAvaColorChangeData as a parameter
 */
struct CAvaViewportColorPickable
{
	template <typename T>
	auto Requires(const T& InConstActor, FAvaColorChangeData& OutColorData, T& InMutableActor, const FAvaColorChangeData& InColorData)->decltype(
		OutColorData = InConstActor.GetColorData(),
		InMutableActor.SetColorData(InColorData)
	);
};

/** Adapter class for a type that can be casted from an Actor (i.e. an Actor Derived type or an IInterface type) */
template<typename T
	UE_REQUIRES(TModels_V<CAvaViewportColorPickable, T>)>
struct TAvaViewportColorPickerActorAdapter : IAvaViewportColorPickerAdapter
{
	virtual ~TAvaViewportColorPickerActorAdapter() override = default;

	//~ Begin IAvaViewportColorPickerActorAdapter
	virtual bool GetColorData(const AActor& InActor, FAvaColorChangeData& OutColorData) const override
	{
		if (const T* CastedActor = Cast<T>(&InActor))
		{
			OutColorData = CastedActor->GetColorData();
			return true;
		}
		return false;
	}

	virtual void SetColorData(AActor& InActor, const FAvaColorChangeData& InColorData) const override
	{
		if (T* CastedActor = Cast<T>(&InActor))
		{
			CastedActor->SetColorData(InColorData);
		}
	}
	//~ End IAvaViewportColorPickerActorAdapter
};
