// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class AActor;
struct FAvaColorChangeData;

struct IAvaViewportColorPickerAdapter
{
	virtual ~IAvaViewportColorPickerAdapter() = default;

	virtual bool GetColorData(const AActor& InActor, FAvaColorChangeData& OutColorData) const = 0;

	virtual void SetColorData(AActor& InActor, const FAvaColorChangeData& InColorData) const = 0;
};
