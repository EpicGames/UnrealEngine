// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheMRQ : ModuleRules
{
    public AvalancheMRQ(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "AvalancheMedia",
                "Core", 
                "MovieRenderPipelineCore",
                "RemoteControl",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "Avalanche",
                "AvalancheRemoteControl",
                "CoreUObject",
                "Engine",
                "Slate",
                "SlateCore",
            }
        );
    }
}