// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMRQRundownPageSetting.h"
#include "AvaRemoteControlRebind.h"
#include "AvaRemoteControlUtils.h"
#include "AvaScene.h"
#include "AvaSceneSubsystem.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "IAvaSceneInterface.h"
#include "LevelSequence.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/PackageName.h"
#include "MoviePipeline.h"
#include "MoviePipelineTelemetry.h"
#include "Playback/AvaPlaybackUtils.h"
#include "Rundown/AvaRundown.h"

DEFINE_LOG_CATEGORY_STATIC(LogAvaMRQSequenceData, Log, All);

namespace UE::AvaMRQ::Private
{
	int32 FindPieInstanceId(UWorld* InWorld)
	{
		check(GEngine);
		for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
		{
			// Return the first PIE world that we can find
			if (WorldContext.WorldType == EWorldType::PIE && InWorld == WorldContext.World())
			{
				return WorldContext.PIEInstance;
			}
		}
		return INDEX_NONE;
	}

	FSoftObjectPath RemovePiePrefix(const FSoftObjectPath& InPiePath, int32 InPieInstanceId)
	{
		FString NewPath = InPiePath.ToString();
		if (FPackageName::GetLongPackageAssetName(NewPath).StartsWith(PLAYWORLD_PACKAGE_PREFIX))
		{
			FString PiePrefix = FString::Printf(TEXT("%s_%d_"), PLAYWORLD_PACKAGE_PREFIX, InPieInstanceId);
			NewPath = NewPath.Replace(*PiePrefix, TEXT(""), ESearchCase::CaseSensitive);
		}
		return FSoftObjectPath(NewPath);
	}
}

void UAvaMRQRundownPageSetting::SetupForPipelineImpl(UMoviePipeline* InPipeline)
{
	UWorld* World = GetWorld();
	if (!InPipeline || !World)
	{
		return;
	}

	ULevelSequence* LevelSequence = InPipeline->GetTargetSequence();
	if (!LevelSequence)
	{
		return;
	}

	UAvaRundown* Rundown = RundownPage.Rundown.LoadSynchronous();
	if (!Rundown)
	{
		UE_LOGF(LogAvaMRQSequenceData, Error
			, "Motion Design Rundown '%ls' was not valid and could not be loaded."
			, *RundownPage.Rundown.ToString());
		return;
	}

	const FAvaRundownPage& Page = Rundown->GetPage(RundownPage.PageId);
	if (!Page.IsValidPage())
	{
		UE_LOGF(LogAvaMRQSequenceData, Error
			, "Motion Design Rundown '%ls' did not have a valid page in %d."
			, *Rundown->GetFullName()
			, RundownPage.PageId);
		return;
	}

	// Ensure Page Source Asset matches
	int32 PieInstanceId = UE::AvaMRQ::Private::FindPieInstanceId(World);
	if (PieInstanceId != INDEX_NONE)
	{
		FSoftObjectPath SourcePath  = Page.GetAssetPath(Rundown);
		FSoftObjectPath CurrentPath = UE::AvaMRQ::Private::RemovePiePrefix(World, PieInstanceId);

		if (CurrentPath != SourcePath)
		{
			UE_LOGF(LogAvaMRQSequenceData, Error
				, "Asset path '%ls' in Page '%d' of Motion Design Rundown '%ls' did not match the provided PIE sanitized world path '%ls'"
				, *SourcePath.ToString()
				, Page.GetPageId()
				, *Rundown->GetFullName()
				, *CurrentPath.ToString());
			return;
		}
	}

	ULevel* const Level = World->PersistentLevel;

	IAvaSceneInterface* Scene = UAvaSceneSubsystem::FindSceneInterface(Level);
	if (!Scene)
	{
		UE_LOGF(LogAvaMRQSequenceData, Error
			, "Motion Design Scene Interface was not found in the provided World '%ls'."
			, *World->GetFullName());
		return;
	}

	URemoteControlPreset* RemoteControlPreset = Scene->GetRemoteControlPreset();
	if (!RemoteControlPreset)
	{
		UE_CLOGF(Scene->IsRemoteControlPresetExpectedToBePresent(), LogAvaMRQSequenceData, Error
			, "A Remote Control Preset was not found in the provided World '%ls'."
			, *World->GetFullName());
		return;
	}

	FAvaRemoteControlRebind::RebindUnboundEntities(RemoteControlPreset, Level);

	const FAvaPlayableRemoteControlValues& RemoteControlValues = Page.GetRemoteControlValues();

	// Synchronously load the object paths stored in the remote control values of the rundown page
	// No async available as MRQ Setup phase does not do deferred setups
	{
		TSet<FSoftObjectPath> ReferencedPaths;

		FAvaPlayableRemoteControlValues::CollectReferencedAssetPaths(RemoteControlValues.EntityValues, ReferencedPaths);
		FAvaPlayableRemoteControlValues::CollectReferencedAssetPaths(RemoteControlValues.ControllerValues, ReferencedPaths);

		for (const FSoftObjectPath& ReferencedPath : ReferencedPaths)
		{
			if (!ReferencedPath.ResolveObject())
			{
				// Synchronously load the object
				ReferencedPath.TryLoad();
			}
		}
	}

	RemoteControlValues.ApplyEntityValuesToRemoteControlPreset(RemoteControlPreset);
}

void UAvaMRQRundownPageSetting::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesMotionDesignRundown = true;
}
