// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoviePipelineSetting.h"
#include "UObject/SoftObjectPtr.h"
#include "AvaMRQRundownPageSetting.generated.h"

class UAvaRundown;

USTRUCT(BlueprintType, DisplayName = "Motion Design MRQ Rundown Page")
struct FAvaMRQRundownPage
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Sequence")
	TSoftObjectPtr<UAvaRundown> Rundown;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Sequence")
	int32 PageId = 0;
};

UCLASS(MinimalAPI, DisplayName = "Rundown Page")
class UAvaMRQRundownPageSetting : public UMoviePipelineSetting
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Sequence", meta=(ShowOnlyInnerProperties))
	FAvaMRQRundownPage RundownPage;

protected:
	//~ Begin UMoviePipelineSetting
	AVALANCHEMRQ_API virtual void SetupForPipelineImpl(UMoviePipeline* InPipeline) override;
	virtual bool IsValidOnShots() const override { return false; }
	virtual bool IsValidOnPrimary() const override { return true; }
	AVALANCHEMRQ_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	//~ End UMoviePipelineSetting
};
