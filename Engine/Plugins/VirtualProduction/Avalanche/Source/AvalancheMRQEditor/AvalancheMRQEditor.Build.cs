// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheMRQEditor : ModuleRules
{
    public AvalancheMRQEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "Avalanche",
                "AvalancheMRQ",
                "AvalancheMedia",
                "AvalancheMediaEditor",
                "AvalancheSequence",
                "AvalancheSequencer",
                "Core",
                "CoreUObject",
                "DeveloperSettings",
                "Engine",
                "MovieRenderPipelineCore",
                "MovieRenderPipelineEditor",
                "MovieRenderPipelineRenderPasses",
                "RemoteControl",
                "Slate",
                "SlateCore",
                "UnrealEd", 
            }
        );
    }
}