// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMRQEditorCommands.h"
#include "AvaMRQEditorStyle.h"

#define LOCTEXT_NAMESPACE "AvaMRQEditorCommands"

FAvaMRQEditorCommands::FAvaMRQEditorCommands()
	: TCommands<FAvaMRQEditorCommands>(TEXT("AvaMRQEditor")
	, LOCTEXT("MotionDesignMRQEditorCommands", "Motion Design MRQ Editor Commands")
	, NAME_None
	, FAvaMRQEditorStyle::Get().GetStyleSetName())
{
}

void FAvaMRQEditorCommands::RegisterCommands()
{
	UI_COMMAND(RenderSelectedPages
		, "Render Selected Pages"
		, "Renders all the selected pages"
		, EUserInterfaceActionType::Button
		, FInputChord())
}

#undef LOCTEXT_NAMESPACE
