// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FAvaMRQEditorCommands : public TCommands<FAvaMRQEditorCommands>
{
public:
	FAvaMRQEditorCommands();

	//~ Begin TCommands
	virtual void RegisterCommands() override;
	//~ End TCommands

	TSharedPtr<FUICommandInfo> RenderSelectedPages;
};
