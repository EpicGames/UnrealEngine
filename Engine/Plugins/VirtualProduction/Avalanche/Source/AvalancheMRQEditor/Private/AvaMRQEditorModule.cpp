// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMRQEditorModule.h"
#include "AvaMRQEditorCommands.h"
#include "AvaMRQEditorRundownUtils.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "IAvaMediaEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownEditor.h"
#include "Toolkits/AssetEditorToolkit.h"

void FAvaMRQEditorModule::StartupModule()
{
	FAvaMRQEditorCommands::Register();

	TSharedPtr<FExtensibilityManager> RundownToolbarExtensibility = IAvaMediaEditorModule::Get().GetRundownToolBarExtensibilityManager();
	if (RundownToolbarExtensibility.IsValid())
	{
		RundownToolbarExtensibilityWeak = RundownToolbarExtensibility;

		TArray<FAssetEditorExtender>& ExtenderDelegates = RundownToolbarExtensibility->GetExtenderDelegates();
		ExtenderDelegates.Add(FAssetEditorExtender::CreateStatic(&FAvaMRQEditorModule::ExtendRundownToolbar));

		RundownToolbarExtenderHandle = ExtenderDelegates.Last().GetHandle();
	}
}

void FAvaMRQEditorModule::ShutdownModule()
{
	FAvaMRQEditorCommands::Unregister();

	if (TSharedPtr<FExtensibilityManager> RundownToolbarExtensibility = RundownToolbarExtensibilityWeak.Pin())
	{
		if (RundownToolbarExtenderHandle.IsValid())
		{
			RundownToolbarExtensibility->GetExtenderDelegates().RemoveAll([this](const FAssetEditorExtender& InExtender)
			{
				return InExtender.GetHandle() == RundownToolbarExtenderHandle;
			});
			RundownToolbarExtenderHandle.Reset();
		}
		RundownToolbarExtensibilityWeak.Reset();
	}
}

TSharedRef<FExtender> FAvaMRQEditorModule::ExtendRundownToolbar(const TSharedRef<FUICommandList> InCommandList, const TArray<UObject*> InObjects)
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();

	UAssetEditorSubsystem* const AssetEditorSubsystem = GEditor ? GEditor->GetEditorSubsystem<UAssetEditorSubsystem>() : nullptr;
	if (!AssetEditorSubsystem)
	{
		return Extender;
	}

	TSharedRef<FAvaMRQRundownContext> Context = MakeShared<FAvaMRQRundownContext>();
	Context->RundownEditors.Reserve(InObjects.Num());

	const FName RundownEditorName = TEXT("AvaRundownEditor");

	// Gather Rundown Editors
	for (UObject* Object : InObjects)
	{
		if (UAvaRundown* Rundown = Cast<UAvaRundown>(Object))
		{
			IAssetEditorInstance* AssetEditor = AssetEditorSubsystem->FindEditorForAsset(Rundown, /*bFocusIfOpen*/false);
			if (!AssetEditor || AssetEditor->GetEditorName() != RundownEditorName)
			{
				continue;
			}

			FAvaRundownEditor* RundownEditor = static_cast<FAvaRundownEditor*>(AssetEditor);
			Context->RundownEditors.Add(StaticCastWeakPtr<FAvaRundownEditor>(RundownEditor->AsWeak()));
		}
	}

	if (Context->RundownEditors.IsEmpty())
	{
		return Extender;
	}

	Extender->AddToolBarExtension("Pages"
		, EExtensionHook::After
		, CreateRundownActions(Context)
		, FToolBarExtensionDelegate::CreateLambda([](FToolBarBuilder& ToolbarBuilder)
		{
			const FAvaMRQEditorCommands& MRQEditorCommands = FAvaMRQEditorCommands::Get();
			ToolbarBuilder.AddToolBarButton(MRQEditorCommands.RenderSelectedPages);
		})
	);

	return Extender;
}

TSharedRef<FUICommandList> FAvaMRQEditorModule::CreateRundownActions(TSharedRef<FAvaMRQRundownContext> InContext)
{
	const FAvaMRQEditorCommands& MRQEditorCommands = FAvaMRQEditorCommands::Get();

	TSharedRef<FUICommandList> CommandList = MakeShared<FUICommandList>();

	static auto HasAnySelectedPage = [](const TWeakPtr<const FAvaRundownEditor>& InRundownEditorWeak)
	{
		TSharedPtr<const FAvaRundownEditor> RundownEditor = InRundownEditorWeak.Pin();
		return RundownEditor.IsValid() && RundownEditor->GetSelectedPagesOnActiveSubListWidget().Num() > 0;
	};

	FCanExecuteAction CanExecuteSelectedPageAction = FCanExecuteAction::CreateLambda([InContext]
	{
		return InContext->RundownEditors.ContainsByPredicate(HasAnySelectedPage);
	});

	CommandList->MapAction(MRQEditorCommands.RenderSelectedPages
		, FExecuteAction::CreateLambda([InContext]{ FAvaMRQEditorRundownUtils::RenderSelectedPages(InContext->RundownEditors); })
		, CanExecuteSelectedPageAction);

	return CommandList;
}

IMPLEMENT_MODULE(FAvaMRQEditorModule, AvalancheMRQEditor)
