// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class FAvaRundownEditor;
class FExtender;
class FExtensibilityManager;
class FUICommandList;
class UAvaRundown;
class UObject;

struct FAvaMRQRundownContext
{
    TArray<TWeakPtr<const FAvaRundownEditor>, TInlineAllocator<1>> RundownEditors;
};

class FAvaMRQEditorModule : public IModuleInterface
{
    //~ Begin IModuleInterface
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;
    //~ End IModuleInterface

    static TSharedRef<FExtender> ExtendRundownToolbar(const TSharedRef<FUICommandList> InCommandList, const TArray<UObject*> InObjects);

    static TSharedRef<FUICommandList> CreateRundownActions(TSharedRef<FAvaMRQRundownContext> InContext);

    TWeakPtr<FExtensibilityManager> RundownToolbarExtensibilityWeak;

    FDelegateHandle RundownToolbarExtenderHandle;
};
