// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Templates/SharedPointerFwd.h"

class FAvaRundownEditor;

struct FAvaMRQEditorRundownUtils
{
	static void RenderSelectedPages(TConstArrayView<TWeakPtr<const FAvaRundownEditor>> InRundownEditors);
};
