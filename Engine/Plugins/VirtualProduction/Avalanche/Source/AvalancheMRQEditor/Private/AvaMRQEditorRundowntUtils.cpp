// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMRQEditorRundownUtils.h"
#include "AvaMRQEditorSettings.h"
#include "AvaMRQRundownPageSetting.h"
#include "AvaSceneSubsystem.h"
#include "IAvaSceneInterface.h"
#include "IAvaSequenceProvider.h"
#include "MoviePipelinePIEExecutor.h"
#include "MoviePipelineQueueSubsystem.h"
#include "Playback/AvaPlaybackUtils.h"
#include "Rundown/AvaRundownEditor.h"

DEFINE_LOG_CATEGORY_STATIC(LogAvaMRQEditorRundown, Log, All);

namespace UE::AvaMRQEditor::Private
{
	void CleanupRootedWorlds(TConstArrayView<TWeakObjectPtr<UWorld>> InRootedWorlds)
	{
		for (const TWeakObjectPtr<UWorld>& RootedWorldWeak : InRootedWorlds)
		{
			if (UWorld* const RootedWorld = RootedWorldWeak.Get())
			{
				RootedWorld->RemoveFromRoot();
			}
		}
	}

	struct FAvaMRQScopedRender
	{
		FAvaMRQScopedRender()
		{
			QueueSubsystem = GEditor ? GEditor->GetEditorSubsystem<UMoviePipelineQueueSubsystem>() : nullptr;
			ensureMsgf(QueueSubsystem, TEXT("Not able to access UMoviePipelineQueueSubsystem (returned null)"));

			// todo: optionally prompt the user for a different preset
			const UAvaMRQEditorSettings* MRQEditorSettings = GetDefault<UAvaMRQEditorSettings>();
			check(MRQEditorSettings);
			PresetConfig = MRQEditorSettings->PresetConfig.LoadSynchronous();
		}

		~FAvaMRQScopedRender()
		{
			if (JobAllocationCount > 0)
			{
				UMoviePipelineExecutorBase* Executor = QueueSubsystem->RenderQueueWithExecutor(UMoviePipelinePIEExecutor::StaticClass());
				if (Executor && !RootedWorlds.IsEmpty())
				{
					Executor->OnExecutorFinished().AddLambda([RootedWorlds = MoveTemp(RootedWorlds)](UMoviePipelineExecutorBase*, bool)
					{
						CleanupRootedWorlds(RootedWorlds);
					});
				}
			}
		}

		UMoviePipelineExecutorJob* AllocateJob()
		{
			UMoviePipelineQueue* const Queue = QueueSubsystem->GetQueue();
			if (!Queue)
			{
				return nullptr;
			}
			if (JobAllocationCount == 0)
			{
				Queue->DeleteAllJobs();
			}
			if (UMoviePipelineExecutorJob* Job = Queue->AllocateNewJob(UMoviePipelineExecutorJob::StaticClass()))
			{
				UMoviePipelinePrimaryConfig* Config = Job->GetConfiguration();
				if (PresetConfig && Config)
				{
					Config->CopyFrom(PresetConfig);
				}
				++JobAllocationCount;
				return Job;
			}
			return nullptr;
		}

		bool IsValid() const
		{
			return ::IsValid(QueueSubsystem) && ::IsValid(QueueSubsystem->GetQueue());
		}

		void EnsureRootedWorld(UWorld* InWorldChecked)
		{
			check(InWorldChecked);
			if (!InWorldChecked->IsRooted())
			{
				InWorldChecked->AddToRoot();
				RootedWorlds.Add(InWorldChecked);
			}
		}

	private:		
		UMoviePipelineQueueSubsystem* QueueSubsystem = nullptr;

		uint32 JobAllocationCount = 0;

		TArray<TWeakObjectPtr<UWorld>> RootedWorlds;

		UMoviePipelineConfigBase* PresetConfig = nullptr;
	};

	void RenderSequence(FAvaMRQScopedRender& InScopedRender, UWorld* InWorld, const UAvaRundown& InRundown, const FAvaRundownPage& InPage, UAvaSequence* InSequence)
	{
		if (!InSequence)
		{
			return;
		}

		UMoviePipelineExecutorJob* Job = InScopedRender.AllocateJob();
		if (!Job)
		{
			return;
		}

		constexpr const TCHAR* Separator = TEXT("-");

		Job->JobName = InRundown.GetName() + Separator + TEXT("Page_") + FString::FromInt(InPage.GetPageId()) + Separator + InSequence->GetName();
		Job->Map = InWorld;
		Job->SetSequence(InSequence);

		UMoviePipelinePrimaryConfig* MRQConfig = Job->GetConfiguration();
		check(MRQConfig);

		auto CreatePipelineSetting = [MRQConfig]<typename InSettingType>(InSettingType*& OutSettings)
		{
			OutSettings = Cast<InSettingType>(MRQConfig->FindOrAddSettingByClass(InSettingType::StaticClass()));
		};

		UAvaMRQRundownPageSetting* RundownPageSetting;
		CreatePipelineSetting(RundownPageSetting);
		RundownPageSetting->RundownPage.Rundown = const_cast<UAvaRundown*>(&InRundown);
		RundownPageSetting->RundownPage.PageId  = InPage.GetPageId();

		UE_LOGF(LogAvaMRQEditorRundown, Log
			, "New MRQ Job '%ls' created for World '%ls' and LevelSequence '%ls'"
			, *Job->GetFullName()
			, *InWorld->GetFullName()
			, *InSequence->GetFullName());
	}

	void RenderPage(FAvaMRQScopedRender& InScopedRender, const UAvaRundown& InRundown, const FAvaRundownPage& InPage)
	{
		FSoftObjectPath PageAssetPath = InPage.GetAssetPath(&InRundown);
		if (!FAvaPlaybackUtils::IsMapAsset(PageAssetPath.GetLongPackageName()))
		{
			UE_LOGF(LogAvaMRQEditorRundown, Error
				, "Page asset path '%ls' (Page Id: '%d', Rundown '%ls) is not a map asset and will not be processed"
				, *PageAssetPath.ToString()
				, InPage.GetPageId()
				, *InRundown.GetName());
			return;
		}

		UWorld* const World = Cast<UWorld>(PageAssetPath.TryLoad());
		if (!World)
		{
			UE_LOGF(LogAvaMRQEditorRundown, Error
				, "World path '%ls' (Page Id: '%d', Rundown '%ls) did not load a valid world and will not be processed"
				, *PageAssetPath.ToString()
				, InPage.GetPageId()
				, *InRundown.GetName());
			return;
		}
		InScopedRender.EnsureRootedWorld(World);

		IAvaSceneInterface* SceneInterface = UAvaSceneSubsystem::FindSceneInterface(World->PersistentLevel);
		if (!SceneInterface || !SceneInterface->GetSequenceProvider())
		{
			UE_LOGF(LogAvaMRQEditorRundown, Error
				, "World '%ls' (Page Id: '%d', Rundown '%ls) did not have a valid Scene Interface to retrieve the Sequences so will not be processed"
				, *PageAssetPath.ToString()
				, InPage.GetPageId()
				, *InRundown.GetName());
			return;
		}

		TConstArrayView<UAvaSequence*> Sequences = SceneInterface->GetSequenceProvider()->GetSequences();
		for (UAvaSequence* Sequence : Sequences)
		{
			RenderSequence(InScopedRender, World, InRundown, InPage, Sequence);
		}
	}

	void RenderPages(FAvaMRQScopedRender& InScopedRender, const UAvaRundown& InRundown, TConstArrayView<int32> InPageIds)
	{
		TSet<int32> ProcessedPages;
		ProcessedPages.Reserve(InPageIds.Num());

		for (int32 PageId : InPageIds)
		{
			bool bIsAlreadyInSet = false;
			ProcessedPages.Add(PageId, &bIsAlreadyInSet);

			if (bIsAlreadyInSet)
			{
				UE_LOGF(LogAvaMRQEditorRundown, Warning
					, "Page Id '%d' is repeated in the page ids to export for Rundown '%ls' and will not be processed multiple times"
					, PageId
					, *InRundown.GetName());
				continue;
			}

			const FAvaRundownPage& Page = InRundown.GetPage(PageId);
			if (!Page.IsValidPage())
			{
				UE_LOGF(LogAvaMRQEditorRundown, Warning
					, "Page Id '%d' is invalid for Rundown '%ls' and will not be processed"
					, PageId
					, *InRundown.GetName());
				continue;
			}

			RenderPage(InScopedRender, InRundown, Page);
		}
	}
}

void FAvaMRQEditorRundownUtils::RenderSelectedPages(TConstArrayView<TWeakPtr<const FAvaRundownEditor>> InRundownEditors)
{
	if (InRundownEditors.IsEmpty())
	{
		return;
	}

	using namespace UE::AvaMRQEditor;
	TSet<TWeakPtr<const FAvaRundownEditor>> ProcessedRundownEditors;
	ProcessedRundownEditors.Reserve(InRundownEditors.Num());

	Private::FAvaMRQScopedRender ScopedRender;
	if (!ScopedRender.IsValid())
	{
		return;
	}

	for (TWeakPtr<const FAvaRundownEditor> RundownEditorWeak : InRundownEditors)
	{
		TSharedPtr<const FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin();
		if (!RundownEditor || !RundownEditor->IsRundownValid())
		{
			continue;
		}

		UAvaRundown* Rundown = RundownEditor->GetRundown();
		check(Rundown);

		bool bIsAlreadyInSet = false;
		ProcessedRundownEditors.Add(RundownEditorWeak, &bIsAlreadyInSet);

		if (bIsAlreadyInSet)
		{
			UE_LOGF(LogAvaMRQEditorRundown, Warning
				, "Rundown Editor '%ls' is repeated in the editors to export and will not be processed again"
				, *Rundown->GetName());
			continue;
		}

		TConstArrayView<int32> PageIds = RundownEditor->GetSelectedPagesOnActiveSubListWidget();
		Private::RenderPages(ScopedRender, *Rundown, PageIds);
	}
}
