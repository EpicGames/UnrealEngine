// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMRQEditorSettings.h"

UAvaMRQEditorSettings::UAvaMRQEditorSettings()
{
	CategoryName = TEXT("Motion Design");
	SectionName  = TEXT("Movie Render Queue");
}
