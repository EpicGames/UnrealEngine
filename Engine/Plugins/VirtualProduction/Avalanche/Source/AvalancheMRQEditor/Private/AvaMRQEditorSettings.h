// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "UObject/SoftObjectPtr.h"
#include "AvaMRQEditorSettings.generated.h"

class UMoviePipelinePrimaryConfig;

UCLASS(config=EditorPerProjectUserSettings, meta=(DisplayName="Movie Render Queue"))
class UAvaMRQEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UAvaMRQEditorSettings();

	UPROPERTY(Config, EditAnywhere, Category="Motion Design")
	TSoftObjectPtr<UMoviePipelinePrimaryConfig> PresetConfig;
};
