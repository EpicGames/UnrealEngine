// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMRQEditorStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

FAvaMRQEditorStyle::FAvaMRQEditorStyle()
	: FSlateStyleSet(TEXT("AvaMRQEditor"))
{
	const FVector2f Icon20(20.f);

	SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	Set("AvaMRQEditor.RenderSelectedPages", new IMAGE_BRUSH_SVG("Starship/MainToolbar/cinematics", Icon20));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FAvaMRQEditorStyle::~FAvaMRQEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
