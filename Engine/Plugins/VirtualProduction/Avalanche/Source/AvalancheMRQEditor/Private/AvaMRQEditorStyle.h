// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FAvaMRQEditorStyle final : public FSlateStyleSet
{
public:
	static FAvaMRQEditorStyle& Get()
	{
		static FAvaMRQEditorStyle Instance;
		return Instance;
	}

	FAvaMRQEditorStyle();
	virtual ~FAvaMRQEditorStyle() override;
};
