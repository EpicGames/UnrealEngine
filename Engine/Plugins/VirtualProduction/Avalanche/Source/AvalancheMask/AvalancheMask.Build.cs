// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheMask : ModuleRules
{
	public AvalancheMask(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"AvalancheMaterial",
				"AvalancheModifiers",
				"AvalancheShapes",
				"Core",
                "CoreUObject",
                "GeometryMask",
            });

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ActorModifierCore",
				"ActorModifier",
				"AssetRegistry",
				"Avalanche",
				"AvalancheCore",
				"AvalancheText",
				"DeveloperSettings",
				"DynamicMaterial",
				"Engine",
				"GeometryFramework",
				"MediaPlate",
				"Slate",
				"SlateCore",
				"Text3D",
			});

        if (Target.bBuildEditor == true)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[] {
	                "AssetTools",
	                "DynamicMaterialEditor",
	                "Projects",
                    "UnrealEd",
                });
        }
    }
}
