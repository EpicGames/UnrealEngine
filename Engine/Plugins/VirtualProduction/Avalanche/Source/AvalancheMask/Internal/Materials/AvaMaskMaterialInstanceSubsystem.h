// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/SharedStruct.h"
#include "Subsystems/EngineSubsystem.h"
#include "Subsystems/WorldSubsystem.h"
#include "UObject/SoftObjectPtr.h"
#include "Templates/SubclassOf.h"

#include "AvaMaskMaterialInstanceSubsystem.generated.h"

class IAvaMaterialHandle;
class UAvaMaskMaterialFactory;
class UAvaMaskMaterialFactoryBase;
class UDynamicMaterialInstance;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class UAvaMaskMaterialInstanceSubsystem;

/** A material instance is identified by a combination of it's parent and requested parameter values */
USTRUCT()
struct FAvaMaskMaterialInstanceKey
{
	GENERATED_BODY()

	UPROPERTY()
	TSoftObjectPtr<UMaterialInterface> ParentMaterial;
	
	UPROPERTY()
	uint32 PermutationKey = 0;

	friend uint32 GetTypeHash(const FAvaMaskMaterialInstanceKey& Value)
	{
		return HashCombineFast(
			GetTypeHash(Value.ParentMaterial),
			Value.PermutationKey);
	}
};

USTRUCT()
struct FAvaMaskMaterialPermutations
{
	GENERATED_BODY()

	UPROPERTY()
	TSoftObjectPtr<UMaterialInterface> ParentMaterial;

	UPROPERTY()
	TMap<uint32, TWeakObjectPtr<UMaterialInstanceDynamic>> Instances;
};

UCLASS(DisplayName = "Motion Design Mask MaterialInstance Provider")
class AVALANCHEMASK_API UAvaMaskMaterialInstanceProvider
	: public UObject
{
	GENERATED_BODY()

public:
	/** Instance Key will usually be the product of the Component and Slot. */
	UMaterialInstanceDynamic* FindOrAddMID(UMaterialInterface* InParentMaterial, const uint32& InInstanceKey, const EBlendMode InBlendMode = EBlendMode::BLEND_Masked);
	
	void SetParent(UAvaMaskMaterialInstanceProvider* InParentProvider);
	void SetFactories(const TMap<TSubclassOf<UMaterialInterface>, TObjectPtr<UAvaMaskMaterialFactoryBase>>& InFactories);

private:
	friend class UAvaMaskMaterialInstanceSubsystem;
	
	UPROPERTY(Transient)
	TWeakObjectPtr<UAvaMaskMaterialInstanceProvider> ParentProvider;

	UPROPERTY()
	TMap<TSubclassOf<UMaterialInterface>, TObjectPtr<UAvaMaskMaterialFactoryBase>> MaterialInstanceFactories;
	
	UPROPERTY()
	TMap<TSoftObjectPtr<UMaterialInterface>, FAvaMaskMaterialPermutations> MaterialInstanceCache;
};

/** Stores duplicates of the MID's provided by the engine subsystem, allowing level actors to reference them. */
UCLASS(DisplayName = "Motion Design Mask MaterialInstance World Subsystem")
class AVALANCHEMASK_API UAvaMaskMaterialInstanceWorldSubsystem
	: public UWorldSubsystem
{
	GENERATED_BODY()

public:
	// ~Begin USubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	// ~End USubsystem

	UAvaMaskMaterialInstanceProvider* GetMaterialInstanceProvider() const;

private:
	UPROPERTY()
	TObjectPtr<UAvaMaskMaterialInstanceProvider> MaterialInstanceProvider;

	UPROPERTY()
	TObjectPtr<UAvaMaskMaterialInstanceSubsystem> MaterialInstanceEngineSubsystem;
};

/** Responsible for providing MID's for a given Material. */
UCLASS(DisplayName = "Motion Design Mask MaterialInstance Engine Subsystem")
class AVALANCHEMASK_API UAvaMaskMaterialInstanceSubsystem
	: public UEngineSubsystem
{
	GENERATED_BODY()

public:
	// ~Begin USubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	// ~End USubsystem

	UAvaMaskMaterialInstanceProvider* GetMaterialInstanceProvider() const;

	/** Call to forcibly clear cached MID's. */
	void ClearCached();

private:
	void FindMaterialFactories();

private:
	UPROPERTY()
	TObjectPtr<UAvaMaskMaterialInstanceProvider> MaterialInstanceProvider;
};
