// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMaskLog.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogAvaMask);

IMPLEMENT_MODULE(FDefaultModuleImpl, AvalancheMask)
