// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMaskSettings.h"

#include "AvaMaskLog.h"

#if WITH_EDITORONLY_DATA
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Interfaces/IPluginManager.h"
#include "Materials/MaterialFunctionInterface.h"
#endif

UAvaMaskSettings::UAvaMaskSettings()
{
	CategoryName = TEXT("Motion Design");
	SectionName = TEXT("Mask");
#if WITH_EDITORONLY_DATA
	MaterialFunction = FSoftObjectPath(TEXT("/Script/Engine.MaterialFunction'/GeometryMask/GeometryMask/MF_ApplyMask2D_Single.MF_ApplyMask2D_Single'"));
#endif
}

#if WITH_EDITORONLY_DATA
UMaterialFunctionInterface* UAvaMaskSettings::GetMaterialFunction() const
{
	if (ResolvedMaterialFunction)
	{
		return ResolvedMaterialFunction;		
	}

	ResolvedMaterialFunction = MaterialFunction.LoadSynchronous();
	return ResolvedMaterialFunction;
}
#endif
