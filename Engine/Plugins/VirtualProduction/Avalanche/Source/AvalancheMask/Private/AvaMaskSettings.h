// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#if WITH_EDITORONLY_DATA
#include "UObject/SoftObjectPtr.h"
#endif

#include "AvaMaskSettings.generated.h"

#if WITH_EDITORONLY_DATA
class UMaterialFunctionInterface;
#endif

/** Settings for Motion Design Mask */
UCLASS(Config = Engine, meta = (DisplayName = "Mask"))
class UAvaMaskSettings
    : public UDeveloperSettings
{
    GENERATED_BODY()

public:
    UAvaMaskSettings();

#if WITH_EDITORONLY_DATA
	/** Get the user specified or default material function. */
	UMaterialFunctionInterface* GetMaterialFunction() const;

private:
	/** Material Function to use to expect or add to a material. */
	UPROPERTY(Config, EditAnywhere, NoClear, Category = "Material", meta=(ConfigRestartRequired=true))
	TSoftObjectPtr<UMaterialFunctionInterface> MaterialFunction;

	UPROPERTY(Transient)
	mutable TObjectPtr<UMaterialFunctionInterface> ResolvedMaterialFunction;
#endif
};
