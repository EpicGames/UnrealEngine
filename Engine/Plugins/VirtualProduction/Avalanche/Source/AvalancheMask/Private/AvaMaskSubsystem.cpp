// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMaskSubsystem.h"
#include "Engine/Engine.h"
#include "Materials/MaterialInterface.h"

UMaterialInterface* UAvaMaskSubsystem::StaticGetDefaultMaskMaterial()
{
	UAvaMaskSubsystem* Subsystem = GEngine ? GEngine->GetEngineSubsystem<UAvaMaskSubsystem>() : nullptr;
	return Subsystem ? Subsystem->GetDefaultMaskMaterial() : nullptr;
}

UMaterialInterface* UAvaMaskSubsystem::GetDefaultMaskMaterial() const
{
	if (DefaultMaskMaterial)
	{
		return DefaultMaskMaterial;
	}

	// fallback if the material soft object is not loaded
	DefaultMaskMaterial = DefaultMaskMaterialSoft.LoadSynchronous();
	return DefaultMaskMaterial;
}

#if WITH_EDITOR
void UAvaMaskSubsystem::SetLastSpecifiedChannelName(const FName InName)
{
	LastSpecifiedChannelName = InName;
}
#endif

void UAvaMaskSubsystem::Initialize(FSubsystemCollectionBase& InCollection)
{
	Super::Initialize(InCollection);

	DefaultMaskMaterialSoft = FSoftObjectPath(TEXT("/Avalanche/MaskResources/M_AvalancheMaskDefaultTranslucent.M_AvalancheMaskDefaultTranslucent"));
	DefaultMaskMaterialSoft.LoadAsync(FLoadSoftObjectPathAsyncDelegate::CreateWeakLambda(this, 
		[this](const FSoftObjectPath&, UObject* InObject)
		{
			if (UMaterialInterface* Material = Cast<UMaterialInterface>(InObject))
			{
				DefaultMaskMaterial = Material;
			}
		}));
}

