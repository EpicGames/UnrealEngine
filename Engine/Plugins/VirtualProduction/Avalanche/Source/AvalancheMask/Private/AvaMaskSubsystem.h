// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"
#include "AvaMaskSubsystem.generated.h"

class UMaterialInterface;

UCLASS()
class UAvaMaskSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	/** Gets the material to use for masking when there's no material set */
	static UMaterialInterface* StaticGetDefaultMaskMaterial();

	/** Gets the material to use for masking when there's no material set */
	UMaterialInterface* GetDefaultMaskMaterial() const;

#if WITH_EDITORONLY_DATA
	/** Stores the last specified (user or auto) channel name to use for subsequently added modifiers in Target mode. */
	void SetLastSpecifiedChannelName(const FName InName);
#endif

	//~ Begin USubsystem
	virtual void Initialize(FSubsystemCollectionBase& InCollection) override;
	//~ End USubsystem

private:
	/** Soft object ptr to the default mask material to use when there's no material set */
	UPROPERTY()
	TSoftObjectPtr<UMaterialInterface> DefaultMaskMaterialSoft;

	/** Default mask material to use when there's no material set */
	UPROPERTY()
	mutable TObjectPtr<UMaterialInterface> DefaultMaskMaterial;

#if WITH_EDITORONLY_DATA
	/** Stores the last specified (user or auto) channel name to use for subsequently added modifiers in Target mode. */
	UPROPERTY(Transient, Getter = "Auto", Setter)
	FName LastSpecifiedChannelName;
#endif
};
