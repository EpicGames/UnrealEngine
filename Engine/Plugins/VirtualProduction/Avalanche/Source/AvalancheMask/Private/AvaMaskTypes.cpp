// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMaskTypes.h"

#include "AvaMaskLog.h"
#include "AvaMaskUtilities.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Misc/EnumerateRange.h"

bool FAvaMask2DMaterialParameters::HasSameParameters(const FAvaMask2DMaterialParameters& InOther) const
{
	constexpr float Tolerance = UE_SMALL_NUMBER;

	return bInvert == InOther.bInvert
		&& bApplyFeathering == InOther.bApplyFeathering
		&& BlendMode == InOther.BlendMode
		&& RenderTarget == InOther.RenderTarget
		&& MaskIndices.Equals(InOther.MaskIndices, Tolerance)
		&& FMath::IsNearlyEqual(BaseOpacity, InOther.BaseOpacity, Tolerance)
		&& FMath::IsNearlyEqual(InnerFeatherRadius, InOther.InnerFeatherRadius, Tolerance)
		&& FMath::IsNearlyEqual(OuterFeatherRadius, InOther.OuterFeatherRadius, Tolerance)
		&& Padding.Equals(InOther.Padding, Tolerance);
}

bool FAvaMask2DMaterialParameters::ApplyToMID(UMaterialInstanceDynamic* InMaterial) const
{
	if (!ensureAlways(InMaterial))
	{
		return false;
	}

	InMaterial->BlendMode = BlendMode;
	InMaterial->BasePropertyOverrides.bOverride_BlendMode = true;
	InMaterial->BasePropertyOverrides.BlendMode = BlendMode;

	InMaterial->SetTextureParameterValueByInfo(UE::AvaMask::Internal::MaskTexturesParameterInfo, RenderTarget);
	InMaterial->SetVectorParameterValueByInfo(UE::AvaMask::Internal::MaskIndicesParameterInfo, MaskIndices);
	InMaterial->SetScalarParameterValueByInfo(UE::AvaMask::Internal::BaseOpacityParameterInfo, FMath::Clamp(BaseOpacity, 0.0f, 1.0f));
	InMaterial->SetScalarParameterValueByInfo(UE::AvaMask::Internal::InvertParameterInfo, bInvert ? 1.f : 0.f);
	InMaterial->SetVectorParameterValueByInfo(UE::AvaMask::Internal::PaddingParameterInfo, FLinearColor(FMath::Max(0, Padding.X), FMath::Max(0, Padding.Y), 0, 0));
	InMaterial->SetVectorParameterValueByInfo(UE::AvaMask::Internal::FeatherParameterInfo, FLinearColor(bApplyFeathering ? 1.0f : 0.0f, FMath::Max(0, OuterFeatherRadius), FMath::Max(0, InnerFeatherRadius), FMath::Max(0, FMath::Max(OuterFeatherRadius, InnerFeatherRadius))));
	return true;
}

bool FAvaMask2DMaterialParameters::StoreFromMaterial(const UMaterialInterface* InMaterial)
{
	if (!ensureAlways(InMaterial))
	{
		return false;
	}

	constexpr float Tolerance = UE_SMALL_NUMBER;

	BlendMode = InMaterial->GetBlendMode();

	auto LogMissingParameter = [InMaterial](const FMaterialParameterInfo& InParameter)
		{
			UE_LOGF(LogAvaMask, Warning, "Could not find parameter '%ls' in material '%ls'", *InParameter.ToString(), *InMaterial->GetFullName());;
		};

	// Mask Textures
	{
		UTexture* FoundTexture = nullptr;
		if (!InMaterial->GetTextureParameterValue(UE::AvaMask::Internal::MaskTexturesParameterInfo, FoundTexture))
		{
			LogMissingParameter(UE::AvaMask::Internal::MaskTexturesParameterInfo);
			return false;
		}
		RenderTarget = Cast<UTextureRenderTarget2DArray>(FoundTexture);
	}

	// Mask Indices
	{
		FLinearColor FoundMaskIndices;
		if (!InMaterial->GetVectorParameterValue(UE::AvaMask::Internal::MaskIndicesParameterInfo, FoundMaskIndices))
		{
			LogMissingParameter(UE::AvaMask::Internal::MaskIndicesParameterInfo);
			return false;
		}
		MaskIndices = FoundMaskIndices;
	}

	// Base Opacity
	{
		float FoundValue = 0.0f;
		if (!InMaterial->GetScalarParameterValue(UE::AvaMask::Internal::BaseOpacityParameterInfo, FoundValue))
		{
			LogMissingParameter(UE::AvaMask::Internal::BaseOpacityParameterInfo);
			return false;
		}
		BaseOpacity = FoundValue;
	}

	// Invert
	{
		float FoundValue = 0.0f;
		if (!InMaterial->GetScalarParameterValue(UE::AvaMask::Internal::InvertParameterInfo, FoundValue))
		{
			LogMissingParameter(UE::AvaMask::Internal::InvertParameterInfo);
			return false;
		}
		bInvert = !FMath::IsNearlyZero(FoundValue, Tolerance);
	}

	// Padding
	{
		FLinearColor FoundValue;
		if (!InMaterial->GetVectorParameterValue(UE::AvaMask::Internal::PaddingParameterInfo, FoundValue))
		{
			LogMissingParameter(UE::AvaMask::Internal::PaddingParameterInfo);
			return false;			
		}
		Padding = FVector2f(FoundValue.R, FoundValue.G);
	}

	// Feathering
	{
		FLinearColor FoundValue;
		if (!InMaterial->GetVectorParameterValue(UE::AvaMask::Internal::FeatherParameterInfo, FoundValue))
		{
			LogMissingParameter(UE::AvaMask::Internal::FeatherParameterInfo);
			return false;
		}
		bApplyFeathering = !FMath::IsNearlyZero(FoundValue.R, Tolerance);
		OuterFeatherRadius = FoundValue.G;
		InnerFeatherRadius = FoundValue.B;
	}

	return true;
}
