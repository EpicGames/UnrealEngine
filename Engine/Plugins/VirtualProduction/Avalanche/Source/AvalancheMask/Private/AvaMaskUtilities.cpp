// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMaskUtilities.h"

#include "AvaMaskLog.h"
#include "AvaMaskSettings.h"
#include "Containers/StringFwd.h"
#include "GameFramework/Actor.h"
#include "Hash/Blake3.h"
#include "Material/DynamicMaterialInstance.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInterface.h"
#include "Model/DynamicMaterialModel.h"

#if WITH_EDITOR
#include "Components/DMMaterialProperty.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#endif

namespace UE::AvaMask::Internal
{
#if WITH_EDITOR
	UDMMaterialProperty* GetMaterialPropertyForBlendMode(TNotNull<UMaterialInterface*> InMaterial, EBlendMode InBlendMode)
	{
		UDynamicMaterialInstance* const Material = Cast<UDynamicMaterialInstance>(InMaterial);
		if (!Material)
		{
			return nullptr;
		}

		UDynamicMaterialModelEditorOnlyData* const EditorOnlyData = UDynamicMaterialModelEditorOnlyData::Get(Material->GetMaterialModel());
		if (!EditorOnlyData)
		{
			return nullptr;
		}

		const EDMMaterialPropertyType PropertyType = InBlendMode == EBlendMode::BLEND_Masked
			? EDMMaterialPropertyType::OpacityMask
			: EDMMaterialPropertyType::Opacity;

		return EditorOnlyData->GetMaterialProperty(PropertyType);
	}
#endif

	FSoftComponentReference MakeComponentReference(
		const AActor* InOwner
		, const UActorComponent* InComponent)
	{
		FSoftComponentReference ComponentReference;

		AActor* ComponentOwner = InComponent->GetOwner();
		ComponentReference.OtherActor = ComponentOwner == InOwner ? nullptr : ComponentOwner;
		ComponentReference.PathToComponent = InComponent->GetPathName(ComponentOwner);
	
		return ComponentReference;
	}

	// Based on FGuid::NewDeterministicGuid
	uint32 MakeMaterialInstanceKey(const UMaterialInterface* InMaterial, const FName InMaskChannelName, EBlendMode InBlendMode, const int32 InSeed)
	{
		if (!ensureAlways(InMaterial))
		{
			return INDEX_NONE;
		}

		// Convert the object path to utf8 so that whether TCHAR is UTF8 or UTF16 does not alter the hash.
		TUtf8StringBuilder<1024> Utf8ObjectPath(InPlace, InMaterial->GetPathName());

		TUtf8StringBuilder<128> Utf8MaskChannelName(InPlace, InMaskChannelName.ToString());

		// Chooses blend mode based on highest requirement, ie. Transparent > Masked
		InBlendMode = GetTargetBlendMode(InMaterial->GetBlendMode(), InBlendMode);

		const FMaterialShadingModelField ShadingModel = InMaterial->GetShadingModels();
		const EMaterialShadingModel FirstShadingModel = ShadingModel.GetFirstShadingModel();

		FBlake3 Builder;
		Builder.Update(Utf8ObjectPath.GetData(), Utf8ObjectPath.Len() * sizeof(UTF8CHAR));
		Builder.Update(Utf8MaskChannelName.GetData(), Utf8ObjectPath.Len() * sizeof(UTF8CHAR));
		Builder.Update(&InBlendMode, sizeof(EBlendMode));
		Builder.Update(&FirstShadingModel, sizeof(EMaterialShadingModel));

		FBlake3Hash Hash = Builder.Finalize();
		const uint32* HashBytes = (uint32*)Hash.GetBytes();

		return HashCombineFast(HashBytes[0], InSeed);
	}

	UActorComponent* FindOrAddComponent(const TSubclassOf<UActorComponent>& InComponentClass, AActor* InActor)
	{
		const UClass* ComponentClass = InComponentClass.Get();	
		if (!InActor || !ComponentClass)
		{
			return nullptr;
		}

		UActorComponent* Component = InActor->FindComponentByClass(InComponentClass);

        // If no existing component is found, create it only if the package the actor is in isn't being saved
		if (!Component && !UE::IsSavingPackage(InActor))
		{
#if WITH_EDITOR
			InActor->Modify(/*bAlwaysMarkDirty*/false);
#endif

			// Construct the new component and attach as needed
			Component = NewObject<UActorComponent>(InActor
				, ComponentClass
				, MakeUniqueObjectName(InActor, ComponentClass)
				, RF_Transactional);

			Component->CreationMethod = EComponentCreationMethod::Instance;

			// Add to SerializedComponents array so it gets saved
			InActor->AddInstanceComponent(Component);
			Component->OnComponentCreated();
			Component->RegisterComponent();

#if WITH_EDITOR
			// Rerun construction scripts
			InActor->RerunConstructionScripts();
#endif

			// Broadcast component creation to let other systems know about it
			USceneComponent::MarkRenderStateDirtyEvent.Broadcast(*Component);
		}

		return Component;
	}

	void RemoveComponentByInterface(const TSubclassOf<UInterface>& InInterfaceType, const AActor* InActor)
	{
		check(InActor);
		
		TArray<UActorComponent*> ComponentsWithInterface = InActor->GetComponentsByInterface(InInterfaceType);
		for (UActorComponent* Component : ComponentsWithInterface)
		{
			Component->DestroyComponent();
		}
	}

	FString GetGeneratedMaterialPath()
	{
		static FString GeneratedMaterialPath =
			FString::Printf(
				TEXT("/Game/%s/%s/"),
				FPackagePath::GetExternalObjectsFolderName(),
				TEXT("MaterialPermutations"));

		return GeneratedMaterialPath;
	}

	FString GetBlendModeString(const EBlendMode InBlendMode)
	{
		static TMap<EBlendMode, FString> BlendModeValueStrings = {
			{ EBlendMode::BLEND_Opaque, TEXT("Opaque") },
			{ EBlendMode::BLEND_Masked, TEXT("Masked") },
			{ EBlendMode::BLEND_Translucent, TEXT("Translucent") },
			{ EBlendMode::BLEND_Additive, TEXT("Additive") },
			{ EBlendMode::BLEND_Modulate, TEXT("Modulate") },
			{ EBlendMode::BLEND_AlphaComposite, TEXT("AlphaComposite") },
			{ EBlendMode::BLEND_AlphaHoldout, TEXT("AlphaHoldout") }
		};

		if (FString* FoundValue = BlendModeValueStrings.Find(InBlendMode))
		{
			return *FoundValue;
		}

		return TEXT("Invalid");
	}

	EBlendMode GetTargetBlendMode(
		const EBlendMode InFromMaterial
		, const EBlendMode InRequired)
	{
		const int32 FromMaterial = InFromMaterial;
		const int32 Required = InRequired;
		int32 Max = FMath::Max(FromMaterial, Required);
		Max = FMath::Min(EBlendMode::BLEND_MAX - 1, Max);
		return static_cast<EBlendMode>(Max);
	}

	UMaterialInterface* GetNonTransientParentMaterial(const UMaterialInstance* InMaterialInstance)
	{
		// Need to create a new asset based material with the correct blendmode, MID's are by definition not assets, so get parent
		TObjectPtr<UMaterialInterface> ParentAsset = Cast<UMaterialInstance>(InMaterialInstance)->Parent;
		while (ParentAsset && !ParentAsset->IsAsset())
		{
			if (const UMaterialInstance* ParentAsMaterialInstance = Cast<UMaterialInstance>(ParentAsset))
			{
				ParentAsset = ParentAsMaterialInstance->Parent;
				continue;
			}
			
			break;
		}

		if (!ParentAsset)
		{
			UE_LOGF(LogAvaMask, Warning, "Asset based parent material could not be found for '%ls', returning nullptr.", *InMaterialInstance->GetName());
			return nullptr;
		}

		return ParentAsset;
	}

#if WITH_EDITOR
	void SetOutputProcessor(TNotNull<UMaterialInterface*> InMaterial, UMaterialFunctionInterface* InOutputProcessor, EBlendMode InBlendMode)
	{
		UDMMaterialProperty* MaterialProperty = GetMaterialPropertyForBlendMode(InMaterial, InBlendMode);
		if (!MaterialProperty)
		{
			return;
		}

		const UMaterialFunctionInterface* CurrentOutputProcessor = MaterialProperty->GetOutputProcessor();
		if (CurrentOutputProcessor != InOutputProcessor)
		{
			MaterialProperty->SetOutputProcessor(InOutputProcessor);
		}
	}

	UMaterialFunctionInterface* GetOutputProcessor(TNotNull<UMaterialInterface*> InMaterial, EBlendMode InBlendMode)
	{
		if (UDMMaterialProperty* MaterialProperty = GetMaterialPropertyForBlendMode(InMaterial, InBlendMode))
		{
			return MaterialProperty->GetOutputProcessor();
		}
		return nullptr;
	}
#endif
}
