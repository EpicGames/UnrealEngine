// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "Engine/EngineTypes.h"
#include "GeometryMaskTypes.h"
#include "Materials/MaterialParameters.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTypeTraits.h"

class UMaterialFunctionInterface;
class UDynamicMaterialInstance;
class UMaterialInstance;

namespace UE::AvaMask::Internal
{
	/** Note that this assumes the parameters have the same name for all materials, which they should if using the provided material function. */
	static FMaterialParameterInfo MaskTexturesParameterInfo = { TEXT("Mask_Textures") };
	static FMaterialParameterInfo MaskIndicesParameterInfo = { TEXT("Mask_TextureIndexVector") };
	static FMaterialParameterInfo BaseOpacityParameterInfo = { TEXT("Mask_BaseOpacity") };
	static FMaterialParameterInfo InvertParameterInfo = { TEXT("Mask_Invert") };
	static FMaterialParameterInfo PaddingParameterInfo = { TEXT("Mask_Padding") };
	static FMaterialParameterInfo FeatherParameterInfo = { TEXT("Mask_Feather") };

	static FName HandleTag = TEXT("Modifier.Mask2D");

	/** Utility function, will set "OtherActor" if the components owner differs from the provided one. */
	FSoftComponentReference MakeComponentReference(const AActor* InOwner, const UActorComponent* InComponent);

	/** Creates a unique (deterministic) key based on the provided parameters. */
	uint32 MakeMaterialInstanceKey(const UMaterialInterface* InMaterial, const FName InMaskChannelName, EBlendMode InBlendMode, const int32 InSeed = 128);

	UActorComponent* FindOrAddComponent(const TSubclassOf<UActorComponent>& InComponentClass, AActor* InActor);

	template<
		typename InComponentClass
		UE_REQUIRES(std::is_base_of_v<UActorComponent, InComponentClass>)>
	InComponentClass* FindOrAddComponent(AActor* InActor)
	{
		return Cast<InComponentClass>(FindOrAddComponent(InComponentClass::StaticClass(), InActor));
	}

	void RemoveComponentByInterface(const TSubclassOf<UInterface>& InInterfaceType, const AActor* InActor);

	FString GetGeneratedMaterialPath();

	FString GetBlendModeString(const EBlendMode InBlendMode);

	/** Returns whichever blend mode satisfies the requirement AND user specified blend mode */
	EBlendMode GetTargetBlendMode(const EBlendMode InFromMaterial, const EBlendMode InRequired);

	UMaterialInterface* GetNonTransientParentMaterial(const UMaterialInstance* InMaterialInstance);

#if WITH_EDITOR
	/** Sets the output processor for Material Designer  materials */
	void SetOutputProcessor(TNotNull<UMaterialInterface*> InMaterial, UMaterialFunctionInterface* InOutputProcessor, EBlendMode InBlendMode);

	/** Gets the output process from Material Designer materials */
	UMaterialFunctionInterface* GetOutputProcessor(TNotNull<UMaterialInterface*> InMaterial, EBlendMode InBlendMode);
#endif
}
