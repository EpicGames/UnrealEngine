// Copyright Epic Games, Inc. All Rights Reserved.

#include "Handling/AvaDesignedMaterialHandle.h"

#include "AvaMaskMaterialReference.h"
#include "AvaMaskSettings.h"
#include "Material/DynamicMaterialInstance.h"

#if WITH_EDITOR
#include "Model/DynamicMaterialModel.h"
#endif

FAvaDesignedMaterialHandle::FAvaDesignedMaterialHandle(const TWeakObjectPtr<UDynamicMaterialInstance>& InWeakDesignedMaterial)
	: FAvaMaterialInstanceHandle(InWeakDesignedMaterial)
	, WeakDesignedMaterial(InWeakDesignedMaterial)
{
}

void FAvaDesignedMaterialHandle::CopyParametersFrom(UMaterialInstance* InSourceMaterial)
{
	Super::CopyParametersFrom(InSourceMaterial);
	
	if (UDynamicMaterialInstance* MaterialInstance = WeakDesignedMaterial.Get())
	{
		MaterialInstance->CopyInterpParameters(InSourceMaterial);
	}
}

bool FAvaDesignedMaterialHandle::IsValid() const
{
	return Super::IsValid() && WeakDesignedMaterial.IsValid();
}

bool FAvaDesignedMaterialHandle::IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag)
{
	return !!InInstance.GetTypedObject<UDynamicMaterialInstance>();
}
