// Copyright Epic Games, Inc. All Rights Reserved.

#include "Handling/AvaHandleUtilities.h"

#include "Engine/Engine.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Handling/AvaObjectHandleSubsystem.h"

namespace UE::Ava::Internal
{
	UAvaObjectHandleSubsystem* GetObjectHandleSubsystem()
	{
		return GEngine->GetEngineSubsystem<UAvaObjectHandleSubsystem>();
	}

	bool IsDefaultMaterial(UMaterialInterface* InMaterial)
	{
		if (!InMaterial)
		{
			return false;
		}

		if (InMaterial->GetPathName() == UAvaObjectHandleSubsystem::DefaultMaterialPath)
		{
			return true;
		}

		if (const UMaterialInterface* BaseMaterial = InMaterial->GetBaseMaterial())
		{
			if (BaseMaterial->GetPathName() == UAvaObjectHandleSubsystem::DefaultMaterialPath)
			{
				return true;
			}
		}

		return false;
	}
}
