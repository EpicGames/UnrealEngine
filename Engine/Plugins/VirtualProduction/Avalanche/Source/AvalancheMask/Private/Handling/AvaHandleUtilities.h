// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PrimitiveComponent.h"
#include "IAvaMaskMaterialHandle.h"
#include "IAvaObjectHandle.h"
#include "StructUtils/StructView.h"
#include "Templates/UnrealTypeTraits.h"

class UAvaObjectHandleSubsystem;

namespace UE::Ava::Internal
{
	UAvaObjectHandleSubsystem* GetObjectHandleSubsystem();

	bool IsDefaultMaterial(UMaterialInterface* InMaterial);

	template <typename HandleType
		UE_REQUIRES(std::is_base_of_v<IAvaObjectHandle, HandleType>)>
	bool IsHandleValid(const TSharedPtr<HandleType>& InHandle)
	{
		return InHandle.IsValid() && InHandle->IsValid();
	}

	template <typename HandleType
		UE_REQUIRES(std::is_base_of_v<IAvaObjectHandle, HandleType>)>
	bool IsHandleValid(const TSharedPtr<HandleType>* InHandle)
	{
		return InHandle && IsHandleValid(*InHandle);
	}
	
	template <typename KeyType, typename HandleType, typename AllocatorType, typename KeyFuncs, typename InitKeyType, typename LambdaType
		UE_REQUIRES(std::is_base_of_v<IAvaObjectHandle, HandleType>)>
	TSharedPtr<HandleType>& FindOrAddHandleByLambda(TMap<KeyType, TSharedPtr<HandleType>, AllocatorType, KeyFuncs>& Map, InitKeyType&& Key, LambdaType Lambda)
	{
		uint32 KeyHash = KeyFuncs::GetKeyHash(Key);
		if (TSharedPtr<HandleType>* Found = Map.FindByHash(KeyHash, Key))
		{
			// Only return as found if both the SharedPtr and Handle itself are valid
			if (IsHandleValid(Found))
			{
				return *Found;
			}
		}

		TSharedPtr<HandleType> NewElement = Invoke(Lambda);
		return Map.EmplaceByHash(KeyHash, Forward<InitKeyType>(Key), MoveTemp(NewElement));
	}

	template <typename ParentHandleDataType
		UE_REQUIRES(TModels_V<CStaticStructProvider, ParentHandleDataType>)>
	FStructView FindOrAddMaterialHandleData(
		ParentHandleDataType* InParentHandleData
		, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle
		, const int32 InSlotIdx)
	{
		if (InParentHandleData && IsHandleValid(InMaterialHandle))
		{
			if (FInstancedStruct* FoundData = InParentHandleData->MaterialHandleData.Find(InSlotIdx))
			{
				if (FoundData->StaticStruct() == InMaterialHandle->GetDataStructType())
				{
					// Already has the data, of the correct type
					return *FoundData;
				}
			}
			
			return InParentHandleData->MaterialHandleData.Add(InSlotIdx, InMaterialHandle->MakeDataStruct());
		}

		return nullptr;
	}

	template <typename ComponentType
		UE_REQUIRES(std::is_base_of_v<UActorComponent, ComponentType>)>
	int32 GetComponents(const AActor* InActor, TSet<ComponentType*>& OutComponents, const bool bInIncludeInstanceComponents = true)
	{
		TArray<ComponentType*> Components;
		InActor->GetComponents<ComponentType>(Components);
		
		OutComponents.Append(Components);
		
		if (bInIncludeInstanceComponents)
		{
			for (UActorComponent* Component : InActor->GetInstanceComponents())
			{
				if (Component && Component->IsA<ComponentType>())
				{
					OutComponents.Add(Cast<ComponentType>(Component));
				}
			}			
		}
		
		return OutComponents.Num();
	}
}
