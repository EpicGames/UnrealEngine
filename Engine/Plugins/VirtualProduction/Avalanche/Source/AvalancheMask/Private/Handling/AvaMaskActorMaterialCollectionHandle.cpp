// Copyright Epic Games, Inc. All Rights Reserved.

#include "Handling/AvaMaskActorMaterialCollectionHandle.h"

#include "AvaHandleUtilities.h"
#include "AvaMaskMaterialAssignmentObserver.h"
#include "AvaMaskMaterialReference.h"
#include "AvaMaskUtilities.h"
#include "AvaObjectHandleSubsystem.h"
#include "Components/ActorComponent.h"
#include "Components/PrimitiveComponent.h"
#include "GameFramework/Actor.h"
#include "IAvaMaskMaterialHandle.h"
#include "Materials/MaterialInterface.h"

namespace UE::AvaMask::Private
{
	template <typename RangeType, typename RangeRefType>
	void RefreshComponentMaterials(
		const AActor* InActor
		, const RangeType& InComponents
		, const RangeRefType& InComponentRefs
		, TMap<FSoftComponentReference, TBitArray<>>& OutOccupiedMaterialSlots
		, TMap<FSoftComponentReference, TArray<TWeakObjectPtr<UMaterialInterface>>>& OutMaterials
		, TMap<FSoftComponentReference, TArray<TSharedPtr<IAvaMaskMaterialHandle>>>& OutMaterialHandles)
	{
		UAvaObjectHandleSubsystem* HandleSubsystem = UE::Ava::Internal::GetObjectHandleSubsystem();
		if (!HandleSubsystem)
		{
			return;
		}

		for (int32 ComponentIdx = 0; ComponentIdx < InComponents.Num(); ++ComponentIdx)
		{
			UActorComponent* Component = InComponents[ComponentIdx];
			
			if (const UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component))
			{
				FSoftComponentReference ComponentReference = InComponentRefs[ComponentIdx];
				TBitArray<>& OccupiedComponentMaterials = OutOccupiedMaterialSlots.Emplace(ComponentReference);
				TArray<TWeakObjectPtr<UMaterialInterface>>& ComponentMaterials = OutMaterials.Emplace(ComponentReference);
				TArray<TSharedPtr<IAvaMaskMaterialHandle>>& ComponentMaterialHandles = OutMaterialHandles.Emplace(ComponentReference);

				const int32 NumComponentMaterials = PrimitiveComponent->GetNumMaterials();
				OccupiedComponentMaterials.Init(false, NumComponentMaterials);
				ComponentMaterials.Init(nullptr, NumComponentMaterials);
				ComponentMaterialHandles.Init(nullptr, NumComponentMaterials);

				for (int32 SlotIdx = 0; SlotIdx < NumComponentMaterials; ++SlotIdx)
				{
					UMaterialInterface* Material = PrimitiveComponent->GetMaterial(SlotIdx);
					OccupiedComponentMaterials[SlotIdx] = Material != nullptr;
					if (Material == nullptr)
					{
						ComponentMaterials[SlotIdx] = Material;
						ComponentMaterialHandles[SlotIdx] = nullptr;
					}
					else
					{
						ComponentMaterials[SlotIdx] = Material;
						ComponentMaterialHandles[SlotIdx] = HandleSubsystem->MakeMaterialHandle<IAvaMaskMaterialHandle>(Material, UE::AvaMask::Internal::HandleTag);
					}
				}
			}
		}
	};
}

FAvaMaskActorMaterialCollectionHandle::FAvaMaskActorMaterialCollectionHandle(AActor* InActor)
	: WeakActor(InActor)
{
	// Refresh here to make GetNumMaterials etc. valid
	RefreshComponents();
	RefreshMaterials();
}

TArray<TObjectPtr<UMaterialInterface>> FAvaMaskActorMaterialCollectionHandle::GetMaterials()
{
	TArray<TObjectPtr<UMaterialInterface>> Materials;
	Materials.Reserve(WeakMaterials.Num());
	
	for (TPair<FSoftComponentReference, TArray<TWeakObjectPtr<UMaterialInterface>>>& ComponentMaterials : WeakMaterials)
	{
		for (int32 SlotIdx = 0; SlotIdx < ComponentMaterials.Value.Num(); ++SlotIdx)
		{
			if (UMaterialInterface* Material = ComponentMaterials.Value[SlotIdx].Get())
			{
				Materials.Emplace(Material);
			}
			else if (!ComponentMaterials.Value[SlotIdx].IsExplicitlyNull())
			{
				// Material is invalid and wasn't intentionally null, refresh materials from host actor
				if (RefreshMaterials())
				{
					return GetMaterials();	
				}

				return { };
			}
		}
	}
	return Materials;
}

TArray<TSharedPtr<IAvaMaskMaterialHandle>> FAvaMaskActorMaterialCollectionHandle::GetMaterialHandles()
{
	// Effectively performs a pre-pass and refreshes if needed
	GetMaterials();

	TArray<TSharedPtr<IAvaMaskMaterialHandle>> MtlHandles;
	MtlHandles.Reserve(MaterialHandles.Num());
	
	for (TPair<FSoftComponentReference, TArray<TSharedPtr<IAvaMaskMaterialHandle>>>& ComponentMaterialHandles : MaterialHandles)
	{
		for (int32 SlotIdx = 0; SlotIdx < ComponentMaterialHandles.Value.Num(); ++SlotIdx)
		{
			if (TSharedPtr<IAvaMaskMaterialHandle>& MaterialHandle = ComponentMaterialHandles.Value[SlotIdx])
			{
				if (!UE::Ava::Internal::IsHandleValid(ComponentMaterialHandles.Value[SlotIdx]))
				{
					if (RefreshMaterials())
					{
						return GetMaterialHandles();
					}

					return { };
				}
				
				MtlHandles.Emplace(MaterialHandle);
			}
		}
	}
	
	return MtlHandles;
}

int32 FAvaMaskActorMaterialCollectionHandle::GetNumMaterials() const
{
	return WeakMaterials.Num();
}

void FAvaMaskActorMaterialCollectionHandle::SetMaterial(
	const FSoftComponentReference& InComponent
	, const int32 InSlotIdx
	, UMaterialInterface* InMaterial)
{
	if (AActor* Actor = WeakActor.Get())
	{
		// Effectively performs a pre-pass and refreshes if needed
		GetMaterials();

		if (UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(InComponent.GetComponent(Actor)))
		{
			if (!ensure(InSlotIdx < Component->GetNumMaterials()))
			{
				UE_LOGF(LogAvaMask, Error, "SlotIdx (%u) was greater than the number of materials (%u)", InSlotIdx, Component->GetNumMaterials());
				return;
			}
			
			Component->SetMaterial(InSlotIdx, InMaterial);
		}
	}
}

void FAvaMaskActorMaterialCollectionHandle::SetMaterials(
	const TArray<TObjectPtr<UMaterialInterface>>& InMaterials
	, const TBitArray<>& InSetToggle)
{
	check(InMaterials.Num() == InSetToggle.Num());

	if (AActor* Actor = WeakActor.Get())
	{
		// Effectively performs a pre-pass and refreshes if needed
		GetMaterials();

		bool bAnyMaterialWasSet = false;
		
		int32 MtlIdx = 0;
		for (TPair<FSoftComponentReference, TArray<TWeakObjectPtr<UMaterialInterface>>>& ComponentMaterials : WeakMaterials)
		{
			if (UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(ComponentMaterials.Key.GetComponent(Actor)))
			{
				for (int32 SlotIdx = 0; SlotIdx < FMath::Min(Component->GetNumMaterials(), ComponentMaterials.Value.Num()); ++SlotIdx)
				{
					if (!InMaterials.IsValidIndex(MtlIdx + 1))
					{
						UE_LOGF(LogAvaMask, Error, "MtlIdx (%u) out of range (%u maaterials)", MtlIdx + 1, InMaterials.Num())
						return;
					}

					if (!InSetToggle[SlotIdx])
					{
						continue;
					}

					UMaterialInterface* NewMaterial = InMaterials[++MtlIdx];

					ComponentMaterials.Value[SlotIdx] = NewMaterial;
					Component->SetMaterial(SlotIdx, NewMaterial);
					bAnyMaterialWasSet = true;
				}

				if (bAnyMaterialWasSet)
				{
					Component->MarkRenderStateDirty();
					bAnyMaterialWasSet = false; // Reset for next componentS
				}
			}
		}
	}
}

void FAvaMaskActorMaterialCollectionHandle::ForEachMaterial(
	TFunctionRef<bool(
		const FSoftComponentReference& InComponent
		, const int32 InIdx
		, UMaterialInterface* InMaterial)>	InFunction)
{
	if (GetActor())
	{
		// Effectively performs a pre-pass and refreshes if needed
		GetMaterials();

		for (TPair<FSoftComponentReference, TArray<TWeakObjectPtr<UMaterialInterface>>>& ComponentMaterials : WeakMaterials)
		{
			for (int32 SlotIdx = 0; SlotIdx < ComponentMaterials.Value.Num(); ++SlotIdx)
			{
				if (UMaterialInterface* Material = ComponentMaterials.Value[SlotIdx].Get())
				{
					if (!InFunction(ComponentMaterials.Key, SlotIdx, Material))
					{
						return;
					}
				}
			}
		}
	}
}

void FAvaMaskActorMaterialCollectionHandle::ForEachMaterialHandle(TFunctionRef<bool(const FSoftComponentReference& InComponent, const int32 InIdx, const bool bInIsSlotOccupied, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)> InFunction)
{
	if (GetActor())
	{
		// Effectively refreshes material handles if necessary
		GetMaterialHandles();

		TGuardValue<bool> IteratingGuard(bIsIteratingMaterials, true);
		for (TPair<FSoftComponentReference, TArray<TSharedPtr<IAvaMaskMaterialHandle>>>& ComponentMaterialHandles : MaterialHandles)
		{
			for (int32 SlotIdx = 0; SlotIdx < ComponentMaterialHandles.Value.Num(); ++SlotIdx)
			{
				if (!InFunction(ComponentMaterialHandles.Key, SlotIdx, IsSlotOccupied(ComponentMaterialHandles.Key, SlotIdx), ComponentMaterialHandles.Value[SlotIdx]))
				{
					return;
				}
			}
		}
	}
}

void FAvaMaskActorMaterialCollectionHandle::MapEachMaterial(
	TFunctionRef<UMaterialInterface*(const FSoftComponentReference& InComponent, const int32 InIdx, UMaterialInterface*
		InMaterial)> InFunction)
{
	// Effectively performs a pre-pass and refreshes if needed
	GetMaterials();

	TArray<TObjectPtr<UMaterialInterface>> MappedMaterials;
	MappedMaterials.Reserve(WeakMaterials.Num());

	TBitArray<> SetFlags;
	SetFlags.Init(false, MappedMaterials.Num());

	for (TPair<FSoftComponentReference, TArray<TWeakObjectPtr<UMaterialInterface>>>& ComponentMaterials : WeakMaterials)
	{
		for (int32 SlotIdx = 0; SlotIdx < ComponentMaterials.Value.Num(); ++SlotIdx)
		{
			if (UMaterialInterface* Material = ComponentMaterials.Value[SlotIdx].Get())
			{
				UMaterialInterface* MappedMaterial = InFunction(ComponentMaterials.Key, SlotIdx, Material);
				MappedMaterials[SlotIdx] = MappedMaterial;
				SetFlags[SlotIdx] = MappedMaterial != nullptr;
			}
		}
	}

	SetMaterials(MappedMaterials, SetFlags);
}

void FAvaMaskActorMaterialCollectionHandle::MapEachMaterialHandle(TFunctionRef<TSharedPtr<IAvaMaskMaterialHandle>(const FSoftComponentReference& InComponent, const int32 InIdx, const bool bInIsSlotOccupied, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)> InFunction)
{
	if (GetActor())
	{
		// Effectively performs a pre-pass and refreshes if needed
		GetMaterialHandles();

		TArray<TObjectPtr<UMaterialInterface>> MappedMaterials;
		TBitArray<> SetFlags;

		TGuardValue<bool> IteratingGuard(bIsIteratingMaterials, true);
		for (TPair<FSoftComponentReference, TArray<TSharedPtr<IAvaMaskMaterialHandle>>>& ComponentMaterialHandles : MaterialHandles)
		{
			for (int32 SlotIdx = 0; SlotIdx < ComponentMaterialHandles.Value.Num(); ++SlotIdx)
			{
				if (TSharedPtr<IAvaMaskMaterialHandle>& MaterialHandle = ComponentMaterialHandles.Value[SlotIdx])
				{
					const TSharedPtr<IAvaMaskMaterialHandle> MappedMaterialHandle = InFunction(ComponentMaterialHandles.Key, SlotIdx, IsSlotOccupied(ComponentMaterialHandles.Key, SlotIdx), MaterialHandle);
					UMaterialInterface* RemappedMaterial = nullptr;
					if (MappedMaterialHandle.IsValid())
					{
						RemappedMaterial = MappedMaterialHandle->GetMaterial();
					}
					else
					{
						RemappedMaterial = MaterialHandle->GetMaterial();
					}

					MappedMaterials.Add(RemappedMaterial);
					SetFlags.Add(RemappedMaterial != nullptr);
				}
			}
		}

		SetMaterials(MappedMaterials, SetFlags);
	}
}

bool FAvaMaskActorMaterialCollectionHandle::SaveOriginalState(const FStructView& InHandleData)
{
	if (!TAvaMaskMaterialCollectionHandle<FAvaMaskActorMaterialCollectionHandleData>::SaveOriginalState(InHandleData))
	{
		return false;
	}

	return false;
}

bool FAvaMaskActorMaterialCollectionHandle::ApplyOriginalState(const FStructView& InHandleData)
{
	if (!TAvaMaskMaterialCollectionHandle<FAvaMaskActorMaterialCollectionHandleData>::ApplyOriginalState(InHandleData))
	{
		return false;
	}

	return false;
}

bool FAvaMaskActorMaterialCollectionHandle::IsValid() const
{
	return WeakActor.IsValid();
}

bool FAvaMaskActorMaterialCollectionHandle::IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag)
{
	return InTag == UE::AvaMask::Internal::HandleTag && InInstance.GetTypedObject<AActor>();
}

FStructView FAvaMaskActorMaterialCollectionHandle::GetMaterialHandleData(
	FHandleData* InParentHandleData
	, const FSoftComponentReference& InComponent
	, const int32 InSlotIdx)
{
	if (InParentHandleData)
	{
		const FAvaMask2DComponentMaterialPath ComponentMaterialPath{ InComponent, InSlotIdx };
		if (InParentHandleData->ComponentMaterialData.Contains(ComponentMaterialPath))
		{
			return InParentHandleData->ComponentMaterialData[ComponentMaterialPath]; 
		}
	}

	return nullptr;
}

FStructView FAvaMaskActorMaterialCollectionHandle::GetOrAddMaterialHandleData(
	FHandleData* InParentHandleData
	, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle
	, const FSoftComponentReference& InComponent
	, const int32 InSlotIdx)
{
	if (InParentHandleData)
	{
		const FAvaMask2DComponentMaterialPath ComponentMaterialPath{ InComponent, InSlotIdx };
		return InParentHandleData->ComponentMaterialData.FindOrAdd(ComponentMaterialPath, InMaterialHandle->MakeDataStruct());
	}

	return nullptr;
}

bool FAvaMaskActorMaterialCollectionHandle::IsSlotOccupied(
	const FSoftComponentReference& InComponent
	, const int32 InSlotIdx)
{
	if (TBitArray<>* OccupiedComponentMaterialSlots = OccupiedMaterialSlots.Find(InComponent))
	{
		if (OccupiedComponentMaterialSlots->IsValidIndex(InSlotIdx))
		{
			return (*OccupiedComponentMaterialSlots)[InSlotIdx];
		}
	}

	return false;
}

void FAvaMaskActorMaterialCollectionHandle::OnMaterialAssignmentChanged(UPrimitiveComponent* InPrimitiveComponent, const int32 InPrevSlotNum, const int32 InNewSlotNum, const TArray<int32>& InChangedMaterialSlots)
{
	check(InPrimitiveComponent);

	// todo: we could either ignore or delay material assignment observer during ApplyModifiedState.
	if (!bIsIteratingMaterials)
	{
		RefreshMaterials();
	}

	if (const AActor* Actor = GetActor())
	{
		const FSoftComponentReference ComponentReference = UE::AvaMask::Internal::MakeComponentReference(Actor, InPrimitiveComponent);
		const TArray<TSharedPtr<IAvaMaskMaterialHandle>>* ComponentMaterialHandles = MaterialHandles.Find(ComponentReference);
		if (!ComponentMaterialHandles)
		{
			UE_LOGF(LogAvaMask, Error, "MaterialHandles not found for component: %ls", *InPrimitiveComponent->GetName());
			return;
		}

		TArray<TSharedPtr<IAvaMaskMaterialHandle>> ChangedHandles;
		ChangedHandles.Reserve(InChangedMaterialSlots.Num());

		for (int32 ChangedMaterialIdx = 0; ChangedMaterialIdx < InChangedMaterialSlots.Num(); ++ChangedMaterialIdx)
		{
			if (!ComponentMaterialHandles->IsValidIndex(ChangedMaterialIdx))
			{
				UE_LOGF(LogAvaMask, Error, "No MaterialHandle found for component: %ls at slot: %i", *InPrimitiveComponent->GetName(), ChangedMaterialIdx);
				ChangedHandles.Emplace(nullptr); // Still need something at this index
			}
			else
			{
				ChangedHandles.Emplace((*ComponentMaterialHandles)[ChangedMaterialIdx]);
			}
		}

		OnSourceMaterialsChanged().ExecuteIfBound(InPrimitiveComponent, ChangedHandles);
	}
}

AActor* FAvaMaskActorMaterialCollectionHandle::GetActor() const
{
	if (AActor* Actor = WeakActor.Get())
	{
		return Actor;
	}
	
	return nullptr;
}

TConstArrayView<UPrimitiveComponent*> FAvaMaskActorMaterialCollectionHandle::GetPrimitiveComponents()
{
	return LastResolvedComponents;
}

void FAvaMaskActorMaterialCollectionHandle::RefreshComponents()
{
	WeakComponents.Reset();
	LastResolvedComponents.Reset();
	
	if (const AActor* Actor = GetActor())
	{
		TSet<UPrimitiveComponent*> PrimitiveComponents;
		const int32 NumComponentsFound = UE::Ava::Internal::GetComponents<UPrimitiveComponent>(Actor, PrimitiveComponents, true);

		WeakComponents.Reserve(NumComponentsFound);
		LastResolvedComponents.Reserve(NumComponentsFound);

		for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
		{
			WeakComponents.Emplace(UE::AvaMask::Internal::MakeComponentReference(Actor, PrimitiveComponent));
			LastResolvedComponents.Emplace(PrimitiveComponent);
		}
	}
}

bool FAvaMaskActorMaterialCollectionHandle::RefreshMaterials()
{
	ensureMsgf(!bIsIteratingMaterials, TEXT("Refresh should not happen when iterating over material handles"));

	const int32 ProbableNumMaterials = FMath::Max(1, GetNumMaterials());

	OccupiedMaterialSlots.Reset();
	OccupiedMaterialSlots.Reserve(ProbableNumMaterials);
	
	WeakMaterials.Reset();
	WeakMaterials.Reserve(ProbableNumMaterials);

	MaterialHandles.Reset();
	MaterialHandles.Reserve(ProbableNumMaterials);

	MaterialAssignmentObservers.Reset();
	MaterialAssignmentObservers.Reserve(ProbableNumMaterials);

	UAvaObjectHandleSubsystem* HandleSubsystem = UE::Ava::Internal::GetObjectHandleSubsystem();
	if (!HandleSubsystem)
	{
		UE_LOGF(LogAvaMask, Error, "ObjectHandleSubsystem was invalid");
		return false;
	}

	const TConstArrayView<UPrimitiveComponent*> PrimitiveComponents = GetPrimitiveComponents();
	TArray<FSoftComponentReference>& ComponentRefs = WeakComponents;

	for (int32 ComponentIdx = 0; ComponentIdx < PrimitiveComponents.Num(); ++ComponentIdx)
	{
		UPrimitiveComponent* PrimitiveComponent = PrimitiveComponents[ComponentIdx];
		FSoftComponentReference ComponentReference = ComponentRefs[ComponentIdx];

		TBitArray<>& OccupiedComponentMaterials = OccupiedMaterialSlots.Emplace(ComponentReference);
		TArray<TWeakObjectPtr<UMaterialInterface>>& ComponentMaterials = WeakMaterials.Emplace(ComponentReference);
		TArray<TSharedPtr<IAvaMaskMaterialHandle>>& ComponentMaterialHandles = MaterialHandles.Emplace(ComponentReference);

		const TUniquePtr<FAvaMaskMaterialAssignmentObserver>& MaterialAssignmentObserver = MaterialAssignmentObservers.Emplace(ComponentReference, MakeUnique<FAvaMaskMaterialAssignmentObserver>(PrimitiveComponent));

		const int32 NumComponentMaterials = PrimitiveComponent->GetNumMaterials();
		OccupiedComponentMaterials.Init(false, NumComponentMaterials);
		ComponentMaterials.Init(nullptr, NumComponentMaterials);
		ComponentMaterialHandles.Init(nullptr, NumComponentMaterials);

		for (int32 SlotIdx = 0; SlotIdx < NumComponentMaterials; ++SlotIdx)
		{
			UMaterialInterface* Material = PrimitiveComponent->GetMaterial(SlotIdx);
			OccupiedComponentMaterials[SlotIdx] = Material != nullptr;
			if (Material == nullptr)
			{
				ComponentMaterials[SlotIdx] = Material;
				ComponentMaterialHandles[SlotIdx] = nullptr;
			}
			else
			{
				ComponentMaterials[SlotIdx] = Material;
				ComponentMaterialHandles[SlotIdx] = HandleSubsystem->MakeMaterialHandle<IAvaMaskMaterialHandle>(Material, UE::AvaMask::Internal::HandleTag);
			}
		}

		MaterialAssignmentObserver->OnMaterialAssigned().AddRaw(this, &FAvaMaskActorMaterialCollectionHandle::OnMaterialAssignmentChanged);
	}

	return true;
}
