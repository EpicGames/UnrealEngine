// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Engine/EngineTypes.h"
#include "IAvaMaskMaterialCollectionHandle.h"
#include "IAvaObjectHandle.h"
#include "Templates/Function.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "AvaMaskActorMaterialCollectionHandle.generated.h"

class AActor;
class AText3DActor;
class FAvaMaskMaterialAssignmentObserver;
class IAvaMaskMaterialHandle;
class UAvaText3DComponent;
class UMaterialInterface;
class UText3DComponent;
struct FAvaMaskMaterialReference;

USTRUCT()
struct FAvaMaskActorMaterialCollectionHandleData
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FAvaMask2DComponentMaterialPath, FInstancedStruct> ComponentMaterialData;
};

class FAvaMaskActorMaterialCollectionHandle
	: public TAvaMaskMaterialCollectionHandle<FAvaMaskActorMaterialCollectionHandleData>
	, public TSharedFromThis<FAvaMaskActorMaterialCollectionHandle>
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaMaskActorMaterialCollectionHandle, TAvaMaskMaterialCollectionHandle<FAvaMaskActorMaterialCollectionHandleData>);
	
	explicit FAvaMaskActorMaterialCollectionHandle(AActor* InActor);
	virtual ~FAvaMaskActorMaterialCollectionHandle() override = default;

	// ~Begin IAvaMaskMaterialCollectionProvider
	virtual TArray<TObjectPtr<UMaterialInterface>> GetMaterials() override;
	virtual TArray<TSharedPtr<IAvaMaskMaterialHandle>> GetMaterialHandles() override;
	virtual int32 GetNumMaterials() const override;

	virtual void SetMaterial(const FSoftComponentReference& InComponent, const int32 InSlotIdx, UMaterialInterface* InMaterial) override;
	virtual void SetMaterials(const TArray<TObjectPtr<UMaterialInterface>>& InMaterials, const TBitArray<>& InSetToggle) override;
	
	virtual void ForEachMaterial(
		TFunctionRef<bool(
			const FSoftComponentReference& InComponent
			, const int32 InIdx
			, UMaterialInterface* InMaterial)> InFunction) override;

	virtual void ForEachMaterialHandle(
		TFunctionRef<bool(
			const FSoftComponentReference& InComponent
			, const int32 InIdx
			, const bool bInIsSlotOccupied
			, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)> InFunction) override;

	virtual void MapEachMaterial(
		TFunctionRef<UMaterialInterface*(
			const FSoftComponentReference& InComponent
			, const int32 InIdx
			, UMaterialInterface* InMaterial)> InFunction) override;

	virtual void MapEachMaterialHandle(
		TFunctionRef<TSharedPtr<IAvaMaskMaterialHandle>(
			const FSoftComponentReference& InComponent
			, const int32 InIdx
			, const bool bInIsSlotOccupied
			, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)> InFunction) override;
	
	virtual bool SaveOriginalState(const FStructView& InHandleData) override;
	virtual bool ApplyOriginalState(const FStructView& InHandleData) override;

	virtual bool IsValid() const override;
	// ~End IAvaMaskMaterialCollectionProvider

	static bool IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag = NAME_None);

protected:
	// ~Begin TAvaMaskMaterialCollectionHandle
	virtual FStructView GetMaterialHandleData(
		FHandleData* InParentHandleData
		, const FSoftComponentReference& InComponent
		, const int32 InSlotIdx) override;
	
	virtual ::FStructView GetOrAddMaterialHandleData(
		FHandleData* InParentHandleData
		, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle
		, const FSoftComponentReference& InComponent
		, const int32 InSlotIdx) override;
	// ~End TAvaMaskMaterialCollectionHandle

	bool IsSlotOccupied(const FSoftComponentReference& InComponent, const int32 InSlotIdx);

	void OnMaterialAssignmentChanged(UPrimitiveComponent* InPrimitiveComponent, const int32 InPrevSlotNum, const int32 InNewSlotNum, const TArray<int32>& InChangedMaterialSlots);

	AActor* GetActor() const;
	TConstArrayView<UPrimitiveComponent*> GetPrimitiveComponents();

	void RefreshComponents();

	/** Return false if Actor invalid. */
	bool RefreshMaterials();

protected:
	TWeakObjectPtr<AActor> WeakActor;
	TArray<FSoftComponentReference> WeakComponents;
	TArray<UPrimitiveComponent*> LastResolvedComponents;
	TMap<FSoftComponentReference, TBitArray<>> OccupiedMaterialSlots;
	TMap<FSoftComponentReference, TArray<TWeakObjectPtr<UMaterialInterface>>> WeakMaterials;
	TMap<FSoftComponentReference, TArray<TSharedPtr<IAvaMaskMaterialHandle>>> MaterialHandles;
	TMap<FSoftComponentReference, TUniquePtr<FAvaMaskMaterialAssignmentObserver>> MaterialAssignmentObservers;
	bool bIsIteratingMaterials = false;
};
