// Copyright Epic Games, Inc. All Rights Reserved.

#include "Handling/AvaMaskAvaShapeMaterialCollectionHandle.h"

#include "AvaMaskMaterialAssignmentObserver.h"
#include "AvaMaskMaterialReference.h"
#include "AvaMaskUtilities.h"
#include "AvaText3DComponent.h"
#include "Components/DynamicMeshComponent.h"
#include "Components/PrimitiveComponent.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "GameFramework/Actor.h"
#include "Handling/AvaHandleUtilities.h"
#include "Handling/AvaObjectHandleSubsystem.h"
#include "Materials/MaterialInterface.h"
#include "IAvaMaskMaterialHandle.h"
#include "Materials/Material.h"

namespace UE::AvaMask::Private
{
	template <typename PairValueType>
	void RemoveUnused(TMap<int32, PairValueType>& InOutMap, const TSet<int32>& InUsedKeys)
	{
		TSet<int32> Keys;
		InOutMap.GetKeys(Keys);

		TSet<int32> UnusedKeys = Keys;
		UnusedKeys = UnusedKeys.Difference(InUsedKeys);

		for (const int32 UnusedKey : UnusedKeys)
		{
			InOutMap.Remove(UnusedKey);
		}
	}
}

FAvaMaskAvaShapeMaterialCollectionHandle::FAvaMaskAvaShapeMaterialCollectionHandle(AActor* InActor)
	: WeakActor(InActor)
	, WeakComponent(InActor->GetComponentByClass<UAvaShapeDynamicMeshBase>())
{
	// Refresh here to make GetNumMaterials etc. valid
	RefreshMaterials();
}

FInstancedStruct FAvaMaskAvaShapeMaterialCollectionHandle::MakeDataStruct()
{
	FInstancedStruct CollectionData = FInstancedStruct::Make<FHandleData>();
	return CollectionData;
}

TArray<TObjectPtr<UMaterialInterface>> FAvaMaskAvaShapeMaterialCollectionHandle::GetMaterials()
{
	TArray<TObjectPtr<UMaterialInterface>> Materials;
	Materials.Reserve(WeakMaterials.Num());
	
	for (TPair<int32, TWeakObjectPtr<UMaterialInterface>>& MeshMaterial : WeakMaterials)
	{
		if (UMaterialInterface* Material = MeshMaterial.Value.Get())
		{
			Materials.Emplace(Material);
		}
		else if (!MeshMaterial.Value.IsExplicitlyNull())
		{
			// Material is invalid and wasn't intentionally null, refresh materials from host actor
			if (RefreshMaterials())
			{
				return GetMaterials();	
			}

			return { };
		}
	}
	
	return Materials;
}

TArray<TSharedPtr<IAvaMaskMaterialHandle>> FAvaMaskAvaShapeMaterialCollectionHandle::GetMaterialHandles()
{
	// Effectively performs a pre-pass and refreshes if needed
	GetMaterials();

	TArray<TSharedPtr<IAvaMaskMaterialHandle>> MtlHandles;
	MtlHandles.Reserve(MaterialHandles.Num());

	if (MaterialHandles.Num() != WeakMaterials.Num())
	{
		RefreshMaterials();
	}
	
	for (const TPair<int32, TSharedPtr<IAvaMaskMaterialHandle>>& SlotMaterialHandle : MaterialHandles)
	{
		// Account for empty/unoccupied slot
		TSharedPtr<IAvaMaskMaterialHandle> MaterialHandle = SlotMaterialHandle.Value;
		if (!UE::Ava::Internal::IsHandleValid(MaterialHandle))
		{
			if (RefreshMaterials())
			{
				MaterialHandles.GenerateValueArray(MtlHandles);
				break;
			}

			return { };
		}
		
		MtlHandles.Emplace(MaterialHandle);
	}
	
	return MtlHandles;
}

int32 FAvaMaskAvaShapeMaterialCollectionHandle::GetNumMaterials() const
{
	return WeakMaterials.Num();
}

void FAvaMaskAvaShapeMaterialCollectionHandle::SetMaterial(
	const FSoftComponentReference& InComponent
	, const int32 InSlotIdx
	, UMaterialInterface* InMaterial)
{
	if (UAvaShapeDynamicMeshBase* ShapeComponent = GetComponent())
	{
		// Effectively performs a pre-pass and refreshes if needed
		GetMaterials();

		if (!ensure(InSlotIdx < GetNumMaterials()))
		{
			UE_LOGF(LogAvaMask, Error, "SlotIdx (%u) was greater than the number of materials (%u)", InSlotIdx, GetNumMaterials());
			return;
		}

		// If this option is on, the MeshIdx should always be INDEX_NONE
		const bool bUseSingleMaterial = ShapeComponent->GetUsePrimaryMaterialEverywhere();
		const int32 MeshIndex = bUseSingleMaterial ? 0 : InSlotIdx;

		if (ShapeComponent->IsValidMeshIndex(MeshIndex))
		{
			// Parametric materials handle refreshing themselves
			if (!ShapeComponent->IsMaterialType(MeshIndex, EMaterialType::Parametric))
			{
				ShapeComponent->SetMaterial(bUseSingleMaterial ? INDEX_NONE : InSlotIdx, InMaterial);
			}
		}
	}
}

void FAvaMaskAvaShapeMaterialCollectionHandle::SetMaterials(
	const TArray<TObjectPtr<UMaterialInterface>>& InMaterials
	, const TBitArray<>& InSetToggle)
{
	check(InMaterials.Num() == InSetToggle.Num());

	if (UAvaShapeDynamicMeshBase* ShapeComponent = GetComponent())
	{
		// Effectively performs a pre-pass and refreshes if needed
		GetMaterials();

		bool bAnyMaterialWasSet = false;

		for (int32 MeshIdx = 0; MeshIdx < FMath::Min(WeakMaterials.Num(), InMaterials.Num()); ++MeshIdx)
		{
			if (InSetToggle[MeshIdx])
			{
				bAnyMaterialWasSet = true;
				ShapeComponent->SetMaterial(MeshIdx, InMaterials[MeshIdx]);	
			}
		}

		if (bAnyMaterialWasSet)
		{
			ShapeComponent->MarkRenderStateDirty();	
		}
	}
}

void FAvaMaskAvaShapeMaterialCollectionHandle::ForEachMaterial(
	TFunctionRef<bool(
		const FSoftComponentReference& InComponent
		, const int32 InIdx
		, UMaterialInterface* InMaterial)> InFunction)
{
	if (const UAvaShapeDynamicMeshBase* ShapeComponent = GetComponent())
	{
		// Effectively performs a pre-pass and refreshes if needed
		GetMaterials();
		
		const FSoftComponentReference ComponentReference = UE::AvaMask::Internal::MakeComponentReference(ShapeComponent->GetOwner(), ShapeComponent);
		for (TPair<int32, TWeakObjectPtr<UMaterialInterface>>& Material : WeakMaterials)
		{
			if (!InFunction(ComponentReference, Material.Key, Material.Value.Get()))
			{
				return;
			}
		}
	}
}

void FAvaMaskAvaShapeMaterialCollectionHandle::ForEachMaterialHandle(TFunctionRef<bool(const FSoftComponentReference& InComponent, const int32 InIdx, const bool bInIsSlotOccupied, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)> InFunction)
{
	if (const UAvaShapeDynamicMeshBase* ShapeComponent = GetComponent())
	{
		// Effectively refreshes material handles if necessary
		GetMaterialHandles();

		// Handles can be destroyed during the loop, so iterate over a copy and check the validity of each handle along the way
		TMap<int32, TSharedPtr<IAvaMaskMaterialHandle>> MaterialHandlesCopy = MaterialHandles;

		const FSoftComponentReference ComponentReference = UE::AvaMask::Internal::MakeComponentReference(ShapeComponent->GetOwner(), ShapeComponent);
		for (TPair<int32, TSharedPtr<IAvaMaskMaterialHandle>>& MaterialHandle : MaterialHandlesCopy)
		{
			if (!InFunction(ComponentReference, MaterialHandle.Key, IsSlotOccupied(MaterialHandle.Key), MaterialHandle.Value))
			{
				return;
			}
		}
	}
}

void FAvaMaskAvaShapeMaterialCollectionHandle::MapEachMaterial(
	TFunctionRef<UMaterialInterface*(
		const FSoftComponentReference& InComponent
		, const int32 InIdx
		, UMaterialInterface* InMaterial)> InFunction)
{
	if (const UAvaShapeDynamicMeshBase* ShapeComponent = GetComponent())
	{
		// Effectively performs a pre-pass and refreshes if needed
		GetMaterials();

		TArray<TObjectPtr<UMaterialInterface>> MappedMaterials;
		MappedMaterials.Init(nullptr, static_cast<TArray<TObjectPtr<UMaterialInterface>>::SizeType>(EText3DGroupType::TypeCount));

		TBitArray<> SetFlags;
		SetFlags.Init(false, MappedMaterials.Num());

		int32 MeshIdx = 0;
		
		const FSoftComponentReference ComponentReference = UE::AvaMask::Internal::MakeComponentReference(ShapeComponent->GetOwner(), ShapeComponent);
		for (TPair<int32, TWeakObjectPtr<UMaterialInterface>>& Material : WeakMaterials)
		{
			UMaterialInterface* RemappedMaterial = InFunction(ComponentReference, Material.Key, Material.Value.Get());
			MappedMaterials[++MeshIdx] = RemappedMaterial;
			SetFlags[MeshIdx] = RemappedMaterial != nullptr;
		}

		SetMaterials(MappedMaterials, SetFlags);
	}
}

void FAvaMaskAvaShapeMaterialCollectionHandle::MapEachMaterialHandle(TFunctionRef<TSharedPtr<IAvaMaskMaterialHandle>(const FSoftComponentReference& InComponent, const int32 InIdx, const bool bInIsSlotOccupied, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)> InFunction)
{
	if (const UAvaShapeDynamicMeshBase* ShapeComponent = GetComponent())
	{
		// Effectively refreshes material handles if necessary
		GetMaterialHandles();
		
		TArray<TObjectPtr<UMaterialInterface>> MappedMaterials;
		TBitArray<> SetFlags;
		
		const FSoftComponentReference ComponentReference = UE::AvaMask::Internal::MakeComponentReference(ShapeComponent->GetOwner(), ShapeComponent);
		for (TPair<int32, TSharedPtr<IAvaMaskMaterialHandle>>& MaterialHandle : MaterialHandles)
		{
			const TSharedPtr<IAvaMaskMaterialHandle> MappedMaterialHandle = InFunction(ComponentReference, MaterialHandle.Key, IsSlotOccupied(MaterialHandle.Key), MaterialHandle.Value);
			UMaterialInterface* RemappedMaterial = nullptr;
			if (UE::Ava::Internal::IsHandleValid(MappedMaterialHandle))
			{
				RemappedMaterial = MappedMaterialHandle->GetMaterial();
			}
			else
			{
				RemappedMaterial = MaterialHandle.Value->GetMaterial();
			}

			MappedMaterials.Add(RemappedMaterial);
			SetFlags.Add(RemappedMaterial != nullptr);
		}

		SetMaterials(MappedMaterials, SetFlags);
	}
}

bool FAvaMaskAvaShapeMaterialCollectionHandle::IsValid() const
{
	return WeakActor.IsValid() && WeakComponent.IsValid();
}

UAvaShapeDynamicMeshBase* FAvaMaskAvaShapeMaterialCollectionHandle::GetComponent()
{
	if (UAvaShapeDynamicMeshBase* Component = WeakComponent.Get())
	{
		return Component;
	}

	if (const AActor* Actor = WeakActor.Get())
	{
		WeakComponent = Actor->GetComponentByClass<UAvaShapeDynamicMeshBase>();
		if (UAvaShapeDynamicMeshBase* Component = WeakComponent.Get())
		{
			return Component;
		}
	}

	return nullptr;
}

void FAvaMaskAvaShapeMaterialCollectionHandle::OnMaterialAssignmentChanged(UPrimitiveComponent* InPrimitiveComponent, const int32 InPrevSlotNum, const int32 InNewSlotNum, const TArray<int32>& InChangedMaterialSlots)
{
	check(InPrimitiveComponent);

	RefreshMaterials();

	if (UAvaShapeDynamicMeshBase* ShapeComponent = GetComponent())
	{
		TArray<TSharedPtr<IAvaMaskMaterialHandle>> ChangedHandles;
		ChangedHandles.Reserve(InChangedMaterialSlots.Num());

		for (int32 ChangedMaterialIdx = 0; ChangedMaterialIdx < InChangedMaterialSlots.Num(); ++ChangedMaterialIdx)
		{
			if (!MaterialHandles.Contains(ChangedMaterialIdx))
			{
				UE_LOGF(LogAvaMask, Verbose, "No MaterialHandle found slot: %i", ChangedMaterialIdx);
				ChangedHandles.Emplace(nullptr); // Still need something at this index
			}
			else
			{
				ChangedHandles.Emplace(MaterialHandles[ChangedMaterialIdx]);
			}
		}

		OnSourceMaterialsChanged().ExecuteIfBound(InPrimitiveComponent, ChangedHandles);
	}
}

bool FAvaMaskAvaShapeMaterialCollectionHandle::RefreshMaterials()
{
	UAvaObjectHandleSubsystem* HandleSubsystem = UE::Ava::Internal::GetObjectHandleSubsystem();
	if (!HandleSubsystem)
	{
		UE_LOGF(LogAvaMask, Error, "ObjectHandleSubsystem was invalid");
		return false;
	}
	
	const int32 ProbableNumMaterials = FMath::Max(1, GetNumMaterials());

	auto ResetMaterials = [this]()
	{
		OccupiedMaterialSlots.Reset();
		WeakMaterials.Reset();
		MaterialHandles.Reset();
	};

	TSet<int32> UsedMaterialSlots;
	UsedMaterialSlots.Reserve(ProbableNumMaterials);

	OccupiedMaterialSlots.SetNum(ProbableNumMaterials, false);

	WeakMaterials.Reserve(ProbableNumMaterials);
	MaterialHandles.Reserve(ProbableNumMaterials);

	if (UAvaShapeDynamicMeshBase* ShapeComponent = GetComponent())
	{
		auto RefreshMaterial = [this, ShapeComponent, HandleSubsystem](const int32 InMeshIdx)
		{
			// @note: material can be nullptr/none
			UMaterialInterface* Material = ShapeComponent->GetMaterial(InMeshIdx);
			if (OccupiedMaterialSlots.IsValidIndex(InMeshIdx))
			{
				OccupiedMaterialSlots[InMeshIdx] = Material != nullptr;
			}
			else
			{
				OccupiedMaterialSlots.Add(Material != nullptr);
			}

			WeakMaterials.Emplace(InMeshIdx, Material);

			TSharedPtr<IAvaMaskMaterialHandle> MaterialHandle = nullptr;

			if (ShapeComponent->IsValidMeshIndex(InMeshIdx))
			{
				if (ShapeComponent->IsMaterialType(InMeshIdx, EMaterialType::Parametric))
				{
					if (ShapeComponent->GetParametricMaterialPtr(InMeshIdx))
					{
						MaterialHandle = HandleSubsystem->MakeHandle<IAvaMaskMaterialHandle>(FAvaMaskMaterialReference(ShapeComponent, InMeshIdx), UE::AvaMask::Internal::HandleTag);

						MaterialHandles.Emplace(InMeshIdx, MaterialHandle);
						if (!ensureAlways(UE::Ava::Internal::IsHandleValid(MaterialHandle)))
						{
							return false;
						}

						OccupiedMaterialSlots[InMeshIdx] = true;
					}
				}
				else
				{
					MaterialHandle = MaterialHandles.Emplace(InMeshIdx, HandleSubsystem->MakeMaterialHandle<IAvaMaskMaterialHandle>(Material, UE::AvaMask::Internal::HandleTag));
					if (!ensureAlways(UE::Ava::Internal::IsHandleValid(MaterialHandle)))
					{
						return false;
					}
					
					OccupiedMaterialSlots[InMeshIdx] = true;
				}
			}

			if (UDynamicMeshComponent* MeshComponent = ShapeComponent->GetShapeMeshComponent())
			{
				MaterialAssignmentObserver = MakeUnique<FAvaMaskMaterialAssignmentObserver>(MeshComponent);
				MaterialAssignmentObserver->OnMaterialAssigned().AddRaw(this, &FAvaMaskAvaShapeMaterialCollectionHandle::OnMaterialAssignmentChanged);
			}

			return true;
		};

		const bool bUseSingleMaterial = ShapeComponent->GetUsePrimaryMaterialEverywhere();
		if (bUseSingleMaterial)
		{
			UsedMaterialSlots.Emplace(0);
			if (!RefreshMaterial(UAvaShapeDynamicMeshBase::MESH_INDEX_PRIMARY))
			{
				ResetMaterials();
				return false;
			}
		}
		else
		{
			TSet<int32> MeshIndices = ShapeComponent->GetMeshesIndexes();
			for (const int32 MeshIdx : MeshIndices)
			{
				if (!RefreshMaterial(MeshIdx))
				{
					ResetMaterials();
					return false;
				}
			}

			UsedMaterialSlots.Append(MeshIndices);
		}

		UE::AvaMask::Private::RemoveUnused(WeakMaterials, UsedMaterialSlots);
		UE::AvaMask::Private::RemoveUnused(MaterialHandles, UsedMaterialSlots);

		return true;
	}

	return false;
}

bool FAvaMaskAvaShapeMaterialCollectionHandle::IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag)
{
	if (InTag == UE::AvaMask::Internal::HandleTag)
	{
		if (const AActor* Actor = InInstance.GetTypedObject<AActor>())
		{
			return Actor->GetComponentByClass<UAvaShapeDynamicMeshBase>() != nullptr;
		}
	}

	return false;
}

FStructView FAvaMaskAvaShapeMaterialCollectionHandle::GetMaterialHandleData(
	FHandleData* InParentHandleData
	, const FSoftComponentReference& InComponent
	, const int32 InSlotIdx)
{
	if (InParentHandleData)
	{
		if (FInstancedStruct* Value = InParentHandleData->MaterialHandleData.Find(InSlotIdx))
		{
			return *Value;
		}
	}

	return nullptr;
}

FStructView FAvaMaskAvaShapeMaterialCollectionHandle::GetOrAddMaterialHandleData(
	FHandleData* InParentHandleData
	, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle
	, const FSoftComponentReference& InComponent
	, const int32 InSlotIdx)
{
	if (!ensureAlways(UE::Ava::Internal::IsHandleValid(InMaterialHandle)))
	{
		return nullptr;
	}

	return UE::Ava::Internal::FindOrAddMaterialHandleData<FHandleData>(
		InParentHandleData
		, InMaterialHandle
		, InSlotIdx);
}

bool FAvaMaskAvaShapeMaterialCollectionHandle::IsSlotOccupied(const int32 InIdx)
{
	return OccupiedMaterialSlots.IsValidIndex(InIdx) ? OccupiedMaterialSlots[InIdx] : false;
}
