// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMaskMaterialAssignmentObserver.h"
#include "Containers/Array.h"
#include "Engine/EngineTypes.h"
#include "IAvaMaskMaterialCollectionHandle.h"
#include "IAvaObjectHandle.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "AvaMaskAvaShapeMaterialCollectionHandle.generated.h"

class AActor;
class AText3DActor;
class IAvaMaskMaterialHandle;
class UAvaShapeDynamicMeshBase;
class UAvaText3DComponent;
class UMaterialInterface;
class UText3DComponent;
struct FAvaMaskMaterialReference;

USTRUCT()
struct FAvaMaskAvaShapeMaterialCollectionHandleData
{
	GENERATED_BODY()

	/** Per MeshIdx. */
	UPROPERTY()
	TMap<int32, FInstancedStruct> MaterialHandleData;
};

class FAvaMaskAvaShapeMaterialCollectionHandle
	: public TAvaMaskMaterialCollectionHandle<FAvaMaskAvaShapeMaterialCollectionHandleData>
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaMaskAvaShapeMaterialCollectionHandle, TAvaMaskMaterialCollectionHandle<FAvaMaskAvaShapeMaterialCollectionHandleData>);
	
	explicit FAvaMaskAvaShapeMaterialCollectionHandle(AActor* InActor);
	virtual ~FAvaMaskAvaShapeMaterialCollectionHandle() override = default;

	// ~Begin IAvaMaskMaterialCollectionProvider
	virtual FInstancedStruct MakeDataStruct() override;
	
	virtual TArray<TObjectPtr<UMaterialInterface>> GetMaterials() override;
	virtual TArray<TSharedPtr<IAvaMaskMaterialHandle>> GetMaterialHandles() override;
	virtual int32 GetNumMaterials() const override;

	virtual void SetMaterial(const FSoftComponentReference& InComponent, const int32 InSlotIdx, UMaterialInterface* InMaterial) override;
	virtual void SetMaterials(const TArray<TObjectPtr<UMaterialInterface>>& InMaterials, const TBitArray<>& InSetToggle) override;

	virtual void ForEachMaterial(
		TFunctionRef<bool(
			const FSoftComponentReference& InComponent
			, const int32 InIdx
			, UMaterialInterface* InMaterial)> InFunction) override;

	virtual void ForEachMaterialHandle(
		TFunctionRef<bool(
			const FSoftComponentReference& InComponent
			, const int32 InIdx
			, const bool bInIsSlotOccupied
			, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)> InFunction) override;

	virtual void MapEachMaterial(
		TFunctionRef<UMaterialInterface*(
			const FSoftComponentReference& InComponent
			, const int32 InIdx
			, UMaterialInterface* InMaterial)> InFunction) override;

	virtual void MapEachMaterialHandle(
		TFunctionRef<TSharedPtr<IAvaMaskMaterialHandle>(
			const FSoftComponentReference& InComponent
			, const int32 InIdx
			, const bool bInIsSlotOccupied
			, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)> InFunction) override;

	virtual bool IsValid() const override;
	// ~End IAvaMaskMaterialCollectionProvider

	static bool IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag = NAME_None);

protected:
	// ~Begin TAvaMaskMaterialCollectionHandle
	virtual FStructView GetMaterialHandleData(
		FHandleData* InParentHandleData
		, const FSoftComponentReference& InComponent
		, const int32 InSlotIdx) override;
	
	virtual ::FStructView GetOrAddMaterialHandleData(
		FHandleData* InParentHandleData
		, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle
		, const FSoftComponentReference& InComponent
		, const int32 InSlotIdx) override;
	// ~End TAvaMaskMaterialCollectionHandle

	bool IsSlotOccupied(const int32 InIdx);
	
	UAvaShapeDynamicMeshBase* GetComponent();

	void OnMaterialAssignmentChanged(UPrimitiveComponent* InPrimitiveComponent, const int32 InPrevSlotNum, const int32 InNewSlotNum, const TArray<int32>& InChangedMaterialSlots);
	
	/** Return false if Actor invalid. */
	bool RefreshMaterials();

protected:
	TWeakObjectPtr<AActor> WeakActor;
	TWeakObjectPtr<UAvaShapeDynamicMeshBase> WeakComponent;
	TBitArray<> OccupiedMaterialSlots;
	TMap<int32, TWeakObjectPtr<UMaterialInterface>> WeakMaterials;
	TMap<int32, TSharedPtr<IAvaMaskMaterialHandle>> MaterialHandles;
	TUniquePtr<FAvaMaskMaterialAssignmentObserver> MaterialAssignmentObserver;
};
