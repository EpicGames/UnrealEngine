// Copyright Epic Games, Inc. All Rights Reserved.

#include "Handling/AvaMaskDesignedMaterialHandle.h"

#include "AvaMaskSettings.h"
#include "AvaMaskUtilities.h"
#include "Engine/Texture.h"
#include "Material/DynamicMaterialInstance.h"
#include "Materials/Material.h"

#if WITH_EDITOR
#include "Components/DMMaterialProperty.h"
#include "Model/DynamicMaterialModel.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#endif

FAvaMaskDesignedMaterialHandle::FAvaMaskDesignedMaterialHandle(const TWeakObjectPtr<UDynamicMaterialInstance>& InWeakDesignedMaterial)
	: FAvaDesignedMaterialHandle(InWeakDesignedMaterial)
{
#if WITH_EDITOR
	if (UDynamicMaterialInstance* MaterialInstance = WeakDesignedMaterial.Get())
	{
		if (UDynamicMaterialModel* Model = MaterialInstance->GetMaterialModel())
		{
			if (UDynamicMaterialModelEditorOnlyData* ModelData = UDynamicMaterialModelEditorOnlyData::Get(Model))
			{
				OnMaterialBuiltHandle = ModelData->GetOnMaterialBuiltDelegate().AddRaw(this, &FAvaMaskDesignedMaterialHandle::OnMaterialBuilt);
			}
		}
	}
#endif
}

FAvaMaskDesignedMaterialHandle::~FAvaMaskDesignedMaterialHandle()
{
#if WITH_EDITOR
	if (UDynamicMaterialInstance* MaterialInstance = WeakDesignedMaterial.Get())
	{
		if (UDynamicMaterialModel* Model = MaterialInstance->GetMaterialModel())
		{
			if (UDynamicMaterialModelEditorOnlyData* ModelData = UDynamicMaterialModelEditorOnlyData::Get(Model))
			{
				if (OnMaterialBuiltHandle.IsValid())
				{
					ModelData->GetOnMaterialBuiltDelegate().Remove(OnMaterialBuiltHandle);	
				}
			}
		}
	}
#endif
}

bool FAvaMaskDesignedMaterialHandle::GetMaskParameters(FAvaMask2DMaterialParameters& OutParameters)
{
	if (const UMaterialInstanceDynamic* MaterialInstance = WeakMaterialInstance.Get())
	{
		OutParameters.StoreFromMaterial(MaterialInstance);
		return true;
	}

	return false;
}

bool FAvaMaskDesignedMaterialHandle::SetMaskParameters(
	UTexture* InTexture
	, float InBaseOpacity
	, bool bInInverted
	, const FVector2f& InPadding
	, bool bInApplyFeathering
	, float InOuterFeathering
	, float InInnerFeathering)
{
	if (UDynamicMaterialInstance* MaterialInstance = WeakDesignedMaterial.Get())
	{
		// note this does no longer deal with textures as the mask texture parameter is deprecated.
		// and this will not deal with texture parameter collections as this mask handle system is deprecated too.

		MaterialInstance->SetScalarParameterValueByInfo(UE::AvaMask::Internal::InvertParameterInfo, bInInverted ? 1.0f : 0.0f);
		MaterialInstance->SetScalarParameterValueByInfo(UE::AvaMask::Internal::BaseOpacityParameterInfo, FMath::Clamp(InBaseOpacity, 0.0f, 1.0f));
		MaterialInstance->SetVectorParameterValueByInfo(UE::AvaMask::Internal::PaddingParameterInfo, FLinearColor(FMath::Max(0, InPadding.X), FMath::Max(0, InPadding.Y), 0, 0));
		MaterialInstance->SetVectorParameterValueByInfo(UE::AvaMask::Internal::FeatherParameterInfo, FLinearColor(bInApplyFeathering ? 1.0f : 0.0f, FMath::Max(0, InOuterFeathering), FMath::Max(0, InInnerFeathering), FMath::Max(0, FMath::Max(InOuterFeathering, InInnerFeathering))));

		return true;
	}

	return false;
}

const EBlendMode FAvaMaskDesignedMaterialHandle::GetBlendMode()
{
	if (const UMaterialInterface* MaterialInstance = GetMaterialInstance())
	{
#if WITH_EDITOR
		if (const UDynamicMaterialInstance* DynamicMaterial = Cast<UDynamicMaterialInstance>(MaterialInstance))
		{
			if (UDynamicMaterialModel* MaterialModel = DynamicMaterial->GetMaterialModel())
			{
				if (UDynamicMaterialModelEditorOnlyData* ModelData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel))
				{
					return ModelData->GetBlendMode();
				}
			}
		}
#endif

		return MaterialInstance->GetBlendMode();
	}
	
	return EBlendMode::BLEND_Opaque;
}

void FAvaMaskDesignedMaterialHandle::SetBlendMode(const EBlendMode InBlendMode)
{
	const EBlendMode TargetBlendMode = UE::AvaMask::Internal::GetTargetBlendMode( GetBlendMode(), InBlendMode);
	{
#if WITH_EDITOR
		if (UDynamicMaterialInstance* MaterialInstance = WeakDesignedMaterial.Get())
		{
			if (UDynamicMaterialModel* MaterialModel = MaterialInstance->GetMaterialModel())
			{
				if (UDynamicMaterialModelEditorOnlyData* ModelData = UDynamicMaterialModelEditorOnlyData::Get(MaterialModel))
				{
					ModelData->SetBlendMode(TargetBlendMode);
				}
			}
		}
#else
		if (UMaterialInstanceDynamic* MaterialInstance = GetMaterialInstance())
		{
			MaterialInstance->BlendMode = TargetBlendMode;
			MaterialInstance->BasePropertyOverrides.BlendMode = TargetBlendMode;
		}
#endif
	}
}

bool FAvaMaskDesignedMaterialHandle::SaveOriginalState(const FStructView& InHandleData)
{
	// @note: this assumes the current material is the original
	if (const UDynamicMaterialInstance* DesignedMaterial = WeakDesignedMaterial.Get())
	{
		if (FHandleData* MaterialHandleData = InHandleData.GetPtr<FHandleData>())
		{
			MaterialHandleData->OriginalMaskMaterialParameters.StoreFromMaterial(DesignedMaterial);

			MaterialHandleData->OriginalBlendMode = GetBlendMode();

			MaterialHandleData->OriginalOutputProcessor = GetOutputProcessor(
				MaterialHandleData->OriginalBlendMode == BLEND_Translucent
					? BLEND_Translucent
					: BLEND_Masked
			);

			return true;
		}
	}

	return false;
}

bool FAvaMaskDesignedMaterialHandle::ApplyOriginalState(const FStructView& InHandleData
	, TUniqueFunction<void(UMaterialInterface*)>&& InMaterialSetter)
{
	if (WeakDesignedMaterial.Get())
	{
		if (const FHandleData* MaterialHandleData = InHandleData.GetPtr<const FHandleData>())
		{
			SetMaskParameters(nullptr
				, MaterialHandleData->OriginalMaskMaterialParameters.BaseOpacity
				, MaterialHandleData->OriginalMaskMaterialParameters.bInvert
				, MaterialHandleData->OriginalMaskMaterialParameters.Padding
				, MaterialHandleData->OriginalMaskMaterialParameters.bApplyFeathering
				, MaterialHandleData->OriginalMaskMaterialParameters.OuterFeatherRadius
				, MaterialHandleData->OriginalMaskMaterialParameters.InnerFeatherRadius);

#if WITH_EDITOR
			SetOutputProcessor(GetBlendMode(), MaterialHandleData->OriginalOutputProcessor.Get());
#endif

			SetBlendMode(MaterialHandleData->OriginalBlendMode);

			return true;
		}
	}

	return false;
}

bool FAvaMaskDesignedMaterialHandle::ApplyModifiedState(
	const FAvaMask2DSubjectParameters& InModifiedParameters
	, const FStructView& InHandleData
	, TUniqueFunction<void(UMaterialInterface*)>&& InMaterialSetter)
{
	if (WeakDesignedMaterial.Get())
	{
		if (InHandleData.GetPtr<const FHandleData>())
		{
			SetMaskParameters(nullptr
				, InModifiedParameters.MaterialParameters.BaseOpacity
				, InModifiedParameters.MaterialParameters.bInvert
				, InModifiedParameters.MaterialParameters.Padding
				, InModifiedParameters.MaterialParameters.bApplyFeathering
				, InModifiedParameters.MaterialParameters.OuterFeatherRadius
				, InModifiedParameters.MaterialParameters.InnerFeatherRadius);

			SetBlendMode(InModifiedParameters.MaterialParameters.BlendMode);

#if WITH_EDITOR
			UMaterialFunctionInterface* MaterialFunctionToUse = GetMutableDefault<UAvaMaskSettings>()->GetMaterialFunction();
			SetOutputProcessor(InModifiedParameters.MaterialParameters.BlendMode, MaterialFunctionToUse);
#endif

			LastAppliedParameters = InModifiedParameters;

			return true;
		}
	}

	return false;
}

bool FAvaMaskDesignedMaterialHandle::HasRequiredParameters(TArray<FString>& OutMissingParameterNames)
{
	// Designed Materials always have the required parameters - they're injected
	return true;
}

bool FAvaMaskDesignedMaterialHandle::IsValid() const
{
	return FAvaDesignedMaterialHandle::IsValid();
}

bool FAvaMaskDesignedMaterialHandle::IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag)
{
	return InTag == UE::AvaMask::Internal::HandleTag && FAvaDesignedMaterialHandle::IsSupported(InInstance, InTag);
}

UMaterialInstanceDynamic* FAvaMaskDesignedMaterialHandle::GetMaterialInstance()
{
	if (UDynamicMaterialInstance* DesignedMaterial = WeakDesignedMaterial.Get())
	{
		return DesignedMaterial;
	}
	
	if (UMaterialInstanceDynamic* MaterialInstance = Super::GetMaterialInstance())
	{
		return MaterialInstance;
	}

	return nullptr;
}

#if WITH_EDITOR
void FAvaMaskDesignedMaterialHandle::OnMaterialBuilt(UDynamicMaterialModelBase* InMaterialModel)
{
	check(InMaterialModel);
	
	// @note: parameter values get reset after build, so we need to re-apply them
	SetMaskParameters(nullptr
		, LastAppliedParameters.MaterialParameters.BaseOpacity
		, LastAppliedParameters.MaterialParameters.bInvert
		, LastAppliedParameters.MaterialParameters.Padding
		, LastAppliedParameters.MaterialParameters.bApplyFeathering
		, LastAppliedParameters.MaterialParameters.OuterFeatherRadius
		, LastAppliedParameters.MaterialParameters.InnerFeatherRadius);

	SetBlendMode(LastAppliedParameters.MaterialParameters.BlendMode);
}

UMaterialFunctionInterface* FAvaMaskDesignedMaterialHandle::GetOutputProcessor(EBlendMode InBlendMode)
{
	if (UDynamicMaterialInstance* DesignedMaterial = WeakDesignedMaterial.Get())
	{
		if (UDynamicMaterialModel* Model = DesignedMaterial->GetMaterialModel())
		{
			auto GetOutputProcessorForPropertyType = [](UDynamicMaterialModel* InModel, EDMMaterialPropertyType InPropertyType)->UMaterialFunctionInterface*
			{
				if (const UDynamicMaterialModelEditorOnlyData* ModelData = UDynamicMaterialModelEditorOnlyData::Get(InModel))
				{
					if (const UDMMaterialProperty* MaterialProperty = ModelData->GetMaterialProperty(InPropertyType))
					{
						return MaterialProperty->GetOutputProcessor();
					}
				}

				return nullptr;
			};

			switch (InBlendMode)
			{
			case EBlendMode::BLEND_Masked:
				return GetOutputProcessorForPropertyType(Model, EDMMaterialPropertyType::OpacityMask);

			case EBlendMode::BLEND_Translucent:
				return GetOutputProcessorForPropertyType(Model, EDMMaterialPropertyType::Opacity);

			default:
				UE_LOGF(LogAvaMask, Error, "BlendMode not supported: %ls", *UE::AvaMask::Internal::GetBlendModeString(InBlendMode));
				return nullptr;
			}
		}
	}

	return nullptr;
}

void FAvaMaskDesignedMaterialHandle::SetOutputProcessor(EBlendMode InBlendMode, UMaterialFunctionInterface* InMaterialFunction)
{
	if (UDynamicMaterialInstance* DesignedMaterial = WeakDesignedMaterial.Get())
	{
		if (!::IsValid(InMaterialFunction))
		{
			InMaterialFunction = GetMutableDefault<UAvaMaskSettings>()->GetMaterialFunction();
		}

		if (UDynamicMaterialModel* Model = DesignedMaterial->GetMaterialModel())
		{
			auto SetOutputProcessorForPropertyType = [InMaterialFunction](UDynamicMaterialModel* InModel, EDMMaterialPropertyType InPropertyType)
			{
				if (const UDynamicMaterialModelEditorOnlyData* ModelData = UDynamicMaterialModelEditorOnlyData::Get(InModel))
				{
					if (UDMMaterialProperty* MaterialProperty = ModelData->GetMaterialProperty(InPropertyType))
					{
						// Check if already set
						const UMaterialFunctionInterface* OutputProcessorMaterialFunction = MaterialProperty->GetOutputProcessor();
						if (OutputProcessorMaterialFunction && OutputProcessorMaterialFunction == InMaterialFunction)
						{
							return;
						}

						MaterialProperty->SetOutputProcessor(InMaterialFunction);
					}
				}
			};
		
			switch (InBlendMode)
			{
			case EBlendMode::BLEND_Masked:
				SetOutputProcessorForPropertyType(Model, EDMMaterialPropertyType::OpacityMask);
				break;

			case EBlendMode::BLEND_Translucent:
				SetOutputProcessorForPropertyType(Model, EDMMaterialPropertyType::Opacity);
				break;

			default:
				UE_LOGF(LogAvaMask, Error, "BlendMode not supported: %ls", *UE::AvaMask::Internal::GetBlendModeString(InBlendMode));
				break;
			}
		}
	}
}
#else
UMaterialFunctionInterface* FAvaMaskDesignedMaterialHandle::GetOutputProcessor(EBlendMode InBlendMode)
{
	return nullptr;
}
#endif
