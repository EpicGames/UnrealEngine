// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Handling/AvaDesignedMaterialHandle.h"
#include "Handling/AvaMaskMaterialInstanceHandle.h"

#include "AvaMaskDesignedMaterialHandle.generated.h"

#if WITH_EDITOR
class UDynamicMaterialModelBase;
#endif

class UDynamicMaterialInstance;

USTRUCT()
struct FAvaMaskDesignedMaterialHandleData
{
	GENERATED_BODY()

	UPROPERTY()
	FAvaMask2DMaterialParameters OriginalMaskMaterialParameters;

	UPROPERTY()
	TEnumAsByte<EBlendMode> OriginalBlendMode = EBlendMode::BLEND_Opaque;

	UPROPERTY()
	TSoftObjectPtr<UMaterialFunctionInterface> OriginalOutputProcessor;
};

class FAvaMaskDesignedMaterialHandle
	: public FAvaDesignedMaterialHandle
	, public TAvaMaskMaterialHandle<FAvaMaskDesignedMaterialHandleData>
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaMaskDesignedMaterialHandle, FAvaDesignedMaterialHandle);
	
	explicit FAvaMaskDesignedMaterialHandle(const TWeakObjectPtr<UDynamicMaterialInstance>& InWeakDesignedMaterial);
	virtual ~FAvaMaskDesignedMaterialHandle() override;

	// ~Begin IAvaMaskMaterialHandle
	virtual bool GetMaskParameters(FAvaMask2DMaterialParameters& OutParameters) override;
	virtual bool SetMaskParameters(UTexture* InTexture, float InBaseOpacity, bool bInInverted, const FVector2f& InPadding, bool bInApplyFeathering, float InOuterFeathering, float InInnerFeathering) override;
	virtual const EBlendMode GetBlendMode() override;
	virtual void SetBlendMode(const EBlendMode InBlendMode) override;
	virtual void SetChildMaterial(UMaterialInstanceDynamic* InChildMaterial) override { }
	virtual bool SaveOriginalState(const FStructView& InHandleData) override;
	virtual bool ApplyOriginalState(const FStructView& InHandleData
		, TUniqueFunction<void(UMaterialInterface*)>&& InMaterialSetter) override;
	virtual bool ApplyModifiedState(const FAvaMask2DSubjectParameters& InModifiedParameters
		, const FStructView& InHandleData
		, TUniqueFunction<void(UMaterialInterface*)>&& InMaterialSetter) override;
	// ~End IAvaMaskMaterialHandle
	
	// ~Begin IAvaMaterialHandle
	virtual FString GetMaterialName() override { return Super::GetMaterialName(); }
	virtual UMaterialInterface* GetMaterial() override { return Super::GetMaterial(); }
	virtual void CopyParametersFrom(UMaterialInstance* InSourceMaterial) override { Super::CopyParametersFrom(InSourceMaterial); }
	virtual bool HasRequiredParameters(TArray<FString>& OutMissingParameterNames) override;
	virtual void SetParentMaterial(UMaterialInterface* InMaterial) override { Super::SetParentMaterial(InMaterial); }
	virtual UMaterialInterface* GetParentMaterial() override { return Super::GetParentMaterial(); }
	// ~End IAvaMaterialHandle

	// ~Begin IAvaObjectHandle
    virtual bool IsValid() const override;
    // ~End IAvaObjectHandle

	static bool IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag = NAME_None);

protected:
	virtual UMaterialInstanceDynamic* GetMaterialInstance() override;

#if WITH_EDITOR
	void OnMaterialBuilt(UDynamicMaterialModelBase* InMaterialModel);
	
	UMaterialFunctionInterface* GetOutputProcessor(EBlendMode InBlendMode);
	void SetOutputProcessor(EBlendMode InBlendMode, UMaterialFunctionInterface* InMaterialFunction);
#else
	// Dummy, returns nullptr
	UMaterialFunctionInterface* GetOutputProcessor(EBlendMode InBlendMode);
#endif

private:
	FAvaMask2DSubjectParameters LastAppliedParameters;
#if WITH_EDITOR
	FDelegateHandle OnMaterialBuiltHandle;
#endif
};
