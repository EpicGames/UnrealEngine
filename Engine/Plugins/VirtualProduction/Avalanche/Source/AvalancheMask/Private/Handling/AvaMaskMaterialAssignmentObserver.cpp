// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMaskMaterialAssignmentObserver.h"

#include "Components/ActorComponent.h"
#include "Components/PrimitiveComponent.h"
#include "GameFramework/Actor.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"

FAvaMaskMaterialAssignmentObserver::FAvaMaskMaterialAssignmentObserver(UPrimitiveComponent* InComponent)
	: ComponentWeak(InComponent)
{
	check(InComponent);

	SlotCount = InComponent->GetNumMaterials();
	Materials.AddDefaulted(InComponent->GetNumMaterials());

	for (int32 MaterialIdx = 0; MaterialIdx < InComponent->GetNumMaterials(); ++MaterialIdx)
	{
		Materials[MaterialIdx] = InComponent->GetMaterial(MaterialIdx);
	}

	if (!UPrimitiveComponent::MarkRenderStateDirtyEvent.IsBoundToObject(this))
	{
		RenderStateDirtyHandle = UPrimitiveComponent::MarkRenderStateDirtyEvent.AddRaw(this, &FAvaMaskMaterialAssignmentObserver::OnComponentRenderStateDirty);
	}
}

FAvaMaskMaterialAssignmentObserver::~FAvaMaskMaterialAssignmentObserver()
{
	if (RenderStateDirtyHandle.IsValid())
	{
		UPrimitiveComponent::MarkRenderStateDirtyEvent.Remove(RenderStateDirtyHandle);
	}
}

FAvaMaskMaterialAssignmentObserver::FOnMaterialAssignedDelegate& FAvaMaskMaterialAssignmentObserver::OnMaterialAssigned()
{
	return OnMaterialAssignedDelegate;
}

UPrimitiveComponent* FAvaMaskMaterialAssignmentObserver::GetPrimitiveComponent() const
{
	return ComponentWeak.Get();
}

void FAvaMaskMaterialAssignmentObserver::OnComponentRenderStateDirty(UActorComponent& InComponent)
{
	// @note: this is global delegate, so we need to check that we care about the component
	if (UPrimitiveComponent* Component = ComponentWeak.Get())
	{
		if (&InComponent == Component)
		{
			const int32 PreviousSlotCount = SlotCount;
			const int32 NewSlotCount = Component->GetNumMaterials();
			SlotCount = NewSlotCount;

			TArray<int32> ChangedMaterialSlots;
			ChangedMaterialSlots.Reserve(NewSlotCount);

			Materials.SetNum(NewSlotCount);
			for (int32 MaterialIdx = 0; MaterialIdx < Component->GetNumMaterials(); ++MaterialIdx)
			{
				UMaterialInterface* NewMaterial = Component->GetMaterial(MaterialIdx);

				if (NewMaterial)
				{
					const UMaterialInterface* OldMaterial = Cast<UMaterialInterface>(Materials[MaterialIdx].ResolveObjectPtr());

					const bool bDifferentObject = OldMaterial != NewMaterial;
					const bool bSameBaseMaterial = OldMaterial != nullptr && NewMaterial->IsA<UMaterialInstanceDynamic>() && OldMaterial->GetMaterial() == NewMaterial->GetMaterial();

					// Compare actual material
					if (bDifferentObject && !bSameBaseMaterial)
					{
						ChangedMaterialSlots.Add(MaterialIdx);
					}
				}

				Materials[MaterialIdx] = NewMaterial;
			}

			if (!ChangedMaterialSlots.IsEmpty())
			{
				OnMaterialAssigned().Broadcast(Component, PreviousSlotCount, NewSlotCount, ChangedMaterialSlots);
			}
		}
	}
}
