// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Delegates/MulticastDelegateBase.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/ObjectKey.h"
#include "UObject/WeakObjectPtr.h"

class AActor;
class UActorComponent;
class UPrimitiveComponent;

class FAvaMaskMaterialAssignmentObserver
{
public:
	using FOnMaterialAssignedDelegate = TMulticastDelegate<void(UPrimitiveComponent* InComponent, const int32 InPrevSlotNum, const int32 InNewSlotNum, const TArray<int32>& InChangedMaterialSlots)>;
	
	explicit FAvaMaskMaterialAssignmentObserver(UPrimitiveComponent* InComponent);
	~FAvaMaskMaterialAssignmentObserver();

	FOnMaterialAssignedDelegate& OnMaterialAssigned();

	UPrimitiveComponent* GetPrimitiveComponent() const;

private:
	void OnComponentRenderStateDirty(UActorComponent& InComponent);

	TWeakObjectPtr<UPrimitiveComponent> ComponentWeak;
	FDelegateHandle RenderStateDirtyHandle;
	FOnMaterialAssignedDelegate OnMaterialAssignedDelegate;
	int32 SlotCount = 0;
	TArray<FObjectKey> Materials; 
};
