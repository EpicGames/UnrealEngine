// Copyright Epic Games, Inc. All Rights Reserved.

#include "Handling/AvaMaskMaterialInstanceHandle.h"

#include "AvaMaskUtilities.h"
#include "Engine/Texture.h"
#include "Materials/AvaMaskMaterialInstanceSubsystem.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"

FAvaMaskMaterialInstanceHandle::FAvaMaskMaterialInstanceHandle(const TWeakObjectPtr<UMaterialInterface>& InWeakParentMaterial)
	: FAvaMaterialInstanceHandle(InWeakParentMaterial)
{
}

const EBlendMode FAvaMaskMaterialInstanceHandle::GetBlendMode()
{
	if (const UMaterialInterface* MaterialInstance = GetMaterialInstance())
	{
		return MaterialInstance->GetBlendMode();
	}
	
	if (const UMaterialInterface* Material = GetMaterial())
	{
		return Material->GetBlendMode();
	}

	return EBlendMode::BLEND_Opaque;
}

void FAvaMaskMaterialInstanceHandle::SetBlendMode(const EBlendMode InBlendMode)
{
	const EBlendMode TargetBlendMode = UE::AvaMask::Internal::GetTargetBlendMode(GetBlendMode(), InBlendMode);
	if (UMaterialInstanceDynamic* MaterialInstance = GetMaterialInstance())
	{
		MaterialInstance->BasePropertyOverrides.bOverride_BlendMode = true;
		MaterialInstance->BasePropertyOverrides.BlendMode = TargetBlendMode;
		MaterialInstance->BlendMode = TargetBlendMode;
#if WITH_EDITOR
		MaterialInstance->UpdateCachedData();
#endif
	}
}

bool FAvaMaskMaterialInstanceHandle::SaveOriginalState(const FStructView& InHandleData)
{
	if (UMaterialInterface* ParentMaterial = GetParentMaterial())
	{
		if (FHandleData* HandleData = InHandleData.GetPtr<FHandleData>())
		{
			HandleData->OriginalMaterial = ParentMaterial;
			HandleData->OriginalMaskMaterialParameters.StoreFromMaterial(ParentMaterial);
			HandleData->OriginalBlendMode = GetBlendMode();
		}

		return true;
	}

	return false;
}

bool FAvaMaskMaterialInstanceHandle::ApplyOriginalState(
	const FStructView& InHandleData
	, TUniqueFunction<void(UMaterialInterface*)>&& InMaterialSetter)
{
	if (const FHandleData* HandleData = InHandleData.GetPtr<FHandleData>())
	{
		SetBlendMode(HandleData->OriginalBlendMode);
		SetMaskParameters(nullptr
			, HandleData->OriginalMaskMaterialParameters.BaseOpacity
			, HandleData->OriginalMaskMaterialParameters.bInvert);
		
		if (UMaterialInterface* OriginalMaterial = HandleData->OriginalMaterial.Get())
		{
			InMaterialSetter(OriginalMaterial);
		}
		else if (UMaterialInterface* ParentMaterial = GetParentMaterial())
		{
			InMaterialSetter(ParentMaterial);
		}
		
		return true;
	}

	return false;
}

bool FAvaMaskMaterialInstanceHandle::ApplyModifiedState(
	const FAvaMask2DSubjectParameters& InModifiedParameters
	, const FStructView& InHandleData
	, TUniqueFunction<void(UMaterialInterface*)>&& InMaterialSetter)
{
	if (InHandleData.GetPtr<FHandleData>())
	{
		const EBlendMode TargetBlendMode = UE::AvaMask::Internal::GetTargetBlendMode(InModifiedParameters.MaterialParameters.BlendMode, GetBlendMode());
		if (UMaterialInterface* MaterialInstance = GetOrCreateMaterialInstance(InModifiedParameters.MaterialParameters.CanvasName, TargetBlendMode))
		{
			SetBlendMode(InModifiedParameters.MaterialParameters.BlendMode);
			SetMaskParameters(nullptr
				, InModifiedParameters.MaterialParameters.BaseOpacity
				, InModifiedParameters.MaterialParameters.bInvert
			);

			InMaterialSetter(MaterialInstance);
		}

		return true;
	}

	return true;
}

bool FAvaMaskMaterialInstanceHandle::IsValid() const
{
	return FAvaMaterialInstanceHandle::IsValid();
}

bool FAvaMaskMaterialInstanceHandle::IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag)
{
	return InTag == UE::AvaMask::Internal::HandleTag && Super::IsSupported(InInstance, InTag);
}

UMaterialInstanceDynamic* FAvaMaskMaterialInstanceHandle::GetMaterialInstance()
{
	return FAvaMaterialInstanceHandle::GetMaterialInstance();
}

UMaterialInstanceDynamic* FAvaMaskMaterialInstanceHandle::GetOrCreateMaterialInstance(
	const FName InMaskName
	, const EBlendMode InBlendMode)
{
	const EBlendMode TargetBlendMode = UE::AvaMask::Internal::GetTargetBlendMode(InBlendMode, GetBlendMode());

	if (UMaterialInstanceDynamic* MaterialInstance = WeakMaterialInstance.Get())
	{
		if ((MaterialInstance == GetParentMaterial() || MaterialInstance->GetBaseMaterial() == GetParentMaterial())
			&& MaterialInstance->BlendMode == TargetBlendMode)
		{
			return MaterialInstance;
		}
	}

	if (UMaterialInterface* ParentMaterial = GetParentMaterial())
	{
		const uint32 MaterialInstanceKey = UE::AvaMask::Internal::MakeMaterialInstanceKey(ParentMaterial, InMaskName, TargetBlendMode);
		UMaterialInstanceDynamic* MaterialInstance = UE::AvaMask::Internal::GetMaterialInstanceSubsystem()->GetMaterialInstanceProvider()->FindOrAddMID(ParentMaterial, MaterialInstanceKey, TargetBlendMode);
		WeakMaterialInstance = MaterialInstance;
		return MaterialInstance;
	}

	return nullptr;
}

UMaterialInstanceDynamic* FAvaMaskMaterialInstanceHandle::GetOrCreateMaterialInstance()
{
	return Super::GetOrCreateMaterialInstance();
}
