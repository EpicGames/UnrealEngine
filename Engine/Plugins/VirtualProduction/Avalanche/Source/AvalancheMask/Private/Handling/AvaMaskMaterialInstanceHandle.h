// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Handling/AvaMaterialInstanceHandle.h"
#include "IAvaMaskMaterialHandle.h"
#include "Materials/MaterialParameters.h"

#include "AvaMaskMaterialInstanceHandle.generated.h"

class UMaterialInstanceDynamic;
class UMaterialInterface;
struct FAvaMaskMaterialReference;

USTRUCT()
struct FAvaMaskMaterialInstanceHandleData
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UMaterialInterface> OriginalMaterial;

	UPROPERTY()
	FAvaMask2DMaterialParameters OriginalMaskMaterialParameters;

	UPROPERTY()
	TEnumAsByte<EBlendMode> OriginalBlendMode = EBlendMode::BLEND_Opaque;
};

/** This handles a Parent and (optionally) a Material Instance. It can also create the MaterialInstance. */
class FAvaMaskMaterialInstanceHandle
	: public FAvaMaterialInstanceHandle
	, public TAvaMaskMaterialHandle<FAvaMaskMaterialInstanceHandleData>
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaMaskMaterialInstanceHandle, FAvaMaterialInstanceHandle);
	
	explicit FAvaMaskMaterialInstanceHandle(const TWeakObjectPtr<UMaterialInterface>& InWeakParentMaterial);
	virtual ~FAvaMaskMaterialInstanceHandle() override = default;

	// ~Begin IAvaMaskMaterialHandle
	virtual const EBlendMode GetBlendMode() override;
	virtual void SetBlendMode(const EBlendMode InBlendMode) override;
	virtual void SetChildMaterial(UMaterialInstanceDynamic* InChildMaterial) override { }
	virtual bool SaveOriginalState(const FStructView& InHandleData) override;
	virtual bool ApplyOriginalState(
		const FStructView& InHandleData
		, TUniqueFunction<void(UMaterialInterface*)>&& InMaterialSetter) override;
	virtual bool ApplyModifiedState(
		const FAvaMask2DSubjectParameters& InModifiedParameters
		, const FStructView& InHandleData
		, TUniqueFunction<void(UMaterialInterface*)>&& InMaterialSetter) override;
	// ~End IAvaMaskMaterialHandle

	// ~Begin IAvaMaterialHandle
	virtual FString GetMaterialName() override { return Super::GetMaterialName(); }
	virtual UMaterialInterface* GetMaterial() override { return FAvaMaterialInstanceHandle::GetMaterial(); }
	virtual void CopyParametersFrom(UMaterialInstance* InSourceMaterial) override { FAvaMaterialInstanceHandle::CopyParametersFrom(InSourceMaterial); }
	virtual void SetParentMaterial(UMaterialInterface* InMaterial) override { FAvaMaterialInstanceHandle::SetParentMaterial(InMaterial); }
	virtual UMaterialInterface* GetParentMaterial() override { return FAvaMaterialInstanceHandle::GetParentMaterial(); }
	// ~End IAvaMaterialHandle

	// ~Begin IAvaObjectHandle	
	virtual bool IsValid() const override;
	// ~End IAvaObjectHandle

	static bool IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag = NAME_None);

protected:
	virtual UMaterialInstanceDynamic* GetMaterialInstance() override;
	virtual UMaterialInstanceDynamic* GetOrCreateMaterialInstance(const FName InMaskName, const EBlendMode InBlendMode);
	virtual UMaterialInstanceDynamic* GetOrCreateMaterialInstance() override;
};
