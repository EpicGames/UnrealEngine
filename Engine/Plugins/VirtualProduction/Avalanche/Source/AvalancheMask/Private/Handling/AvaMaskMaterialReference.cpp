// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMaskMaterialReference.h"
#include "UObject/Object.h"

FString FAvaMaskMaterialReference::ToString() const
{
	const UObject* const Object = GetObject();
	if (!Object)
	{
		return TEXT("(Invalid Reference)");
	}

	if (Index == INDEX_NONE)
	{
		return Object->GetName();
	}

	return FString::Printf(TEXT("%s [%d]"), *Object->GetName(), Index);
}

UObject* FAvaMaskMaterialReference::GetObject() const
{
	return ObjectWeak.Get();
}
