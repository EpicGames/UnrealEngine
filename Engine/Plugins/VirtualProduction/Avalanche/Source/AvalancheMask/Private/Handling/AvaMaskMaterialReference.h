// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakObjectPtr.h"

/** Describes a reference to an object containing a material, or the material itself */
struct FAvaMaskMaterialReference
{
	FAvaMaskMaterialReference() = default;

	explicit FAvaMaskMaterialReference(UObject* InObject, int32 InIndex = INDEX_NONE)
		: ObjectWeak(InObject)
		, Index(InIndex)
	{
	}

	FString ToString() const;

	/** Resolves the weak object reference */
	UObject* GetObject() const;

	template<typename T>
	T* GetTypedObject() const
	{
		return Cast<T>(GetObject());
	}

	/** The material itself or the object holding the material */
	FWeakObjectPtr ObjectWeak;

	/** Optional index to further identify a material in the object */
	int32 Index = INDEX_NONE;
};
