// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMaskMediaPlateMaterialHandle.h"

#include "AvaMaskMaterialReference.h"
#include "AvaMaterialInstanceHandle.h"
#include "Components/StaticMeshComponent.h"
#include "Materials/MaterialInstance.h"
#include "MediaPlate.h"

FAvaMaskMediaPlateMaterialHandle::FAvaMaskMediaPlateMaterialHandle(const TWeakObjectPtr<UMaterialInterface>& InWeakParentMaterial)
	: FAvaMaskMaterialInstanceHandle(InWeakParentMaterial)
{
	
}

void FAvaMaskMediaPlateMaterialHandle::SetBlendMode(const EBlendMode InBlendMode)
{
	if (GetParentMaterial() && GetParentMaterial() == GetMaterialInstance())
	{
		return;	// Don't set blend mode on the parent material.
	}

	Super::SetBlendMode(InBlendMode);
}

bool FAvaMaskMediaPlateMaterialHandle::IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag)
{
	// Make sure this is generally supported as a material instance.
	if (!FAvaMaterialInstanceHandle::IsSupported(InInstance, InTag))
	{
		return false;
	}

	// MediaPlate makes either a MIC or MID.
	// Check if the material is embedded under the static mesh component.
	if (const UMaterialInstance* MaterialInstance = InInstance.GetTypedObject<UMaterialInstance>())
	{
		if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MaterialInstance->GetOuter()))
		{
			// Need to check that it is a media plate to be sure
			if (const AMediaPlate* MediaPlate = Cast<AMediaPlate>(StaticMeshComponent->GetOuter()))
			{
				// only consider the mesh component that the media plate uses
				return MediaPlate->StaticMeshComponent == StaticMeshComponent;
			}
		}
	}
	return false;
}

UMaterialInstanceDynamic* FAvaMaskMediaPlateMaterialHandle::GetOrCreateMaterialInstance(const FName InMaskName, const EBlendMode InBlendMode)
{
	if (UMaterialInstanceDynamic* MaterialInstance = GetOrCreateMaterialInstanceImpl())
	{
		return MaterialInstance;
	}

	// Fallback to material instance handling.
	return Super::GetOrCreateMaterialInstance(InMaskName, InBlendMode);
}

UMaterialInstanceDynamic* FAvaMaskMediaPlateMaterialHandle::GetOrCreateMaterialInstance()
{
	if (UMaterialInstanceDynamic* MaterialInstance = GetOrCreateMaterialInstanceImpl())
	{
		return MaterialInstance;
	}
	return Super::GetOrCreateMaterialInstance();
}

UMaterialInstanceDynamic* FAvaMaskMediaPlateMaterialHandle::GetOrCreateMaterialInstanceImpl()
{
	if (UMaterialInstanceDynamic* MaterialInstance = GetMaterialInstance())
	{
		return MaterialInstance;
	}

	if (UMaterialInterface* ParentMaterial = GetParentMaterial())
	{
		// Need to be embedded in the level.
		if (UObject* Outer = Cast<UStaticMeshComponent>(ParentMaterial->GetOuter()))
		{
			UMaterialInstanceDynamic* MaterialInstance = UMaterialInstanceDynamic::Create(ParentMaterial, Outer);
			WeakMaterialInstance = MaterialInstance;
			return MaterialInstance;
		}
	}
	return nullptr;
}
