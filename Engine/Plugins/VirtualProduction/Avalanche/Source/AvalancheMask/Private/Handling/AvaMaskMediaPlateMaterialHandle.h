// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Handling/AvaMaskMaterialInstanceHandle.h"

class FAvaMaskMediaPlateMaterialHandle : public FAvaMaskMaterialInstanceHandle
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaMaskMediaPlateMaterialHandle, FAvaMaskMaterialInstanceHandle);
	
	explicit FAvaMaskMediaPlateMaterialHandle(const TWeakObjectPtr<UMaterialInterface>& InWeakParentMaterial);
	virtual ~FAvaMaskMediaPlateMaterialHandle() override = default;
	
	//~ Begin IAvaMaskMaterialHandle
	virtual void SetBlendMode(const EBlendMode InBlendMode) override;
	//~ End IAvaMaskMaterialHandle

	static bool IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag = NAME_None);

protected:
	//~ Begin FAvaMaskMaterialInstanceHandle
	virtual UMaterialInstanceDynamic* GetOrCreateMaterialInstance(const FName InMaskName, const EBlendMode InBlendMode) override;
	//~ End FAvaMaskMaterialInstanceHandle

	//~ Begin FAvaMaterialInstanceHandle
	virtual UMaterialInstanceDynamic* GetOrCreateMaterialInstance() override;
	//~ End FAvaMaterialInstanceHandle

	UMaterialInstanceDynamic* GetOrCreateMaterialInstanceImpl();
};
