// Copyright Epic Games, Inc. All Rights Reserved.

#include "Handling/AvaMaskParametricMaterialHandle.h"

#include "AvaMaskUtilities.h"
#include "AvaShapeParametricMaterial.h"
#include "Engine/Texture.h"
#include "Handling/AvaParametricMaterialHandle.h"
#include "Materials/MaterialInstanceDynamic.h"

FAvaMaskParametricMaterialHandle::FAvaMaskParametricMaterialHandle(const FAvaMaskMaterialReference& InParametricMaterial)
	: FAvaParametricMaterialHandle(InParametricMaterial)
{
	if (FAvaShapeParametricMaterial* ParametricMtl = GetParametricMaterial())
	{
		if (UMaterialInstanceDynamic* MID = ParametricMtl->GetMaterial())
		{
			LastShadingModel = MID->GetShadingModels();
			LastBlendMode = MID->GetBlendMode();
			LastMaterialInstance = MID;
		}

		OnMaterialParameterChangedHandle = FAvaShapeParametricMaterial::OnMaterialParameterChanged().AddRaw(this
			, &FAvaMaskParametricMaterialHandle::OnMaterialChanged);
	}
}

FAvaMaskParametricMaterialHandle::~FAvaMaskParametricMaterialHandle()
{
	FAvaShapeParametricMaterial::OnMaterialParameterChanged().Remove(OnMaterialParameterChangedHandle);
	OnMaterialParameterChangedHandle.Reset();
}

bool FAvaMaskParametricMaterialHandle::GetMaskParameters(FAvaMask2DMaterialParameters& OutParameters)
{
	if (const UMaterialInstanceDynamic* MaterialInstance = WeakMaterialInstance.Get())
	{
		OutParameters.StoreFromMaterial(MaterialInstance);
		return true;
	}

	return false;
}

bool FAvaMaskParametricMaterialHandle::SetMaskParameters(
	UTexture* InTexture
	, float InBaseOpacity
	, bool bInInverted
	, const FVector2f& InPadding
	, bool bInApplyFeathering
	, float InOuterFeathering
	, float InInnerFeathering)
{
	if (UMaterialInstanceDynamic* MaterialInstance = GetMaterialInstance())
	{
		FLinearColor PaddingValueV = FLinearColor(FMath::Max(0, InPadding.X), FMath::Max(0, InPadding.Y), 0, 0);
		FLinearColor FeatherValueV = FLinearColor(bInApplyFeathering ? 1.0f : 0.0f, FMath::Max(0, InOuterFeathering), FMath::Max(0, InInnerFeathering), FMath::Max(0, FMath::Max(InOuterFeathering, InInnerFeathering)));
		
		UE_LOGF(LogAvaMask, VeryVerbose, "SetParameters:\nFeather:%ls\n", *FeatherValueV.ToString());

		// note this does no longer deal with textures as the mask texture parameter is deprecated.
		// and this will not deal with texture parameter collections as this mask handle system is deprecated too.

		MaterialInstance->SetScalarParameterValueByInfo(UE::AvaMask::Internal::InvertParameterInfo, bInInverted ? 1.0f : 0.0f);
		MaterialInstance->SetScalarParameterValueByInfo(UE::AvaMask::Internal::BaseOpacityParameterInfo, FMath::Clamp(InBaseOpacity, 0.0f, 1.0f));
		MaterialInstance->SetVectorParameterValueByInfo(UE::AvaMask::Internal::PaddingParameterInfo, PaddingValueV);
		MaterialInstance->SetVectorParameterValueByInfo(UE::AvaMask::Internal::FeatherParameterInfo, FeatherValueV);

		return true;
	}

	return false;
}

const EBlendMode FAvaMaskParametricMaterialHandle::GetBlendMode()
{
	if (const FAvaShapeParametricMaterial* ParametricMtl = GetParametricMaterial())
	{
		return ParametricMtl->ShouldUseTranslucentMaterial() ? EBlendMode::BLEND_Translucent : EBlendMode::BLEND_Opaque;
	}
	
	if (const UMaterialInterface* MaterialInstance = GetMaterialInstance())
	{
		return MaterialInstance->GetBlendMode();
	}
	
	if (const UMaterialInterface* Material = GetMaterial())
	{
		return Material->GetBlendMode();
	}

	return EBlendMode::BLEND_Opaque;
}

void FAvaMaskParametricMaterialHandle::SetBlendMode(const EBlendMode InBlendMode)
{
	const EBlendMode TargetBlendMode = UE::AvaMask::Internal::GetTargetBlendMode( GetBlendMode(), InBlendMode);
	if (FAvaShapeParametricMaterial* ParametricMtl = GetParametricMaterial())
	{
		// Disable auto-switch so that user changes won't override the mask
		ParametricMtl->SetTranslucency(TargetBlendMode != EBlendMode::BLEND_Opaque ? EAvaShapeParametricMaterialTranslucency::Enabled : EAvaShapeParametricMaterialTranslucency::Disabled);
		return;
	}
	
	if (UMaterialInstanceDynamic* MaterialInstance = GetMaterialInstance())
	{
		MaterialInstance->BlendMode = TargetBlendMode;
		MaterialInstance->BasePropertyOverrides.BlendMode = TargetBlendMode;
	}
}

void FAvaMaskParametricMaterialHandle::SetChildMaterial(UMaterialInstanceDynamic* InChildMaterial)
{
	ChildMaterial = InChildMaterial;
}

bool FAvaMaskParametricMaterialHandle::SaveOriginalState(const FStructView& InHandleData)
{
	// @note: this assumes the current material is the original
	if (const FAvaShapeParametricMaterial* ParametricMtl = GetParametricMaterial())
	{
		ensureAlways(IsValid());
		
		if (FHandleData* MaterialHandleData = InHandleData.GetPtr<FHandleData>())
		{
			MaterialHandleData->OriginalBlendMode = GetBlendMode();

			const UMaterialInterface* Material = ParametricMtl->GetMaterial();

			// Can happen when Parametric material not yet initialized - but initialization is accounted for elsewhere in the Handle
			if (!Material)
			{
				Material = ParametricMtl->GetDefaultMaterial();
			}

			MaterialHandleData->OriginalMaskMaterialParameters.StoreFromMaterial(Material);
			MaterialHandleData->OriginalMaterialStyle = ParametricMtl->GetStyle();
			MaterialHandleData->OriginalBlendMode = GetBlendMode();

			return true;
		}
	}
	
	return false;
}

bool FAvaMaskParametricMaterialHandle::ApplyOriginalState(
	const FStructView& InHandleData
	, TUniqueFunction<void(UMaterialInterface*)>&& InMaterialSetter)
{
	if (FAvaShapeParametricMaterial* ParametricMtl = GetParametricMaterial())
	{
		if (const FHandleData* MaterialHandleData = InHandleData.GetPtr<const FHandleData>())
		{
			SetBlendMode(MaterialHandleData->OriginalBlendMode);
			
			SetMaskParameters(nullptr
				, MaterialHandleData->OriginalMaskMaterialParameters.BaseOpacity
				, MaterialHandleData->OriginalMaskMaterialParameters.bInvert
				, MaterialHandleData->OriginalMaskMaterialParameters.Padding
				, MaterialHandleData->OriginalMaskMaterialParameters.bApplyFeathering
				, MaterialHandleData->OriginalMaskMaterialParameters.OuterFeatherRadius
				, MaterialHandleData->OriginalMaskMaterialParameters.InnerFeatherRadius);

			ParametricMtl->SetStyle(MaterialHandleData->OriginalMaterialStyle);
			ParametricMtl->SetTranslucency(MaterialHandleData->OriginalBlendMode == EBlendMode::BLEND_Translucent ? EAvaShapeParametricMaterialTranslucency::Enabled : EAvaShapeParametricMaterialTranslucency::Auto);

			InMaterialSetter(ParametricMtl->GetMaterial());

			return true;
		}
	}

	return false;
}

bool FAvaMaskParametricMaterialHandle::ApplyModifiedState(
	const FAvaMask2DSubjectParameters& InModifiedParameters
	, const FStructView& InHandleData
	, TUniqueFunction<void(UMaterialInterface*)>&& InMaterialSetter)
{
	if (FAvaShapeParametricMaterial* ParametricMtl = GetParametricMaterial())
	{
		if (InHandleData.GetPtr<const FHandleData>())
		{
			SetBlendMode(InModifiedParameters.MaterialParameters.BlendMode);

			SetMaskParameters(nullptr
				, InModifiedParameters.MaterialParameters.BaseOpacity
				, InModifiedParameters.MaterialParameters.bInvert
				, InModifiedParameters.MaterialParameters.Padding
				, InModifiedParameters.MaterialParameters.bApplyFeathering
				, InModifiedParameters.MaterialParameters.OuterFeatherRadius
				, InModifiedParameters.MaterialParameters.InnerFeatherRadius);

			InMaterialSetter(ParametricMtl->GetMaterial());

			return true;
		}
	}

	return false;
}

bool FAvaMaskParametricMaterialHandle::IsValid() const
{
	return FAvaParametricMaterialHandle::IsValid();
}

bool FAvaMaskParametricMaterialHandle::IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag)
{
	return InTag == UE::AvaMask::Internal::HandleTag && FAvaParametricMaterialHandle::IsSupported(InInstance, InTag);
}

UMaterialInstanceDynamic* FAvaMaskParametricMaterialHandle::GetMaterialInstance()
{
	return FAvaParametricMaterialHandle::GetMaterialInstance();
}

void FAvaMaskParametricMaterialHandle::OnMaterialChanged(const FAvaShapeParametricMaterial& InMaterial)
{
	if ((&InMaterial) != GetParametricMaterial())
	{
		return;
	}

	if (const FAvaShapeParametricMaterial* ParametricMtl = GetParametricMaterial())
	{
		if (UMaterialInstanceDynamic* MaterialInstance = ParametricMtl->GetMaterial())
		{
			const FMaterialShadingModelField ShadingModel = MaterialInstance->GetShadingModels();
			const EBlendMode BlendMode = MaterialInstance->GetBlendMode();

			if (UMaterialInstance* LastMaterial = LastMaterialInstance.Get())
			{
				MaterialInstance->CopyInterpParameters(LastMaterial);	
			}

			LastShadingModel = ShadingModel;
			LastBlendMode = BlendMode;
			LastMaterialInstance = MaterialInstance;
		}

		if (UMaterialInstanceDynamic* ChildMtl = ChildMaterial.Get())
		{
			// Write the changed params to the Child Material
			ParametricMtl->SetMaterialParameterValues(ChildMtl);
		}
	}
}
