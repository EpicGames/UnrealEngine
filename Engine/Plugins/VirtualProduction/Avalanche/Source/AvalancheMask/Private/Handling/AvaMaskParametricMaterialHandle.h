// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaShapesDefs.h"
#include "Engine/EngineTypes.h"
#include "Handling/AvaParametricMaterialHandle.h"

#include "AvaMaskParametricMaterialHandle.generated.h"

struct FAvaShapeParametricMaterial;

USTRUCT()
struct FAvaMaskParametricMaterialData
{
	GENERATED_BODY()

	UPROPERTY()
	FAvaMask2DMaterialParameters OriginalMaskMaterialParameters;

	UPROPERTY()
	TEnumAsByte<EBlendMode> OriginalBlendMode = EBlendMode::BLEND_Opaque;

	UPROPERTY()
	EAvaShapeParametricMaterialStyle OriginalMaterialStyle = EAvaShapeParametricMaterialStyle::Solid;
};

class FAvaMaskParametricMaterialHandle
	: public FAvaParametricMaterialHandle
	, public TAvaMaskMaterialHandle<FAvaMaskParametricMaterialData>
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaMaskParametricMaterialHandle, FAvaParametricMaterialHandle);
	
	explicit FAvaMaskParametricMaterialHandle(const FAvaMaskMaterialReference& InParametricMaterial);
	virtual ~FAvaMaskParametricMaterialHandle() override;

	// ~Begin IAvaMaskMaterialHandle
	virtual bool GetMaskParameters(FAvaMask2DMaterialParameters& OutParameters) override;
	virtual bool SetMaskParameters(UTexture* InTexture, float InBaseOpacity, bool bInInverted, const FVector2f& InPadding, bool bInApplyFeathering, float InOuterFeathering, float InInnerFeathering) override;
	virtual const EBlendMode GetBlendMode() override;
	virtual void SetBlendMode(const EBlendMode InBlendMode) override;
	virtual void SetChildMaterial(UMaterialInstanceDynamic* InChildMaterial) override;
	virtual bool SaveOriginalState(const FStructView& InHandleData) override;
	virtual bool ApplyOriginalState(const FStructView& InHandleData
		, TUniqueFunction<void(UMaterialInterface*)>&& InMaterialSetter) override;
	virtual bool ApplyModifiedState(const FAvaMask2DSubjectParameters& InModifiedParameters
		, const FStructView& InHandleData
		, TUniqueFunction<void(UMaterialInterface*)>&& InMaterialSetter) override;
	// ~End IAvaMaskMaterialHandle

	// ~Begin IAvaMaterialHandle
	virtual FString GetMaterialName() override { return Super::GetMaterialName(); }
	virtual UMaterialInterface* GetMaterial() override { return FAvaParametricMaterialHandle::GetMaterial(); }
	virtual void CopyParametersFrom(UMaterialInstance* InSourceMaterial) override { FAvaParametricMaterialHandle::CopyParametersFrom(InSourceMaterial); }
	virtual void SetParentMaterial(UMaterialInterface* InMaterial) override { FAvaParametricMaterialHandle::SetParentMaterial(InMaterial); }
	virtual UMaterialInterface* GetParentMaterial() override { return FAvaParametricMaterialHandle::GetParentMaterial(); }
	// ~End IAvaMaterialHandle

	// ~Begin IAvaObjectHandle
	virtual bool IsValid() const override;
	// ~End IAvaObjectHandle

	static bool IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag = NAME_None);

protected:
	virtual UMaterialInstanceDynamic* GetMaterialInstance() override;
	
	void OnMaterialChanged(const FAvaShapeParametricMaterial& InMaterial);

private:
	FMaterialShadingModelField LastShadingModel;
	EBlendMode LastBlendMode;
	TWeakObjectPtr<UMaterialInstance> LastMaterialInstance;
	TWeakObjectPtr<UMaterialInstanceDynamic> ChildMaterial = nullptr;

	FDelegateHandle OnMaterialParameterChangedHandle;
};
