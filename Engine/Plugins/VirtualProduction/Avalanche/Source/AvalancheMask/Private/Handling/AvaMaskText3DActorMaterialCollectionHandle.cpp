// Copyright Epic Games, Inc. All Rights Reserved.

#include "Handling/AvaMaskText3DActorMaterialCollectionHandle.h"

#include "AvaMaskLog.h"
#include "AvaMaskMaterialReference.h"
#include "AvaMaskUtilities.h"
#include "AvaObjectHandleSubsystem.h"
#include "Components/PrimitiveComponent.h"
#include "Extensions/Text3DDefaultMaterialExtension.h"
#include "GameFramework/Actor.h"
#include "Handling/AvaHandleUtilities.h"
#include "IAvaMaskMaterialCollectionHandle.h"
#include "IAvaMaskMaterialHandle.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Text3DComponent.h"

namespace UE::AvaMask::Private
{
	EText3DMaterialBlendMode GetTargetTranslucencyType(
		const EText3DMaterialBlendMode InFromMaterial
		, const EBlendMode InRequired)
	{
		if (InRequired == EBlendMode::BLEND_Opaque)
		{
			return InFromMaterial;
		}

		if (InFromMaterial == EText3DMaterialBlendMode::Opaque)
		{
			return EText3DMaterialBlendMode::Translucent;
		}

		return InFromMaterial;
	}
}

FAvaMaskText3DActorMaterialCollectionHandle::FAvaMaskText3DActorMaterialCollectionHandle(AActor* InActor)
	: WeakActor(InActor)
	, WeakComponent(InActor->GetComponentByClass<UText3DComponent>())
{
	if (UText3DComponent* Text3DComponent = WeakComponent.Get())
	{
		Text3DComponent->OnTextPostUpdate().AddRaw(this, &FAvaMaskText3DActorMaterialCollectionHandle::OnTextPostUpdate);
	}
}

FAvaMaskText3DActorMaterialCollectionHandle::~FAvaMaskText3DActorMaterialCollectionHandle()
{
	if (UText3DComponent* Text3DComponent = WeakComponent.Get())
	{
		Text3DComponent->OnTextPostUpdate().RemoveAll(this);
	}
}

TArray<TObjectPtr<UMaterialInterface>> FAvaMaskText3DActorMaterialCollectionHandle::GetMaterials()
{
	TArray<TObjectPtr<UMaterialInterface>> Materials;

	if (const UText3DComponent* TextComponent = GetComponent())
	{
		const UText3DMaterialExtensionBase* MaterialExtension = TextComponent->GetMaterialExtension();
		Materials.Reserve(GetNumMaterials());

		MaterialExtension->ForEachMaterial([&Materials](const UE::Text3D::Material::FMaterialParameters& InParameters, UMaterialInterface* InMaterial)->bool
		{
			Materials.Add(InMaterial);
			return true; // continue
		});
	}

	return Materials;
}

TArray<TSharedPtr<IAvaMaskMaterialHandle>> FAvaMaskText3DActorMaterialCollectionHandle::GetMaterialHandles()
{
	if (const UText3DComponent* TextComponent = GetComponent())
	{
		const UText3DMaterialExtensionBase* MaterialExtension = TextComponent->GetMaterialExtension();

		const int32 MaterialCount = GetNumMaterials();

		if (MaterialHandles.Num() != MaterialCount
			|| MaterialHandles.ContainsByPredicate([](const TSharedPtr<IAvaMaskMaterialHandle>& InHandle)
			{
				return !UE::Ava::Internal::IsHandleValid(InHandle);
			}))
		{
			UAvaObjectHandleSubsystem* HandleSubsystem = UE::Ava::Internal::GetObjectHandleSubsystem();
			if (!HandleSubsystem)
			{
				return MaterialHandles;
			}

			MaterialHandles.SetNum(MaterialCount);
			int32 MaterialIndex = 0;
			MaterialExtension->ForEachMaterial([this, HandleSubsystem, &MaterialIndex](const UE::Text3D::Material::FMaterialParameters& InParameters, UMaterialInterface* InMaterial)->bool
			{
				if (!MaterialHandles[MaterialIndex] || MaterialHandles[MaterialIndex]->GetMaterial() != InMaterial)
				{
					MaterialHandles[MaterialIndex] = HandleSubsystem->MakeMaterialHandle<IAvaMaskMaterialHandle>(InMaterial, UE::AvaMask::Internal::HandleTag);
				}

				MaterialIndex++;
				return true; // continue
			});
		}
	}

	return MaterialHandles;
}

void FAvaMaskText3DActorMaterialCollectionHandle::SetMaterial(
	const FSoftComponentReference& InComponent
	, const int32 InSlotIdx
	, UMaterialInterface* InMaterial)
{
	if (UText3DComponent* TextComponent = GetComponent())
	{
		UText3DMaterialExtensionBase* MaterialExtension = TextComponent->GetMaterialExtension();

		int32 MaterialIndex = 0;
		bool bUpdate = false;
		MaterialExtension->ForEachMaterial([InSlotIdx, &MaterialIndex, MaterialExtension, InMaterial, &bUpdate](const UE::Text3D::Material::FMaterialParameters& InParams, UMaterialInterface* InCurrentMaterial)
			{
				if (MaterialIndex++ == InSlotIdx)
				{
					if (InCurrentMaterial != InMaterial)
					{
						MaterialExtension->SetMaterial(InParams, InMaterial);
						bUpdate = true;
					}

					return false; // Stop
				}

				return true; // continue
			}
		);

		if (bUpdate)
		{
			constexpr bool bImmediate = true;
			TextComponent->RequestUpdate(EText3DRendererFlags::Material, bImmediate);
		}
	}
}

void FAvaMaskText3DActorMaterialCollectionHandle::SetMaterials(
	const TArray<TObjectPtr<UMaterialInterface>>& InMaterials
	, const TBitArray<>& InSetToggle)
{
	if (UText3DComponent* TextComponent = GetComponent())
	{
		const int32 MaterialCount = GetNumMaterials();

		if (!ensure(InMaterials.Num() == MaterialCount))
		{
			UE_LOGF(LogAvaMask, Warning, "Expected %i materials, got %u", MaterialCount, InMaterials.Num());
			return;
		}

		if (!ensure(InSetToggle.Num() == MaterialCount))
		{
			UE_LOGF(LogAvaMask, Warning, "Expected %i materials, got %u", MaterialCount, InSetToggle.Num());
			return;
		}

		UText3DMaterialExtensionBase* MaterialExtension = TextComponent->GetMaterialExtension();

		int32 MaterialIndex = 0;
		bool bUpdate = false;
		MaterialExtension->ForEachMaterial([&InMaterials, &InSetToggle, &MaterialIndex, MaterialExtension, &bUpdate](const UE::Text3D::Material::FMaterialParameters& InParams, UMaterialInterface* InCurrentMaterial)
			{
				if (!InMaterials.IsValidIndex(MaterialIndex))
				{
					UE_LOGF(LogAvaMask, Warning, "Expected index %i materials, got %u", MaterialIndex, InMaterials.Num());
					return false; // stop
				}

				if (InSetToggle[MaterialIndex])
				{
					UMaterialInterface* Material = InMaterials[MaterialIndex];
					
					if (InCurrentMaterial != Material)
					{
						MaterialExtension->SetMaterial(InParams, Material);
						bUpdate = true;
					}
				}

				MaterialIndex++;
				return true; // continue
			}
		);

		if (bUpdate)
		{
			constexpr bool bImmediate = true;
			TextComponent->RequestUpdate(EText3DRendererFlags::Material, bImmediate);
		}
	}
}

int32 FAvaMaskText3DActorMaterialCollectionHandle::GetNumMaterials() const
{
	int32 MaterialCount = 0;
	if (const UText3DComponent* TextComponent = GetComponent())
	{
		if (const UText3DMaterialExtensionBase* MaterialExtension = TextComponent->GetMaterialExtension())
		{
			MaterialExtension->ForEachMaterial([&MaterialCount](const UE::Text3D::Material::FMaterialParameters&, UMaterialInterface*)
				{
					MaterialCount++;
					return true; // Continue
				}
			);
		}
	}
	return MaterialCount;
}

void FAvaMaskText3DActorMaterialCollectionHandle::ForEachMaterial(
	TFunctionRef<bool(
		const FSoftComponentReference& InComponent
		, const int32 InIdx
		, UMaterialInterface* InMaterial)> InFunction)
{
	if (const UText3DComponent* TextComponent = GetComponent())
	{
		const FSoftComponentReference ComponentReference = UE::AvaMask::Internal::MakeComponentReference(TextComponent->GetOwner(), TextComponent);

		const UText3DMaterialExtensionBase* MaterialExtension = TextComponent->GetMaterialExtension();
		int32 Index = 0;
		MaterialExtension->ForEachMaterial([&InFunction, &ComponentReference, &Index](const UE::Text3D::Material::FMaterialParameters& InParameters, UMaterialInterface* InMaterial)
		{
			return InFunction(ComponentReference, Index++, InMaterial);
		});
	}
}

void FAvaMaskText3DActorMaterialCollectionHandle::ForEachMaterialHandle(TFunctionRef<bool(const FSoftComponentReference& InComponent, const int32 InIdx, const bool bInIsSlotOccupied, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)> InFunction)
{
	if (const UText3DComponent* TextComponent = GetComponent())
	{
		// Effectively refreshes material handles if necessary
		GetMaterialHandles();

		const int32 MaterialCount = GetNumMaterials();

		if (MaterialHandles.Num() != MaterialCount)
		{
			return;
		}

		const FSoftComponentReference ComponentReference = UE::AvaMask::Internal::MakeComponentReference(TextComponent->GetOwner(), TextComponent);

		int32 Index = 0;
		const TArray<TSharedPtr<IAvaMaskMaterialHandle>> MaterialHandlesCopy(MaterialHandles);
		for (const TSharedPtr<IAvaMaskMaterialHandle>& MaterialHandle : MaterialHandlesCopy)
		{
			if (!InFunction(ComponentReference, Index++, MaterialHandle.IsValid(), MaterialHandle))
			{
				return;
			}
		}
	}
}

void FAvaMaskText3DActorMaterialCollectionHandle::MapEachMaterial(
	TFunctionRef<UMaterialInterface*(
		const FSoftComponentReference& InComponent
		, const int32 InIdx
		, UMaterialInterface* InMaterial)> InFunction)
{	
	if (const UText3DComponent* TextComponent = GetComponent())
	{
		const int32 MaterialCount = GetNumMaterials();

		TArray<TObjectPtr<UMaterialInterface>> MappedMaterials;
		MappedMaterials.Init(nullptr, MaterialCount);

		TBitArray<> SetFlags;
        SetFlags.Init(false, MaterialCount);

		const FSoftComponentReference ComponentReference = UE::AvaMask::Internal::MakeComponentReference(TextComponent->GetOwner(), TextComponent);

		const UText3DMaterialExtensionBase* MaterialExtension = TextComponent->GetMaterialExtension();

		int32 MaterialIndex = 0;
		MaterialExtension->ForEachMaterial([&InFunction, &ComponentReference, &MaterialIndex, &MappedMaterials, &SetFlags](const UE::Text3D::Material::FMaterialParameters& InParameters, UMaterialInterface* InMaterial)
		{
			MappedMaterials[MaterialIndex] = InFunction(ComponentReference, MaterialIndex, InMaterial);
			SetFlags[MaterialIndex] = MappedMaterials[MaterialIndex] != nullptr;
			MaterialIndex++;
			return true; // continue
		});

		SetMaterials(MappedMaterials, SetFlags);
	}
}

void FAvaMaskText3DActorMaterialCollectionHandle::MapEachMaterialHandle(TFunctionRef<TSharedPtr<IAvaMaskMaterialHandle>(const FSoftComponentReference& InComponent, const int32 InIdx, const bool bInIsSlotOccupied, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)> InFunction)
{
	if (const UText3DComponent* TextComponent = GetComponent())
	{
		// Effectively refreshes material handles if necessary
		GetMaterialHandles();

		const int32 MaterialCount = GetNumMaterials();

		TArray<TObjectPtr<UMaterialInterface>> MappedMaterials;
		MappedMaterials.SetNum(MaterialCount);

		TBitArray<> SetFlags;
		SetFlags.Init(false, MaterialCount);

		const FSoftComponentReference ComponentReference = UE::AvaMask::Internal::MakeComponentReference(TextComponent->GetOwner(), TextComponent);

		const UText3DMaterialExtensionBase* MaterialExtension = TextComponent->GetMaterialExtension();

		int32 MaterialIndex = 0;
		MaterialExtension->ForEachMaterial([this, &InFunction, &ComponentReference, &MaterialIndex, &MappedMaterials, &SetFlags](const UE::Text3D::Material::FMaterialParameters& InParameters, UMaterialInterface* InMaterial)
		{
			if (!MaterialHandles.IsValidIndex(MaterialIndex))
			{
				UE_LOGF(LogAvaMask, Warning, "Expected index %i materials, got %u", MaterialIndex, MaterialHandles.Num());
				return false; // stop
			}
				
			TSharedPtr<IAvaMaskMaterialHandle>& MaterialHandle = MaterialHandles[MaterialIndex];
			const TSharedPtr<IAvaMaskMaterialHandle> MappedMaterialHandle = InFunction(ComponentReference, MaterialIndex, true, MaterialHandle);

			UMaterialInterface* RemappedMaterial = nullptr;
			if (MappedMaterialHandle.IsValid())
			{
				RemappedMaterial = MappedMaterialHandle->GetMaterial();	
			}
			else if (MaterialHandle.IsValid())
			{
				RemappedMaterial = MaterialHandle->GetMaterial();
			}

			MappedMaterials[MaterialIndex] = RemappedMaterial;
			SetFlags[MaterialIndex] = RemappedMaterial != nullptr;
			MaterialIndex++;
			return true; // continue
		});

		SetMaterials(MappedMaterials, SetFlags);
	}
}

bool FAvaMaskText3DActorMaterialCollectionHandle::IsValid() const
{
	return WeakActor.IsValid() && WeakComponent.IsValid();
}

bool FAvaMaskText3DActorMaterialCollectionHandle::SaveOriginalState(const FStructView& InHandleData)
{
	if (FHandleData* HandleData = InHandleData.GetPtr<FHandleData>())
	{
		ForEachMaterialHandle([this, HandleData](
			const FSoftComponentReference& InComponent
			, const int32 InSlotIdx
			, const bool bInIsSlotOccupied
			, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)
		{
			if (InMaterialHandle.IsValid())
			{
				FInstancedStruct& MaterialHandleData = HandleData->GroupMaterialData.FindOrAdd(InSlotIdx, InMaterialHandle->MakeDataStruct());
				return InMaterialHandle->SaveOriginalState(MaterialHandleData);
			}

			return false;
		});

		if (const UText3DComponent* TextComponent = GetComponent())
		{
			if (const UText3DDefaultMaterialExtension* MaterialExtension = TextComponent->GetCastedMaterialExtension<UText3DDefaultMaterialExtension>())
			{
				HandleData->BlendMode = MaterialExtension->GetBlendMode();
			}

			return true;
		}
		
		return true;
	}

	return false;
}

bool FAvaMaskText3DActorMaterialCollectionHandle::ApplyOriginalState(const FStructView& InHandleData)
{
	if (const FHandleData* HandleData = InHandleData.GetPtr<FHandleData>())
	{
		if (UText3DComponent* TextComponent = GetComponent())
		{
			// Only restore if the original was NOT translucent AND the current alpha is 1.0
			if (UText3DDefaultMaterialExtension* MaterialExtension = TextComponent->GetCastedMaterialExtension<UText3DDefaultMaterialExtension>())
			{
				if (HandleData->BlendMode != MaterialExtension->GetBlendMode())
				{
					MaterialExtension->SetBlendMode(HandleData->BlendMode);

					// Refresh materials immediately
					constexpr bool bImmediateUpdate = true;
					TextComponent->RequestUpdate(EText3DRendererFlags::Material, bImmediateUpdate);

					// Need to refresh handles since blend mode didn't match
					MaterialHandles.Reset();
				}
			}

			if (Super::ApplyOriginalState(InHandleData))
			{
				return true;
			}
		}
	}

	return false;
}

bool FAvaMaskText3DActorMaterialCollectionHandle::ApplyModifiedState(const FAvaMask2DSubjectParameters& InModifiedParameters,
	const FStructView& InHandleData)
{
	if (InHandleData.GetPtr<FHandleData>())
	{
		if (UText3DComponent* TextComponent = GetComponent())
		{
			if (UText3DDefaultMaterialExtension* MaterialExtension = TextComponent->GetCastedMaterialExtension<UText3DDefaultMaterialExtension>())
			{
				const EText3DMaterialBlendMode TargetTranslucencyStyle = UE::AvaMask::Private::GetTargetTranslucencyType(MaterialExtension->GetBlendMode(), InModifiedParameters.MaterialParameters.BlendMode);

				if (MaterialExtension->GetBlendMode() != TargetTranslucencyStyle)
				{
					MaterialExtension->SetBlendMode(TargetTranslucencyStyle);

					// Refresh materials immediately
					constexpr bool bImmediateUpdate = true;
					TextComponent->RequestUpdate(EText3DRendererFlags::Material, bImmediateUpdate);

					// Need to refresh handles since blend mode didn't match
					MaterialHandles.Reset();
				}
			}
		}

		if (!Super::ApplyModifiedState(InModifiedParameters, InHandleData))
		{
			return false;
		}

		return true;
	}

	return false;
}

bool FAvaMaskText3DActorMaterialCollectionHandle::IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag)
{
	if (InTag == UE::AvaMask::Internal::HandleTag)
	{
		if (const AActor* Actor = InInstance.GetTypedObject<AActor>())
		{
			return Actor->GetComponentByClass<UText3DComponent>() != nullptr;
		}
	}

	return false;
}

FStructView FAvaMaskText3DActorMaterialCollectionHandle::GetMaterialHandleData(
	FHandleData* InParentHandleData
	, const FSoftComponentReference& InComponent
	, const int32 InSlotIdx)
{
	if (InParentHandleData)
	{
		if (FInstancedStruct* Value = InParentHandleData->GroupMaterialData.Find(InSlotIdx))
		{
			return *Value;
		}
	}

	return nullptr;
}

FStructView FAvaMaskText3DActorMaterialCollectionHandle::GetOrAddMaterialHandleData(
	FHandleData* InParentHandleData
	, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle
	, const FSoftComponentReference& InComponent
	, const int32 InSlotIdx)
{
	if (InParentHandleData)
	{
		return InParentHandleData->GroupMaterialData.FindOrAdd(InSlotIdx, InMaterialHandle->MakeDataStruct());
	}

	return nullptr;
}

UText3DComponent* FAvaMaskText3DActorMaterialCollectionHandle::GetComponent() const
{
	if (UText3DComponent* Component = WeakComponent.Get())
	{
		return Component;
	}

	if (const AActor* Actor = WeakActor.Get())
	{
		UText3DComponent* Component = Actor->GetComponentByClass<UText3DComponent>();
		FAvaMaskText3DActorMaterialCollectionHandle* MutableThis = const_cast<FAvaMaskText3DActorMaterialCollectionHandle*>(this);
		MutableThis->WeakComponent = Component;
		return Component;
	}

	return nullptr;
}

void FAvaMaskText3DActorMaterialCollectionHandle::OnTextPostUpdate(UText3DComponent* InComponent, EText3DRendererFlags InFlags)
{
	if (EnumHasAnyFlags(InFlags, EText3DRendererFlags::Material))
	{
		auto HasInvalidHandles = [this]()
		{
			return MaterialHandles.ContainsByPredicate([](const TSharedPtr<IAvaMaskMaterialHandle>& InHandle)
			{
				return !UE::Ava::Internal::IsHandleValid(InHandle);
			});
		};

		UText3DMaterialExtensionBase* MaterialExtension = InComponent->GetMaterialExtension();
		auto CheckMaterialsMatch = [this, MaterialExtension]()
		{
			if (MaterialHandles.Num() != GetNumMaterials())
			{
				return false;
			}

			int32 MaterialIndex = 0;
			bool bMatch = true;
			MaterialExtension->ForEachMaterial([this, &bMatch, &MaterialIndex](const UE::Text3D::Material::FMaterialParameters& InParameters, UMaterialInterface* InMaterial)
			{
				UMaterialInterface* Material = MaterialHandles[MaterialIndex]->GetMaterial();
				if (Material != InMaterial)
				{
					bMatch = false;
				}

				MaterialIndex++;
				return bMatch;
			});
	
			return bMatch;
		};

		const bool bHasInvalidHandles = HasInvalidHandles();
		const bool bAllMaterialsMatch = !bHasInvalidHandles && CheckMaterialsMatch();
	
		if (!bAllMaterialsMatch)
		{
			// refresh handles
			MaterialHandles.Empty();
			GetMaterialHandles();

			if (!HasInvalidHandles())
			{
				OnSourceMaterialsChanged().ExecuteIfBound(InComponent, MaterialHandles);
			}
		}
	}
}
