// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTextDefs.h"
#include "Containers/Array.h"
#include "Engine/EngineTypes.h"
#include "IAvaMaskMaterialCollectionHandle.h"
#include "IAvaObjectHandle.h"
#include "Templates/Function.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "AvaMaskText3DActorMaterialCollectionHandle.generated.h"

class UAvaText3DComponent;
class IAvaMaskMaterialHandle;
class UText3DComponent;
class AText3DActor;
class AActor;
class UMaterialInterface;
enum class EText3DRendererFlags : uint8;
struct FAvaMaskMaterialReference;

USTRUCT()
struct FAvaMaskText3DActorMaterialCollectionHandleData
{
	GENERATED_BODY()

	UPROPERTY()
	EText3DMaterialBlendMode BlendMode = EText3DMaterialBlendMode::Opaque;

	UPROPERTY()
	TMap<int32, FInstancedStruct> GroupMaterialData;
};

/** Note that this doesn't explicitly require a Text3D Actor, it just checks that whatever Actor is given has a UText3DComponent (or derived). */
class FAvaMaskText3DActorMaterialCollectionHandle
	: public TAvaMaskMaterialCollectionHandle<FAvaMaskText3DActorMaterialCollectionHandleData>
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaMaskText3DActorMaterialCollectionHandle, TAvaMaskMaterialCollectionHandle<FAvaMaskText3DActorMaterialCollectionHandleData>);

	explicit FAvaMaskText3DActorMaterialCollectionHandle(AActor* InActor);
	virtual ~FAvaMaskText3DActorMaterialCollectionHandle() override;

	// ~Begin IAvaMaskMaterialCollectionProvider
	virtual TArray<TObjectPtr<UMaterialInterface>> GetMaterials() override;

	virtual TArray<TSharedPtr<IAvaMaskMaterialHandle>> GetMaterialHandles() override;

	virtual void SetMaterial(const FSoftComponentReference& InComponent, const int32 InSlotIdx, UMaterialInterface* InMaterial) override;
	virtual void SetMaterials(const TArray<TObjectPtr<UMaterialInterface>>& InMaterials, const TBitArray<>& InSetToggle) override;
	
	virtual int32 GetNumMaterials() const override;

	virtual void ForEachMaterial(
		TFunctionRef<bool(
			const FSoftComponentReference& InComponent
			, const int32 InIdx
			, UMaterialInterface* InMaterial)> InFunction) override;

	virtual void ForEachMaterialHandle(
		TFunctionRef<bool(
			const FSoftComponentReference& InComponent
			, const int32 InIdx
			, const bool bInIsSlotOccupied
			, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)> InFunction) override;

	virtual void MapEachMaterial(
		TFunctionRef<UMaterialInterface*(
			const FSoftComponentReference& InComponent
			, const int32 InIdx
			, UMaterialInterface* InMaterial)> InFunction) override;

	virtual void MapEachMaterialHandle(
		TFunctionRef<TSharedPtr<IAvaMaskMaterialHandle>(
			const FSoftComponentReference& InComponent
			, const int32 InIdx
			, const bool bInIsSlotOccupied
			, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)> InFunction) override;

	virtual bool IsValid() const override;
	virtual bool SaveOriginalState(const FStructView& InHandleData) override;
	virtual bool ApplyOriginalState(const FStructView& InHandleData) override;
	virtual bool ApplyModifiedState(const FAvaMask2DSubjectParameters& InModifiedParameters, const FStructView& InHandleData) override;
	// ~End IAvaMaskMaterialCollectionProvider

	static bool IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag = NAME_None);

protected:
	// ~Begin TAvaMaskMaterialCollectionHandle
	virtual FStructView GetMaterialHandleData(
		FHandleData* InParentHandleData
		, const FSoftComponentReference& InComponent
		, const int32 InSlotIdx) override;
	
	virtual ::FStructView GetOrAddMaterialHandleData(
		FHandleData* InParentHandleData
		, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle
		, const FSoftComponentReference& InComponent
		, const int32 InSlotIdx) override;
	// ~End TAvaMaskMaterialCollectionHandle
	
	UText3DComponent* GetComponent() const;

	void OnTextPostUpdate(UText3DComponent* InComponent, EText3DRendererFlags InFlags);

	TWeakObjectPtr<AActor> WeakActor;
	TWeakObjectPtr<UText3DComponent> WeakComponent;
	TArray<TSharedPtr<IAvaMaskMaterialHandle>> MaterialHandles;
};
