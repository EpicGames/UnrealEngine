// Copyright Epic Games, Inc. All Rights Reserved.

#include "Handling/AvaMaterialInstanceHandle.h"

#include "AvaMaskMaterialReference.h"
#include "Engine/Texture.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "UObject/Package.h"

FAvaMaterialInstanceHandle::FAvaMaterialInstanceHandle(const TWeakObjectPtr<UMaterialInterface>& InWeakParentMaterial)
{
	// Use base material as parent if material is a MID
	if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(InWeakParentMaterial.Get()))
	{
		WeakMaterialInstance = MID;
		WeakParentMaterial = MID->GetBaseMaterial();
	}
	else
	{
		WeakParentMaterial = InWeakParentMaterial;
	}
}

FString FAvaMaterialInstanceHandle::GetMaterialName()
{
	if (const UMaterialInterface* MaterialInstance = GetMaterial())
	{
		return MaterialInstance->GetName();
	}

	return { };
}

UMaterialInterface* FAvaMaterialInstanceHandle::GetMaterial()
{
	if (UMaterialInterface* MaterialInstance = GetMaterialInstance())
	{
		return MaterialInstance;
	}

	return WeakParentMaterial.Get();
}

UMaterialInterface* FAvaMaterialInstanceHandle::GetParentMaterial()
{
	if (UMaterialInterface* ParentMaterial = WeakParentMaterial.Get())
	{
		return ParentMaterial;
	}

	if (UMaterialInstanceDynamic* MID = WeakMaterialInstance.Get())
	{
		return MID->GetBaseMaterial();
	}

	return nullptr;
}

void FAvaMaterialInstanceHandle::CopyParametersFrom(UMaterialInstance* InSourceMaterial)
{
	if (UMaterialInstanceDynamic* MaterialInstance = GetMaterialInstance())
	{
		MaterialInstance->CopyInterpParameters(InSourceMaterial);
	}
}

void FAvaMaterialInstanceHandle::SetParentMaterial(UMaterialInterface* InMaterial)
{
	if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(InMaterial))
	{
		WeakParentMaterial = MID->GetBaseMaterial();
	}
	else
	{
		WeakParentMaterial = InMaterial;
	}

	if (UMaterialInstanceDynamic* MID = WeakMaterialInstance.Get())
	{
		if (MID->GetBaseMaterial() != WeakParentMaterial.Get())
		{
			WeakMaterialInstance.Reset();
		}
	}
}

bool FAvaMaterialInstanceHandle::IsValid() const
{
	// @note: MaterialInstance doesn't have to be valid - this handle should deal with MID creation
	return WeakParentMaterial.IsValid() || WeakMaterialInstance.IsValid();
}

bool FAvaMaterialInstanceHandle::IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag)
{
	return !!InInstance.GetTypedObject<UMaterialInterface>();
}

UMaterialInstanceDynamic* FAvaMaterialInstanceHandle::GetMaterialInstance()
{
	if (UMaterialInstanceDynamic* MaterialInstance = WeakMaterialInstance.Get())
	{
		return MaterialInstance;
	}

	return nullptr;
}

UMaterialInstanceDynamic* FAvaMaterialInstanceHandle::GetOrCreateMaterialInstance()
{
	if (UMaterialInstanceDynamic* MaterialInstance = GetMaterialInstance())
	{
		return MaterialInstance;
	}
	
	if (UMaterialInterface* ParentMaterial = GetParentMaterial())
	{
		UObject* Outer = GetTransientPackage();
		UMaterialInstanceDynamic* MaterialInstance = UMaterialInstanceDynamic::Create(ParentMaterial, Outer);
		WeakMaterialInstance = MaterialInstance;
		return MaterialInstance;
	}

	return nullptr;
}
