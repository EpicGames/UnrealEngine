// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "IAvaMaterialHandle.h"
#include "IAvaObjectHandle.h"
#include "Materials/MaterialParameters.h"
#include "Misc/TVariant.h"
#include "StructUtils/StructView.h"

class UMaterialInstance;
class UMaterialInstanceDynamic;
class UMaterialInterface;
struct FAvaMaskMaterialReference;

class FAvaMaterialInstanceHandle
	: public IAvaMaterialHandle
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaMaterialInstanceHandle, IAvaMaterialHandle);

	static bool IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag = NAME_None);

	explicit FAvaMaterialInstanceHandle(const TWeakObjectPtr<UMaterialInterface>& InWeakParentMaterial);
	virtual ~FAvaMaterialInstanceHandle() override = default;

	// ~Begin IAvaMaterialHandle
	virtual FString GetMaterialName() override;
	virtual UMaterialInterface* GetMaterial() override;
	virtual UMaterialInterface* GetParentMaterial() override;
	virtual void CopyParametersFrom(UMaterialInstance* InSourceMaterial) override;
	virtual void SetParentMaterial(UMaterialInterface* InMaterial) override;
	// ~End IAvaMaterialHandle

	// ~Begin IAvaObjectHandle
	virtual bool IsValid() const override;
	// ~End IAvaObjectHandle

protected:
	virtual UMaterialInstanceDynamic* GetMaterialInstance();
	virtual UMaterialInstanceDynamic* GetOrCreateMaterialInstance();

	TWeakObjectPtr<UMaterialInterface> WeakParentMaterial;
	TWeakObjectPtr<UMaterialInstanceDynamic> WeakMaterialInstance;
};
