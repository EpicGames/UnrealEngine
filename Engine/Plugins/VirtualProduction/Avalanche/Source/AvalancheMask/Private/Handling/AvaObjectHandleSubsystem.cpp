// Copyright Epic Games, Inc. All Rights Reserved.

#include "Handling/AvaObjectHandleSubsystem.h"

#include "AvaHandleUtilities.h"
#include "AvaMaskActorMaterialCollectionHandle.h"
#include "AvaMaskAvaShapeMaterialCollectionHandle.h"
#include "AvaMaskLog.h"
#include "AvaMaskMaterialReference.h"
#include "AvaMaskMediaPlateMaterialHandle.h"
#include "AvaMaskText3DActorMaterialCollectionHandle.h"
#include "Engine/Engine.h"
#include "GameFramework/Actor.h"
#include "Handling/AvaMaskDesignedMaterialHandle.h"
#include "Handling/AvaMaskMaterialInstanceHandle.h"
#include "Handling/AvaMaskParametricMaterialHandle.h"
#include "IAvaMaskMaterialCollectionHandle.h"
#include "Material/DynamicMaterialInstance.h"
#include "Materials/MaterialInterface.h"
#include "UObject/UObjectIterator.h"

FString UAvaObjectHandleSubsystem::DefaultMaterialPath = TEXT("/Avalanche/MaskResources/M_AvalancheMaskDefaultTranslucent.M_AvalancheMaskDefaultTranslucent");

void UAvaObjectHandleSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	DefaultMaterial = TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(DefaultMaterialPath));
	FindObjectHandleFactories();
}

TSharedPtr<IAvaObjectHandle> UAvaObjectHandleSubsystem::MakeHandleDirect(const FAvaMaskMaterialReference& InInstance, FName InTag)
{
	for (const TPair<FIsSupportedFunction, FMakeHandleFunction>& SupportedFactoryPair : ObjectHandleFactories)
	{
		if (SupportedFactoryPair.Key(InInstance, InTag))
		{
			TSharedPtr<IAvaObjectHandle> Handle = SupportedFactoryPair.Value(InInstance);
			if (!UE::Ava::Internal::IsHandleValid(Handle))
			{
				if (InTag.IsNone())
				{
					UE_LOGF(LogAvaMask, Warning, "Object Handle for '%ls' was created but invalid.", *InInstance.ToString())
				}
				else
				{
					UE_LOGF(LogAvaMask, Warning, "Object Handle for '%ls' (with tag '%ls') was created but invalid.", *InInstance.ToString(), *InTag.ToString())
				}
			}
			return Handle;
		}
	}

	UE_LOGF(LogAvaMask, Verbose, "No ObjectHandle found for '%ls'", *InInstance.ToString());
	return nullptr;
}

// @note: registration order matters!
void UAvaObjectHandleSubsystem::FindObjectHandleFactories()
{
	// Material Collection Handles
	{
		ObjectHandleFactories.Add({
			FAvaMaskAvaShapeMaterialCollectionHandle::IsSupported
			, [](const FAvaMaskMaterialReference& InObject) -> TSharedPtr<IAvaMaskMaterialCollectionHandle>
			{
				return MakeShared<FAvaMaskAvaShapeMaterialCollectionHandle>(InObject.GetTypedObject<AActor>());
			}});

		ObjectHandleFactories.Add({
			FAvaMaskText3DActorMaterialCollectionHandle::IsSupported
			, [](const FAvaMaskMaterialReference& InObject) -> TSharedPtr<IAvaMaskMaterialCollectionHandle>
			{
				return MakeShared<FAvaMaskText3DActorMaterialCollectionHandle>(InObject.GetTypedObject<AActor>());
			}});

		ObjectHandleFactories.Add({
			FAvaMaskActorMaterialCollectionHandle::IsSupported
			, [](const FAvaMaskMaterialReference& InObject) -> TSharedPtr<IAvaMaskMaterialCollectionHandle>
			{
				return MakeShared<FAvaMaskActorMaterialCollectionHandle>(InObject.GetTypedObject<AActor>());
			}});
	}

	// Material Handles
	{
		ObjectHandleFactories.Add({
			FAvaMaskDesignedMaterialHandle::IsSupported
			, [](const FAvaMaskMaterialReference& InMaterial) -> TSharedPtr<IAvaMaskMaterialHandle>
			{
				return MakeShared<FAvaMaskDesignedMaterialHandle>(InMaterial.GetTypedObject<UDynamicMaterialInstance>());
			}});

		ObjectHandleFactories.Add({
			FAvaMaskParametricMaterialHandle::IsSupported
			, [](const FAvaMaskMaterialReference& InMaterial) -> TSharedPtr<IAvaMaskMaterialHandle>
			{
				return MakeShared<FAvaMaskParametricMaterialHandle>(InMaterial);
			}});

		ObjectHandleFactories.Add({
			FAvaMaskMediaPlateMaterialHandle::IsSupported
			, [](const FAvaMaskMaterialReference& InMaterial) -> TSharedPtr<IAvaMaskMaterialHandle>
			{
				return MakeShared<FAvaMaskMediaPlateMaterialHandle>(InMaterial.GetTypedObject<UMaterialInterface>());
			}});

		ObjectHandleFactories.Add({
			FAvaMaskMaterialInstanceHandle::IsSupported
			, [](const FAvaMaskMaterialReference& InMaterial) -> TSharedPtr<IAvaMaskMaterialHandle>
			{
				return MakeShared<FAvaMaskMaterialInstanceHandle>(InMaterial.GetTypedObject<UMaterialInterface>());
			}});
	}
}
