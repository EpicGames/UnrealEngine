// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMaskLog.h"
#include "AvaMaskMaterialReference.h"
#include "IAvaObjectHandle.h"
#include "Subsystems/EngineSubsystem.h"
#include "UObject/SoftObjectPtr.h"
#include "AvaObjectHandleSubsystem.generated.h"

class IAvaMaskMaterialCollectionHandle;
class IAvaMaskMaterialHandle;
class UMaterialInterface;
class UObject;

/** Responsible for providing Handlers for a given UObject. */
UCLASS()
class UAvaObjectHandleSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	static FString DefaultMaterialPath;

	//~ Begin USubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	//~ End USubsystem

	TSharedPtr<IAvaObjectHandle> MakeHandleDirect(const FAvaMaskMaterialReference& InInstance, FName InTag = NAME_None);

	/**
	 * Used to create material collection handles, Owner provided must be valid
	 */
	template <typename HandleType
		UE_REQUIRES(!std::is_const_v<HandleType> && std::is_base_of_v<IAvaMaskMaterialCollectionHandle, HandleType>)>
    TSharedPtr<HandleType> MakeCollectionHandle(UObject* InOwner, FName InTag = NAME_None)
    {
		return MakeHandle<HandleType>(InOwner, InTag);
    }

	/**
	 * Used to create material handles, material instance can be null, in that case the default material will be used
	 */
	template <typename HandleType
		UE_REQUIRES(!std::is_const_v<HandleType> && std::is_base_of_v<IAvaMaskMaterialHandle, HandleType>)>
	TSharedPtr<HandleType> MakeMaterialHandle(UMaterialInterface* InInstance, FName InTag = NAME_None)
	{
		if (!InInstance)
		{
			InInstance = DefaultMaterial.LoadSynchronous();
		}

		return MakeHandle<HandleType>(InInstance, InTag);
	}

	template <typename HandleType, typename ObjectType = UObject
	UE_REQUIRES(!std::is_const_v<HandleType> && std::is_base_of_v<IAvaObjectHandle, HandleType>
			&&	!std::is_const_v<ObjectType> && TModels_V<CStaticClassProvider, std::decay_t<ObjectType>>)>
	TSharedPtr<HandleType> MakeHandle(ObjectType* InInstance, FName InTag = NAME_None)
	{
		if (!InInstance)
		{
			UE_LOGF(LogAvaMask, Warning, "Invalid or null instance object provided to MakeHandle");
			return nullptr;
		}

		return StaticCastSharedPtr<HandleType>(MakeHandleDirect(FAvaMaskMaterialReference(InInstance), InTag));
	}

	template<typename HandleType UE_REQUIRES(!std::is_const_v<HandleType> && std::is_base_of_v<IAvaObjectHandle, HandleType>)>
	TSharedPtr<HandleType> MakeHandle(const FAvaMaskMaterialReference& InInstance, FName InTag = NAME_None)
	{
		return StaticCastSharedPtr<HandleType>(MakeHandleDirect(InInstance, InTag));
	}

private:
	void FindObjectHandleFactories();

	using FIsSupportedFunction = TFunction<bool(const FAvaMaskMaterialReference&, FName)>;
	using FMakeHandleFunction = TFunction<TSharedPtr<IAvaObjectHandle>(const FAvaMaskMaterialReference&)>;

	TArray<TPair<FIsSupportedFunction, FMakeHandleFunction>> ObjectHandleFactories;

	UPROPERTY()
	TSoftObjectPtr<UMaterialInterface> DefaultMaterial;
};
