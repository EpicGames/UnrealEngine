// Copyright Epic Games, Inc. All Rights Reserved.

#include "Handling/AvaParametricMaterialHandle.h"
#include "AvaShapeParametricMaterial.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"

namespace UE::AvaMask::Private
{
	FAvaShapeParametricMaterial* GetParametricMaterialData(const FAvaMaskMaterialReference& InParametricMaterial)
	{
		if (UAvaShapeDynamicMeshBase* ShapeComponent = InParametricMaterial.GetTypedObject<UAvaShapeDynamicMeshBase>())
		{
			return ShapeComponent->GetParametricMaterialPtr(InParametricMaterial.Index);
		}
		return nullptr;
	}

	UMaterialInterface* GetParametricMaterial(const FAvaMaskMaterialReference& InParametricMaterial)
	{
		if (const FAvaShapeParametricMaterial* ParametricMaterial = GetParametricMaterialData(InParametricMaterial))
		{
			return ParametricMaterial->GetMaterial();
		}
		return nullptr;
	}
}

FAvaParametricMaterialHandle::FAvaParametricMaterialHandle(const FAvaMaskMaterialReference& InMaterialReference)
	: FAvaMaterialInstanceHandle(UE::AvaMask::Private::GetParametricMaterial(InMaterialReference))
	, MaterialReference(InMaterialReference)
{
}

void FAvaParametricMaterialHandle::CopyParametersFrom(UMaterialInstance* InSourceMaterial)
{
	Super::CopyParametersFrom(InSourceMaterial);
	
	if (FAvaShapeParametricMaterial* ParametricMtl = GetParametricMaterial())
	{
		ParametricMtl->CopyFromMaterialParameters(InSourceMaterial);
	}
}

bool FAvaParametricMaterialHandle::IsValid() const
{
	return !!UE::AvaMask::Private::GetParametricMaterialData(MaterialReference);
}

bool FAvaParametricMaterialHandle::IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag)
{
	return !!UE::AvaMask::Private::GetParametricMaterialData(InInstance);
}

FAvaShapeParametricMaterial* FAvaParametricMaterialHandle::GetParametricMaterial()
{
	return UE::AvaMask::Private::GetParametricMaterialData(MaterialReference);
}

UMaterialInstanceDynamic* FAvaParametricMaterialHandle::GetMaterialInstance()
{
	if (const FAvaShapeParametricMaterial* ParametricMtl = GetParametricMaterial())
	{
		return ParametricMtl->GetMaterial();
	}
	return FAvaMaterialInstanceHandle::GetMaterialInstance();
}
