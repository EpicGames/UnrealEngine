// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMaskMaterialReference.h"
#include "AvaType.h"
#include "Handling/AvaMaterialInstanceHandle.h"
#include "IAvaMaskMaterialHandle.h"
#include "IAvaObjectHandle.h"

struct FAvaShapeParametricMaterial;

class FAvaParametricMaterialHandle : public FAvaMaterialInstanceHandle
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaParametricMaterialHandle, FAvaMaterialInstanceHandle);
	
	explicit FAvaParametricMaterialHandle(const FAvaMaskMaterialReference& InParametricMaterial);

	// ~Begin IAvaMaterialHandle
	virtual void CopyParametersFrom(UMaterialInstance* InSourceMaterial) override;
	// ~End IAvaMaterialHandle

	// ~Begin IAvaObjectHandle
	virtual bool IsValid() const override;
	// ~End IAvaObjectHandle

	static bool IsSupported(const FAvaMaskMaterialReference& InInstance, FName InTag = NAME_None);

protected:
	FAvaShapeParametricMaterial* GetParametricMaterial();
	virtual UMaterialInstanceDynamic* GetMaterialInstance() override;

	FAvaMaskMaterialReference MaterialReference;
};
