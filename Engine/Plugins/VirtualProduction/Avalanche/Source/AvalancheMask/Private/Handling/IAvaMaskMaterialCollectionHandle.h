// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaHandleUtilities.h"
#include "AvaObjectHandleSubsystem.h"
#include "Containers/Array.h"
#include "Engine/EngineTypes.h"
#include "IAvaMaskMaterialHandle.h"
#include "IAvaObjectHandle.h"
#include "Templates/Function.h"
#include "UObject/ObjectPtr.h"

class AActor;
class AText3DActor;
class IAvaMaskMaterialHandle;
class UAvaText3DComponent;
class UMaterialInterface;
class UText3DComponent;
struct FStructView;

/** Interface between the mask system and an arbitrary number of materials that need to be instantiated and parameters manipulated. */
class IAvaMaskMaterialCollectionHandle
	: public IAvaObjectHandle
{
public:
	UE_AVA_INHERITS(IAvaMaskMaterialCollectionHandle, IAvaObjectHandle);
	
	using FOnSourceMaterialPreAssignment = TDelegate<UMaterialInterface*(const UMaterialInterface* InPreviousMaterial, UMaterialInterface* InNewMaterial)>;
	using FOnSourceMaterialsChanged = TDelegate<void(UObject* InMaterialOwner, const TArray<TSharedPtr<IAvaMaskMaterialHandle>>& InMaterialHandles)>;
	
public:
	virtual ~IAvaMaskMaterialCollectionHandle() override = default;

	virtual FInstancedStruct MakeDataStruct() = 0;
		
	virtual TArray<TObjectPtr<UMaterialInterface>> GetMaterials() = 0;
	virtual TArray<TSharedPtr<IAvaMaskMaterialHandle>> GetMaterialHandles() = 0;
	virtual int32 GetNumMaterials() const = 0;

	virtual void SetMaterial(const FSoftComponentReference& InComponent, const int32 InSlotIdx, UMaterialInterface* InMaterial) = 0;
	virtual void SetMaterials(const TArray<TObjectPtr<UMaterialInterface>>& InMaterials) = 0;
	virtual void SetMaterials(const TArray<TObjectPtr<UMaterialInterface>>& InMaterials, const TBitArray<>& InSetToggle) = 0;

	virtual void ForEachMaterial(
		TFunctionRef<bool(
			const FSoftComponentReference& InComponent
			, const int32 InIdx
			, UMaterialInterface* InMaterial)>	InFunction) = 0;

	virtual void ForEachMaterialHandle(
		TFunctionRef<bool(
			const FSoftComponentReference& InComponent
			, const int32 InIdx
			, const bool bInIsSlotOccupied
			, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)> InFunction) = 0;

	virtual void MapEachMaterial(
		TFunctionRef<UMaterialInterface*(
			const FSoftComponentReference& InComponent
			, const int32 InIdx
			, UMaterialInterface* InMaterial)> InFunction) = 0;

	virtual void MapEachMaterialHandle(
		TFunctionRef<TSharedPtr<IAvaMaskMaterialHandle>(
			const FSoftComponentReference& InComponent
			, const int32 InIdx
			, const bool bInIsSlotOccupied
			, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)> InFunction) = 0;

	/** Return if found. */
	virtual bool GetComponentAndSlotForMaterial(
		const UMaterialInterface* InMaterial,
		FSoftComponentReference& OutComponent,
		int32& OutSlotIdx) = 0;

	virtual bool SaveOriginalState(const FStructView& InHandleData) = 0;
	virtual bool ApplyOriginalState(const FStructView& InHandleData) = 0;
	virtual bool ApplyModifiedState(
		const FAvaMask2DSubjectParameters& InModifiedParameters
		, const FStructView& InHandleData) = 0;

	/** Provides the opportunity to modify or replace a material on assignment. */
	virtual FOnSourceMaterialPreAssignment& OnSourceMaterialPreAssignment() = 0;

	virtual FOnSourceMaterialsChanged& OnSourceMaterialsChanged() = 0;

	virtual bool ValidateMaterials(FText& OutFailReason) = 0;
};

template <typename HandleDataType>
class TAvaMaskMaterialCollectionHandle
	: public IAvaMaskMaterialCollectionHandle
{
	static_assert(TModels_V<CStaticStructProvider, HandleDataType>, "HandleDataType should be a UStruct");

public:
	UE_AVA_INHERITS_WITH_SUPER(TAvaMaskMaterialCollectionHandle, IAvaMaskMaterialCollectionHandle);
	
	using FHandleData = HandleDataType;

public:
	virtual FInstancedStruct MakeDataStruct() override;

	virtual void SetMaterials(const TArray<TObjectPtr<UMaterialInterface>>& InMaterials) override;
	virtual void SetMaterials(const TArray<TObjectPtr<UMaterialInterface>>& InMaterials, const TBitArray<>& InSetToggle) override = 0;
	
	/** Return if found. */
	virtual bool GetComponentAndSlotForMaterial(
		const UMaterialInterface* InMaterial,
		FSoftComponentReference& OutComponent,
		int32& OutSlotIdx) override;

	virtual bool SaveOriginalState(const FStructView& InHandleData) override;
	virtual bool ApplyOriginalState(const FStructView& InHandleData) override;
	virtual bool ApplyModifiedState(
		const FAvaMask2DSubjectParameters& InModifiedParameters
		, const FStructView& InHandleData) override;

	/** Provides the opportunity to modify or replace a material on assignment. */
	virtual IAvaMaskMaterialCollectionHandle::FOnSourceMaterialPreAssignment& OnSourceMaterialPreAssignment() override;

	virtual FOnSourceMaterialsChanged& OnSourceMaterialsChanged() override;

	virtual bool ValidateMaterials(FText& OutFailReason) override;

protected:
	virtual FStructView GetMaterialHandleData(
		FHandleData* InParentHandleData
		, const FSoftComponentReference& InComponent
		, const int32 InSlotIdx) = 0;
	
	virtual FStructView GetOrAddMaterialHandleData(
		FHandleData* InParentHandleData
		, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle
		, const FSoftComponentReference& InComponent
		, const int32 InSlotIdx) = 0;

protected:
	IAvaMaskMaterialCollectionHandle::FOnSourceMaterialPreAssignment OnSourceMaterialPreAssignmentDelegate;
	FOnSourceMaterialsChanged OnSourceMaterialsChangedDelegate;
	FAvaMask2DSubjectParameters LastAppliedParameters;
};

template <typename HandleDataType>
FInstancedStruct TAvaMaskMaterialCollectionHandle<HandleDataType>::MakeDataStruct()
{
	FInstancedStruct Data = FInstancedStruct::Make<FHandleData>();
	return Data;
}

template <typename HandleDataType>
void TAvaMaskMaterialCollectionHandle<HandleDataType>::SetMaterials(
	const TArray<TObjectPtr<UMaterialInterface>>& InMaterials)
{
	const TBitArray<> SetToggles(true, InMaterials.Num());
	SetMaterials(InMaterials, SetToggles);
}

template <typename HandleDataType>
bool TAvaMaskMaterialCollectionHandle<HandleDataType>::GetComponentAndSlotForMaterial(
	const UMaterialInterface* InMaterial
	, FSoftComponentReference& OutComponent
	, int32& OutSlotIdx)
{
	if (!InMaterial)
	{
		return false;
	}

	ForEachMaterial([this, InMaterial, &OutComponent, &OutSlotIdx](
		const FSoftComponentReference& InComponent
		, const int32 InSlotIdx
		, const UMaterialInterface* InMtl)
	{
		if (InMaterial == InMtl)
		{
			OutComponent = InComponent;
			OutSlotIdx = InSlotIdx;
			return false;
		}
		return true;
	});

	return false;
}

template <typename HandleDataType>
bool TAvaMaskMaterialCollectionHandle<HandleDataType>::SaveOriginalState(const FStructView& InHandleData)
{
	if (FHandleData* HandleData = InHandleData.GetPtr<FHandleData>())
	{
		ForEachMaterialHandle([this, HandleData](
			const FSoftComponentReference& InComponent
			, const int32 InSlotIdx
			, const bool bInIsSlotOccupied
			, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)
		{
			if (!bInIsSlotOccupied)
			{
				return true;
			}
			
			if (const FStructView MaterialHandleDataView = GetOrAddMaterialHandleData(HandleData, InMaterialHandle, InComponent, InSlotIdx);
				MaterialHandleDataView.IsValid())
			{
				return InMaterialHandle->SaveOriginalState(MaterialHandleDataView);
			}
			
			return false;
		});

		return true;
	}

	return false;
}

template <typename HandleDataType>
bool TAvaMaskMaterialCollectionHandle<HandleDataType>::ApplyOriginalState(const FStructView& InHandleData)
{
	if (FHandleData* HandleData = InHandleData.GetPtr<FHandleData>())
	{
		ForEachMaterialHandle([this, HandleData](
			const FSoftComponentReference& InComponent
			, const int32 InSlotIdx
			, const bool bInIsSlotOccupied
			, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)
		{
			if (!bInIsSlotOccupied)
			{
				return true;
			}

			const FStructView MaterialHandleDataView = GetMaterialHandleData(HandleData, InComponent, InSlotIdx);
			
			if (MaterialHandleDataView.IsValid())
			{
				return InMaterialHandle->ApplyOriginalState(
					MaterialHandleDataView
					, [InComponent, InSlotIdx, this](UMaterialInterface* InMaterial)
					{
						// If default mask material is used, restore back to "none" since this material is used for empty slots
						if (UE::Ava::Internal::IsDefaultMaterial(InMaterial))
						{
							InMaterial = nullptr;
						}

						SetMaterial(InComponent, InSlotIdx, InMaterial);
					});
			}

			UE_LOGF(LogAvaMask, Verbose, "MaterialData not found for slot %u", InSlotIdx);

			return false;
		});

		return true;
	}

	return false;
}

template <typename HandleDataType>
bool TAvaMaskMaterialCollectionHandle<HandleDataType>::ApplyModifiedState(
	const FAvaMask2DSubjectParameters& InModifiedParameters
	, const FStructView& InHandleData)
{
	if (FHandleData* HandleData = InHandleData.GetPtr<FHandleData>())
	{
		LastAppliedParameters = InModifiedParameters;

		ForEachMaterialHandle([this, InModifiedParameters, HandleData](
			const FSoftComponentReference& InComponent
			, const int32 InSlotIdx
			, const bool bInIsSlotOccupied
			, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)
		{
			if (!bInIsSlotOccupied)
			{
				return true;
			}

			const FStructView MaterialHandleDataView = GetMaterialHandleData(HandleData, InComponent, InSlotIdx);

			if (MaterialHandleDataView.IsValid())
			{
				return InMaterialHandle->ApplyModifiedState(
					InModifiedParameters
					, MaterialHandleDataView
					, [InComponent, InSlotIdx, this](UMaterialInterface* InMaterial)
					{
						SetMaterial(InComponent, InSlotIdx, InMaterial);
					});
			}

			UE_LOGF(LogAvaMask, Verbose, "MaterialData not found for slot %u", InSlotIdx);

			return false;
		});

		return true;
	}

	return false;
}

template <typename HandleDataType>
IAvaMaskMaterialCollectionHandle::FOnSourceMaterialPreAssignment& TAvaMaskMaterialCollectionHandle<HandleDataType>::OnSourceMaterialPreAssignment()
{
	return OnSourceMaterialPreAssignmentDelegate;
}

template <typename HandleDataType>
IAvaMaskMaterialCollectionHandle::FOnSourceMaterialsChanged& TAvaMaskMaterialCollectionHandle<HandleDataType>::OnSourceMaterialsChanged()
{
	return OnSourceMaterialsChangedDelegate;
}

template <typename HandleDataType>
bool TAvaMaskMaterialCollectionHandle<HandleDataType>::ValidateMaterials(FText& OutFailReason)
{
	bool bAllMaterialsValid = true;

	TMap<FString, TArray<FString>> FailedMaterialNames;
	FailedMaterialNames.Reserve(GetNumMaterials());

	ForEachMaterialHandle([this, &bAllMaterialsValid, &FailedMaterialNames](
		const FSoftComponentReference& InComponent
		, const int32 InSlotIdx
		, const bool bInIsSlotOccupied
		, const TSharedPtr<IAvaMaskMaterialHandle>& InMaterialHandle)
	{
		if (!bInIsSlotOccupied)
		{
			return true;
		}

		TArray<FString> MissingParameterNames;
		if (!InMaterialHandle->HasRequiredParameters(MissingParameterNames))
		{
			FailedMaterialNames.Emplace(InMaterialHandle->GetMaterialName(), MoveTemp(MissingParameterNames));
			bAllMaterialsValid = false;
			return false;
		}

		return true;
	});

	if (!bAllMaterialsValid)
	{
		FString FailedMaterialNameStr;
		for (const TPair<FString, TArray<FString>>& FailedMaterial : FailedMaterialNames)
		{
			FailedMaterialNameStr += FailedMaterial.Key;
			FailedMaterialNameStr += TEXT("\n\t");
			FailedMaterialNameStr += FString::Join(FailedMaterial.Value, TEXT("\n\t"));
		}
		
		OutFailReason = FText::Format(NSLOCTEXT("AvaMaskMaterialCollectionHandle", "MaterialsInvalid", "The listed materials are missing the required parameters. Ensure they have the appropriate MaterialFunction.\n{0}"), FText::FromString(FailedMaterialNameStr));
	}

	return bAllMaterialsValid;
}
