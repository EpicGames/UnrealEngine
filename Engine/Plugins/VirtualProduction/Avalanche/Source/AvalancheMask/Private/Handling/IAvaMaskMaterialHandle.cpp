// Copyright Epic Games, Inc. All Rights Reserved.

#include "IAvaMaskMaterialHandle.h"

#include "Engine/Engine.h"
#include "Materials/AvaMaskMaterialInstanceSubsystem.h"

UAvaMaskMaterialInstanceSubsystem* UE::AvaMask::Internal::GetMaterialInstanceSubsystem()
{
	return GEngine->GetEngineSubsystem<UAvaMaskMaterialInstanceSubsystem>();
}

bool IAvaMaskMaterialHandle::ApplyModifiedState(const FAvaMask2DSubjectParameters& InModifiedParameters
	, const FStructView& InHandleData
	, TUniqueFunction<void(UMaterialInterface*)>&& InMaterialSetter)
{
	return false;
}

bool IAvaMaskMaterialHandle::ApplyOriginalState(const FStructView& InHandleData
	, TUniqueFunction<void(UMaterialInterface*)>&& InMaterialSetter)
{
	return false;
}
