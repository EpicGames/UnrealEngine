// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMaskLog.h"
#include "AvaMaskTypes.h"
#include "AvaMaskUtilities.h"
#include "Engine/Texture.h"
#include "IAvaMaterialHandle.h"
#include "StructUtils/InstancedStruct.h"
#include "Materials/AvaMaterialUtils.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "StructUtils/StructView.h"

class UAvaMaskMaterialInstanceSubsystem;

namespace UE::AvaMask::Internal
{
	UAvaMaskMaterialInstanceSubsystem* GetMaterialInstanceSubsystem();
}

/** Interface between the mask system and an individual material. */
class IAvaMaskMaterialHandle
	: public IAvaMaterialHandle
{
public:
	UE_AVA_INHERITS(IAvaMaskMaterialHandle, IAvaMaterialHandle);

	virtual ~IAvaMaskMaterialHandle() override = default;

	virtual FInstancedStruct MakeDataStruct() = 0;
	virtual const UScriptStruct* GetDataStructType() = 0;

	virtual bool GetMaskParameters(FAvaMask2DMaterialParameters& OutParameters) = 0;
	virtual bool SetMaskParameters(UTexture* InTexture, float InBaseOpacity, bool bInInverted = false, const FVector2f& InPadding = FVector2f::Zero(), bool bInApplyFeathering = false, float InOuterFeathering = 0.0f, float InInnerFeathering = 0.0f) = 0;

	virtual const EBlendMode GetBlendMode() = 0;
	virtual void SetBlendMode(const EBlendMode InBlendMode) = 0;

	/** Provides the opportunity to propagate changes to a derived material. */
	virtual void SetChildMaterial(UMaterialInstanceDynamic* InChildMaterial) = 0;

	virtual bool SaveOriginalState(const FStructView& InHandleData) = 0;
	
	virtual bool ApplyOriginalState(
		const FStructView& InHandleData
		, TUniqueFunction<void(UMaterialInterface*)>&& InMaterialSetter) = 0;
	
	virtual bool ApplyModifiedState(
		const FAvaMask2DSubjectParameters& InModifiedParameters
		, const FStructView& InHandleData
		, TUniqueFunction<void(UMaterialInterface*)>&& InMaterialSetter) = 0;

	virtual bool HasRequiredParameters(TArray<FString>& OutMissingParameterNames) = 0;	
};

template <typename HandleDataType>
class TAvaMaskMaterialHandle
	: public IAvaMaskMaterialHandle
{
	static_assert(TModels_V<CStaticStructProvider, HandleDataType>, "HandleDataType should be a UStruct");

public:
	UE_AVA_INHERITS_WITH_SUPER(TAvaMaskMaterialHandle, IAvaMaskMaterialHandle);
	
	using FHandleData = HandleDataType;

public:
	virtual FInstancedStruct MakeDataStruct() override;
	virtual const UScriptStruct* GetDataStructType() override;

	virtual bool GetMaskParameters(FAvaMask2DMaterialParameters& OutParameters) override;
	virtual bool SetMaskParameters(UTexture* InTexture, float InBaseOpacity, bool bInInverted = false, const FVector2f& InPadding = FVector2f::Zero(), bool bInApplyFeathering = false, float InOuterFeathering = 0.0f, float InInnerFeathering = 0.0f) override;

	virtual bool HasRequiredParameters(TArray<FString>& OutMissingParameterNames) override;

protected:
	virtual UMaterialInstanceDynamic* GetMaterialInstance() = 0;
};

template <typename HandleDataType>
FInstancedStruct TAvaMaskMaterialHandle<HandleDataType>::MakeDataStruct()
{
	FInstancedStruct MaterialData = FInstancedStruct::Make<FHandleData>();
	return MaterialData;
}

template <typename HandleDataType>
const UScriptStruct* TAvaMaskMaterialHandle<HandleDataType>::GetDataStructType()
{
	return FHandleData::StaticStruct();
}

template <typename HandleDataType>
bool TAvaMaskMaterialHandle<HandleDataType>::GetMaskParameters(FAvaMask2DMaterialParameters& OutParameters)
{
	if (const UMaterialInstanceDynamic* MaterialInstance = GetMaterialInstance())
	{
		OutParameters.StoreFromMaterial(MaterialInstance);
		return true;
	}

	return false;
}

template <typename HandleDataType>
bool TAvaMaskMaterialHandle<HandleDataType>::SetMaskParameters(UTexture* InTexture
	, float InBaseOpacity
	, bool bInInverted
	, const FVector2f& InPadding
	, bool bInApplyFeathering
	, float InOuterFeathering
	, float InInnerFeathering)
{
	if (UMaterialInstanceDynamic* MaterialInstance = GetMaterialInstance())
	{
		UE_LOGF(LogAvaMask, Verbose, "SetParameters: Material:%ls", *MaterialInstance->GetName());

		// note this does no longer deal with textures as the mask texture parameter is deprecated.
		// and this will not deal with texture parameter collections as this mask handle system is deprecated too.

		MaterialInstance->SetScalarParameterValueByInfo(UE::AvaMask::Internal::InvertParameterInfo, bInInverted ? 1.0f : 0.0f);
		MaterialInstance->SetScalarParameterValueByInfo(UE::AvaMask::Internal::BaseOpacityParameterInfo, FMath::Clamp(InBaseOpacity, 0.0f, 1.0f));
		MaterialInstance->SetVectorParameterValueByInfo(UE::AvaMask::Internal::PaddingParameterInfo, FLinearColor(FMath::Max(0, InPadding.X), FMath::Max(0, InPadding.Y), 0, 0));
		MaterialInstance->SetVectorParameterValueByInfo(UE::AvaMask::Internal::FeatherParameterInfo, FLinearColor(bInApplyFeathering ? 1.0f : 0.0f, FMath::Max(0, InOuterFeathering), FMath::Max(0, InInnerFeathering), FMath::Max(0, FMath::Max(InOuterFeathering, InInnerFeathering))));

		return true;
	}

	return false;
}

template <typename HandleDataType>
bool TAvaMaskMaterialHandle<HandleDataType>::HasRequiredParameters(TArray<FString>& OutMissingParameterNames)
{
	if (const UMaterialInterface* Material = GetMaterial())
	{
		return UE::Ava::MaterialHasParameters(
			*Material
			, { 
				{ UE::AvaMask::Internal::BaseOpacityParameterInfo.Name, EMaterialParameterType::Scalar }, 
				{ UE::AvaMask::Internal::InvertParameterInfo.Name, EMaterialParameterType::Scalar }
			}
			, OutMissingParameterNames);
	}

	UE_LOGF(LogAvaMask, Warning, "HasRequiredParameters: Material was invalid.");

	return false;
}
