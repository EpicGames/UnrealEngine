// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "IAvaObjectHandle.h"

class UMaterialInstance;
class UMaterialInterface;

class IAvaMaterialHandle
	: public IAvaObjectHandle
{
public:
	UE_AVA_INHERITS(IAvaMaterialHandle, IAvaObjectHandle);
	
	virtual ~IAvaMaterialHandle() override = default;

	virtual FString GetMaterialName() = 0;
	virtual UMaterialInterface* GetMaterial() = 0;
	virtual UMaterialInterface* GetParentMaterial() = 0;
	virtual void SetParentMaterial(UMaterialInterface* InMaterial) = 0;

	virtual void CopyParametersFrom(UMaterialInstance* InSourceMaterial) = 0;
};
