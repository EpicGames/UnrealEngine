// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"

/*
 * Object Handles provide a light weight interface between a system and object.
 * - They are stateful, and can become invalid if the wrapped object goes out of scope. 
 * - They are not persistent, and serialized data should be set from an external source (ie. from a UStruct)
 * - They should be extended or specialized according to your use, ie. for a particular set of operations
 * - There is no access control - multiple handles can be created for a given object
 */

/** Interface between a calling system to perform operations on a specific object. */
class IAvaObjectHandle
	: public IAvaTypeCastable
{
public:
	UE_AVA_INHERITS(IAvaObjectHandle, IAvaTypeCastable)
	
	using FIsSupportedFunction = TFunction<bool(const UStruct*, const void*)>;

	virtual ~IAvaObjectHandle() = default;

	virtual bool IsValid() const = 0;
};
