// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mask2D/AvaMask2DBaseModifier.h"

#include "AvaMaskLog.h"
#include "AvaMaskSubsystem.h"
#include "AvaMaskUtilities.h"
#include "Components/BillboardComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/CanvasRenderTarget2D.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Framework/AvaGizmoComponent.h"
#include "GeometryMaskCanvas.h"
#include "GeometryMaskReadComponent.h"
#include "GeometryMaskSubsystem.h"
#include "GeometryMaskWorldSubsystem.h"
#include "GeometryMaskWriteComponent.h"
#include "Handling/AvaHandleUtilities.h"
#include "Handling/AvaObjectHandleSubsystem.h"
#include "Handling/IAvaMaskMaterialCollectionHandle.h"
#include "Handling/IAvaMaskMaterialHandle.h"
#include "Materials/AvaMaskMaterialInstanceSubsystem.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/Utilities/ActorModifierCoreUtilities.h"
#include "Particles/ParticleSystemComponent.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "Text3DComponent.h"

#define LOCTEXT_NAMESPACE "AvaMask2DModifier"

namespace UE::AvaMask::Private
{
	// @note: It's crucial these remain in sync with the defaults in UGeometryMaskCanvas
	namespace CanvasPropertyDefaults
	{
		static bool bApplyBlur = false;
		static double BlurStrength = 16;
		static bool bApplyFeather = false;
		static int32 OuterFeatherRadius = 16;
		static int32 InnerFeatherRadius = 16;
	}
}

#if WITH_EDITOR
const TAvaPropertyChangeDispatcher<UAvaMask2DBaseModifier> UAvaMask2DBaseModifier::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UAvaMask2DBaseModifier, bUseParentChannel), &UAvaMask2DBaseModifier::OnUseParentChannelChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaMask2DBaseModifier, Channel), &UAvaMask2DBaseModifier::OnChannelChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaMask2DBaseModifier, bInverted), &UAvaMask2DBaseModifier::OnInvertedChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaMask2DBaseModifier, bUseBlur), &UAvaMask2DBaseModifier::OnBlurChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaMask2DBaseModifier, BlurStrength), &UAvaMask2DBaseModifier::OnBlurChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaMask2DBaseModifier, bUseFeathering), &UAvaMask2DBaseModifier::OnFeatherChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaMask2DBaseModifier, OuterFeatherRadius), &UAvaMask2DBaseModifier::OnFeatherChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaMask2DBaseModifier, InnerFeatherRadius), &UAvaMask2DBaseModifier::OnFeatherChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaMask2DBaseModifier, CanvasWeak), &UAvaMask2DBaseModifier::OnCanvasChanged }
};
#endif

void UAvaMask2DBaseModifier::SetUseParentChannel(const bool bInUseParentChannel)
{
	if (bUseParentChannel != bInUseParentChannel)
	{
		bUseParentChannel = bInUseParentChannel;
		OnUseParentChannelChanged();
	}
}

const FName UAvaMask2DBaseModifier::GetChannel() const
{
	return bUseParentChannel ? ParentChannel : Channel;
}

void UAvaMask2DBaseModifier::SetChannel(FName InChannel)
{
	if (bUseParentChannel)
	{
		if (ParentChannel != InChannel)
		{
			ParentChannel = InChannel;
			OnChannelChanged();
		}
	}
	else
	{
		if (Channel != InChannel)
		{
			Channel = InChannel;
			OnChannelChanged();
		}
	}
}

void UAvaMask2DBaseModifier::SetIsInverted(const bool bInInvert)
{
	if (bInverted != bInInvert)
	{
		bInverted = bInInvert;
		OnInvertedChanged();
	}
}

void UAvaMask2DBaseModifier::UseBlur(bool bInUseBlur)
{
	if (bUseBlur != bInUseBlur)
	{
		bUseBlur = bInUseBlur;
		OnBlurChanged();
	}
}

void UAvaMask2DBaseModifier::SetBlurStrength(float InBlurStrength)
{
	InBlurStrength = FMath::Max(0.0f, InBlurStrength);
	if (!FMath::IsNearlyEqual(BlurStrength, InBlurStrength))
	{
		BlurStrength = InBlurStrength;
		OnBlurChanged();
	}
}

void UAvaMask2DBaseModifier::UseFeathering(bool bInUseFeathering)
{
	if (bUseFeathering != bInUseFeathering)
	{
		bUseFeathering = bInUseFeathering;
		OnFeatherChanged();
	}
}

void UAvaMask2DBaseModifier::SetOuterFeatherRadius(int32 InFeatherRadius)
{
	InFeatherRadius = FMath::Max(0, InFeatherRadius);
	if (OuterFeatherRadius != InFeatherRadius)
	{
		OuterFeatherRadius = InFeatherRadius;
		OnFeatherChanged();
	}
}

void UAvaMask2DBaseModifier::SetInnerFeatherRadius(int32 InFeatherRadius)
{
	InFeatherRadius = FMath::Max(0, InFeatherRadius);
	if (InnerFeatherRadius != InFeatherRadius)
	{
		InnerFeatherRadius = InFeatherRadius;
		OnFeatherChanged();
	}
}

void UAvaMask2DBaseModifier::OnBlurChanged()
{
	MarkModifierDirty();
}

void UAvaMask2DBaseModifier::OnFeatherChanged()
{
	MarkModifierDirty();
}

void UAvaMask2DBaseModifier::OnCanvasChanged()
{
	CanvasParamsToLocal();
	
	OnBlurChanged();
	OnFeatherChanged();
}

void UAvaMask2DBaseModifier::UpdateCanvas()
{
	// Reset/invalidate cached
	LastResolvedCanvasName = NAME_None;
	ResetCanvas();

#if WITH_EDITOR
	if (!Channel.IsNone())
	{
		if (UAvaMaskSubsystem* Subsystem = GEngine->GetEngineSubsystem<UAvaMaskSubsystem>())
		{
			Subsystem->SetLastSpecifiedChannelName(Channel);
		}
	}
#endif

	TryResolveCanvas();
}

void UAvaMask2DBaseModifier::CanvasParamsToLocal(bool bInResolveCanvas)
{
	if (UGeometryMaskCanvas* Canvas = GetCurrentCanvas(bInResolveCanvas))
	{
		bUseBlur = Canvas->IsBlurApplied();
		BlurStrength = Canvas->GetBlurStrength();

		bUseFeathering = Canvas->IsFeatherApplied();
		OuterFeatherRadius = Canvas->GetOuterFeatherRadius();
		InnerFeatherRadius = Canvas->GetInnerFeatherRadius();
	}
}

void UAvaMask2DBaseModifier::LocalParamsToCanvas()
{
	if (UGeometryMaskCanvas* Canvas = GetCurrentCanvas())
	{
		// Only set if not defaults
		
		const bool bModifierUseBlur = bUseBlur;
		if (bModifierUseBlur != UE::AvaMask::Private::CanvasPropertyDefaults::bApplyBlur)
		{
			Canvas->SetApplyBlur(bModifierUseBlur);
		}

		const double ModifierBlurStrength = BlurStrength;		
		if (!FMath::IsNearlyEqual(ModifierBlurStrength, UE::AvaMask::Private::CanvasPropertyDefaults::BlurStrength))
		{
			const double CanvasBlurStrength = Canvas->GetBlurStrength();
			Canvas->SetBlurStrength(FMath::Max(ModifierBlurStrength, CanvasBlurStrength));
		}

		const bool bModifierUseFeathering = bUseFeathering;
		if (bModifierUseFeathering != UE::AvaMask::Private::CanvasPropertyDefaults::bApplyFeather)
		{
			Canvas->SetApplyFeather(bModifierUseFeathering);
		}

		const int32 ModifierOuterFeatherRadius = OuterFeatherRadius;
		if (ModifierOuterFeatherRadius != UE::AvaMask::Private::CanvasPropertyDefaults::OuterFeatherRadius)
		{
			const int32 CanvasOuterFeatherRadius = Canvas->GetOuterFeatherRadius();
			Canvas->SetOuterFeatherRadius(FMath::Max(ModifierOuterFeatherRadius, CanvasOuterFeatherRadius));
		}

		const int32 ModifierInnerFeatherRadius = InnerFeatherRadius;
		if (ModifierInnerFeatherRadius != UE::AvaMask::Private::CanvasPropertyDefaults::InnerFeatherRadius)
		{
			const int32 CanvasInnerFeatherRadius = Canvas->GetInnerFeatherRadius();
			Canvas->SetInnerFeatherRadius(FMath::Max(ModifierInnerFeatherRadius, CanvasInnerFeatherRadius));
		}
	}
}

#if WITH_EDITOR
void UAvaMask2DBaseModifier::VisualizeMask()
{
	if (GEngine)
	{
		GEngine->Exec(nullptr, TEXT("GeometryMask.Visualize"));
	}
}

void UAvaMask2DBaseModifier::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UAvaMask2DBaseModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

	UpdateCanvas();

	if (InReason == EActorModifierCoreEnableReason::User
		|| InReason == EActorModifierCoreEnableReason::Duplicate)
	{
		SetupChannelName();
	}

	if (InReason == EActorModifierCoreEnableReason::Load)
	{
		LocalParamsToCanvas();
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		RestoreActors();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		CanvasParamsToLocal();
	}
}

void UAvaMask2DBaseModifier::OnModifierRemoved(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierRemoved(InReason);

	if (InReason == EActorModifierCoreDisableReason::Destroyed)
	{
		constexpr bool bResolveCanvas = false;
		CanvasParamsToLocal(bResolveCanvas);
	}

	TArray<AActor*> ActorDataActors;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ActorDataActors.Reserve(ActorData_DEPRECATED.Num());	
	Algo::Transform(ActorData_DEPRECATED, ActorDataActors, [](const TPair<TWeakObjectPtr<AActor>, FAvaMask2DActorData>& InActorDataPair)
	{
		return InActorDataPair.Key.Get();
	});
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	for (AActor* Actor : ActorDataActors)
	{
		if (!Actor)
		{
			continue;
		}
		
		RemoveFromActor(Actor);
	}
}

void UAvaMask2DBaseModifier::OnModifiedActorTransformed()
{
	// This is needed to override parent behavior (to disable it)
}

void UAvaMask2DBaseModifier::RestorePreState()
{
	Super::RestorePreState();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RestoreActors();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UAvaMask2DBaseModifier::Apply()
{
	LocalParamsToCanvas();
}

void UAvaMask2DBaseModifier::SaveActorPreState(AActor* InActor, FAvaMask2DActorData& InActorData)
{
}

void UAvaMask2DBaseModifier::RestoreActorPreState(AActor* InActor, FAvaMask2DActorData& InActorData)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Get MaterialCollectionHandle
    const TSharedPtr<IAvaMaskMaterialCollectionHandle> MaterialCollectionHandle =
    	UE::Ava::Internal::FindOrAddHandleByLambda(
    		MaterialCollectionHandles
    		, InActor
    		, [this, InActor]()
    		{
    			TSharedPtr<IAvaMaskMaterialCollectionHandle> Handle = GetObjectHandleSubsystem()->MakeCollectionHandle<IAvaMaskMaterialCollectionHandle>(InActor, UE::AvaMask::Internal::HandleTag);

    			Handle->OnSourceMaterialsChanged().Unbind();

    			return Handle;
    		}
    	);

    FInstancedStruct* MaterialCollectionData = MaterialCollectionHandleData_DEPRECATED.Find(InActor);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (MaterialCollectionData)
	{
		MaterialCollectionHandle->ApplyOriginalState(*MaterialCollectionData);
	}
}

void UAvaMask2DBaseModifier::OnSceneTreeTrackedActorParentChanged(int32 InIdx, const TArray<TWeakObjectPtr<AActor>>& InPreviousParentActor, const TArray<TWeakObjectPtr<AActor>>& InNewParentActor)
{
	Super::OnSceneTreeTrackedActorParentChanged(InIdx, InPreviousParentActor, InNewParentActor);

	// We don't use the provided parent here, instead we just use the event to trigger custom parent discovery
	if (bUseParentChannel)
	{
		if (TryResolveParentChannel())
		{
			OnChannelChanged();
		}
	}
}

void UAvaMask2DBaseModifier::OnSceneTreeTrackedActorChildrenChanged(int32 InIdx, const TSet<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TSet<TWeakObjectPtr<AActor>>& InNewChildrenActors)
{
	Super::OnSceneTreeTrackedActorChildrenChanged(InIdx, InPreviousChildrenActors, InNewChildrenActors);

	TSet<TWeakObjectPtr<AActor>> UnparentedActors = InPreviousChildrenActors.Difference(InNewChildrenActors);
	for (TWeakObjectPtr<AActor>& ActorWeak : UnparentedActors)
	{
		if (AActor* Actor = ActorWeak.Get())
		{
			RemoveFromActor(Actor);
		}
	}
}

void UAvaMask2DBaseModifier::OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent)
{
	Super::OnRenderStateUpdated(InActor, InComponent);
}

FName UAvaMask2DBaseModifier::GenerateUniqueMaskName() const
{
	const AActor* ModifierActor = GetModifiedActor();
	if (!ModifierActor)
	{
		return NAME_None;
	}

	return FName(ModifierActor->GetActorNameOrLabel() + TEXT("_Mask"));
}

UAvaMask2DBaseModifier* UAvaMask2DBaseModifier::FindMaskModifierOnActor(const AActor* InActor)
{
	// Get from modifier, if present
	if (const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get())
	{
		const UActorModifierCoreStack* ModifierStack = ModifierSubsystem->GetActorModifierStack(InActor);
		if (!ModifierStack)
		{
			return nullptr;
		}

		UAvaMask2DBaseModifier* FoundMaskModifier = nullptr;
		
		TArray<UAvaMask2DBaseModifier*> FoundModifiers;
		ModifierStack->GetClassModifiers<UAvaMask2DBaseModifier>(FoundModifiers);
		if (!FoundModifiers.IsEmpty())
		{
			FoundMaskModifier = FoundModifiers.Last();
		}

		return FoundMaskModifier;
	}

	return nullptr;
}

bool UAvaMask2DBaseModifier::ForEachUsedCanvasName(TFunctionRef<bool(FName)> InFunc) const
{
	return InFunc(GetChannel());
}

void UAvaMask2DBaseModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("Mask"));
	InMetadata.SetCategory(TEXT("Rendering"));
#if WITH_EDITOR
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Allows to use a custom mask texture on attached actors materials"));
#endif
}

UActorComponent* UAvaMask2DBaseModifier::FindOrAddMaskComponent(TSubclassOf<UActorComponent> InComponentClass, AActor* InActor)
{
	if (!InActor || !InComponentClass)
	{
		return nullptr;
	}

	// Skip adding to actors that already write to a mask
	if (InComponentClass->ImplementsInterface(UGeometryMaskReadInterface::StaticClass()))
	{
		if (InActor->FindComponentByInterface(UGeometryMaskWriteInterface::StaticClass()))
		{
			UE_LOGF(LogAvaMask, Verbose, "Attempting to add a Mask Read component to an Actor that has a Mask Write component: '%ls'", *InActor->GetName());
			return nullptr;
		}
	}

	UActorComponent* MaskComponent = UE::AvaMask::Internal::FindOrAddComponent(InComponentClass, InActor);
	SetupMaskComponent(MaskComponent);

	return MaskComponent;
}

bool UAvaMask2DBaseModifier::ActorSupportsMaskReadWrite(const AActor* InActor)
{
	// Only supports primitives and Text3D component.
	// Note: Actors holding Text3D Components do eventually get Text Mesh Subobjects; but this should still return true even if these components are not yet built (the build is queued)
	if (InActor->FindComponentByClass<UText3DComponent>())
	{
		return true;
	}

	TArray<UPrimitiveComponent*> PrimitiveComponents;
	InActor->GetComponents(PrimitiveComponents);

	// Skip components that are editor only (visualizations) and FX/Billboard
	for (const UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
	{
		if (PrimitiveComponent
			&& !PrimitiveComponent->IsEditorOnly()
			&& !PrimitiveComponent->IsA<UFXSystemComponent>()
			&& !PrimitiveComponent->IsA<UBillboardComponent>())
		{
			return true;
		}
	}

	return false;
}

bool UAvaMask2DBaseModifier::TryResolveParentChannel()
{
	bool bParentChannelWasFound = false;
	if (const AActor* ActorModified = GetModifiedActor())
	{
		const AActor* Parent = ActorModified->GetAttachParentActor();

		const IGeometryMaskReadInterface* ReadComponent = nullptr;
		while (ReadComponent == nullptr && Parent)
		{
			ReadComponent = Parent->FindComponentByInterface<IGeometryMaskReadInterface>();
			Parent = Parent->GetAttachParentActor();
		}

		if (ReadComponent)
		{
			ParentChannel = ReadComponent->GetParameters().CanvasName;
			bParentChannelWasFound = true;
		}
		else
		{
			// If parent invalid or not found, just use the previously specified one (not from parent)
			ParentChannel = Channel;
		}
	}

	return bParentChannelWasFound;
}

UGeometryMaskWorldSubsystem* UAvaMask2DBaseModifier::GetGeometryMaskWorldSubsystem(const ULevel** OutLevel) const
{
	const AActor* Actor = GetModifiedActor();
	if (!Actor)
	{
		return nullptr;
	}

	const ULevel* Level = Actor->GetLevel();
	if (!Level || !Level->OwningWorld)
	{
		return nullptr;
	}

	if (OutLevel)
	{
		*OutLevel = Level;
	}
	return Level->OwningWorld->GetSubsystem<UGeometryMaskWorldSubsystem>();
}

void UAvaMask2DBaseModifier::SetupMaskComponent(UActorComponent* InComponent)
{
}

void UAvaMask2DBaseModifier::RemoveFromActor(AActor* InActor)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ActorData_DEPRECATED.Remove(InActor);
	MaterialCollectionHandleData_DEPRECATED.Remove(InActor);
	MaterialCollectionHandles.Remove(InActor);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UAvaMask2DBaseModifier::OnChannelChanged()
{
	UpdateCanvas();
	MarkModifierDirty();
}

void UAvaMask2DBaseModifier::OnInvertedChanged()
{
	MarkModifierDirty();
}

void UAvaMask2DBaseModifier::SetupChannelName()
{
	// Already named, leave as-is
	if (!Channel.IsNone())
	{
		return;
	}

	AutoChannelName = GenerateUniqueMaskName();	

#if WITH_EDITOR
	if (const UAvaMaskSubsystem* Subsystem = GEngine->GetEngineSubsystem<UAvaMaskSubsystem>())
	{
		Channel = Subsystem->GetLastSpecifiedChannelName();
	}

	if (Channel.IsNone())
	{
		Channel = TEXT("0");
	}
#else
	Channel = TEXT("0");
#endif
}

void UAvaMask2DBaseModifier::OnUseParentChannelChanged()
{
	if (bUseParentChannel)
	{
		if (TryResolveParentChannel())
		{
			OnChannelChanged();
		}
	}

	MarkModifierDirty();
}

void UAvaMask2DBaseModifier::OnMaskSetCanvas(const UGeometryMaskCanvas* InCanvas, AActor* InActor)
{
}

void UAvaMask2DBaseModifier::TryResolveCanvas()
{
	const ULevel* Level = nullptr;
	if (UGeometryMaskWorldSubsystem* MaskSubsystem = GetGeometryMaskWorldSubsystem(&Level))
	{
		if (UGeometryMaskCanvas* Canvas = MaskSubsystem->GetNamedCanvas(Level, GetChannel(), this))
		{
			LastResolvedCanvasName = Canvas->GetCanvasName();
			CanvasWeak = Canvas;
			OnCanvasSet(Canvas);
		}
	}
}

UTexture* UAvaMask2DBaseModifier::TryResolveCanvasTexture(AActor* InActor, FAvaMask2DActorData& InActorData)
{
	return nullptr;
}

UAvaObjectHandleSubsystem* UAvaMask2DBaseModifier::GetObjectHandleSubsystem()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (UAvaObjectHandleSubsystem* Subsystem = ObjectHandleSubsystem.Get())
	{
		return Subsystem;
	}

	ObjectHandleSubsystem = GEngine->GetEngineSubsystem<UAvaObjectHandleSubsystem>();
	return ObjectHandleSubsystem.Get();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

UAvaMaskMaterialInstanceSubsystem* UAvaMask2DBaseModifier::GetMaterialInstanceSubsystem()
{
	if (UAvaMaskMaterialInstanceSubsystem* Subsystem = MaterialInstanceSubsystem.Get())
	{
		return Subsystem;
	}

	MaterialInstanceSubsystem = GEngine->GetEngineSubsystem<UAvaMaskMaterialInstanceSubsystem>();
	return MaterialInstanceSubsystem.Get();
}

UGeometryMaskCanvas* UAvaMask2DBaseModifier::GetCurrentCanvas(bool bInResolveCanvas)
{
	if (LastResolvedCanvasName.IsNone() || LastResolvedCanvasName != GetChannel())
	{
		LastResolvedCanvasName = NAME_None;
		ResetCanvas();
	}

	if (!CanvasWeak.IsValid() && bInResolveCanvas)
	{
		TryResolveCanvas();
	}
	
	if (UGeometryMaskCanvas* Canvas = CanvasWeak.Get())
	{
		return Canvas;
	}

	return nullptr;
}

void UAvaMask2DBaseModifier::OnMaterialsChanged(UObject* InMaterialOwner, const TArray<TSharedPtr<IAvaMaskMaterialHandle>>& InMaterialHandles)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!InMaterialHandles.IsEmpty() && !ActorData_DEPRECATED.IsEmpty())
	{
		if (AActor* OwningActor = InMaterialOwner->GetTypedOuter<AActor>())
		{
			const UActorModifierCoreBase* MaskModifier = UE::ActorModifierCore::Utilities::FindFirstActorModifierByClass(OwningActor, UAvaMask2DBaseModifier::StaticClass());

			// Only trigger update if first modifier found in the hierarchy above us is this modifier
			if (MaskModifier == this && ActorData_DEPRECATED.Contains(OwningActor))
			{
				FInstancedStruct* HandleData = MaterialCollectionHandleData_DEPRECATED.Find(OwningActor);
				const TSharedPtr<IAvaMaskMaterialCollectionHandle>* CollectionHandle = MaterialCollectionHandles.Find(OwningActor);

				if (HandleData && CollectionHandle && CollectionHandle->IsValid())
				{
					(*CollectionHandle)->SaveOriginalState(*HandleData);
				}

				MarkModifierDirty();
			}
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UAvaMask2DBaseModifier::RestoreActors()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TMap<TWeakObjectPtr<AActor>, FAvaMask2DActorData> ActorDataCopy = ActorData_DEPRECATED;
	for (TPair<TWeakObjectPtr<AActor>, FAvaMask2DActorData>& ActorDataPair : ActorDataCopy)
	{
		if (AActor* Actor = ActorDataPair.Key.Get())
		{
			RestoreActorPreState(Actor, ActorDataPair.Value);
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UAvaMask2DBaseModifier::ResetCanvas()
{
	OnCanvasReset();
	CanvasWeak.Reset();
}

#undef LOCTEXT_NAMESPACE
