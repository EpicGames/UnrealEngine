// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mask2D/AvaMask2DMaskState.h"
#include "AvaMaskSettings.h"
#include "AvaMaskUtilities.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"

#if WITH_EDITOR
#include "Components/DMMaterialProperty.h"
#include "DMDefs.h"
#include "Material/DynamicMaterialInstance.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#endif

void FAvaMask2DMaskState::Apply(UMaterialInstanceDynamic* InMaterial) const
{
	if (!InMaterial)
	{
		return;
	}

	InMaterial->BlendMode = UE::AvaMask::Internal::GetTargetBlendMode(MaterialParameters.BlendMode, InMaterial->BlendMode);

#if WITH_EDITOR
	UMaterialFunctionInterface* OutputProcessorFunction = OutputProcessor;
	if (!OutputProcessorFunction)
	{
		OutputProcessorFunction = GetMutableDefault<UAvaMaskSettings>()->GetMaterialFunction();
	}

	if (UDynamicMaterialInstance* MaterialDesigner = Cast<UDynamicMaterialInstance>(InMaterial))
	{
		UE::AvaMask::Internal::SetOutputProcessor(MaterialDesigner, OutputProcessorFunction, InMaterial->BlendMode);
	}
#endif

	MaterialParameters.ApplyToMID(InMaterial);
}

void FAvaMask2DMaskState::Store(UMaterialInstanceDynamic* InMaterial)
{
	if (!InMaterial)
	{
		return;
	}

	MaterialParameters.StoreFromMaterial(InMaterial);

#if WITH_EDITOR
	if (UDynamicMaterialInstance* MaterialDesigner = Cast<UDynamicMaterialInstance>(InMaterial))
	{
		OutputProcessor = UE::AvaMask::Internal::GetOutputProcessor(MaterialDesigner, InMaterial->BlendMode);
	}
#endif
}

FAvaMask2DMaterialSlotId::FAvaMask2DMaterialSlotId(FString&& InMaterialContainerPath, const FAvaMaterialBridgeSlotId& InSlotId, const UMaterialInterface* InMaterial)
	: MaterialContainerPath(MoveTemp(InMaterialContainerPath)) 
	, SlotId(InSlotId)
	, BaseMaterial(InMaterial ? InMaterial->GetMaterial() : nullptr)
{
}

bool FAvaMask2DMaterialSlotId::operator==(const FAvaMask2DMaterialSlotId& InOther) const
{
	return SlotId == InOther.SlotId
		&& BaseMaterial == InOther.BaseMaterial
		&& MaterialContainerPath == InOther.MaterialContainerPath;
}

FAvaMask2DMaterialMaskState::FAvaMask2DMaterialMaskState(const FAvaMask2DMaterialSlotId& InSlotId)
	: SlotMaterialId(InSlotId)
{
}

bool FAvaMask2DMaterialMaskState::operator==(const FAvaMask2DMaterialSlotId& InSlotId) const
{
	return SlotMaterialId == InSlotId;
}

void FAvaMask2DMaterialMaskState::Apply() const
{
	if (UMaterialInstanceDynamic* Material = MaterialWeak.Get())
	{
		MaskState.Apply(Material);
	}
}

void FAvaMask2DMaterialMaskState::Store()
{
	if (UMaterialInstanceDynamic* Material = MaterialWeak.Get())
	{
		MaskState.Store(Material);
	}
}

bool FAvaMask2DMaterialMaskState::IsDependentOf(UMaterialInterface* InMaterial) const
{
	UMaterialInstanceDynamic* const Material = MaterialWeak.Get();
	return Material && Material->IsDependent(InMaterial);
}
