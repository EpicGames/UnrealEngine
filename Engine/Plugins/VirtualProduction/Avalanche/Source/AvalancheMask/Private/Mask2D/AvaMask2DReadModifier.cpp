// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mask2D/AvaMask2DReadModifier.h"
#include "AvaDataView.h"
#include "AvaMaskLog.h"
#include "AvaMaskSettings.h"
#include "AvaMaskSubsystem.h"
#include "AvaMaskUtilities.h"
#include "Engine/CanvasRenderTarget2D.h"
#include "Engine/Engine.h"
#include "GeometryMaskCanvas.h"
#include "GeometryMaskCanvasResource.h"
#include "GeometryMaskCanvasSharedData.h"
#include "GeometryMaskSubsystem.h"
#include "GeometryMaskWorldSubsystem.h"
#include "Components/PrimitiveComponent.h"
#include "MaterialBridge/AvaMaterialBridge.h"
#include "MaterialBridge/AvaMaterialBridgeRegistry.h"
#include "MaterialBridge/Context/AvaMaterialBridgeApplyStateContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeReadSlotContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeStoreStateContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeWriteSlotContext.h"
#include "MaterialBridge/Slot/AvaMaterialBridgeReadSlot.h"
#include "MaterialBridge/Slot/AvaMaterialBridgeWriteSlot.h"
#include "MaterialBridge/Slot/CommonFeatures/AvaMaterialBridgeBlendModeFeature.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/ActorModifierRenderStateDirtyEvent.h"
#include "Modifiers/Utilities/ActorModifierCoreUtilities.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#define LOCTEXT_NAMESPACE "AvaMask2DReadModifier"

#if WITH_EDITOR
const TAvaPropertyChangeDispatcher<UAvaMask2DReadModifier> UAvaMask2DReadModifier::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UAvaMask2DReadModifier, BaseOpacity), &UAvaMask2DReadModifier::OnBaseOpacityChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaMask2DReadModifier, AdditionalChannels), &UAvaMask2DReadModifier::OnAdditionalChannelsChanged },
};
#endif

void UAvaMask2DReadModifier::SetBaseOpacity(float InBaseOpacity)
{
	InBaseOpacity = FMath::Clamp(InBaseOpacity, 0.0f, 1.0f);
	if (!FMath::IsNearlyEqual(BaseOpacity, InBaseOpacity))
	{
		BaseOpacity = InBaseOpacity;
		OnBaseOpacityChanged();
	}
}

#if WITH_EDITOR
void UAvaMask2DReadModifier::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	UAvaMask2DReadModifier::PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}

void UAvaMask2DReadModifier::OnMaterialCompiled(UMaterialInterface* InMaterial)
{
	if (!InMaterial)
	{
		return;
	}

	const bool bHasMaterialDependency = MaterialStates.ContainsByPredicate(
		[InMaterial](const FAvaMask2DMaterialMaskState& InMaterialState)
		{
			return InMaterialState.IsDependentOf(InMaterial);
		});

	if (bHasMaterialDependency)
	{
		MarkModifierDirty();
	}
}

void UAvaMask2DReadModifier::OnPIEEnded(bool bInIsSimulating)
{
	// Used to refresh mask canvas
	MarkModifierDirty();
}
#endif

void UAvaMask2DReadModifier::Apply()
{
	Super::Apply();

	TArray<TNotNull<const UGeometryMaskCanvas*>, TInlineAllocator<1>> Canvases;
	if (const UGeometryMaskCanvas* Canvas = GetCurrentCanvas())
	{
		Canvases.Add(Canvas);

		const ULevel* Level = nullptr;
		if (UGeometryMaskWorldSubsystem* MaskSubsystem = GetGeometryMaskWorldSubsystem(&Level))
		{
			for (FName AdditionalChannel : AdditionalChannels)
			{
				if (UGeometryMaskCanvas* AdditionalCanvas = MaskSubsystem->GetNamedCanvas(Level, AdditionalChannel, this))
				{
					Canvases.Add(AdditionalCanvas);
				}
			}
		}
	}

	if (!Canvases.IsEmpty())
	{
		if (Canvases.Num() > MaxCanvasCount && UE_LOG_ACTIVE(LogAvaMask, Warning))
		{
			const AActor* Actor = GetModifiedActor();
			UE_LOGF(LogAvaMask, Warning, "%d canvases were found in Masked Layer (Output) for Actor '%ls'. Only the first %d (limit) will be considered."
				, Canvases.Num()
				, Actor ? *Actor->GetActorNameOrLabel() : TEXT("(None)")
				, MaxCanvasCount);
		}

		for (TPair<TObjectPtr<UObject>, TInstancedStruct<FAvaMaterialContainerState>>& Pair : MaterialContainerStates)
		{
			if (Pair.Key)
			{
				ApplyRead(Pair.Key, Canvases);
			}
		}
	}

	Next();
}

void UAvaMask2DReadModifier::OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent)
{
	Super::OnRenderStateUpdated(InActor, InComponent);

	if (!CanMarkModifierDirty())
	{
		return;
	}

	const UActorModifierCoreBase* MaskModifier = UE::ActorModifierCore::Utilities::FindFirstActorModifierByClass(InActor, UAvaMask2DBaseModifier::StaticClass());

	// Only trigger update if first modifier found in the hierarchy above us is this modifier
	if (MaskModifier != this)
	{
		return;
	}

	const UE::Ava::FMaterialBridge* const MaterialBridge = UE::Ava::FMaterialBridgeRegistry::Get().GetMaterialBridge(UE::Ava::FConstDataView(InComponent));
	if (!MaterialBridge)
	{
		return;
	}

	bool bHasMaterials = false;

	MaterialBridge->AccessSlots(UE::Ava::FMaterialBridgeReadSlotContext(InComponent), 
		[&bHasMaterials](const UE::Ava::FMaterialBridgeReadSlotContext&, const UE::Ava::FMaterialBridgeReadSlot& InSlot)->UE::Ava::EControlFlow
		{
			bHasMaterials = true;
			return UE::Ava::EControlFlow::Break;
		}
		, /*Options*/{});

	if (bHasMaterials)
	{
		MarkModifierDirty();
	}
}

void UAvaMask2DReadModifier::RemoveFromActor(AActor* InActor)
{
	Super::RemoveFromActor(InActor);

	MaterialContainerStates.Remove(InActor);

	MaterialStates.RemoveAll(
		[InActor](const FAvaMask2DMaterialMaskState& InState)
		{
			return InState.TopmostMaterialContainer == InActor;
		});
}

EBlendMode UAvaMask2DReadModifier::GetBlendMode() const
{
	return bUseBlur || bUseFeathering ? EBlendMode::BLEND_Translucent : EBlendMode::BLEND_Masked;
}

bool UAvaMask2DReadModifier::ApplyRead(TNotNull<UObject*> InMaterialContainer, TConstArrayView<TNotNull<const UGeometryMaskCanvas*>> InCanvases)
{
	if (InCanvases.IsEmpty())
	{
		return true;
	}

	const UE::Ava::FMaterialBridge* const MaterialBridge = UE::Ava::FMaterialBridgeRegistry::Get().GetMaterialBridge(UE::Ava::FConstDataView(InMaterialContainer));
	if (!MaterialBridge)
	{
		// Ok to continue, just nothing done for this actor
		return true;
	}

	const TSharedPtr<FGeometryMaskCanvasSharedData> CanvasSharedData = InCanvases[0]->GetSharedData();
	if (!ensure(CanvasSharedData.IsValid()))
	{
		return true;
	}

	// using only the first canvas parameters as it is consistent across the other canvases.
	const FVector2f ViewportPadding = CanvasSharedData->TextureSize == FIntPoint::ZeroValue 
		? FVector2f::Zero()
		: FVector2f::One() - (FVector2f(CanvasSharedData->TextureSize + CanvasSharedData->ViewportPadding) / CanvasSharedData->TextureSize);

	FAvaMask2DMaskState MaskedState;

#if WITH_EDITOR
	MaskedState.OutputProcessor = GetDefault<UAvaMaskSettings>()->GetMaterialFunction();
#endif
	MaskedState.MaterialParameters.BlendMode = GetBlendMode();
	MaskedState.MaterialParameters.BaseOpacity = BaseOpacity;
	MaskedState.MaterialParameters.bInvert = bInverted;
	MaskedState.MaterialParameters.Padding = ViewportPadding;
	MaskedState.MaterialParameters.bApplyFeathering = bUseFeathering;
	MaskedState.MaterialParameters.OuterFeatherRadius = OuterFeatherRadius;
	MaskedState.MaterialParameters.InnerFeatherRadius = InnerFeatherRadius;
	MaskedState.MaterialParameters.RenderTarget = InCanvases[0]->GetRenderTarget();

	// Fill the mask indices of each of the canvases
	for (int32 Index = 0; Index < MaxCanvasCount; ++Index)
	{
		float& MaskIndexComponent = MaskedState.MaterialParameters.MaskIndices.Component(Index);
		if (InCanvases.IsValidIndex(Index) && ensure(InCanvases[Index]->GetRenderTarget() == MaskedState.MaterialParameters.RenderTarget))
		{
			const int16 SliceIndex = InCanvases[Index]->GetRenderTargetSliceIndex();
			MaskIndexComponent = static_cast<float>(SliceIndex);
		}
		else
		{
			MaskIndexComponent = -1.f;
		}
	}

	TSet<TObjectKey<UPrimitiveComponent>> DirtiedComponentKeys;

	MaterialBridge->AccessSlots(UE::Ava::FMaterialBridgeWriteSlotContext(InMaterialContainer)
		, [This=this, &MaskedState, &DirtiedComponentKeys](const UE::Ava::FMaterialBridgeWriteSlotContext& InContext, UE::Ava::FMaterialBridgeWriteSlot& InSlot)->UE::Ava::EControlFlow
		{
			if (This->ApplyMaskState(InContext, InSlot, MaskedState))
			{
				UObject* const MaterialContainer = InContext.GetMaterialContainerObject();
				check(MaterialContainer); // ApplyMask succeeded. MaterialContainer should be valid.

				UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(MaterialContainer);
				if (!Component)
				{
					Component = MaterialContainer->GetTypedOuter<UPrimitiveComponent>();
				}
				// Components that are already marked render state dirty will not get the RenderStateDirty delegate called.
				// Systems like cloners rely on this event to correctly update their materials.
				if (Component && Component->IsRenderStateDirty())
				{
					DirtiedComponentKeys.Add(Component);
				}
			}
			return UE::Ava::EControlFlow::Continue;
		}
		, /*Options*/{});

	UE::ActorModifierCore::FRenderStateDirtyReasonScope Scope(UE::ActorModifierCore::ERenderStateDirtyReason::Material);
	for (TObjectKey<UPrimitiveComponent> DirtiedComponentKey : DirtiedComponentKeys)
	{
		if (UPrimitiveComponent* DirtiedComponent = DirtiedComponentKey.ResolveObjectPtr())
		{
			UActorComponent::MarkRenderStateDirtyEvent.Broadcast(*DirtiedComponent);
		}
	}

	return true;
}

bool UAvaMask2DReadModifier::ApplyMaskState(const UE::Ava::FMaterialBridgeWriteSlotContext& InContext, UE::Ava::FMaterialBridgeWriteSlot& InSlot, const FAvaMask2DMaskState& InMaskState)
{
	UE::ActorModifierCore::FRenderStateDirtyReasonScope Scope(UE::ActorModifierCore::ERenderStateDirtyReason::Material);

	// Request the container to support the current blend mode.
	// Do this first as the slot material could change from this request.
	FAvaMaterialBridgeBlendModeFeature BlendModeFeature;
	BlendModeFeature.BlendMode = GetBlendMode();
	InSlot.RequestFeature(BlendModeFeature);

	UMaterialInterface* const SlotMaterial = InSlot.GetMaterial() ? InSlot.GetMaterial() : UAvaMaskSubsystem::StaticGetDefaultMaskMaterial();
	if (!SlotMaterial)
	{
		return false;
	}

	UObject* const MaterialContainer = InContext.GetMaterialContainerObject();
	if (!ensure(MaterialContainer))
	{
		return false;
	}

	UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(SlotMaterial);
	if (!MID)
	{
		// Slot's material is an MIC or a UMaterial. Create an MID out of these
		MID = UMaterialInstanceDynamic::Create(SlotMaterial, MaterialContainer);
	}

	// Update Material State's MID
	{
		const UObject* const TopmostContainer = InContext.GetTopmostContext().GetMaterialContainerObject();
		const UObject* const TopmostContainerOuter = TopmostContainer ? TopmostContainer->GetOuter() : nullptr;

		const FAvaMask2DMaterialSlotId SlotId(MaterialContainer->GetPathName(TopmostContainerOuter), InSlot.GetSlotId(), SlotMaterial);

		if (FAvaMask2DMaterialMaskState* MaterialState = MaterialStates.FindByKey(SlotId))
		{
			MaterialState->MaterialWeak = MID;
		}
	}

	InSlot.SetMaterial(MID);
	InMaskState.Apply(MID);
	return true;
}

bool UAvaMask2DReadModifier::ApplyRead(AActor* InActor, FAvaMask2DActorData& InActorData)
{
	FText FailReason;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return ApplyRead(InActor, InActorData, FailReason);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAvaMask2DReadModifier::ApplyRead(AActor* InActor, FAvaMask2DActorData& InActorData, FText& OutFailReason)
{
	if (InActor)
	{
		if (UGeometryMaskCanvas* Canvas = GetCurrentCanvas())
		{
			return ApplyRead(InActor, { Canvas });
		}
	}
	// Ok to continue, just nothing done for this actor
	return true;
}

bool UAvaMask2DReadModifier::ForEachUsedCanvasName(TFunctionRef<bool(FName)> InFunc) const
{
	if (!Super::ForEachUsedCanvasName(InFunc))
	{
		return false; 
	}

	for (FName AdditionalChannel : AdditionalChannels)
	{
		if (!InFunc(AdditionalChannel))
		{
			return false;
		}
	}

	return true;
}

void UAvaMask2DReadModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("MaskRead"));
	InMetadata.SetCategory(TEXT("Rendering"));
	InMetadata.DisallowAfter(TEXT("MaskWrite"));
	InMetadata.DisallowBefore(TEXT("MaskWrite"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Masked Layer (Output)"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Reads from a canvas texture and uses it on materials"));
#endif
}

void UAvaMask2DReadModifier::OnModifierEnabled(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierEnabled(InReason);

#if WITH_EDITOR
	UMaterial::OnMaterialCompilationFinished().AddUObject(this, &UAvaMask2DReadModifier::OnMaterialCompiled);
	FEditorDelegates::EndPIE.AddUObject(this, &UAvaMask2DReadModifier::OnPIEEnded);
#endif
}

void UAvaMask2DReadModifier::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierDisabled(InReason);

#if WITH_EDITOR
	UMaterial::OnMaterialCompilationFinished().RemoveAll(this);
	FEditorDelegates::EndPIE.RemoveAll(this);
#endif
}

void UAvaMask2DReadModifier::SavePreState()
{
	Super::SavePreState();

	ForEachActor<AActor>(
		[This=this](AActor* InActor)
		{
			const UActorModifierCoreBase* MaskModifier = UE::ActorModifierCore::Utilities::FindFirstActorModifierByClass(InActor, UAvaMask2DBaseModifier::StaticClass());

			// If first modifier in the hierarchy is not this modifier, then skip actor
			if (MaskModifier == This)
			{
				This->StoreActorState(InActor);
			}
			return true;
		}
		, EActorModifierCoreLookup::SelfAndAllChildren);
}

void UAvaMask2DReadModifier::RestorePreState()
{
	Super::RestorePreState();

	// Restore material container state
	TMap<TObjectPtr<UObject>, TInstancedStruct<FAvaMaterialContainerState>> OldContainerStateData = (MaterialContainerStates);
	MaterialContainerStates.Reset();

	for (TPair<TObjectPtr<UObject>, TInstancedStruct<FAvaMaterialContainerState>>& Pair : OldContainerStateData)
	{
		if (!Pair.Key)
		{
			continue;
		}

		const UE::Ava::FMaterialBridge* const MaterialBridge = UE::Ava::FMaterialBridgeRegistry::Get().GetMaterialBridge(UE::Ava::FConstDataView(Pair.Key));

		// Actors have a built-in bridge so there should always be a valid material bridge found
		if (ensureMsgf(MaterialBridge, TEXT("Could not find a valid material bridge for actor of type '%s'!"), *GetNameSafe(Pair.Key->GetClass())))
		{
			UE::ActorModifierCore::FRenderStateDirtyReasonScope Scope(UE::ActorModifierCore::ERenderStateDirtyReason::Material);
			MaterialBridge->ApplyState(UE::Ava::FMaterialBridgeApplyStateContext(Pair.Key), Pair.Value, /*Options*/{});
		}
	}

	// Restore material state
	for (const FAvaMask2DMaterialMaskState& MaterialState : MaterialStates)
	{
		MaterialState.Apply();
	}
	MaterialStates.Reset();
}

void UAvaMask2DReadModifier::StoreActorState(AActor* InActor)
{
	const UE::Ava::FMaterialBridge* const MaterialBridge = UE::Ava::FMaterialBridgeRegistry::Get().GetMaterialBridge(UE::Ava::FConstDataView(InActor));

	// Actors have a built-in bridge so there should always be a valid material bridge found
	if (!ensureMsgf(MaterialBridge, TEXT("Could not find a valid material bridge for actor of type '%s'!"), *GetNameSafe(InActor->GetClass())))
	{
		return;
	}

	TInstancedStruct<FAvaMaterialContainerState>* MaterialContainerState = MaterialContainerStates.Find(InActor);
	if (!MaterialContainerState)
	{
		// only call store on new states
		MaterialContainerState = &MaterialContainerStates.Add(InActor);
		MaterialBridge->StoreState(UE::Ava::FMaterialBridgeStoreStateContext(InActor), MaterialContainerState, /*Options*/{});
	}

	MaterialBridge->AccessSlots(UE::Ava::FMaterialBridgeReadSlotContext(InActor), 
		[&MaterialStates = MaterialStates, InActor](const UE::Ava::FMaterialBridgeReadSlotContext& InContext, const UE::Ava::FMaterialBridgeReadSlot& InSlot)->UE::Ava::EControlFlow
		{
			const UObject* const MaterialContainer = InContext.GetMaterialContainerObject();
			if (!MaterialContainer)
			{
				return UE::Ava::EControlFlow::Continue;
			}

			const FAvaMask2DMaterialSlotId SlotId(MaterialContainer->GetPathName(InActor->GetOuter()), InSlot.GetSlotId(), InSlot.GetMaterial());

			FAvaMask2DMaterialMaskState* MaterialState = MaterialStates.FindByKey(SlotId);
			if (!MaterialState)
			{
				// only call store on new states
				MaterialState = &MaterialStates.Emplace_GetRef(SlotId);
				MaterialState->MaterialWeak = Cast<UMaterialInstanceDynamic>(InSlot.GetMaterial());
				MaterialState->TopmostMaterialContainer = InActor;
				MaterialState->Store();
			}
			return UE::Ava::EControlFlow::Continue;
		}
		, /*Options*/{});
}

void UAvaMask2DReadModifier::OnBaseOpacityChanged()
{
	MarkModifierDirty();
}

void UAvaMask2DReadModifier::OnAdditionalChannelsChanged()
{
	MarkModifierDirty();
}

void UAvaMask2DReadModifier::SetupMaskComponent(UActorComponent* InComponent)
{
}

void UAvaMask2DReadModifier::SetupMaskReadComponent(IGeometryMaskReadInterface* InMaskReader)
{
}

#undef LOCTEXT_NAMESPACE
