// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mask2D/AvaMask2DWriteModifier.h"

#include "AvaMaskUtilities.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/CanvasRenderTarget2D.h"
#include "Engine/Engine.h"
#include "GeometryMaskCanvas.h"
#include "GeometryMaskWriteComponent.h"
#include "GeometryMaskWriter.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/Utilities/ActorModifierCoreUtilities.h"

#define LOCTEXT_NAMESPACE "AvaMask2DWriteModifier"

#if WITH_EDITOR
const TAvaPropertyChangeDispatcher<UAvaMask2DWriteModifier> UAvaMask2DWriteModifier::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UAvaMask2DWriteModifier, WriteOperation), &UAvaMask2DWriteModifier::OnWriteOperationChanged }
};
#endif

UAvaMask2DWriteModifier::UAvaMask2DWriteModifier()
{
	MaskWriter = UE::GeometryMask::FMaskWriter::Create();
}

#if WITH_EDITOR
void UAvaMask2DWriteModifier::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	UAvaMask2DWriteModifier::PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UAvaMask2DWriteModifier::Apply()
{
	Super::Apply();
	MaskActors.Reset();

	if (UGeometryMaskCanvas* Canvas = GetCurrentCanvas())
	{
		ForEachActor<AActor>(
			[This=this, Canvas](AActor* InActor)
			{
				const UActorModifierCoreBase* MaskModifier = UE::ActorModifierCore::Utilities::FindFirstActorModifierByClass(InActor, UAvaMask2DBaseModifier::StaticClass());

				// If first modifier in the hierarchy is not this modifier, then skip actor
				if (MaskModifier == This)
				{
					This->ApplyWrite(InActor);
				}
				return true;
			}
			, EActorModifierCoreLookup::SelfAndAllChildren);
	}

	Next();
}

void UAvaMask2DWriteModifier::OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent)
{
	Super::OnRenderStateUpdated(InActor, InComponent);

	if (!CanMarkModifierDirty())
	{
		return;
	}

	const UActorModifierCoreBase* MaskModifier = UE::ActorModifierCore::Utilities::FindFirstActorModifierByClass(InActor, UAvaMask2DBaseModifier::StaticClass());

	// Only trigger update if first modifier found in the hierarchy above us is this modifier
	if (MaskModifier == this)
	{
		RemoveFromActor(InActor);
		MarkModifierDirty();
	}
}

bool UAvaMask2DWriteModifier::ApplyWrite(TNotNull<AActor*> InActor)
{
	// Only add read/write component to actors with primitives
	if (ActorSupportsMaskReadWrite(InActor))
	{
		MaskActors.Add(InActor);
	}
	return true;
}

bool UAvaMask2DWriteModifier::ApplyWrite(AActor* InActor, FAvaMask2DActorData& InActorData)
{
	if (InActor)
	{
		if (UGeometryMaskCanvas* Canvas = GetCurrentCanvas())
		{
			return ApplyWrite(InActor);
		}
	}
	return true;
}

void UAvaMask2DWriteModifier::SetWriteOperation(const EGeometryMaskCompositeOperation InWriteOperation)
{
	if (WriteOperation != InWriteOperation)
	{
		WriteOperation = InWriteOperation;
		OnWriteOperationChanged();
	}
}

void UAvaMask2DWriteModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("MaskWrite"));
	InMetadata.SetCategory(TEXT("Rendering"));
	InMetadata.DisallowAfter(TEXT("MaskRead"));
	InMetadata.DisallowBefore(TEXT("MaskRead"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Mask Layer (Input)"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Writes to a canvas texture used by the masked layer in materials"));
#endif
}

void UAvaMask2DWriteModifier::OnModifierEnabled(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierEnabled(InReason);
	TryResolveCanvas();
}

void UAvaMask2DWriteModifier::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierDisabled(InReason);
	ResetCanvas();
}

void UAvaMask2DWriteModifier::SetupMaskComponent(UActorComponent* InComponent)
{
	if (!InComponent)
	{
		return;
	}
	
	if (IGeometryMaskWriteInterface* Writer = Cast<IGeometryMaskWriteInterface>(InComponent))
	{
		SetupMaskWriteComponent(Writer);
	}

	Super::SetupMaskComponent(InComponent);
}

void UAvaMask2DWriteModifier::OnCanvasSet(TNotNull<UGeometryMaskCanvas*> InCanvas)
{
	InCanvas->AddWriter(this);
}

void UAvaMask2DWriteModifier::OnCanvasReset()
{
	if (UGeometryMaskCanvas* Canvas = CanvasWeak.Get())
	{
		Canvas->RemoveWriter(this);
	}
}

void UAvaMask2DWriteModifier::RemoveFromActor(AActor* InActor)
{
	Super::RemoveFromActor(InActor);
	MaskActors.Remove(InActor);
}

bool UAvaMask2DWriteModifier::IsMaskWriterEnabled() const
{
	return IsModifierEnabled() && CanvasWeak.IsValid();
}

const FGeometryMaskWriteParameters& UAvaMask2DWriteModifier::GetParameters() const
{
	return Parameters;
}

void UAvaMask2DWriteModifier::SetParameters(FGeometryMaskWriteParameters& InParameters)
{
	Parameters = InParameters;

	UGeometryMaskCanvas* Canvas = CanvasWeak.Get();

	// We have changed canvas, unregister this writer
	if (Canvas && Canvas->GetFName() != InParameters.CanvasName)
	{
		ResetCanvas();
	}

	// Update canvas
	if (!CanvasWeak.IsValid())
	{
		TryResolveCanvas();
	}
}

void UAvaMask2DWriteModifier::DrawToCanvas(FCanvas* InCanvas)
{
	if (IsModifierEnabled())
	{
		Parameters.CanvasName = Channel;
		Parameters.OperationType = WriteOperation;

		UE::GeometryMask::FMaskWriter::FDrawParams DrawParams
			{
				.Canvas = InCanvas,
				.Parameters = &Parameters,
				.bWriteWhenHidden = true,
			};

		for (AActor* Actor : MaskActors)
		{
			DrawParams.Actor = Actor;
			MaskWriter->DrawToCanvas(DrawParams);
		}
	}
}

FOnGeometryMaskSetCanvasNativeDelegate& UAvaMask2DWriteModifier::OnSetCanvas()
{
	return OnSetCanvasDelegate;
}

UE::GeometryMask::FMaskWriter* UAvaMask2DWriteModifier::GetMaskWriter()
{
	return MaskWriter.Get();
}

void UAvaMask2DWriteModifier::SetupMaskWriteComponent(IGeometryMaskWriteInterface* InMaskWriter)
{
	if (const UGeometryMaskCanvas* Canvas = GetCurrentCanvas())
	{
		// Set canvas name to write to
		FGeometryMaskWriteParameters WriteParameters = InMaskWriter->GetParameters();
		WriteParameters.CanvasName = Channel;

		InMaskWriter->SetParameters(WriteParameters);
	}
}

void UAvaMask2DWriteModifier::OnWriteOperationChanged()
{
	MarkModifierDirty();
}

#undef LOCTEXT_NAMESPACE
