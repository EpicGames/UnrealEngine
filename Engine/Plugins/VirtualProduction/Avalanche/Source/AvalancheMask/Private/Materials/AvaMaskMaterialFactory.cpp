// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMaskMaterialFactory.h"

#include "AvaMaskLog.h"
#include "AvaMaskUtilities.h"
#include "Material/DynamicMaterialInstance.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Misc/PackageName.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "EditorScriptingHelpers.h"
#include "Factories/MaterialInstanceConstantFactoryNew.h"
#include "FileHelpers.h"
#include "IAssetTools.h"
#include "Model/DynamicMaterialModelEditorOnlyData.h"
#endif

namespace UE::AvaMask::Private
{
	static FString GetMICOutputPath(const TObjectPtr<UMaterialInterface>& InMaterial, const EBlendMode InBlendMode, FString& OutShortName)
	{
		FString CurrentPath = InMaterial.GetPackage()->GetName();
		const EBlendMode CurrentBlendMode = InMaterial->GetBlendMode();
		const FString CurrentBlendModeStr = UE::AvaMask::Internal::GetBlendModeString(CurrentBlendMode);

		FString OutputPath;
		FString OutputBlendModeStr = UE::AvaMask::Internal::GetBlendModeString(InBlendMode);

		FString MountPoint, PackagePath, ShortName;
		
		// Check if is already in ExternalObjects folder
		static FString GeneratedMaterialPath = UE::AvaMask::Internal::GetGeneratedMaterialPath();
		if (CurrentPath.StartsWith(GeneratedMaterialPath))
		{
			OutputPath = CurrentPath;
			
			// Material is already in generated path, attempt to strip the blendmode suffix
			OutputPath.RemoveFromEnd(CurrentBlendModeStr);

			// Then re-add the new one
			OutputPath += OutputBlendModeStr;
		}
		else
		{
			FPackageName::SplitLongPackageName(CurrentPath, MountPoint, PackagePath, OutShortName);
		
			OutputPath = FString::Printf(
				TEXT("%s%s%s%s_%s"),
				*GeneratedMaterialPath,
				*MountPoint.RightChop(1),
				*PackagePath,
				*OutShortName,
				*OutputBlendModeStr);
		}

		FPackageName::SplitLongPackageName(OutputPath, MountPoint, PackagePath, OutShortName);

		return OutputPath;
	}
}

TSubclassOf<UMaterialInterface> UAvaMaskMaterialFactory::GetSupportedMaterialClass() const
{
	return UMaterial::StaticClass();
}

UMaterialInstanceDynamic* UAvaMaskMaterialFactory::CreateMID(
	UObject* InOuter
	, const TObjectPtr<UMaterialInterface>& InParentMaterial
	, const EBlendMode InBlendMode) const
{
	const EBlendMode TargetBlendMode = UE::AvaMask::Internal::GetTargetBlendMode(InParentMaterial->GetBlendMode(), InBlendMode);
	const bool bHasRequestedBlendMode = InParentMaterial->GetBlendMode() == TargetBlendMode;
	if (!bHasRequestedBlendMode)
	{
		UMaterialInstanceConstant* MaterialInstanceConstant	=
			GetDefault<UAvaMaskMaterialInstanceConstantFactory>()
				->CreateMIC(InOuter, InParentMaterial, TargetBlendMode);

		if (!MaterialInstanceConstant)
		{
			return nullptr;
		}

		return GetDefault<UAvaMaskMaterialInstanceConstantFactory>()
				->CreateMID(InOuter, MaterialInstanceConstant, TargetBlendMode);
	}
	
	return UMaterialInstanceDynamic::Create(InParentMaterial, InOuter);
}

TSubclassOf<UMaterialInterface> UAvaMaskMaterialInstanceConstantFactory::GetSupportedMaterialClass() const
{
	return UMaterialInstanceConstant::StaticClass();
}

UMaterialInstanceDynamic* UAvaMaskMaterialInstanceConstantFactory::CreateMID(
	UObject* InOuter
	, const TObjectPtr<UMaterialInterface>& InParentMaterial
	, const EBlendMode InBlendMode) const
{
	const EBlendMode TargetBlendMode = UE::AvaMask::Internal::GetTargetBlendMode(InParentMaterial->GetBlendMode(), InBlendMode);
	const bool bHasRequestedBlendMode = InParentMaterial->GetBlendMode() == TargetBlendMode;
	if (!bHasRequestedBlendMode)
	{
		// Get or create with the correct blendmode
		UMaterialInstanceConstant* MIC = CreateMIC(InOuter, InParentMaterial, TargetBlendMode);

		// Won't be valid if attempted at runtime
		if (!MIC)
		{
			return nullptr;
		}

		// Then create an MID
		return UMaterialInstanceDynamic::Create(MIC, InOuter); 
	}

	return UMaterialInstanceDynamic::Create(InParentMaterial, InOuter);
}

UMaterialInstanceConstant* UAvaMaskMaterialInstanceConstantFactory::CreateMIC(
	UObject* InOuter
	, const TObjectPtr<UMaterialInterface>& InParentMaterial
	, const EBlendMode InBlendMode) const
{
	const EBlendMode TargetBlendMode = UE::AvaMask::Internal::GetTargetBlendMode(InParentMaterial->GetBlendMode(), InBlendMode);
	
	FString ShortName;
	FString OutputPath = UE::AvaMask::Private::GetMICOutputPath(InParentMaterial, TargetBlendMode, ShortName);

	if (UMaterialInstanceConstant* MIC = LoadObject<UMaterialInstanceConstant>(nullptr, *OutputPath))
	{
#if WITH_EDITOR
		MIC->UpdateCachedData();
#endif
		return MIC;
	}

#if WITH_EDITOR
	FString FailureReason;
	OutputPath = EditorScriptingHelpers::ConvertAnyPathToObjectPath(OutputPath, FailureReason);
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	if (const FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(OutputPath));
		AssetData.IsValid())
	{
		UE_LOGF(LogAvaMask, Warning, "Failed to load MIC at path '%ls' but the asset itself exists.", *OutputPath);
		return nullptr;
	}
#endif

#if WITH_EDITOR
	{
		IAssetTools& AssetTools = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

		UMaterialInstanceConstantFactoryNew* Factory = NewObject<UMaterialInstanceConstantFactoryNew>();
		Factory->InitialParent = InParentMaterial;

		UMaterialInstanceConstant* MIC = Cast<UMaterialInstanceConstant>(
			AssetTools.CreateAsset(
				ShortName,
				FPackageName::GetLongPackagePath(OutputPath),
				UMaterialInstanceConstant::StaticClass(),
				Factory));

		MIC->BasePropertyOverrides.bOverride_BlendMode = true;
		MIC->BasePropertyOverrides.BlendMode = TargetBlendMode;
		MIC->BlendMode = TargetBlendMode;

		MIC->UpdateCachedData();

		// Save package/asset
		{
			FEditorFileUtils::FPromptForCheckoutAndSaveParams SaveParams;
			SaveParams.bCheckDirty = true;
			SaveParams.bPromptToSave = false;
			FEditorFileUtils::PromptForCheckoutAndSave({ MIC->GetOutermost() }, SaveParams);
		}

		return MIC;
	}
#else
	UE_LOGF(LogAvaMask, Warning, "Can't create a MaterialInstanceConstant at runtime, ensure this is done at edit time first.")
	return nullptr;
#endif
}

TSubclassOf<UMaterialInterface> UAvaMaskMaterialInstanceDynamicFactory::GetSupportedMaterialClass() const
{
	return UMaterialInstanceDynamic::StaticClass();
}

UMaterialInstanceDynamic* UAvaMaskMaterialInstanceDynamicFactory::CreateMID(
	UObject* InOuter
	, const TObjectPtr<UMaterialInterface>& InParentMaterial
	, const EBlendMode InBlendMode) const
{
	const EBlendMode TargetBlendMode = UE::AvaMask::Internal::GetTargetBlendMode(InParentMaterial->GetBlendMode(), InBlendMode);
	const bool bHasRequestedBlendMode = InParentMaterial->GetBlendMode() == TargetBlendMode;
	if (!bHasRequestedBlendMode)
	{
		// Need to create a new asset based material with the correct blendmode, MID's are by definition not assets, so get parent
		UMaterialInterface* ParentAsset = UE::AvaMask::Internal::GetNonTransientParentMaterial(Cast<UMaterialInstance>(InParentMaterial));

		// @note: this is likely to fail if called at runtime (unless the same MID had an MIC created for it previously)
		UMaterialInstanceDynamic* MID = GetDefault<UAvaMaskMaterialInstanceConstantFactory>()->CreateMID(InOuter, ParentAsset, TargetBlendMode);
		MID->CopyInterpParameters(Cast<UMaterialInstance>(InParentMaterial));
		return MID;
	}

	// Correct blend mode, so simply duplicate in memory
	return DuplicateObject<UMaterialInstanceDynamic>(Cast<UMaterialInstanceDynamic>(InParentMaterial), InOuter ? InOuter : InParentMaterial->GetOuter());
}

TSubclassOf<UMaterialInterface> UAvaMaskDesignedMaterialFactory::GetSupportedMaterialClass() const
{
	return UDynamicMaterialInstance::StaticClass();
}

UMaterialInstanceDynamic* UAvaMaskDesignedMaterialFactory::CreateMID(
	UObject* InOuter
	, const TObjectPtr<UMaterialInterface>& InParentMaterial
	, const EBlendMode InBlendMode) const
{
	if (UDynamicMaterialInstance* DesignedMaterial = Cast<UDynamicMaterialInstance>(InParentMaterial.Get()))
	{
		return DesignedMaterial;
	}
	
	return nullptr;
}
