// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "UObject/Object.h"

#include "AvaMaskMaterialFactory.generated.h"

class UMaterialInstanceConstant;
class UMaterialInstanceDynamic;
class UMaterialInterface;

/**
 * Implementations are responsible for creating MaterialInstanceDynamic's for a given parent UMaterialInterface.
 * This differs to AvaObjectHandle implementations, as they only deal with existing object instances and don't create them.
 */
UCLASS(Abstract)
class UAvaMaskMaterialFactoryBase
	: public UObject
{
	GENERATED_BODY()

public:
	virtual TSubclassOf<UMaterialInterface> GetSupportedMaterialClass() const
	PURE_VIRTUAL(UAvaMaskMaterialFactoryBase::GetSupportedMaterialClass, return nullptr;)
	
	virtual UMaterialInstanceDynamic* CreateMID(UObject* InOuter, const TObjectPtr<UMaterialInterface>& InParentMaterial, const EBlendMode InBlendMode = EBlendMode::BLEND_Masked) const
	PURE_VIRTUAL(UAvaMaskMaterialFactoryBase::CreateMID, return nullptr;)
};

/** Creates an MID for a given Material asset. */
UCLASS()
class UAvaMaskMaterialFactory
	: public UAvaMaskMaterialFactoryBase
{
	GENERATED_BODY()
	
public:
	virtual TSubclassOf<UMaterialInterface> GetSupportedMaterialClass() const override;
	virtual UMaterialInstanceDynamic* CreateMID(UObject* InOuter, const TObjectPtr<UMaterialInterface>& InParentMaterial, const EBlendMode InBlendMode) const override;
};

/** Creates an MID for a given MIC. */
UCLASS()
class UAvaMaskMaterialInstanceConstantFactory
	: public UAvaMaskMaterialFactoryBase
{
	GENERATED_BODY()
	
public:
	virtual TSubclassOf<UMaterialInterface> GetSupportedMaterialClass() const override;
	virtual UMaterialInstanceDynamic* CreateMID(UObject* InOuter, const TObjectPtr<UMaterialInterface>& InParentMaterial, const EBlendMode InBlendMode) const override;
	UMaterialInstanceConstant* CreateMIC(UObject* InOuter, const TObjectPtr<UMaterialInterface>& InParentMaterial, const EBlendMode InBlendMode) const;
};

/** Creates an MID for a given MID. */
UCLASS()
class UAvaMaskMaterialInstanceDynamicFactory
	: public UAvaMaskMaterialFactoryBase
{
	GENERATED_BODY()
	
public:
	virtual TSubclassOf<UMaterialInterface> GetSupportedMaterialClass() const override;
	virtual UMaterialInstanceDynamic* CreateMID(UObject* InOuter, const TObjectPtr<UMaterialInterface>& InParentMaterial, const EBlendMode InBlendMode) const override;
};

/** Creates an MID for a given Material Designer material. */
UCLASS()
class UAvaMaskDesignedMaterialFactory
	: public UAvaMaskMaterialFactoryBase
{
	GENERATED_BODY()
	
public:
	virtual TSubclassOf<UMaterialInterface> GetSupportedMaterialClass() const override;
	virtual UMaterialInstanceDynamic* CreateMID(UObject* InOuter, const TObjectPtr<UMaterialInterface>& InParentMaterial, const EBlendMode InBlendMode) const override;
};
