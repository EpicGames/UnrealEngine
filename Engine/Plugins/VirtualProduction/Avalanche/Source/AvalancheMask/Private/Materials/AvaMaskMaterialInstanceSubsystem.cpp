// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/AvaMaskMaterialInstanceSubsystem.h"

#include "AvaMaskLog.h"
#include "AvaMaskMaterialFactory.h"
#include "Engine/Engine.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "UObject/UObjectIterator.h"

UMaterialInstanceDynamic* UAvaMaskMaterialInstanceProvider::FindOrAddMID(
	UMaterialInterface* InParentMaterial
	, const uint32& InInstanceKey
	, const EBlendMode InBlendMode)
{
	if (!ensureMsgf(InParentMaterial, TEXT("ParentMaterial was invalid.")))
	{
		return nullptr;
	}

	FAvaMaskMaterialPermutations* MaterialPermutations = MaterialInstanceCache.Find(InParentMaterial);
	if (!MaterialPermutations)
	{
		MaterialPermutations = &MaterialInstanceCache.Add(InParentMaterial, { InParentMaterial, {} });
	}
	check(MaterialPermutations);

	if (const TWeakObjectPtr<UMaterialInstanceDynamic>* WeakMaterialInstance = MaterialPermutations->Instances.Find(InInstanceKey))
	{
		if (WeakMaterialInstance->IsValid())
		{
			return WeakMaterialInstance->Get();
		}

		// Otherwise it was found but stale
	}

	if (UAvaMaskMaterialInstanceProvider* Parent = ParentProvider.Get())
	{
		if (UMaterialInstanceDynamic* ResultFromParentProvider = Parent->FindOrAddMID(InParentMaterial, InInstanceKey))
		{
			// If found in parent, duplicate with this as outer
			MaterialPermutations->Instances.Emplace(InInstanceKey, DuplicateObject(ResultFromParentProvider, this));
			return ResultFromParentProvider;
		}
	}

	if (MaterialInstanceFactories.IsEmpty())
	{
		UE_LOGF(LogAvaMask, Warning, "No MID found for the given key, and MaterialInstanceFactories is empty. Returning nullptr.")
		return nullptr;
	}

	// If we're here, we need to create the MID
	UMaterialInstanceDynamic* MaterialInstance = nullptr;
	if (const TObjectPtr<UAvaMaskMaterialFactoryBase>* MaterialInstanceFactory = MaterialInstanceFactories.Find(InParentMaterial->GetClass()))
	{
		MaterialInstance = (*MaterialInstanceFactory)->CreateMID(this, InParentMaterial, InBlendMode);
	}
	else
	{
		UE_LOGF(LogAvaMask, Warning, "No UAvaMaskMaterialFactory was found for class '%ls', using UMaterialInstanceDynamic::Create", *InParentMaterial->StaticClass()->GetName());
		MaterialInstance = UMaterialInstanceDynamic::Create(InParentMaterial, this);
	}

	MaterialPermutations->Instances.Emplace(InInstanceKey, MaterialInstance);

	return MaterialInstance;
}

void UAvaMaskMaterialInstanceProvider::SetParent(UAvaMaskMaterialInstanceProvider* InParentProvider)
{
	ParentProvider = InParentProvider;
}

void UAvaMaskMaterialInstanceProvider::SetFactories(const TMap<TSubclassOf<UMaterialInterface>, TObjectPtr<UAvaMaskMaterialFactoryBase>>& InFactories)
{
	MaterialInstanceFactories = InFactories;
}

void UAvaMaskMaterialInstanceWorldSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	MaterialInstanceEngineSubsystem = GEngine->GetEngineSubsystem<UAvaMaskMaterialInstanceSubsystem>();

	MaterialInstanceProvider = NewObject<UAvaMaskMaterialInstanceProvider>(this);
	MaterialInstanceProvider->SetParent(MaterialInstanceEngineSubsystem->GetMaterialInstanceProvider());
}

UAvaMaskMaterialInstanceProvider* UAvaMaskMaterialInstanceWorldSubsystem::GetMaterialInstanceProvider() const
{
	return MaterialInstanceProvider;
}

void UAvaMaskMaterialInstanceSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	MaterialInstanceProvider = NewObject<UAvaMaskMaterialInstanceProvider>(this);

	FindMaterialFactories();
}

UAvaMaskMaterialInstanceProvider* UAvaMaskMaterialInstanceSubsystem::GetMaterialInstanceProvider() const
{
	return MaterialInstanceProvider;
}

void UAvaMaskMaterialInstanceSubsystem::ClearCached()
{
	MaterialInstanceProvider->MaterialInstanceCache.Reset();
}

void UAvaMaskMaterialInstanceSubsystem::FindMaterialFactories()
{
	TMap<TSubclassOf<UMaterialInterface>, TObjectPtr<UAvaMaskMaterialFactoryBase>> MaterialInstanceFactories;
	MaterialInstanceFactories.Reserve(16); // Sensible initial number for amount of UMaterialInterface implementations
	
	for (const UClass* Class : TObjectRange<UClass>())
	{
		if (Class->IsChildOf(UAvaMaskMaterialFactoryBase::StaticClass())
			&& !Class->HasAnyClassFlags(CLASS_Deprecated | CLASS_Abstract | CLASS_NotPlaceable)
#if WITH_EDITOR
			&& Class->ClassGeneratedBy == nullptr
#endif
			)
		{
			UAvaMaskMaterialFactoryBase* FactoryCDO = Class->GetDefaultObject<UAvaMaskMaterialFactoryBase>();
			TSubclassOf<UMaterialInterface> SupportedMaterialClass = FactoryCDO->GetSupportedMaterialClass();
			MaterialInstanceFactories.Emplace(SupportedMaterialClass, FactoryCDO);
		}
	}

	MaterialInstanceProvider->SetFactories(MaterialInstanceFactories);
}
