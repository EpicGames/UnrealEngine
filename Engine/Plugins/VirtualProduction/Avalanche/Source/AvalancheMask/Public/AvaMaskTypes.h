// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryMaskTypes.h"
#include "Materials/MaterialParameters.h"
#include "AvaMaskTypes.generated.h"

class UMaterialInstanceDynamic;
class UMaterialInterface;
class UTexture;
class UTextureRenderTarget2DArray;

/** Describes a component/material slot pair. */
USTRUCT()
struct FAvaMask2DComponentMaterialPath
{
	GENERATED_BODY()

	UPROPERTY()
	FSoftComponentReference Component;

	UPROPERTY()
	int32 SlotIdx = 0;

	/** FAvaMask2DComponentMaterialPath == operator */
	bool operator==(const FAvaMask2DComponentMaterialPath& Other) const
	{
		return (Component == Other.Component) && (SlotIdx == Other.SlotIdx);
	}
};

inline uint32 GetTypeHash(const FAvaMask2DComponentMaterialPath& MaterialPath)
{
	return HashCombineFast(GetTypeHash(MaterialPath.Component), GetTypeHash(MaterialPath.SlotIdx));
}

/** Used to store, set and compare material parameter values. */
USTRUCT()
struct FAvaMask2DMaterialParameters
{
	GENERATED_BODY()

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAvaMask2DMaterialParameters() = default;
	FAvaMask2DMaterialParameters(const FAvaMask2DMaterialParameters&) = default;
	FAvaMask2DMaterialParameters(FAvaMask2DMaterialParameters&&) = default;
	FAvaMask2DMaterialParameters& operator=(const FAvaMask2DMaterialParameters&) = default;
	FAvaMask2DMaterialParameters& operator=(FAvaMask2DMaterialParameters&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	AVALANCHEMASK_API bool HasSameParameters(const FAvaMask2DMaterialParameters& InOther) const;

	/** Canvas/Channel name. */
	UPROPERTY()
	FName CanvasName;

	/** Mask texture. */
	UE_DEPRECATED(5.8, "Mask texture is deprecated. Use the mask texture collection member variable instead")
	UPROPERTY()
	TObjectPtr<UTexture> Texture_DEPRECATED = nullptr;

	/** Render target array to use for masking */
	UPROPERTY()
	TObjectPtr<UTextureRenderTarget2DArray> RenderTarget;

	/** Indices to the mask textures array. Each component (R,G,B,A) represents an index */
	UPROPERTY()
	FLinearColor MaskIndices = FLinearColor(-1, -1, -1, -1);

	/** Multiplies the mask texture to determine which channel to read from. */
	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	UPROPERTY()
	EGeometryMaskColorChannel Channel_DEPRECATED = EGeometryMaskColorChannel::Red;

	/** Multiplies the mask texture to determine which channel to read from. */
	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	UPROPERTY()
	FLinearColor ChannelAsVector_DEPRECATED = FLinearColor::Red;

	/** Whether the mask result should be inverted when applied. This is stored in the material as a float to allow runtime modification. */
	UPROPERTY()
	bool bInvert = false;

	/** Base opacity/alpha to use in Read mode. */
	UPROPERTY()
	float BaseOpacity = 0.0f;

	/** The render target might be larger/overscanned, so we need to compensate. */
	UPROPERTY()
	FVector2f Padding = FVector2f::Zero();

	UPROPERTY()
	bool bApplyFeathering = false;

	UPROPERTY()
	float OuterFeatherRadius = 0.0f;

	UPROPERTY()
	float InnerFeatherRadius = 0.0f;

	UPROPERTY()
	TEnumAsByte<EBlendMode> BlendMode = EBlendMode::BLEND_Opaque;

	/** Applies the current values of this struct to the given material parameters. Returns true if successful. */
	bool ApplyToMID(UMaterialInstanceDynamic* InMaterial) const;

	/** Set current values of the given material and stores it in the member variables of this struct. Returns true if successful. */
	bool StoreFromMaterial(const UMaterialInterface* InMaterial);
};

/** Encapsulates all parameters to apply to a given modifier subject. */
USTRUCT()
struct FAvaMask2DSubjectParameters
{
	GENERATED_BODY()

	UPROPERTY()
	FAvaMask2DMaterialParameters MaterialParameters;
};
