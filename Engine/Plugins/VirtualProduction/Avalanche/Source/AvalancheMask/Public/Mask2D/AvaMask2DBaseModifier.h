// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMaskTypes.h"
#include "AvaPropertyChangeDispatcher.h"
#include "GeometryMaskTypes.h"
#include "IGeometryMaskClient.h"
#include "Modifiers/ActorModifierArrangeBaseModifier.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "AvaMask2DBaseModifier.generated.h"

#define UE_API AVALANCHEMASK_API

namespace UE::Ava
{
	class FMaterialBridge;
}

class IAvaMaskMaterialCollectionHandle;
class IAvaMaskMaterialHandle;
class UAvaMaskMaterialInstanceSubsystem;
class UAvaObjectHandleSubsystem;
class UGeometryMaskCanvas;
class UGeometryMaskWorldSubsystem;
class UMaterialInterface;
class UPrimitiveComponent;
class UTexture;
struct FAvaMaterialContainerState;

UENUM(BlueprintType)
enum class EAvaMask2DMode : uint8
{
	Read		UMETA(DisplayName = "Target", ToolTip = "Use the specified Mask Channel to apply to this geometry"),
	Write		UMETA(DisplayName = "Source", ToolTip = "Use the specified Mask Channel to render this geometry to"),
};

/**
 * DEPRECATED for 5.8
 * Used for target actors to store essential information 
 */
USTRUCT(meta=(Deprecated="5.8"))
struct FAvaMask2DActorData
{
	GENERATED_BODY()

	/** The canvas texture to apply to this actor materials */
	UPROPERTY(Transient)
	TWeakObjectPtr<UTexture> CanvasTextureWeak;

	/** Original assigned materials, used to create instances from, and restore to when modifier removed. */
	UPROPERTY()
	TMap<FAvaMask2DComponentMaterialPath, TObjectPtr<UMaterialInterface>> OriginalMaterials;
};

/** Uses scene actors to create a mask texture and applies it to attached actors */
UCLASS(Abstract, MinimalAPI)
class UAvaMask2DBaseModifier : public UActorModifierArrangeBaseModifier, public IGeometryMaskClient
{
	GENERATED_BODY()

	friend class UAvaMask2DReadModifierShared;
	
public:
	//~ Begin UObject
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	bool UseParentChannel() const
	{
		return bUseParentChannel;
	}
	UE_API void SetUseParentChannel(const bool bInUseParentChannel);

	UE_API const FName GetChannel() const;
	UE_API void SetChannel(FName InChannel);

	bool IsInverted() const
	{
		return bInverted;
	}
	UE_API void SetIsInverted(const bool bInInvert);

	bool UseBlur() const
	{
		return bUseBlur;
	}
	UE_API void UseBlur(bool bInUseBlur);

	float GetBlurStrength() const
	{
		return BlurStrength;
	}
	UE_API void SetBlurStrength(float InBlurStrength);

	bool UseFeathering() const
	{
		return bUseFeathering;
	}
	UE_API void UseFeathering(bool bInUseFeathering);

	int32 GetOuterFeatherRadius() const
	{
		return OuterFeatherRadius;
	}
	UE_API void SetOuterFeatherRadius(int32 InFeatherRadius);

	int32 GetInnerFeatherRadius() const
	{
		return InnerFeatherRadius;
	}
	UE_API void SetInnerFeatherRadius(int32 InFeatherRadius);

	/** Utility to generate a unique mask name based on the associated Actor */
	UE_API FName GenerateUniqueMaskName() const;

	/** Utility to find an existing modifier on the provided actor or it's parent */
	UE_API static UAvaMask2DBaseModifier* FindMaskModifierOnActor(const AActor* InActor);

protected:
	//~ Begin IGeometryMaskClient
	UE_API virtual bool ForEachUsedCanvasName(TFunctionRef<bool(FName)> InFunc) const override;
	//~ End IGeometryMaskClient

	//~ Begin UActorModifierCoreBase
	UE_API virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	UE_API virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	UE_API virtual void OnModifierRemoved(EActorModifierCoreDisableReason InReason) override;
	UE_API virtual void OnModifiedActorTransformed() override;
	UE_API virtual void RestorePreState() override;
	UE_API virtual void Apply() override;
	//~ End UActorModifierCoreBase

	//~ Begin IAvaSceneTreeUpdateModifierExtension
	UE_API virtual void OnSceneTreeTrackedActorParentChanged(int32 InIdx, const TArray<TWeakObjectPtr<AActor>>& InPreviousParentActor, const TArray<TWeakObjectPtr<AActor>>& InNewParentActor) override;
	UE_API virtual void OnSceneTreeTrackedActorChildrenChanged(int32 InIdx, const TSet<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TSet<TWeakObjectPtr<AActor>>& InNewChildrenActors) override;
	//~ End IAvaSceneTreeUpdateModifierExtension

	//~ Begin IAvaRenderStateUpdateExtension
	UE_API virtual void OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent) override;
	//~ End IAvaRenderStateUpdateExtension

	UE_DEPRECATED(5.8, "Actor pre state logic has been moved to the Read modifier.")
	UE_API virtual void SaveActorPreState(AActor* InActor, FAvaMask2DActorData& InActorData);

	UE_DEPRECATED(5.8, "Actor pre state logic has been moved to the Read modifier.")
	UE_API virtual void RestoreActorPreState(AActor* InActor, FAvaMask2DActorData& InActorData);

	UE_API void OnUseParentChannelChanged();
	UE_API void OnChannelChanged();
	UE_API void OnInvertedChanged();
	UE_API void OnBlurChanged();
	UE_API void OnFeatherChanged();
	UE_API void OnCanvasChanged();

	UE_API void UpdateCanvas();

	/** 
	 * Stores specific properties on the canvas locally. 
	 * @param bInResolveCanvas whether to resolve canvas if the cached one is not valid
	 */	
	UE_API void CanvasParamsToLocal(bool bInResolveCanvas=true);

	/** Apply locally stored parameters to the canvas. */
	UE_API void LocalParamsToCanvas();

	UE_API void SetupChannelName();
	UE_API virtual void SetupMaskComponent(UActorComponent* InComponent);
	UE_API virtual void RemoveFromActor(AActor* InActor);

	UE_DEPRECATED(5.8, "ActorData has been deprecated. Actor restore takes place in the Material Read Modifier directly.")
	UE_API void OnMaskSetCanvas(const UGeometryMaskCanvas* InCanvas, AActor* InActor);

	UE_API UActorComponent* FindOrAddMaskComponent(TSubclassOf<UActorComponent> InComponentClass, AActor* InActor);

	template<typename InComponentClass UE_REQUIRES(std::is_base_of_v<UActorComponent, InComponentClass>)>
	InComponentClass* FindOrAddMaskComponent(AActor* InActor)
	{
		return Cast<InComponentClass>(FindOrAddMaskComponent(InComponentClass::StaticClass(), InActor));
	}

	UE_API static bool ActorSupportsMaskReadWrite(const AActor* InActor);

	/** Returns true if parent channel was found */
	UE_API bool TryResolveParentChannel();

	/** Returns world subsystem and, optionally, the level for this modifier */
	UGeometryMaskWorldSubsystem* GetGeometryMaskWorldSubsystem(const ULevel** OutLevel = nullptr) const;

	UE_API void TryResolveCanvas();

	UE_DEPRECATED(5.8, "ActorData has been deprecated. TryResolveCanvasTexture with actor data is no longer supported.")
	UE_API UTexture* TryResolveCanvasTexture(AActor* InActor, FAvaMask2DActorData& InActorData);

	UE_DEPRECATED(5.8, "Material Mask Handles have been deprecated. Material Bridge is used instead.")
	UE_API UAvaObjectHandleSubsystem* GetObjectHandleSubsystem();

	UE_API UAvaMaskMaterialInstanceSubsystem* GetMaterialInstanceSubsystem();

	/** 
	 * Returns or resolves the currently referenced canvas. 
	 * @param bInResolveCanvas whether to resolve for a new canvas if the cached canvas is not valid
	 */
	UE_API UGeometryMaskCanvas* GetCurrentCanvas(bool bInResolveCanvas=true);

	UE_DEPRECATED(5.8, "Material Mask Handles have been deprecated. Material Bridge is used instead.")
	UE_API virtual void OnMaterialsChanged(UObject* InMaterialOwner, const TArray<TSharedPtr<IAvaMaskMaterialHandle>>& InMaterialHandles);

	/** Restores modified actors to their original state and clears cache data */
	UE_DEPRECATED(5.8, "ActorData has been deprecated. Actor restore takes place in the Material Read Modifier directly.")
	UE_API void RestoreActors();

	/** Called to reset the canvas. Calls OnCanvasReset */
	UE_API void ResetCanvas();

	/** Called when the canvas has been set */
	virtual void OnCanvasSet(TNotNull<UGeometryMaskCanvas*> InCanvas)
	{
	}

	/** Called when the canvas has reset*/
	virtual void OnCanvasReset()
	{
	}

#if WITH_EDITOR
	UFUNCTION(CallInEditor, Category = "Mask2D")
	UE_API void VisualizeMask();
#endif

	/** Whether to get the channel from the parent (first one that specifies a mask channel) */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter = "UseParentChannel", Setter = "SetUseParentChannel", Category = "Mask2D", meta = (AllowPrivateAccess = "true", ValidEnumValues = "Self,Parent"))
	bool bUseParentChannel = false;

	/** Channel found when GetChannelFromParent is true */
	UPROPERTY(VisibleAnywhere, Category = "Mask2D", meta = (EditCondition = "bUseParentChannel", EditConditionHides))
	FName ParentChannel = NAME_None;

	/** Channel to read to or write from */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, Category = "Mask2D", meta = (EditCondition = "!bUseParentChannel", EditConditionHides, AllowPrivateAccess = "true"))
	FName Channel = NAME_None;

	/** Whether to apply the mask as inverted (visible becomes invisible and vice versa) */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter = "IsInverted", Setter = "SetIsInverted", Category = "Mask2D", meta = (AllowPrivateAccess = "true"))
	bool bInverted = false;

	UPROPERTY(BlueprintReadWrite, Getter = "UseBlur", Setter = "UseBlur", Category = "Mask2D|Shared", meta = (AllowPrivateAccess = "true"))
	bool bUseBlur = false;

	UPROPERTY(BlueprintReadWrite, Getter, Setter, Category = "Mask2D|Shared", meta = (ClampMin = 0.0, AllowPrivateAccess = "true", EditCondition = "bUseBlur", EditConditionHides))
	float BlurStrength = 16.0f;

	UPROPERTY(BlueprintReadWrite, Getter = "UseFeathering", Setter = "UseFeathering", Category = "Mask2D|Shared", meta = (AllowPrivateAccess = "true"))
	bool bUseFeathering = false;

	UPROPERTY(BlueprintReadWrite, Getter, Setter, Category = "Mask2D|Shared", meta = (ClampMin = 0, AllowPrivateAccess = "true", EditCondition = "bUseFeathering", EditConditionHides))
	int32 OuterFeatherRadius = 16;

	UPROPERTY(BlueprintReadWrite, Getter, Setter, Category = "Mask2D|Shared", meta = (ClampMin = 0, AllowPrivateAccess = "true", EditCondition = "bUseFeathering", EditConditionHides))
	int32 InnerFeatherRadius = 16;

	UE_DEPRECATED(5.8, "Material Mask Handles have been deprecated. Material Bridge is used instead.")
	UPROPERTY(Transient)
	TWeakObjectPtr<UAvaObjectHandleSubsystem> ObjectHandleSubsystem;

	UPROPERTY(Transient)
	TWeakObjectPtr<UAvaMaskMaterialInstanceSubsystem> MaterialInstanceSubsystem;

	/**
	 * Cached auto channel name, used when the user adds the modifier and immediately sets the mode to Source,
	 * in which case a different channel name is chosen.
	 */
	UPROPERTY(Transient)
	FName AutoChannelName;

	/** Cached actor data to apply/restore */
	UE_DEPRECATED(5.8, "ActorData has been deprecated. Material Container State is used instead.")
	UPROPERTY(NonPIEDuplicateTransient, meta=(DeprecatedProperty))
	TMap<TWeakObjectPtr<AActor>, FAvaMask2DActorData> ActorData_DEPRECATED;

	/** Reference to the last resolved canvas name, as stored in CanvasWeak */
	UPROPERTY(Transient, DuplicateTransient)
	FName LastResolvedCanvasName;

	/** Reference to the Canvas used */
	UPROPERTY(EditAnywhere, Transient, DuplicateTransient, Category = "Mask2D|Shared", NoClear,	meta = (DisplayName = "Canvas", AllowPrivateAccess = "true", ShowInnerProperties, NoResetToDefault))
	TWeakObjectPtr<UGeometryMaskCanvas> CanvasWeak;

	UE_DEPRECATED(5.8, "Material Collection Handles have been deprecated. Material Container State is used instead.")
	UPROPERTY()
	TMap<TWeakObjectPtr<UObject>, FInstancedStruct> MaterialCollectionHandleData_DEPRECATED;

	UE_DEPRECATED(5.8, "Material Handles have been deprecated. Material Pre State is used instead.")
	UPROPERTY(DuplicateTransient)
	TMap<TWeakObjectPtr<UMaterialInterface>, FInstancedStruct> MaterialHandleData_DEPRECATED;

	UE_DEPRECATED(5.8, "Material Collection Handles have been deprecated. Material Container State is used instead.")
	TMap<TObjectKey<UObject>, TSharedPtr<IAvaMaskMaterialCollectionHandle>> MaterialCollectionHandles;

	/** Last scanned primitive components */
	UE_DEPRECATED(5.8, "ScannedPrimitiveComponents has been deprecated. Material Container State is used instead.")
	TSet<TObjectKey<UPrimitiveComponent>> ScannedPrimitiveComponents;

private:
#if WITH_EDITOR
	/** Used for PECP */
	UE_API static const TAvaPropertyChangeDispatcher<UAvaMask2DBaseModifier> PropertyChangeDispatcher;
#endif
};

#undef UE_API
