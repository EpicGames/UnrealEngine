// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMaskTypes.h"
#include "MaterialBridge/Slot/AvaMaterialBridgeSlotId.h"
#include "UObject/WeakObjectPtr.h"
#include "AvaMask2DMaskState.generated.h"

class UMaterial;
class UMaterialFunctionInterface;
class UMaterialInstanceDynamic;

/** Represents the state for a mask to apply to / restore from a material */
USTRUCT()
struct FAvaMask2DMaskState
{
	GENERATED_BODY()

	FAvaMask2DMaskState() = default;

	/** Applies this state to the given material */
	void Apply(UMaterialInstanceDynamic* InMaterial) const;

	/** Stores the given material's mask state */
	void Store(UMaterialInstanceDynamic* InMaterial);

	/** The material parameters to save / restore */
	UPROPERTY()
	FAvaMask2DMaterialParameters MaterialParameters;

#if WITH_EDITORONLY_DATA
	/** The output processor was added to the material if material was from material designer */
	UPROPERTY()
	TObjectPtr<UMaterialFunctionInterface> OutputProcessor;
#endif
};

/** Struct identifying a slot by its slot id and material */
USTRUCT()
struct FAvaMask2DMaterialSlotId
{
	GENERATED_BODY()

	FAvaMask2DMaterialSlotId() = default;

	explicit FAvaMask2DMaterialSlotId(FString&& InMaterialContainerPath, const FAvaMaterialBridgeSlotId& InSlotId, const UMaterialInterface* InMaterial);

	bool operator==(const FAvaMask2DMaterialSlotId& InOther) const;

	/** Object containing the material slot */
	UPROPERTY()
	FString MaterialContainerPath;

	/** The slot id to save / restore the material parameters for */
	UPROPERTY()
	FAvaMaterialBridgeSlotId SlotId;

	/**
	 * Base material of the material whose parameter state to save / restore 
	 * This base is expected to be non-transient.
	 */
	UPROPERTY()
	TObjectPtr<const UMaterial> BaseMaterial;
};

/** Represents a material and its mask state */
USTRUCT()
struct FAvaMask2DMaterialMaskState
{
	GENERATED_BODY()

	FAvaMask2DMaterialMaskState() = default;

	explicit FAvaMask2DMaterialMaskState(const FAvaMask2DMaterialSlotId& InSlotId);

	bool operator==(const FAvaMask2DMaterialSlotId& InSlotId) const;

	/** Applies the mask state to the material */
	void Apply() const;

	/** Stores the material's mask state */
	void Store();

	/** Determines whether this material is dependent of the given material interface */
	bool IsDependentOf(UMaterialInterface* InMaterial) const;

	/** The slot id to save / restore the material parameters for */
	UPROPERTY()
	FAvaMask2DMaterialSlotId SlotMaterialId;

	/** The mask state of the material */
	UPROPERTY()
	FAvaMask2DMaskState MaskState;

	/** Material instance dynamic assigned for this state */
	TWeakObjectPtr<UMaterialInstanceDynamic> MaterialWeak;

	/** The topmost material containers that this state belongs to */
	TWeakObjectPtr<UObject> TopmostMaterialContainer;
};
