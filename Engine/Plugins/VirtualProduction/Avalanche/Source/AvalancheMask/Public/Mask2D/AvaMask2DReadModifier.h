// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMask2DBaseModifier.h"
#include "AvaMask2DMaskState.h"
#include "AvaPropertyChangeDispatcher.h"
#include "Modifiers/ActorModifierArrangeBaseModifier.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "AvaMask2DReadModifier.generated.h"

#define UE_API AVALANCHEMASK_API

class AActor;
class IAvaActorHandle;
class IAvaComponentHandle;
class IAvaMaskMaterialCollectionHandle;
class IAvaMaskMaterialHandle;
class IGeometryMaskReadInterface;
class IGeometryMaskWriteInterface;
class UActorComponent;
class UAvaMaskMaterialInstanceSubsystem;
class UAvaObjectHandleSubsystem;
class UCanvasRenderTarget2D;
class UGeometryMaskCanvas;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class UPrimitiveComponent;
class UTexture;
struct FAvaShapeParametricMaterial;
struct FMaterialParameterInfo;

namespace UE::Ava
{
	class FMaterialBridgeWriteSlot;
	class FMaterialBridgeWriteSlotContext;
}

/** Uses scene actors to create a mask texture and applies it to attached actors */
UCLASS(BlueprintType, MinimalAPI)
class UAvaMask2DReadModifier : public UAvaMask2DBaseModifier
{
	GENERATED_BODY()

	friend class UAvaMask2DReadModifierShared;

	/** Limit canvas count to 4 because a single linear color parameter is used to transmit the indices over. */
	static constexpr int32 MaxCanvasCount = 4;

public:
	//~ Begin UObject
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	float GetBaseOpacity() const
	{
		return BaseOpacity;
	}
	UE_API void SetBaseOpacity(float InBaseOpacity);

protected:
	//~ Begin IGeometryMaskClient
	UE_API virtual bool ForEachUsedCanvasName(TFunctionRef<bool(FName)> InFunc) const override;
	//~ End IGeometryMaskClient

	//~ Begin UActorModifierCoreBase
	UE_API virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	UE_API virtual void OnModifierEnabled(EActorModifierCoreEnableReason InReason) override;
	UE_API virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;
	UE_API virtual void SavePreState() override;
	UE_API virtual void RestorePreState() override;
	UE_API virtual void Apply() override;
	//~ End UActorModifierCoreBase

	//~ Begin UAvaMask2DBaseModifier
	UE_API virtual void OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent) override;
	UE_API virtual void RemoveFromActor(AActor* InActor) override;
	//~ End UAvaMask2DBaseModifier

	/** Get target blend mode dependent on modifier settings. */
	UE_API EBlendMode GetBlendMode() const;

	UE_API void OnBaseOpacityChanged();
	UE_API void OnAdditionalChannelsChanged();

	UE_DEPRECATED(5.8, "SetupMaskReadComponent is deprecated. Mask Read Modifier no longer allocates a component")
	UE_API virtual void SetupMaskComponent(UActorComponent* InComponent) override;

	UE_DEPRECATED(5.8, "SetupMaskReadComponent is deprecated. Mask Read Modifier no longer allocates a component")
	UE_API void SetupMaskReadComponent(IGeometryMaskReadInterface* InMaskReader);

	UE_API bool ApplyRead(TNotNull<UObject*> InMaterialContainer, TConstArrayView<TNotNull<const UGeometryMaskCanvas*>> InCanvases);

	bool ApplyMaskState(const UE::Ava::FMaterialBridgeWriteSlotContext& InContext, UE::Ava::FMaterialBridgeWriteSlot& InSlot, const FAvaMask2DMaskState& InMaskState);

	UE_DEPRECATED(5.8, "Use ApplyRead that takes in an array of canvas instead")
	UE_API bool ApplyRead(AActor* InActor, FAvaMask2DActorData& InActorData);

	UE_DEPRECATED(5.8, "Use ApplyRead that takes in an array of canvas instead")
	UE_API bool ApplyRead(AActor* InActor, FAvaMask2DActorData& InActorData, FText& OutFailReason);

	/** Stores the container and material states for a given actor */
	void StoreActorState(AActor* InActor);

	/** Additional channels to read from */
	UPROPERTY(EditInstanceOnly, Category = "Mask2D", meta = (AllowPrivateAccess = "true", DisplayAfter="Channel"))
	TArray<FName> AdditionalChannels;

	/** Base opacity/alpha to use in Read mode */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, Category = "Mask2D", meta = (ClampMin = 0.0, ClampMax = 1.0, AllowPrivateAccess = "true"))
	float BaseOpacity = 0.0f;

	/** The state of the actors to their material containers to save / restore */
	UPROPERTY(NonPIEDuplicateTransient)
	TMap<TObjectPtr<UObject>, TInstancedStruct<FAvaMaterialContainerState>> MaterialContainerStates;

	/** State of the materials to save/restore */
	UPROPERTY(NonPIEDuplicateTransient)
	TArray<FAvaMask2DMaterialMaskState> MaterialStates;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TAvaPropertyChangeDispatcher<UAvaMask2DReadModifier> PropertyChangeDispatcher;

	void OnMaterialCompiled(UMaterialInterface* InMaterial);
	void OnPIEEnded(bool bInIsSimulating);
#endif
};

#undef UE_API
