// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMask2DBaseModifier.h"
#include "AvaPropertyChangeDispatcher.h"
#include "GeometryMaskTypes.h"
#include "IGeometryMaskWriteInterface.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/WeakObjectPtr.h"

#include "AvaMask2DWriteModifier.generated.h"

#define UE_API AVALANCHEMASK_API

class AActor;
class IAvaActorHandle;
class IAvaComponentHandle;
class IAvaMaskMaterialCollectionHandle;
class IAvaMaskMaterialHandle;
class IGeometryMaskReadInterface;
class IGeometryMaskWriteInterface;
class UActorComponent;
class UAvaMaskMaterialInstanceSubsystem;
class UAvaObjectHandleSubsystem;
class UCanvasRenderTarget2D;
class UGeometryMaskCanvas;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class UPrimitiveComponent;
class UTexture;
struct FAvaShapeParametricMaterial;
struct FMaterialParameterInfo;

namespace UE::GeometryMask
{
	struct FMaskWriter;
}

/** Uses scene actors to create a mask texture and applies it to attached actors */
UCLASS(BlueprintType)
class UAvaMask2DWriteModifier : public UAvaMask2DBaseModifier, public IGeometryMaskWriteInterface
{
	GENERATED_BODY()

	friend class UAvaMask2DReadModifierShared;
	
public:
	UE_API UAvaMask2DWriteModifier();

	EGeometryMaskCompositeOperation GetWriteOperation() const
	{
		return WriteOperation;
	}
	UE_API void SetWriteOperation(const EGeometryMaskCompositeOperation InWriteOperation);

protected:
	//~ Begin UObject
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	UE_API virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	UE_API virtual void OnModifierEnabled(EActorModifierCoreEnableReason InReason) override;
	UE_API virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;
	UE_API virtual void Apply() override;
	//~ End UActorModifierCoreBase

	//~ Begin UAvaMask2DBaseModifier
	UE_API virtual void OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent) override;
	UE_API virtual void SetupMaskComponent(UActorComponent* InComponent) override;
	UE_API virtual void OnCanvasSet(TNotNull<UGeometryMaskCanvas*> InCanvas) override;
	UE_API virtual void OnCanvasReset() override;
	UE_API virtual void RemoveFromActor(AActor* InActor) override;
	//~ End UAvaMask2DBaseModifier

	//~ Begin IGeometryMaskWriteInterface
	UE_API virtual bool IsMaskWriterEnabled() const override;
	UE_API virtual const FGeometryMaskWriteParameters& GetParameters() const override;
	UE_API virtual void SetParameters(FGeometryMaskWriteParameters& InParameters) override;
	UE_API virtual void DrawToCanvas(FCanvas* InCanvas) override;
	UE_API virtual FOnGeometryMaskSetCanvasNativeDelegate& OnSetCanvas() override;
	UE_API virtual UE::GeometryMask::FMaskWriter* GetMaskWriter() override;
	//~ End IGeometryMaskWriteInterface

	UE_API void OnWriteOperationChanged();
	UE_API void SetupMaskWriteComponent(IGeometryMaskWriteInterface* InMaskWriter);

	UE_API bool ApplyWrite(TNotNull<AActor*> InActor);

	UE_DEPRECATED(5.8, "Actor data has been deprecated. Use the overload that takes in an actor only")
	UE_API bool ApplyWrite(AActor* InActor, FAvaMask2DActorData& InActorData);

	/** How to write to the chosen mask channel */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Getter, Setter, Category = "Mask2D", meta = (ValidEnumValues = "Add, Subtract", AllowPrivateAccess = "true"))
	EGeometryMaskCompositeOperation WriteOperation = EGeometryMaskCompositeOperation::Add;

	/** Actors to get the components to mask with */
	UPROPERTY(Transient, DuplicateTransient)
	TArray<TObjectPtr<AActor>> MaskActors;

	/** Object handling masking and component caching */
	TSharedPtr<UE::GeometryMask::FMaskWriter> MaskWriter;

	UPROPERTY(Transient, DuplicateTransient)
	FGeometryMaskWriteParameters Parameters;

	FOnGeometryMaskSetCanvasNativeDelegate OnSetCanvasDelegate;

#if WITH_EDITOR
	/** Used for PECP */
	UE_API static const TAvaPropertyChangeDispatcher<UAvaMask2DWriteModifier> PropertyChangeDispatcher;
#endif
};

#undef UE_API
