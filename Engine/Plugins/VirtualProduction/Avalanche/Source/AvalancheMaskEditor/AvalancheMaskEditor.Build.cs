// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheMaskEditor : ModuleRules
{
	public AvalancheMaskEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
            });

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ActorModifierCore",
				"Avalanche",
				"AvalancheEditorCore",
				"AvalancheMask",
				"AvalancheModifiers",
				"AvalancheShapes",
				"CoreUObject",
				"DynamicMaterialEditor",
				"EditorFramework",
				"EditorSubsystem",
				"Engine",
				"GeometryMask",
				"GeometryMaskEditor",
				"InputCore",
				"LevelEditor",
				"MediaPlateEditor",
				"Projects",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"TypedElementFramework",
				"TypedElementRuntime",
				"UnrealEd",
			});
    }
}
