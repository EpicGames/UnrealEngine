// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMaskEditorCommands.h"

#define LOCTEXT_NAMESPACE "AvalancheMaskEditor"

void FAvaMaskEditorCommands::RegisterCommands()
{
	UI_COMMAND(
		ShowVisualizeMasks,
		"ShowVisualizeMasks",
		"Visualize Masks",
		EUserInterfaceActionType::Button,
		FInputChord());
	
	UI_COMMAND(
		ToggleMaskMode,
		"ToggleMaskMode",
		"Toggle Mask Mode",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(
		ToggleShowAllMasks,
		"ToggleShowAllMasks",
		"Show all Mask objects",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	UI_COMMAND(
    	ToggleIsolateMask,
    	"Toggle Isolate Mask",
    	"Isolate selected mask",
    	EUserInterfaceActionType::ToggleButton,
    	FInputChord());
    		
	UI_COMMAND(
		ToggleEnableMask,
		"ToggleEnableMask",
		"Toggle masks on/off",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());
}

#undef LOCTEXT_NAMESPACE
