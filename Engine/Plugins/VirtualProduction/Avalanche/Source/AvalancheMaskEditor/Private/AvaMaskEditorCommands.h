// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMaskEditorStyle.h"
#include "Framework/Commands/Commands.h"

#define LOCTEXT_NAMESPACE "AvaMaskEditorCommands"

class FAvaMaskEditorCommands : public TCommands<FAvaMaskEditorCommands>
{
public:
	FAvaMaskEditorCommands()
		: TCommands<FAvaMaskEditorCommands>(
			TEXT("AvaMaskEditor"),
			LOCTEXT("MotionDesignMaskEditor", "Motion Design Masking"),
			NAME_None,
			FAvaMaskEditorStyle::Get().GetStyleSetName())
	{
	}

	virtual void RegisterCommands() override;

public:
	TSharedPtr<FUICommandInfo> ShowVisualizeMasks;
	TSharedPtr<FUICommandInfo> ToggleMaskMode;
	TSharedPtr<FUICommandInfo> ToggleShowAllMasks;
	TSharedPtr<FUICommandInfo> ToggleIsolateMask;
	TSharedPtr<FUICommandInfo> ToggleEnableMask;
};

#undef LOCTEXT_NAMESPACE
