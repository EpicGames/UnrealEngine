// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMaskEditorMode.h"

#include "AvaMaskEditorCommands.h"
#include "AvaMaskEditorLog.h"
#include "AvaMaskEditorStyle.h"
#include "AvaMaskEditorSubsystem.h"
#include "Editor.h"
#include "EditorModeManager.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/AvaGizmoComponent.h"
#include "GeometryMaskSubsystem.h"
#include "GeometryMaskWorldSubsystem.h"
#include "IAvaMaskEditor.h"
#include "IGeometryMaskWriteInterface.h"
#include "LevelEditor.h"
#include "Mask2D/AvaMask2DBaseModifier.h"
#include "Mask2D/AvaMask2DReadModifier.h"
#include "Mask2D/AvaMask2DWriteModifier.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modules/ModuleManager.h"
#include "Selection.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SGeometryMaskCanvasPreview.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AvalancheMaskEditorMode"

const FEditorModeID UAvaMaskEditorMode::EM_MotionDesignMaskEditorModeId(UE::AvaMaskEditor::MotionDesignMaskEditorModeName);

UAvaMaskEditorMode::UAvaMaskEditorMode()
{
	Info = FEditorModeInfo(UAvaMaskEditorMode::EM_MotionDesignMaskEditorModeId,
		LOCTEXT("MotionDesignMaskEditorModeName", "MotionDesign Mask Editor Mode"),
		FSlateIcon(),
		false);
}

void UAvaMaskEditorMode::Enter()
{
	UEdMode::Enter();

	if (GEditor && GEngine)
	{
		UTypedElementSelectionSet* SelectionSet = GetModeManager()->GetSelectedActors()->GetElementSelectionSet();
		SelectionSet->OnChanged().AddUObject(this, &UAvaMaskEditorMode::OnSelectionChanged);
		
		WeakLastSelectedActor = SelectionSet->GetTopSelectedObject<AActor>();
		WeakActorSelectionSet = SelectionSet;
		
		OnActorSpawnedHandle = GetWorld()->AddOnActorSpawnedHandler(FOnActorSpawned::FDelegate::CreateUObject(this, &UAvaMaskEditorMode::OnActorSpawned));

		GEngine->OnLevelActorAdded().AddUObject(this, &UAvaMaskEditorMode::OnActorSpawned);
	}

	for (AActor* Actor : TActorRange<AActor>(GetWorld()))
	{
		if (Actor->FindComponentByInterface(UGeometryMaskWriteInterface::StaticClass()))
		{
			WeakMaskWriterActors.Add(Actor);
			if (UAvaGizmoComponent* GizmoComponent = Actor->GetComponentByClass<UAvaGizmoComponent>())
			{
				GizmoComponent->SetVisibleInEditor(true);
			}
		}
	}

	const FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	if (TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor())
	{
		FText DisplayText = LOCTEXT("MaskViewMode", "Mask View");
    
    	TSharedPtr<SWidget> ToolWidget = nullptr;
    	{
    		static FName ToolkitOverlayMenuName = UE::AvaMaskEditor::Internal::ToolkitOverlayMenuName;
    		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu(ToolkitOverlayMenuName);
    		Menu->SetStyleSet(&FAvaMaskEditorStyle::Get());
    		Menu->StyleName = TEXT("AvaMaskEditor.ViewportOverlayToolbar");
    		{
    			FToolMenuSection& Section = Menu->FindOrAddSection(TEXT("Default"));
    			{
    				FToolMenuEntry& Entry = Section.AddEntry(FToolMenuEntry::InitToolBarButton(FAvaMaskEditorCommands::Get().ToggleShowAllMasks));
    				Entry.Icon.Set(FSlateIcon(FAvaMaskEditorStyle::Get().GetStyleSetName(), TEXT("AvaMaskEditor.ToggleShowAllMasks")));
    			}
    
    			{
    				FToolMenuEntry& Entry = Section.AddEntry(FToolMenuEntry::InitToolBarButton(FAvaMaskEditorCommands::Get().ToggleEnableMask));
    				Entry.Icon.Set(FSlateIcon(FAvaMaskEditorStyle::Get().GetStyleSetName(), TEXT("AvaMaskEditor.ToggleDisableMask")));
    			}
    
    			{
    				FToolMenuEntry& Entry = Section.AddEntry(FToolMenuEntry::InitToolBarButton(FAvaMaskEditorCommands::Get().ToggleIsolateMask));
    				Entry.Icon.Set(FSlateIcon(FAvaMaskEditorStyle::Get().GetStyleSetName(), TEXT("AvaMaskEditor.ToggleIsolateMask")));
    			}
    		}
    
    		ToolWidget = UToolMenus::Get()->GenerateWidget(Menu);	
    	}
    	
		// ViewportOverlay
		SAssignNew(ViewportOverlayWidget, SOverlay)
			+SOverlay::Slot(-2)
			[
				SAssignNew(CanvasPreviewWidget, SGeometryMaskCanvasPreview)
				.Visibility(EVisibility::HitTestInvisible)
				.Opacity(0.5f)
			]
	
			+SOverlay::Slot()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Bottom)
				.Padding(FMargin(0.0f, 0.0f, 0.f, 15.f))
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush(TEXT("EditorViewport.OverlayBrush")))
					.Padding(8.f)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
						[
							SNew(SImage)
							.Image(FAvaMaskEditorStyle::Get().GetBrush(TEXT("AvaMaskEditor.ToggleMaskMode")))
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
						[
							SNew(STextBlock)
							.Text(DisplayText)
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
						[
							SNew(SButton)
							.ButtonStyle(FAppStyle::Get(), TEXT("PrimaryButton"))
							.TextStyle(FAppStyle::Get(), TEXT("DialogButtonText"))
							.Text(LOCTEXT("ExitEdit", "Exit"))
							.ToolTipText(LOCTEXT("ExitTooltip", "Exit Mask Mode"))
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							.OnClicked_Lambda([EditorModeManager = GetModeManager()]() 
							{
								EditorModeManager->DeactivateMode(UAvaMaskEditorMode::EM_MotionDesignMaskEditorModeId);						
								return FReply::Handled(); 
							})
						]
					]
				]
			];

		const TSharedRef<SWidget> ViewportOverlayWidgetRef = ViewportOverlayWidget.ToSharedRef();
		ViewportOverlayWidgetRef->SetTag(TEXT("AvaMaskOverlayWidget"));

		LevelEditor->AddViewportOverlayWidget(ViewportOverlayWidgetRef);

		if (const TSharedPtr<SOverlay> ParentOverlay = StaticCastSharedPtr<SOverlay>(ViewportOverlayWidget->GetParentWidget());
			ParentOverlay.IsValid())
		{
			// Now that we have the Overlay widget, remove the slot
			ParentOverlay->RemoveSlot(ViewportOverlayWidgetRef);

			// Then re-add it, specifying zorder
			ParentOverlay->AddSlot(-2)
			[
				ViewportOverlayWidgetRef
			];
		}

		// Initialize, re-using the callback
		OnSelectionChanged(WeakActorSelectionSet.Get());
	}
}

void UAvaMaskEditorMode::Exit()
{
	if (GEditor && GEngine)
	{
		for (TWeakObjectPtr<AActor>& WeakActor : WeakMaskWriterActors)
		{
			if (const AActor* Actor = WeakActor.Get())
			{
				if (UAvaGizmoComponent* GizmoComponent = Actor->GetComponentByClass<UAvaGizmoComponent>())
				{
					GizmoComponent->SetVisibleInEditor(false);
				}
			}
		}

		// Remove selection events
		if (UTypedElementSelectionSet* SelectionSet = WeakActorSelectionSet.Get())
		{
			SelectionSet->OnChanged().RemoveAll(this);
		}

		if (ViewportOverlayWidget.IsValid())
		{
			const FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
			if (TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor())
			{
				LevelEditor->RemoveViewportOverlayWidget(ViewportOverlayWidget.ToSharedRef());
			}
		}
		
		WeakLastSelectedActor = nullptr;
		WeakMaskWriterActors.Reset();
		WeakActorSelectionSet.Reset();

		GetWorld()->RemoveOnActorSpawnedHandler(OnActorSpawnedHandle);

		GEngine->OnLevelActorAdded().RemoveAll(this);
	}

	UEdMode::Exit();
}

bool UAvaMaskEditorMode::UsesToolkits() const
{
	return false;
}

void UAvaMaskEditorMode::CreateToolkit()
{
	// Toolkit = MakeShared<FAvalancheMaskEditorModeToolkit>();
}

void UAvaMaskEditorMode::ModeTick(float DeltaTime)
{
	Super::ModeTick(DeltaTime);
}

bool UAvaMaskEditorMode::IsCompatibleWith(FEditorModeID OtherModeID) const
{
	return true;
}

AActor* UAvaMaskEditorMode::GetActorToParentTo() const
{
	if (AActor* LastSelectedActor = WeakLastSelectedActor.Get())
	{
		return LastSelectedActor;
	}

	USelection* SelectedActors = GetModeManager()->GetSelectedActors();
	if (SelectedActors->Num() > 0)
	{
		return SelectedActors->GetTop<AActor>();
	}

	return nullptr;
}

void UAvaMaskEditorMode::OnSelectionChanged(const UTypedElementSelectionSet* InSelectionSet)
{
	if (InSelectionSet)
	{
		PreviewCanvasId.ResetToNone();

		if (const UTypedElementSelectionSet* SelectionSet = WeakActorSelectionSet.Get())
		{
			TArray<UObject*> AllSelected = SelectionSet->GetSelectedObjects();
			WeakLastSelectedActor = SelectionSet->GetTopSelectedObject<AActor>();
			if (const AActor* FirstActor = SelectionSet->GetTopSelectedObject<AActor>())
			{
				SelectedMaskModifier = UAvaMask2DBaseModifier::FindMaskModifierOnActor(FirstActor);
				if (SelectedMaskModifier.IsValid())
				{
					UGeometryMaskCanvas* const MaskCanvas = GetCanvasReferencedByActor(FirstActor);
					SelectedMaskCanvas = MaskCanvas;
					if (MaskCanvas)
					{
						for (const TWeakObjectPtr<AActor>& WeakWriterActor : WeakMaskWriterActors)
						{
							if (const AActor* WriterActor = WeakWriterActor.Get())
							{
								if (const IGeometryMaskWriteInterface* WriterObject = WriterActor->FindComponentByInterface<IGeometryMaskWriteInterface>())
								{
									if (UAvaGizmoComponent* GizmoComponent = WriterActor->GetComponentByClass<UAvaGizmoComponent>())
									{
										FName WriterCanvasName = WriterObject->GetParameters().CanvasName;
										GizmoComponent->SetsStencil(WriterCanvasName == MaskCanvas->GetCanvasName());
										GizmoComponent->SetStencilId(WriterCanvasName != MaskCanvas->GetCanvasName() ? 0 : 150);
									}
								}
							}
						}
						PreviewCanvasId = MaskCanvas->GetCanvasId();
					}
				}
			}
			else
			{
				// Otherwise the selection has changed and the previous MaskModifier is no longer selected
				SelectedMaskModifier = nullptr;
				SelectedMaskCanvas = nullptr;
			}
		}

		UpdatePreviewWidget();
	}
}

UGeometryMaskCanvas* UAvaMaskEditorMode::GetCanvasReferencedByActor(const AActor* InActor)
{
	FName CanvasName = NAME_None;
	if (const IGeometryMaskWriteInterface* WriteComponent = InActor->FindComponentByInterface<IGeometryMaskWriteInterface>())
	{
		const FGeometryMaskWriteParameters WriteComponentParameters = WriteComponent->GetParameters();
		CanvasName = WriteComponentParameters.CanvasName;
	}
	else if (const IGeometryMaskReadInterface* ReadComponent = InActor->FindComponentByInterface<IGeometryMaskReadInterface>())
	{
		const FGeometryMaskReadParameters ReadComponentParameters = ReadComponent->GetParameters();
		CanvasName = ReadComponentParameters.CanvasName;
	}

	if (UGeometryMaskCanvas* Canvas = GetWorld()->GetSubsystem<UGeometryMaskWorldSubsystem>()->GetNamedCanvas(InActor->GetLevel(), CanvasName))
	{
		return Canvas;
	}

	return nullptr;
}

void UAvaMaskEditorMode::UpdatePreviewWidget()
{
	if (CanvasPreviewWidget.IsValid())
	{
		CanvasPreviewWidget->SetCanvasId(PreviewCanvasId);
	}
}

void UAvaMaskEditorMode::OnActorSpawned(AActor* InActor)
{
	if (CanMaskSelected(WeakLastSelectedActor.Get()))
	{
		if (AddMaskToSelected({InActor}))
		{
			UpdatePreviewWidget();
		}
	}
}

bool UAvaMaskEditorMode::AddMaskToSelected(const TArray<AActor*>& InMaskingActors)
{
	// @note: This assumes CanMaskSelected passed
	
	AActor* ParentActor = GetActorToParentTo();
	
	UAvaMask2DReadModifier* ParentMaskReadModifier = FindOrAddMaskModifier<UAvaMask2DReadModifier>(ParentActor);
	if (!ParentMaskReadModifier)
	{
		return false;
	}

	const UGeometryMaskCanvas* ParentCanvas = GetCanvasReferencedByActor(ParentActor);
	const FName ChannelName = ParentMaskReadModifier->GetChannel();

	for (AActor* PlacedActor : InMaskingActors)
	{
		// Parent to Actor
		PlacedActor->AttachToActor(ParentActor, FAttachmentTransformRules::KeepWorldTransform);
		
		// Setup modifier
		if (UAvaMask2DWriteModifier* MaskWriteModifier = FindOrAddMaskModifier<UAvaMask2DWriteModifier>(PlacedActor))
		{
			MaskWriteModifier->SetChannel(ChannelName);
			MaskWriteModifier->SetUseParentChannel(true);

			// Flush unused canvases, in case temporary actors were created/destroyed
			if (UGeometryMaskWorldSubsystem* Subsystem = GetWorld()->GetSubsystem<UGeometryMaskWorldSubsystem>())
			{
				Subsystem->RemoveWithoutWriters();
			}

			PreviewCanvasId = FGeometryMaskCanvasId(ParentActor->GetLevel(), ChannelName);
		}
	}
		
	return true;
}

UAvaMask2DBaseModifier* UAvaMaskEditorMode::FindOrAddMaskModifier(AActor* InActor, const TSubclassOf<UAvaMask2DBaseModifier>& InMaskModifierType)
{
	if (const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get())
	{
		UActorModifierCoreStack* ModifierStack = ModifierSubsystem->GetActorModifierStack(InActor);
		if (!ModifierStack)
		{
			ModifierStack = ModifierSubsystem->AddActorModifierStack(InActor);
		}

		const FName MaskModifierName = GetDefault<UActorModifierCoreBase>(InMaskModifierType)->GetModifierName();

		TArray<UAvaMask2DBaseModifier*> FoundModifiers;
		if (UActorModifierCoreBase* FoundModifier = ModifierStack->FindModifier(InMaskModifierType))
		{
			return Cast<UAvaMask2DBaseModifier>(FoundModifier);
		}
		
		FActorModifierCoreStackInsertOp InsertOp;
		InsertOp.NewModifierName = MaskModifierName;
			
		UAvaMask2DBaseModifier* MaskModifier = Cast<UAvaMask2DBaseModifier>(ModifierSubsystem->InsertModifier(ModifierStack, InsertOp));
		if (!MaskModifier)
		{
			if (InsertOp.FailReason)
			{
				UE_LOGF(LogAvaMaskEditor, Error, "Error inserting Mask modifier: %ls", *InsertOp.FailReason->ToString());
			}
			else
			{
				UE_LOGF(LogAvaMaskEditor, Error, "Error inserting Mask modifier.");
			}
		}
		else
		{
			MaskModifier->SetChannel(MaskModifier->GenerateUniqueMaskName());
		}

		return MaskModifier;
	}

	return nullptr;
}

bool UAvaMaskEditorMode::CanMaskSelected(AActor* InSelectedActor)
{
	auto CanMaskActor = [](AActor* InActor)
	{
		if (const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get())
		{
			const FName MaskReadModifierName = ModifierSubsystem->GetRegisteredModifierName(UAvaMask2DReadModifier::StaticClass());

			if (const UActorModifierCoreStack* ExistingModifierStack = ModifierSubsystem->GetActorModifierStack(InActor))
			{
				// Can't double-mask, and can't apply a mask to this object if it's itself being masked
				if (ExistingModifierStack->ContainsModifier(UAvaMask2DReadModifier::StaticClass())
					|| ExistingModifierStack->ContainsModifier(UAvaMask2DWriteModifier::StaticClass()))
				{
					return true;
				}
			}

			const TSet<FName> AllowedModifiers = ModifierSubsystem->GetAllowedModifiers(InActor);
			if (AllowedModifiers.Contains(MaskReadModifierName))
			{
				return true;
			}
		}

		return false;
	};

	// Check provided Actor if specified
	if (InSelectedActor)
	{
		return CanMaskActor(InSelectedActor);
	}

	// Otherwise get from USelection
	USelection* SelectedActors = GetModeManager()->GetSelectedActors();

	// Can only add to a single Actor
	if (SelectedActors->Num() != 1)
	{
		return false;
	}

	AActor* SelectedActor = SelectedActors->GetTop<AActor>();
	return CanMaskActor(SelectedActor);
}

#undef LOCTEXT_NAMESPACE
