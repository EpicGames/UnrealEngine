// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/UEdMode.h"
#include "Framework/Commands/UICommandList.h"
#include "GeometryMaskTypes.h"

#include "AvaMaskEditorMode.generated.h"

class UGeometryMaskCanvas;
class SGeometryMaskCanvasPreview;
class UAvaMask2DBaseModifier;
class UTypedElementSelectionSet;

UCLASS(MinimalAPI)
class UAvaMaskEditorMode : public UEdMode
{
	GENERATED_BODY()
	
public:
	static const FEditorModeID EM_MotionDesignMaskEditorModeId;

	UAvaMaskEditorMode();
	virtual ~UAvaMaskEditorMode() override = default;

	// Begin UEdMode
	virtual void Enter() override;
	virtual void Exit() override;
	virtual bool UsesToolkits() const override;
	virtual void CreateToolkit() override;
	virtual void ModeTick(float DeltaTime) override;
	
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override;

	/** Only accept saving on current asset */
	virtual bool IsOperationSupportedForCurrentAsset(EAssetOperation InOperation) const { return InOperation == EAssetOperation::Save; }
	// End UEdMode

private:
	void OnSelectionChanged(const UTypedElementSelectionSet* InSelectionSet);
	UGeometryMaskCanvas* GetCanvasReferencedByActor(const AActor* InActor);
	void UpdatePreviewWidget();

private:
	FDelegateHandle OnPreActorSpawnedHandle;
	FDelegateHandle OnActorSpawnedHandle;

private:
	TWeakObjectPtr<AActor> WeakLastSelectedActor = nullptr;
	TWeakObjectPtr<UTypedElementSelectionSet> WeakActorSelectionSet;
	TArray<TWeakObjectPtr<AActor>> WeakMaskWriterActors;

	TSharedPtr<SWidget> ViewportOverlayWidget;
	TSharedPtr<SGeometryMaskCanvasPreview> CanvasPreviewWidget;

	TWeakObjectPtr<UAvaMask2DBaseModifier> SelectedMaskModifier;
	TWeakObjectPtr<UGeometryMaskCanvas> SelectedMaskCanvas;

	// @note: these can differ from the selection when interacting
	FGeometryMaskCanvasId PreviewCanvasId;

	AActor* GetActorToParentTo() const;

	void OnActorSpawned(AActor* InActor);

	bool AddMaskToSelected(const TArray<AActor*>& InMaskingActors);

	UAvaMask2DBaseModifier* FindOrAddMaskModifier(AActor* InActor, const TSubclassOf<UAvaMask2DBaseModifier>& InMaskModifierType);

	template <typename MaskModifierType
		UE_REQUIRES(std::is_base_of_v<UAvaMask2DBaseModifier, MaskModifierType>)>
	MaskModifierType* FindOrAddMaskModifier(AActor* InActor)
	{
		return Cast<MaskModifierType>(FindOrAddMaskModifier(InActor, MaskModifierType::StaticClass()));
	}

	/** Uses USelection if SelectedActor not specified. */
	bool CanMaskSelected(AActor* InSelectedActor = nullptr);
};
