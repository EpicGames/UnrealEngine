// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMaskEditorModule.h"

#include "AvaMaskEditorCommands.h"
#include "AvaMaskEditorMode.h"
#include "AvaMaskEditorSubsystem.h"
#include "Details/AvaMask2DModifierDetails.h"
#include "EditorModeManager.h"
#include "Mask2D/AvaMask2DReadModifier.h"
#include "Mask2D/AvaMask2DWriteModifier.h"
#include "MediaPlateEditorModule.h"
#include "PropertyEditorModule.h"
#include "Templates/SharedPointer.h"
#include "ToolMenuEntry.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SAvaMultiComboButton.h"

#define LOCTEXT_NAMESPACE "AvalancheMaskEditor"

void FAvalancheMaskEditorModule::StartupModule()
{
	// Details
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FOnGetDetailCustomizationInstance MaskCustomization = FOnGetDetailCustomizationInstance::CreateStatic(&FAvaMask2DModifierDetails::MakeInstance);
		PropertyModule.RegisterCustomClassLayout(UAvaMask2DReadModifier::StaticClass()->GetFName(), MaskCustomization);
		PropertyModule.RegisterCustomClassLayout(UAvaMask2DWriteModifier::StaticClass()->GetFName(), MaskCustomization);
		
		PropertyModule.NotifyCustomizationModuleChanged();
	}

	if (FMediaPlateEditorModule* EditorModule = FModuleManager::LoadModulePtr<FMediaPlateEditorModule>("MediaPlateEditor"))
	{
		EditorModule->OnGetMediaPlateMaterialAssetPaths().AddRaw(this, &FAvalancheMaskEditorModule::OnGetMediaPlateMaterialAssetPaths);
	}

	FAvaMaskEditorCommands::Register();

	CommandList = MakeShared<FUICommandList>();

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FAvalancheMaskEditorModule::RegisterMenus));
}

void FAvalancheMaskEditorModule::ShutdownModule()
{
	// Details
	if (FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
	{
		if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
		{
			PropertyModule->UnregisterCustomClassLayout(UAvaMask2DReadModifier::StaticClass()->GetFName());
			PropertyModule->UnregisterCustomClassLayout(UAvaMask2DWriteModifier::StaticClass()->GetFName());
			PropertyModule->NotifyCustomizationModuleChanged();
		}
	}

	if (FMediaPlateEditorModule* EditorModule = FModuleManager::GetModulePtr<FMediaPlateEditorModule>("MediaPlateEditor"))
	{
		EditorModule->OnGetMediaPlateMaterialAssetPaths().RemoveAll(this);
	}
	
	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);

	FAvaMaskEditorCommands::Unregister();
}

TSharedPtr<FUICommandList> FAvalancheMaskEditorModule::GetCommandList() const
{
	return CommandList;
}

void FAvalancheMaskEditorModule::ToggleEditorMode()
{
	GLevelEditorModeTools().ActivateMode(UAvaMaskEditorMode::EM_MotionDesignMaskEditorModeId, true);
}

TSharedRef<SWidget> FAvalancheMaskEditorModule::GetStatusBarWidgetMenuContent()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	
	static const FName MenuName = TEXT("AvaMaskEditor.StatusBar");
	UToolMenu* ContextMenu = ToolMenus->FindMenu(MenuName);
	if (!ContextMenu)
	{
		ContextMenu = ToolMenus->RegisterMenu(MenuName, NAME_None, EMultiBoxType::Menu);
		FToolMenuSection& Section = ContextMenu->AddSection(TEXT("Mask"));
		Section.AddEntry(FToolMenuEntry::InitMenuEntryWithCommandList(
			FAvaMaskEditorCommands::Get().ShowVisualizeMasks,
			GetCommandList(),
			LOCTEXT("ShowVisualizeMasks", "Visualize Masks")));
	}

	if (!ContextMenu)
	{
		return SNullWidget::NullWidget;
	}

	return ToolMenus->GenerateWidget(ContextMenu);
}

FReply FAvalancheMaskEditorModule::OnToggleMaskModeClicked()
{
	check(CommandList.IsValid());
	
	return GetCommandList()->ExecuteAction(FAvaMaskEditorCommands::Get().ToggleMaskMode.ToSharedRef())
		? FReply::Handled()
		: FReply::Unhandled();
}

void FAvalancheMaskEditorModule::OnGetMediaPlateMaterialAssetPaths(TArray<FName>& OutAssetPaths)
{
	// Add package path(s) to media plate material variations for mask modifier. 
	OutAssetPaths.AddUnique(FName("/Avalanche/MaskResources/MediaPlate"));
}

void FAvalancheMaskEditorModule::RegisterMenus()
{
	// Owner will be used for cleanup in call to UToolMenus::UnregisterOwner
	FToolMenuOwnerScoped OwnerScoped(this);

	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Window");
		{
			FToolMenuSection& Section = Menu->FindOrAddSection("WindowLayout");
			Section.AddMenuEntryWithCommandList(FAvaMaskEditorCommands::Get().ToggleMaskMode, CommandList);
		}
	}

	// Bottom center viewport overlay when mode is active
	{
		static FName ToolkitOverlayMenuName = UE::AvaMaskEditor::Internal::ToolkitOverlayMenuName;

		FToolMenuContext MenuContext(CommandList);
		
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(ToolkitOverlayMenuName, NAME_None, EMultiBoxType::ToolBar, false);
		Menu->Context = MenuContext;
	}

	{
		UToolMenu* ToolbarMenu = UToolMenus::Get()->ExtendMenu(TEXT("AvalancheLevelViewport.StatusBar"));
		{
			FToolMenuSection& Section = ToolbarMenu->FindOrAddSection("ModeToggles");
			{
				static const FButtonStyle* Style = &FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton");
				static const FMargin Margin = FMargin(1.0f, 2.0f, 1.0f, 2.0f);
				static const FVector2D ImageSize = FVector2D(16.0f, 16.0f);

				TSharedRef<SWidget> MaskStatusBarWidget =
					SNew(SAvaMultiComboButton)
						.ButtonStyle(Style)
						.ContentPadding(Margin)
						.ToolTipText(FAvaMaskEditorCommands::Get().ToggleMaskMode->GetDescription())
						.HasDownArrow(false)
						.OnGetMenuContent(FOnGetContent::CreateRaw(this, &FAvalancheMaskEditorModule::GetStatusBarWidgetMenuContent))
						.OnButtonClicked(FOnClicked::CreateRaw(this, &FAvalancheMaskEditorModule::OnToggleMaskModeClicked))
						.ButtonContent()
						[
							SNew(SOverlay)
							+ SOverlay::Slot()
							.Padding(0.f, 0.f, ImageSize.X - 1.f, 0.f)
							[
								SNew(SImage)
								.Image(FSlateIcon(FAvaMaskEditorStyle::Get().GetStyleSetName(), TEXT("AvaMaskEditor.ToggleMaskMode.Small")).GetIcon())
								.DesiredSizeOverride(ImageSize)
							]
							+ SOverlay::Slot()
							.Padding(ImageSize.X - 1.f, 0.f, 0.f, 0.f)
							[
								SNew(SImage)
								.Image(FAppStyle::Get().GetBrush("Icons.ChevronDown"))
								.DesiredSizeOverride(ImageSize)
								.ColorAndOpacity(FSlateColor(FLinearColor(0.5f, 0.5f, 0.5f, 1.f)))
							]
						];

				Section.AddEntry(FToolMenuEntry::InitWidget(TEXT("AvaMaskEditor.StatusBar.Toggle"), MaskStatusBarWidget, FText::GetEmpty(), true));
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FAvalancheMaskEditorModule, AvalancheMaskEditor)
