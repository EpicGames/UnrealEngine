// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#include "Framework/Commands/UICommandList.h"
#include "Input/Reply.h"
#include "Templates/SharedPointerFwd.h"

class SWidget;

class FAvalancheMaskEditorModule
	: public IModuleInterface
{
public:	
    // ~Begin IModuleInterface
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;
    // ~End IModuleInterface

    TSharedPtr<FUICommandList> GetCommandList() const;
	
private:
	void RegisterMenus();	
	void ToggleEditorMode();

	TSharedRef<SWidget> GetStatusBarWidgetMenuContent();
	FReply OnToggleMaskModeClicked();
	void OnGetMediaPlateMaterialAssetPaths(TArray<FName>& OutAssetPaths);

private:
	TSharedPtr<FUICommandList> CommandList;
};
