// Copyright Epic Games, Inc. All Rights Reserved. 

#include "AvaMaskEditorSVE.h"

#include "AvaMaskEditorLog.h"
#include "AvaMaskEditorMode.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorModeManager.h"
#include "Engine/Engine.h"
#include "Engine/RendererSettings.h"
#include "Engine/Scene.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "SceneView.h"

namespace UE::AvaMask::Private
{
	static const constexpr TCHAR* SceneViewPostProcessMaterialPath = TEXT("/Script/Engine.Material'/Avalanche/MaskResources/PP_MaskOverlay.PP_MaskOverlay'");
}

FAvaMaskSceneViewExtension::FAvaMaskSceneViewExtension(
	const FAutoRegister& AutoRegister
	, UWorld* InWorld)
	: FWorldSceneViewExtension(AutoRegister, InWorld)
{
	PostProcessMaterial = FString(UE::AvaMask::Private::SceneViewPostProcessMaterialPath);
}

void FAvaMaskSceneViewExtension::SetupView(
	FSceneViewFamily& InViewFamily
	, FSceneView& InView)
{
	// Don't apply without this setting
	if (GetDefault<URendererSettings>()->CustomDepthStencil != ECustomDepthStencil::EnabledWithStencil)
	{
		UE_LOGF(LogAvaMaskEditor, Warning, "The Mask overlay requires CustomDepth to be \"EnabledWithStencil\" in Project Settings.")
	}
	else
	{
		FPostProcessSettings PostProcessSettings = InView.FinalPostProcessSettings;
		if (UMaterialInterface* BlendableMaterial = PostProcessMaterial.LoadSynchronous())
		{
			PostProcessMaterialMID = UMaterialInstanceDynamic::Create(BlendableMaterial, GetTransientPackage());
			PostProcessSettings.AddBlendable(PostProcessMaterialMID, 1.0f);
		}
		InView.OverridePostProcessSettings(PostProcessSettings, 1.0f);
	}
}

void FAvaMaskSceneViewExtension::BeginRenderViewFamily(FSceneViewFamily& InViewFamily)
{
}

bool FAvaMaskSceneViewExtension::IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const
{
	bool bIsActive = FWorldSceneViewExtension::IsActiveThisFrame_Internal(Context);
	if (!bIsActive)
	{
		return false;
	}

	if (GEditor)
	{
		if (GEditor->IsSimulatingInEditor())
		{
			bIsActive = GetWorld()->WorldType == EWorldType::Editor;
		}
		else if (GEditor->PlayWorld)
		{
			bIsActive = GetWorld()->WorldType == EWorldType::PIE;
		}
		else
		{
			bIsActive = GetWorld()->WorldType == EWorldType::Editor;	
		}
	}

	if (bIsActive)
	{
		// @todo: use events instead, this is expensive to check every tick
		bIsActive = GLevelEditorModeTools().IsModeActive(UAvaMaskEditorMode::EM_MotionDesignMaskEditorModeId);
	}

	return bIsActive;
}

void FAvaMaskSceneViewExtension::AddReferencedObjects(FReferenceCollector& InCollector)
{
	InCollector.AddReferencedObject(PostProcessMaterialMID);
}

FString FAvaMaskSceneViewExtension::GetReferencerName() const
{
	return TEXT("FAvaMaskSceneViewExtension");
}
