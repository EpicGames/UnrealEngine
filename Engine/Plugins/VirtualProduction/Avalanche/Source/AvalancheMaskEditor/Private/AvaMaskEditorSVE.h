// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneViewExtension.h"

class UGeometryMaskCanvas;

class FAvaMaskSceneViewExtension
	: public FWorldSceneViewExtension
	, public FGCObject
{
public:
	FAvaMaskSceneViewExtension(const FAutoRegister& AutoRegister, UWorld* InWorld);

	// ~Begin ISceneViewExtension
	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override;
	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;
	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;
	// ~End ISceneViewExtension

	// ~Begin FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	virtual FString GetReferencerName() const override;
	// ~End FGCObject Interface

private:
	TSoftObjectPtr<UMaterialInterface> PostProcessMaterial;
	TObjectPtr<UMaterialInstanceDynamic> PostProcessMaterialMID;
};

