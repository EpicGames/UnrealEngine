// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMaskEditorStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Math/MathFwd.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StyleColors.h"
#include "Styling/ToolBarStyle.h"

FAvaMaskEditorStyle::FAvaMaskEditorStyle()
	: FSlateStyleSet(TEXT("AvaMaskEditor"))
{
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	SetContentRoot(Plugin->GetBaseDir() / TEXT("Resources"));

	Set("AvaMaskEditor.ToggleMaskMode.Small", new IMAGE_BRUSH_SVG(TEXT("Icons/MaskIcons/Mode_On"), Icon16x16));
	Set("AvaMaskEditor.ToggleMaskMode", new IMAGE_BRUSH_SVG(TEXT("Icons/MaskIcons/Mode_On"), Icon20x20));
	Set("AvaMaskEditor.ToggleShowAllMasks", new IMAGE_BRUSH_SVG(TEXT("Icons/MaskIcons/Disable"), Icon20x20));
	Set("AvaMaskEditor.ToggleDisableMask", new IMAGE_BRUSH_SVG(TEXT("Icons/MaskIcons/Disable"), Icon20x20));
	Set("AvaMaskEditor.ToggleIsolateMask", new IMAGE_BRUSH_SVG(TEXT("Icons/MaskIcons/Disable"), Icon20x20));

	{
		FToolBarStyle ViewportOverlayToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>(TEXT("AssetEditorToolbar"));

		ViewportOverlayToolbarStyle.SetButtonPadding(       FMargin(0.0f, 0.0f));
		ViewportOverlayToolbarStyle.SetCheckBoxPadding(     FMargin(0.0f, 0.0f));
		ViewportOverlayToolbarStyle.SetComboButtonPadding(  FMargin(0.0f, 0.0f));
		ViewportOverlayToolbarStyle.SetIndentedBlockPadding(FMargin(0.0f, 0.0f));
		ViewportOverlayToolbarStyle.SetBlockPadding(        FMargin(0.0f, 0.0f));
		ViewportOverlayToolbarStyle.SetSeparatorPadding(    FMargin(0.0f, 0.0f));
		ViewportOverlayToolbarStyle.bShowLabels = false;
		ViewportOverlayToolbarStyle.SetBackground(FSlateColorBrush(FStyleColors::Transparent));
		ViewportOverlayToolbarStyle.SetBackgroundPadding(0);
		ViewportOverlayToolbarStyle.SetButtonPadding(0);
		ViewportOverlayToolbarStyle.SetCheckBoxPadding(0);
		
		Set("AvaMaskEditor.ViewportOverlayToolbar", ViewportOverlayToolbarStyle);
	}
	
	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FAvaMaskEditorStyle::~FAvaMaskEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
