// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FAvaMaskEditorStyle final : public FSlateStyleSet
{
public:
	static FAvaMaskEditorStyle& Get()
	{
		static FAvaMaskEditorStyle Instance;
		return Instance;
	}

	FAvaMaskEditorStyle();
	virtual ~FAvaMaskEditorStyle() override;
};
