// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMaskEditorSubsystem.h"

#include "AvaMaskEditorCommands.h"
#include "AvaMaskEditorMode.h"
#include "AvaMaskEditorModule.h"
#include "AvaMaskEditorSVE.h"
#include "EditorModeManager.h"
#include "Modules/ModuleManager.h"
#include "SceneViewExtension.h"

class FGeometryMaskSceneViewExtension;

void UAvaMaskEditorWorldSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	UWorld* World = GetWorld();

    SceneViewExtension = FSceneViewExtensions::NewExtension<FAvaMaskSceneViewExtension>(World);
}

void UAvaMaskEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	/* Commands for this subsystem */
	const TSharedPtr<FUICommandList> CommandList = FModuleManager::Get().LoadModuleChecked<FAvalancheMaskEditorModule>(UE_MODULE_NAME).GetCommandList();

	CommandList->MapAction(
		FAvaMaskEditorCommands::Get().ShowVisualizeMasks,
		FExecuteAction::CreateUObject(this, &UAvaMaskEditorSubsystem::ShowVisualizeMasks),
		FCanExecuteAction());

	CommandList->MapAction(
		FAvaMaskEditorCommands::Get().ToggleMaskMode,
		FExecuteAction::CreateUObject(this, &UAvaMaskEditorSubsystem::ToggleEditorMode),
		FCanExecuteAction(),
		FIsActionChecked::CreateUObject(this, &UAvaMaskEditorSubsystem::IsEditorModeActive),
		FIsActionButtonVisible());

	CommandList->MapAction(
		FAvaMaskEditorCommands::Get().ToggleShowAllMasks,
		FExecuteAction::CreateUObject(this, &UAvaMaskEditorSubsystem::ToggleShowAllMasks),
		FCanExecuteAction(),
		FIsActionChecked::CreateUObject(this, &UAvaMaskEditorSubsystem::IsShowingAllMasks),
		FIsActionButtonVisible());

	CommandList->MapAction(
		FAvaMaskEditorCommands::Get().ToggleIsolateMask,
		FExecuteAction::CreateUObject(this, &UAvaMaskEditorSubsystem::ToggleIsolateSelectedMask),
		FCanExecuteAction(),
		FIsActionChecked::CreateUObject(this, &UAvaMaskEditorSubsystem::IsIsolatingSelectedMask),
		FIsActionButtonVisible());

	CommandList->MapAction(
		FAvaMaskEditorCommands::Get().ToggleEnableMask,
		FExecuteAction::CreateUObject(this, &UAvaMaskEditorSubsystem::ToggleEnableSelectedMask),
		FCanExecuteAction(),
		FIsActionChecked::CreateUObject(this, &UAvaMaskEditorSubsystem::IsSelectedMaskEnabled),
		FIsActionButtonVisible());

	Super::Initialize(Collection);
}

void UAvaMaskEditorSubsystem::Deinitialize()
{
	Super::Deinitialize();
}

void UAvaMaskEditorSubsystem::ShowVisualizeMasks()
{
	static constexpr const TCHAR* Cmd = TEXT("GeometryMask.Visualize");
	GEngine->Exec(nullptr, Cmd);
}

void UAvaMaskEditorSubsystem::ToggleEditorMode()
{
	GLevelEditorModeTools().ActivateMode(UAvaMaskEditorMode::EM_MotionDesignMaskEditorModeId, true);
}

bool UAvaMaskEditorSubsystem::IsEditorModeActive()
{
	return GLevelEditorModeTools().IsModeActive(UAvaMaskEditorMode::EM_MotionDesignMaskEditorModeId);
}

void UAvaMaskEditorSubsystem::ToggleShowAllMasks()
{
}

bool UAvaMaskEditorSubsystem::IsShowingAllMasks()
{
	return bIsShowingAllMasks;
}

void UAvaMaskEditorSubsystem::ToggleIsolateSelectedMask()
{
}

bool UAvaMaskEditorSubsystem::IsIsolatingSelectedMask()
{
	return bIsIsolatingSelectedMask;
}

void UAvaMaskEditorSubsystem::ToggleEnableSelectedMask()
{
}

bool UAvaMaskEditorSubsystem::IsSelectedMaskEnabled()
{
	return bIsSelectedMaskEnabled;
}
