// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "Subsystems/WorldSubsystem.h"

#include "AvaMaskEditorSubsystem.generated.h"

class FAvaMaskSceneViewExtension;

namespace UE::AvaMaskEditor::Internal
{
	static FName ToolkitOverlayMenuName = TEXT("AvalancheMask.Editor.ModeOverlayToolbar");
}

UCLASS()
class UAvaMaskEditorWorldSubsystem
	: public UWorldSubsystem
{
	GENERATED_BODY()

public:
	// ~Begin USubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	// ~End USubsystem

private:
	TSharedPtr<FAvaMaskSceneViewExtension> SceneViewExtension;
};

UCLASS()
class UAvaMaskEditorSubsystem
	: public UEditorSubsystem
{
	GENERATED_BODY()

public:
	// ~Begin UEditorSubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	// ~End UEditorSubsystem

private:
	void ShowVisualizeMasks();
	
	void ToggleEditorMode();
	bool IsEditorModeActive();

	void ToggleShowAllMasks();
	bool IsShowingAllMasks();

	void ToggleIsolateSelectedMask();
	bool IsIsolatingSelectedMask();

	void ToggleEnableSelectedMask();
	bool IsSelectedMaskEnabled();

private:
	UPROPERTY(Transient)
	bool bIsShowingAllMasks = true;

	UPROPERTY(Transient)
	bool bIsIsolatingSelectedMask = true;

	UPROPERTY(Transient)
	bool bIsSelectedMaskEnabled = true;
};
