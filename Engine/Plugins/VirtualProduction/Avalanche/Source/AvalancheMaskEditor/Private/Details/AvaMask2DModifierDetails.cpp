// Copyright Epic Games, Inc. All Rights Reserved.

#include "Details/AvaMask2DModifierDetails.h"

#include "Algo/RemoveIf.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "GeometryMaskCanvas.h"
#include "PropertyAccess.h"

void FAvaMask2DModifierDetails::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	IDetailCategoryBuilder& MaskCategory = InDetailBuilder.EditCategory(TEXT("Mask2D"));
	MaskCategory.SetIsEmpty(true);

	TFunction<TSharedPtr<IPropertyHandle>(const TSharedPtr<IPropertyHandle>&)> GetInnerRootHandle;
	GetInnerRootHandle = [&GetInnerRootHandle](const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		uint32 NumChildren;
		InPropertyHandle->GetNumChildren(NumChildren);
		if (NumChildren == 1)
		{
			return GetInnerRootHandle(InPropertyHandle->GetChildHandle(0));
		}

		// No children, or more than one, so return self as the "last valid" handle
		return InPropertyHandle;
	};

	// Effects
	{
		IDetailCategoryBuilder& EffectsCategory = InDetailBuilder.EditCategory(TEXT("Effects"));

		static const FName CanvasPropertyPath = TEXT("CanvasWeak");
		TSharedRef<IPropertyHandle> CanvasPropertyHandle = InDetailBuilder.GetProperty(CanvasPropertyPath, UAvaMask2DBaseModifier::StaticClass());

		EffectsCategory.InitiallyCollapsed(false);
		EffectsCategory.SetSortOrder(MaskCategory.GetSortOrder() + 1);

		if (CanvasPropertyHandle->IsValidHandle())
		{
			CanvasPropertyHandle->MarkHiddenByCustomization();

			static TArray<FName, TInlineAllocator<3>> CanvasExcludePropertyNames = {
				TEXT("CanvasName"),
				TEXT("CanvasResource"),
				TEXT("ColorChannel")
			};

			const TSharedPtr<IPropertyHandle> CanvasInnerHandle = GetInnerRootHandle(CanvasPropertyHandle);

			uint32 NumChildren = 0;
			if (CanvasInnerHandle->GetNumChildren(NumChildren) == FPropertyAccess::Success)
			{
				for (uint32 ChildIdx = 0; ChildIdx < NumChildren; ++ChildIdx)
				{
					TSharedPtr<IPropertyHandle> ChildHandle = CanvasInnerHandle->GetChildHandle(ChildIdx);
					if (!ensure(ChildHandle->GetProperty()))
					{
						continue;
					}
					
					if (!CanvasExcludePropertyNames.Contains(ChildHandle->GetProperty()->NamePrivate))
					{
						EffectsCategory.AddProperty(ChildHandle);
					}
				}
			}
		}
	}
}
