// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Mask2D/AvaMask2DBaseModifier.h"

class FAvaMask2DModifierDetails
	: public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FAvaMask2DModifierDetails>();
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
};
