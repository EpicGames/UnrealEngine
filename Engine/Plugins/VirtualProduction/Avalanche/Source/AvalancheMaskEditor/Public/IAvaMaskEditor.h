// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::AvaMaskEditor
{
	const static FName MotionDesignMaskEditorModeName("EditMode.MotionDesignMask");
}
