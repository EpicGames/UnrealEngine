// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectKey.h"

#define UE_API AVALANCHEMATERIAL_API

class FMaterialUpdateContext;
class UAvaMaterialCacheSettings;
class UMaterialInterface;

namespace UE::Ava
{
	struct FResolvedShaderProfile;
}

namespace UE::Ava
{

/** Helper to cache materials over multiple frames */
class FMaterialCacheHelper : public TSharedFromThis<FMaterialCacheHelper>, public FGCObject
{
	struct FMaterialKey
	{
		/** Object key to the Material being cached */
		TObjectKey<UMaterialInterface> Material;
		/** Shader profile to use for caching the material shaders */
		FName ShaderProfile;

		bool operator==(const FMaterialKey& InOther) const
		{
			return Material == InOther.Material && ShaderProfile == InOther.ShaderProfile;
		}
		friend uint32 GetTypeHash(const FMaterialKey& InKey)
		{
			return HashCombineFast(GetTypeHash(InKey.Material), GetTypeHash(InKey.ShaderProfile));
		}
	};

	struct FObjectData
	{
		/** Shader profile to use for caching the material shaders */
		FName ShaderProfile;
		/** All the keys to the materials belonging to an object that are still incomplete */
		TSet<FMaterialKey> IncompleteMaterialKeys;
		/** List of subobjects that were pending completion and could not gather materials */
		TSet<TObjectPtr<const UObject>> PendingSubobjects;
		/** Time elapsed before checking pending subobjects */
		float PendingSubobjectElapsedTime = 0.f;
	};

	struct FMaterialData
	{
		/** Material that is pending completion */
		TObjectPtr<UMaterialInterface> Material;
	};

public:
	UE_API static FMaterialCacheHelper& Get();

	/**
	 * Caches the materials in the given object.
	 * @param InObject the object to gather materials for via the material bridge api
	 * @param InShaderProfile the shader profile to use for caching shaders. If none, it will cache via UMaterial::CacheShaders
	 * @return true if materials were gathered for the object, false otherwise. (e.g. if object's materials were already gathered, or no material bridge found)
	 */
	UE_API bool RequestCacheMaterials(const UObject* InObject, FName InShaderProfile = NAME_None);

	/** Processes materials pending completion with a limited time budget */
	UE_API void Tick();

	/** Returns true if there are still any materials being cached */
	UE_API bool IsCaching() const;

	/** Returns true if there are materials from the given object that are still being cached */
	UE_API bool IsCaching(const UObject* InObject) const;

	/** Dumps all the materials that are still being processed (i.e. still incomplete) to the output log */
	UE_API void DumpMaterials();

	//~ Begin FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	virtual FString GetReferencerName() const override;
	//~ End FGCObject

private:
	/** Returns true if the material is still being cached */
	bool IsCachingMaterial(const FMaterialKey& InMaterialKey) const;

	/** Resets the object and material map */
	void Reset();

	/** Registers the tick function to get called every frame */
	void RegisterTick();

	/** Unregisters the tick function from being called every frame */
	void UnregisterTick();

	/** Processes subobjects that did not have its materials gathered */
	void ProcessPendingObjects();

	/** Caches as many queued materials as the time budget allows */
	void ProcessMaterialsPendingCache();

	/** Checks the status on materials that are being cached, and removes from the map once they're complete */
	void ProcessMaterialsCaching();

	/** Removes the complete materials from the objects' IncompleteMaterials set */
	void RemoveCompletedMaterials();

	/** Process async results if shader is compiling */
	void ProcessShadersCompiling();

	/**
	 * Gathers the material data for an object
	 * returns true if a material bridge was found for the object and visited its materials
	 */
	bool GatherMaterialData(const UObject* InObject, FObjectData& InObjectData, FName InShaderProfile);

	/** Objects that have already been gathered, mapped to the incomplete materials that were gathered for the object */
	TMap<FObjectKey, FObjectData> Objects;

	/** All the materials that have been gathered and are pending to start caching their shaders. Once materials start caching, they're removed from this map and added to MaterialsCaching */
	TMap<FMaterialKey, FMaterialData> MaterialsPendingCache;

	/** All the materials that are caching. Once materials are complete, they're removed from this map */
	TMap<FMaterialKey, FMaterialData> MaterialsCaching;

	/** The delta time at the start of the tick */
	float PreTickFrameDuration = 0.0;

	/** Time when BeginFrame was called */
	double BeginFrameTime = 0.0;

	/** Current time budget left for the tick. Resets every tick */
	float RemainingTimeBudget = 0.f;

	/** Number of frames where Tick was skipped because the frame time was past max frame time */
	int32 FramesStarved = 0;

	/** Handle to the BeginFrame delegate to keep track of BeginFrameTime */
	FDelegateHandle BeginFrameHandle;

	/** Handle to the delegate for ticking */
	FDelegateHandle EndFrameHandle;

	/** Reset material cache */
	bool bResetRequested = false;
};

} // UE::Ava

#undef UE_API
