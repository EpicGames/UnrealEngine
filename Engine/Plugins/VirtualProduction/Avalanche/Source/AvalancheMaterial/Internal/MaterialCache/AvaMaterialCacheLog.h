// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogCategory.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAvaMaterialCache, Log, All);
