// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "AvaMaterialCacheSettings.generated.h"

#define UE_API AVALANCHEMATERIAL_API

class FShaderPipelineType;
class FShaderType;
class FVertexFactoryType;

UENUM()
enum class EAvaShaderProfileType : uint8
{
	/**
	 * CacheShaders will be used and will gather all the shaders it might need
	 * NOTE: this might cause more shaders to be compiled than required.
	 */
	Generic,
	/**
	 * Only the specified shaders will be cached. Should yield better performance than generic.
	 * NOTE: Specified shaders only will apply in editor builds. This will default to generic on non-editor builds
	 */
	Specific,
	/** No shaders will be cached */
	SkipCaching,
};

namespace UE::Ava
{

/** Shader Profile that has all the type pointers cached */
struct FResolvedShaderProfile
{
	EAvaShaderProfileType ProfileType = EAvaShaderProfileType::Generic;
	TArray<const FVertexFactoryType*> VertexFactoryTypes;
	TArray<const FShaderPipelineType*> PipelineTypes;
	TArray<const FShaderType*> ShaderTypes;
};

} // UE::Ava

USTRUCT()
struct FAvaShaderCombination
{
	GENERATED_BODY()

	FAvaShaderCombination() = default;

	explicit FAvaShaderCombination(FName InVertexFactorType, TConstArrayView<FName> InPipelineTypes, TConstArrayView<FName> InShaderTypes)
		: VertexFactoryType(InVertexFactorType)
		, PipelineTypes(InPipelineTypes)
		, ShaderTypes(InShaderTypes)
	{
	}

	UPROPERTY(EditAnywhere, Category="Material Cache")
	FName VertexFactoryType;

	UPROPERTY(EditAnywhere, Category="Material Cache")
	TArray<FName> PipelineTypes;

	UPROPERTY(EditAnywhere, Category="Material Cache")
	TArray<FName> ShaderTypes;
};

USTRUCT()
struct FAvaShaderProfile
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Material Cache")
	FName ProfileName;

	UPROPERTY(EditAnywhere, Category="Material Cache")
	EAvaShaderProfileType ProfileType = EAvaShaderProfileType::Generic;

	/** Specific Shader Combinations to use */
	UPROPERTY(EditAnywhere, Category="Material Cache", meta=(EditCondition="ProfileType==EAvaShaderProfileType::Specific"))
	TArray<FAvaShaderCombination> ShaderCombinations;
};

UCLASS(MinimalAPI, config=Engine, meta=(DisplayName="Material Cache Settings"))
class UAvaMaterialCacheSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API static const FLazyName DefaultRealtimeProfile;
	UE_API static const FLazyName DefaultOfflineProfile;

	UAvaMaterialCacheSettings();

	FName GetRealtimeProfile() const
	{
		return RealtimeProfile;
	}

	FName GetOfflineProfile() const
	{
		return OfflineProfile;
	}

	/** Finds the resolved shader profile for a given profile name */
	const UE::Ava::FResolvedShaderProfile* FindResolvedShaderProfile(FName InProfileName) const;

protected:
	//~ Begin UObject
	virtual void PostInitProperties() override;
	virtual void PostReloadConfig(FProperty* InPropertyThatWasLoaded) override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

private:
	/** Add the built-in shader profiles */
	void RegisterDefaultShaderProfile();

	/** Resolves all the registered shader profiles to its type pointers */
	void ResolveShaderProfiles();

	/** Resolves a single shader profile to its type pointers */
	void ResolveShaderProfile(const FAvaShaderProfile& InShaderProfile, TMap<FName, FHashedName>& InOutHashedNames);

	/** Profile to use in realtime (e.g. when playing levels) */
	UPROPERTY(Config, EditAnywhere, Category="Material Cache")
	FName RealtimeProfile;

	/** Profile to use offline (e.g. when preloading levels) */
	UPROPERTY(Config, EditAnywhere, Category="Material Cache")
	FName OfflineProfile;

	/** Shader profiles that can be saved to disk */
	UPROPERTY(Config, EditAnywhere, Category="Material Cache")
	TArray<FAvaShaderProfile> ShaderProfiles;

	/** The saved shader profiles turned into resolved pointers */
	TMap<FName, UE::Ava::FResolvedShaderProfile> ResolvedShaderProfiles;
};

#undef UE_API
