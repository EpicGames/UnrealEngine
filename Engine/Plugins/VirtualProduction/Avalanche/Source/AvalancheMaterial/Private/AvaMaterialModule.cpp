// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMaterialModule.h"
#include "MaterialBridge/AvaMaterialBridge.h"
#include "MaterialBridge/AvaMaterialBridgeLog.h"
#include "MaterialBridge/AvaMaterialBridgeRegistry.h"
#include "MaterialBridge/Common/AvaActorMaterialBridge.h"
#include "MaterialBridge/Common/AvaDecalComponentMaterialBridge.h"
#include "MaterialBridge/Common/AvaLevelMaterialBridge.h"
#include "MaterialBridge/Common/AvaPrimitiveComponentMaterialBridge.h"
#include "MaterialBridge/Common/AvaStaticMeshMaterialBridge.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogAvaMaterialBridge);

IMPLEMENT_MODULE(FAvaMaterialModule, AvalancheMaterial)

void FAvaMaterialModule::StartupModule()
{
	// Low priority for common bridges
	constexpr uint32 DefaultPriority = 0;

	// Register common material bridges
	UE::Ava::FMaterialBridgeRegistry& MaterialBridgeRegistry = UE::Ava::FMaterialBridgeRegistry::GetMutable();
	MaterialBridgeRegistry.Register<UE::Ava::FLevelMaterialBridge>(DefaultPriority);
	MaterialBridgeRegistry.Register<UE::Ava::FActorMaterialBridge>(DefaultPriority);
	MaterialBridgeRegistry.Register<UE::Ava::FPrimitiveComponentMaterialBridge>(DefaultPriority);
	MaterialBridgeRegistry.Register<UE::Ava::FDecalComponentMaterialBridge>(DefaultPriority);
	MaterialBridgeRegistry.Register<UE::Ava::FStaticMeshMaterialBridge>(DefaultPriority);
}
