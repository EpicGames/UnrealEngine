// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialBridge/AvaMaterialBridge.h"
#include "MaterialBridge/AvaMaterialBridgeLog.h"
#include "MaterialBridge/AvaMaterialBridgeRegistry.h"
#include "MaterialBridge/AvaMaterialContainerState.h"
#include "MaterialBridge/Context/AvaMaterialBridgeReadSlotContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeWriteSlotContext.h"

namespace UE::Ava
{

void FMaterialBridge::Initialize(uint32 InPriority)
{
	BridgedType = OnGetBridgedType();
	ContainerStateType = OnGetContainerStateType();
	Priority = InPriority;
}

const UStruct* FMaterialBridge::GetBridgedType() const
{
	return BridgedType;
}

uint32 FMaterialBridge::GetPriority() const
{
	return Priority;
}

bool FMaterialBridge::IsMaterialContainerSupported(FConstDataView InMaterialContainer) const
{
	if (InMaterialContainer.IsValidFor(BridgedType))
	{
		return OnIsMaterialContainerSupported(InMaterialContainer);
	}
	return false;
}

EControlFlow FMaterialBridge::AccessSlots(const FReadSlotContext& InContext, TFunctionRef<EControlFlow(const FReadSlotContext&, const FReadSlot&)> InFunc, const FReadSlotOptions& InOptions) const
{
	if (InContext.MaterialContainer.IsValidFor(BridgedType))
	{
		return OnAccessSlots(InContext, InFunc, InOptions);
	}
	return EControlFlow::Continue;
}

EControlFlow FMaterialBridge::AccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const
{
	if (InContext.MaterialContainer.IsValidFor(BridgedType))
	{
		return OnAccessSlots(InContext, InFunc, InOptions);
	}
	return EControlFlow::Continue;
}

bool FMaterialBridge::CanCreateContainerState() const
{
	return !!ContainerStateType;
}

TInstancedStruct<FAvaMaterialContainerState> FMaterialBridge::CreateContainerState() const
{
	if (CanCreateContainerState())
	{
		return TInstancedStruct<FAvaMaterialContainerState>(ContainerStateType);
	}
	return TInstancedStruct<FAvaMaterialContainerState>();
}

bool FMaterialBridge::ApplyState(const FApplyStateContext& InContext, TConstStructView<FAvaMaterialContainerState> InContainerState, const FApplyStateOptions& InOptions) const
{
	if (InContainerState.IsValid() && InContainerState.GetScriptStruct() == ContainerStateType)
	{
		OnApplyState(InContext, InContainerState, InOptions);
		return true;
	}
	return false;
}

bool FMaterialBridge::StoreState(const FStoreStateContext& InContext, TNotNull<TInstancedStruct<FAvaMaterialContainerState>*> InOutContainerState, const FStoreStateOptions& InOptions) const
{
	if (!InOutContainerState->IsValid() || InOutContainerState->GetScriptStruct() != ContainerStateType)
	{
		TInstancedStruct<FAvaMaterialContainerState> NewContainerState = CreateContainerState();
		if (!NewContainerState.IsValid())
		{
			UE_LOGF(LogAvaMaterialBridge, Verbose, "Material Container State (Type: '%ls') for Material Bridge (Bridged Type: '%ls') could not be created."
				, *GetNameSafe(ContainerStateType)
				, *GetNameSafe(BridgedType));
			return false;
		}

		*InOutContainerState = MoveTemp(NewContainerState);
	}
	// Call the overload where state data is passed in as a FStructView 
	return StoreState(InContext, *InOutContainerState, InOptions);
}

bool FMaterialBridge::StoreState(const FStoreStateContext& InContext, TStructView<FAvaMaterialContainerState> InContainerState, const FStoreStateOptions& InOptions) const
{
	if (InContainerState.IsValid() && InContainerState.GetScriptStruct() == ContainerStateType)
	{
		OnStoreState(InContext, InContainerState, InOptions);
		return true;
	}
	return false;
}

} // UE::Ava
