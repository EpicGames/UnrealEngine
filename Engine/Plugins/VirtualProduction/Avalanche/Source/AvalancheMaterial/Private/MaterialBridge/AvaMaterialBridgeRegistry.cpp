// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialBridge/AvaMaterialBridgeRegistry.h"
#include "MaterialBridge/AvaMaterialBridge.h"

namespace UE::Ava
{

const FMaterialBridgeRegistry& FMaterialBridgeRegistry::Get()
{
	return GetMutable();
}

FMaterialBridgeRegistry& FMaterialBridgeRegistry::GetMutable()
{
	static FMaterialBridgeRegistry MaterialBridgeRegistry;
	return MaterialBridgeRegistry;
}

const FMaterialBridge* FMaterialBridgeRegistry::GetMaterialBridge(FConstDataView InMaterialContainer) const
{
	if (!InMaterialContainer.IsValid())
	{
		return nullptr;
	}
	for (const UStruct* Struct : InMaterialContainer.GetStruct()->GetSuperStructIterator())
	{
		const TArray<TUniquePtr<FMaterialBridge>>* MaterialBridges = RegisteredMaterialBridges.Find(Struct);
		if (!MaterialBridges)
		{
			continue;
		}
		for (const TUniquePtr<FMaterialBridge>& MaterialBridge : *MaterialBridges)
		{
			if (MaterialBridge.IsValid() && MaterialBridge->IsMaterialContainerSupported(InMaterialContainer))
			{
				return MaterialBridge.Get();
			}
		}
	}
	return nullptr;
}

void FMaterialBridgeRegistry::RegisterInternal(TUniquePtr<UE::Ava::FMaterialBridge>&& InMaterialBridge, uint32 InPriority)
{
	if (!ensureMsgf(UObjectInitialized(), TEXT("Cannot register Material Bridges. UObject system not initialized!")))
	{
		return;
	}

	InMaterialBridge->Initialize(InPriority);

	const UStruct* const BridgedType = InMaterialBridge->GetBridgedType();
	if (!ensureMsgf(BridgedType, TEXT("Cannot register Material Bridge with a null bridged type!")))
	{
		return;
	}

	TArray<TUniquePtr<FMaterialBridge>>& MaterialBridges = RegisteredMaterialBridges.FindOrAdd(BridgedType);
	MaterialBridges.Add(MoveTemp(InMaterialBridge));

	// Sort the bridges from highest to lowest priority
	MaterialBridges.StableSort(
		[](const TUniquePtr<FMaterialBridge>& A, const TUniquePtr<FMaterialBridge>& B)
		{
			return A->GetPriority() > B->GetPriority();
		});
}

} // UE::Ava
