// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialBridge/Common/AvaActorMaterialBridge.h"
#include "GameFramework/Actor.h"
#include "MaterialBridge/AvaMaterialBridgeLog.h"
#include "MaterialBridge/AvaMaterialBridgeRegistry.h"
#include "MaterialBridge/Context/AvaMaterialBridgeApplyStateContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeReadSlotContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeStoreStateContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeWriteSlotContext.h"

namespace UE::Ava::Private
{
// Iterates the components of an actor and calls the given func for every valid component with a found material bridge
EControlFlow ForEachBridgedComponent(const AActor& InActor, TNotNull<const FMaterialBridgeRegistry*> InRegistry, TFunctionRef<EControlFlow(TNotNull<UActorComponent*>, TNotNull<const FMaterialBridge*>)> InFunc)
{
	EControlFlow ControlFlow = EControlFlow::Continue;

	constexpr bool bIncludeFromChildActors = true;

	InActor.ForEachComponent(bIncludeFromChildActors,
		[&ControlFlow, &InFunc, &InRegistry](UActorComponent* InComponent)
		{
			if (!InComponent || ControlFlow != EControlFlow::Continue)
			{
				return;
			}
			if (const FMaterialBridge* MaterialBridge = InRegistry->GetMaterialBridge(FConstDataView(InComponent)))
			{
				ControlFlow = InFunc(InComponent, MaterialBridge);
			}
		});

	return ControlFlow;
}

} // UE::Ava::Private

void FAvaActorMaterialContainerStateComponent::SetComponent(UActorComponent* InComponent)
{
	if (InComponent)
	{
		ComponentWeak = InComponent;
		Path = InComponent->GetPathName(InComponent->GetOwner());
	}
	else
	{
		ComponentWeak.Reset();
		Path.Reset();
	}
}

UActorComponent* FAvaActorMaterialContainerStateComponent::ResolveComponent(const AActor* InActor) const
{
	UActorComponent* Component = ComponentWeak.Get();
	if (Component || !InActor)
	{
		return Component;
	}

	if (Path.IsEmpty())
	{
		Component = InActor->GetRootComponent();
	}
	else
	{
		Component = FindObject<UActorComponent>(const_cast<AActor*>(InActor), *Path);	
	}

	ComponentWeak = Component;
	return Component;
}

namespace UE::Ava
{

const UStruct* FActorMaterialBridge::OnGetBridgedType() const
{
	return AActor::StaticClass();
}

EControlFlow FActorMaterialBridge::OnAccessSlots(const FReadSlotContext& InContext, TFunctionRef<EControlFlow(const FReadSlotContext&, const FReadSlot&)> InFunc, const FReadSlotOptions& InOptions) const
{
	const AActor& Actor = InContext.MaterialContainer.Get<const AActor>();

	return Private::ForEachBridgedComponent(Actor, InOptions.MaterialBridgeRegistry,
		[&InContext, &InFunc, &InOptions](TNotNull<UActorComponent*> InComponent, TNotNull<const FMaterialBridge*> InMaterialBridge)->EControlFlow
		{
			return InMaterialBridge->AccessSlots(FReadSlotContext(InComponent, &InContext), InFunc, InOptions);
		});

}

EControlFlow FActorMaterialBridge::OnAccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const
{
	const AActor& Actor = InContext.MaterialContainer.Get<const AActor>();

	return Private::ForEachBridgedComponent(Actor, InOptions.MaterialBridgeRegistry,
		[&InContext, &InFunc, &InOptions](TNotNull<UActorComponent*> InComponent, TNotNull<const FMaterialBridge*> InMaterialBridge)->EControlFlow
		{
			return InMaterialBridge->AccessSlots(FWriteSlotContext(InComponent, &InContext), InFunc, InOptions);
		});
}

TSubScriptStructOf<FAvaMaterialContainerState> FActorMaterialBridge::OnGetContainerStateType() const
{
	return FContainerState::StaticStruct();
}

void FActorMaterialBridge::OnApplyState(const FApplyStateContext& InContext, TConstStructView<FAvaMaterialContainerState> InContainerState, const FApplyStateOptions& InOptions) const
{
	const AActor& Actor = InContext.MaterialContainer.Get<const AActor>();
	const FContainerState& ContainerState = InContainerState.Get<FContainerState>();

	// Call ApplyState on existing component data that still point to valid component
	for (const FAvaActorMaterialContainerStateComponent& ComponentData : ContainerState.Components)
	{
		UActorComponent* const Component = ComponentData.ResolveComponent(&Actor);
		if (!Component)
		{
			continue;
		}

		const FMaterialBridge* const MaterialBridge = InOptions.MaterialBridgeRegistry->GetMaterialBridge(FConstDataView(Component));
		if (!MaterialBridge)
		{
			continue;
		}

		FApplyStateContext ApplyContext(Component, &InContext);
		MaterialBridge->ApplyState(ApplyContext, ComponentData.ContainerState, InOptions);
	}
}

void FActorMaterialBridge::OnStoreState(const FStoreStateContext& InContext, TStructView<FAvaMaterialContainerState> InContainerState, const FStoreStateOptions& InOptions) const
{
	const AActor& Actor = InContext.MaterialContainer.Get<const AActor>();

	FContainerState& ContainerState = InContainerState.Get<FContainerState>();
	ContainerState.Components.Reset();

	Private::ForEachBridgedComponent(Actor, InOptions.MaterialBridgeRegistry,
		[&InContext, &InOptions, &ContainerState](TNotNull<UActorComponent*> InComponent, TNotNull<const FMaterialBridge*> InMaterialBridge)->EControlFlow
		{
			FAvaActorMaterialContainerStateComponent ComponentData;
			ComponentData.SetComponent(InComponent);

			FStoreStateContext StoreContext(InComponent, &InContext);
			if (InMaterialBridge->StoreState(StoreContext, &ComponentData.ContainerState, InOptions))
			{
				ContainerState.Components.Add(MoveTemp(ComponentData));
			}
			return EControlFlow::Continue;
		});
}

} // UE::Ava
