// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialBridge/Common/AvaDecalComponentMaterialBridge.h"
#include "AvaDataView.h"
#include "Components/DecalComponent.h"
#include "MaterialBridge/Context/AvaMaterialBridgeApplyStateContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeReadSlotContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeStoreStateContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeWriteSlotContext.h"
#include "MaterialBridge/Slot/AvaMaterialBridgeReadSlot.h"
#include "MaterialBridge/Slot/AvaMaterialBridgeWriteSlot.h"

namespace UE::Ava
{

const UStruct* FDecalComponentMaterialBridge::OnGetBridgedType() const
{
	return UDecalComponent::StaticClass();
}

EControlFlow FDecalComponentMaterialBridge::OnAccessSlots(const FReadSlotContext& InContext, TFunctionRef<EControlFlow(const FReadSlotContext&, const FReadSlot&)> InFunc, const FReadSlotOptions& InOptions) const
{
	const UDecalComponent& Component = InContext.MaterialContainer.Get<const UDecalComponent>();

	const FReadSlot Slot(Component.GetDecalMaterial(), FSlotId(0));
	return InFunc(InContext, Slot);
}

EControlFlow FDecalComponentMaterialBridge::OnAccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const
{
	UDecalComponent& Component = InContext.MaterialContainer.GetMutable<UDecalComponent>();

	FWriteSlot Slot(Component.GetDecalMaterial(), FSlotId(0));

	const EControlFlow ControlFlow = InFunc(InContext, Slot);

	if (Component.GetDecalMaterial() != Slot.GetMaterial())
	{
		Component.SetDecalMaterial(Slot.GetMaterial());
	}
	return ControlFlow;
}

TSubScriptStructOf<FAvaMaterialContainerState> FDecalComponentMaterialBridge::OnGetContainerStateType() const
{
	return FContainerState::StaticStruct();
}

void FDecalComponentMaterialBridge::OnApplyState(const FApplyStateContext& InContext, TConstStructView<FAvaMaterialContainerState> InContainerState, const FApplyStateOptions& InOptions) const
{
	UDecalComponent& Component = InContext.MaterialContainer.GetMutable<UDecalComponent>();

	const FContainerState& ContainerState = InContainerState.Get<FContainerState>();
	Component.SetDecalMaterial(ContainerState.DecalMaterial);
}

void FDecalComponentMaterialBridge::OnStoreState(const FStoreStateContext& InContext, TStructView<FAvaMaterialContainerState> InContainerState, const FStoreStateOptions& InOptions) const
{
	const UDecalComponent& Component = InContext.MaterialContainer.Get<const UDecalComponent>();

	FContainerState& ContainerState = InContainerState.Get<FContainerState>();
	ContainerState.DecalMaterial = Component.GetDecalMaterial();
}

} // UE::Ava
