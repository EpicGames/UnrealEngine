// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialBridge/Common/AvaLevelMaterialBridge.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "MaterialBridge/AvaMaterialBridgeRegistry.h"
#include "MaterialBridge/Context/AvaMaterialBridgeReadSlotContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeWriteSlotContext.h"

namespace UE::Ava
{

const UStruct* FLevelMaterialBridge::OnGetBridgedType() const
{
	return ULevel::StaticClass();
}

EControlFlow FLevelMaterialBridge::OnAccessSlots(const FReadSlotContext& InContext, TFunctionRef<EControlFlow(const FReadSlotContext&, const FReadSlot&)> InFunc, const FReadSlotOptions& InOptions) const
{
	const ULevel& Level = InContext.MaterialContainer.Get<const ULevel>();

	for (AActor* Actor : Level.Actors)
	{
		if (const FMaterialBridge* MaterialBridge = InOptions.MaterialBridgeRegistry->GetMaterialBridge(FConstDataView(Actor)))
		{
			const EControlFlow ControlFlow = MaterialBridge->AccessSlots(FReadSlotContext(Actor, &InContext), InFunc, InOptions);
			if (ControlFlow == EControlFlow::Break)
			{
				return EControlFlow::Break;
			}
		}
	}

	return EControlFlow::Continue;
}

EControlFlow FLevelMaterialBridge::OnAccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const
{
	const ULevel& Level = InContext.MaterialContainer.Get<const ULevel>();

	for (AActor* Actor : Level.Actors)
	{
		if (const FMaterialBridge* MaterialBridge = InOptions.MaterialBridgeRegistry->GetMaterialBridge(FConstDataView(Actor)))
		{
			const EControlFlow ControlFlow = MaterialBridge->AccessSlots(FWriteSlotContext(Actor, &InContext), InFunc, InOptions);
			if (ControlFlow == EControlFlow::Break)
			{
				return EControlFlow::Break;
			}
		}
	}

	return EControlFlow::Continue;
}

} // UE::Ava
