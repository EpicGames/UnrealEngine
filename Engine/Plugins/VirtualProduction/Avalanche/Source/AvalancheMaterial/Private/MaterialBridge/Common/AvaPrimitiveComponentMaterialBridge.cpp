// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialBridge/Common/AvaPrimitiveComponentMaterialBridge.h"
#include "Components/PrimitiveComponent.h"
#include "MaterialBridge/Context/AvaMaterialBridgeApplyStateContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeReadSlotContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeStoreStateContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeWriteSlotContext.h"
#include "MaterialBridge/Slot/AvaMaterialBridgeReadSlot.h"
#include "MaterialBridge/Slot/AvaMaterialBridgeWriteSlot.h"
#include "Misc/EnumerateRange.h"

namespace UE::Ava
{

const UStruct* FPrimitiveComponentMaterialBridge::OnGetBridgedType() const
{
	return UPrimitiveComponent::StaticClass();
}

EControlFlow FPrimitiveComponentMaterialBridge::OnAccessSlots(const FReadSlotContext& InContext, TFunctionRef<EControlFlow(const FReadSlotContext&, const FReadSlot&)> InFunc, const FReadSlotOptions& InOptions) const
{
	const UPrimitiveComponent& Component = InContext.MaterialContainer.Get<const UPrimitiveComponent>();

	for (int32 MaterialIndex = 0; MaterialIndex < Component.GetNumMaterials(); ++MaterialIndex)
	{
		const FReadSlot Slot(Component.GetMaterial(MaterialIndex), FSlotId(MaterialIndex));
		if (InFunc(InContext, Slot) == EControlFlow::Break)
		{
			return EControlFlow::Break;
		}
	}

	return EControlFlow::Continue;
}

EControlFlow FPrimitiveComponentMaterialBridge::OnAccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const
{
	UPrimitiveComponent& Component = InContext.MaterialContainer.GetMutable<UPrimitiveComponent>();

	for (int32 MaterialIndex = 0; MaterialIndex < Component.GetNumMaterials(); ++MaterialIndex)
	{
		FWriteSlot Slot(Component.GetMaterial(MaterialIndex), FSlotId(MaterialIndex));
		const EControlFlow ControlFlow = InFunc(InContext, Slot);

		if (Component.GetMaterial(MaterialIndex) != Slot.GetMaterial())
		{
			Component.SetMaterial(MaterialIndex, Slot.GetMaterial());
		}
		if (ControlFlow == EControlFlow::Break)
		{
			return EControlFlow::Break;
		}
	}
	return EControlFlow::Continue;
}

TSubScriptStructOf<FAvaMaterialContainerState> FPrimitiveComponentMaterialBridge::OnGetContainerStateType() const
{
	return FContainerState::StaticStruct();
}

void FPrimitiveComponentMaterialBridge::OnApplyState(const FApplyStateContext& InContext, TConstStructView<FAvaMaterialContainerState> InContainerState, const FApplyStateOptions& InOptions) const
{
	UPrimitiveComponent& Component = InContext.MaterialContainer.GetMutable<UPrimitiveComponent>();

	const FContainerState& ContainerState = InContainerState.Get<FContainerState>();

	for (TEnumerateRef<const TObjectPtr<UMaterialInterface>> Material : EnumerateRange(ContainerState.Materials))
	{
		Component.SetMaterial(Material.GetIndex(), *Material);
	}
}

void FPrimitiveComponentMaterialBridge::OnStoreState(const FStoreStateContext& InContext, TStructView<FAvaMaterialContainerState> InContainerState, const FStoreStateOptions& InOptions) const
{
	const UPrimitiveComponent& Component = InContext.MaterialContainer.Get<const UPrimitiveComponent>();
	FContainerState& ContainerState = InContainerState.Get<FContainerState>();

	ContainerState.Materials.SetNum(Component.GetNumMaterials());

	for (TEnumerateRef<TObjectPtr<UMaterialInterface>> Material : EnumerateRange(ContainerState.Materials))
	{
		*Material = Component.GetMaterial(Material.GetIndex());
	}
}

} // UE::Ava
