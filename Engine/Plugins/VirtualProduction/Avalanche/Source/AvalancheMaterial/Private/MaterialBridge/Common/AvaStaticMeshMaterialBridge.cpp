// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialBridge/Common/AvaStaticMeshMaterialBridge.h"
#include "Engine/StaticMesh.h"
#include "MaterialBridge/AvaMaterialBridgeLog.h"
#include "MaterialBridge/Context/AvaMaterialBridgeReadSlotContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeWriteSlotContext.h"
#include "MaterialBridge/Slot/AvaMaterialBridgeReadSlot.h"
#include "MaterialBridge/Slot/AvaMaterialBridgeWriteSlot.h"
#include "Misc/EnumerateRange.h"

namespace UE::Ava
{

namespace Private
{

static const FLazyName BaseSlot(TEXT("Base"));
static const FLazyName OverlaySlot(TEXT("Overlay"));

} // UE::Ava::Private
	
const UStruct* FStaticMeshMaterialBridge::OnGetBridgedType() const
{
	return UStaticMesh::StaticClass();
}

EControlFlow FStaticMeshMaterialBridge::OnAccessSlots(const FReadSlotContext& InContext, TFunctionRef<EControlFlow(const FReadSlotContext&, const FReadSlot&)> InFunc, const FReadSlotOptions& InOptions) const
{
	const UStaticMesh& StaticMesh = InContext.MaterialContainer.Get<const UStaticMesh>();

	// When static mesh is compiling GetStaticMaterials() waits until LockedProperties does not have EStaticMeshAsyncProperties::StaticMaterials set.
	// If static mesh is compiling and caller does not want to wait on materials, bail out.
	// NOTE: Ideally only EStaticMeshAsyncProperties::StaticMaterials needs to be checked in LockedProperties, but there's currently no option to check for it directly.
	if (StaticMesh.IsCompiling() && InOptions.bTrySkipWaitOnContainerCompletion)
	{
		if (InOptions.OnContainerPendingCompletion.IsBound())
		{
			UE_LOGF(LogAvaMaterialBridge, Verbose, "Material Bridge could not visit materials of static mesh '%ls'. It is currently compiling.", *StaticMesh.GetName());
			return InOptions.OnContainerPendingCompletion.Execute(InContext);
		}

		UE_LOGF(LogAvaMaterialBridge, Warning, "Material Bridge could not visit materials of static mesh '%ls'. It is currently compiling.", *StaticMesh.GetName());
		return EControlFlow::Continue;
	}

	for (TConstEnumerateRef<FStaticMaterial> StaticMaterial : EnumerateRange(StaticMesh.GetStaticMaterials()))
	{
		// Base Material
		{
			const FReadSlot Slot(StaticMaterial->MaterialInterface, FSlotId(Private::BaseSlot, StaticMaterial.GetIndex()));
			if (InFunc(InContext, Slot) == EControlFlow::Break)
			{
				return EControlFlow::Break;
			}
		}

		// Overlay Material
		{
			const FReadSlot Slot(StaticMaterial->OverlayMaterialInterface, FSlotId(Private::OverlaySlot, StaticMaterial.GetIndex()));
			if (InFunc(InContext, Slot) == EControlFlow::Break)
			{
				return EControlFlow::Break;
			}
		}
	}
	return EControlFlow::Continue;
}

EControlFlow FStaticMeshMaterialBridge::OnAccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const
{
	UStaticMesh& StaticMesh = InContext.MaterialContainer.GetMutable<UStaticMesh>();

	// When static mesh is compiling GetStaticMaterials() waits until LockedProperties does not have EStaticMeshAsyncProperties::StaticMaterials set.
	// If static mesh is compiling and caller does not want to wait on materials, bail out.
	// NOTE: Ideally only EStaticMeshAsyncProperties::StaticMaterials needs to be checked in LockedProperties, but there's currently no option to check for it directly.
	if (StaticMesh.IsCompiling() && InOptions.bTrySkipWaitOnContainerCompletion)
	{
		if (InOptions.OnContainerPendingCompletion.IsBound())
		{
			UE_LOGF(LogAvaMaterialBridge, Verbose, "Material Bridge could not access material slots of static mesh '%ls'. Mesh is currently compiling.", *StaticMesh.GetName());
			return InOptions.OnContainerPendingCompletion.Execute(InContext);
		}

		UE_LOGF(LogAvaMaterialBridge, Warning, "Material Bridge could not access material slots of static mesh '%ls'. Mesh is currently compiling.", *StaticMesh.GetName());
		return EControlFlow::Continue;
	}

	TArray<FStaticMaterial> StaticMaterials = StaticMesh.GetStaticMaterials();
	if (StaticMaterials.IsEmpty())
	{
		return EControlFlow::Continue;
	}

	ON_SCOPE_EXIT
	{
		StaticMesh.SetStaticMaterials(StaticMaterials);
	};

	for (TEnumerateRef<FStaticMaterial> StaticMaterial : EnumerateRange(StaticMaterials))
	{
		// Base Material
		{
			FWriteSlot Slot(StaticMaterial->MaterialInterface, FSlotId(Private::BaseSlot, StaticMaterial.GetIndex()));

			const EControlFlow ControlFlow = InFunc(InContext, Slot);
			StaticMaterial->MaterialInterface = Slot.GetMaterial();

			if (ControlFlow == EControlFlow::Break)
			{
				return EControlFlow::Break;
			}
		}

		// Overlay Material
		{
			FWriteSlot Slot(StaticMaterial->OverlayMaterialInterface, FSlotId(Private::OverlaySlot, StaticMaterial.GetIndex()));

			const EControlFlow ControlFlow = InFunc(InContext, Slot);
			StaticMaterial->OverlayMaterialInterface = Slot.GetMaterial();

			if (ControlFlow == EControlFlow::Break)
			{
				return EControlFlow::Break;
			}
		}
	}

	return EControlFlow::Continue;
}

} // UE::Ava
