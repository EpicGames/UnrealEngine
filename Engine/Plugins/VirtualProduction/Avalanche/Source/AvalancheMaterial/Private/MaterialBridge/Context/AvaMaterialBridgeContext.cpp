// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialBridge/Context/AvaMaterialBridgeContext.h"
#include "MaterialBridge/AvaMaterialBridgeRegistry.h"
#include "MaterialBridge/Context/AvaMaterialBridgeApplyStateContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeReadSlotContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeStoreStateContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeWriteSlotContext.h"

namespace UE::Ava
{

FMaterialBridgeBaseOptions::FMaterialBridgeBaseOptions()
	: MaterialBridgeRegistry(&FMaterialBridgeRegistry::Get())
{
}

template<typename InContextType>
TMaterialBridgeContext<InContextType>::TMaterialBridgeContext(FDataViewType InMaterialContainer)
	: MaterialContainer(InMaterialContainer)
{
}

template<typename InContextType>
TMaterialBridgeContext<InContextType>::TMaterialBridgeContext(FDataViewType InMaterialContainer, TNotNull<const InContextType*> InParentContext)
	: MaterialContainer(InMaterialContainer)
	, ParentContext(InParentContext)
{
}

template<typename InContextType>
TMaterialBridgeContext<InContextType>::TMaterialBridgeContext(UObjectType* InMaterialContainer)
	: MaterialContainer(InMaterialContainer)
{
}

template<typename InContextType>
TMaterialBridgeContext<InContextType>::TMaterialBridgeContext(UObjectType* InMaterialContainer, TNotNull<const InContextType*> InParentContext)
	: MaterialContainer(InMaterialContainer)
	, ParentContext(InParentContext)
{
}

template<typename InContextType>
const InContextType& TMaterialBridgeContext<InContextType>::GetTopmostContext() const
{
	const InContextType* TopmostContext = static_cast<const InContextType*>(this);
	while (TopmostContext->ParentContext)
	{
		TopmostContext = TopmostContext->ParentContext;
	}
	return *TopmostContext;
}

template<typename InContextType>
TMaterialBridgeContext<InContextType>::UObjectType* TMaterialBridgeContext<InContextType>::GetMaterialContainerObject() const
{
	if constexpr (bWithConstMaterialContainer)
	{
		if (UObjectType* MaterialContainerObject = MaterialContainer.template GetPtr<UObjectType>())
		{
			return MaterialContainerObject;
		}
	}
	else
	{
		if (UObjectType* MaterialContainerObject = MaterialContainer.template GetMutablePtr<UObjectType>())
		{
			return MaterialContainerObject;
		}
	}

	if (ParentContext)
	{
		return ParentContext->GetMaterialContainerObject();
	}
	return nullptr;
}

// Explicit template instantiations
template class TMaterialBridgeContext<FMaterialBridgeApplyStateContext>;
template class TMaterialBridgeContext<FMaterialBridgeReadSlotContext>;
template class TMaterialBridgeContext<FMaterialBridgeStoreStateContext>;
template class TMaterialBridgeContext<FMaterialBridgeWriteSlotContext>;

} // UE::Ava

