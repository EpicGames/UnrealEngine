// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialBridge/Slot/AvaMaterialBridgeSlot.h"

namespace UE::Ava
{

FMaterialBridgeSlot::FMaterialBridgeSlot(UMaterialInterface* InMaterial, const FSlotId& InSlotId)
	: Material(InMaterial)
	, SlotId(InSlotId)
{
}

} // UE::Ava
