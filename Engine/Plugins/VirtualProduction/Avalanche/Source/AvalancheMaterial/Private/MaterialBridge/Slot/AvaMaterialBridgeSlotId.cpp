// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialBridge/Slot/AvaMaterialBridgeSlotId.h"

FAvaMaterialBridgeSlotId::FAvaMaterialBridgeSlotId(int32 InSlotIndex)
	: Index(InSlotIndex)
{
}

FAvaMaterialBridgeSlotId::FAvaMaterialBridgeSlotId(FName InSlotName)
	: Name(InSlotName)
{
}

FAvaMaterialBridgeSlotId::FAvaMaterialBridgeSlotId(FName InSlotName, int32 InSlotIndex)
	: Name(InSlotName)
	, Index(InSlotIndex)
{
}

bool FAvaMaterialBridgeSlotId::IsValid() const
{
	return Name != NAME_None || Index != INDEX_NONE;
}
