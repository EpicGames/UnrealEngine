// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialBridge/Slot/AvaMaterialBridgeWriteSlot.h"

namespace UE::Ava
{

void FMaterialBridgeWriteSlot::SetMaterial(UMaterialInterface* InMaterial)
{
	if (InMaterial != Material)
	{
		SetMaterialInternal(InMaterial);
		OnMaterialChanged();
	}
}

bool FMaterialBridgeWriteSlot::RequestFeature(TConstStructView<FAvaMaterialBridgeFeature> InFeature)
{
	if (InFeature.IsValid())
	{
		return OnFeatureRequest(InFeature);
	}
	return false;
}

void FMaterialBridgeWriteSlot::SetMaterialInternal(UMaterialInterface* InMaterial)
{
	Material = InMaterial;
}

} // UE::Ava
