// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialCache/AvaMaterialCacheHelper.h"
#include "AvaDataView.h"
#include "MaterialBridge/AvaMaterialBridge.h"
#include "MaterialBridge/AvaMaterialBridgeRegistry.h"
#include "MaterialBridge/Context/AvaMaterialBridgeReadSlotContext.h"
#include "MaterialBridge/Slot/AvaMaterialBridgeReadSlot.h"
#include "MaterialCache/AvaMaterialCacheLog.h"
#include "MaterialCache/AvaMaterialCacheSettings.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "ShaderCompiler.h"
#include "UnrealEngine.h"

DEFINE_LOG_CATEGORY(LogAvaMaterialCache);

namespace UE::Ava
{

namespace Private
{

FAutoConsoleCommand MaterialCacheDumpMaterials(TEXT("MotionDesign.MaterialCache.DumpMaterials")
	, TEXT("Dumps all the materials that are currently being processed in the Material Cache Helper")
	, FConsoleCommandDelegate::CreateLambda([]
		{
			FMaterialCacheHelper::Get().DumpMaterials();
		}));

TAutoConsoleVariable<float> CVarMaterialCacheFrameTimeStarveThreshold(TEXT("MotionDesign.MaterialCache.FrameTimeStarveThreshold")
	, 1.f / 30.f
	, TEXT("The frame time threshold for starving tick. If the current duration of the frame surpasses this, FMaterialCacheHelper::Tick will be skipped and starve.")
	);

TAutoConsoleVariable<int32> CVarMaterialCacheTickMaxStarveFrames(TEXT("MotionDesign.MaterialCache.TickMaxStarveFrames")
	, 8
	, TEXT("The maximum number of consecutive frames that FMaterialCacheHelper::Tick can starve for. After this, tick will be forced to execute.")
	);

TAutoConsoleVariable<float> CVarMaterialCacheTickTimeBudget(TEXT("MotionDesign.MaterialCache.TickTimeBudget")
	, 0.002f
	, TEXT("The time budget FMaterialCacheHelper::Tick has to process its pending elements.")
	);

#if WITH_EDITOR
void ForEachMaterialResource(TNotNull<UMaterialInterface*> InMaterial, TFunctionRef<bool(FMaterialResource*)> InFunc)
{
	const EMaterialQualityLevel::Type ActiveQualityLevel = GetCachedScalabilityCVars().MaterialQualityLevel;
	uint32 FeatureLevelsToCompile = UMaterial::GetFeatureLevelsToCompileForAllMaterials();

	while (FeatureLevelsToCompile != 0)
	{
		const ERHIFeatureLevel::Type FeatureLevel = static_cast<ERHIFeatureLevel::Type>(FBitSet::GetAndClearNextBit(FeatureLevelsToCompile));
		const EShaderPlatform ShaderPlatform = GShaderPlatformForFeatureLevel[FeatureLevel];

		// Only cache shaders for the quality level that will actually be used to render
		// In cooked build, there is no shader compilation but this is still needed to register the loaded shadermap
		FMaterialResource* CurrentResource = InMaterial->GetMaterialResource(ShaderPlatform, ActiveQualityLevel);
		if (ensure(CurrentResource) && !InFunc(CurrentResource))
		{
			break;
		}
	}
}
#endif

/** Finds the shader profile matching the given name */
const FResolvedShaderProfile* FindShaderProfile(FName InShaderProfile)
{
	if (const UAvaMaterialCacheSettings* MaterialCacheSettings = GetDefault<UAvaMaterialCacheSettings>())
	{
		return MaterialCacheSettings->FindResolvedShaderProfile(InShaderProfile);
	}
	UE_LOGF(LogAvaMaterialCache, Verbose, "Shader Profile '%ls' not found in Material Cache Settings", *InShaderProfile.ToString());
	return nullptr;
}

/** Performs shader caching of the given material with the given shader profile */
void CacheMaterial(TNotNull<UMaterialInterface*> InMaterial, TNotNull<const FResolvedShaderProfile*> InShaderProfile)
{
	UE_LOGF(LogAvaMaterialCache, VeryVerbose, "Caching Material '%ls'", *InMaterial->GetFullName());

#if WITH_EDITOR
	if (InShaderProfile->ProfileType == EAvaShaderProfileType::Specific)
	{
		InMaterial->CacheShaders(EMaterialShaderPrecompileMode::None);

		ForEachMaterialResource(InMaterial,
			[InShaderProfile](FMaterialResource* InMaterialResource)->bool
			{
				InMaterialResource->CacheGivenTypes(InShaderProfile->VertexFactoryTypes, InShaderProfile->PipelineTypes, InShaderProfile->ShaderTypes);
				return true; // continue
			});

		return;
	}
#endif

	InMaterial->CacheShaders(EMaterialShaderPrecompileMode::Background);
}

/** Checks whether the material is complete */
bool IsMaterialComplete(TNotNull<UMaterialInterface*> InMaterial, TNotNull<const FResolvedShaderProfile*> InShaderProfile)
{
#if WITH_EDITOR
	if (InShaderProfile->ProfileType == EAvaShaderProfileType::Specific)
	{
		bool bIsComplete = true;

		ForEachMaterialResource(InMaterial, 
			[&bIsComplete](FMaterialResource* InMaterialResource)->bool
			{
				if (InMaterialResource->IsCachingShaders())
				{
					bIsComplete = false;
					return false; // break
				}
				return true; // continue
			});

		return bIsComplete;
	}
#endif
	return InMaterial->IsComplete();
}

inline double GetCurrentTime()
{
	return FPlatformTime::Seconds();
}

} // UE::Ava::Private

FMaterialCacheHelper& FMaterialCacheHelper::Get()
{
	static TSharedRef<FMaterialCacheHelper> MaterialCacheHelper = MakeShared<FMaterialCacheHelper>();
	return *MaterialCacheHelper;
}

bool FMaterialCacheHelper::RequestCacheMaterials(const UObject* InObject, FName InShaderProfile)
{
	if (!InObject || Objects.Contains(InObject))
	{
		return false;
	}

	FObjectData ObjectData;
	if (!GatherMaterialData(InObject, ObjectData, InShaderProfile))
	{
		return false;
	}

	Objects.Add(InObject, MoveTemp(ObjectData));
	RegisterTick();
	return true;
}

void FMaterialCacheHelper::Tick()
{
	if (bResetRequested)
	{
		bResetRequested = false;
		Reset();
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialCacheHelper::Tick);

	// Duration of the frame so far prior to ticking
	PreTickFrameDuration = (Private::GetCurrentTime() - BeginFrameTime);

	// If the current frame time is already past the threshold, tick should be skipped unless it has been starving for a while already.
	if (PreTickFrameDuration >= Private::CVarMaterialCacheFrameTimeStarveThreshold.GetValueOnGameThread())
	{
		if (FramesStarved < Private::CVarMaterialCacheTickMaxStarveFrames.GetValueOnGameThread())
		{
			++FramesStarved;
			return;
		}
		UE_LOGF(LogAvaMaterialCache, Verbose, "FMaterialCacheHelper::Tick was starved for %d frames. Forcing tick...", FramesStarved);
	}

	FramesStarved = 0;
	RemainingTimeBudget = Private::CVarMaterialCacheTickTimeBudget.GetValueOnGameThread();

	ProcessPendingObjects();
	ProcessMaterialsPendingCache();
	ProcessMaterialsCaching();

	if (IsCaching())
	{
		ProcessShadersCompiling();
		RemoveCompletedMaterials();
	}
	else
	{
		bResetRequested = true;
	}
}

bool FMaterialCacheHelper::IsCaching() const
{
	// If there is at least one element in either map, return true
	if (!MaterialsPendingCache.IsEmpty() || !MaterialsCaching.IsEmpty())
	{
		return true;
	}

	// If there is one pending subobject still loading to cache, return true
	for (const TPair<FObjectKey, FObjectData>& Object : Objects)
	{
		if (!Object.Value.PendingSubobjects.IsEmpty())
		{
			return true;
		}
	}
	return false;
}

bool FMaterialCacheHelper::IsCaching(const UObject* InObject) const
{
	const FObjectData* ObjectData = Objects.Find(InObject);
	if (!ObjectData)
	{
		return false;
	}

	// There are pending subobjects waiting on completion for material gathering
	if (!ObjectData->PendingSubobjects.IsEmpty())
	{
		return true;
	}

	for (const FMaterialKey& MaterialKey : ObjectData->IncompleteMaterialKeys)
	{
		if (IsCachingMaterial(MaterialKey))
		{
			return true;
		}
	}
	return false;
}

void FMaterialCacheHelper::DumpMaterials()
{
	auto PrintLog = [](const TPair<FMaterialKey, FMaterialData>& InPair)
		{
			const FMaterialData& MaterialData = InPair.Value;
			UE_LOGF(LogAvaMaterialCache, Log, "\t%ls", MaterialData.Material ? *MaterialData.Material->GetFullName() : TEXT("(invalid)"));
		};

	UE_LOGF(LogAvaMaterialCache, Log, "--- Materials Pending Cache ------------------------------------");
	for (const TPair<FMaterialKey, FMaterialData>& Pair : MaterialsPendingCache)
	{
		PrintLog(Pair);
	}
	UE_LOGF(LogAvaMaterialCache, Log, "--- Materials Caching ------------------------------------------");
	for (const TPair<FMaterialKey, FMaterialData>& Pair : MaterialsCaching)
	{
		PrintLog(Pair);
	}
}

void FMaterialCacheHelper::AddReferencedObjects(FReferenceCollector& InCollector)
{
	for (TPair<FObjectKey, FObjectData>& Pair : Objects)
	{
		InCollector.AddReferencedObjects(Pair.Value.PendingSubobjects);
	}
	for (TPair<FMaterialKey, FMaterialData>& Pair : MaterialsPendingCache)
	{
		InCollector.AddReferencedObject(Pair.Value.Material);
	}
	for (TPair<FMaterialKey, FMaterialData>& Pair : MaterialsCaching)
	{
		InCollector.AddReferencedObject(Pair.Value.Material);
	}
}

FString FMaterialCacheHelper::GetReferencerName() const
{
	return TEXT("UE::Ava::FMaterialCacheHelper");
}

bool FMaterialCacheHelper::IsCachingMaterial(const FMaterialKey& InMaterialKey) const
{
	return MaterialsPendingCache.Contains(InMaterialKey) || MaterialsCaching.Contains(InMaterialKey);
}

void FMaterialCacheHelper::Reset()
{
	MaterialsPendingCache.Empty();
	MaterialsCaching.Empty();
	Objects.Empty();
	UnregisterTick();
}

void FMaterialCacheHelper::RegisterTick()
{
	bResetRequested = false;

	if (!BeginFrameHandle.IsValid())
	{
		// It is likely RegisterTick is called between OnBeginFrame and OnEndFrame delegate broadcasts.
		// This will cause BeginFrameTime to have its last value in the first tick (e.g. 0.0, or an old begin frame's time) and likely starve tick for that one frame.
		// And this is ok. An alternative could be to use a timer that is set at BeginFrame always.  
		BeginFrameHandle = FCoreDelegates::OnBeginFrame.AddSPLambda(this, [this]
			{
				BeginFrameTime = Private::GetCurrentTime();
			});
	}

	if (!EndFrameHandle.IsValid())
	{
		EndFrameHandle = FCoreDelegates::OnEndFrame.AddSP(this, &FMaterialCacheHelper::Tick);
	}
}

void FMaterialCacheHelper::UnregisterTick()
{
	if (BeginFrameHandle.IsValid())
	{
		FCoreDelegates::OnBeginFrame.Remove(BeginFrameHandle);
		BeginFrameHandle.Reset();
	}

	if (EndFrameHandle.IsValid())
	{
		FCoreDelegates::OnEndFrame.Remove(EndFrameHandle);
		EndFrameHandle.Reset();
	}
}

void FMaterialCacheHelper::ProcessPendingObjects()
{
	if (RemainingTimeBudget <= 0.f || Objects.IsEmpty())
	{
		return;
	}

	// Time when budget runs out
	const double EndBudgetTime = Private::GetCurrentTime() + RemainingTimeBudget;
	bool bTimeBudgetExceeded = false;

	// Check the pending status every 100ms (0.1s) to give it some time to load
	constexpr float PendingSubobjectCheckTime = 0.1f;

	for (TPair<FObjectKey, FObjectData>& Object : Objects)
	{
		Object.Value.PendingSubobjectElapsedTime += PreTickFrameDuration;
		if (Object.Value.PendingSubobjectElapsedTime < PendingSubobjectCheckTime)
		{
			continue;
		}
		Object.Value.PendingSubobjectElapsedTime = 0.f;

		// All pending subobjects might be ready so move them into a temp set.
		// This also allows adding elements to the existing set.
		// Those that are still not gathered pending load will be re-added to the set.
		TSet<TObjectPtr<const UObject>> PendingSubobjects = MoveTemp(Object.Value.PendingSubobjects);
		Object.Value.PendingSubobjects.Empty();

		for (TSet<TObjectPtr<const UObject>>::TIterator Iter(PendingSubobjects); Iter; ++Iter)
		{
			GatherMaterialData(*Iter, Object.Value, Object.Value.ShaderProfile);
			Iter.RemoveCurrent();

			// Check if time is past end budget time
			bTimeBudgetExceeded = Private::GetCurrentTime() >= EndBudgetTime;
			if (bTimeBudgetExceeded)
			{
				UE_LOGF(LogAvaMaterialCache, Verbose, "FMaterialCacheHelper::Tick ran out of budget in ProcessPendingObjects. Time Budget: %f", RemainingTimeBudget);
				break;
			}
		}

		// Add pending subobjects that could not be processed due to time budget back
		Object.Value.PendingSubobjects.Append(MoveTemp(PendingSubobjects));
		if (bTimeBudgetExceeded)
		{
			break;
		}
	}

	RemainingTimeBudget = EndBudgetTime - Private::GetCurrentTime();
}

void FMaterialCacheHelper::ProcessMaterialsPendingCache()
{
	if (RemainingTimeBudget <= 0.f)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialCacheHelper::ProcessMaterialsPendingCache);

	// Time when budget runs out
	const double EndBudgetTime = Private::GetCurrentTime() + RemainingTimeBudget;

	// Material update context. Only allocated when needed
	TOptional<FMaterialUpdateContext> UpdateContext;

	for (TMap<FMaterialKey, FMaterialData>::TIterator Iter(MaterialsPendingCache); Iter; ++Iter)
	{
		const FMaterialKey& MaterialKey = Iter.Key();
		FMaterialData& MaterialData = Iter.Value();

		if (!MaterialData.Material)
		{
			Iter.RemoveCurrent();
			continue;
		}

		const FResolvedShaderProfile* ShaderProfile = Private::FindShaderProfile(MaterialKey.ShaderProfile);
		if (!ShaderProfile || ShaderProfile->ProfileType == EAvaShaderProfileType::SkipCaching)
		{
			Iter.RemoveCurrent();
			continue;
		}

		if (!UpdateContext.IsSet())
		{
			UpdateContext.Emplace(0);
		}
		UpdateContext->AddMaterialInterface(MaterialData.Material);
		Private::CacheMaterial(MaterialData.Material, ShaderProfile);

		MaterialsCaching.Add(MaterialKey, MoveTemp(MaterialData));
		Iter.RemoveCurrent();

		// Check if time is past end budget time
		// This is only checked for materials waiting, since the other operations are fast.
		if (Private::GetCurrentTime() >= EndBudgetTime)
		{
			UE_LOGF(LogAvaMaterialCache, Verbose, "FMaterialCacheHelper::Tick ran out of budget in ProcessMaterials. Time Budget: %f", RemainingTimeBudget);
			break;
		}
	}

	RemainingTimeBudget = EndBudgetTime - Private::GetCurrentTime();
}

void FMaterialCacheHelper::ProcessMaterialsCaching()
{
	if (RemainingTimeBudget <= 0.f)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialCacheHelper::ProcessCachingMaterials);

	// Time when budget runs out
	const double StartTime = Private::GetCurrentTime();

	for (TMap<FMaterialKey, FMaterialData>::TIterator Iter(MaterialsCaching); Iter; ++Iter)
	{
		const FMaterialKey& MaterialKey = Iter.Key();
		FMaterialData& MaterialData = Iter.Value();

		if (!MaterialData.Material)
		{
			Iter.RemoveCurrent();
			continue;
		}

		const FResolvedShaderProfile* ShaderProfile = Private::FindShaderProfile(MaterialKey.ShaderProfile);
		if (!ShaderProfile)
		{
			Iter.RemoveCurrent();
			continue;
		}

		if (Private::IsMaterialComplete(MaterialData.Material, ShaderProfile))
		{
			Iter.RemoveCurrent();
		}
	}

	RemainingTimeBudget -= (Private::GetCurrentTime() - StartTime);
}

void FMaterialCacheHelper::RemoveCompletedMaterials()
{
	if (RemainingTimeBudget <= 0.f)
	{
		return;
	}

	const double StartTime = Private::GetCurrentTime();

	for (TPair<FObjectKey, FObjectData>& Object : Objects)
	{
		for (TSet<FMaterialKey>::TIterator MaterialKeyIter(Object.Value.IncompleteMaterialKeys); MaterialKeyIter; ++MaterialKeyIter)
		{
			if (!IsCachingMaterial(*MaterialKeyIter))
			{
				MaterialKeyIter.RemoveCurrent();
			}
		}
	}

	RemainingTimeBudget -= (Private::GetCurrentTime() - StartTime);
}

void FMaterialCacheHelper::ProcessShadersCompiling()
{
	if (RemainingTimeBudget <= 0.f || !GShaderCompilingManager || !GShaderCompilingManager->IsCompiling())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialCacheHelper::ProcessShadersCompiling);

	const double StartTime = Private::GetCurrentTime();

	constexpr bool bBlockOnGlobalShaderCompletion = false;
	GShaderCompilingManager->ProcessAsyncResults(RemainingTimeBudget, bBlockOnGlobalShaderCompletion);

	RemainingTimeBudget -= (Private::GetCurrentTime() - StartTime);
}

bool FMaterialCacheHelper::GatherMaterialData(const UObject* InObject, FObjectData& InObjectData, FName InShaderProfile)
{
	if (!InObject)
	{
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FMaterialCacheHelper::GatherMaterialData);

	const FMaterialBridge* MaterialBridge = FMaterialBridgeRegistry::Get().GetMaterialBridge(FConstDataView(InObject));
	if (!MaterialBridge)
	{
		UE_LOGF(LogAvaMaterialCache, VeryVerbose, "Object '%ls' (Class: %ls) does not have a valid bridge to cache materials!"
			, *InObject->GetName()
			, *GetNameSafe(InObject->GetClass()));
		return false;
	}

	InObjectData.ShaderProfile = InShaderProfile;

	auto AddIncompleteMaterial =
		[&MaterialsPendingCache = MaterialsPendingCache, &InObjectData, InShaderProfile](TNotNull<UMaterialInterface*> InMaterial)
		{
			FMaterialKey MaterialKey;
			MaterialKey.Material = InMaterial;
			MaterialKey.ShaderProfile = InShaderProfile;

			FMaterialData& MaterialData = MaterialsPendingCache.FindOrAdd(MaterialKey);
			MaterialData.Material = InMaterial;
			InObjectData.IncompleteMaterialKeys.Add(MaterialKey);
		};

	// Options to not wait for the container to have materials available and instead get a callback
	FMaterialBridgeReadSlotOptions ReadSlotOptions;
	ReadSlotOptions.bTrySkipWaitOnContainerCompletion = true;
	ReadSlotOptions.OnContainerPendingCompletion.BindLambda([&InObjectData](const FMaterialBridgeReadSlotContext& InContext)->EControlFlow
		{
			if (const UObject* Container = InContext.MaterialContainer.GetPtr<const UObject>())
			{
				InObjectData.PendingSubobjects.Add(Container);
			}
			else
			{
				ensureMsgf(0, TEXT("Container of type '%s' not supported for deferring material gather"), *GetNameSafe(InContext.MaterialContainer.GetStruct()));
			}
			return EControlFlow::Continue;
		});

	// Gather all materials for object
	MaterialBridge->AccessSlots(FMaterialBridgeReadSlotContext(InObject),
		[&AddIncompleteMaterial](const FMaterialBridgeReadSlotContext& InContext, const FMaterialBridgeReadSlot& InSlot)->EControlFlow
		{
			UMaterialInterface* const SlotMaterial = InSlot.GetMaterial();
			if (!SlotMaterial)
			{
				return EControlFlow::Continue;
			}

			UMaterial* const Material = SlotMaterial->GetMaterial();

			// Add interface's material if it is different from the interface
			if (Material && Material != SlotMaterial && !Material->IsComplete())
			{
				AddIncompleteMaterial(Material);
			}
			if (!SlotMaterial->IsComplete())
			{
				AddIncompleteMaterial(SlotMaterial);
			}
			return EControlFlow::Continue;
		}
		, ReadSlotOptions);

	return true;
}

} // UE::Ava
