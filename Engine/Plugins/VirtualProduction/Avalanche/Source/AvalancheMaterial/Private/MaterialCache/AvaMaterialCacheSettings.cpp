// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialCache/AvaMaterialCacheSettings.h"
#include "MaterialCache/AvaMaterialCacheLog.h"
#include "Shader.h"
#include "UObject/UnrealType.h"
#include "VertexFactory.h"

const FLazyName UAvaMaterialCacheSettings::DefaultRealtimeProfile(TEXT("BasicSceneRealtime"));
const FLazyName UAvaMaterialCacheSettings::DefaultOfflineProfile(TEXT("BasicSceneOffline"));

namespace UE::Ava::Private
{

TArray<FAvaShaderCombination> MakeDefaultShaderCombinations()
{
	// Vertex Factories
	const FName LocalVertexFactory       = TEXT("FLocalVertexFactory");
	const FName NiagaraMeshVertexFactory = TEXT("FNiagaraMeshVertexFactory");

	// Pipelines
	const FName StandardVelocityPipeline = TEXT("StandardVelocityPipeline");
	const FName DepthPipeline            = TEXT("DepthPipeline");

	// Shaders
	const FName BasePassVS               = TEXT("TBasePassVSFNoLightMapPolicy");
	const FName BasePassPS               = TEXT("TBasePassPSFNoLightMapPolicy");
	const FName NiagaraShader            = TEXT("FNiagaraShader");
	const FName MaterialCHS              = TEXT("TMaterialCHSFNoLightMapPolicyFAnyHitShader");
	const FName LumenCardVS              = TEXT("FLumenCardVS");
	const FName LumenCardPS              = TEXT("FLumenCardPS");
	const FName VelocityVS               = TEXT("TVelocityVS<EVelocityPassMode::Velocity_Standard>");
	const FName VelocityPS               = TEXT("TVelocityPS<EVelocityPassMode::Velocity_Standard>");
	const FName DepthOnlyVS              = TEXT("TDepthOnlyVS<false>");
	const FName DepthOnlyPS              = TEXT("FDepthOnlyPS");
	const FName ShadowDepthVS            = TEXT("TShadowDepthVSCombined");
	const FName ShadowDepthPS            = TEXT("TShadowDepthPSCombined");

	return 
	{
		FAvaShaderCombination(LocalVertexFactory,
			{
				StandardVelocityPipeline,
				DepthPipeline,
			},
			{
				BasePassVS,
				BasePassPS,
				NiagaraShader,
				MaterialCHS,
				LumenCardVS,
				LumenCardPS,
				VelocityVS,
				VelocityPS,
				DepthOnlyVS,
				DepthOnlyPS,
				ShadowDepthVS,
				ShadowDepthPS,
			}
		),
		FAvaShaderCombination(NiagaraMeshVertexFactory,
			{
				StandardVelocityPipeline
			},
			{
				BasePassVS,
				BasePassPS,
				ShadowDepthVS,
				ShadowDepthPS,
				VelocityVS,
				VelocityPS,
			}
		)
	};
}

} // UE::Ava::Private

UAvaMaterialCacheSettings::UAvaMaterialCacheSettings()
{
	RealtimeProfile = UAvaMaterialCacheSettings::DefaultRealtimeProfile;
	OfflineProfile = UAvaMaterialCacheSettings::DefaultOfflineProfile;

	CategoryName = TEXT("Motion Design");
	SectionName  = TEXT("Material Cache");
	RegisterDefaultShaderProfile();
}

const UE::Ava::FResolvedShaderProfile* UAvaMaterialCacheSettings::FindResolvedShaderProfile(FName InProfileName) const
{
	return ResolvedShaderProfiles.Find(InProfileName);
}

void UAvaMaterialCacheSettings::PostInitProperties()
{
	Super::PostInitProperties();
	ResolveShaderProfiles();
}

void UAvaMaterialCacheSettings::PostReloadConfig(FProperty* InPropertyThatWasLoaded)
{
	Super::PostReloadConfig(InPropertyThatWasLoaded);
	ResolveShaderProfiles();
}

#if WITH_EDITOR
void UAvaMaterialCacheSettings::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UAvaMaterialCacheSettings, ShaderProfiles))
	{
		ResolveShaderProfiles();
	}
}
#endif

void UAvaMaterialCacheSettings::RegisterDefaultShaderProfile()
{
	const TArray<FAvaShaderCombination> DefaultShaderCombinations = UE::Ava::Private::MakeDefaultShaderCombinations();

	ShaderProfiles.Add(
		{
			.ProfileName = UAvaMaterialCacheSettings::DefaultRealtimeProfile,
			.ProfileType = EAvaShaderProfileType::Specific,
			.ShaderCombinations = DefaultShaderCombinations,
		});

	ShaderProfiles.Add(
		{
			.ProfileName = UAvaMaterialCacheSettings::DefaultOfflineProfile,
			.ProfileType = EAvaShaderProfileType::Specific,
			.ShaderCombinations = DefaultShaderCombinations,
		});
}

void UAvaMaterialCacheSettings::ResolveShaderProfiles()
{
	UE_LOGF(LogAvaMaterialCache, Verbose, "Resolving Shader Profiles");

	ResolvedShaderProfiles.Empty(ShaderProfiles.Num());

	TMap<FName, FHashedName> HashedNames;
	for (const FAvaShaderProfile& ShaderProfile : ShaderProfiles)
	{
		ResolveShaderProfile(ShaderProfile, HashedNames);
	}
}

void UAvaMaterialCacheSettings::ResolveShaderProfile(const FAvaShaderProfile& InShaderProfile, TMap<FName, FHashedName>& InOutHashedNames)
{
	if (ResolvedShaderProfiles.Contains(InShaderProfile.ProfileName))
	{
		UE_LOGF(LogAvaMaterialCache, Verbose, "Shader Profile '%ls' already exists. New entry will be ignored.", *InShaderProfile.ProfileName.ToString());
		return;
	}

	UE::Ava::FResolvedShaderProfile& ResolvedShaderProfile = ResolvedShaderProfiles.Add(InShaderProfile.ProfileName);
	ResolvedShaderProfile.ProfileType = InShaderProfile.ProfileType;

	// Keep track of FHashedName instances to avoid recomputation
	auto GetHashedName = [&InOutHashedNames](FName InName)
		{
			if (const FHashedName* HashedName = InOutHashedNames.Find(InName))
			{
				return *HashedName;
			}
			return InOutHashedNames.Add(InName, InName);
		};

	int32 ReserveCount = 0;
	for (const FAvaShaderCombination& ShaderCombination : InShaderProfile.ShaderCombinations)
	{
		ReserveCount += ShaderCombination.PipelineTypes.Num();
		ReserveCount += ShaderCombination.ShaderTypes.Num();
	}

	// The types must all have the same count at the end
	ResolvedShaderProfile.VertexFactoryTypes.Reserve(ReserveCount);
	ResolvedShaderProfile.PipelineTypes.Reserve(ReserveCount);
	ResolvedShaderProfile.ShaderTypes.Reserve(ReserveCount);

	for (const FAvaShaderCombination& ShaderCombination : InShaderProfile.ShaderCombinations)
	{
		const FVertexFactoryType* const VertexFactoryType = FVertexFactoryType::GetVFByName(GetHashedName(ShaderCombination.VertexFactoryType));
		if (!VertexFactoryType)
		{
			UE_LOGF(LogAvaMaterialCache, Verbose, "Invalid Shader Config. Vertex Factory Type '%ls' was not found", *ShaderCombination.VertexFactoryType.ToString());
			continue;
		}

		for (FName PipelineTypeName : ShaderCombination.PipelineTypes)
		{
			const FShaderPipelineType* PipelineType = FShaderPipelineType::GetShaderPipelineTypeByName(GetHashedName(PipelineTypeName));
			if (!PipelineType)
			{
				UE_LOGF(LogAvaMaterialCache, Verbose, "Invalid Shader Config. Pipeline Type '%ls' was found", *PipelineTypeName.ToString());
				continue;
			}

			ResolvedShaderProfile.VertexFactoryTypes.Add(VertexFactoryType);
			ResolvedShaderProfile.PipelineTypes.Add(PipelineType);
			ResolvedShaderProfile.ShaderTypes.Add(nullptr);
		}

		for (FName ShaderTypeName : ShaderCombination.ShaderTypes)
		{
			const FShaderType* ShaderType = FindShaderTypeByName(GetHashedName(ShaderTypeName));
			if (!ShaderType)
			{
				UE_LOGF(LogAvaMaterialCache, Verbose, "Invalid Shader Config. Shader Type '%ls' was not found", *ShaderTypeName.ToString());
				continue;
			}

			ResolvedShaderProfile.VertexFactoryTypes.Add(VertexFactoryType);
			ResolvedShaderProfile.PipelineTypes.Add(nullptr);
			ResolvedShaderProfile.ShaderTypes.Add(ShaderType);
		}
	}

	ensureMsgf(ResolvedShaderProfile.VertexFactoryTypes.Num() == ResolvedShaderProfile.PipelineTypes.Num()
		&& ResolvedShaderProfile.VertexFactoryTypes.Num() == ResolvedShaderProfile.ShaderTypes.Num()
		, TEXT("Resolved Shader Profile '%s' unexpectedly has mismatched number of types.")
		, *InShaderProfile.ProfileName.ToString());
}
