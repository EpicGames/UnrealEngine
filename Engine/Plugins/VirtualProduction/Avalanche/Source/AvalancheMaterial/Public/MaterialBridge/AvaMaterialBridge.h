// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaDataView.h"
#include "AvaEnums.h"
#include "Containers/ContainersFwd.h"
#include "StructUtils/StructView.h"
#include "Templates/SubScriptStructOf.h"
#include "UObject/ObjectPtr.h"

#define UE_API AVALANCHEMATERIAL_API

struct FAvaMaterialBridgeSlotId;
struct FAvaMaterialContainerState;

namespace UE::Ava
{
	class FMaterialBridgeApplyStateContext;
	class FMaterialBridgeReadSlot;
	class FMaterialBridgeReadSlotContext;
	class FMaterialBridgeStoreStateContext;
	class FMaterialBridgeWriteSlot;
	class FMaterialBridgeWriteSlotContext;
	struct FMaterialBridgeApplyStateOptions;
	struct FMaterialBridgeReadSlotOptions;
	struct FMaterialBridgeStoreStateOptions;
	struct FMaterialBridgeWriteSlotOptions;
}

namespace UE::Ava
{
/**
 * Base class that allows interacting with material containers from any type (Actors, ActorComponents, etc.). 
 * A Material Bridge allows:
 *
 * 1) Access material slots: 
 *    - Access can either be to read or write. 
 *    - Read access allows only getting the material
 *    - Write access allows getting the material slot and replacing its material.
 *
 * 2) Apply and store material container state
 *    - Prior to replacing materials in a slot, the state of a material container can be stored via StoreState.
 *    - The Material Bridge can then revert the material container back to its original state via ApplyState with the data that was stored prior to change.
 *    - The expected flow could look like the following:
 *        1) Call StoreState. Caller owns the state data.
 *        2) Call AccessSlots (with write context) and replace slot material with a new one.
 *        3) When needing to revert back, call ApplyState with the state data saved from (1)
 *
 * Things to note:
 * - The Material Bridge does not own material container state data.
 *   It can only create and interact with it via Apply/Store.
 *   The responsibility of the ownership lies with the caller.
 *
 * - ApplyState and StoreState is only concerned with the state of the material container, not the material itself.
 *   For example the parameters of a Material Instance Dynamic or any other property within a material object are not considered.
 */
class FMaterialBridge
{
public:
	using FSlotId            = FAvaMaterialBridgeSlotId;
	using FReadSlot          = FMaterialBridgeReadSlot;
	using FReadSlotOptions   = FMaterialBridgeReadSlotOptions;
	using FReadSlotContext   = FMaterialBridgeReadSlotContext;
	using FWriteSlot         = FMaterialBridgeWriteSlot;
	using FWriteSlotOptions  = FMaterialBridgeWriteSlotOptions;
	using FWriteSlotContext  = FMaterialBridgeWriteSlotContext;
	using FApplyStateOptions = FMaterialBridgeApplyStateOptions;
	using FApplyStateContext = FMaterialBridgeApplyStateContext;
	using FStoreStateOptions = FMaterialBridgeStoreStateOptions;
	using FStoreStateContext = FMaterialBridgeStoreStateContext;

	virtual ~FMaterialBridge() = default;

	/** Initializes the bridge for use */
	void Initialize(uint32 InPriority);

	/** Retrieves the type that this bridge supports */
	UE_API const UStruct* GetBridgedType() const;

	/** Get the priority of this bridge in relation to other bridges of the same bridged type */
	uint32 GetPriority() const;

	/**
	 * Whether this Material Bridge supports the given Material Container.
	 * @param InMaterialContainer a valid material container of the supported type of this material bridge.
	 * @return true if this material bridge supports the container, false otherwise.
	 */
	UE_API bool IsMaterialContainerSupported(FConstDataView InMaterialContainer) const;

	/**
	 * Access the found material slots in the given context for read.
	 * @param InContext the current context to access materials
	 * @param InFunc function to call for each material slot accessed.
	 * @param InOptions access options. see FMaterialBridgeReadSlotAccessOptions
	 * @return the iteration result of AccessSlots.
	 */
	UE_API EControlFlow AccessSlots(const FReadSlotContext& InContext, TFunctionRef<EControlFlow(const FReadSlotContext&, const FReadSlot&)> InFunc, const FReadSlotOptions& InOptions) const;

	/**
	 * Access the found material slots in the given context for write.
	 * @param InContext the current context to access materials.
	 * @param InFunc function to call for each material slot accessed.
	 * @param InOptions access options. see FMaterialBridgeWriteSlotAccessOptions
	 * @return the iteration result of AccessSlots.
	 */
	UE_API EControlFlow AccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const;

	/** Returns whether this material bridge supports container states */
	UE_API bool CanCreateContainerState() const;

	/** Creates a material container state instance that can be used for this material bridge */
	UE_API TInstancedStruct<FAvaMaterialContainerState> CreateContainerState() const;

	/** 
	 * Applies the material container state to the context object.
	 * @param InContext context used when applying the container state. see FMaterialBridgeApplyStateContext
	 * @param InContainerState const view of the container state. Required to match the container state type of this bridge.
	 * @param InOptions additional options to how apply should take place. see FMaterialBridgeApplyStateOptions
	 * @return true if the container state was valid and was processed
	 */
	UE_API bool ApplyState(const FApplyStateContext& InContext, TConstStructView<FAvaMaterialContainerState> InContainerState, const FApplyStateOptions& InOptions) const;

	/** 
	 * Stores the state of the context object in the material container state
	 * This overload initializes the container state if it is invalid.
	 * @param InContext context used when storing the container state. see FMaterialBridgeStoreStateContext
	 * @param InOutContainerState the instanced struct where the container state will be stored
	 * @param InOptions additional options to how store should take place. see FMaterialBridgeStoreStateOptions
	 * @return true if the container state was valid (or initialized to valid) and was processed
	 */
	UE_API bool StoreState(const FStoreStateContext& InContext, TNotNull<TInstancedStruct<FAvaMaterialContainerState>*> InOutContainerState, const FStoreStateOptions& InOptions) const;

	/** 
	 * Stores the state of the context object in the material container state
	 * @param InContext context used when storing the container state. see FMaterialBridgeStoreStateContext
	 * @param InContainerState the container state to store to. Required to match the container state type of this bridge.
	 * @param InOptions additional options to how store should take place. see FMaterialBridgeStoreStateOptions
	 * @return true if the container state was valid and was processed
	 */
	UE_API bool StoreState(const FStoreStateContext& InContext, TStructView<FAvaMaterialContainerState> InContainerState, const FStoreStateOptions& InOptions) const;

protected:
	/** Retrieves the type that this bridge supports. Only called once in Initialize() */
	virtual const UStruct* OnGetBridgedType() const = 0;

	/**  Whether this material bridge supports the given material container.*/
	virtual bool OnIsMaterialContainerSupported(FConstDataView InMaterialContainer) const
	{
		return true;
	}

	/** Accesses the found material slots in the given context for read */
	virtual EControlFlow OnAccessSlots(const FReadSlotContext& InContext, TFunctionRef<EControlFlow(const FReadSlotContext&, const FReadSlot&)> InFunc, const FReadSlotOptions& InOptions) const
	{
		return EControlFlow::Continue;
	}

	/** Accesses the found material slots in the given context for write */
	virtual EControlFlow OnAccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const
	{
		return EControlFlow::Continue;
	}

	/** Returns the material container state type required by this bridge, if any */
	virtual TSubScriptStructOf<FAvaMaterialContainerState> OnGetContainerStateType() const
	{
		return TSubScriptStructOf<FAvaMaterialContainerState>();
	}

	/** Applies the container state to the context object */
	virtual void OnApplyState(const FApplyStateContext& InContext, TConstStructView<FAvaMaterialContainerState> InContainerState, const FApplyStateOptions& InOptions) const
	{
	}

	/** Stores the state of the context object in the container state */
	virtual void OnStoreState(const FStoreStateContext& InContext, TStructView<FAvaMaterialContainerState> InContainerState, const FStoreStateOptions& InOptions) const
	{
	}

private:
	/** The type that this Material Bridge works on */
	TObjectPtr<const UStruct> BridgedType;

	/** The material container state type of this bridge, if any */
	TSubScriptStructOf<FAvaMaterialContainerState> ContainerStateType;

	/** The priority of this bridge in relation to other bridges of the same bridged type */
	uint32 Priority = 0;
};

} // UE::Ava

#undef UE_API
