// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/DerivedFrom.h"
#include "Containers/Map.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/ObjectKey.h"

#define UE_API AVALANCHEMATERIAL_API

namespace UE::Ava
{
	class FMaterialBridge;
	struct FConstDataView;
}

namespace UE::Ava
{

/** Holds all the Material Bridges to use */
class FMaterialBridgeRegistry
{
public:
	UE_API static const FMaterialBridgeRegistry& Get();
	UE_API static FMaterialBridgeRegistry& GetMutable();

	template<typename InMaterialBridgeType, typename... InArgTypes> requires (UE::CDerivedFrom<InMaterialBridgeType, UE::Ava::FMaterialBridge>)
	void Register(uint32 InPriority, InArgTypes&&... InArgs)
	{
		this->RegisterInternal(MakeUnique<InMaterialBridgeType>(InArgs...), InPriority);
	}

	/** Tries to find the most relevant material bridge for a given material container, returning a null if object is invalid, or if a relevant material bridge was not found. */
	UE_API const FMaterialBridge* GetMaterialBridge(FConstDataView InMaterialContainer) const;

protected:
	/** Initializes the material bridge and adds it to this registry */
	UE_API void RegisterInternal(TUniquePtr<UE::Ava::FMaterialBridge>&& InMaterialBridge, uint32 InPriority);

private:
	/** Registered material bridges by their supported type */
	TMap<TObjectKey<UStruct>, TArray<TUniquePtr<FMaterialBridge>>> RegisteredMaterialBridges;
};

} // UE::Ava

#undef UE_API
