// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMaterialContainerState.generated.h"

/** Base struct for all material container states */
USTRUCT()
struct FAvaMaterialContainerState
{
	GENERATED_BODY()
};
