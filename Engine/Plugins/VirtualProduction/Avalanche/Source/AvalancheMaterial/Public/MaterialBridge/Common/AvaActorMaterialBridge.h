// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialBridge/AvaMaterialBridge.h"
#include "MaterialBridge/AvaMaterialContainerState.h"
#include "AvaActorMaterialBridge.generated.h"

#define UE_API AVALANCHEMATERIAL_API

class UActorComponent;

/** Struct holding the path for each actor component and the container state data mapped to it  */
USTRUCT()
struct FAvaActorMaterialContainerStateComponent
{
	GENERATED_BODY()

	/** Sets the main component of this material container state */
	UE_API void SetComponent(UActorComponent* InComponent);

	/** Tries to resolve component by first resolving the weak ptr, or else using the component path and the given actor */
	UE_API UActorComponent* ResolveComponent(const AActor* InActor) const;

	/** Path to the actor component */
	UPROPERTY()
	FString Path;

	/** Resolved component */
	mutable TWeakObjectPtr<UActorComponent> ComponentWeak;

	/** The container state data for the component. The type varies as each component could have a different Material Bridge */
	UPROPERTY()
	TInstancedStruct<FAvaMaterialContainerState> ContainerState;
};

/** Material container state for actors */
USTRUCT()
struct FAvaActorMaterialContainerState : public FAvaMaterialContainerState
{
	GENERATED_BODY()

	/** All the actor component data found for the actor mapped to this data */
	UPROPERTY()
	TArray<FAvaActorMaterialContainerStateComponent> Components;
};

namespace UE::Ava
{

/** Material Bridge for Actors */
class FActorMaterialBridge : public FMaterialBridge
{
public:
	using FContainerState = FAvaActorMaterialContainerState;

protected:
	//~ Begin FMaterialBridge
	UE_API virtual const UStruct* OnGetBridgedType() const override;
	UE_API virtual EControlFlow OnAccessSlots(const FReadSlotContext& InContext, TFunctionRef<EControlFlow(const FReadSlotContext&, const FReadSlot&)> InFunc, const FReadSlotOptions& InOptions) const override;
	UE_API virtual EControlFlow OnAccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const override;
	UE_API virtual TSubScriptStructOf<FAvaMaterialContainerState> OnGetContainerStateType() const override;
	UE_API virtual void OnApplyState(const FApplyStateContext& InContext, TConstStructView<FAvaMaterialContainerState> InContainerState, const FApplyStateOptions& InOptions) const override;
	UE_API virtual void OnStoreState(const FStoreStateContext& InContext, TStructView<FAvaMaterialContainerState> InContainerState, const FStoreStateOptions& InOptions) const override;
	//~ End FMaterialBridge
};

} // UE::Ava

#undef UE_API
