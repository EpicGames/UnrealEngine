// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialBridge/AvaMaterialBridge.h"
#include "MaterialBridge/AvaMaterialContainerState.h"
#include "AvaDecalComponentMaterialBridge.generated.h"

#define UE_API AVALANCHEMATERIAL_API

class UMaterialInterface;

/** Material container state for decal components */
USTRUCT()
struct FAvaDecalComponentMaterialContainerState : public FAvaMaterialContainerState
{
	GENERATED_BODY()

	/** The decal material of the component */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> DecalMaterial;
};

namespace UE::Ava
{

/** Material Bridge for Decal Components */
class FDecalComponentMaterialBridge : public FMaterialBridge
{
public:
	using FContainerState = FAvaDecalComponentMaterialContainerState;

protected:
	//~ Begin FMaterialBridge
	UE_API virtual const UStruct* OnGetBridgedType() const override;
	UE_API virtual EControlFlow OnAccessSlots(const FReadSlotContext& InContext, TFunctionRef<EControlFlow(const FReadSlotContext&, const FReadSlot&)> InFunc, const FReadSlotOptions& InOptions) const override;
	UE_API virtual EControlFlow OnAccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const override;
	UE_API virtual TSubScriptStructOf<FAvaMaterialContainerState> OnGetContainerStateType() const override;
	UE_API virtual void OnApplyState(const FApplyStateContext& InContext, TConstStructView<FAvaMaterialContainerState> InContainerState, const FApplyStateOptions& InOptions) const override;
	UE_API virtual void OnStoreState(const FStoreStateContext& InContext, TStructView<FAvaMaterialContainerState> InContainerState, const FStoreStateOptions& InOptions) const override;
	//~ End FMaterialBridge
};

} // UE::Ava

#undef UE_API
