// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialBridge/AvaMaterialBridge.h"

#define UE_API AVALANCHEMATERIAL_API

namespace UE::Ava
{

/**
 * Material Bridge for Levels
 *
 * NOTE: Does not support Material Container State as it's probably better for users to do it per actor on-demand than to store/instantiate an entire level's material data.
 * If it's ever needed, functionality to do this incrementally would be needed so as not to stall on one frame while creating large amounts of data.
 */
class FLevelMaterialBridge : public FMaterialBridge
{
protected:
	//~ Begin FMaterialBridge
	UE_API virtual const UStruct* OnGetBridgedType() const override;
	UE_API virtual EControlFlow OnAccessSlots(const FReadSlotContext& InContext, TFunctionRef<EControlFlow(const FReadSlotContext&, const FReadSlot&)> InFunc, const FReadSlotOptions& InOptions) const override;
	UE_API virtual EControlFlow OnAccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const override;
	//~ End FMaterialBridge
};

} // UE::Ava

#undef UE_API
