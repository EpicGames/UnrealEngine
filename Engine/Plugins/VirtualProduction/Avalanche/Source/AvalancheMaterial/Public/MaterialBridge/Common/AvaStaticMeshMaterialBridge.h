// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialBridge/AvaMaterialBridge.h"

#define UE_API AVALANCHEMATERIAL_API

namespace UE::Ava
{

/**
 * Material Bridge for Static Meshes
 * Only deals with the Static Mesh's static materials.
 */
class FStaticMeshMaterialBridge : public FMaterialBridge
{
protected:
	//~ Begin FMaterialBridge
	UE_API virtual const UStruct* OnGetBridgedType() const override;
	UE_API virtual EControlFlow OnAccessSlots(const FReadSlotContext& InContext, TFunctionRef<EControlFlow(const FReadSlotContext&, const FReadSlot&)> InFunc, const FReadSlotOptions& InOptions) const override;
	UE_API virtual EControlFlow OnAccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const override;
	//~ End FMaterialBridge
};

} // UE::Ava

#undef UE_API
