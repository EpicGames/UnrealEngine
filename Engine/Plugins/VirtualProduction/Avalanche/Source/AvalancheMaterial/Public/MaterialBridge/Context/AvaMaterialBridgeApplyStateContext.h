// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMaterialBridgeContext.h"

namespace UE::Ava
{
	class FMaterialBridgeApplyStateContext;
}

namespace UE::Ava
{

/** Additional options when applying state to a material container  */
struct FMaterialBridgeApplyStateOptions : public FMaterialBridgeBaseOptions
{
};

template<>
struct TMaterialBridgeContextFlags<FMaterialBridgeApplyStateContext> : public TMaterialBridgeContextFlagsBase<FMaterialBridgeApplyStateContext>
{
	enum
	{
		WithConstMaterialContainer = false,
	};
};

/** Short-lived context information when applying state to a material container */
class FMaterialBridgeApplyStateContext : public TMaterialBridgeContext<FMaterialBridgeApplyStateContext>
{
public:
	using TMaterialBridgeContext<FMaterialBridgeApplyStateContext>::TMaterialBridgeContext;
};

} // UE::Ava
