// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaDataView.h"

#define UE_API AVALANCHEMATERIAL_API

class UObject;

namespace UE::Ava
{
	class FMaterialBridgeRegistry;
}

namespace UE::Ava
{

/** Base for all options structs */
struct FMaterialBridgeBaseOptions
{
	UE_API FMaterialBridgeBaseOptions();

	/** The registry used to get material bridges. Defaults to the global registry. */
	TNotNull<const FMaterialBridgeRegistry*> MaterialBridgeRegistry;
};

template<typename InContextType>
struct TMaterialBridgeContextFlagsBase
{
	enum
	{
		WithConstMaterialContainer = true,
	};
};

template<typename InContextType>
struct TMaterialBridgeContextFlags : public TMaterialBridgeContextFlagsBase<InContextType>
{
};

/**  Base context class containing common functionality shared among all material bridge contexts */
template<typename InContextType>
class TMaterialBridgeContext
{
public:
	static constexpr bool bWithConstMaterialContainer = TMaterialBridgeContextFlags<InContextType>::WithConstMaterialContainer;

	using UObjectType   = std::conditional_t<bWithConstMaterialContainer, const UObject, UObject>;
	using FDataViewType = std::conditional_t<bWithConstMaterialContainer, FConstDataView, FDataView>;

	TMaterialBridgeContext() = default;

	UE_API explicit TMaterialBridgeContext(FDataViewType InMaterialContainer);
	UE_API explicit TMaterialBridgeContext(FDataViewType InMaterialContainer, TNotNull<const InContextType*> InParentContext);
	UE_API explicit TMaterialBridgeContext(UObjectType* InMaterialContainer);
	UE_API explicit TMaterialBridgeContext(UObjectType* InMaterialContainer, TNotNull<const InContextType*> InParentContext);

	/** Gets the topmost slot context in the context chain */
	UE_API const InContextType& GetTopmostContext() const;

	/** Gets the first material container that is a UObject */
	UE_API UObjectType* GetMaterialContainerObject() const;

	/** The object containing the material */
	FDataViewType MaterialContainer;

	/** Parent context. Non-null if in a nested call. Should always be valid checked */
	const InContextType* ParentContext = nullptr;
};

} // UE::Ava

#undef UE_API
