// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaEnums.h"
#include "AvaMaterialBridgeContext.h"

namespace UE::Ava
{
	class FMaterialBridgeReadSlotContext;

} // UE::Ava

namespace UE::Ava
{

/** Options to how read access should take place */
struct FMaterialBridgeReadSlotOptions : public FMaterialBridgeBaseOptions
{
	/** Delegate called when a container is not yet complete and cannot access its materials. Only called when bTrySkipWaitOnContainerCompletion is true. */
	TDelegate<EControlFlow(const FMaterialBridgeReadSlotContext&)> OnContainerPendingCompletion;

	/**
	 * Whether to try to skip wait for container completion to access its materials.
	 * 'Try' because there is no guarantee that all material containers have an API that determines whether accessing a material will cause a wait.
	 */
	bool bTrySkipWaitOnContainerCompletion = false;
};

/** Short-lived context information when accessing a material slot for read  */
class FMaterialBridgeReadSlotContext : public TMaterialBridgeContext<FMaterialBridgeReadSlotContext>
{
public:
	using TMaterialBridgeContext<FMaterialBridgeReadSlotContext>::TMaterialBridgeContext;
};

} // UE::Ava
