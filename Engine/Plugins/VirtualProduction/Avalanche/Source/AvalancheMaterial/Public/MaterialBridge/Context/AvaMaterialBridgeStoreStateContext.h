// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMaterialBridgeContext.h"

namespace UE::Ava
{

/** Additional options when applying/storing material container state  */
struct FMaterialBridgeStoreStateOptions : public FMaterialBridgeBaseOptions
{
};

/** Short-lived context information when storing the state of a material container */
class FMaterialBridgeStoreStateContext : public TMaterialBridgeContext<FMaterialBridgeStoreStateContext>
{
public:
	using TMaterialBridgeContext<FMaterialBridgeStoreStateContext>::TMaterialBridgeContext;
};

} // UE::Ava
