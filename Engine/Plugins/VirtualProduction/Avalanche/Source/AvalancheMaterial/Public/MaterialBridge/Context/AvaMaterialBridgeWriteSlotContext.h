// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaEnums.h"
#include "AvaMaterialBridgeContext.h"

namespace UE::Ava
{
	class FMaterialBridgeWriteSlotContext;

} // UE::Ava

namespace UE::Ava
{

/** Options used when accessing a slot for write */
struct FMaterialBridgeWriteSlotOptions : public FMaterialBridgeBaseOptions
{
	/** Delegate called when a container is not yet complete and cannot access its materials. Only called when bTrySkipWaitOnContainerCompletion is true. */
	TDelegate<EControlFlow(const FMaterialBridgeWriteSlotContext&)> OnContainerPendingCompletion;

	/**
	 * Whether to try to skip wait for container completion to access its materials.
	 * 'Try' because there is no guarantee that all material containers have an API that determines whether accessing a material will cause a wait.
	 */
	bool bTrySkipWaitOnContainerCompletion = false;
};

template<>
struct TMaterialBridgeContextFlags<FMaterialBridgeWriteSlotContext> : public TMaterialBridgeContextFlagsBase<FMaterialBridgeWriteSlotContext>
{
	enum
	{
		WithConstMaterialContainer = false,
	};
};

/** Short-lived context information when accessing a material slot for write  */
class FMaterialBridgeWriteSlotContext : public TMaterialBridgeContext<FMaterialBridgeWriteSlotContext>
{
public:
	using TMaterialBridgeContext<FMaterialBridgeWriteSlotContext>::TMaterialBridgeContext;
};

} // UE::Ava
