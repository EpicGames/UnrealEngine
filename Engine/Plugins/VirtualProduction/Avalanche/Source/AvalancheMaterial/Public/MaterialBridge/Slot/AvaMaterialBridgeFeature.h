// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMaterialBridgeFeature.generated.h"

/** Base struct for all slot features */
USTRUCT()
struct FAvaMaterialBridgeFeature
{
	GENERATED_BODY()
};
