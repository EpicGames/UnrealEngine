// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMaterialBridgeSlot.h"

#define UE_API AVALANCHEMATERIAL_API


namespace UE::Ava
{

/** Short-lived representation of an accessed material slot for read */
class FMaterialBridgeReadSlot : public FMaterialBridgeSlot
{
public:
	using FMaterialBridgeSlot::FMaterialBridgeSlot;
};

} // UE::Ava

#undef UE_API
