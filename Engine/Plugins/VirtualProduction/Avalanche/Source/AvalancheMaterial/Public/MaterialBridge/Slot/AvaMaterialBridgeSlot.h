// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMaterialBridgeSlotId.h"
#include "UObject/ObjectPtr.h"

#define UE_API AVALANCHEMATERIAL_API

class UMaterialInterface;

namespace UE::Ava
{

/** Base slot class for both the read and write slot */
class FMaterialBridgeSlot
{
public:
	using FSlotId = FAvaMaterialBridgeSlotId;

	/** Constructor for when a material slot is indexed */
	UE_API explicit FMaterialBridgeSlot(UMaterialInterface* InMaterial, const FSlotId& InSlotId);

	virtual ~FMaterialBridgeSlot() = default;

	/** Gets the material for this slot */
	UMaterialInterface* GetMaterial() const
	{
		return Material;
	}

	/** Gets the identifier for this slot */
	const FSlotId& GetSlotId() const
	{
		return SlotId;
	}

protected:
	/** Material in this slot */
	TObjectPtr<UMaterialInterface> Material;

	/** Identifier for the slot within its container */
	FSlotId SlotId;
};

} // UE::Ava

#undef UE_API
