// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMaterialBridgeSlotId.generated.h"

#define UE_API AVALANCHEMATERIAL_API

/** Holds the data to identify a slot within a material container */
USTRUCT()
struct FAvaMaterialBridgeSlotId
{
	GENERATED_BODY()

	FAvaMaterialBridgeSlotId() = default;

	/** Constructor for when a material slot can be identified by its index */
	UE_API explicit FAvaMaterialBridgeSlotId(int32 InSlotIndex);

	/** Constructor for when a material slot can be identified by an FName */
	UE_API explicit FAvaMaterialBridgeSlotId(FName InSlotName);

	/** Constructor for when a material slot can be identified by a name and index */
	UE_API explicit FAvaMaterialBridgeSlotId(FName InSlotName, int32 InSlotIndex);

	/** Whether the slot id is valid (i.e. not the defaults) */
	UE_API bool IsValid() const;

	bool operator==(const FAvaMaterialBridgeSlotId& InOther) const
	{
		return Name == InOther.Name 
			&& Index == InOther.Index;
	}

	friend uint32 GetTypeHash(const FAvaMaterialBridgeSlotId& InSlotId)
	{
		return HashCombineFast(GetTypeHash(InSlotId.Name), GetTypeHash(InSlotId.Index));
	}

private:
	/** Slot name, if any */
	FName Name;

	/** Slot index, if any */
	int32 Index = INDEX_NONE;
};

#undef UE_API
