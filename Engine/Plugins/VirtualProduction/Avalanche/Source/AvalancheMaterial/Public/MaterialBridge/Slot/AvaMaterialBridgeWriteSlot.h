// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMaterialBridgeSlot.h"
#include "StructUtils/StructView.h"

#define UE_API AVALANCHEMATERIAL_API

struct FAvaMaterialBridgeFeature;

namespace UE::Ava
{

/** Base struct for short-lived representation of an accessed material slot for write */
class FMaterialBridgeWriteSlot : public FMaterialBridgeSlot
{
public:
	using FMaterialBridgeSlot::FMaterialBridgeSlot;

	/** Sets the new material for the slot */
	UE_API void SetMaterial(UMaterialInterface* InMaterial);

	/** 
	 * Requests a feature for this material slot 
	 * @param InFeature the feature to request
	 * @return true if the feature was recognized by the slot
	 */
	UE_API bool RequestFeature(TConstStructView<FAvaMaterialBridgeFeature> InFeature);

protected:
	/** Sets the material property directly without calling the change notification */
	UE_API void SetMaterialInternal(UMaterialInterface* InMaterial);

	/** Notifies when a new material has been set for the slot */
	virtual void OnMaterialChanged()
	{
	}

	/**
	 * Requests a feature for this material slot
	 * @param InFeature the feature to request
	 * @return true if the feature was recognized by the slot
	 */
	virtual bool OnFeatureRequest(TConstStructView<FAvaMaterialBridgeFeature> InFeature)
	{
		return false;
	}
};

} // UE::Ava

#undef UE_API
