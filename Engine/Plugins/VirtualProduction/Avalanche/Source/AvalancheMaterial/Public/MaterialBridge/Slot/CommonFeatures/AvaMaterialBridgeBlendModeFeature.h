// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "MaterialBridge/Slot/AvaMaterialBridgeFeature.h"
#include "AvaMaterialBridgeBlendModeFeature.generated.h"

/** Feature representing the blend mode to configure for the slot */
USTRUCT()
struct FAvaMaterialBridgeBlendModeFeature : public FAvaMaterialBridgeFeature
{
	GENERATED_BODY()

	/** The desired blend mode for the slot */
	UPROPERTY()
	TEnumAsByte<EBlendMode> BlendMode = EBlendMode::BLEND_Opaque;
};
