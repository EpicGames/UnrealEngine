// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class AvalancheMedia : ModuleRules
{
	public AvalancheMedia(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"Messaging",
			});

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Avalanche",
				"AvalancheSequence",
				"AvalancheTag",
				"AvalancheTransition",
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"MediaIOCore",
				"RemoteControl",
				"RemoteControlLogic",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"AssetRegistry",
				"AvalancheCore",
				"AvalancheMaterial",
				"AvalancheRemoteControl",
				"HeadMountedDisplay",
				"Json",
				"JsonUtilities",
				"ImageCore",
				"InputCore",
				"Projects",
				"RHI",
				"RemoteControlCommon",
				"RenderCore",
				"Renderer",
				"Slate",
				"SlateCore",
				"StateTreeModule",
				"UMG",
				"Serialization",
				"XmlSerialization",
				"HTTPServer", 
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Messaging",
				"MessagingCommon",
			});

		if (Target.Type == TargetRules.TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"UnrealEd",
				"GraphEditor",
			});
		}

		if ((Target.IsInPlatformGroup(UnrealPlatformGroup.Windows)))
		{
			// Uses DXGI to query GPU hardware and Monitor enumeration.
			PublicSystemLibraries.Add("DXGI.lib");
		}
	}
}
