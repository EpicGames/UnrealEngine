// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::AvaMediaMessageUtils
{
	/**
	 * Some of the message bus bridge have limitations to the size of the messages.
	 * This function returns the current largest message size that can "safely" be sent
	 * through the message bus without causing (known) issues.
	 */
	AVALANCHEMEDIA_API uint32 GetSafeMessageSizeLimit();
}
