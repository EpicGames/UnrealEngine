// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/MathFwd.h"
#include "PixelFormat.h"

class FName;
class UTextureRenderTarget2D;
struct FLinearColor;
enum EPixelFormat : uint8;

namespace UE::AvaMediaRenderTargetUtils
{
	/** Creates a Render Target with default settings like bForceLinearGamma to be false. */
	AVALANCHEMEDIA_API UTextureRenderTarget2D* CreateDefaultRenderTarget(FName InBaseName);

	/** Update the render target with the given parameters. */
	AVALANCHEMEDIA_API void UpdateRenderTarget(UTextureRenderTarget2D* InRenderTarget, const FIntPoint& InSize, EPixelFormat InFormat, const FLinearColor& InClearColor);
	
	/**
	 * Setup the render target gamma appropriately depending on the format.
	 * @return true if the TargetGamma was modified, false otherwise.
	 */
	AVALANCHEMEDIA_API bool SetupRenderTargetGamma(UTextureRenderTarget2D* InRenderTarget);

	/**
	 * Returns the size of the render target.
	 * This version of the function doesn't ensure if the render target is not valid.
	 */ 
	AVALANCHEMEDIA_API FIntPoint GetRenderTargetSizeSafe(const UTextureRenderTarget2D* InRenderTarget);
	
	/** 
	 * Returns the size of the render target.
	 * An ensure will be fired if the render target is not valid.
	 */ 
	AVALANCHEMEDIA_API FIntPoint GetRenderTargetSize(const UTextureRenderTarget2D* InRenderTarget);

	/** Returns true if render target's format is floating point. */
	AVALANCHEMEDIA_API bool IsFloatFormat(const UTextureRenderTarget2D* InRenderTarget);

	/** Returns the number of color components for the given pixel format. */
	AVALANCHEMEDIA_API int32 GetNumColorComponents(EPixelFormat InFormat);

	/** Returns true if the given format has an alpha channel. */
	AVALANCHEMEDIA_API bool HasAlpha(EPixelFormat InFormat);

	/**
	 * Returns the number of bits per color component (assuming all the same) for the given pixel format.
	 * Known exceptions: PF_R5G6B5_UNORM, PF_FloatR11G11B10. But those are not render target formats.
	 */
	AVALANCHEMEDIA_API int32 GetNumColorChannelBits(EPixelFormat InFormat);

	/** Returns the number of bits for the alpha channel. */
	AVALANCHEMEDIA_API int32 GetNumAlphaChannelBits(EPixelFormat InFormat);
}
