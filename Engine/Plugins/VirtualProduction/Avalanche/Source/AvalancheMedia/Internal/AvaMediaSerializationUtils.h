// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Templates/Function.h"

class FProperty;
class FStructuredArchiveFormatter;
class UObject;

namespace UE::AvaMediaSerializationUtils
{
	/** Serialize the given object with the given formatter. */
	AVALANCHEMEDIA_API void SerializeObject(FStructuredArchiveFormatter& Formatter, UObject* InObject, const TFunction<bool(FProperty*)>& InShouldSkipProperty = TFunction<bool(FProperty*)>());

	/**
	 * The following functions are meant to be used to convert the serialized values of FJsonStruct(De)SerializerBackend
	 * to/from FString.
	 * 
	 * @remark FJsonStructSerializerBackend and FJsonStructDeserializerBackend are currently hardcoded to use UCS2CHAR.
	 */
	namespace JsonValueConversion
	{
		/** Converts a serialized value stored as raw bytes into a string. In place version. */
		AVALANCHEMEDIA_API void BytesToString(const TArray<uint8>& InValueAsBytes, FString& OutValueAsString);

		/** This ArrayView version is meant to work with FMemoryReaderView. */
		inline TArrayView<const uint8> ValueToConstBytesView(const FString& InValueAsString)
		{
			// Len() excludes the terminating character, and it is desired so it matches
			// what FJsonStructSerializerBackend does.
			return TArrayView<const uint8>(reinterpret_cast<const uint8*>(*InValueAsString), InValueAsString.Len() * sizeof(TCHAR));
		}
	}
}
