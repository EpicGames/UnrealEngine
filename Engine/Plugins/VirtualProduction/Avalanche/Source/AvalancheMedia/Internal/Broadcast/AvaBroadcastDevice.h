// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "MediaIOCoreDefinitions.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"

#define LOCTEXT_NAMESPACE "AvaBroadcastDevice"

struct FAvaBroadcastDevice
{
	FAvaBroadcastDevice(const FMediaIODevice& InDevice, const FName& InDeviceProviderName, const FString& InServerName)
		: Device(InDevice)
		, DeviceProviderName(InDeviceProviderName)
		, ServerName(InServerName)
	{
	}
	
	bool IsValid() const
	{
		return Device.DeviceName != NAME_None && Device.IsValid();
	}
	
	bool operator==(const FAvaBroadcastDevice& Other) const
	{
		return Device.DeviceName == Other.Device.DeviceName
			&& Device.DeviceIdentifier == Other.Device.DeviceIdentifier
			&& ServerName == Other.ServerName;
	}
	
	FText GetDisplayNameText() const
	{
		return FText::Format(LOCTEXT("AvaMediaDeviceDisplayName", "{0}")
			, FText::FromName(Device.DeviceName));
	}

	friend uint32 GetTypeHash(const FAvaBroadcastDevice& InDevice)
	{
		uint32 Hash = GetTypeHash(InDevice.Device.DeviceName);
		Hash = HashCombine(Hash, InDevice.Device.DeviceIdentifier);
		Hash = HashCombine(Hash, GetTypeHash(InDevice.ServerName));
		return Hash;
	}

	const FMediaIODevice& GetDevice() const { return Device; }
	const FName& GetDeviceProviderName() const { return DeviceProviderName;}
	const FString& GetServerName() const { return ServerName;}
	
protected:
	FMediaIODevice Device;

	/** Name of the device provider for this device. */
	FName DeviceProviderName;
	
	/** Name of the server hosting this device. */
	FString ServerName;
};

#undef LOCTEXT_NAMESPACE
