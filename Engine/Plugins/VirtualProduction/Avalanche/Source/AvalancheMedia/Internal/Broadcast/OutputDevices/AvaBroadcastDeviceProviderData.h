// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaIOCoreDeviceProvider.h"
#include "AvaBroadcastDeviceProviderData.generated.h"

USTRUCT()
struct AVALANCHEMEDIA_API FAvaBroadcastDeviceProviderData
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FName Name;

	UPROPERTY()
	TArray<FMediaIOConnection> Connections;

	UPROPERTY()
	TArray<FMediaIOConfiguration> Configurations;

	UPROPERTY()
	TArray<FMediaIODevice> Devices;

	UPROPERTY()
	TArray<FMediaIOMode> Modes;

	UPROPERTY()
	TArray<FMediaIOInputConfiguration> InputConfigurations;

	UPROPERTY()
	TArray<FMediaIOOutputConfiguration> OutputConfigurations;

	UPROPERTY()
	TArray<FMediaIOVideoTimecodeConfiguration> TimecodeConfigurations;
	
	UPROPERTY()
	FMediaIOConfiguration DefaultConfiguration;

	UPROPERTY()
	FMediaIOMode DefaultMode;

	UPROPERTY()
	FMediaIOInputConfiguration DefaultInputConfiguration;

	UPROPERTY()
	FMediaIOOutputConfiguration DefaultOutputConfiguration;

	UPROPERTY()
	FMediaIOVideoTimecodeConfiguration DefaultTimecodeConfiguration;
	
	UPROPERTY()
	bool bShowInputTransportInSelector = true;

	UPROPERTY()
	bool bShowOutputTransportInSelector = true;

	UPROPERTY()
	bool bShowInputKeyInSelector = true;

	UPROPERTY()
	bool bShowOutputKeyInSelector = true;

	UPROPERTY()
	bool bShowReferenceInSelector = true;

public:
	void PopulateFrom(const FName& InName, const IMediaIOCoreDeviceProvider* InDeviceProvider);
	void ApplyServerName(const FString& InServerName);
};

USTRUCT()
struct FAvaBroadcastDeviceProviderDataList
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString ServerName;
	
	UPROPERTY()
	TArray<FAvaBroadcastDeviceProviderData> DeviceProviders;

	bool SaveToJson();
	bool SaveToXml();
	bool LoadFromJson();
	void Populate(const FString& InAssignedServerName);
};