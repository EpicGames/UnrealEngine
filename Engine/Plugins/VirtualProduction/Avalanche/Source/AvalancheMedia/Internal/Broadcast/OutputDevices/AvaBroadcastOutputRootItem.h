// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaBroadcastOutputTreeItem.h"

class AVALANCHEMEDIA_API FAvaBroadcastOutputRootItem : public FAvaBroadcastOutputTreeItem
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaBroadcastOutputRootItem, FAvaBroadcastOutputTreeItem);

	FAvaBroadcastOutputRootItem()
		: Super(nullptr)
	{}

	//~ Begin IAvaBroadcastOutputTreeItem
	virtual void RefreshChildren(const FRefreshChildrenParams& InParams) override;
private:
	virtual FText GetDisplayName() const override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual TSharedPtr<SWidget> GenerateRowWidget() override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual UMediaOutput* AddMediaOutputToChannel(FName InTargetChannel, const FAvaBroadcastMediaOutputInfo& InOutputInfo) override;
	//~ End IAvaBroadcastOutputTreeItem
};
