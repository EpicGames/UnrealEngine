// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "Broadcast/Channel/AvaBroadcastMediaOutputInfo.h"

class FAvaBroadcastOutputTreeItem;
class FDragDropOperation;
class FReply;
class IAvaBroadcastOutputTreeItem;
class SWidget;
class UMediaOutput;
struct FGeometry;
struct FPointerEvent;
struct FSlateBrush;

using FAvaOutputTreeItemPtr = TSharedPtr<IAvaBroadcastOutputTreeItem>;

class IAvaBroadcastOutputTreeItem : public IAvaTypeCastable, public TSharedFromThis<IAvaBroadcastOutputTreeItem>
{
public:
	UE_AVA_INHERITS(IAvaBroadcastOutputTreeItem, IAvaTypeCastable);

	virtual FText GetDisplayName() const = 0;

	virtual const FSlateBrush* GetIconBrush() const = 0;

	struct FRefreshChildrenParams
	{
		/** Include all the media output classes. If false, only classes with a device provider will be included. */
		bool bShowAllMediaOutputClasses = false;
	};
	
	/** Refreshes what the Children are of this Item. (not recursive!) */
	virtual void RefreshChildren(const FRefreshChildrenParams& InParams) = 0;

	virtual TSharedPtr<SWidget> GenerateRowWidget() = 0;
	
	virtual const TWeakPtr<FAvaBroadcastOutputTreeItem>& GetParent() const = 0;

	virtual const TArray<FAvaOutputTreeItemPtr>& GetChildren() const = 0;

	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) = 0;

	/** Returns true if it is valid to add this item to the given channel. */
	virtual bool IsValidToDropInChannel(FName InTargetChannelName) = 0;

	virtual UMediaOutput* AddMediaOutputToChannel(FName InTargetChannel, const FAvaBroadcastMediaOutputInfo& InOutputInfo) = 0;

	/** Delegating the drag and drop operation to editor module. */
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<FDragDropOperation>, FOnCreateDragDropOperation, const FAvaOutputTreeItemPtr&)
	virtual FOnCreateDragDropOperation& OnCreateDragDropOperation() = 0;
};

class AVALANCHEMEDIA_API FAvaBroadcastOutputTreeItem : public IAvaBroadcastOutputTreeItem
{
public:
	UE_AVA_INHERITS(FAvaBroadcastOutputTreeItem, IAvaBroadcastOutputTreeItem);

	FAvaBroadcastOutputTreeItem(const TSharedPtr<FAvaBroadcastOutputTreeItem>& InParent)
		: ParentWeak(InParent)
	{
	}

	//~ Begin IAvaBroadcastOutputTreeItem
	virtual ~FAvaBroadcastOutputTreeItem() override {};
	virtual const TWeakPtr<FAvaBroadcastOutputTreeItem>& GetParent() const override;
	virtual const TArray<FAvaOutputTreeItemPtr>& GetChildren() const override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual bool IsValidToDropInChannel(FName InTargetChannelName) override { return true; }
	virtual FOnCreateDragDropOperation& OnCreateDragDropOperation() override { return OnCreateDragDropOperationDelegate; }
	//~ End IAvaBroadcastOutputTreeItem

	/** Calls RefreshChildren() on the tree of item. */
	static void RefreshTree(const FAvaOutputTreeItemPtr& InItem, const FRefreshChildrenParams& InParams);

protected:
	TWeakPtr<FAvaBroadcastOutputTreeItem> ParentWeak;
	TArray<FAvaOutputTreeItemPtr> Children;
	FOnCreateDragDropOperation OnCreateDragDropOperationDelegate;
};
