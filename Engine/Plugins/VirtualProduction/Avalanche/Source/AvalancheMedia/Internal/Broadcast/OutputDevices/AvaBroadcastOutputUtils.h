// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"

class UMediaOutput;
class UObject;
struct FAvaBroadcastOutputData;

namespace UE::AvaBroadcastOutputUtils
{
	/** Get the device's name from the given MediaOutput. */
	AVALANCHEMEDIA_API FString GetDeviceName(const UMediaOutput* InMediaOutput);

	/** Returns true if the class has MediaIOCustomLayout meta data. */
	AVALANCHEMEDIA_API bool HasDeviceProviderName(const UClass* InMediaOutputClass);

	/** Retrieves the device provider name from the MediaIOCustomLayout class meta data. */
	AVALANCHEMEDIA_API FName GetDeviceProviderName(const UClass* InMediaOutputClass);

	/** Retrieves the device provider name from the MediaIOCustomLayout class meta data. */
	AVALANCHEMEDIA_API FName GetDeviceProviderName(const UMediaOutput* InMediaOutput);

	/** Returns true if the given media output is a game viewport output. */
	AVALANCHEMEDIA_API bool IsGameViewportOutput(const UMediaOutput* InMediaOutput);

	/** Serializes the given MediaOutput object into a FAvaBroadcastOutputData. */ 
	AVALANCHEMEDIA_API FAvaBroadcastOutputData CreateMediaOutputData(UMediaOutput* InMediaOutput);

	/** Create a MediaOutput object from the given data. */
	AVALANCHEMEDIA_API UMediaOutput* CreateMediaOutput(const FAvaBroadcastOutputData& InMediaOutputData, UObject* InOuter);
}

