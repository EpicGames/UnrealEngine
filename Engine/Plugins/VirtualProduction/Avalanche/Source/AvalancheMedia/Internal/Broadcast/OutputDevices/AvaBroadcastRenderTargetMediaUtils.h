// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHI.h"

class FRHICommandListImmediate;
class UTextureRenderTarget2D;

namespace UE::AvaBroadcastRenderTargetMediaUtils
{
	static inline const TCHAR* VirtualShaderMountPoint = TEXT("/Plugin/MotionDesign");

	/**
	 * Clears the given render target to it's specified clear color.
	 * @remark called from the main thread.
	 */
	AVALANCHEMEDIA_API void ClearRenderTarget(UTextureRenderTarget2D* InRenderTarget);

	/**
	 * Copy a texture from source to destination.
	 * Handles different sizes, will rescale.
	 * Handles different formats, will convert.
	 */
	AVALANCHEMEDIA_API void CopyTexture(FRHICommandListImmediate& InRHICmdList, FTextureRHIRef InSourceTexture, FTextureRHIRef InDestTarget);
	
	/** Implementation of texture conversion from SCS_FinalColorSDR back to SCS_FinalToneCurveHDR. */
	AVALANCHEMEDIA_API void ConvertTextureRGBGamma(FRHICommandListImmediate& InRHICmdList, FTextureRHIRef InSourceTexture, FTextureRHIRef InDestTarget, bool bInSrgbToLinear, float InGamma);
}