// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Attribute.h"
#include "Widgets/Images/SImage.h"

class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
struct FGeometry;

class AVALANCHEMEDIA_API SAvaBroadcastCaptureImage : public SImage
{
public:
	SLATE_BEGIN_ARGS(SAvaBroadcastCaptureImage)
		: _ImageArgs()
		, _EnableGammaCorrection(true)
	{}
		SLATE_ARGUMENT(SImage::FArguments, ImageArgs)
		SLATE_ARGUMENT(bool, EnableGammaCorrection)
		SLATE_ATTRIBUTE(bool, EnableBlending)
		SLATE_ATTRIBUTE(bool, ShouldInvertAlpha)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);
	
	virtual int32 OnPaint(const FPaintArgs& Args
		, const FGeometry& AllottedGeometry
		, const FSlateRect& MyCullingRect
		, FSlateWindowElementList& OutDrawElements
		, int32 LayerId
		, const FWidgetStyle& InWidgetStyle
		, bool bParentEnabled) const override;

protected:
	bool bEnableGammaCorrection = true;
	TAttribute<bool> EnableBlending;
	TAttribute<bool> ShouldInvertAlpha;
};
