// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModularFeature/IAvaMediaSynchronizedEventsFeature.h"

class FAvaMediaSynchronizedEventsFeature
{
public:
	static void Startup();
	
	static void Shutdown();
	
	/**
	 * Returns currently selected implementation.
	 * This function always returns a valid implementation.
	 */
	static AVALANCHEMEDIA_API IAvaMediaSynchronizedEventsFeature* Get();

	/**
	 * Creates a new event dispatcher from the currently selected implementation. 
	 */ 
	static AVALANCHEMEDIA_API TSharedPtr<IAvaMediaSynchronizedEventDispatcher> CreateDispatcher(const FString& InSignature);

	static AVALANCHEMEDIA_API void EnumerateImplementations(TFunctionRef<void(const IAvaMediaSynchronizedEventsFeature*)> InCallback);

	/**
	 * Find the implementation with the given name.
	 * The name can be "default", in which case it will automatically select the highest priority implementation.
	 * If no implementation match the given name, it will return an internal "no sync" implementation.
	 * This function always returns a valid implementation.
	 */
	static AVALANCHEMEDIA_API IAvaMediaSynchronizedEventsFeature* FindImplementation(FName InImplementation);

private:
	static IAvaMediaSynchronizedEventsFeature* GetInternalImplementation();
};