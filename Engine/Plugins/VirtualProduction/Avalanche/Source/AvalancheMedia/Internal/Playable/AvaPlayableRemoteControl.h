// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "RemoteControlPreset.h"
#include "AvaPlayableRemoteControl.generated.h"

AVALANCHEMEDIA_API DECLARE_LOG_CATEGORY_EXTERN(LogAvaPlayableRemoteControl, Log, All);

/**
 * Return code for Remote Control operations.
 */
UENUM()
enum class EAvaPlayableRemoteControlResult : int8
{
	/** Operation completed successfully */
	Completed = 0,
	/** Operation was not necessary. */
	UpToDate = 1,	
	/** Input parameter is invalid */ 
	InvalidParameter = -1,
	/** Object reference read access failed. */
	ReadAccessDenied = -2,
	/** Object reference write access failed. */
	WriteAccessDenied = -3,
	/** Serialization from property failed. */
	ReadPropertyFailed = -4,
	/** Serialization to property failed. */
	WritePropertyFailed = -5
};

namespace UE::AvaPlayableRemoteControl
{
	/** Returns true if the binding is valid and access to read the property. */
	AVALANCHEMEDIA_API bool HasReadAccess(const TSharedPtr<const FRemoteControlEntity>& InRemoteControlEntity, FString& OutErrorText);

	/** Returns true if the binding is valid and access to write the property. */
	AVALANCHEMEDIA_API bool HasWriteAccess(const TSharedPtr<const FRemoteControlEntity>& InRemoteControlEntity, FString& OutErrorText);

	/** Gets the value (as bytes) of the given property (remote control entity). Returns true if the operation was successful, false otherwise. */
	AVALANCHEMEDIA_API EAvaPlayableRemoteControlResult GetValueOfEntity(const TSharedPtr<const FRemoteControlEntity>& InRemoteControlEntity, TArray<uint8>& OutValue);
	AVALANCHEMEDIA_API EAvaPlayableRemoteControlResult GetValueOfEntity(const TSharedPtr<const FRemoteControlEntity>& InRemoteControlEntity, FString& OutValue);
	
	/** Set the value (as bytes) of the given property (remote control entity). Returns true if the operation was successful, false otherwise. */
	AVALANCHEMEDIA_API EAvaPlayableRemoteControlResult SetValueOfEntity(const TSharedPtr<FRemoteControlEntity>& InRemoteControlEntity, const TArrayView<const uint8>& InValue);
	AVALANCHEMEDIA_API EAvaPlayableRemoteControlResult SetValueOfEntity(const TSharedPtr<FRemoteControlEntity>& InRemoteControlEntity, const FString& InValue);

	/** Gets the value (as bytes) of the given controller. Returns true if the operation was successful, false otherwise. */
	AVALANCHEMEDIA_API EAvaPlayableRemoteControlResult GetValueOfController(URCVirtualPropertyBase* InController, TArray<uint8>& OutValue);
	AVALANCHEMEDIA_API EAvaPlayableRemoteControlResult GetValueOfController(URCVirtualPropertyBase* InController, FString& OutValue);
	
	/** Set the value (as bytes) of the given controller. Returns true if the operation was successful, false otherwise. */
	AVALANCHEMEDIA_API EAvaPlayableRemoteControlResult SetValueOfController(URCVirtualPropertyBase* InController, const TArrayView<const uint8>& InValue);
	AVALANCHEMEDIA_API EAvaPlayableRemoteControlResult SetValueOfController(URCVirtualPropertyBase* InController, const FString& InValue);
	AVALANCHEMEDIA_API EAvaPlayableRemoteControlResult SetValueOfController(URCVirtualPropertyBase* InController, const FString& InValue, bool bInBehaviorsEnabled);

	inline bool Failed(EAvaPlayableRemoteControlResult InResult)
	{
		// Negative result codes are failure.
		return InResult < EAvaPlayableRemoteControlResult::Completed;
	}

	AVALANCHEMEDIA_API FString EnumToString(EAvaPlayableRemoteControlResult InValue);
	
	/**
	 * Utility class to temporarily push the enabled state of the controller's behaviours.
	 * This class is intended to be used temporarily on local stack. Lifetime of virtual property is not managed.
	 */
	class AVALANCHEMEDIA_API FScopedPushControllerBehavioursEnable
	{
	public:
		FScopedPushControllerBehavioursEnable(URCVirtualPropertyBase* InVirtualProperty, bool bInBehavioursEnabled);
		~FScopedPushControllerBehavioursEnable();
		
	private:
		URCVirtualPropertyBase* VirtualProperty = nullptr;
		TArray<bool> PreviousBehavioursEnabled;
	};
}