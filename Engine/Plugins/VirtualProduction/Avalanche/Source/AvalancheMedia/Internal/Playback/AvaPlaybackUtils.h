// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"
#include "UObject/SoftObjectPath.h"

class UPackage;
class UStreamableRenderAsset;
struct FAssetData;

class AVALANCHEMEDIA_API FAvaPlaybackUtils
{
public:
	/**
	 * Flushes loaded packages so the client can write to the files.
	*/
	static void FlushPackageLoading(UPackage* InPackage, bool bInResetLoaders=false);

	/**
	 * Checks if the given packages has been deleted on disk.
	 * @remark With Editor only.
	 * @return true if the package was deleted, false if the package still exists on disk.
	 */
	static bool IsPackageDeleted(const UPackage* InExistingPackage);

	/**
	 *	Purge all the objects in memory owned by the given packages.
	 */
	static void PurgePackages(const TArray<UPackage*>& InExistingPackages);

	/**
	 * Reloads the given package.
	 * @remark With Editor only.
	 * @return true if the package was reloaded.
	 */
	static bool ReloadPackages(const TArray<UPackage*>& InExistingPackages);

	/**
	 * @brief Determines if the asset is a map by checking the file extension.
	 * @remark The file must exist on disk.
	 * @param InPackageName Package name.
	 * @return true if the file on disk is a .umap file.
	 */
	static bool IsMapAsset(const FString& InPackageName);

	/**
	 * @brief Determines if the asset is a playable (can be used as template) asset, using the asset class.
	 * @param InAssetData Asset Data
	 * @return true if the asset is an ava playable.
	 */
	static bool IsPlayableAsset(const FAssetData& InAssetData);
};

namespace UE::AvaPlayback::Utils
{
	/**
	 *	Returns a compactly formatted time stamp information for the current frame.
	 *	This is to used for logging and tracing.
	 */
	AVALANCHEMEDIA_API FString GetBriefFrameInfo();

	template<typename InEnumType>
	FString StaticEnumToString(InEnumType InValue)
	{
		return StaticEnum<InEnumType>()->GetNameStringByValue(static_cast<int64>(InValue));
	}

	/**
	 * Streams in the given object if it's a streamable asset.
	 * returns a valid pointer to the object as a render asset if the asset was streamable
	 */
	UStreamableRenderAsset* StreamRenderAsset(UObject* InObject);

	/** 
	 * Caches the materials of the given object using the material bridge api
	 * @param InObject the object to gather and cache materials for
	 * @param InShaderProfile the shader profile to use for caching
	 */
	bool CacheObjectMaterials(UObject* InObject, FName InShaderProfile);

	/**
	 * Utility class to perform async load of assets. 
	 */
	class FAsyncAssetLoader : public FGCObject, public TSharedFromThis<FAsyncAssetLoader>
	{
	public:
		void ProcessLoadedObject(UObject* InObject, FName InShaderProfile);

		/**
		 * @brief Issue the async load command. Returns immediately.
		 * @param InAssetsToLoad Assets to load.
		 *
		 * The specified asset to load will added to the current set of pending assets.
		 */
		AVALANCHEMEDIA_API void BeginLoadingAssets(const TArray<FSoftObjectPath>& InAssetsToLoad);

		/** Returns true when all the requested assets have finished loading. */
		AVALANCHEMEDIA_API bool IsLoadingCompleted() const;

		/** Removes objects that have all their materials completed */
		void CleanupObjectsWithCompleteMaterials() const;

		/** Removes streamable render assets that have fully been streamed in */
		void CleanupStreamingAssets() const;

		//~ Begin FGCObject
		AVALANCHEMEDIA_API virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
		AVALANCHEMEDIA_API virtual FString GetReferencerName() const override;
		//~ End FGCObject

		/** Delegate called when all assets have finished loading. */
		DECLARE_MULTICAST_DELEGATE(FOnLoadingCompleted);
		FOnLoadingCompleted OnLoadingCompleted;

	private:
		TSet<FSoftObjectPath> PendingAssets;
		mutable TArray<TObjectPtr<UObject>> PendingMaterialCompletionObjects;
		mutable TArray<TObjectPtr<UStreamableRenderAsset>> PendingStreamingAssets;
	};
}
