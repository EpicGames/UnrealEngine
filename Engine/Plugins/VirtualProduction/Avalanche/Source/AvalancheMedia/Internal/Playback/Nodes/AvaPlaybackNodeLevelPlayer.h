// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playback/Nodes/AvaPlaybackNodePlayer.h"
#include "UObject/NoExportTypes.h"
#include "UObject/SoftObjectPtr.h"
#include "AvaPlaybackNodeLevelPlayer.generated.h"

class UWorld;
struct FAvaSoftAssetPtr;

UCLASS(MinimalAPI)
class UAvaPlaybackNodeLevelPlayer : public UAvaPlaybackNodePlayer
{
	GENERATED_BODY()

public:
	UAvaPlaybackNodeLevelPlayer();
	
	virtual void RefreshNode(bool bDryRunGraph) override;
	virtual void PostLoad() override;
	
	TSoftObjectPtr<UWorld> GetAsset() const { return LevelAsset; }
	void SetAsset(const TSoftObjectPtr<UWorld>& InAsset);
	
	void UpdateDisplayNameText();

	virtual const FSoftObjectPath& GetAssetPath() const override { return LevelAsset.ToSoftObjectPath();}

	virtual FAvaSoftAssetPtr GetAssetPtr() const override;

protected:
	UPROPERTY(EditAnywhere, Category = "Motion Design")
	TSoftObjectPtr<UWorld> LevelAsset;
};
