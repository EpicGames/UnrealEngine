// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "Playable/AvaPlayableRemoteControlPresetInfo.h"
#include "Playable/AvaPlayableRemoteControlValues.h"
#include "UObject/GCObject.h"
#include "UObject/SoftObjectPath.h"

class FAvaRundownManagedInstanceCache;
class IAvaSceneInterface;
class UAvaRundown;
class URemoteControlPreset;
struct FAvaPlayableRemoteControlPresetInfo;

class AVALANCHEMEDIA_API FAvaRundownManagedInstance : public IAvaTypeCastable, public FGCObject, public TSharedFromThis<FAvaRundownManagedInstance>
{
public:
	UE_AVA_INHERITS(FAvaRundownManagedInstance, IAvaTypeCastable);
	
	FAvaRundownManagedInstance(FAvaRundownManagedInstanceCache* InParentCache, const FSoftObjectPath& InAssetPath);
	virtual ~FAvaRundownManagedInstance() override;

	/** Returns true if the instanced asset is valid. */
	virtual bool IsValid() const { return false; }
	
	/** Get the Managed Motion Design Asset's (top level) Remote Control Preset. */
	virtual URemoteControlPreset* GetRemoteControlPreset() const { return nullptr; }
	
	/** Access a copy of the Remote Control values from the source (a.k.a. template). */
	const FAvaPlayableRemoteControlValues& GetDefaultRemoteControlValues() const { return DefaultRemoteControlValues; }

	/** Access cached preset info for this instance. */
	const FAvaPlayableRemoteControlPresetInfo& GetRemoteControlPresetInfo() const
	{
		return RemoteControlPresetInfo && !RemoteControlPresetInfo->IsDirty() ?
			*RemoteControlPresetInfo : GetRemoteControlPresetInfoFromCache(GetRemoteControlPreset());
	}

	/** Get the Ava scene interface from the asset. */
	virtual IAvaSceneInterface* GetSceneInterface() const { return nullptr; }

	const FSoftObjectPath& GetSourceAssetPath() const { return SourceAssetPath; }

	/** Set the current rundown page for routing controller events from this managed instance. */
	void SetCurrentRundownPageForEvents(UAvaRundown* InRundown, int32 InPageId);

	/** Returns the rundown currently set to receive the RC Controller events from this managed instance. */
	UAvaRundown* GetRundownForEvents() const
	{
		return RundownForEvents.Get();
	}

	/** Returns the rundown page id that is currently set to receive RC Controller events from this managed instance. */
	int32 GetActivePageIdForEvents() const
	{
		return ActivePageId;
	}

protected:
	FAvaPlayableRemoteControlPresetInfo& GetRemoteControlPresetInfoFromCache(URemoteControlPreset* InRemoteControlPreset) const;
	void InvalidateFromCache();

	void RegisterSourceRemoteControlPresetDelegates(URemoteControlPreset* InSourceRemoteControlPreset);
	void UnregisterSourceRemoteControlPresetDelegates(URemoteControlPreset* InSourceRemoteControlPreset) const;
	void RegisterManagedRemoteControlPresetDelegates(URemoteControlPreset* InManagedRemoteControlPreset);
	void UnregisterManagedRemoteControlPresetDelegates(URemoteControlPreset* InManagedRemoteControlPreset) const;

	void OnSourceRemoteControlEntitiesExposed(URemoteControlPreset* InSourcePreset, const FGuid& InEntityId);
	void OnSourceRemoteControlEntitiesUnexposed(URemoteControlPreset* InSourcePreset, const FGuid& InEntityId);
	void OnSourceRemoteControlEntitiesUpdated(URemoteControlPreset* InSourcePreset, const TSet<FGuid>& InModifiedEntities);
	void OnSourceRemoteControlExposedPropertiesModified(URemoteControlPreset* InSourcePreset, const TSet<FGuid>& InModifiedProperties);
	void OnSourceRemoteControlControllerAdded(URemoteControlPreset* InSourcePreset, const FName NewControllerName, const FGuid& InControllerId);
	void OnSourceRemoteControlControllerRemoved(URemoteControlPreset* InSourcePreset, const FGuid& InControllerId);
	void OnSourceRemoteControlControllerRenamed(URemoteControlPreset* InSourcePreset, const FName InOldLabel, const FName InNewLabel);
	void OnSourceRemoteControlControllerModified(URemoteControlPreset* InSourcePreset, const TSet<FGuid>& InModifiedControllerIds);

	void OnManagedRemoteControlControllerModified(URemoteControlPreset* InPreset, const TSet<FGuid>& InModifiedControllerIds);

protected:
	FAvaRundownManagedInstanceCache* ParentCache;
	FSoftObjectPath SourceAssetPath;
	FAvaPlayableRemoteControlValues DefaultRemoteControlValues;
	mutable TSharedPtr<FAvaPlayableRemoteControlPresetInfo> RemoteControlPresetInfo;

	/** Current rundown for routing controller events. */
	TWeakObjectPtr<UAvaRundown> RundownForEvents;

	/** Current active page for routing controller events. */
	int32 ActivePageId;
};