// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/AvaRundownManagedInstance.h"
#include "Rundown/AvaRundownManagedInstanceHandle.h"

class IAvaMediaSyncProvider;
class UAvaRundown;
class UPackage;
struct FAvaRundownPage;

/**
 * Cache for the managed Motion Design instances.
 *
 * This is LRU cache.
 *
 * Todo: This needs to be refactored as a pool instead.
 * A managed asset instance is needed for each page that is edited. Since rundown editor and server
 * can edit different pages (of the same template), it needs more than one instance and it needs to be
 * tied to the rundown (for RC event routing).
 */
class AVALANCHEMEDIA_API FAvaRundownManagedInstanceCache
{
public:
	FAvaRundownManagedInstanceCache();
	virtual ~FAvaRundownManagedInstanceCache();

	/**
	 *	Retrieves elements from the cache or load a new one if not present.
	 *	If the cache size is exceeded, the oldest elements are going to be flushed.
	 */
	TSharedPtr<FAvaRundownManagedInstance> GetOrLoadInstance(const FSoftObjectPath& InAssetPath);

	/**
	 * Invalidates a cached entry without deleting it immediately.
	 * It will be deleted and reloaded on the next access query.
	 */
	void InvalidateNoDelete(const FSoftObjectPath& InAssetPath);

	/**
	 * Invalidates a cached entry. The entry may be deleted as a result.
	 */
	void Invalidate(const FSoftObjectPath& InAssetPath);

	/**
	 *	Delegate called when an entry is invalidated. Reason for an entry to be invalidated is if the source
	 *	has been modified.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnEntryInvalidated, const FSoftObjectPath& /*InAssetPath*/);
	FOnEntryInvalidated OnEntryInvalidated;
	
	/**
	 * Get the maximum size of the cache beyond witch it will start flushing elements.
	 */
	int32 GetMaximumCacheSize() const;
	
	/**
	 *	Flush specified entry from the cache.
	 */
	void Flush(const FSoftObjectPath& InAssetPath);

	/**
	 * Flush all unused entries from the cache.
	 */
	void Flush();

	/**
	 * Trim the cache elements that exceed the cache capacity according to LRU replacement policy.
	 */
	void TrimCache();

	/**
	 *	Perform any pending actions, such has removing invalidated entries. This may lead to
	 *	objects being deleted.
	 */
	void FinishPendingActions();

	/**
	 * Utility function to get all the managed instances for a given page, including all sub-pages in case of a combo page. 
	 * @param InRundown Rundown owning the current page.
	 * @param InPage Page to get the assets from.
	 * @return managed asset instances for the given page.
	 */
	TArray<TSharedPtr<FAvaRundownManagedInstance>> GetManagedInstancesForPage(const UAvaRundown* InRundown, const FAvaRundownPage& InPage);

	/**
	 * Utility function to acquire all the managed instances for a given page, including all sub-pages in case of a combo page.
	 * This will register the given rundown page for controller events routing in the managed instances (potentially overriding
	 * previous rundown or page).
	 * 
	 * @param InRundown Rundown owning the current page.
	 * @param InPage Page to get the assets from.
	 * @return managed asset instance handles for the given page.
	 */
	FAvaRundownManagedInstanceHandles GetManagedHandlesForPage(UAvaRundown* InRundown, const FAvaRundownPage& InPage);
	
private:
	void RemovePendingInvalidatedPaths();
	void RemoveEntry(const FSoftObjectPath& InAssetPath);
	void RemoveEntries(TFunctionRef<bool(const FSoftObjectPath&, const TSharedPtr<FAvaRundownManagedInstance>&)> InRemovePredicate, bool bInNotify);

	void OnPackageSaved(const FString& InPackageFileName, UPackage* InPackage, FObjectPostSaveContext InObjectSaveContext);
	void OnAvaSyncPackageModified(IAvaMediaSyncProvider* InAvaMediaSyncProvider, const FName& InPackageName);
	void OnAssetRemoved(const FAssetData& InAssetData);
	void OnPackageModified(const FName& InPackageName);
	void OnSettingChanged(UObject* , struct FPropertyChangedEvent&);

private:
	TMap<FSoftObjectPath, TSharedPtr<FAvaRundownManagedInstance>> Instances;
	TSet<FSoftObjectPath> PendingInvalidatedPaths;
	TArray<FSoftObjectPath> OrderQueue;
};