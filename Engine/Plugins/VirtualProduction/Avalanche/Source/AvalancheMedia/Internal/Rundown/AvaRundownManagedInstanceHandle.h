// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Rundown/AvaRundownDefines.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API AVALANCHEMEDIA_API

class FAvaRundownManagedInstance;
class UAvaRundown;

/**
 * Set of rundown managed asset instances tied to a given rundown for RC editing operations.
 * It will hook the controller events routing to the provided rundown and page on construction
 * and handles auto-unregister of Rundown on destruction.
 *
 * As this object represents the ownership of the "event routing" for a given rundown and page, it is not copiable.
 */
struct FAvaRundownManagedInstanceHandles
{
	FAvaRundownManagedInstanceHandles() = default;
	
	/**
	 * The constructor will register the given rundown page for controller events routing in the managed instances (potentially overriding
	 * previous rundown or page).
	 */
	UE_API FAvaRundownManagedInstanceHandles(const TArray<TSharedPtr<FAvaRundownManagedInstance>>& InInstances, UAvaRundown* InRundown, int32 InPageId);

	FAvaRundownManagedInstanceHandles(const FAvaRundownManagedInstanceHandles& InOther) = delete;
	FAvaRundownManagedInstanceHandles& operator=(const FAvaRundownManagedInstanceHandles& InOther) = delete;

	FAvaRundownManagedInstanceHandles(FAvaRundownManagedInstanceHandles&& InOther)
	{
		*this = MoveTemp(InOther);
	}

	/** Move operator handles unregistering instances if needed. */
	UE_API FAvaRundownManagedInstanceHandles& operator=(FAvaRundownManagedInstanceHandles&& InOther);

	~FAvaRundownManagedInstanceHandles()
	{
		Reset();
	}

	bool IsEmpty() const
	{
		return Instances.IsEmpty();
	}

	int32 Num() const
	{
		return Instances.Num();
	}

	/**
	 * Unregisters the rundown from the instances (only if it hasn't been overriden already) and resets the array of instances. 
	 */
	UE_API void Reset();

	/** Array of managed asset instances. */
	TArray<TSharedPtr<FAvaRundownManagedInstance>> Instances;

private:
	/** Keep track of the registered Rundown to be able to unregister if needed. */
	TWeakObjectPtr<UAvaRundown> RegisteredRundown;

	/** Keep track of the registered page id to be able to unregister if needed. */
	int32 RegisteredPageId = UE::AvaMedia::Rundown::InvalidPageId;
};

#undef UE_API