// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Broadcast/Channel/AvaBroadcastMediaOutputInfo.h"
#include "Rundown/AvaRundownPage.h"
#include "Viewport/AvaViewportQualitySettings.h"
#include "AvaRundownMessages.generated.h"

namespace EAvaRundownApiVersion
{
	/**
	 * Defines the protocol version of the Rundown Server API.
	 *
	 * API versioning is used to provide legacy support either on
	 * the client side or server side for non compatible changes.
	 * Clients can request a version of the API that they where implemented against,
	 * if the server can still honor the request it will accept.
	 */
	enum Type
	{
		Unspecified = -1,
		
		Initial = 1,
		/**
		 * The rundown server has been moved to the runtime module.
		 * All message scripts paths moved from AvalancheMediaEditor to AvalancheMedia.
		 * However, all server requests messages have been added to core redirect, so
		 * previous path will still get through, but all response messages will be the new path.
		 * Clients can still issue a ping with the old path and will get a response.
		 */ 
		MoveToRuntime = 2,

		// -----<new versions can be added before this line>-------------------------------------------------
		// - this needs to be the last line (see note below)
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
}

/**
 * Build targets.
 * This will help determine the set of features that are available.
 */
UENUM()
enum class EAvaRundownServerBuildTargetType : uint8
{
	Unknown = 0,
	Editor,
	Game,
	Server,
	Client,
	Program
};

/**
 * An editor build can be launched in different modes but it could also be
 * a dedicated build target. The engine mode combined with the build target
 * will determine the set of functionalities available.
 */
UENUM()
enum class EAvaRundownServerEngineMode : uint8
{
	Unknown = 0,
	Editor,
	Game,
	Server,
	Commandlet,
	Other
};

/** Base class for all rundown server messages. */
USTRUCT()
struct FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/**
	 * Request Identifier (client assigned) for matching server responses with their corresponding requests. 
	 */
	UPROPERTY()
	int32 RequestId = INDEX_NONE;
};

/**
 * This message is the default response message for all requests, unless a specific response message type
 * is specified for the request.
 * On success, the message will have a Verbosity of "Log" and the text may contain response payload related data.
 * On failure, a message with Verbosity "Error" will be sent.
 * This message's RequestId mirrors that of the corresponding request from the client.
 */
USTRUCT()
struct FAvaRundownServerMsg : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/**
	 * Debug, Log, Warning, Error, etc.
	 */
	UPROPERTY()
	FString Verbosity;

	/**
	 * Message Text.
	 */
	UPROPERTY()
	FString Text;
};

/**
 * Request published by client to discover servers on the message bus.
 * The available servers will respond with a FAvaRundownPong.
 */
USTRUCT()
struct FAvaRundownPing : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** True if the request originates from an automatic timer. False if request originates from user interaction. */
	UPROPERTY()
	bool bAuto = true;

	/**
	 * API Version the client has been implemented against.
	 * If unspecified the server will consider the latest version is requested.
	 */
	UPROPERTY()
	int32 RequestedApiVersion = EAvaRundownApiVersion::Unspecified;
};

/**
 * The server will send this message to the client in response to FAvaRundownPing.
 * This is used to discover the server's entry point on the message bus.
 */
USTRUCT()
struct FAvaRundownPong : public FAvaRundownMsgBase
{
	GENERATED_BODY()

public:
	/** True if it is a reply to an auto ping. Mirrors the bAuto flag from Ping message. */
	UPROPERTY()
	bool bAuto = true;

	/**
	 * API Version the server will communicate with for this client.
	 * The server may honor the requested version if possible.
	 * Versions newer than server implementation will obviously not be honored either.
	 * Clients should expect an older server to reply with an older version.
	 */
	UPROPERTY()
	int32 ApiVersion = EAvaRundownApiVersion::Unspecified;

	/** Minimum API Version the server implements. */
	UPROPERTY()
	int32 MinimumApiVersion = EAvaRundownApiVersion::Unspecified;

	/** Latest API Version the server support. */
	UPROPERTY()
	int32 LatestApiVersion = EAvaRundownApiVersion::Unspecified;

	/** Server Host Name */
	UPROPERTY()
	FString HostName;
};

/**
 * Requests the extended server information.
 * Response is FAvaRundownServerInfo.
 */
USTRUCT()
struct FAvaRundownGetServerInfo : public FAvaRundownMsgBase
{
	GENERATED_BODY()
};

/**
 * Extended server information.
 */
USTRUCT()
struct FAvaRundownServerInfo : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/** API Version the server will communicate with for this client. */
	UPROPERTY()
	int32 ApiVersion = EAvaRundownApiVersion::Unspecified;

	/** Minimum API Version the server implements. */
	UPROPERTY()
	int32 MinimumApiVersion = EAvaRundownApiVersion::Unspecified;

	/** Latest API Version the server support. */
	UPROPERTY()
	int32 LatestApiVersion = EAvaRundownApiVersion::Unspecified;

	/** Server Host Name */
	UPROPERTY()
	FString HostName;

	/** Holds the engine version checksum */
	UPROPERTY()
	uint32 EngineVersion = 0;
	
	/** Application Instance Identifier. */
	UPROPERTY()
	FGuid InstanceId;

	UPROPERTY()
	EAvaRundownServerBuildTargetType InstanceBuild = EAvaRundownServerBuildTargetType::Unknown;

	UPROPERTY()
	EAvaRundownServerEngineMode InstanceMode = EAvaRundownServerEngineMode::Unknown;

	/** Holds the identifier of the session that the application belongs to. */
	UPROPERTY()
	FGuid SessionId;
	
	/** The unreal project name this server is running from. */
	UPROPERTY()
	FString ProjectName;
	
	/** The unreal project directory this server is running from. */
	UPROPERTY()
	FString ProjectDir;

	/** Http Server Port of the remote control service. */
	UPROPERTY()
	uint32 RemoteControlHttpServerPort = 0;

	/** WebSocket Server Port of the remote control service. */
	UPROPERTY()
	uint32 RemoteControlWebSocketServerPort = 0;
};

/**
 * Requests a list of playable assets that can be added to a
 * rundown template.
 * Response is FAvaRundownPlayableAssets.
 */
USTRUCT()
struct FAvaRundownGetPlayableAssets : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/**
	 * The search query which will be compared with the asset names.
	*/
	UPROPERTY()
	FString Query;

	/**
	 * The maximum number of search results returned.
	*/
	UPROPERTY()
	int32 Limit = 0;
};

/**
 *	List of all available playable assets on the server.
 *	Expected Response from FAvaRundownGetPlayableAssets.
 */
USTRUCT()
struct FAvaRundownPlayableAssets : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FSoftObjectPath> Assets;
};

/**
 *	Requests the list of rundowns that can be opened on the current server.
 *	Response is FAvaRundownRundowns.
 */
USTRUCT()
struct FAvaRundownGetRundowns : public FAvaRundownMsgBase
{
	GENERATED_BODY()
};

/**
 *	List of all rundowns.
 *	Expected Response from FAvaRundownGetRundowns.
 */
USTRUCT()
struct FAvaRundownRundowns : public FAvaRundownMsgBase
{
	GENERATED_BODY()

public:
	/**
	 * List of Rundown asset paths in format: [PackagePath]/[AssetName].[AssetName]
	 */
	UPROPERTY()
	TArray<FString> Rundowns;
};

/**
 *	Loads the given rundown for playback operations.
 *	This will also open an associated playback context.
 *	Only one rundown can be opened for playback at a time by the rundown server.
 *	If another rundown is opened, the previous one will be closed and all currently playing pages stopped,
 *	unless the rundown editor is opened. The rundown editor will keep the playback context alive.
 *	
 *	If the path is empty, nothing will be done and the server will reply with
 *	a FAvaRundownServerMsg message indicating which rundown is currently loaded.
 */
USTRUCT()
struct FAvaRundownLoadRundown : public FAvaRundownMsgBase
{
	GENERATED_BODY()

public:
	/**
	 * Rundown asset path: [PackagePath]/[AssetName].[AssetName]
	 */
	UPROPERTY()
	FString Rundown;
};

/**
 * Creates a new rundown asset.
 *
 * The full package name is going to be: [PackagePath]/[AssetName] 
 * The full asset path is going to be: [PackagePath]/[AssetName].[AssetName]
 * For all other requests, the rundown reference is the full asset path.
 *
 * Response is FAvaRundownServerMsg.
 */
USTRUCT()
struct FAvaRundownCreateRundown : public FAvaRundownMsgBase
{
	GENERATED_BODY()

public:
	/** Package path (excluding the package name) */
	UPROPERTY()
	FString PackagePath;

	/** Asset Name. */
	UPROPERTY()
	FString AssetName;

	/**
	 * Create the rundown as a transient object.
	 * @remark For game builds, the created rundown will always be transient, regardless of this flag. 
	 */
	UPROPERTY()
	bool bTransient = true;
};

/**
 * Deletes an existing rundown.
 *
 * Response is FAvaRundownServerMsg.
 */
USTRUCT()
struct FAvaRundownDeleteRundown : public FAvaRundownMsgBase
{
	GENERATED_BODY()

public:
	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;
};

/**
 * Imports rundown from json data or file.
 */
USTRUCT()
struct FAvaRundownImportRundown : public FAvaRundownMsgBase
{
	GENERATED_BODY()

public:
	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/**
	 * If specified, this is a server local path to a json file from which the rundown will be imported.
	 */
	UPROPERTY()
	FString RundownFile;

	/**
	 * If specified, json data containing the rundown to import.
	 */
	UPROPERTY()
	FString RundownData;
};

/**
 * Exports a rundown to json data or file.
 * This command is supported in game build.
 */
USTRUCT()
struct FAvaRundownExportRundown : public FAvaRundownMsgBase
{
	GENERATED_BODY()

public:
	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** Optional path to a server local file where the rundown will be saved. */
	UPROPERTY()
	FString RundownFile;
};

/**
 * Server reply to FAvaRundownExportRundown containing the exported rundown.
 */
USTRUCT()
struct FAvaRundownExportedRundown : public FAvaRundownMsgBase
{
	GENERATED_BODY()

public:
	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** Exported rundown in json format. */
	UPROPERTY()
	FString RundownData;
};

/**
 * Requests that the given rundown be saved to disk.
 * The rundown asset must have been loaded, either by an edit command
 * or playback, prior to this command.
 * Unloaded assets will not be loaded by this command.
 * This command is not supported in game builds.
 */
USTRUCT()
struct FAvaRundownSaveRundown : public FAvaRundownMsgBase
{
	GENERATED_BODY()

public:
	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** The save command will be executed only if the asset package is dirty. */
	UPROPERTY()
	bool bOnlyIfIsDirty = false;
};

/**
 * Rundown specific events broadcast by the server to help status display or related contexts in control applications.
 */
USTRUCT()
struct FAvaRundownPlaybackContextChanged : public FAvaRundownMsgBase
{
	GENERATED_BODY()

public:
	/**
	 * Previous rundown (can be empty).
	 */
	UPROPERTY()
	FString PreviousRundown;
	
	/**
	 * New current rundown (can be empty).
	 */
	UPROPERTY()
	FString NewRundown;
};

/**
 * Requests the list of pages from the given rundown.
 */
USTRUCT()
struct FAvaRundownGetPages : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;
};

/**
 * Defines the parameters for the page id generator algorithm.
 * The Id generator uses a sequence strategy to search for an unused id.
 * It is defined by a starting id and a search direction.
 */
USTRUCT()
struct FAvaRundownCreatePageIdGeneratorParams
{
	GENERATED_BODY()

	/** Starting Id for the search. */
	UPROPERTY()
	int32 ReferenceId = FAvaRundownPage::InvalidPageId;

	/**
	 * @brief (Initial) Search increment.
	 * @remark For negative increment search, the limit of the search space can be reached. If no unique id is found,
	 *		   the search will continue in the positive direction instead.		   
	 */
	UPROPERTY()
	int32 Increment = 1;
};

/**
 * Requests a new page be created from the specified template in the given rundown.
 */
USTRUCT()
struct FAvaRundownCreatePage : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** Defines the parameters for the page id generator algorithm. */
	UPROPERTY()
	FAvaRundownCreatePageIdGeneratorParams IdGeneratorParams;
	
	/** Specifies the template for the newly created page. */
	UPROPERTY()
	int32 TemplateId = FAvaRundownPage::InvalidPageId;
};

/**
 * Requests the page be deleted from the given rundown.
 */
USTRUCT()
struct FAvaRundownDeletePage : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** Id of the page to be deleted. */
	UPROPERTY()
	int32 PageId = FAvaRundownPage::InvalidPageId;
};

/**
 * Requests the creation of a new template.
 * If successful, the response is FAvaRundownServerMsg with a "Template [Id] Created" text.
 * The id of the created template can be parsed from that message's text.
 * Also a secondary FAvaRundownPageListChanged event with added template id will be sent.
 */
USTRUCT()
struct FAvaRundownCreateTemplate : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** Defines the parameters for the page id generator algorithm. */
	UPROPERTY()
	FAvaRundownCreatePageIdGeneratorParams IdGeneratorParams;

	/** Specifies the asset path to assign to the template. */
	UPROPERTY()
	FString AssetPath;
};

/**
 * Requests the creation of a new combo template.
 * If successful, the response is FAvaRundownServerMsg with a "Template [Id] Created" text.
 * The id of the created template can be parsed from that message's text.
 * Also a secondary FAvaRundownPageListChanged event with added template id will be sent.
 *
 * @remark A combination template can only be created using transition logic templates that are in different transition layers.
 */
USTRUCT()
struct FAvaRundownCreateComboTemplate : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** Defines the parameters for the page id generator algorithm. */
	UPROPERTY()
	FAvaRundownCreatePageIdGeneratorParams IdGeneratorParams;

	/** Specifies the template ids that are combined. */
	UPROPERTY()
	TArray<int32> CombinedTemplateIds;
};

/** Requests deletion of the given template. */
USTRUCT()
struct FAvaRundownDeleteTemplate : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** Specifies the *template* id to delete. */
	UPROPERTY()
	int32 PageId = FAvaRundownPage::InvalidPageId;
};

/**
 * Sets the Page's template asset. This applies to template pages only.
 */
USTRUCT()
struct FAvaRundownChangeTemplateBP : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** Specifies the template id to modify. */
	UPROPERTY()
	int32 TemplateId = FAvaRundownPage::InvalidPageId;

	/** Specifies the asset path to assign. */
	UPROPERTY()
	FString AssetPath;

	/** If true, the asset will be re-imported and the template information will be refresh from the source asset. */
	UPROPERTY()
	bool bReimport = false;
};

/** Page Information */
USTRUCT()
struct FAvaRundownPageInfo
{
	GENERATED_BODY()
public:
	/** Unique identifier for the page within the rundown. */
	UPROPERTY()
	int32 PageId = FAvaRundownPage::InvalidPageId;

	/**
	 * Short page name, usually the asset name for templates.
	 * It is displayed as the page description if there is no page summary or user friendly name specified.
	 */
	UPROPERTY()
	FString PageName;

	/**
	 * Summary is generated from the remote control values for this page.
	 * It is displayed as the page description if there is no user friendly name specified.
	 */
	UPROPERTY()
	FString PageSummary;

	/** User editable page description. If not empty, this should be used as the page description. */
	UPROPERTY()
	FString FriendlyName;

	/** Indicates if the page is a template (true) or an instance (false). */
	UPROPERTY()
	bool IsTemplate = false;

	/** Page Instance property: Template Id for this page. */
	UPROPERTY()
	int32 TemplateId = FAvaRundownPage::InvalidPageId;

	/** Template property: For combination template, lists the templates that are combined. */ 
	UPROPERTY()
	TArray<int32> CombinedTemplateIds;

	/** Template property: playable asset path for this template. */
	UPROPERTY()
	FSoftObjectPath AssetPath;

	/**
	 * List of page channel statuses.
	 * There will be an entry for each channel the page is playing/previewing in.
	 */
	UPROPERTY()
	TArray<FAvaRundownChannelPageStatus> Statuses;

	/** Transition Layer Name (indicates the page has transition logic). */
	UPROPERTY()
	FString TransitionLayerName;

	/** Indicate if the template asset has transition logic. */
	UPROPERTY()
	bool bTransitionLogicEnabled = false;

	/** Page Commands that can be executed when playing this page. */
	UPROPERTY()
	TArray<FAvaRundownPageCommandData> Commands;
	
	UPROPERTY()
	FString OutputChannel;

	/** Specifies if the page is enabled (i.e. can be played). */
	UPROPERTY()
	bool bIsEnabled = false;

	/**
	 * Indicates if the page is currently playing in it's program channel. 
	 */
	UPROPERTY()
	bool bIsPlaying = false;
};

/*
 * List of pages from the current rundown.
 */
USTRUCT()
struct FAvaRundownPages : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** List of page descriptors */
	UPROPERTY()
	TArray<FAvaRundownPageInfo> Pages;
};

/**
 * Requests the page details from the given rundown.
 * Response is FAvaRundownPageDetails.
 */
USTRUCT()
struct FAvaRundownGetPageDetails : public FAvaRundownMsgBase
{
	GENERATED_BODY()

public:
	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** Specified the requested page id. */
	UPROPERTY()
	int32 PageId = FAvaRundownPage::InvalidPageId;

	/** This will request that a managed asset instance gets loaded to be
	 * accessible through WebRC. */
	UPROPERTY()
	bool bLoadRemoteControlPreset = false;
};

/**
 *	Server response to FAvaRundownGetPageDetails request.
 */
USTRUCT()
struct FAvaRundownPageDetails : public FAvaRundownMsgBase
{
	GENERATED_BODY()

public:
	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** Page Information. */
	UPROPERTY()
	FAvaRundownPageInfo PageInfo;

	/** Remote Control Values for this page. */
	UPROPERTY()
	FAvaPlayableRemoteControlValues RemoteControlValues;

	/**
	 * Name of the remote control preset to resolve through WebRC API.
	 * In case of combo page, this is the first sub-page only. See RemoteControlPresetNames for all the names in that case.
	 */
	UPROPERTY()
	FString RemoteControlPresetName;

	/**
	 * Uuid of the remote control preset to resolve through WebRC API.
	 * In case of combo page, this is the first sub-page only. See RemoteControlPresetIds for all the uuids in that case.
	 */
	UPROPERTY()
	FString RemoteControlPresetId;

	/** 
	 * This list will be populated with the names of the remote control preset of each sub-page in case of a combo page.
	 * The provided Uuids can be used to with the WebRC API. 
	 */
	UPROPERTY()
	TArray<FString> RemoteControlPresetNames;
	
	/** 
	 * This list will be populated with the uuid of the remote control preset of each sub-page in case of a combo page.
	 * The provided uuids can be used to with the WebRC API. 
	 */
	UPROPERTY()
	TArray<FString> RemoteControlPresetIds;
};

/**
 * Event sent when a page status changes.
 */
USTRUCT()
struct FAvaRundownPagesStatuses : public FAvaRundownMsgBase
{
	GENERATED_BODY()

public:
	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** Page Information. */
	UPROPERTY()
	FAvaRundownPageInfo PageInfo;
};

/**
 * Event sent when a page list (can be templates, pages or page views) has been modified.
 */
USTRUCT()
struct FAvaRundownPageListChanged : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** Specifies which page list has been modified. */
	UPROPERTY()
	EAvaRundownPageListType ListType = EAvaRundownPageListType::Instance;

	/** Specifies the uuid of the page view, in case the event concerns a page view. */
	UPROPERTY()
	FGuid SubListId;
	
	/**
	 * Bitfield value indicating what has changed:
	 * - bit 0: Added Pages
	 * - bit 1: Remove Pages
	 * - bit 2: Page Id Renumbered
	 * - bit 3: Sublist added or removed
	 * - bit 4: Sublist renamed
	 * - bit 5: Page View reordered
	 * 
	 * See EAvaPageListChange flags.
	 */
	UPROPERTY()
	uint8 ChangeType = 0;

	/** List of page Ids affected by this event. */
	UPROPERTY();
	TArray<int32> AffectedPages;
};

/**
 * Event sent when a page's asset is modified.
 * Note: this applies to templates only.
 */
USTRUCT()
struct FAvaRundownPageBlueprintChanged : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** Specified the modified page id. */
	UPROPERTY()
	int32 PageId = FAvaRundownPage::InvalidPageId;

	/** Asset the page is currently assigned to (post modification). */
	UPROPERTY()
	FString BlueprintPath;
};

/**
 * Event sent when a page's channel is modified.
 */
USTRUCT()
struct FAvaRundownPageChannelChanged : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** Specified the modified page id. */
	UPROPERTY()
	int32 PageId = FAvaRundownPage::InvalidPageId;

	/** Channel the page is currently assigned to (post modification). */
	UPROPERTY()
	FString ChannelName;
};

/**
 * Event sent when a page's name is modified.
 */
USTRUCT()
struct FAvaRundownPageNameChanged : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** Specified the modified page id. */
	UPROPERTY()
	int32 PageId = FAvaRundownPage::InvalidPageId;

	/** new page name is currently assigned to (post modification). */
	UPROPERTY()
	FString PageName;

	/** Indicate whether the name or friendly name that changed. */ 
	UPROPERTY()
	bool bFriendlyName = true;
};

/**
 * Event sent when a page's animation settings is modified.
 */
USTRUCT()
struct FAvaRundownPageAnimSettingsChanged : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** Specified the modified page id. */
	UPROPERTY()
	int32 PageId = FAvaRundownPage::InvalidPageId;
};

/**
 * Sets the channel of the given page.
 * The page must be valid (and not a template) and the channel must exist in the current profile.
 * Along with the corresponding response, this will also trigger a FAvaRundownPageChannelChanged event.
 */
USTRUCT()
struct FAvaRundownPageChangeChannel : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** Specifies the page that will be modified. */
	UPROPERTY()
	int32 PageId = FAvaRundownPage::InvalidPageId;

	/** Specifies a valid channel to set for the specified page. */
	UPROPERTY()
	FString ChannelName;
};

/**
 * Sets page name. Works for template or instance pages.
 * By default, the command will set the page's "friendly" name as it is the one used for
 * display purposes. The page name is reserved for native code uses.
 * Along with the corresponding response, this will also trigger a FAvaRundownPageNameChanged event.
 */
USTRUCT()
struct FAvaRundownChangePageName : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/** Rundown asset path: [PackagePath]/[AssetName].[AssetName] */
	UPROPERTY()
	FString Rundown;

	/** Specifies the page or template that will be modified. */
	UPROPERTY()
	int32 PageId = FAvaRundownPage::InvalidPageId;

	/** Specifies the new page name. */
	UPROPERTY()
	FString PageName;

	/**
	 * If true, the page's friendly name will be set.
	 * The page name is usually set by the native code.
	 * For display purposes, it is preferable to use the "friendly" name.
	 */
	UPROPERTY()
	bool bSetFriendlyName = true;
};


/** This is a request to save the managed Remote Control Preset (RCP) back to the corresponding page values. */
USTRUCT()
struct FAvaRundownUpdatePageFromRCP : public FAvaRundownMsgBase
{
	GENERATED_BODY()

public:
	/** Unregister the Remote Control Preset from the WebRC. */
	UPROPERTY()
	bool bUnregister = false;
};

/** Supported Page actions for playback. */
UENUM()
enum class EAvaRundownPageActions
{
	None UMETA(Hidden),
	Load,
	Unload,
	Play,
	PlayNext,
	Stop,
	ForceStop,
	Continue,
	UpdateValues,
	TakeToProgram
};

/**
 * Request for a program page command on the current playback rundown.
 */
USTRUCT()
struct FAvaRundownPageAction : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Specifies the Page Id that is the target of this action command. */
	UPROPERTY()
	int32 PageId = FAvaRundownPage::InvalidPageId;

	/** Specifies the page action to execute. */
	UPROPERTY()
	EAvaRundownPageActions Action = EAvaRundownPageActions::None;
};

/**
 * Request for a preview page command on the current playback rundown.
 */
USTRUCT()
struct FAvaRundownPagePreviewAction : public FAvaRundownPageAction
{
	GENERATED_BODY()
public:
	/** Specifies which preview channel to use. If left empty, the rundown's default preview channel is used. */
	UPROPERTY()
	FString PreviewChannelName;
};

/**
 * Command to execute a program action on multiple pages at the same time.
 * This is necessary for pages to be part of the same transition.
 */
USTRUCT()
struct FAvaRundownPageActions : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Specifies a list of page Ids that are the target of this action command. */
	UPROPERTY()
	TArray<int32> PageIds;

	/** Specifies the page action to execute. */
	UPROPERTY()
	EAvaRundownPageActions Action = EAvaRundownPageActions::None;
};

/**
 * Command to execute a preview action on multiple pages at the same time.
 * This is necessary for pages to be part of the same transition.
 */
USTRUCT()
struct FAvaRundownPagePreviewActions : public FAvaRundownPageActions
{
	GENERATED_BODY()
public:
	/** Specifies which preview channel to use. If left empty, the rundown's default preview channel is used. */
	UPROPERTY()
	FString PreviewChannelName;
};

/** Supported Transition actions for playback. */
UENUM()
enum class EAvaRundownTransitionActions
{
	None UMETA(Hidden),
	/** This action will forcefully stop specified transitions. */
	ForceStop
};

/**
 * Command to override transition logic directly.
 * As it currently stands, we can only have 1 transition per channel.
 * If there is an issue with it, it may block further playback.
 */
USTRUCT()
struct FAvaRundownTransitionAction : public FAvaRundownMsgBase
{
	GENERATED_BODY()

public:
	/** Specifies the channel that is the target of this action command. */
	UPROPERTY()
	FString ChannelName;

	/** Specifies the page transition action to execute. */
	UPROPERTY()
	EAvaRundownTransitionActions Action = EAvaRundownTransitionActions::None;
};

/** Supported Page Logic Layer actions for playback. */
UENUM()
enum class EAvaRundownTransitionLayerActions
{
	None UMETA(Hidden),
	/** Trigger the out transition for the specified layer. */
	Stop,
	/** Forcefully stop, without transition, pages on the specified layer. */
	ForceStop
};

/**
 * Command to override transition logic.
 */
USTRUCT()
struct FAvaRundownTransitionLayerAction : public FAvaRundownMsgBase
{
	GENERATED_BODY()

public:
	/** Specifies the channel that is the target of this action command. */
	UPROPERTY()
	FString ChannelName;

	/** Specifies the transition logic layers for this action command. */
	UPROPERTY()
	TArray<FString> LayerNames;
	
	/** Specifies the page layer action to execute. */
	UPROPERTY()
	EAvaRundownTransitionLayerActions Action = EAvaRundownTransitionLayerActions::None;
};

/**
 * This message is sent by the server when a page sequence event occurs.
 */
USTRUCT()
struct FAvaRundownPageSequenceEvent : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/** Specifies the broadcast channel the event occurred in. */
	UPROPERTY()
	FString Channel;

	/** Page Id associated with this event. */
	UPROPERTY()
	int32 PageId = FAvaRundownPage::InvalidPageId;

	/** Playable Instance uuid. */
	UPROPERTY()
	FGuid InstanceId;

	/** Full asset path: /PackagePath/PackageName.AssetName */
	UPROPERTY()
	FString AssetPath;

	/** Specifies the label used to identify the sequence. */
	UPROPERTY()
	FString SequenceLabel;

	/** Started, Paused, Finished */
	UPROPERTY()
	EAvaPlayableSequenceEventType Event = EAvaPlayableSequenceEventType::None;
};

UENUM()
enum class EAvaRundownPageTransitionEvents
{
	None UMETA(Hidden),
	Started,
	Finished
};

/**
 * This message is sent by the server when a page transition event occurs.
 */
USTRUCT()
struct FAvaRundownPageTransitionEvent : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/** Specifies the broadcast channel the event occurred in. */
	UPROPERTY()
	FString Channel;

	/** UUID of the transition. */
	UPROPERTY()
	FGuid TransitionId;

	/** Pages that are entering the scene during this transition. */
	UPROPERTY()
	TArray<int32> EnteringPageIds;

	/** Pages that are already in the scene. May get kicked out or change during this transition. */
	UPROPERTY()
	TArray<int32> PlayingPageIds;

	/** Pages that are requested to exit the scene during this transition. Typically part of a "Take Out" transition.  */
	UPROPERTY()
	TArray<int32> ExitingPageIds;

	/** Started, Finished */
	UPROPERTY()
	EAvaRundownPageTransitionEvents Event = EAvaRundownPageTransitionEvents::None;
};

/**
 * Requests a list of all profiles loaded for the current broadcast configuration.
 * Response is FAvaRundownProfiles.
 */
USTRUCT()
struct FAvaRundownGetProfiles : public FAvaRundownMsgBase
{
	GENERATED_BODY()
};

/**
 * Response to FAvaRundownGetProfiles.
 * Contains the list of all profiles in the broadcast configuration.
 */
USTRUCT()
struct FAvaRundownProfiles : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/** List of all profiles. */
	UPROPERTY()
	TArray<FString> Profiles;

	/** Current Active Profile. */
	UPROPERTY()
	FString CurrentProfile;
};

/**
 * Creates a new empty profile with the given name.
 * Fails if the profile already exist.
 */
USTRUCT()
struct FAvaRundownCreateProfile : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Name to be given to the newly created profile. */
	UPROPERTY()
	FString ProfileName;

	/**
	 * If true the created profile is also made "current".
	 * Equivalent to FAvaRundownSetCurrentProfile.
	 */
	UPROPERTY()
	bool bMakeCurrent = true;
};

/**
 * Duplicates an existing profile.
 * Fails if the new profile name already exist.
 * Fails if the source profile does not exist.
 */
USTRUCT()
struct FAvaRundownDuplicateProfile : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Specifies the existing profile to be duplicated. */
	UPROPERTY()
	FString SourceProfileName;

	/** Specifies the name of the new profile to be created. */
	UPROPERTY()
	FString NewProfileName;

	/**
	 * If true the created profile is also made "current".
	 * Equivalent to FAvaRundownSetCurrentProfile.
	 */
	UPROPERTY()
	bool bMakeCurrent = true;
};

/**
 * Renames an existing profile.
 */
USTRUCT()
struct FAvaRundownRenameProfile : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Specifies the name of the existing profile to be renamed. */
	UPROPERTY()
	FString OldProfileName;

	/** Specifies the new name. */
	UPROPERTY()
	FString NewProfileName;
};

/**
 * Deletes the specified profile.
 * Fails if profile to be deleted is the current profile.
 */
USTRUCT()
struct FAvaRundownDeleteProfile : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Specifies the target profile. */
	UPROPERTY()
	FString ProfileName;
};

/**
 * Specified profile is made "current".
 * The current profile becomes the context for all other broadcasts commands.
 * Fails if some channels are currently broadcasting.
 */
USTRUCT()
struct FAvaRundownSetCurrentProfile : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Specifies the requested profile. */
	UPROPERTY()
	FString ProfileName;
};

/**
 * Output Device information
 */
USTRUCT()
struct FAvaRundownOutputDeviceItem
{
	GENERATED_BODY()
public:
	/**
	 * Specifies the device name.
	 * This is used as "MediaOutputName" in FAvaRundownAddChannelDevice and FAvaRundownEditChannelDevice.
	 */
	UPROPERTY()
	FString Name;

	/** Extra information about the device. */
	UPROPERTY()
	FAvaBroadcastMediaOutputInfo OutputInfo;

	/** Specifies the status of the output device. */
	UPROPERTY()
	EAvaBroadcastOutputState OutputState = EAvaBroadcastOutputState::Invalid;

	/** In case the device is live, this extra status indicates if the device is operating normally. */
	UPROPERTY()
	EAvaBroadcastIssueSeverity IssueSeverity = EAvaBroadcastIssueSeverity::None;

	/** List of errors or warnings. */ 
	UPROPERTY()
	TArray<FString> IssueMessages;

	/**
	 * Raw Json string representing a serialized UMediaOutput.
	 * This data can be edited, then used in FAvaRundownEditChannelDevice.
	 */
	UPROPERTY()
	FString Data;
};

/**
 * Output Device Class Information
 */
USTRUCT()
struct FAvaRundownOutputClassItem
{
	GENERATED_BODY()
public:
	/** Class name */
	UPROPERTY()
	FString Name;

	/**
	 * Name of the playback server this class was seen on.
	 * The name will be empty for the "local process" device.
	 */
	UPROPERTY()
	FString Server;

	/**
	 * Enumeration of the available devices of this class on the given host.
	 * Note that not all classes can be enumerated.
	 */
	UPROPERTY()
	TArray<FAvaRundownOutputDeviceItem> Devices;
};

/**
 * Response to FAvaRundownGetDevices.
 */
USTRUCT()
struct FAvaRundownDevicesList : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/**
	 * List of Output Device Classes
	 */
	UPROPERTY()
	TArray<FAvaRundownOutputClassItem> DeviceClasses;
};

/**
 * Requests information (devices, status, etc) on a specified channel.
 * 
 * Response is FAvaRundownChannelResponse.
 */
USTRUCT()
struct FAvaRundownGetChannel : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Specifies the requested channel. */
	UPROPERTY()
	FString ChannelName;
};

/**
 * Requests information (devices, status, etc) on all channels of the current profile.
 * 
 * Response is FAvaRundownChannels.
 */
USTRUCT()
struct FAvaRundownGetChannels : public FAvaRundownMsgBase
{
	GENERATED_BODY()
};

/** Channel Information */
USTRUCT()
struct FAvaRundownChannel
{
	GENERATED_BODY()
public:
	/** Specifies the Channel Name. */
	UPROPERTY()
	FString Name;

	UPROPERTY()
	EAvaBroadcastChannelType Type = EAvaBroadcastChannelType::Program;

	UPROPERTY()
	EAvaBroadcastChannelState State = EAvaBroadcastChannelState::Offline;

	UPROPERTY()
	EAvaBroadcastIssueSeverity IssueSeverity = EAvaBroadcastIssueSeverity::None;

	/** List of devices. */
	UPROPERTY()
	TArray<FAvaRundownOutputDeviceItem> Devices;
};

/**
 * This message is sent by the server if the list of channels is modified
 * in the current profile. Channel added, removed, pinned or type (preview vs program) changed.
 */
USTRUCT()
struct FAvaRundownChannelListChanged : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/** List of channel information. */
	UPROPERTY()
	TArray<FAvaRundownChannel> Channels;
};

/**
 * This message is sent by the server in response to FAvaRundownGetChannel or
 * as an event if a channel's states, render target, devices or settings is changed.
 */
USTRUCT()
struct FAvaRundownChannelResponse : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Channel Information */
	UPROPERTY()
	FAvaRundownChannel Channel;
};

/** Response to FAvaRundownGetChannels */
USTRUCT()
struct FAvaRundownChannels : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** List of channel information. */
	UPROPERTY()
	TArray<FAvaRundownChannel> Channels;
};

/**
 * Generic asset event
 */
UENUM()
enum class EAvaRundownAssetEvent : uint8
{
	Unknown = 0 UMETA(Hidden),
	Added,
	Removed,
	//Saved, // todo
	//Modified // todo
};

/**
 * Event broadcast when an asset event occurs on the server.
 */
USTRUCT()
struct FAvaRundownAssetsChanged : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/** Asset name only, without the package path. (Keeping for legacy) */
	UPROPERTY()
	FString AssetName;

	/** Full asset path: /PackagePath/PackageName.AssetName */
	UPROPERTY()
	FString AssetPath;

	/** Full asset class path. */
	UPROPERTY()
	FString AssetClass;

	/** true if the asset is a "playable" asset, i.e. an asset that can be set in a page's asset. */
	UPROPERTY()
	bool bIsPlayable = false;

	/** Specifies the event type, i.e. Added, Remove, etc. */
	UPROPERTY()
	EAvaRundownAssetEvent EventType = EAvaRundownAssetEvent::Unknown;
};

/**
 * Channel broadcast actions
 */
UENUM()
enum class EAvaRundownChannelActions
{
	None UMETA(Hidden),
	/** Start broadcast of the specified channel(s). */
	Start,
	/** Stops broadcast of the specified channel(s). */
	Stop
};

/**
 * Requests a broadcast action on the specified channel(s).
 */
USTRUCT()
struct FAvaRundownChannelAction : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/**
	 * Specifies the target channel for the action.
	 * If left empty, the action will apply to all channels of the current profile.
	 */
	UPROPERTY()
	FString ChannelName;

	/** Specifies the broadcast action to perform on the target channel(s). */
	UPROPERTY()
	EAvaRundownChannelActions Action = EAvaRundownChannelActions::None;
};

UENUM()
enum class EAvaRundownChannelEditActions
{
	None UMETA(Hidden),
	/** Add new channel with given name. */
	Add,
	/** Removes channel with given name. */
	Remove
};

/** Requests an edit action on the specified channel. */
USTRUCT()
struct FAvaRundownChannelEditAction : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Specifies the target channel for the action. */
	UPROPERTY()
	FString ChannelName;

	/** Specifies the edit action to perform on the target channel. */
	UPROPERTY()
	EAvaRundownChannelEditActions Action = EAvaRundownChannelEditActions::None;
};

/** Requests a channel to be renamed. */
USTRUCT()
struct FAvaRundownRenameChannel : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Existing channel to be renamed. */
	UPROPERTY()
	FString OldChannelName;

	/** Specifies the new channel name. */
	UPROPERTY()
	FString NewChannelName;
};

/**
 * Requests a list of devices from the rundown server.
 * The server will reply with FAvaRundownDevicesList containing
 * the devices that can be enumerated from the local host and all connected hosts
 * through the motion design playback service.
 */
USTRUCT()
struct FAvaRundownGetDevices : public FAvaRundownMsgBase
{
	GENERATED_BODY()

	/**
	 * If true, listing all media output classes on the server, even if they don't have a device provider.
	 */
	UPROPERTY()
	bool bShowAllMediaOutputClasses = false;
};

/**
 * Add an enumerated device to the given channel.
 * This command will fail if the channel is live.
 */
USTRUCT()
struct FAvaRundownAddChannelDevice : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Specifies the target channel. */
	UPROPERTY()
	FString ChannelName;

	/**
	 * The specified name is one of the enumerated device from FAvaRundownDevicesList,
	 * FAvaRundownOutputDeviceItem::Name.
	 */
	UPROPERTY()
	FString MediaOutputName;

	/** Save broadcast configuration after this operation (true by default). */
	UPROPERTY()
	bool bSaveBroadcast = true;
};

/**
 * Modify an existing device in the given channel.
 * This command will fail if the channel is live.
 */
USTRUCT()
struct FAvaRundownEditChannelDevice : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Specifies the target channel. */
	UPROPERTY()
	FString ChannelName;

	/**
	 * The specified name is one of the enumerated device from FAvaRundownChannel::Devices,
	 * FAvaRundownOutputDeviceItem::Name field.
	 * Must be the instanced devices from either FAvaRundownChannels, FAvaRundownChannelResponse
	 * or FAvaRundownChannelListChanged. These names are not the same as when adding a device.
	 */
	UPROPERTY()
	FString MediaOutputName;

	/**
	 * (Modified) Device Data in the same format as FAvaRundownOutputDeviceItem::Data.
	 * See: FAvaRundownChannel, FAvaRundownDevicesList
	 */
	UPROPERTY()
	FString Data;

	/** Save broadcast configuration after this operation (true by default). */
	UPROPERTY()
	bool bSaveBroadcast = true;
};

/**
 * Remove an existing device from the given channel.
 * This command will fail if the channel is live.
 */
USTRUCT()
struct FAvaRundownRemoveChannelDevice : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Specifies the target channel. */
	UPROPERTY()
	FString ChannelName;

	/**
	 * The specified name is one of the enumerated device from FAvaRundownChannel::Devices,
	 * FAvaRundownOutputDeviceItem::Name field.
	 * Must be the instanced devices from either FAvaRundownChannels, FAvaRundownChannelResponse
	 * or FAvaRundownChannelListChanged. These names are not the same as when adding a device.
	 */
	UPROPERTY()
	FString MediaOutputName;

	/** Save broadcast configuration after this operation (true by default). */
	UPROPERTY()
	bool bSaveBroadcast = true;
};

/**
 * Captures an image from the specified channel.
 * The captured image is 25% of the channel's resolution.
 * Intended for preview.
 * Response is FAvaRundownChannelImage.
 */
USTRUCT()
struct FAvaRundownGetChannelImage : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Specifies the target channel. */
	UPROPERTY()
	FString ChannelName;
};

/**
 * Response to FAvaRundownGetChannelImage.
 */
USTRUCT()
struct FAvaRundownChannelImage : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/**
	 * Byte array containing the image data.
	 * Expected format is compressed jpeg.
	 */ 
	UPROPERTY()
	TArray<uint8> ImageData;
};

/**
 * Queries the given channel's quality settings.
 * Response is FAvaRundownChannelQualitySettings.
 */
USTRUCT()
struct FAvaRundownGetChannelQualitySettings : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Specifies the target channel. */
	UPROPERTY()
	FString ChannelName;
};

/** Response to FAvaRundownGetChannelQualitySettings. */
USTRUCT()
struct FAvaRundownChannelQualitySettings : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Specifies the target channel. */
	UPROPERTY()
	FString ChannelName;

	/** Advanced viewport client engine features indexed by FEngineShowFlags names. */
	UPROPERTY()
	TArray<FAvaViewportQualitySettingsFeature> Features;
};

/** Sets the given channel's quality settings. */
USTRUCT()
struct FAvaRundownSetChannelQualitySettings : public FAvaRundownMsgBase
{
	GENERATED_BODY()
public:
	/** Specifies the target channel. */
	UPROPERTY()
	FString ChannelName;

	/** Advanced viewport client engine features indexed by FEngineShowFlags names. */
	UPROPERTY()
	TArray<FAvaViewportQualitySettingsFeature> Features;
};

/**
 * Save current broadcast configuration to a json file in the Config folder on the server. 
 */
USTRUCT()
struct FAvaRundownSaveBroadcast : public FAvaRundownMsgBase
{
	GENERATED_BODY()
};