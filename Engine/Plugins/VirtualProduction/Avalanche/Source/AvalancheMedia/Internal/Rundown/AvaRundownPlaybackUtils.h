// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/AvaRundownPage.h"

class UAvaRundown;

struct AVALANCHEMEDIA_API FAvaRundownPlaybackUtils
{
	static bool IsPageIdValid(int32 InPageId)
	{
		return InPageId != FAvaRundownPage::InvalidPageId;
	}
	
	static int32 GetPageIdToPlayNext(const UAvaRundown* InRundown, const FAvaRundownPageListReference& InPageListReference, bool bInPreview, FName InPreviewChannelName);
	
	static TArray<int32> GetPagesToTakeToProgram(const UAvaRundown* InRundown, const TArray<int32>& InSelectedPageIds, FName InPreviewChannel = NAME_None);

	/** Version of AddUnique that will only add the tag handle to the array if it does not have an ExactMatch. */
	static void AddTagHandleUnique(TArray<FAvaTagHandle>& InTagHandles, const FAvaTagHandle& InTagHandleToAdd)
	{
		const bool bTagAlreadyAdded = InTagHandles.ContainsByPredicate([&InTagHandleToAdd](const FAvaTagHandle& InTagHandle)
		{
			return InTagHandle.MatchesExact(InTagHandleToAdd); // Exact math: Source and TagId.
		});
			
		if (!bTagAlreadyAdded)
		{
			InTagHandles.Add(InTagHandleToAdd);
		}
	}
};