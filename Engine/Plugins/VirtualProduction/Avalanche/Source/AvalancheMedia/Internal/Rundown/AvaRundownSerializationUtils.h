// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class FArchive;
class FString;
class FText;
class UAvaRundown;
class UStruct;
struct FAvaRundownPageCommand;

namespace UE::AvaMedia::RundownSerializationUtils
{
	AVALANCHEMEDIA_API bool SaveRundownToJson(const UAvaRundown* InRundown, FArchive& InArchive, FText& OutErrorMessage);
	AVALANCHEMEDIA_API bool SaveRundownToJson(const UAvaRundown* InRundown, const TCHAR* InFilepath, FText& OutErrorMessage);

	AVALANCHEMEDIA_API bool LoadRundownFromJson(UAvaRundown* InRundown, FArchive& InArchive, FText& OutErrorMessage);
	AVALANCHEMEDIA_API bool LoadRundownFromJson(UAvaRundown* InRundown, const TCHAR* InFilepath, FText& OutErrorMessage);

	AVALANCHEMEDIA_API bool SerializeRundownPageCommandToJsonString(const FAvaRundownPageCommand* InRundownPageCommand, UStruct& InStruct, FString& OutString);
	AVALANCHEMEDIA_API bool DeserializeRundownPageCommandFromJson(FAvaRundownPageCommand* InRundownPageCommand, UStruct& InStruct, const FString& InString);
}