// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

class UMediaOutput;

/*
 *	Class designed to deal with Serialization and Editing
 *	of Media Output device data coming from Rundown Server
 *
 *	It is meant to give full control and provide all information to Rundown server
 *	in a form that is usable outside of unreal engine
 */
struct AVALANCHEMEDIA_API FAvaRundownServerMediaOutputUtils
{
	static FString SerializeMediaOutput(const UMediaOutput* InMediaOutput);
	static void EditMediaOutput(UMediaOutput* InMediaOutput, const FString& InDeviceData);
};
