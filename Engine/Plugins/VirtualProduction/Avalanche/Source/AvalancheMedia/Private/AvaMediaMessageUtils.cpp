// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMediaMessageUtils.h"

uint32 UE::AvaMediaMessageUtils::GetSafeMessageSizeLimit()
{
	// Testing with various message sizes over LAN has shown that the UDP
	// segmenter is struggling with larger messages. 
	// This value is taken from MultiUser ShouldEmbedPackageDataAsByteArray().
	// Coincidentally, it is the same as UDP_MESSAGING_RECEIVE_BUFFER_SIZE.
	static constexpr uint32 SafeDataSizeLimitForUdpSegmenter = 2 * 1024 * 1024;	
	return SafeDataSizeLimitForUdpSegmenter;
}