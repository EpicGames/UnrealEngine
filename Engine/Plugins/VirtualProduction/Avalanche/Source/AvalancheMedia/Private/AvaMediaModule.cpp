// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMediaModule.h"

#include "Application/ThrottleManager.h"
#include "AvaMediaSettings.h"
#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/OutputDevices/AvaBroadcastDeviceProviderRegistry.h"
#include "Broadcast/OutputDevices/AvaBroadcastRenderTargetMediaUtils.h"
#include "IMediaIOCoreModule.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "ModularFeature/AvaMediaSynchronizedEventsFeature.h"
#include "Playable/AvaPlayableGroupSceneViewExtension.h"
#include "Playable/AvaPlayableRemoteControlPresetInfoCache.h"
#include "Playback/AvaPlaybackClientDelegates.h"
#include "Playback/AvaPlaybackClientDummy.h"
#include "SceneViewExtension.h"
#include "ShaderCore.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

DEFINE_LOG_CATEGORY(LogAvaMedia);

namespace UE::AvaMediaModule::Private
{
	// Command line parsing helper.
	bool IsPlaybackServerManuallyStarted(FString& OutPlaybackServerName)
	{
		return FParse::Value(FCommandLine::Get(),TEXT("MotionDesignPlaybackServerStart="), OutPlaybackServerName) ||
		 FParse::Param(FCommandLine::Get(), TEXT("MotionDesignPlaybackServerStart"));
	}

	bool IsDisplayClusterNode(FString& OutDisplayClusterNodeName)
	{
		return FParse::Value(FCommandLine::Get(),TEXT("dc_node="), OutDisplayClusterNodeName);
	}

	// Command line parsing helper.
	bool IsRundownServerManuallyStarted(FString& OutRundownServerName)
	{
		return FParse::Value(FCommandLine::Get(), TEXT("MotionDesignRundownServerStart="), OutRundownServerName) ||
			FParse::Param(FCommandLine::Get(), TEXT("MotionDesignRundownServerStart"));
	}
}

FAvaMediaModule::FAvaMediaModule()
: BroadcastSettingsBridge(this)
{}

void FAvaMediaModule::StartupModule()
{
	using namespace UE::AvaMediaModule::Private;
	
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	const FString PluginShaderDir = FPaths::Combine(Plugin->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(UE::AvaBroadcastRenderTargetMediaUtils::VirtualShaderMountPoint, PluginShaderDir);

	FAvaMediaSynchronizedEventsFeature::Startup();

	IMediaIOCoreModule::Get().RegisterDeviceProvider(&AvaDisplayDeviceProvider);

	ConsoleCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("MotionDesignPlaybackServer.Start"),
				TEXT("Starts the playback server. Arguments: [ServerName]\n")
				TEXT("ServerName\tSpecifies the server name. Will use computer name if left empty."),
				FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaMediaModule::StartPlaybackServerCommand),
				ECVF_Default
				));
	ConsoleCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("MotionDesignPlaybackServer.Stop"),
				TEXT("Stops the playback server."),
				FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaMediaModule::StopPlaybackServerCommand),
				ECVF_Default
				));
	ConsoleCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("MotionDesignPlaybackClient.Start"),
				TEXT("Starts the playback client. Arguments: [-Force]\n")
				TEXT("-Force\tStops playback server and force client to start."),
				FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaMediaModule::StartPlaybackClientCommand),
				ECVF_Default
				));
	ConsoleCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("MotionDesignPlaybackClient.Stop"),
				TEXT("Stops the playback client."),
				FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaMediaModule::StopPlaybackClientCommand),
				ECVF_Default
				));
	ConsoleCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("MotionDesignPlaybackLocalServer.Launch"),
				TEXT("Launches the local playback server."),
				FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaMediaModule::LaunchLocalPlaybackServerCommand),
				ECVF_Default
				));
	ConsoleCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("MotionDesignPlaybackLocalServer.Stop"),
				TEXT("Stops the local playback server."),
				FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaMediaModule::StopLocalPlaybackServerCommand),
				ECVF_Default
				));
	ConsoleCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("MotionDesignPlaybackHttpServer.Start"),
				TEXT("Starts the http playback server."),
				FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaMediaModule::StartHttpPlaybackServerCommand),
				ECVF_Default
				));
	ConsoleCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("MotionDesignPlaybackHttpServer.Stop"),
				TEXT("Stops the http playback server."),
				FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaMediaModule::StopHttpPlaybackServerCommand),
				ECVF_Default
				));
	ConsoleCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("MotionDesignRundownServer.Start"),
				TEXT("Starts the rundown server. Arguments: [ServerName]\n")
				TEXT("ServerName\tSpecifies the server name. Will use computer name if left empty."),
				FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaMediaModule::StartRundownServerCommand),
				ECVF_Default
				));
	ConsoleCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("MotionDesignRundownServer.Stop"),
				TEXT("Stops the rundown server."),
				FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaMediaModule::StopRundownServerCommand),
				ECVF_Default
				));
	ConsoleCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("MotionDesignPlaybackDevices.Save"),
				TEXT("Save Device Providers data. Arguments: [ServerName]\n")
				TEXT("ServerName\tSpecifies the server name. Will use computer name if left empty."),
				FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaMediaModule::SaveDeviceProvidersCommand),
				ECVF_Default
				));
	ConsoleCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("MotionDesignPlaybackDevices.Load"),
				TEXT("Load device providers."),
				FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaMediaModule::LoadDeviceProvidersCommand),
				ECVF_Default
				));
	ConsoleCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("MotionDesignPlaybackDevices.Unload"),
				TEXT("Load device providers."),
				FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaMediaModule::UnloadDeviceProvidersCommand),
				ECVF_Default
				));
	ConsoleCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("MotionDesignPlaybackDevices.List"),
				TEXT("List device providers."),
				FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaMediaModule::ListDeviceProvidersCommand),
				ECVF_Default
				));
	ConsoleCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
				TEXT("MotionDesignPlayback.Stat"),
				TEXT("Enable engine performance statistics. Same as 'stat' command but will affect Motion Design Playback outputs and propagate to connected servers."),
				FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaMediaModule::HandleStatCommand),
				ECVF_Default
				));

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FAvaMediaModule::PostEngineInit);
	FCoreDelegates::OnEnginePreExit.AddRaw(this, &FAvaMediaModule::EnginePreExit);
#if WITH_EDITOR
	FEditorDelegates::PrePIEEnded.AddRaw(this, &FAvaMediaModule::PrePIEEnded);
#endif
	
	FString DummyServerName;
	if (IsPlaybackServerManuallyStarted(DummyServerName) || IsRundownServerManuallyStarted(DummyServerName))
	{
		// Prevent throttling when server is started.
		// This has to be done before any SLevelViewport are ticked since the cvar value is cached on first tick.
		static const FSlateThrottleManager & ThrottleManager = FSlateThrottleManager::Get();
		if (IConsoleVariable* AllowThrottling = IConsoleManager::Get().FindConsoleVariable(TEXT("Slate.bAllowThrottling")))
		{
			AllowThrottling->Set(0);
			UE_LOGF(LogAvaMedia, Log, "Setting Slate.bAllowThrottling to false.");
		}
	}

	AvaMediaSync = MakeUnique<FAvaMediaSync>();

	// StormSyncAvaBridge has some issues with servers in game mode. This option
	// allows us to disable some of it until all the bugs are fixed.
	if (FParse::Param(FCommandLine::Get(), TEXT("DisableMotionDesignSync")))
	{
		AvaMediaSync->SetFeatureEnabled(false);
	}

	RemoteControlPresetInfoCache = MakeShared<FAvaPlayableRemoteControlPresetInfoCache>();
}

void FAvaMediaModule::ShutdownModule()
{
	StopAllServices();
	
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
#if WITH_EDITOR
	FEditorDelegates::PrePIEEnded.RemoveAll(this);
#endif

	if (IMediaIOCoreModule::IsAvailable())
	{
		IMediaIOCoreModule::Get().UnregisterDeviceProvider(&AvaDisplayDeviceProvider);
	}

	for (IConsoleObject* ConsoleCmd : ConsoleCmds)
	{
		IConsoleManager::Get().UnregisterConsoleObject(ConsoleCmd);
	}
	ConsoleCmds.Empty();

	AvaMediaSync.Reset();
	FAvaMediaSynchronizedEventsFeature::Shutdown();
}

void FAvaMediaModule::StartPlaybackClient()
{
	TArray<FString> CommandArgs;

	// In editor mode, the client has priority over the server.
	const bool bIsEditorMode = GIsEditor && !IsRunningGame();
	if (bIsEditorMode)
	{
		CommandArgs.Add(TEXT("-Force"));
	}

	StartPlaybackClientCommand(CommandArgs);
}

void FAvaMediaModule::StopPlaybackClient()
{
	StopPlaybackClientCommand({});
}

void FAvaMediaModule::StartPlaybackServer(const FString& InPlaybackServerName)
{
	StartPlaybackServerCommand({InPlaybackServerName});
}

void FAvaMediaModule::StopPlaybackServer()
{
	StopPlaybackServerCommand({});
}

void FAvaMediaModule::StartRundownServer(const FString& InRundownServerName)
{
	StartRundownServerCommand({InRundownServerName});
}

void FAvaMediaModule::StopRundownServer()
{
	StopRundownServerCommand({});
}

TSharedPtr<IAvaRundownServer> FAvaMediaModule::MakeDetachedRundownServer(const FString& InServerName)
{
	const TSharedRef<FAvaRundownServer, ESPMode::ThreadSafe> DetachedRundownServer = MakeShared<FAvaRundownServer>();
	DetachedRundownServer->Init(InServerName);
	OnRundownServerStarted.Broadcast(DetachedRundownServer);
	return DetachedRundownServer;
}

IAvaPlaybackClient& FAvaMediaModule::GetPlaybackClient()
{
	if (AvaPlaybackClient.IsValid())
	{
		return *AvaPlaybackClient;
	}
	
	static FAvaPlaybackClientDummy DummyClient;
	return DummyClient;
}

const IMediaIOCoreDeviceProvider* FAvaMediaModule::GetDeviceProvider(FName InProviderName, const FMediaIOOutputConfiguration* InMediaIOOutputConfiguration) const
{
	if (AvaPlaybackClient.IsValid() && InMediaIOOutputConfiguration)
	{
		const FAvaBroadcastDeviceProviderWrapper* Wrapper = DeviceProviderProxyManager.GetDeviceProviderWrapper(InProviderName);
		if (Wrapper)
		{
			return Wrapper->GetProviderForDeviceName(InMediaIOOutputConfiguration->MediaConfiguration.MediaConnection.Device.DeviceName);
		}
	}
	return IMediaIOCoreModule::Get().GetDeviceProvider(InProviderName);
}

TArray<const IMediaIOCoreDeviceProvider*> FAvaMediaModule::GetDeviceProvidersForServer(const FString& InServerName) const
{
	if (AvaPlaybackClient.IsValid())
	{
		return DeviceProviderProxyManager.GetDeviceProvidersForServer(InServerName);
	}
	else
	{
		return TArray<const IMediaIOCoreDeviceProvider*>();
	}
}

FString FAvaMediaModule::GetServerNameForDevice(const FName& InDeviceProviderName, const FName& InDeviceName) const
{
	// (New Method) Search in the device provider proxies.
	// This method doesn't assume the device name starts with the server name.
	FString FoundServerName = DeviceProviderProxyManager.FindServerNameForDevice(InDeviceProviderName, InDeviceName);
	if (!FoundServerName.IsEmpty())
	{
		return FoundServerName;
	}

	// Fallback (legacy method)
	// Assumes the device name starts with the server name.
	if (AvaPlaybackClient.IsValid())
	{
		const FString DeviceName = InDeviceName.ToString();
		TArray<FString> ServerNames = AvaPlaybackClient->GetServerNames();
		for (const FString& ServerName : ServerNames)
		{
			if (DeviceName.StartsWith(ServerName))
			{
				return ServerName;
			}
		}
	}
	
	return FString();
}

bool FAvaMediaModule::IsLocalDevice(const FName& InDeviceProviderName, const FName& InDeviceName) const
{
	return DeviceProviderProxyManager.IsLocalDevice(InDeviceProviderName, InDeviceName);
}

void FAvaMediaModule::LaunchGameModeLocalPlaybackServer()
{
	// In order to talk to the server, we need the client.
	if (!IsPlaybackClientStarted())
	{
		// For now, we issue an error.
		UE_LOGF(LogAvaMedia, Error, "Playback Client must be started prior to starting a local playback server.");
		return;

		// TODO: We could automatically start the playback client.
		// However, we would need to wait for the first round of server discovery to complete prior to
		// search for the "local" server process. There are no events for this currently.
		//StartPlaybackClientCommand({});
	}

	if (!LocalPlaybackServerProcess.IsValid())
	{
		LocalPlaybackServerProcess = FAvaPlaybackServerProcess::FindOrCreate(*AvaPlaybackClient);
	}

	if (LocalPlaybackServerProcess.IsValid())
	{
		if (!LocalPlaybackServerProcess->IsLaunched())
		{
			LocalPlaybackServerProcess->Launch();
		}
	}
}

void FAvaMediaModule::StopGameModeLocalPlaybackServer()
{
	// We may not have a local handle to the server process if it was started by
	// another client instance.
	if (!LocalPlaybackServerProcess.IsValid() && AvaPlaybackClient.IsValid())
	{
		LocalPlaybackServerProcess = FAvaPlaybackServerProcess::Find(*AvaPlaybackClient);
	}
	
	if (LocalPlaybackServerProcess.IsValid() && LocalPlaybackServerProcess->IsLaunched())
	{
		LocalPlaybackServerProcess->Stop();
		UE_LOGF(LogAvaMedia, Log, "Local playback server has been stopped.");
	}

	LocalPlaybackServerProcess.Reset();
}

bool FAvaMediaModule::IsGameModeLocalPlaybackServerLaunched() const
{
	return LocalPlaybackServerProcess.IsValid() && LocalPlaybackServerProcess->IsLaunched();
}

const IAvaBroadcastSettings& FAvaMediaModule::GetBroadcastSettings() const
{
	return BroadcastSettingsBridge;
}

const FAvaInstanceSettings& FAvaMediaModule::GetAvaInstanceSettings() const
{
	// if the server is enabled, fetch the settings from the currently connected client.
	if (AvaPlaybackServer.IsValid())
	{
		if (const FAvaInstanceSettings* SettingsFromClient = AvaPlaybackServer->GetAvaInstanceSettings())
		{
			return *SettingsFromClient;
		}
	}
	// Return the local settings.
	return UAvaMediaSettings::Get().AvaInstanceSettings;
}

const FAvaPlayableSettings& FAvaMediaModule::GetPlayableSettings() const
{
	// if the server is enabled, fetch the settings from the currently connected client.
	if (AvaPlaybackServer.IsValid())
	{
		if (const FAvaPlayableSettings* SettingsFromClient = AvaPlaybackServer->GetPlayableSettings())
		{
			return *SettingsFromClient;
		}
	}
	// Return the local settings.
	return UAvaMediaSettings::Get().PlayableSettings;
}

bool FAvaMediaModule::IsLocalPlaybackManagerAvailable() const
{
	return LocalPlaybackManager.IsValid(); 
}

FAvaPlaybackManager& FAvaMediaModule::GetLocalPlaybackManager() const
{
	check(LocalPlaybackManager.IsValid());
	return *LocalPlaybackManager;
}

bool FAvaMediaModule::IsManagedInstanceCacheAvailable() const
{
	return ManagedInstanceCache.IsValid();
}

FAvaRundownManagedInstanceCache& FAvaMediaModule::GetManagedInstanceCache() const
{
	check(ManagedInstanceCache.IsValid());
	return *ManagedInstanceCache;
}

bool FAvaMediaModule::IsAvaMediaSyncProviderFeatureAvailable() const
{
	return AvaMediaSync.IsValid() && AvaMediaSync->IsFeatureAvailable(); 
}

IAvaMediaSyncProvider* FAvaMediaModule::GetAvaMediaSyncProvider() const
{
	check(AvaMediaSync.IsValid());
	return AvaMediaSync->GetCurrentProvider();
}

void FAvaMediaModule::NotifyMapChangedEvent(UWorld* InWorld, EAvaMediaMapChangeType InEventType)
{
	OnMapChangedEvent.Broadcast(InWorld, InEventType);
}

void FAvaMediaModule::PostEngineInit()
{
	using namespace UE::AvaMediaModule::Private;

	PlayableGroupSceneViewExtension = FSceneViewExtensions::NewExtension<FAvaPlayableGroupSceneViewExtension>();
	
	ConditionalCreateLocalPlaybackManager();
	ConditionalCreateManagedInstanceCache();

	// This needs to happen late in the loading process, otherwise it fails.
	const UAvaMediaSettings& Settings = UAvaMediaSettings::Get();

#if WITH_EDITOR
	// Initialise the device provider registry.
	FAvaBroadcastDeviceProviderRegistry::Get();
#endif
	
	// Allow for specification of the server name in the command line.
	// Command line has priority over project settings. 
	FString PlaybackServerName = Settings.PlaybackServerName;
	const bool bIsServerManuallyStarted = IsPlaybackServerManuallyStarted(PlaybackServerName);
	const bool bIsClientManuallyStarted = FParse::Param(FCommandLine::Get(), TEXT("MotionDesignPlaybackClientStart"));

	FString DisplayClusterNode;
	if (IsDisplayClusterNode(DisplayClusterNode))
	{
		if (PlaybackServerName.IsEmpty())
		{
			PlaybackServerName = FPlatformProcess::ComputerName();
		}
		PlaybackServerName += FString::Printf(TEXT("_%s"), *DisplayClusterNode);
	}
	
	// Adding a command to suppress the client from auto-starting. This is used when spawning
	// extra server process from the same project, while preventing extra clients.
	const bool bIsClientAutoStartSuppressed = FParse::Param(FCommandLine::Get(), TEXT("MotionDesignPlaybackClientSuppress"));
	
	bool bShouldStartClient = bIsClientManuallyStarted || (Settings.bAutoStartPlaybackClient && !bIsClientAutoStartSuppressed && !IsRunningCommandlet());
	bool bShouldStartServer = bIsServerManuallyStarted || (Settings.bAutoStartPlaybackServer && !IsRunningCommandlet());
	
	const bool bIsGameMode = !GIsEditor || IsRunningGame();

	// The playback client and server can't both run in the same process.
	// For the editor, we will keep the client and suppress the server.
	// For the game, we will keep the server and suppress the client.

	// In game mode, the auto start client is honored only if auto start server is not set.
	// In editor mode, the auto start server is honored only if auto start client is not set.
	
	if (bShouldStartServer && bShouldStartClient)
	{
		if (bIsGameMode)
		{
			UE_LOGF(LogAvaMedia, Log, "Auto start of Playback Client has been suppressed in game mode in favor of Playback Server.");
			bShouldStartClient = false; // In game mode, the client is suppressed.

		}
		else
		{
			UE_LOGF(LogAvaMedia, Log, "Auto start of Playback Server has been suppressed in editor mode in favor of Playback Client.");
			bShouldStartServer = false; // In editor mode, the server is suppressed.
		}
	}
	
	if (bShouldStartClient)
	{
		StartPlaybackClientCommand({});
	}

	if (bShouldStartServer)
	{
		StartPlaybackServerCommand({PlaybackServerName});
	}
	
	// Playback server required by Http server
	if (AvaPlaybackServer.IsValid() && Settings.bAutoStartWebServer)
	{
		StartHttpPlaybackServerCommand({});
	}

	// Allow for specification of the rundown server name in the command line.
	// Note: auto start of rundown server is only done in the editor module.
	FString RundownServerName;
	if (IsRundownServerManuallyStarted(RundownServerName))
	{
		StartRundownServerCommand({RundownServerName});
	}
}

void FAvaMediaModule::EnginePreExit()
{
	StopAllServices();

	PlayableGroupSceneViewExtension.Reset();
}

void FAvaMediaModule::PrePIEEnded(const bool)
{
	// Only Local Playback Manager should handle tear down for PIE End.
	if (LocalPlaybackManager)
	{
		LocalPlaybackManager->OnParentWorldBeginTearDown();
	}
}

void FAvaMediaModule::StopAllServices()
{
	StopRundownServerCommand({});
	StopPlaybackServerCommand({});
	StopPlaybackClientCommand({});
	
	if (LocalPlaybackManager)
	{
		LocalPlaybackManager->StartShuttingDown();
		LocalPlaybackManager->StopAllPlaybacks(true);	
	}
	LocalPlaybackManager.Reset();
	ManagedInstanceCache.Reset();
}

void FAvaMediaModule::ConditionalCreateLocalPlaybackManager()
{
	if (!LocalPlaybackManager)
	{
		LocalPlaybackManager = MakeShared<FAvaPlaybackManager>();
	}
}

void FAvaMediaModule::ConditionalCreateManagedInstanceCache()
{
	if (!ManagedInstanceCache)
	{
		ManagedInstanceCache = MakeShared<FAvaRundownManagedInstanceCache>();
	}
}

void FAvaMediaModule::StartRundownServerCommand(const TArray<FString>& Args)
{
	if (RundownServer)
	{
		UE_LOGF(LogAvaMedia, Log, "Rundown Server is already started.");
		return;
	}

	// Make sure all the components used by the rundown server are created.
	ConditionalCreateLocalPlaybackManager();
	ConditionalCreateManagedInstanceCache();
	
	RundownServer = MakeShared<FAvaRundownServer>();
	
	// Remark: Only the module's rundown server register console commands to avoid
	// conflicts with temporary servers (for testing).
	RundownServer->RegisterConsoleCommands();

	RundownServer->Init(Args.Num() > 0 ? Args[0] : TEXT(""));
	OnRundownServerStarted.Broadcast(RundownServer);

	UE_LOGF(LogAvaMedia, Log, "Rundown Server Started.");
}

void FAvaMediaModule::StopRundownServerCommand(const TArray<FString>& Args)
{
	if (RundownServer)
	{
		UE_LOGF(LogAvaMedia, Log, "Stopping Rundown Server...");
		OnRundownServerStopping.Broadcast(RundownServer);
	}
	RundownServer.Reset();
}

void FAvaMediaModule::StartPlaybackServerCommand(const TArray<FString>& InArgs)
{
	// Starting a playback server in the same process as playback client is forbidden.
	if (AvaPlaybackClient)
	{
		UE_LOGF(LogAvaMedia, Error, "A Playback Server can't be started in the same process as a Playback Client.");
		return;
	}
	
	if (!AvaPlaybackServer)
	{
		AvaPlaybackServer = MakeShared<FAvaPlaybackServer>();
		AvaPlaybackServer->Init(InArgs.Num() > 0 ? InArgs[0] : TEXT(""));
		OnAvaPlaybackServerStarted.Broadcast();
		UE_LOGF(LogAvaMedia, Log, "Playback Server Started");
	}
}

void FAvaMediaModule::StopPlaybackServerCommand(const TArray<FString>& InArgs)
{
	if (AvaPlaybackServer)
	{
		AvaPlaybackServer->StartShuttingDown();
		AvaPlaybackServer->StopBroadcast();
		AvaPlaybackServer->StopPlaybacks();
		OnAvaPlaybackServerStopped.Broadcast();
	}
	AvaPlaybackServer.Reset();
}

void FAvaMediaModule::StartPlaybackClientCommand(const TArray<FString>& InArgs)
{
	if (AvaPlaybackClient)
	{
		UE_LOGF(LogAvaMedia, Log, "Playback client already started.");
		return;
	}
	
	bool bForce = false;
	for (const FString& Arg : InArgs)
	{
		if (Arg.Compare(TEXT("-Force"), ESearchCase::IgnoreCase) == 0)
		{
			bForce = true;
		}
	}
	
	// Starting a playback server in the same process as playback client is forbidden.
	if (AvaPlaybackServer)
	{
		if (!bForce)
		{
			UE_LOGF(LogAvaMedia, Error, "A playback client can't be started in the same process as a playback server.");
			return;
		}

		UE_LOGF(LogAvaMedia, Warning, "Playback server has been stopped in editor mode in favor of playback client.");
		StopPlaybackServer();

		// Stopping the playback server requires a reload of the broadcast client config.
		UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
		Broadcast.LoadBroadcast();
		Broadcast.QueueNotifyChange(EAvaBroadcastChange::CurrentProfile); // Force a refresh of broadcast editor (if opened).
	}

	using namespace UE::AvaPlaybackClient::Delegates;
	if (GetOnConnectionEvent().IsBoundToObject(this))
	{
		GetOnConnectionEvent().AddRaw(this, &FAvaMediaModule::OnAvaPlaybackClientConnectionEvent);
	}

	AvaPlaybackClient = MakeShared<FAvaPlaybackClient>(this);
	AvaPlaybackClient->Init();
	OnAvaPlaybackClientStarted.Broadcast();
	UE_LOGF(LogAvaMedia, Log, "Playback client started");
}

void FAvaMediaModule::StopPlaybackClientCommand(const TArray<FString>& InArgs)
{
	if (AvaPlaybackClient)
	{
		OnAvaPlaybackClientStopped.Broadcast();
	}
	AvaPlaybackClient.Reset();
	UE::AvaPlaybackClient::Delegates::GetOnConnectionEvent().RemoveAll(this);
}

void FAvaMediaModule::StartHttpPlaybackServerCommand(const TArray<FString>& InArgs)
{
	if (!AvaPlaybackServer.IsValid())
	{
		UE_LOGF(LogAvaMedia, Error, "Playback Server must be started in order to start Http Playback Server.");
		return;
	}
	
	if (!AvaPlaybackHttpPlaybackServer)
	{
		AvaPlaybackHttpPlaybackServer = MakeShared<FAvaPlaybackHttpServer>();
		
		if (InArgs.Num() > 0)
		{
			// ...
		}
	}
	
	if (!AvaPlaybackHttpPlaybackServer->IsRunning())
	{
		const int32 Port = GetDefault<UAvaMediaSettings>()->HttpServerPort;
		AvaPlaybackHttpPlaybackServer->Start(AvaPlaybackServer, Port);
		UE_LOGF(LogAvaMedia, Log, "Http Playback Server Started");
	}
}

void FAvaMediaModule::StopHttpPlaybackServerCommand(const TArray<FString>& InArgs)
{
	AvaPlaybackHttpPlaybackServer.Reset();
}


void FAvaMediaModule::SaveDeviceProvidersCommand(const TArray<FString>& InArgs)
{
	FAvaBroadcastDeviceProviderDataList OutProviders;
	OutProviders.Populate(InArgs.Num() > 0 ? InArgs[0] : FPlatformProcess::ComputerName());
	OutProviders.SaveToJson();	// Saves in the project's config folder.
	OutProviders.SaveToXml();	// Saves in the project's config folder.
}

void FAvaMediaModule::LoadDeviceProvidersCommand(const TArray<FString>& InArgs)
{
	DeviceProviderProxyManager.TestInstall();
}

void FAvaMediaModule::UnloadDeviceProvidersCommand(const TArray<FString>& InArgs)
{
	DeviceProviderProxyManager.TestUninstall();
}

void FAvaMediaModule::ListDeviceProvidersCommand(const TArray<FString>& InArgs)
{
	DeviceProviderProxyManager.ListAllProviders();
}

void FAvaMediaModule::HandleStatCommand(const TArray<FString>& InArgs)
{
	if (InArgs.IsEmpty())
	{
		UE_LOGF(LogAvaMedia, Error, "Stat Command: No arguments specified.");
		return;
	}

	const bool bLocalCommandSucceeded = LocalPlaybackManager->HandleStatCommand(InArgs);
	
	if (AvaPlaybackClient.IsValid())
	{
		AvaPlaybackClient->BroadcastStatCommand(InArgs[0], bLocalCommandSucceeded);
	}
}

void FAvaMediaModule::OnAvaPlaybackClientConnectionEvent(IAvaPlaybackClient& InPlaybackClient,
		const UE::AvaPlaybackClient::Delegates::FConnectionEventArgs& InArgs)
{
	using namespace UE::AvaPlaybackClient::Delegates;
	// When a playback server connection event occurs, update the status of the playback server process.
	switch (InArgs.Event)
	{
	case EConnectionEvent::ServerConnected:
		if (!IsGameModeLocalPlaybackServerLaunched() && IsPlaybackClientStarted())
		{
			LocalPlaybackServerProcess = FAvaPlaybackServerProcess::Find(*AvaPlaybackClient);
		}
		break;
	case EConnectionEvent::ServerDisconnected:
		// Clean up the process handle.
		if (LocalPlaybackServerProcess.IsValid() && !LocalPlaybackServerProcess->IsLaunched())
		{
			LocalPlaybackServerProcess.Reset();
		}
		break;
	}
}

const FLinearColor& FAvaMediaModule::FLocalBroadcastSettings::GetChannelClearColor() const
{
	return UAvaMediaSettings::Get().ChannelClearColor;
}

EPixelFormat FAvaMediaModule::FLocalBroadcastSettings::GetDefaultPixelFormat() const
{
	return UAvaMediaSettings::Get().ChannelDefaultPixelFormat;
}

const FIntPoint& FAvaMediaModule::FLocalBroadcastSettings::GetDefaultResolution() const
{
	return UAvaMediaSettings::Get().ChannelDefaultResolution;
}

bool FAvaMediaModule::FLocalBroadcastSettings::IsDrawPlaceholderWidget() const
{
	return UAvaMediaSettings::Get().bDrawPlaceholderWidget;
}

const FSoftObjectPath& FAvaMediaModule::FLocalBroadcastSettings::GetPlaceholderWidgetClass() const
{
	return UAvaMediaSettings::Get().PlaceholderWidgetClass.ToSoftObjectPath();
}

const FLinearColor& FAvaMediaModule::FBroadcastSettingsBridge::GetChannelClearColor() const
{
	return GetSettings().GetChannelClearColor();
}

EPixelFormat FAvaMediaModule::FBroadcastSettingsBridge::GetDefaultPixelFormat() const
{
	return GetSettings().GetDefaultPixelFormat();
}

const FIntPoint& FAvaMediaModule::FBroadcastSettingsBridge::GetDefaultResolution() const
{
	return GetSettings().GetDefaultResolution();
}

bool FAvaMediaModule::FBroadcastSettingsBridge::IsDrawPlaceholderWidget() const
{
	return GetSettings().IsDrawPlaceholderWidget();
}

const FSoftObjectPath& FAvaMediaModule::FBroadcastSettingsBridge::GetPlaceholderWidgetClass() const
{
	return GetSettings().GetPlaceholderWidgetClass();
}

const IAvaBroadcastSettings& FAvaMediaModule::FBroadcastSettingsBridge::GetSettings() const
{
	// if server is enabled, fetch the setting from the currently connected client.
	if (ParentModule->AvaPlaybackServer.IsValid())
	{
		if (const IAvaBroadcastSettings* SettingsFromClient = ParentModule->AvaPlaybackServer->GetBroadcastSettings())
		{
			return *SettingsFromClient;
		}
	}
	return ParentModule->LocalBroadcastSettings;
}

IAvaBroadcastDeviceProviderProxyManager& FAvaMediaModule::GetDeviceProviderProxyManager()
{
	return DeviceProviderProxyManager;
}

IAvaPlayableRemoteControlPresetInfoCache& FAvaMediaModule::GetPlayableRemoteControlPresetInfoCache() const
{
	return *RemoteControlPresetInfoCache;
}

IMPLEMENT_MODULE(FAvaMediaModule, AvalancheMedia)