// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Broadcast/IAvaBroadcastSettings.h"
#include "Broadcast/OutputDevices/AvaBroadcastDeviceProviderProxy.h"
#include "Broadcast/OutputDevices/AvaBroadcastDisplayDeviceProvider.h"
#include "IAvaMediaModule.h"
#include "ModularFeature/AvaMediaSync.h"
#include "Playback/AvaPlaybackClient.h"
#include "Playback/AvaPlaybackClientDelegates.h"
#include "Playback/AvaPlaybackManager.h"
#include "Playback/AvaPlaybackServer.h"
#include "Playback/AvaPlaybackServerProcess.h"
#include "Playback/Http/AvaPlaybackHttpServer.h"
#include "Rundown/AvaRundownManagedInstanceCache.h"
#include "Rundown/AvaRundownServer.h"

class FAvaPlayableGroupSceneViewExtension;
class FAvaPlayableRemoteControlPresetInfoCache;

class FAvaMediaModule : public IAvaMediaModule
{
public:
	FAvaMediaModule();

	//~ Begin IAvaMediaModule
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool IsPlaybackClientStarted() const override { return AvaPlaybackClient.IsValid();}
	virtual void StartPlaybackClient() override;
	virtual void StopPlaybackClient() override;
	virtual bool IsPlaybackServerStarted() const override { return AvaPlaybackServer.IsValid();}
	virtual void StartPlaybackServer(const FString& InPlaybackServerName) override;
	virtual void StopPlaybackServer() override;
	
	virtual IAvaPlaybackClient& GetPlaybackClient() override;
	virtual IAvaPlaybackServer* GetPlaybackServer() const override { return AvaPlaybackServer.Get(); }
	virtual const IMediaIOCoreDeviceProvider* GetDeviceProvider(FName InProviderName, const FMediaIOOutputConfiguration* InMediaIOOutputConfiguration) const override;
	virtual TArray<const IMediaIOCoreDeviceProvider*> GetDeviceProvidersForServer(const FString& InServerName) const override;
	virtual FString GetServerNameForDevice(const FName& InDeviceProviderName, const FName& InDeviceName) const override;
	virtual bool IsLocalDevice(const FName& InDeviceProviderName, const FName& InDeviceName) const override;
	virtual void LaunchGameModeLocalPlaybackServer() override;
	virtual void StopGameModeLocalPlaybackServer() override;
	virtual bool IsGameModeLocalPlaybackServerLaunched() const override;
	virtual const IAvaBroadcastSettings& GetBroadcastSettings() const override;
	virtual const FAvaInstanceSettings& GetAvaInstanceSettings() const override;
	virtual const FAvaPlayableSettings& GetPlayableSettings() const override;
	virtual bool IsLocalPlaybackManagerAvailable() const override;
	virtual FAvaPlaybackManager& GetLocalPlaybackManager() const override;
	virtual bool IsManagedInstanceCacheAvailable() const override;
	virtual FAvaRundownManagedInstanceCache& GetManagedInstanceCache() const override;
	virtual bool IsAvaMediaSyncProviderFeatureAvailable() const override;
	virtual IAvaMediaSyncProvider* GetAvaMediaSyncProvider() const override;
	virtual void NotifyMapChangedEvent(UWorld* InWorld, EAvaMediaMapChangeType InEventType) override;
	virtual FOnAvaMediaSyncProviderChanged& GetOnAvaMediaSyncProviderChanged() override { return OnAvaMediaSyncProviderChanged; }
	virtual FOnAvaMediaSyncPackageModified& GetOnAvaMediaSyncPackageModified() override { return OnAvaMediaSyncPackageModified; }
	virtual FOnMapChangedEvent& GetOnMapChangedEvent() override { return OnMapChangedEvent; }
	virtual FOnAvaPlaybackClientStarted& GetOnAvaPlaybackClientStarted() override { return OnAvaPlaybackClientStarted; }
	virtual FOnAvaPlaybackClientStopped& GetAvaPlaybackClientStopped() override { return OnAvaPlaybackClientStopped; }
	virtual FOnAvaPlaybackServerStarted& GetOnAvaPlaybackServerStarted() override { return OnAvaPlaybackServerStarted; }
	virtual FOnAvaPlaybackServerStopped& GetAvaPlaybackServerStopped() override { return OnAvaPlaybackServerStopped; }
	virtual FGetEditorViewportClient& GetEditorViewportClientDelegate() override { return GetEditorViewportClient; }
	virtual FOnRundownServerEvent& GetOnRundownServerStarted() override { return OnRundownServerStarted; }
	virtual FOnRundownServerEvent& GetOnRundownServerStopping() override { return OnRundownServerStopping; }
	virtual bool IsRundownServerStarted() const override { return RundownServer.IsValid();}
	virtual void StartRundownServer(const FString& InRundownServerName) override;
	virtual void StopRundownServer() override;
	virtual TSharedPtr<IAvaRundownServer> GetRundownServer() const override { return RundownServer; }
	virtual TSharedPtr<IAvaRundownServer> MakeDetachedRundownServer(const FString& InServerName) override;
	virtual IAvaBroadcastDeviceProviderProxyManager& GetDeviceProviderProxyManager() override;
	virtual IAvaPlayableRemoteControlPresetInfoCache& GetPlayableRemoteControlPresetInfoCache() const override;
	//~ End IAvaMediaModule
	
private:
	void PostEngineInit();
	void EnginePreExit();
	void PrePIEEnded(const bool);
	void StopAllServices();
	void ConditionalCreateLocalPlaybackManager();
	void ConditionalCreateManagedInstanceCache();
	
	// Command handlers
	void StartRundownServerCommand(const TArray<FString>& Args);
	void StopRundownServerCommand(const TArray<FString>& Args);
	void StartPlaybackServerCommand(const TArray<FString>& InArgs);
	void StopPlaybackServerCommand(const TArray<FString>& InArgs);
	void StartPlaybackClientCommand(const TArray<FString>& InArgs);
	void StopPlaybackClientCommand(const TArray<FString>& InArgs);
	void LaunchLocalPlaybackServerCommand(const TArray<FString>& InArgs) { LaunchGameModeLocalPlaybackServer(); }
	void StopLocalPlaybackServerCommand(const TArray<FString>& InArgs) { StopGameModeLocalPlaybackServer(); }

	void StartHttpPlaybackServerCommand(const TArray<FString>& InArgs);
	void StopHttpPlaybackServerCommand(const TArray<FString>& InArgs);
	
	void SaveDeviceProvidersCommand(const TArray<FString>& InArgs);
	void LoadDeviceProvidersCommand(const TArray<FString>& InArgs);
	void UnloadDeviceProvidersCommand(const TArray<FString>& InArgs);
	void ListDeviceProvidersCommand(const TArray<FString>& InArgs);
	void HandleStatCommand(const TArray<FString>& InArgs);

	// Event Handlers
	void OnAvaPlaybackClientConnectionEvent(IAvaPlaybackClient& InPlaybackClient,
		const UE::AvaPlaybackClient::Delegates::FConnectionEventArgs& InArgs);

private:
	FAvaBroadcastDisplayDeviceProvider AvaDisplayDeviceProvider;
	FAvaBroadcastDeviceProviderProxyManager DeviceProviderProxyManager;
	TUniquePtr<FAvaMediaSync> AvaMediaSync;

	TArray<IConsoleObject*> ConsoleCmds;

	TSharedPtr<FAvaPlayableGroupSceneViewExtension> PlayableGroupSceneViewExtension;
	
	/**
	 *	Wraps the local default UAvaMediaSettings.
	 */
	class FLocalBroadcastSettings : public IAvaBroadcastSettings
	{
	public:
		virtual ~FLocalBroadcastSettings() override = default;

		//~ Begin IAvaBroadcastSettings
		virtual const FLinearColor& GetChannelClearColor() const override;
		virtual EPixelFormat GetDefaultPixelFormat() const override;
		virtual const FIntPoint& GetDefaultResolution() const override;
		virtual bool IsDrawPlaceholderWidget() const override;
		virtual const FSoftObjectPath& GetPlaceholderWidgetClass() const override;
		//~ End IAvaBroadcastSettings
	};
	FLocalBroadcastSettings LocalBroadcastSettings;

	/**
	 * The purpose of the settings bridge is to provide a valid object
	 * even if a client disconnects, it will either switch to another client
	 * or use the local settings.
	 */
	class FBroadcastSettingsBridge final : public IAvaBroadcastSettings
	{
	public:
		explicit FBroadcastSettingsBridge(FAvaMediaModule* Module) : ParentModule(Module) {}
		virtual ~FBroadcastSettingsBridge() override = default;

		//~ Begin IAvaBroadcastSettings
		virtual const FLinearColor& GetChannelClearColor() const override;
		virtual EPixelFormat GetDefaultPixelFormat() const override;
		virtual const FIntPoint& GetDefaultResolution() const override;
		virtual bool IsDrawPlaceholderWidget() const override;
		virtual const FSoftObjectPath& GetPlaceholderWidgetClass() const override;
		//~ End IAvaBroadcastSettings

	private:
		const IAvaBroadcastSettings& GetSettings() const;
		
		FAvaMediaModule* ParentModule = nullptr;
	};
	FBroadcastSettingsBridge BroadcastSettingsBridge;

	TSharedPtr<FAvaRundownServer> RundownServer;
	TSharedPtr<FAvaPlaybackServer> AvaPlaybackServer;	
	TSharedPtr<FAvaPlaybackClient> AvaPlaybackClient;
	TSharedPtr<FAvaPlaybackServerProcess> LocalPlaybackServerProcess;
	TSharedPtr<FAvaPlaybackManager> LocalPlaybackManager;
	TSharedPtr<FAvaRundownManagedInstanceCache> ManagedInstanceCache;
	TSharedPtr<FAvaPlayableRemoteControlPresetInfoCache> RemoteControlPresetInfoCache;

	TSharedPtr<FAvaPlaybackHttpServer> AvaPlaybackHttpPlaybackServer;

	FOnAvaMediaSyncProviderChanged OnAvaMediaSyncProviderChanged;
	FOnAvaMediaSyncPackageModified OnAvaMediaSyncPackageModified;
	FOnMapChangedEvent OnMapChangedEvent;
	FOnAvaPlaybackClientStarted OnAvaPlaybackClientStarted;
	FOnAvaPlaybackClientStopped OnAvaPlaybackClientStopped;
	FOnAvaPlaybackServerStarted OnAvaPlaybackServerStarted;
	FOnAvaPlaybackServerStopped OnAvaPlaybackServerStopped;
	FOnRundownServerEvent OnRundownServerStarted;
	FOnRundownServerEvent OnRundownServerStopping;
	FGetEditorViewportClient GetEditorViewportClient;
};
