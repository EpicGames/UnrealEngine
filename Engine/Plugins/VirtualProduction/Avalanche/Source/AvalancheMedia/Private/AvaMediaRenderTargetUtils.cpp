// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMediaRenderTargetUtils.h"
#include "Engine/Engine.h"
#include "Engine/TextureRenderTarget2D.h"
#include "PixelFormat.h"
#include "UObject/Package.h"
#include "UObject/NameTypes.h"
#include "UObject/NoExportTypes.h"

namespace UE::AvaMediaRenderTargetUtils
{
	UTextureRenderTarget2D* CreateDefaultRenderTarget(FName InBaseName)
	{
		const FName TargetName = MakeUniqueObjectName(GetTransientPackage(), UTextureRenderTarget2D::StaticClass(), InBaseName);
		UTextureRenderTarget2D* const RenderTarget = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), TargetName);
		// Linear gamma would normally be true for everything but RGB8_SRGB format.
		RenderTarget->bForceLinearGamma = false;
		RenderTarget->bAutoGenerateMips = false;
		return RenderTarget;
	}

	bool SetupRenderTargetGamma(UTextureRenderTarget2D* InRenderTarget)
	{
		// Note: In CreateDefaultRenderTarget, we set bForceLinearGamma to false, meaning,
		// the output is supposed to be gammatized (i.e. non linear).
		// However, bForceLinearGamma is ignored for floating point textures
		// so we have to override by specifying a target gamma instead.
		if (IsFloatFormat(InRenderTarget))
		{
			// This will force the float texture to be gammatized like the integer textures,
			// thus making it consistent. For Ava rendering, the outputs are expecting a gammatized texture,
			// regardless of it's format.
			const float TargetGamma = GEngine ? GEngine->GetDisplayGamma() : 2.2f;
			if (InRenderTarget->TargetGamma != TargetGamma)
			{
				InRenderTarget->TargetGamma = TargetGamma;
				return true;
			}
		}
		return false;
	}
	
	void UpdateRenderTarget(UTextureRenderTarget2D* InRenderTarget, const FIntPoint& InSize, EPixelFormat InFormat, const FLinearColor& InClearColor)
	{
		check(InSize.X > 0 && InSize.Y > 0);
		check(InFormat != PF_Unknown);

		if (!ensure(InRenderTarget))
		{
			return;
		}

		bool bNeedInitFormat = InRenderTarget->OverrideFormat != InFormat
			|| InRenderTarget->ClearColor != InClearColor;
		
		bNeedInitFormat |= SetupRenderTargetGamma(InRenderTarget);

		if (bNeedInitFormat)
		{
			InRenderTarget->ClearColor = InClearColor;
			InRenderTarget->InitCustomFormat(InSize.X, InSize.Y, InFormat, InRenderTarget->bForceLinearGamma);
			InRenderTarget->UpdateResourceImmediate();	// Doesn't seem to be necessary, but doesn't hurt either.
		}
		else
		{
			InRenderTarget->ResizeTarget(InSize.X, InSize.Y);
		}
	}

	FIntPoint GetRenderTargetSizeSafe(const UTextureRenderTarget2D* InRenderTarget)
	{
		return InRenderTarget ? FIntPoint(InRenderTarget->SizeX, InRenderTarget->SizeY) : FIntPoint(0,0);
	}

	FIntPoint GetRenderTargetSize(const UTextureRenderTarget2D* InRenderTarget)
	{
		if (ensure(IsValid(InRenderTarget)))
		{
			return FIntPoint(InRenderTarget->SizeX, InRenderTarget->SizeY);
		}
		return FIntPoint(0,0);
	}

	inline bool IsFloatFormat(ETextureRenderTargetFormat InFormat)
	{
		return InFormat == RTF_R16f || InFormat == RTF_RG16f || InFormat == RTF_RGBA16f
			|| InFormat == RTF_R32f || InFormat == RTF_RG32f || InFormat == RTF_RGBA32f;
	}
	
	bool IsFloatFormat(const UTextureRenderTarget2D* InRenderTarget)
	{
		if (ensure(IsValid(InRenderTarget)))
		{
			if (InRenderTarget->OverrideFormat != PF_Unknown)
			{
				// Note: Assumes HDR is float. However, upon inspection, it is missing PF_G16R16F, PF_G32R32F and PF_FloatR11G11B10.
				const EPixelFormat Format = InRenderTarget->GetFormat();
				return IsHDR(Format) || Format == PF_G16R16F || Format == PF_G32R32F || Format == PF_FloatR11G11B10;
			}
			return IsFloatFormat(InRenderTarget->RenderTargetFormat);
		}
		return false;
	}

	inline bool IsSingleChannelAlpha(EPixelFormat InFormat)
	{
		return (InFormat == PF_A1 || InFormat == PF_A8);
	}

	int32 GetNumColorComponents(EPixelFormat InFormat)
	{
		const FPixelFormatInfo& FormatInfo = GPixelFormats[InFormat];
		// Exceptions: alpha only format have no color components.
		// (Haven't seen luminance-alpha formats)
		if (IsSingleChannelAlpha(InFormat) || FormatInfo.Supported == false)
		{
			return 0;
		}
		
		// Maximum of 3 color components.
		return (FormatInfo.NumComponents > 3) ? 3 : FormatInfo.NumComponents;
	}

	// Only works for channel that have the same number of bits per channel for
	// all channels.
	// Some exceptions: PF_R5G6B5_UNORM, PF_FloatR11G11B10.
	// But those are not render target formats.
	inline int32 GetNumBitsPerChannel(EPixelFormat InFormat)
	{
		const FPixelFormatInfo& FormatInfo = GPixelFormats[InFormat];
		if (FormatInfo.Supported == false || FormatInfo.NumComponents == 0)
		{
			return 0;
		}
		
		const int32 PixelsPerBlock = FormatInfo.BlockSizeX * FormatInfo.BlockSizeY * FormatInfo.BlockSizeZ;
		if (PixelsPerBlock == 0)
		{
			return 0;
		}
		return 8 * FormatInfo.BlockBytes / (PixelsPerBlock * FormatInfo.NumComponents);
	}
	
	// Note: we only have to deal with possible RGBA render target formats.
	int32 GetNumColorChannelBits(EPixelFormat InFormat)
	{
		// Exceptions:
		// - alpha only format have no color components:
		if (IsSingleChannelAlpha(InFormat))
		{
			return 0;
		}
		// - This format doesn't have the same number of bits for alpha and color.
		if (InFormat == PF_A2B10G10R10)
		{
			return 10;
		}
		
		return GetNumBitsPerChannel(InFormat);
	}

	bool HasAlpha(EPixelFormat InFormat)
	{
		// Exception: single channel alpha.
		if (IsSingleChannelAlpha(InFormat))
		{
			return true;
		}

		// Rule: 4 components formats have alpha.
		const FPixelFormatInfo& FormatInfo = GPixelFormats[InFormat];
		return FormatInfo.Supported == true && FormatInfo.NumComponents == 4;
	}
	
	// Note: we only have to deal with possible render target formats.
	int32 GetNumAlphaChannelBits(EPixelFormat InFormat)
	{
		// Exceptions:
		if (InFormat == PF_A2B10G10R10)
		{
			return 2;
		}
		// Rule:
		return HasAlpha(InFormat) ? GetNumBitsPerChannel(InFormat) : 0;
	}	
}
