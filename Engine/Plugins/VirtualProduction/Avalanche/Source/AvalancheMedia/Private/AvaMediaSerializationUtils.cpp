// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMediaSerializationUtils.h"

#include "Serialization/StructuredArchive.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"

namespace UE::AvaMediaSerializationUtils::Private
{
	void SerializeObject(FStructuredArchiveRecord& InRootRecord, UObject* InObject, const TFunction<bool(FProperty*)>& InShouldSkipProperty)
	{
		if (!InObject)
		{
			return;
		}
	
		FStructuredArchiveRecord ObjectRecord = InRootRecord.EnterRecord(*InObject->GetName());
		FArchive& UnderlyingArchive = InRootRecord.GetUnderlyingArchive();
	
		for (FProperty* Property = InObject->GetClass()->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			if (!Property || Property->HasAnyPropertyFlags(CPF_Transient) || (InShouldSkipProperty && InShouldSkipProperty(Property)))
			{
				continue;
			}

			FStructuredArchiveSlot PropertyField = ObjectRecord.EnterField(*Property->GetName());
		
			for (int32 Index = 0; Index < Property->ArrayDim; ++Index)
			{
				UObject* const Archetype = InObject->GetArchetype();
			
				void* const Data = Property->ContainerPtrToValuePtr<void>(InObject, Index);
				const void* const Defaults
					= Property->ContainerPtrToValuePtrForDefaults<void>(Archetype->GetClass(), Archetype, Index);
			
				//if (!Property->Identical(Target, Default, UnderlyingArchive.GetPortFlags()))
				{
					FSerializedPropertyScope SerializedProperty(UnderlyingArchive, Property);
					Property->SerializeItem(PropertyField, Data, Defaults);
				}
			}
		}
	}
}

void UE::AvaMediaSerializationUtils::SerializeObject(FStructuredArchiveFormatter& InFormatter, UObject* InObject, const TFunction<bool(FProperty*)>& InShouldSkipProperty)
{
#if WITH_TEXT_ARCHIVE_SUPPORT
	FStructuredArchive StructuredArchive(InFormatter);
	FStructuredArchiveRecord RootRecord = StructuredArchive.Open().EnterRecord();
	Private::SerializeObject(RootRecord, InObject, InShouldSkipProperty);
	StructuredArchive.Close();
#endif
}

void UE::AvaMediaSerializationUtils::JsonValueConversion::BytesToString(const TArray<uint8>& InValueAsBytes, FString& OutValueAsString)
{
	// Reinterpret as TCHAR array. FJsonStructSerializerBackend uses UCS2CHAR which is compatible with TCHAR.
	const TCHAR* ValueAsChars = reinterpret_cast<const TCHAR*>(InValueAsBytes.GetData());
	int32 SizeInChar = InValueAsBytes.Num() / sizeof(TCHAR);

	// Exclude terminating character, if any.
	// Remark: From observation, the arrays from FJsonStructSerializerBackend don't seem to have a terminating character, but we check anyway.
	if (SizeInChar > 0 && ValueAsChars[SizeInChar-1] == 0)
	{
		--SizeInChar; // Exclude terminating character.
	}

	OutValueAsString.Reset(SizeInChar);
	OutValueAsString.AppendChars(ValueAsChars, SizeInChar);
}