// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMediaSettings.h"
#include "IAvaMediaModule.h"
#include "Misc/MessageDialog.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "AvaMediaSettings"

const FName UAvaMediaSettings::SynchronizedEventsFeatureSelection_Default(TEXT("Default"));

UAvaMediaSettings::UAvaMediaSettings()
{
	CategoryName = TEXT("Motion Design");
	SectionName  = TEXT("Playback & Broadcast");
	PlayableSettings.SynchronizedEventsFeature.Implementation = SynchronizedEventsFeatureSelection_Default.ToString();
	// The choice of going with a trailing "__" as the default ignored postfix is inspired by a Python naming convention to
	// indicate ignored/hidden functions not meant to be called directly by users.
	PlayableSettings.IgnoredControllerPostfix.Add(TEXT("__"));
}

UAvaMediaSettings* UAvaMediaSettings::GetSingletonInstance()
{
	UAvaMediaSettings* DefaultSettings = GetMutableDefault<UAvaMediaSettings>();
	static bool bInitialized = false;
	if (!bInitialized)
	{
		bInitialized = true;
		DefaultSettings->SetFlags(RF_Transactional);
	}
	return DefaultSettings;
}

ELogVerbosity::Type UAvaMediaSettings::ToLogVerbosity(EAvaMediaLogVerbosity InAvaMediaLogVerbosity)
{
	switch (InAvaMediaLogVerbosity)
	{
	case EAvaMediaLogVerbosity::NoLogging:
		return ELogVerbosity::NoLogging;
	case EAvaMediaLogVerbosity::Fatal:
		return ELogVerbosity::Fatal;
	case EAvaMediaLogVerbosity::Error:
		return ELogVerbosity::Error;
	case EAvaMediaLogVerbosity::Warning:
		return ELogVerbosity::Warning;
	case EAvaMediaLogVerbosity::Display:
		return ELogVerbosity::Display;
	case EAvaMediaLogVerbosity::Log:
		return ELogVerbosity::Log;
	case EAvaMediaLogVerbosity::Verbose:
		return ELogVerbosity::Verbose;
	case EAvaMediaLogVerbosity::VeryVerbose:
		return ELogVerbosity::VeryVerbose;
	default:
		return ELogVerbosity::NoLogging;
	}
}

FString UAvaMediaSettings::GenerateLocalPlaybackServerCommandLine() const
{
	const FAvaMediaLocalPlaybackServerSettings& Settings = LocalPlaybackServerSettings;

	FString CommandLine;
	CommandLine += FString::Printf(TEXT("-game -windowed -ResX=%d -ResY=%d -messaging"), Settings.Resolution.X, Settings.Resolution.Y);
	if (Settings.bEnableLogConsole)
	{
		CommandLine += TEXT(" -log");
	}
	if (Settings.bDisablePython)
	{
		CommandLine += TEXT(" -DisablePython");
	}

	// Motion Design arguments
	CommandLine += FString::Printf(TEXT(" -MotionDesignPlaybackServerStart=\"%s\""), *Settings.ServerName);
	CommandLine += TEXT(" -MotionDesignPlaybackClientSuppress");
	CommandLine += FString::Printf(TEXT(" -MotionDesignPlaybackServerLogReplication=%s"), ToString(UAvaMediaSettings::ToLogVerbosity(PlaybackServerLogReplicationVerbosity)));

	// Storm Sync arguments
	CommandLine += TEXT(" -NoStormSyncServerAutoStart");

	if (!Settings.ExtraCommandLineArguments.IsEmpty())
	{
		CommandLine += TEXT(" ");
		CommandLine += Settings.ExtraCommandLineArguments;
	}

	FString LogCommands;
	for (const FAvaPlaybackServerLoggingEntry& LoggingEntry : Settings.Logging)
	{
		if (!LoggingEntry.Category.IsNone())
		{
			LogCommands += FString::Printf(TEXT("%s%s %s"),
				LogCommands.IsEmpty() ? TEXT("") : TEXT(", "), *LoggingEntry.Category.ToString(),
				ToString(UAvaMediaSettings::ToLogVerbosity(LoggingEntry.VerbosityLevel)));
		}
	}

	if (!LogCommands.IsEmpty())
	{
		CommandLine += FString::Printf(TEXT(" -LogCmds=\"%s\""), *LogCommands);
	}
	return CommandLine;
}

void UAvaMediaSettings::PostInitProperties()
{
	Super::PostInitProperties();
#if WITH_EDITOR
	UpdateLocalPlaybackServerCommandLinePreview();
#endif
}

void UAvaMediaSettings::PostReloadConfig(FProperty* InPropertyThatWasLoaded)
{
	Super::PostReloadConfig(InPropertyThatWasLoaded);
#if WITH_EDITOR
	UpdateLocalPlaybackServerCommandLinePreview();
#endif
}

#if WITH_EDITOR
void UAvaMediaSettings::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	// Check if playback client is auto-start.
	if (InPropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAvaMediaSettings, bAutoStartPlaybackClient))
	{
		IAvaMediaModule& AvaMediaModule = IAvaMediaModule::Get();
		if (bAutoStartPlaybackClient && !AvaMediaModule.IsPlaybackClientStarted())
		{
			const FText MessageText = LOCTEXT("StartPlaybackClientQuestion", "Do you want to start the playback client now?");
			const EAppReturnType::Type Reply = FMessageDialog::Open(EAppMsgType::YesNo, EAppReturnType::Yes, MessageText);

			if (Reply == EAppReturnType::Yes)
			{
				AvaMediaModule.StartPlaybackClient();
			}
		}
		else if (!bAutoStartPlaybackClient && AvaMediaModule.IsPlaybackClientStarted())
		{
			const FText MessageText = LOCTEXT("StopPlaybackClientQuestion", "Playback Client is currently running. Do you want to stop it now?");
			const EAppReturnType::Type Reply = FMessageDialog::Open(EAppMsgType::YesNo, EAppReturnType::Yes, MessageText);

			if (Reply == EAppReturnType::Yes)
			{
				AvaMediaModule.StopPlaybackClient();
			}
		}
	}
	// Update user-facing command line string if server settings has changed 
	else if (InPropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UAvaMediaSettings, LocalPlaybackServerSettings))
	{
		UpdateLocalPlaybackServerCommandLinePreview();
	}
}

void UAvaMediaSettings::UpdateLocalPlaybackServerCommandLinePreview()
{
	LocalPlaybackServerCommandLinePreview = GenerateLocalPlaybackServerCommandLine();
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
