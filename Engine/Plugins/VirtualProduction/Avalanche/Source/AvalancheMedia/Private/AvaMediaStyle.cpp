// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMediaStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

FAvaMediaStyle::FAvaMediaStyle()
	: FSlateStyleSet(TEXT("AvaMedia"))
{
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon200x200(200.f, 200.f);

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	SetContentRoot(Plugin->GetBaseDir() / TEXT("Resources"));
	
	Set("AvaMedia.UnrealIcon", new IMAGE_BRUSH("Icons/MediaIcons/ue_logo", Icon200x200));
	Set("ClassIcon.AvaBroadcast", new IMAGE_BRUSH("Icons/MediaIcons/MediaOutput", Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FAvaMediaStyle::~FAvaMediaStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
