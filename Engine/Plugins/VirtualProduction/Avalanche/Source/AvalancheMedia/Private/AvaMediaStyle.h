// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FAvaMediaStyle final : public FSlateStyleSet
{
public:
	static FAvaMediaStyle& Get()
	{
		static FAvaMediaStyle Instance;
		return Instance;
	}

	FAvaMediaStyle();
	virtual ~FAvaMediaStyle() override;
};
