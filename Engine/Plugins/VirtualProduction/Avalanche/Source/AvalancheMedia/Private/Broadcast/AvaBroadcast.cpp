// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/AvaBroadcast.h"
#include "Async/Async.h"
#include "AvaBroadcastSerialization.h"
#include "Containers/UnrealString.h"
#include "Delegates/IDelegateInstance.h"
#include "HAL/FileManager.h"
#include "IAvaMediaModule.h"
#include "Misc/Paths.h"
#include "Rundown/AvaRundown.h"
#include "UObject/NameTypes.h"
#include "UObject/Package.h"

DEFINE_LOG_CATEGORY(LogAvaBroadcast);

#define LOCTEXT_NAMESPACE "AvaBroadcast"

namespace UE::AvaBroadcast::Private
{
	static FString EnsureExtension(const FString& InFilename, const FString& InExtension)
	{
		if (!FPaths::GetExtension(InFilename).Equals(InExtension, ESearchCase::IgnoreCase))
		{
			// Appending extension instead of using SetExtension to respect
			// dot naming convention for xml/yaml config files.
			return InFilename + TEXT(".") + InExtension;
		}
		return InFilename;
	}

	static FString GetConfigFilepath(const FString& InExtension)
	{
		FString BroadcastConfigName;

		// Allowing command line specification of the broadcast configuration file name. This allows
		// starting the same project (from a shared location) with different configurations.
		if (!FParse::Value(FCommandLine::Get(),TEXT("MotionDesignBroadcastConfig="), BroadcastConfigName))
		{
			// When launching the server from the same project location, we want to avoid loading the same
			// broadcast configuration as the client. The server needs a clean configuration.
			const bool bIsServerRunning = (IAvaMediaModule::IsModuleLoaded() && IAvaMediaModule::Get().IsPlaybackServerStarted());	
			BroadcastConfigName = bIsServerRunning ? TEXT("MotionDesignServerBroadcastConfig") : TEXT("MotionDesignBroadcastConfig");
		}
		return EnsureExtension(FPaths::ProjectConfigDir() / BroadcastConfigName, InExtension);
	}

	static FString GetXmlSaveFilepath()
	{
		return GetConfigFilepath(TEXT("xml"));
	}

	static FString GetJsonSaveFilepath()
	{
		return GetConfigFilepath(TEXT("json"));
	}

}

UAvaBroadcast& UAvaBroadcast::Get()
{
	static UAvaBroadcast* Broadcast = nullptr;
	
	if (!IsValid(Broadcast))
	{
		static const TCHAR* PackageName = TEXT("/Temp/AvaMedia/AvaBroadcast");
		
		UPackage* const BroadcastPackage = CreatePackage(PackageName);
		BroadcastPackage->SetFlags(RF_Transient);
		BroadcastPackage->AddToRoot();

		//Don't Mark as Transient for "marking package dirty"
		Broadcast = NewObject<UAvaBroadcast>(BroadcastPackage
			, TEXT("AvaBroadcast")
			, RF_Transactional | RF_Standalone);

		Broadcast->AddToRoot();
		Broadcast->LoadBroadcast();
	}
	
	check(Broadcast);
	return *Broadcast;
}

void UAvaBroadcast::BeginDestroy()
{
	for (TPair<FName, FAvaBroadcastProfile>& Pair : Profiles)
	{
		Pair.Value.BeginDestroy();
	}
	Super::BeginDestroy();
}

UAvaBroadcast* UAvaBroadcast::GetBroadcast()
{
	return &UAvaBroadcast::Get();
}

void UAvaBroadcast::StartBroadcast()
{
	if (IsBroadcastingAllChannels())
	{
		return;
	}

	FAvaBroadcastProfile& Profile = GetCurrentProfile();
	Profile.StartChannelBroadcast();
}

void UAvaBroadcast::StopBroadcast()
{
	GetCurrentProfile().StopChannelBroadcast();
}

bool UAvaBroadcast::IsBroadcastingAnyChannel() const
{
	return GetCurrentProfile().IsBroadcastingAnyChannel();
}

bool UAvaBroadcast::IsBroadcastingAllChannels() const
{
	return GetCurrentProfile().IsBroadcastingAllChannels();
}

bool UAvaBroadcast::ConditionalStartBroadcastChannel(const FName& InChannelName)
{
	FAvaBroadcastOutputChannel& Channel = GetCurrentProfile().GetChannelMutable(InChannelName);
	if (!Channel.IsValidChannel())
	{
		UE_LOGF(LogAvaBroadcast, Error,
			"Start Broadcast failed: Channel \"%ls\" of Profile \"%ls\" is not valid.",
			*InChannelName.ToString(), *GetCurrentProfileName().ToString());
		return false;
	}
	
	if (Channel.GetState() == EAvaBroadcastChannelState::Idle)
	{
		Channel.StartChannelBroadcast();
	}
	return true;
}


TArray<FName> UAvaBroadcast::GetProfileNames() const
{
	TArray<FName> ProfileNames;
	Profiles.GetKeys(ProfileNames);
	return ProfileNames;
}

const TMap<FName, FAvaBroadcastProfile>& UAvaBroadcast::GetProfiles() const
{
	return Profiles;
}

FName UAvaBroadcast::CreateProfile(FName InProfileName, bool bMakeCurrentProfile)
{
	if (InProfileName == NAME_None)
	{
		static const FText DefaultProfileName = LOCTEXT("DefaultProfileName", "Profile");
		InProfileName = FName(DefaultProfileName.ToString(), 0);
	}
	
	FAvaBroadcastProfile& Profile = CreateProfileInternal(InProfileName);
	
	if (bMakeCurrentProfile)
	{
		SetCurrentProfile(Profile.GetName());
	}
	
	return Profile.GetName();
}

bool UAvaBroadcast::DuplicateProfile(FName InNewProfile, FName InTemplateProfile, bool bMakeCurrentProfile)
{
	const FAvaBroadcastProfile* const TemplateProfile = Profiles.Find(InTemplateProfile);

	if (TemplateProfile)
	{		
		if (InNewProfile == NAME_None)
		{
			InNewProfile = InTemplateProfile;
		}

		FAvaBroadcastProfile& Profile = CreateProfileInternal(InNewProfile);
		FAvaBroadcastProfile::CopyProfiles(*TemplateProfile, Profile);
		
		if (bMakeCurrentProfile)
		{
			SetCurrentProfile(Profile.GetName());
		}
		
		return true;
	}
	
	return false;
}

bool UAvaBroadcast::DuplicateCurrentProfile(FName InProfileName, bool bMakeCurrentProfile)
{
	return DuplicateProfile(InProfileName, CurrentProfile, bMakeCurrentProfile);
}

bool UAvaBroadcast::RemoveProfile(FName InProfileName)
{
	const bool bRemovingCurrentProfile = CurrentProfile == InProfileName;
	const bool bIsLastRemainingProfile = Profiles.Num() == 1;
	const bool bIsBroadcasting = IsBroadcastingAnyChannel();
	
	//The only condition that would prevent us from doing Removal is if we're currently Broadcasting and we want to remove Current Profile.
	const bool bCanRemoveProfile = !bIsLastRemainingProfile && !(bIsBroadcasting && bRemovingCurrentProfile);
	
	if (bCanRemoveProfile)
	{
		const int32 RemoveCount = Profiles.Remove(InProfileName);

		//If Removing Current Profile, we need to find a new Current Profile
		if (bRemovingCurrentProfile)
		{
			CurrentProfile = NAME_None;
			EnsureValidCurrentProfile();
		}
		
		return RemoveCount > 0;
	}
	return false;
}

bool UAvaBroadcast::CanRenameProfile(FName InProfileName, FName InNewProfileName, FText* OutErrorMessage) const
{
	if (InNewProfileName.IsNone())
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = LOCTEXT("RenameError_ProfileNone", "Invalid profile name.");
		}
		return false;
	}

	if (Profiles.Contains(InNewProfileName))
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = LOCTEXT("RenameError_ProfileExists", "Profile name already exists.");
		}
		return false;
	}
	
	const bool bRenamingCurrentProfile = (CurrentProfile == InProfileName);
	const bool bIsBroadcasting = IsBroadcastingAnyChannel();

	if (bIsBroadcasting && bRenamingCurrentProfile)
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = LOCTEXT("RenameError_ProfileInUse", "Profile is currently Broadcasting Channels.");
		}
		return false;
	}
	
	return true;
}

bool UAvaBroadcast::RenameProfile(FName InProfileName, FName InNewProfileName)
{
	if (CanRenameProfile(InProfileName, InNewProfileName))
	{
		FAvaBroadcastProfile Profile = MoveTemp(Profiles[InProfileName]);
		Profiles.Remove(InProfileName);
		Profile.SetProfileName(InNewProfileName);
		Profiles.Add(InNewProfileName, MoveTemp(Profile));
		
		if (CurrentProfile == InProfileName)
		{
			CurrentProfile = InNewProfileName;
		}

		// Rename pinned channel's profile.
		for (TPair<FName, FName>& PinnedChannel : PinnedChannels)
		{
			if (PinnedChannel.Value == InProfileName)
			{
				PinnedChannel.Value = InNewProfileName;
			}
		}
		
		return true;
	}
	return false;
}

bool UAvaBroadcast::SetCurrentProfile(FName InProfileName)
{
	const bool bIsBroadcasting = IsBroadcastingAnyChannel();
	
	//Can only set a new Current Profile if not Broadcasting.
	if (!bIsBroadcasting && CurrentProfile != InProfileName && Profiles.Contains(InProfileName))
	{
		if (GetCurrentProfile().IsValidProfile())
		{
			// Deallocate previous profile's resources.
			GetCurrentProfile().UpdateChannels(false);
		}
		
		CurrentProfile = InProfileName;
		
		// Allocate new profile's resources.
		GetCurrentProfile().UpdateChannels(true);
		
		QueueNotifyChange(EAvaBroadcastChange::CurrentProfile);
		return true;
	}
	return false;
}

FAvaBroadcastProfile& UAvaBroadcast::GetProfile(FName InProfileName)
{
	if (Profiles.Contains(InProfileName))
	{
		return Profiles[InProfileName];
	}
	return FAvaBroadcastProfile::GetNullProfile();
}

const FAvaBroadcastProfile& UAvaBroadcast::GetProfile(FName InProfileName) const
{
	if (Profiles.Contains(InProfileName))
	{
		return Profiles[InProfileName];
	}
	return FAvaBroadcastProfile::GetNullProfile();
}

void UAvaBroadcast::LoadBroadcast()
{
	using namespace UE::AvaBroadcast::Private;

	LoadedConfigFilepath = GetJsonSaveFilepath();
	bool bConfigLoaded = FAvaBroadcastSerialization::LoadBroadcastFromJson(LoadedConfigFilepath, this);

#if WITH_EDITOR
	if (!bConfigLoaded)
	{
		// Fallback to legacy xml format.
		LoadedConfigFilepath = GetXmlSaveFilepath();
		bConfigLoaded = FAvaBroadcastSerialization::LoadBroadcastFromXml(LoadedConfigFilepath, this);
	}
#endif

	if (!bConfigLoaded)
	{
		LoadedConfigFilepath.Reset();
	}

	// Set the profile names early because it is needed to resolve the pinned channels below.
	UpdateProfileNames();
	
	if (Profiles.Num() > 0)
	{
		for (TPair<FName, FAvaBroadcastProfile>& Profile : Profiles)
		{
			const bool bIsProfileActive = (Profile.Key == CurrentProfile);
			Profile.Value.PostLoadProfile(bIsProfileActive, this);
		}
	}
	else
	{
		CreateProfile(NAME_None, true);
		SaveBroadcast();
	}
	
	EnsureValidCurrentProfile();
}

void UAvaBroadcast::SaveBroadcast()
{
	using namespace UE::AvaBroadcast::Private;
	
	bool bIsBroadcastSaved = FAvaBroadcastSerialization::SaveBroadcastToJson(this, GetJsonSaveFilepath());

#if WITH_EDITOR
	// In case of failure, fallback to xml format.
	// Temporary until the json format is battle tested.
	if (!bIsBroadcastSaved)
	{
		bIsBroadcastSaved = FAvaBroadcastSerialization::SaveBroadcastToXml(this, GetXmlSaveFilepath());
	}
#endif
	
	if (bIsBroadcastSaved)
	{
		GetPackage()->SetDirtyFlag(false);
	}
	else
	{
		UE_LOGF(LogAvaBroadcast, Error, "Failed to save broadcast configuration."); 
	}
}

FString UAvaBroadcast::GetBroadcastSaveFilepath() const
{
	if (!LoadedConfigFilepath.IsEmpty())
	{
		return LoadedConfigFilepath;
	}
	
	return UE::AvaBroadcast::Private::GetJsonSaveFilepath();
}

void UAvaBroadcast::QueueNotifyChange(EAvaBroadcastChange InChange)
{
	if (InChange != EAvaBroadcastChange::None)
	{		
		const bool bCreateAsyncTask = (QueuedBroadcastChanges == EAvaBroadcastChange::None);
		EnumAddFlags(QueuedBroadcastChanges, InChange);

		if (bCreateAsyncTask)
		{
			TWeakObjectPtr<UAvaBroadcast> ThisWeak(this);
			AsyncTask(ENamedThreads::GameThread, [ThisWeak]()
			{
				if (ThisWeak.IsValid())
				{
					ThisWeak->OnBroadcastChanged.Broadcast(ThisWeak->QueuedBroadcastChanges);
					ThisWeak->QueuedBroadcastChanges = EAvaBroadcastChange::None;
				}
			});
		}
	}
}

FDelegateHandle UAvaBroadcast::AddChangeListener(FOnAvaBroadcastChanged::FDelegate&& InDelegate)
{
	return OnBroadcastChanged.Add(MoveTemp(InDelegate));
}

void UAvaBroadcast::RemoveChangeListener(FDelegateHandle InDelegateHandle)
{
	OnBroadcastChanged.Remove(InDelegateHandle);
}

void UAvaBroadcast::RemoveChangeListener(FDelegateUserObjectConst InUserObject)
{
	OnBroadcastChanged.RemoveAll(InUserObject);
}

int32 UAvaBroadcast::GetChannelNameCount() const
{
	return ChannelNames.Num();
}

int32 UAvaBroadcast::GetChannelIndex(FName InChannelName) const
{
	return ChannelNames.Find(InChannelName);
}

FName UAvaBroadcast::GetChannelName(int32 ChannelIndex) const
{
	if (ChannelNames.IsValidIndex(ChannelIndex))
	{
		return ChannelNames[ChannelIndex];
	}
	return NAME_None;
}

FName UAvaBroadcast::GetOrAddChannelName(int32 ChannelIndex)
{
	if (ChannelIndex < 0)
	{
		return NAME_None;
	}
	
	if (ChannelIndex < ChannelNames.Num())
	{
		return ChannelNames[ChannelIndex];
	}
	
	//Generate new items and set a unique names for them
	{
		//Store the current names as a Set for Fast Search
		TSet ChannelNamesSet(ChannelNames);

		//Add the New Items as Defaulted
		const int32 OldItemCount = ChannelNames.Num();
		ChannelNames.AddDefaulted(ChannelIndex - OldItemCount + 1);
		ChannelNamesSet.Reserve(ChannelNames.Num());
		
		FName UniqueName = TEXT("Channel");
		uint32 UniqueIndex = 1;
		
		for (int32 Index = OldItemCount; Index <= ChannelIndex; ++Index)
		{
			do
			{
				UniqueName.SetNumber(UniqueIndex++);
			}
			while (ChannelNamesSet.Contains(UniqueName));
			
			ChannelNamesSet.Add(UniqueName);
			ChannelNames[Index] = UniqueName;
		}
	}
	
	return ChannelNames[ChannelIndex];
}

int32 UAvaBroadcast::AddChannelName(FName InChannelName)
{
	return ChannelNames.AddUnique(InChannelName);
}

TArray<int32> UAvaBroadcast::BuildChannelIndices() const
{
	TArray<int32> ChannelIndices;
	ChannelIndices.Reserve(ChannelNames.Num());
	for (const TPair<FName, FAvaBroadcastProfile>& Pair : Profiles)
	{
		for (const FAvaBroadcastOutputChannel& Channel : Pair.Value.Channels)
		{
			ChannelIndices.AddUnique(Channel.GetChannelIndex());
		}
	}
	return ChannelIndices;
}

void UAvaBroadcast::UpdateChannelNames()
{
	// This is called when a channel is added or removed from a profile.
	// We have to reconcile all channel names from all the profiles.

	// First pass, build the new channel names list.
	//
	// Different profile may have sub-set of channels.
	// Ex:
	// Profile 1: channel1, channel3
	// Profile 2: channel1, channel2
	//	
	// Sorting by channel indices. The channel indices are
	// used for the connections (pins) in the playback graph.
	// So we want to preserve that order.
	TArray<int32> ChannelIndices = BuildChannelIndices();
	ChannelIndices.Sort();

	// Build the new channel names list from the sorted channel indices.
	TArray<FName> NewChannelNames;
	NewChannelNames.Reserve(ChannelIndices.Num());
	for (const int32 Index : ChannelIndices)
	{
		NewChannelNames.Add(GetOrAddChannelName(Index));
	}
	
	if (NewChannelNames == ChannelNames)
	{
		return;
	}
	
	// Update the channel indices in all profiles. 
	for (TPair<FName, FAvaBroadcastProfile>& Pair : Profiles)
	{
		for (FAvaBroadcastOutputChannel& Channel : Pair.Value.Channels)
		{
			Channel.SetChannelIndex(NewChannelNames.Find(Channel.GetChannelName()));
		}
	}

	TArray<FName> RemovedNames;
	for (const FName& PreviousChannelName : ChannelNames)
	{
		if (!NewChannelNames.Contains(PreviousChannelName))
		{
			RemovedNames.Add(PreviousChannelName);
		}
	}
	
	// We can finally update the ChannelNames array.
	// Channel.GetChannelName() (above) was still using old ChannelNames to find new indices.
	ChannelNames = NewChannelNames;

	// Housekeeping for internal data:
	// Remove ChannelType and PinnedChannels entries that where removed.
	for (const FName& RemovedChannelName : RemovedNames)
	{
		ChannelTypes.Remove(RemovedChannelName);
		PinnedChannels.Remove(RemovedChannelName);
	}
}

bool UAvaBroadcast::CanRenameChannel(FName InChannelName, FName InNewChannelName) const
{
	FText ErrorMessage;
	return CanRenameChannel(InChannelName, InNewChannelName, ErrorMessage);
}

bool UAvaBroadcast::CanRenameChannel(FName InChannelName, FName InNewChannelName, FText& OutErrorMessage) const
{
	if (InNewChannelName.IsNone())
	{
		OutErrorMessage = LOCTEXT("RenameChannelError_None", "New name is not valid");
		return false;
	}

	if (InChannelName == InNewChannelName)
	{
		OutErrorMessage = LOCTEXT("RenameChannelError_Unchanged", "The channel name is unchanged");
		return false;
	}

	if (!ChannelNames.Contains(InChannelName))
	{
		OutErrorMessage = FText::Format(LOCTEXT("RenameChannelError_NoChannel", "The channel to rename '{0}' was not found."), FText::FromName(InChannelName));
		return false;
	}

	if (ChannelNames.Contains(InNewChannelName))
	{
		OutErrorMessage = FText::Format(LOCTEXT("RenameChannelError_ExistingChannel", "A channel named '{0}' already exists."), FText::FromName(InNewChannelName));
		return false;
	}

	// If the new channel name is the default preview channel name,
	// check that the channel is set to be preview type to avoid unexpectedly disabling preview playback for these.
	if (InNewChannelName == UAvaRundown::GetDefaultPreviewChannelName())
	{
		if (GetChannelType(InChannelName) != EAvaBroadcastChannelType::Preview)
		{
			OutErrorMessage = FText::Format(LOCTEXT("RenameChannelError_DefaultPreview", "'{0}' is a reserved name for default preview channels, but the channel type is not preview"), FText::FromName(InNewChannelName));
			return false;
		}
	}

	return true;
}

bool UAvaBroadcast::RenameChannel(FName InChannelName, FName InNewChannelName)
{
	FText ErrorMessage;
	return RenameChannel(InChannelName, InNewChannelName, ErrorMessage);
}

bool UAvaBroadcast::RenameChannel(FName InChannelName, FName InNewChannelName, FText& OutErrorMessage)
{
	if (CanRenameChannel(InChannelName, InNewChannelName, OutErrorMessage))
	{
		const int32 Index = GetChannelIndex(InChannelName);
		check(ChannelNames.IsValidIndex(Index));
		
		ChannelNames[Index] = InNewChannelName;

		if (const EAvaBroadcastChannelType* ExistingChannelType = ChannelTypes.Find(InChannelName))
		{
			const EAvaBroadcastChannelType ExistingChannelTypeCopy = *ExistingChannelType;
			ChannelTypes.Remove(InChannelName);
			ChannelTypes.Add(InNewChannelName, ExistingChannelTypeCopy);
		}

		if (const FName* ExistingPinnedProfileName = PinnedChannels.Find(InChannelName))
		{
			const FName ExistingPinnedProfileNameCopy = *ExistingPinnedProfileName;
			PinnedChannels.Remove(InChannelName);
			PinnedChannels.Add(InNewChannelName, ExistingPinnedProfileNameCopy);
		}
		
		QueueNotifyChange(EAvaBroadcastChange::ChannelRename);
		return true;
	}
	return false;
}

void UAvaBroadcast::SetChannelType(FName InChannelName, EAvaBroadcastChannelType InChannelType)
{
	ChannelTypes.Add(InChannelName, InChannelType);
}

EAvaBroadcastChannelType UAvaBroadcast::GetChannelType(FName InChannelName) const
{
	const EAvaBroadcastChannelType* ChannelType = ChannelTypes.Find(InChannelName);
	// For backward compatibility, if the channel type is not set, defaults to Broadcast.
	return ChannelType ? *ChannelType : EAvaBroadcastChannelType::Program;
}

void UAvaBroadcast::PinChannel(FName InChannelName, FName InProfileName)
{
	PinnedChannels.Add(InChannelName, InProfileName);
}

void UAvaBroadcast::UnpinChannel(FName InChannelName)
{
	PinnedChannels.Remove(InChannelName);
}

FName UAvaBroadcast::GetPinnedChannelProfileName(FName InChannelName) const
{
	const FName* ProfileName = PinnedChannels.Find(InChannelName);
	return ProfileName ? *ProfileName : NAME_None;
}

void UAvaBroadcast::RebuildProfiles()
{
	for (TPair<FName, FAvaBroadcastProfile>& Profile : Profiles)
	{
		const bool bIsProfileActive = Profile.Key == CurrentProfile;
		Profile.Value.UpdateChannels(bIsProfileActive);
	}
}

#if WITH_EDITOR
void UAvaBroadcast::PostEditUndo()
{
	UObject::PostEditUndo();
	for (TPair<FName, FAvaBroadcastProfile>& Profile : Profiles)
	{
		const bool bIsProfileActive = Profile.Key == CurrentProfile;
		Profile.Value.UpdateChannels(bIsProfileActive);
	}
	QueueNotifyChange(EAvaBroadcastChange::All);
}
#endif

FAvaBroadcastProfile& UAvaBroadcast::CreateProfileInternal(FName InProfileName)
{	
	uint32 UniqueIndex = FMath::Max(1, InProfileName.GetNumber());
	
	while (Profiles.Contains(InProfileName))
	{
		InProfileName.SetNumber(++UniqueIndex);
	};
		
	FAvaBroadcastProfile& Profile = Profiles.Add(InProfileName, {this, InProfileName});
	Profile.AddChannel();
	return Profile;
}

void UAvaBroadcast::EnsureValidCurrentProfile()
{
	if (CurrentProfile == NAME_None || !Profiles.Contains(CurrentProfile))
	{
		if (!Profiles.IsEmpty())
		{
			SetCurrentProfile(Profiles.begin()->Key);
		}
	}
}

void UAvaBroadcast::UpdateProfileNames()
{
	for (TPair<FName, FAvaBroadcastProfile>& Pair : Profiles)
	{
		Pair.Value.SetProfileName(Pair.Key);
	}
}

#undef LOCTEXT_NAMESPACE
