// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/AvaBroadcastComponent.h"

#include "Broadcast/AvaBroadcast.h"
#include "Engine/World.h"

UAvaBroadcastComponent::UAvaBroadcastComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bWantsInitializeComponent = true;
}

bool UAvaBroadcastComponent::StartBroadcasting(FString& OutErrorMessage)
{
	UAvaBroadcast::Get().StartBroadcast();
	
	// Build the error/warning messages from all outputs of all channels.
	const TArray<FAvaBroadcastOutputChannel*>& Channels = UAvaBroadcast::Get().GetCurrentProfile().GetChannels();
	for (const FAvaBroadcastOutputChannel* Channel : Channels)
	{
		if (Channel->GetIssueSeverity() != EAvaBroadcastIssueSeverity::None)
		{
			FString OutputMessages;
			const TArray<UMediaOutput*>& Outputs = Channel->GetMediaOutputs();
			for (const UMediaOutput* Output : Outputs)
			{
				const TArray<FString>& OutputIssueMessages = Channel->GetMediaOutputIssueMessages(Output);
				const EAvaBroadcastIssueSeverity OutputSeverity = Channel->GetMediaOutputIssueSeverity(Channel->GetMediaOutputState(Output), Output);
				const FString OutputIssue = StaticEnum<EAvaBroadcastIssueSeverity>()->GetDisplayValueAsText(OutputSeverity).ToString();
				for (const FString& OutputIssueMessage : OutputIssueMessages)
				{
					OutputMessages += FString::Format(TEXT(" - Output {0}: {1}\n"),{OutputIssue,OutputIssueMessage});
				}
			}

			if (!OutputMessages.IsEmpty())
			{
				OutErrorMessage += FString::Format(TEXT("{0} on Channel {1}: \n"),
					{	StaticEnum<EAvaBroadcastIssueSeverity>()->GetDisplayValueAsText(Channel->GetIssueSeverity()).ToString(),
						Channel->GetChannelName().ToString()});
			}
			
			OutErrorMessage += OutputMessages;
		}
	}
	
	// Return true if any of the channels is broadcasting.
	return UAvaBroadcast::Get().IsBroadcastingAnyChannel();
}

bool UAvaBroadcastComponent::StopBroadcasting()
{
	UAvaBroadcast::Get().StopBroadcast();
	return true;
}

void UAvaBroadcastComponent::InitializeComponent()
{
	Super::InitializeComponent();
	FWorldDelegates::OnWorldBeginTearDown.AddUObject(this, &UAvaBroadcastComponent::OnWorldBeginTearDown);
}

void UAvaBroadcastComponent::UninitializeComponent()
{
	FWorldDelegates::OnWorldBeginTearDown.RemoveAll(this);
	Super::UninitializeComponent();
}

void UAvaBroadcastComponent::OnWorldBeginTearDown(UWorld* InWorld)
{
	if (InWorld == GetWorld() && bStopBroadcastOnTearDown)
	{
		UAvaBroadcast::Get().StopBroadcast();
	}
}

