// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "AvaBroadcastComponent.generated.h"

/**
 * Add this actor component to blueprint actor to expose the API to control the broadcasting
 * of channels in game.
 */
UCLASS(BlueprintType, Blueprintable, ClassGroup = "Motion Design Broadcast",
	meta = (Displayname = "Motion Design Broadcast Component", BlueprintSpawnableComponent))
class UAvaBroadcastComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	UAvaBroadcastComponent(const FObjectInitializer& ObjectInitializer);

	/**
	 *	Automatically stop broadcast when the world this component is part of is torn down.
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Motion Design Media")
	bool bStopBroadcastOnTearDown = true;

	/**
	 *	Start broadcasting all channels.
	 *	Returns true on partial success, i.e. will be true even if some channels didn't start.
	 *	Outputs error messages related to outputs that caused problems.
	 */
	UFUNCTION(BlueprintCallable, Category = "Motion Design Media")
	bool StartBroadcasting(FString& OutErrorMessage);

	/**
	*	Stop broadcasting all channels.
	*/
	UFUNCTION(BlueprintCallable, Category = "Motion Design Media")
	bool StopBroadcasting();
	
	virtual void InitializeComponent() override;
	virtual void UninitializeComponent() override;

protected:
	void OnWorldBeginTearDown(UWorld* InWorld);
};
