// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/AvaBroadcastLibrary.h"

#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/Channel/AvaBroadcastOutputChannel.h"
#include "Engine/Engine.h"
#include "Engine/GameInstance.h"
#include "Engine/GameViewportClient.h"
#include "Playable/AvaPlayable.h"
#include "Playable/AvaPlayableGroup.h"
#include "Playable/AvaPlayableLibrary.h"

FVector2D UAvaBroadcastLibrary::GetChannelViewportSize(const UObject* InWorldContextObject)
{
	// Attempt to access the game instance's viewport (generic method)
	if (InWorldContextObject && GEngine)
	{
		if (const UWorld* World = GEngine->GetWorldFromContextObject(InWorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
		{
			if (const UGameInstance* GameInstance = World->GetGameInstance())
			{
				if (const UGameViewportClient* ViewportClient = GameInstance->GetGameViewportClient())
				{
					FVector2d ViewportSize;
					ViewportClient->GetViewportSize(ViewportSize);
					return ViewportSize;					
				}
			}
		}
	}

	// Fallback to Channel Render Target (for AvaGameInstance)
	if (const UAvaPlayable* Playable = UAvaPlayableLibrary::GetPlayable(InWorldContextObject))
	{
		if (const UAvaPlayableGroup* PlayableGroup = Playable->GetPlayableGroup())
		{
			const FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetChannel(PlayableGroup->GetChannelName());
			if (Channel.IsValidChannel())
			{
				if (const UTextureRenderTarget2D* RenderTarget = Channel.GetCurrentRenderTarget(/*bInFallbackToPlaceholder*/true))
				{
					return FVector2D(RenderTarget->SizeX, RenderTarget->SizeY);
				}
				return Channel.DetermineRenderTargetSize();
			}
		}
	}
	
	return FAvaBroadcastOutputChannel::GetDefaultMediaOutputSize(EAvaBroadcastChannelType::Program);
}

FName UAvaBroadcastLibrary::GetChannelName(const UObject* InWorldContextObject)
{
	if (const UAvaPlayable* Playable = UAvaPlayableLibrary::GetPlayable(InWorldContextObject))
	{
		if (const UAvaPlayableGroup* PlayableGroup = Playable->GetPlayableGroup())
		{
			return PlayableGroup->GetChannelName();
		}
	}
	return NAME_None;
}

EAvaBroadcastChannelState UAvaBroadcastLibrary::GetChannelStatus(const FName InChannelName)
{
	const FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetChannel(InChannelName);
	if (Channel.IsValidChannel())
	{
		return Channel.GetState();
	}
	return EAvaBroadcastChannelState::Offline;;
}

EAvaBroadcastChannelType UAvaBroadcastLibrary::GetChannelType(const FName InChannelName)
{
	const FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetChannel(InChannelName);
	if (Channel.IsValidChannel())
	{
		return Channel.GetChannelType();
	}
	return EAvaBroadcastChannelType::Program;
}