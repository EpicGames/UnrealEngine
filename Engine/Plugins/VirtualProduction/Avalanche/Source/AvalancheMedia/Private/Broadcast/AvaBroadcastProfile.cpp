// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/AvaBroadcastProfile.h"

#include "Broadcast/AvaBroadcast.h"

#define LOCTEXT_NAMESPACE "AvaBroadcastProfile"

FAvaBroadcastProfile& FAvaBroadcastProfile::GetNullProfile()
{
	static FAvaBroadcastProfile NullProfile(nullptr, NAME_None);

	return NullProfile;
}

FAvaBroadcastProfile::FAvaBroadcastProfile(UAvaBroadcast* InBroadcast, FName InProfileName)
	: ParentBroadcastWeak(InBroadcast)
	, ProfileName(InProfileName)
{
}

void FAvaBroadcastProfile::BeginDestroy()
{
	for (FAvaBroadcastOutputChannel& Channel : GetLocalChannels())
	{
		Channel.ReleasePlaceholderResources();
		Channel.ReleaseOutputs();
		Channel.ReleasePlaceholderRenderTargets();
	}
}

UAvaBroadcast& FAvaBroadcastProfile::GetBroadcast() const
{
	// Note: during the migration towards a non-singleton UAvaBroadcast,
	// a fallback path to the singleton is kept, but it has an ensure to try and catch
	// the code paths that lead to this.
	UAvaBroadcast* ParentBroadcast = ParentBroadcastWeak.Get();
	return ensure(ParentBroadcast) ? *ParentBroadcast : UAvaBroadcast::Get();
}

void FAvaBroadcastProfile::SetProfileName(FName InProfileName)
{
	if (ProfileName != InProfileName)
	{
		ProfileName = InProfileName;
		for (FAvaBroadcastOutputChannel& Channel : Channels)
		{
			Channel.SetProfileName(InProfileName);
		}
	}
}

void FAvaBroadcastProfile::CopyProfiles(const FAvaBroadcastProfile& InSourceProfile, FAvaBroadcastProfile& OutTargetProfile)
{
	OutTargetProfile.Channels.Empty(InSourceProfile.Channels.Num());
	for (const FAvaBroadcastOutputChannel& SourceChannel : InSourceProfile.Channels)
	{
		FAvaBroadcastOutputChannel& TargetChannel = OutTargetProfile.Channels.Add_GetRef(FAvaBroadcastOutputChannel(OutTargetProfile.GetName(), &OutTargetProfile.GetBroadcast()));
		TargetChannel.SetChannelIndex(SourceChannel.GetChannelIndex());
		
		FAvaBroadcastOutputChannel::DuplicateChannel(SourceChannel, TargetChannel);
	}

	// Assuming destination profile is not current yet.
	OutTargetProfile.UpdateChannels(false);
}

bool FAvaBroadcastProfile::StartChannelBroadcast()
{
	bool bBroadcastStarted = false;
	for (FAvaBroadcastOutputChannel* Channel : GetChannels())
	{
		if (Channel->StartChannelBroadcast())
		{
			bBroadcastStarted = true;
		}
	}
	return bBroadcastStarted;
}

void FAvaBroadcastProfile::StopChannelBroadcast()
{
	for (FAvaBroadcastOutputChannel* Channel : GetChannels())
	{
		Channel->StopChannelBroadcast();
	}
}

bool FAvaBroadcastProfile::IsBroadcastingAnyChannel() const
{
	for (const FAvaBroadcastOutputChannel* Channel : GetChannels())
	{
		if (Channel->GetState() == EAvaBroadcastChannelState::Live)
		{
			return true;
		}
	}
	return false;
}

bool FAvaBroadcastProfile::IsBroadcastingAllChannels() const
{
	for (const FAvaBroadcastOutputChannel* Channel : GetChannels())
	{
		if (Channel->GetState() != EAvaBroadcastChannelState::Live)
		{
			return false;
		}
	}
	return !GetChannels().IsEmpty();
}

bool FAvaBroadcastProfile::IsValidProfile() const
{
	return this != &FAvaBroadcastProfile::GetNullProfile() && !ProfileName.IsNone();
}

namespace UE::AvaBroadcastProfile::Private
{
	// Since the channel index is serialized (now), we will initialize it to the
	// channel's profile index only if it hasn't been set already.
	inline void ConditionalSetChannelIndex(FAvaBroadcastOutputChannel& InChannel, int32 InIndex)
	{
		if (InChannel.GetChannelIndex() == INDEX_NONE)
		{
			InChannel.SetChannelIndex(InIndex);
		}
	}
}

void FAvaBroadcastProfile::PostLoadProfile(bool bInIsProfileActive, UAvaBroadcast* InBroadcast)
{
	ParentBroadcastWeak = InBroadcast;
	int32 Index = 0;
	for (FAvaBroadcastOutputChannel& Channel : Channels)
	{
		UE::AvaBroadcastProfile::Private::ConditionalSetChannelIndex(Channel, Index++);
		Channel.PostLoadMediaOutputs(bInIsProfileActive, ProfileName, InBroadcast);
	}

	ResolveChannels();
}

void FAvaBroadcastProfile::UpdateChannels(bool bInIsProfileActive)
{
	int32 Index = 0;
	for (FAvaBroadcastOutputChannel& Channel : Channels)
	{
		UE::AvaBroadcastProfile::Private::ConditionalSetChannelIndex(Channel, Index++);
	}

	ResolveChannels();

	// Remark: include the pinned channels when updating resources.
	for (FAvaBroadcastOutputChannel* Channel : ResolvedChannels)
	{
		Channel->UpdateChannelResources(bInIsProfileActive);
	}
}

void FAvaBroadcastProfile::ResolveChannels()
{
	const UAvaBroadcast& Broadcast = GetBroadcast();
	ResolvedChannels.Empty(Broadcast.GetChannelNameCount());

	for (int32 ChannelIndex = 0; ChannelIndex < Broadcast.GetChannelNameCount(); ++ChannelIndex)
	{
		const FName ChannelName = Broadcast.GetChannelName(ChannelIndex);
		FAvaBroadcastOutputChannel& Channel = GetChannelMutable(ChannelName);
		if (Channel.IsValidChannel())
		{
			ResolvedChannels.Add(&Channel);
		}
	}
}

FAvaBroadcastOutputChannel& FAvaBroadcastProfile::AddChannel(FName InChannelName)
{
	UAvaBroadcast& Broadcast = GetBroadcast();

	int32 ChannelNameIndex;
	if (!InChannelName.IsNone())
	{
		// If a name is provided, added it in the broadcast channel names.
		// It may exist already, in which case the existing index is used.
		ChannelNameIndex = Broadcast.AddChannelName(InChannelName);	
	}
	else
	{
		// Determine the last name index in this profile.
		int32 LastChannelNameIndex = -1;
		for (const FAvaBroadcastOutputChannel& Channel : Channels)
		{
			if (Channel.GetChannelIndex() > LastChannelNameIndex)
			{
				LastChannelNameIndex = Channel.GetChannelIndex();
			}
		}
		
		ChannelNameIndex = LastChannelNameIndex + 1;

		// Add the name to the broadcast if needed.
		Broadcast.GetOrAddChannelName(ChannelNameIndex);
	}

	FAvaBroadcastOutputChannel& Channel = Channels.Add_GetRef(FAvaBroadcastOutputChannel(ProfileName, &GetBroadcast()));
	Channel.SetChannelIndex(ChannelNameIndex);
	Channel.UpdateChannelResources(Broadcast.GetCurrentProfileName() == GetName());
	
	Broadcast.UpdateChannelNames();
	ResolveChannels();

	Broadcast.QueueNotifyChange(EAvaBroadcastChange::ChannelGrid);
	Broadcast.GetOnChannelsListChanged().Broadcast(*this);
	
	return Channel;
}

bool FAvaBroadcastProfile::RemoveChannel(FName InChannelName)
{
	UAvaBroadcast& Broadcast = GetBroadcast();
	
	const int32 ChannelIndex = GetLocalChannelIndexInProfile(InChannelName);

	if (ChannelIndex == INDEX_NONE)
	{
		UE_LOGF(LogAvaBroadcast, Error,
			"Can't remove channel \"%ls\" from profile \"%ls\", not found in profile.",
			*InChannelName.ToString(), *ProfileName.ToString());
		return false;
	}

	if (!Channels.IsValidIndex(ChannelIndex))
	{
		UE_LOGF(LogAvaBroadcast, Error,
			"Can't remove channel \"%ls\" (index %d) from profile \"%ls\": invalid index.",
			*InChannelName.ToString(), ChannelIndex, *ProfileName.ToString());
		return false;
	}
	
	Channels.RemoveAt(ChannelIndex);
	Broadcast.UpdateChannelNames();	// Will update channel name indices in all profiles.

	// Handle removing a pinned channel.
	if (Broadcast.GetPinnedChannelProfileName(InChannelName) == GetName())
	{
		Broadcast.UnpinChannel(InChannelName);
		Broadcast.RebuildProfiles();	// Channel names must be updated before calling this.
	}
	else
	{
		// If the channel is not pinned, resolve only this profile's channels.
		ResolveChannels();
	}

	Broadcast.QueueNotifyChange(EAvaBroadcastChange::ChannelGrid);
	Broadcast.GetOnChannelsListChanged().Broadcast(*this);
	return true;
}

int32 FAvaBroadcastProfile::GetLocalChannelIndexInProfile(FName InChannelName) const
{
	return Channels.IndexOfByPredicate([InChannelName](const FAvaBroadcastOutputChannel& InChannel)
	{
		return InChannel.GetChannelName() == InChannelName;
	});
}

int32 FAvaBroadcastProfile::GetChannelIndexInProfile(FName InChannelName) const
{
	return ResolvedChannels.IndexOfByPredicate([InChannelName](const FAvaBroadcastOutputChannel* InChannel)
	{
		return InChannel->GetChannelName() == InChannelName;
	});
}

const FAvaBroadcastOutputChannel& FAvaBroadcastProfile::GetLocalChannel(FName InChannelName) const
{
	const int32 ChannelIndex = GetLocalChannelIndexInProfile(InChannelName);
	
	if (Channels.IsValidIndex(ChannelIndex))
	{
		return Channels[ChannelIndex];
	}
	
	return FAvaBroadcastOutputChannel::GetNullChannel();
}

FAvaBroadcastOutputChannel& FAvaBroadcastProfile::GetLocalChannelMutable(FName InChannelName)
{
	const int32 ChannelIndex = GetLocalChannelIndexInProfile(InChannelName);
	
	if (Channels.IsValidIndex(ChannelIndex))
	{
		return Channels[ChannelIndex];
	}
	
	return FAvaBroadcastOutputChannel::GetNullChannel();
}

const FAvaBroadcastOutputChannel& FAvaBroadcastProfile::GetChannel(FName InChannelName) const
{
	UAvaBroadcast& Broadcast = GetBroadcast();
	const FName PinnedChannelProfileName = Broadcast.GetPinnedChannelProfileName(InChannelName);
	if (PinnedChannelProfileName != NAME_None)
	{
		return Broadcast.GetProfile(PinnedChannelProfileName).GetLocalChannel(InChannelName);
	}	
	return GetLocalChannel(InChannelName); 
}

FAvaBroadcastOutputChannel& FAvaBroadcastProfile::GetChannelMutable(FName InChannelName)
{
	UAvaBroadcast& Broadcast = GetBroadcast();
	const FName PinnedChannelProfileName = Broadcast.GetPinnedChannelProfileName(InChannelName);
	if (PinnedChannelProfileName != NAME_None)
	{
		const FAvaBroadcastProfile& PinnedProfile = Broadcast.GetProfile(PinnedChannelProfileName); 
		if (PinnedProfile.IsValidProfile())
		{
			return Broadcast.GetProfile(PinnedChannelProfileName).GetLocalChannelMutable(InChannelName);
		}

		UE_LOGF(LogAvaBroadcast, Error,
			"Can't find profile \"%ls\" for pinned channel \"%ls\". Using local channel instead.",
			*PinnedChannelProfileName.ToString(), *InChannelName.ToString());	
	}
	return GetLocalChannelMutable(InChannelName);
}

FAvaBroadcastOutputChannel& FAvaBroadcastProfile::GetOrAddChannel(FName InChannelName)
{
	FAvaBroadcastOutputChannel& Channel = GetChannelMutable(InChannelName);
	return Channel.IsValidChannel() ? Channel : AddChannel(InChannelName);
}

UMediaOutput* FAvaBroadcastProfile::AddChannelMediaOutput(FName InChannelName, const UClass* InMediaOutputClass, const FAvaBroadcastMediaOutputInfo& InOutputInfo)
{
	FAvaBroadcastOutputChannel& Channel = GetChannelMutable(InChannelName);
	if (Channel.IsValidChannel())
	{
		return Channel.AddMediaOutput(InMediaOutputClass, InOutputInfo);
	}
	return nullptr;
}

int32 FAvaBroadcastProfile::RemoveChannelMediaOutputs(FName InChannelName, const TArray<UMediaOutput*>& InMediaOutputs)
{
	FAvaBroadcastOutputChannel& Channel = GetChannelMutable(InChannelName);
	if (Channel.IsValidChannel())
	{
		int32 RemovedCount = 0;
		for (UMediaOutput* const MediaOutput : InMediaOutputs)
		{
			RemovedCount += Channel.RemoveMediaOutput(MediaOutput);
		}		
		return RemovedCount;
	}
	return 0;
}

#undef LOCTEXT_NAMESPACE