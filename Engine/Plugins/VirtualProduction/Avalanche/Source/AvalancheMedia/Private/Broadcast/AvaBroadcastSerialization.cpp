// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaBroadcastSerialization.h"

#include "AvaMediaSerializationUtils.h"
#include "Broadcast/AvaBroadcast.h"
#include "Formatters/XmlArchiveInputFormatter.h"
#include "Formatters/XmlArchiveOutputFormatter.h"
#include "JsonObjectConverter.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"

namespace UE::AvaMedia::BroadcastSerialization::Private
{
	struct FJsonObjectSerializerContext
	{
		TArray<TSharedPtr<FJsonValue>> ReferencedObjects;

		TSharedPtr<FJsonValue> ExportPropertyCallback(FProperty* InProperty, const void* InValue)
		{
			TSharedPtr<FJsonValue> Result = nullptr;

			if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InProperty))
			{
				const UObject* const Object = ObjectProperty->GetObjectPropertyValue_InContainer(InValue);
				if (IsValid(Object))
				{
					const TSoftObjectPtr<UClass> ObjectClass =  Object->GetClass();
					const TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
					// Do we need to keep the path to the outer?
					JsonObject->SetStringField(TEXT("Class"), ObjectClass.ToSoftObjectPath().ToString());
					JsonObject->SetStringField(TEXT("Name"), Object->GetName());
					JsonObject->SetStringField(TEXT("Flags"), FString::Printf(TEXT("%d"), static_cast<uint32>(Object->GetFlags())));
					JsonObject->SetObjectField(TEXT("Object"), SerializeToJson(Object));

					ReferencedObjects.Add(MakeShared<FJsonValueObject>(JsonObject));
				}
			}
		
			return Result;
		}

		TSharedPtr<FJsonObject> SerializeToJson(const UObject* InObject)
		{
			FJsonObjectConverter::CustomExportCallback CustomCB;
			CustomCB.BindRaw(this, &FJsonObjectSerializerContext::ExportPropertyCallback);

			const TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			constexpr int64 CheckFlags = 0;
			constexpr int64 SkipFlags = 0;
			FJsonObjectConverter::UStructToJsonObject(InObject->GetClass(), InObject, JsonObject.ToSharedRef(), CheckFlags, SkipFlags, &CustomCB, EJsonObjectConversionFlags::SkipStandardizeCase);

			return JsonObject;
		}
	};
	
	UObject* LoadReferencedObject(UObject* InRootObject, const TSharedPtr<FJsonObject>& InReferenceObject)
	{
		const FSoftObjectPath ClassPath(InReferenceObject->GetStringField(TEXT("Class")));
		const TSoftObjectPtr<UClass> ClassPtr(ClassPath);
		UClass* const ObjectClass = ClassPtr.LoadSynchronous();
		
		if (!ObjectClass)
		{
			return nullptr;
		}

		const TSharedPtr<FJsonObject> JsonObjectData = InReferenceObject->GetObjectField(TEXT("Object"));
		if (!JsonObjectData)
		{
			return nullptr;
		}
		
		const FString OriginalObjectName = InReferenceObject->GetStringField(TEXT("Name"));

		// Try to find the object, it may exist already.
		//UObject* NestedObject = StaticFindObject(ObjectClass, InRootObject, *OriginalObjectName, EFindObjectFlags::ExactClass );
		UObject* NestedObject = static_cast<UObject*>(FindObjectWithOuter(InRootObject, ObjectClass, FName(OriginalObjectName)));

		if (!NestedObject)
		{
			// We need to create the object
			const FString FlagsString = InReferenceObject->GetStringField(TEXT("Flags"));
			const uint32 Flags = static_cast<uint32>(FCString::Atoi(*FlagsString));
			const EObjectFlags ObjectFlags = static_cast<EObjectFlags>(Flags);
			
			NestedObject = NewObject<UObject>(InRootObject, ObjectClass, FName(OriginalObjectName), ObjectFlags);
		}

		if (!FJsonObjectConverter::JsonObjectToUStruct(JsonObjectData.ToSharedRef(), ObjectClass, NestedObject))
		{
			return nullptr;
		}

		return NestedObject;
	}

	void FixupReferences(const TSharedPtr<FJsonValue>& InValue, const FString& InOldReference, const FString& InNewReference);

	void FixupReferences(const TArray<TSharedPtr<FJsonValue>>& InValues, const FString& InOldReference, const FString& InNewReference)
	{
		for (const TSharedPtr<FJsonValue>& Value : InValues)
		{
			FixupReferences(Value, InOldReference, InNewReference);
		}
	}
	
	void FixupReferences(const TSharedPtr<FJsonObject>& InJsonObject, const FString& InOldReference, const FString& InNewReference)
	{
		for (TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Value : InJsonObject->Values)
		{
			FixupReferences(Value.Value, InOldReference, InNewReference);
		}
	}

	// Can't seem to find a way to set the string value other than this.
	class FJsonStringFixupHelper : public FJsonValueString
	{
	public:
		FJsonStringFixupHelper(const FString& InString) : FJsonValueString(InString) {}
			
		void FixupString(const FString& InOldReference, const FString& InNewReference)
		{
			if (Value.Contains(InOldReference, ESearchCase::CaseSensitive))
			{
				Value = Value.Replace(*InOldReference, *InNewReference, ESearchCase::CaseSensitive);
			}
		}
	};

	void FixupReferences(const TSharedPtr<FJsonValue>& InValue, const FString& InOldReference, const FString& InNewReference)
	{
		switch(InValue->Type)
		{
		case EJson::String:
			static_cast<FJsonStringFixupHelper*>(InValue.Get())->FixupString(InOldReference, InNewReference);	
			break;
		case EJson::Array:
			FixupReferences(InValue->AsArray(), InOldReference, InNewReference);
			break;
		case EJson::Object:
			FixupReferences(InValue->AsObject(), InOldReference, InNewReference);
			break;
		default:
			break;
		}
	}

	// Custom import callback for UObjects (i.e. primarily Media Outputs) so that if it fails, it can continue serializing the rest of the properties
	bool OnCustomImportJsonObjectValue(const TSharedPtr<FJsonValue>& InJsonValue, FProperty* InProperty, void* InValue)
	{
		if (!InJsonValue.IsValid() || InJsonValue->Type != EJson::Object)
		{
			return false;
		}

		const FObjectProperty* const ObjectProperty = CastField<FObjectProperty>(InProperty);
		if (!ObjectProperty)
		{
			return false;
		}

		const TSharedPtr<FJsonObject> Object = InJsonValue->AsObject();

		const FString ClassString = Object->GetStringField(TEXT("_ClassName"));
		if (ClassString.IsEmpty())
		{
			ObjectProperty->SetObjectPropertyValue(InValue, nullptr);
			return true; // Logic overriden, the built-in logic won't be executed
		}

		const UClass* const FoundClass = FPackageName::IsShortPackageName(ClassString) ? FindFirstObject<UClass>(*ClassString) : LoadClass<UObject>(nullptr, *ClassString);
		if (!FoundClass || FoundClass->HasAnyClassFlags(CLASS_Abstract))
		{
			ObjectProperty->SetObjectPropertyValue(InValue, nullptr);
			return true; // Logic overriden, the built-in logic won't be executed
		}

		// Object should be valid, execute built-in logic
		return false;
	}
}

bool FAvaBroadcastSerialization::SaveBroadcastToJson(const UAvaBroadcast* InBroadcast, const FString& InFilename)
{
	using namespace UE::AvaMedia::BroadcastSerialization::Private;
	FJsonObjectSerializerContext SerializerContext;
	
	const TSharedPtr<FJsonObject> JsonBroadcast = SerializerContext.SerializeToJson(InBroadcast);

	const FSoftObjectPath BroadcastPath(InBroadcast);
	
	const TSharedPtr<FJsonObject> JsonRoot = MakeShared<FJsonObject>();
	JsonRoot->SetStringField(TEXT("BroadcastPath"), BroadcastPath.ToString());	// Used to fixup reference when loading.
	JsonRoot->SetObjectField(TEXT("Broadcast"), JsonBroadcast);
	JsonRoot->SetArrayField(TEXT("ReferencedObjects"), SerializerContext.ReferencedObjects);
	
	FString OutJsonString;
	const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutJsonString);
	if (!FJsonSerializer::Serialize(JsonRoot.ToSharedRef(), Writer))
	{
		return false;
	}

	if (!FFileHelper::SaveStringToFile(OutJsonString, *InFilename))
	{
		UE_LOGF(LogAvaBroadcast, Error, "Couldn't save data to file: %ls", *InFilename);
		return false;
	}

	return true;
}

bool FAvaBroadcastSerialization::LoadBroadcastFromJson(const FString& InFilename, UAvaBroadcast* OutBroadcast)
{
	// A missing file will not be considered an error as part of initializing a broadcast object
	// since it may not have been saved yet. However, we log it to help troubleshoot potential issues.
	if (!FPaths::FileExists(InFilename))
	{
		UE_LOGF(LogAvaBroadcast, Log, "Json Configuration file \"%ls\" not found.", *InFilename);
		return false;
	}

	FString JsonText;

	// Load json text to the string object
	if (!FFileHelper::LoadFileToString(JsonText, *InFilename))
	{
		UE_LOGF(LogAvaBroadcast, Error, "Couldn't read file: %ls", *InFilename);
		return false;
	}

	TSharedPtr<FJsonObject> JsonRoot = MakeShared<FJsonObject>();

	const TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(JsonText);
	if (!FJsonSerializer::Deserialize(JsonReader, JsonRoot))
	{
		UE_LOGF(LogAvaBroadcast, Error, "Couldn't parse json data from file: %ls", *InFilename);
		return false;
	}

	const TSharedPtr<FJsonObject> BroadcastJson = JsonRoot->GetObjectField(TEXT("Broadcast"));
	if (!BroadcastJson)
	{
		return false;
	}

	using namespace UE::AvaMedia::BroadcastSerialization::Private;
	
	const FSoftObjectPath BroadcastPath(OutBroadcast);
	const FSoftObjectPath OrigBroadcastPath(JsonRoot->GetStringField(TEXT("BroadcastPath")));

	if (BroadcastPath != OrigBroadcastPath)
	{
		// In case the root object is not the same, we need to fixup the references.
		FixupReferences(BroadcastJson, OrigBroadcastPath.ToString(), BroadcastPath.ToString());
	}
	
	TArray<TSharedPtr<FJsonValue>> ReferencedObjects = JsonRoot->GetArrayField(TEXT("ReferencedObjects"));

	// The save traversal produces leaf objects last, so we reverse to get them first for loading. 
	Algo::Reverse(ReferencedObjects);
	
	for (TSharedPtr<FJsonValue> ReferenceObjectValue : ReferencedObjects)
	{
		const TSharedPtr<FJsonObject> ReferenceObject = ReferenceObjectValue.Get()->AsObject(); 
		if (BroadcastPath != OrigBroadcastPath)
		{
			FixupReferences(ReferenceObject, OrigBroadcastPath.ToString(), BroadcastPath.ToString());
		}
		LoadReferencedObject(OutBroadcast, ReferenceObject);
	}

	FJsonObjectConverter::CustomImportCallback CustomImporter;
	CustomImporter.BindStatic(&OnCustomImportJsonObjectValue);

	constexpr int64 CheckFlags = 0;
	constexpr int64 SkipFlags = 0;
	constexpr bool bStrictMode = false;
	return FJsonObjectConverter::JsonObjectToUStruct(BroadcastJson.ToSharedRef(), OutBroadcast->GetClass(), OutBroadcast, CheckFlags, SkipFlags, bStrictMode, /*FailReason*/nullptr, &CustomImporter);
}

bool FAvaBroadcastSerialization::SaveBroadcastToXml(UAvaBroadcast* InBroadcast, const FString& InFilename)
{
	bool bIsBroadcastSaved = false;
#if WITH_EDITOR
	const TUniquePtr<FArchive> FileWriter(IFileManager::Get().CreateFileWriter(*InFilename));
	if (FileWriter.IsValid())	
	{
		FXmlArchiveOutputFormatter XmlOutput(*FileWriter);
		XmlOutput.SerializeObjectsInPlace(true); // We want the media outputs nested.
		UE::AvaMediaSerializationUtils::SerializeObject(XmlOutput, InBroadcast);
		bIsBroadcastSaved = XmlOutput.SaveDocumentToInnerArchive();
		FileWriter->Close();
	}
#endif
	return bIsBroadcastSaved;
}

bool FAvaBroadcastSerialization::LoadBroadcastFromXml(const FString& InFilename, UAvaBroadcast* OutBroadcast)
{
	bool bIsBroadcastLoaded = false;
#if WITH_EDITOR
	const TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*InFilename));
	if (FileReader.IsValid())
	{
		FXmlArchiveInputFormatter InputFormatter(*FileReader, OutBroadcast);
		UE::AvaMediaSerializationUtils::SerializeObject(InputFormatter, OutBroadcast);
		FileReader->Close();
		bIsBroadcastLoaded = true;
	}
#endif	
	return bIsBroadcastLoaded;
}
