// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

class UAvaBroadcast;

struct FAvaBroadcastSerialization
{
	static bool SaveBroadcastToJson(const UAvaBroadcast* InBroadcast, const FString& InFilename);
	static bool LoadBroadcastFromJson(const FString& InFilename, UAvaBroadcast* OutBroadcast);

	static bool SaveBroadcastToXml(UAvaBroadcast* InBroadcast, const FString& InFilename);
	static bool LoadBroadcastFromXml(const FString& InFilename, UAvaBroadcast* OutBroadcast);
};