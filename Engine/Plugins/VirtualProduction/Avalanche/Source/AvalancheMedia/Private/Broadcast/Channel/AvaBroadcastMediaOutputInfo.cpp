// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/Channel/AvaBroadcastMediaOutputInfo.h"
#include "Broadcast/OutputDevices/AvaBroadcastDeviceProviderProxy.h"

bool FAvaBroadcastMediaOutputInfo::IsRemote(const FString& InServerName)
{
	return !InServerName.IsEmpty() && InServerName != FAvaBroadcastDeviceProviderProxyManager::LocalServerName;
}

void FAvaBroadcastMediaOutputInfo::PostLoad()
{
	if (!Guid.IsValid())
	{
		Guid = FGuid::NewGuid();
	}
}
