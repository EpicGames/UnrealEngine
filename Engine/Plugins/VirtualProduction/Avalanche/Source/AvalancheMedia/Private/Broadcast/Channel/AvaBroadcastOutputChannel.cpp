// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/Channel/AvaBroadcastOutputChannel.h"
#include "AvaMediaRenderTargetUtils.h"
#include "AvaMediaSettings.h"
#include "Blueprint/UserWidget.h"
#include "Blueprint/WidgetBlueprintGeneratedClass.h"
#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/AvaBroadcastProfile.h"
#include "Broadcast/Channel/IAvaBroadcastOutputChannelViewInterface.h"
#include "Broadcast/Channel/Slate/SAvaBroadcastPlaceholderWidget.h"
#include "Broadcast/IAvaBroadcastSettings.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputUtils.h"
#include "Broadcast/OutputDevices/AvaBroadcastRenderTargetMediaUtils.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Extensions/UserWidgetExtension.h"
#include "Framework/Application/SlateApplication.h"
#include "GameTime.h"
#include "GlobalShader.h"
#include "IAvaMediaModule.h"
#include "MediaCapture.h"
#include "MediaOutput.h"
#include "MediaShaders.h"
#include "PixelFormat.h"
#include "Playable/AvaPlayableGroup.h"
#include "Playback/IAvaPlaybackClient.h"
#include "RenderGraphBuilder.h"
#include "RenderResource.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "SceneView.h"
#include "Slate/WidgetRenderer.h"
#include "ScreenPass.h"
#include "TextureResource.h"
#include "UObject/NameTypes.h"
#include "UObject/Package.h"

DEFINE_LOG_CATEGORY_STATIC(LogAvaOutputChannel, Log, All);

void FAvaBroadcastCapture::Reset(UMediaCapture* InMediaCapture, UMediaOutput* InMediaOutput, UTextureRenderTarget2D* InRenderTarget)
{
	if (IsValid(MediaCapture))
	{
		MediaCapture->OnStateChangedNative.RemoveAll(this);
	}
	
	OnMediaCaptureStateChanged.Clear();
	MediaCapture = InMediaCapture;
	MediaOutput = InMediaOutput;
	RenderTarget = InRenderTarget;
	if (IsValid(MediaCapture))
	{
		MediaCapture->OnStateChangedNative.AddRaw(this, &FAvaBroadcastCapture::OnStateChangedNative);
	}
}

void FAvaBroadcastCapture::OnStateChangedNative()
{
	if (OnMediaCaptureStateChanged.IsBound())
	{
		OnMediaCaptureStateChanged.Broadcast(MediaCapture, MediaOutput);
	}
}

FAvaBroadcastOutputChannel::FAvaBroadcastOutputChannel(ENoInit NoInit)
	: QualitySettings(NoInit)
{
}

FAvaBroadcastOutputChannel::FAvaBroadcastOutputChannel(FAvaBroadcastProfile* InProfile)
	: FAvaBroadcastOutputChannel(
		InProfile ? InProfile->GetName() : NAME_None,
		InProfile ? &InProfile->GetBroadcast() : &UAvaBroadcast::Get())
{
	
}

FAvaBroadcastOutputChannel::FAvaBroadcastOutputChannel(FName InProfileName, UAvaBroadcast* InBroadcast)
	: ProfileName(InProfileName)
	, ParentBroadcastWeak(InBroadcast)
{
	
}

namespace UE::AvaMedia::Private
{
	EMediaCaptureOverrunAction GetCaptureOverrunAction()
	{
		const UAvaMediaSettings& Settings = UAvaMediaSettings::Get();
		switch (Settings.ChannelOutputOverrunAction)
		{
		case EAvaBroadcastOutputOverrunAction::Skip:
			return EMediaCaptureOverrunAction::Skip;
		case EAvaBroadcastOutputOverrunAction::Flush:
		default:
			return EMediaCaptureOverrunAction::Flush;
		}
	}
	
	bool IsCapturing(const UMediaCapture* InMediaCapture)
	{
		if (IsValid(InMediaCapture))
		{
			const EMediaCaptureState CaptureState = InMediaCapture->GetState();

			return CaptureState == EMediaCaptureState::Capturing
				|| CaptureState == EMediaCaptureState::Preparing;
		}
		return false;
	}

	static void AddInvertAlphaConversionPass(FRDGBuilder& GraphBuilder
		, const FRDGTextureRef& SourceRGBTexture
		, const FResolveRect& ResolveRect
		, const FVector2D& SizeU
		, const FVector2D& SizeV
		, FRDGTextureRef OutputResource)
	{
		// Rectangle area to use from source
		const FIntRect ViewRect(ResolveRect.X1, ResolveRect.Y1, ResolveRect.X2, ResolveRect.Y2);

		//Dummy ViewFamily/ViewInfo created to use built in Draw Screen/Texture Pass
		FSceneViewFamily ViewFamily(FSceneViewFamily::ConstructionValues(nullptr, nullptr, FEngineShowFlags(ESFIM_Game))
			.SetTime(FGameTime()));
		FSceneViewInitOptions ViewInitOptions;
		ViewInitOptions.ViewFamily = &ViewFamily;
		ViewInitOptions.SetViewRectangle(ViewRect);
		ViewInitOptions.ViewOrigin = FVector::ZeroVector;
		ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
		ViewInitOptions.ProjectionMatrix = FMatrix::Identity;
		FSceneView ViewInfo(ViewInitOptions);

		//At some point we should support color conversion (ocio) but for now we push incoming texture as is
		constexpr bool bDoLinearToSRGB = false;

		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FScreenPassVS> VertexShader(GlobalShaderMap);

		//Configure source/output viewport to get the right UV scaling from source texture to output texture
		FScreenPassTextureViewport InputViewport(SourceRGBTexture, ViewRect);
		FScreenPassTextureViewport OutputViewport(OutputResource);

		// In cases where texture is converted from a format that doesn't have A channel, we want to force set it to 1.
		EMediaCaptureConversionOperation MediaConversionOperation = EMediaCaptureConversionOperation::INVERT_ALPHA;
		FModifyAlphaSwizzleRgbaPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FModifyAlphaSwizzleRgbaPS::FConversionOp>(static_cast<int32>(MediaConversionOperation));

		TShaderMapRef<FModifyAlphaSwizzleRgbaPS> PixelShader(GlobalShaderMap, PermutationVector);
		FModifyAlphaSwizzleRgbaPS::FParameters* Parameters = PixelShader->AllocateAndSetParameters(GraphBuilder, SourceRGBTexture, OutputResource);
		AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("AvaOutputInvertAlpha"), ViewInfo, OutputViewport, InputViewport, VertexShader, PixelShader, Parameters);
	}

	void ExecuteInvertAlphaConversionPass(FRHICommandListImmediate& RHICmdList, UTextureRenderTarget2D* SourceRT, UTextureRenderTarget2D* DestinationRT)
	{
		FRDGBuilder GraphBuilder(RHICmdList);

		const FTextureRHIRef SourceTexture = SourceRT->GetRenderTargetResource()->GetTexture2DRHI();
		const FRDGTextureRef SourceRGBTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceTexture, TEXT("SourceTexture")));

		const FTextureRHIRef DestinationTexture = DestinationRT->GetRenderTargetResource()->GetTexture2DRHI();
		const FRDGTextureRef OutputResource = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(DestinationTexture, TEXT("DestTexture")));

		const FVector2D SizeU = { 0.0f, 1.0f };
		const FVector2D SizeV = { 0.0f, 1.0f };

		const FResolveRect ResolveRect(0, 0, SourceTexture->GetSizeX(), SourceTexture->GetSizeY());

		AddInvertAlphaConversionPass(GraphBuilder, SourceRGBTexture, ResolveRect, SizeU, SizeV, OutputResource);

		GraphBuilder.Execute();
	}

	namespace MediaOutputInfoHelper
	{
		/**
		 * Legacy support helper.
		 * Initialize the media output info from a given media output object.
		 * Note: this will only work correctly if the device can be found in
		 * a device provider. If the device is on a remote server, the server
		 * must be connected when this function is called.
		*/
		FAvaBroadcastMediaOutputInfo InitFrom(const UMediaOutput* InMediaOutput)
		{
			FAvaBroadcastMediaOutputInfo OutputInfo;

			if (::IsValid(InMediaOutput))
			{
				const FString DeviceNameString = UE::AvaBroadcastOutputUtils::GetDeviceName(InMediaOutput);
				if (!DeviceNameString.IsEmpty())
				{
					const FName DeviceName = *DeviceNameString;
					const FName DeviceProviderName = UE::AvaBroadcastOutputUtils::GetDeviceProviderName(InMediaOutput);
					const FString ServerName = IAvaMediaModule::Get().GetServerNameForDevice(DeviceProviderName, DeviceName);
					if (!ServerName.IsEmpty() || IAvaMediaModule::Get().IsLocalDevice(DeviceProviderName, DeviceName))
					{
						OutputInfo.DeviceName = DeviceName;
						OutputInfo.DeviceProviderName = DeviceProviderName;
						OutputInfo.ServerName = ServerName;
					}
				}
			}
			return OutputInfo;
		}
	}
}

FAvaBroadcastOutputChannel& FAvaBroadcastOutputChannel::GetNullChannel()
{
	static FAvaBroadcastOutputChannel NullChannel(NoInit);

	return NullChannel;
}

FAvaBroadcastOutputChannel::FOnAvaChannelChanged FAvaBroadcastOutputChannel::OnChannelChanged;
FAvaBroadcastOutputChannel::FMediaOutputStateChanged FAvaBroadcastOutputChannel::OnMediaOutputStateChanged;

FAvaBroadcastOutputChannel::~FAvaBroadcastOutputChannel()
{
	StopPlaceholderTick();

#if WITH_EDITOR
	for (UMediaOutput* MediaOutput : MediaOutputs)
	{
		if (MediaOutput)
		{
			MediaOutput->OnOutputModified().RemoveAll(this);
		}
	}
#endif
}

void FAvaBroadcastOutputChannel::ReleasePlaceholderResources()
{
	OverridePlaceholderWidget.Reset();
	if (OverridePlaceholderUserWidget)
	{
		OverridePlaceholderUserWidget->ReleaseSlateResources(true);
		OverridePlaceholderUserWidget = nullptr;
	}
}

void FAvaBroadcastOutputChannel::ReleasePlaceholderRenderTargets()
{
	PlaceholderRenderTarget = nullptr;
	PlaceholderRenderTargetTmp = nullptr;
}

void FAvaBroadcastOutputChannel::ReleaseOutputs()
{
	for (UMediaOutput* MediaOutput : MediaOutputs)
	{
		StopCaptureForOutput(MediaOutput);
#if WITH_EDITOR
		MediaOutput->OnOutputModified().RemoveAll(this);
#endif
	}
	MediaOutputs.Reset();
}

void FAvaBroadcastOutputChannel::DuplicateChannel(const FAvaBroadcastOutputChannel& InSourceChannel, FAvaBroadcastOutputChannel& OutTargetChannel)
{
	OutTargetChannel.MediaOutputs.Reset(InSourceChannel.MediaOutputs.Num());
	for (const UMediaOutput* const SourceMediaOutput : InSourceChannel.MediaOutputs)
	{
		UMediaOutput* const TargetMediaOutput = DuplicateObject<UMediaOutput>(SourceMediaOutput
			, SourceMediaOutput->GetOuter()
			, NAME_None);

		OutTargetChannel.MediaOutputs.Add(TargetMediaOutput);
		OutTargetChannel.MediaOutputInfos.Add(InSourceChannel.GetMediaOutputInfo(SourceMediaOutput));
		OutTargetChannel.MediaOutputInfos.Last().Guid = FGuid::NewGuid();	// Allocate a new guid for the duplicate.
	}
	OnChannelChanged.Broadcast(OutTargetChannel, EAvaBroadcastChannelChange::MediaOutputs);
	OutTargetChannel.UpdateChannelResources(/*bInIsProfileActive*/false);
	OutTargetChannel.RefreshState();
}

void FAvaBroadcastOutputChannel::SetChannelIndex(int32 InIndex)
{
	ChannelIndex = InIndex;
}

FName FAvaBroadcastOutputChannel::GetChannelName() const
{
	return GetProfile().GetBroadcast().GetOrAddChannelName(ChannelIndex);
}

EAvaBroadcastChannelType FAvaBroadcastOutputChannel::GetChannelType() const
{
	UAvaBroadcast& Broadcast = GetProfile().GetBroadcast();
	const FName ChannelName = Broadcast.GetOrAddChannelName(ChannelIndex);
	return Broadcast.GetChannelType(ChannelName);
}

FName FAvaBroadcastOutputChannel::GetProfileName() const
{
	return ProfileName;
}

FAvaBroadcastProfile& FAvaBroadcastOutputChannel::GetProfile() const
{
	// Note: During the migration towards a non-singleton UAvaBroadcast,
	// a fallback path to the singleton is kept, but it has an ensure to try and catch
	// the code paths that lead to this.
	ensure(ParentBroadcastWeak.IsValid()); 
	UAvaBroadcast& Broadcast = ParentBroadcastWeak.IsValid() ? *ParentBroadcastWeak.Get() : UAvaBroadcast::Get();
	FAvaBroadcastProfile& Profile = Broadcast.GetProfile(ProfileName);
	return Profile.IsValidProfile() ? Profile : Broadcast.GetCurrentProfile();
}

EAvaBroadcastChannelState FAvaBroadcastOutputChannel::RefreshState()
{
	// Resolve the channel state from current output states. (PROTOTYPE LOGIC)
	EAvaBroadcastChannelState NewChannelState = EAvaBroadcastChannelState::Offline;
	EAvaBroadcastIssueSeverity NewChannelIssueSeverity = EAvaBroadcastIssueSeverity::None;

	for (const UMediaOutput* MediaOutput : GetMediaOutputs())
	{
		// Offline outputs don't contribute to state (i.e. channel will be offline too).
		const EAvaBroadcastOutputState OutputState = GetMediaOutputState(MediaOutput);
		if (OutputState == EAvaBroadcastOutputState::Live || OutputState == EAvaBroadcastOutputState::Preparing)
		{
			NewChannelState = EAvaBroadcastChannelState::Live;
		}
		else if (OutputState == EAvaBroadcastOutputState::Idle && NewChannelState != EAvaBroadcastChannelState::Live)
		{
			NewChannelState = EAvaBroadcastChannelState::Idle;
		}

		const EAvaBroadcastIssueSeverity OutputIssueSeverity = GetMediaOutputIssueSeverity(OutputState, MediaOutput);
		if (OutputIssueSeverity > NewChannelIssueSeverity)
		{
			NewChannelIssueSeverity = OutputIssueSeverity;
		}
	}
	
	// Note: this will broadcast to delegates which may then request states of media outputs.
	// So the back store needs to be updated before calling this.
	SetState(NewChannelState, NewChannelIssueSeverity);

	return GetState();
}

EAvaBroadcastIssueSeverity FAvaBroadcastOutputChannel::GetMediaOutputIssueSeverity(EAvaBroadcastOutputState InOutputState, const UMediaOutput* InMediaOutput) const
{
	if (!IsValid(InMediaOutput))
	{
		return EAvaBroadcastIssueSeverity::Errors;
	}
	
	if (InOutputState == EAvaBroadcastOutputState::Live || InOutputState == EAvaBroadcastOutputState::Preparing)
	{
		// If the output is broadcasting remote, fetch the status from the playback client (which is proxying that output's status).
		if (IAvaMediaModule::Get().IsPlaybackClientStarted() && IsMediaOutputRemote(InMediaOutput))
		{
			const FAvaBroadcastMediaOutputInfo& OutputInfo = GetMediaOutputInfo(InMediaOutput);
			const IAvaPlaybackClient& PlaybackClient = IAvaMediaModule::Get().GetPlaybackClient();
			return PlaybackClient.GetMediaOutputIssueSeverity(GetMediaOutputServerName(InMediaOutput), GetChannelName().ToString(), OutputInfo.Guid);
		}

		const UMediaCapture* const MediaCapture = GetMediaCaptureForOutput(InMediaOutput);
		
		// For now, we don't have a way to determine if the capture has warnings.
		// We only know if it worked or not.
		if (IsValid(MediaCapture) && UE::AvaMedia::Private::IsCapturing(MediaCapture))
		{
			return GetLocalSeverityForOutput(InMediaOutput);
		}

		// If the output is live but we don't have a media capture, that would be an error.
		return EAvaBroadcastIssueSeverity::Errors;
	}

	// If the state is "error", we override severity with "errors", but the local severity should be "errors" as well
	// and we expect error messages to have been recorded.
	return (InOutputState == EAvaBroadcastOutputState::Error) ?  EAvaBroadcastIssueSeverity::Errors : GetLocalSeverityForOutput(InMediaOutput);
}

const TArray<FString>& FAvaBroadcastOutputChannel::GetMediaOutputIssueMessages(const UMediaOutput* InMediaOutput) const
{
	if (IsValid(InMediaOutput))
	{
		if (IAvaMediaModule::Get().IsPlaybackClientStarted() && IsMediaOutputRemote(InMediaOutput))
		{
			const FAvaBroadcastMediaOutputInfo& OutputInfo = GetMediaOutputInfo(InMediaOutput);
			const IAvaPlaybackClient& PlaybackClient = IAvaMediaModule::Get().GetPlaybackClient();
			return PlaybackClient.GetMediaOutputIssueMessages(GetMediaOutputServerName(InMediaOutput), GetChannelName().ToString(), OutputInfo.Guid);
		}
		
		if (const FLocalMediaOutputStatus* LocalMediaOutputStatus = LocalMediaOutputStatuses.Find(InMediaOutput))
		{
			return LocalMediaOutputStatus->Messages;
		}
	}

	static const TArray<FString> EmptyStringArray;
	return EmptyStringArray;
}

EAvaBroadcastOutputState FAvaBroadcastOutputChannel::GetMediaOutputState(const UMediaOutput* InMediaOutput) const
{
	if (!IsValid(InMediaOutput))
	{
		return EAvaBroadcastOutputState::Invalid;
	}
	
	if (IsMediaOutputRemote(InMediaOutput))
	{
		if (IAvaMediaModule::Get().IsPlaybackClientStarted())
		{
			const FAvaBroadcastMediaOutputInfo& OutputInfo = GetMediaOutputInfo(InMediaOutput);
			if (!OutputInfo.IsValid())
			{
				UE_LOGF(LogAvaOutputChannel, Error, "Channel \"%ls\": MediaOutput \"%ls\" has invalid info.",
					*GetNameSafe(InMediaOutput), *GetChannelName().ToString());
			}
			
			// If the output is broadcasting remote, fetch the status from the playback client (which is proxying that output's status).
			const IAvaPlaybackClient& PlaybackClient = IAvaMediaModule::Get().GetPlaybackClient();
			return PlaybackClient.GetMediaOutputState(GetMediaOutputServerName(InMediaOutput),GetChannelName().ToString(), OutputInfo.Guid);
		}
		else
		{
			// Remote output if offline if the client is not running.
			return EAvaBroadcastOutputState::Offline;
		}
	}
	
	const UMediaCapture* const MediaCapture = GetMediaCaptureForOutput(InMediaOutput);

	if (IsValid(MediaCapture))
	{
		// For now, we don't have a way to determine if the capture has warnings.
		// We only know if it worked or not.
		switch (MediaCapture->GetState())
		{
		case EMediaCaptureState::Capturing:
		case EMediaCaptureState::StopRequested:
			return EAvaBroadcastOutputState::Live;
		case EMediaCaptureState::Preparing:
			return EAvaBroadcastOutputState::Preparing;
		case EMediaCaptureState::Error:
			return EAvaBroadcastOutputState::Error;
		default:
			return EAvaBroadcastOutputState::Idle;
		}
	}

	// Check if this output has reported errors.	
	return GetLocalSeverityForOutput(InMediaOutput) == EAvaBroadcastIssueSeverity::Errors ?
		EAvaBroadcastOutputState::Error : EAvaBroadcastOutputState::Idle;
}

const FAvaBroadcastMediaOutputInfo& FAvaBroadcastOutputChannel::GetMediaOutputInfo(const UMediaOutput* InMediaOutput) const
{
	const int32 Index = MediaOutputs.Find(const_cast<UMediaOutput*>(InMediaOutput));
	if (MediaOutputInfos.IsValidIndex(Index))
	{
		return MediaOutputInfos[Index];
	}
	static FAvaBroadcastMediaOutputInfo Empty;
	return Empty;
}

FAvaBroadcastMediaOutputInfo* FAvaBroadcastOutputChannel::GetMediaOutputInfoMutable(const UMediaOutput* InMediaOutput)
{
	const int32 Index = MediaOutputs.Find(const_cast<UMediaOutput*>(InMediaOutput));
	return (MediaOutputInfos.IsValidIndex(Index)) ? &MediaOutputInfos[Index] : nullptr; 
}

namespace UE::AvaMedia::BroadcastChannel
{
	inline void UpdatePlaceHolderRenderTarget(TObjectPtr<UTextureRenderTarget2D>& InOutRenderTarget, FName InBaseName, const FIntPoint& InSize, EPixelFormat InFormat, const FLinearColor& InClearColor)
	{
		using namespace UE::AvaMediaRenderTargetUtils;
		if (!InOutRenderTarget)
		{
			InOutRenderTarget = CreateDefaultRenderTarget(InBaseName);
		}

		UpdateRenderTarget(InOutRenderTarget.Get(), InSize, InFormat, InClearColor);
	}
}

bool FAvaBroadcastOutputChannel::IsEditingEnabled() const
{
	return ChannelState != EAvaBroadcastChannelState::Live;
}

void FAvaBroadcastOutputChannel::SetState(EAvaBroadcastChannelState InState, EAvaBroadcastIssueSeverity InIssueSeverity)
{
	if (ChannelState != InState || ChannelIssueSeverity != InIssueSeverity)
	{
		ChannelState = InState;
		ChannelIssueSeverity = InIssueSeverity;

		if (InState == EAvaBroadcastChannelState::Live)
		{
			StartPlaceholderTick();
		}
		else
		{
			StopPlaceholderTick();
		}
		OnChannelChanged.Broadcast(*this, EAvaBroadcastChannelChange::State);
	}
}

void FAvaBroadcastOutputChannel::UpdatePlaceholder()
{
	if (!FSlateApplication::IsInitialized())
	{
		return;
	}
	
	if (!PlaceholderWidget.IsValid())
	{
		PlaceholderWidget = SNew(SAvaBroadcastPlaceholderWidget);
	}

	if (!WidgetRenderer.IsValid())
	{
		WidgetRenderer = MakeShared<FWidgetRenderer>();
		
		WidgetRenderer->SetIsPrepassNeeded(true);
		
		//Enable Gamma Correction for Widget Rendering
		WidgetRenderer->SetUseGammaCorrection(true);
		
		//Clear Target before rendering
		WidgetRenderer->SetShouldClearTarget(true);
	}

	UpdateOverridePlaceholder();

	PlaceholderWidget->SetChannelName(ChannelNameText);
	
	UpdatePlaceholderRenderTargets();
}

void FAvaBroadcastOutputChannel::UpdatePlaceholderRenderTargets()
{
	const FIntPoint RenderTargetSize = DetermineRenderTargetSize();
	const EPixelFormat RenderTargetFormat = DetermineRenderTargetFormat();
	const FLinearColor ClearColor = IAvaMediaModule::Get().GetBroadcastSettings().GetChannelClearColor();
	static const FName PlaceholderRenderTargetName = TEXT("AvaOutputChannel_PlaceHolderRT");
	static const FName PlaceholderRenderTargetTmpName = TEXT("AvaOutputChannel_TmpPlaceHolderRT");

	using namespace UE::AvaMediaRenderTargetUtils;
	const FIntPoint PreviousRenderTargetSize = GetRenderTargetSizeSafe(PlaceholderRenderTarget);
	
	using namespace UE::AvaMedia::BroadcastChannel;
	UpdatePlaceHolderRenderTarget(PlaceholderRenderTarget, PlaceholderRenderTargetName, RenderTargetSize, RenderTargetFormat, ClearColor);
	UpdatePlaceHolderRenderTarget(PlaceholderRenderTargetTmp, PlaceholderRenderTargetTmpName, RenderTargetSize, RenderTargetFormat, ClearColor);

	// Broadcast the render target resize for editor to resize preview brushes.
	if (PreviousRenderTargetSize != GetRenderTargetSizeSafe(PlaceholderRenderTarget))
	{
		OnChannelChanged.Broadcast(*this, EAvaBroadcastChannelChange::RenderTarget);
	}
}

void FAvaBroadcastOutputChannel::UpdateOverridePlaceholder()
{
	const IAvaBroadcastSettings& BroadcastSettings = IAvaMediaModule::Get().GetBroadcastSettings();

	if (!BroadcastSettings.IsDrawPlaceholderWidget())
	{
		OverridePlaceholderWidget.Reset();
		return;
	}
	
	UClass* const CurrentOverridePlaceholderClass = OverridePlaceholderUserWidget
		? OverridePlaceholderUserWidget->GetClass()
		: nullptr;

	// Remark: on the server, if this settings comes from a client, the server may not
	// have the asset and will need to sync it.
	// TODO: use sync service to sync asset and use it when ready.
	TSoftClassPtr<UUserWidget> PlaceholderWidgetClass(BroadcastSettings.GetPlaceholderWidgetClass());
	UClass* const TargetOverridePlaceholderClass = PlaceholderWidgetClass.LoadSynchronous();
	
	//Update only if the Current Placeholder Class is different from the one in Media Settings
	if (CurrentOverridePlaceholderClass != TargetOverridePlaceholderClass)
	{
		OverridePlaceholderUserWidget = nullptr;
		if (TargetOverridePlaceholderClass)
		{			
			OverridePlaceholderUserWidget = NewObject<UUserWidget>(GetTransientPackage()
				, TargetOverridePlaceholderClass
				, NAME_None
				, RF_Transient);
		}

		OverridePlaceholderWidget.Reset();
		if (OverridePlaceholderUserWidget)
		{
			OverridePlaceholderWidget = OverridePlaceholderUserWidget->TakeWidget();

			//Update Tick State
			if (OverridePlaceholderWidget.IsValid())
			{
				// Default to never tick, only recompute for auto
				bool bCanTick = false;

				//These rules are pretty much the same as UUserWidget::UpdateCanTick but without the UWorld Checks
				if (OverridePlaceholderUserWidget->GetDesiredTickFrequency() == EWidgetTickFrequency::Auto)
				{
					// Note: WidgetBPClass can be NULL in a cooked build.
					UWidgetBlueprintGeneratedClass* const WidgetBPClass = Cast<UWidgetBlueprintGeneratedClass>(OverridePlaceholderUserWidget->GetClass());
					bCanTick |= !WidgetBPClass || WidgetBPClass->ClassRequiresNativeTick();
					bCanTick |= OverridePlaceholderUserWidget->bHasScriptImplementedTick;
					if (!bCanTick)
					{
						for (UUserWidgetExtension* const Extension : OverridePlaceholderUserWidget->GetExtensions(UUserWidgetExtension::StaticClass()))
						{
							if (Extension->RequiresTick())
							{
								bCanTick = true;
								break;
							}
						}
					}
				}

				OverridePlaceholderWidget->SetCanTick(bCanTick);
			}
		}
	}

	if (OverridePlaceholderUserWidget && OverridePlaceholderUserWidget->Implements<UAvaBroadcastOutputChannelViewInterface>())
	{
		IAvaBroadcastOutputChannelViewInterface::Execute_SetChannelName(OverridePlaceholderUserWidget, ChannelNameText);
	}
}

inline bool AreDimensionsValid(const UTextureRenderTarget2D* InRenderTarget, const int32 InMaxAllowedDrawSize)
{
	return (InRenderTarget->SizeX <= 0
		|| InRenderTarget->SizeY <= 0
		|| InRenderTarget->SizeX > InMaxAllowedDrawSize
		|| InRenderTarget->SizeY > InMaxAllowedDrawSize) ? false : true;
}

void FAvaBroadcastOutputChannel::DrawPlaceholderWidget() const
{
	if (!IsValid(PlaceholderRenderTarget) || !IsValid(PlaceholderRenderTargetTmp))
	{
		return;
	}
	
	const int32 MaxAllowedDrawSize = GetMax2DTextureDimension();
	
	if (!AreDimensionsValid(PlaceholderRenderTarget.Get(), MaxAllowedDrawSize) 
		|| !AreDimensionsValid(PlaceholderRenderTargetTmp.Get(), MaxAllowedDrawSize))
	{
		return;
	}

	const bool bHasWidgetResources = PlaceholderWidget.IsValid() && WidgetRenderer.IsValid();
	
	if (IAvaMediaModule::Get().GetBroadcastSettings().IsDrawPlaceholderWidget() && bHasWidgetResources)
	{
		TSharedRef<SWidget> Widget = OverridePlaceholderWidget.IsValid()
			? OverridePlaceholderWidget.ToSharedRef()
			: PlaceholderWidget.ToSharedRef();
	
		// Render the widget in the tmp render target.
		WidgetRenderer->DrawWidget(PlaceholderRenderTargetTmp.Get()
			, Widget
			, FVector2D(PlaceholderRenderTargetTmp->SizeX, PlaceholderRenderTargetTmp->SizeY)
			, FSlateApplication::Get().GetDeltaTime());

		// Invert the Alpha on the Placeholder Widget
		UTextureRenderTarget2D* SourceRT = PlaceholderRenderTargetTmp.Get();
		UTextureRenderTarget2D* DestinationRT = PlaceholderRenderTarget.Get();

		ENQUEUE_RENDER_COMMAND(FAvaOutputInvertAlpha)(
			[SourceRT, DestinationRT](FRHICommandListImmediate& RHICmdList)
			{
				UE::AvaMedia::Private::ExecuteInvertAlphaConversionPass(RHICmdList, SourceRT, DestinationRT);
			});
	}
	else
	{
		UE::AvaBroadcastRenderTargetMediaUtils::ClearRenderTarget(PlaceholderRenderTarget.Get());
	}
}

bool FAvaBroadcastOutputChannel::IsValidChannel() const
{
	return this != &FAvaBroadcastOutputChannel::GetNullChannel() && GetChannelName() != NAME_None;
}

bool FAvaBroadcastOutputChannel::StartChannelBroadcast()
{
	IAvaMediaModule& Module = IAvaMediaModule::Get();
	if (Module.IsPlaybackClientStarted())
	{
		const TArray<UMediaOutput*> RemoteMediaOutputs = GetRemoteMediaOutputs();
		if (!RemoteMediaOutputs.IsEmpty())
		{
			Module.GetPlaybackClient().RequestBroadcast(ProfileName.ToString(), GetChannelName(), RemoteMediaOutputs, EAvaBroadcastAction::Start);
		}
	}

	const FString ChannelName = GetChannelName().ToString();
	
	if (GetState() == EAvaBroadcastChannelState::Live)
	{
		UE_LOGF(LogAvaOutputChannel, Warning
			, "Output Channel (%ls) attempted to Start Channel Broadcast while already Running!"
			, *ChannelName);
		return true;
	}

	// Ensure placeholder render targets are compatible with media outputs.
	UpdateChannelResources(/*bInIsProfileActive*/true);
	
	UTextureRenderTarget2D* const RenderTarget = GetCurrentRenderTarget(/*bInFallbackToPlaceholder*/true);
	
	for (UMediaOutput* MediaOutput : MediaOutputs)
	{
		if (!IsValid(MediaOutput))
		{
			UE_LOGF(LogAvaOutputChannel, Warning
				, "Output Channel (%ls) has an Invalid Media Output."
				, *ChannelName);
			continue;
		}

		ResetLocalMediaOutputStatus(MediaOutput);
		
		// Don't locally start remote media outputs.
		if (IsMediaOutputRemote(MediaOutput))
		{
			continue;
		}
		
		// Check if this MediaOutput can start with the given render target
		if (!IsMediaOutputCompatible(MediaOutput, RenderTarget))
		{
			StopCaptureForOutput(MediaOutput);	// Ensures no stale capture is running.
			continue;
		}

		UMediaCapture* const MediaCapture = MediaOutput->CreateMediaCapture();
		
		if (!IsValid(MediaCapture))
		{
			FString& Message = EmplaceMessageForOutput(MediaOutput, EAvaBroadcastIssueSeverity::Errors);
			
			// Try to get a failure reason.
			FString FailureReason;
			if (!MediaOutput->Validate(FailureReason))
			{
				Message = FString::Printf(TEXT("Couldn't create media capture. %s"), *FailureReason);
			}
			else
			{
				Message = FString::Printf(TEXT("Couldn't create media capture."));
			}

			UE_LOGF(LogAvaOutputChannel, Error, "Output Channel (%ls) has a Media Output (%ls): %ls",
				*ChannelName, *MediaOutput->GetName(), *Message);
			
			StopCaptureForOutput(MediaOutput);	// Ensures no stale capture is running.
			continue;
		}
		
		// TODO: Expose this along with MediaOutput.
		FMediaCaptureOptions CaptureOptions;
		
		// Don't stop Capture when Throttling the Engine
		CaptureOptions.bSkipFrameWhenRunningExpensiveTasks = false;
		// Allow the formats to be converted if different.
		CaptureOptions.bConvertToDesiredPixelFormat = true;
		CaptureOptions.OverrunAction = UE::AvaMedia::Private::GetCaptureOverrunAction();
		
		check(RenderTarget);
		if (MediaCapture->CaptureTextureRenderTarget2D(RenderTarget, CaptureOptions))
		{
			FAvaBroadcastCapture& AvaMediaCapture = MediaCaptures.Emplace(MediaOutput);
			AvaMediaCapture.Reset(MediaCapture, MediaOutput, RenderTarget);
			AvaMediaCapture.OnMediaCaptureStateChanged.AddRaw(this, &FAvaBroadcastOutputChannel::OnAvaMediaCaptureStateChanged); 
		}
		else
		{
			FString& Message = EmplaceMessageForOutput(MediaOutput, EAvaBroadcastIssueSeverity::Errors);
			Message = TEXT("Failed to initialize. See logs for more info.");
		}
	}

	RefreshState();
	return true;
}

void FAvaBroadcastOutputChannel::StopChannelBroadcast()
{
	StopPlaceholderTick();
	
	IAvaMediaModule& Module = IAvaMediaModule::Get();
	if (Module.IsPlaybackClientStarted())
	{
		const TArray<UMediaOutput*> RemoteMediaOutputs = GetRemoteMediaOutputs();
		if (!RemoteMediaOutputs.IsEmpty())
		{
			Module.GetPlaybackClient().RequestBroadcast(ProfileName.ToString(), GetChannelName(), RemoteMediaOutputs, EAvaBroadcastAction::Stop);
		}
	}

	for (const UMediaOutput* MediaOutput : MediaOutputs)
	{
		ResetLocalMediaOutputStatus(MediaOutput);
		StopCaptureForOutput(MediaOutput);
	}
	
	MediaCaptures.Reset();
	LastActivePlayableGroup.Reset();
	CurrentRenderTarget.Reset();
	
	if (IsValid(PlaceholderRenderTarget))
	{
		PlaceholderRenderTarget->UpdateResourceImmediate();
	}

	if (IsValid(PlaceholderRenderTargetTmp))
	{
		PlaceholderRenderTargetTmp->UpdateResourceImmediate();
	}

	SetState(EAvaBroadcastChannelState::Idle, EAvaBroadcastIssueSeverity::None);
}

UMediaOutput* FAvaBroadcastOutputChannel::AddMediaOutput(const UClass* InMediaOutputClass, const FAvaBroadcastMediaOutputInfo& InOutputInfo)
{
	// Don't add a remote output to a local preview channel.
	if (InOutputInfo.IsRemote() && GetChannelType() == EAvaBroadcastChannelType::Preview)
	{
		return nullptr;
	}
	
	if (InMediaOutputClass && InMediaOutputClass->IsChildOf(UMediaOutput::StaticClass()))
	{
		UMediaOutput* const MediaOutput = NewObject<UMediaOutput>(UAvaBroadcast::GetBroadcast()
			, InMediaOutputClass
			, NAME_None
			, RF_Transactional);

		// If this is a MediaOutput class that requires render targets to be resolved to cpu memory,
		// we need more buffers to avoid having the flush rendering commands.
		static const int32 MinimumNumberOfTextureBuffers = 3;
		if (MediaOutput->NumberOfTextureBuffers < MinimumNumberOfTextureBuffers)
		{
			MediaOutput->NumberOfTextureBuffers = MinimumNumberOfTextureBuffers;
		}

		// Set bInvertKeyOutput property to true by default since we always output a scene render with inverted alpha.
		{
			// Known implementations that have it: BlackMagic, Aja, NDI.
			// Ava Display and Rivermax don't have it, but don't support Key.
			FBoolProperty* const Property = FindFProperty<FBoolProperty>(MediaOutput->GetClass(), TEXT("bInvertKeyOutput"));
			if (Property)
			{
				Property->SetPropertyValue(Property->ContainerPtrToValuePtr<bool>(MediaOutput, 0), true);
			}
		}

		FAvaBroadcastMediaOutputInfo OutputInfo = InOutputInfo;

		// If the device was not enumerated, we try to get the device name from the MediaOutput object.
		if (OutputInfo.DeviceName.IsNone())
		{
			OutputInfo.DeviceName = FName(UE::AvaBroadcastOutputUtils::GetDeviceName(MediaOutput));
		}

		AddMediaOutput(MediaOutput, OutputInfo);
		
		return MediaOutput;
	}
	return nullptr;
}

void FAvaBroadcastOutputChannel::AddMediaOutput(UMediaOutput* InMediaOutput, const FAvaBroadcastMediaOutputInfo& InOutputInfo)
{
	if (IsValid(InMediaOutput))
	{
		// Check that we don't have a remote output on a local preview.
		check(!(InOutputInfo.IsValid() && InOutputInfo.IsRemote() && GetChannelType() == EAvaBroadcastChannelType::Preview));
#if WITH_EDITOR
		InMediaOutput->OnOutputModified().AddRaw(this, &FAvaBroadcastOutputChannel::OnMediaOutputModified);
#endif
		MediaOutputs.Add(InMediaOutput);
		MediaOutputInfos.Add(InOutputInfo);
		// Note: the media output may not be fully configured at that point.
		OnChannelChanged.Broadcast(*this, EAvaBroadcastChannelChange::MediaOutputs);
		UpdateChannelResources(/*bInIsProfileActive*/true);
		RefreshState();
	}
}

int32 FAvaBroadcastOutputChannel::RemoveMediaOutput(UMediaOutput* InMediaOutput)
{
	int32 RemoveCount = 0;
	const int32 IndexToRemove = MediaOutputs.Find(InMediaOutput);
	if (IndexToRemove != INDEX_NONE)
	{
#if WITH_EDITOR
		InMediaOutput->OnOutputModified().RemoveAll(this);
#endif
		RemoveCount = 1;
		MediaOutputs.RemoveAt(IndexToRemove);
		MediaOutputInfos.RemoveAt(IndexToRemove);
		LocalMediaOutputStatuses.Remove(InMediaOutput);
		OnChannelChanged.Broadcast(*this, EAvaBroadcastChannelChange::MediaOutputs);
		UpdateChannelResources(/*bInIsProfileActive*/true);
		RefreshState();
	}
	return RemoveCount;
}

void FAvaBroadcastOutputChannel::OnMediaOutputModified(UMediaOutput* InMediaOutput)
{
	// Update media output info's device name.
	if (const FString DeviceName = UE::AvaBroadcastOutputUtils::GetDeviceName(InMediaOutput); !DeviceName.IsEmpty())
	{
		if (FAvaBroadcastMediaOutputInfo* OutputInfo = GetMediaOutputInfoMutable(InMediaOutput))
		{
			OutputInfo->DeviceName = FName(DeviceName);
		}
	}
	
	IAvaMediaModule& AvaMediaModule = IAvaMediaModule::Get();
	if (IsMediaOutputRemote(InMediaOutput) && AvaMediaModule.IsPlaybackClientStarted())
	{
		IAvaPlaybackClient& PlaybackClient = AvaMediaModule.GetPlaybackClient();
		PlaybackClient.RequestBroadcast(ProfileName.ToString(), GetChannelName(), GetRemoteMediaOutputs(), EAvaBroadcastAction::UpdateConfig);
	}
	else
	{
		UpdateChannelResources(/*bInIsProfileActive*/true);
		RefreshState();
	}
}

void FAvaBroadcastOutputChannel::SetProfileName(FName InProfileName)
{
	ProfileName = InProfileName;	
}

void FAvaBroadcastOutputChannel::PostLoadMediaOutputs(bool bInIsProfileActive, FAvaBroadcastProfile* InProfile)
{
	PostLoadMediaOutputs(bInIsProfileActive, InProfile ? InProfile->GetName() : NAME_None,
		InProfile ? &InProfile->GetBroadcast() : &UAvaBroadcast::Get());
}

void FAvaBroadcastOutputChannel::PostLoadMediaOutputs(bool bInIsProfileActive, FName InProfileName, UAvaBroadcast* InBroadcast)
{
	ProfileName = InProfileName;
	ParentBroadcastWeak = InBroadcast;
	
	// Create missing output infos (legacy support)
	if (MediaOutputInfos.Num() != MediaOutputs.Num())
	{
		MediaOutputInfos.Empty(MediaOutputs.Num());
		for (const UMediaOutput* Output : MediaOutputs)
		{
			MediaOutputInfos.Add(UE::AvaMedia::Private::MediaOutputInfoHelper::InitFrom(Output));
		}
	}

	// Remove invalid outputs and corresponding infos.
	for (int32 OutputIndex = 0; OutputIndex < MediaOutputs.Num();)
	{
		if (!IsValid(MediaOutputs[OutputIndex]))
		{
			MediaOutputs.RemoveAt(OutputIndex);
			MediaOutputInfos.RemoveAt(OutputIndex);
		}
		else
		{
			++OutputIndex;
		}
	}

	// Complete missing fields (legacy support)
	for (FAvaBroadcastMediaOutputInfo& MediaOutputInfo : MediaOutputInfos)
	{
		MediaOutputInfo.PostLoad();
	}

#if WITH_EDITOR
	for (UMediaOutput* MediaOutput : MediaOutputs)
	{
		if (!MediaOutput->OnOutputModified().IsBoundToObject(this))
		{
			MediaOutput->OnOutputModified().AddRaw(this, &FAvaBroadcastOutputChannel::OnMediaOutputModified);
		}
	}
#endif
	
	OnChannelChanged.Broadcast(*this, EAvaBroadcastChannelChange::MediaOutputs);
	UpdateChannelResources(bInIsProfileActive);
	RefreshState();
}

void FAvaBroadcastOutputChannel::UpdateRenderTarget(UAvaPlayableGroup* InPlayableGroup, UTextureRenderTarget2D* InRenderTarget)
{
	// In the current design, the render target is associated with a "playable group".
	// It can only be associated with one group at a time.

	const bool bRenderTargetChanged = CurrentRenderTarget.Get() != InRenderTarget;
	
	LastActivePlayableGroup = InPlayableGroup;
	CurrentRenderTarget = InRenderTarget;

	if (bRenderTargetChanged)
	{
		UpdateViewportTarget();
	}
}

void FAvaBroadcastOutputChannel::UpdateAudioDevice(const FAudioDeviceHandle& InAudioDeviceHandle)
{
	for (TPair<TObjectPtr<UMediaOutput>, FAvaBroadcastCapture>& MediaCapture : MediaCaptures)
	{
		UMediaCapture* MediaCapturePtr = MediaCapture.Value.MediaCapture;
		if (MediaCapturePtr->GetCaptureAudioDevice().GetDeviceID() != InAudioDeviceHandle.GetDeviceID())
		{
			MediaCapturePtr->SetCaptureAudioDevice(InAudioDeviceHandle);
		}
	}
}

UAvaPlayableGroup* FAvaBroadcastOutputChannel::GetLastActivePlayableGroup() const
{
	return LastActivePlayableGroup.Get();
}

UTextureRenderTarget2D* FAvaBroadcastOutputChannel::GetCurrentRenderTarget(bool bInFallbackToPlaceholder) const
{
	if (CurrentRenderTarget.IsValid())
	{
		return CurrentRenderTarget.Get();
	}
	
	return bInFallbackToPlaceholder
		? GetPlaceholderRenderTarget()
		: nullptr;
}

UTextureRenderTarget2D* FAvaBroadcastOutputChannel::GetPlaceholderRenderTarget() const
{
	return PlaceholderRenderTarget.Get();
}

bool FAvaBroadcastOutputChannel::HasAnyLocalMediaOutputs() const
{
	for (const UMediaOutput* MediaOutput : MediaOutputs)
	{
		if (IsValid(MediaOutput) && !IsMediaOutputRemote(MediaOutput))
		{
			return true;
		}
	}
	return false;
}

bool FAvaBroadcastOutputChannel::HasAnyRemoteMediaOutputs() const
{
	for (const UMediaOutput* MediaOutput : MediaOutputs)
	{
		if (IsValid(MediaOutput) && IsMediaOutputRemote(MediaOutput))
		{
			return true;
		}
	}
	return false;
}


bool FAvaBroadcastOutputChannel::IsMediaOutputRemote(const UMediaOutput* InMediaOutput) const
{
	// Ideally, the output info is valid here, so we just need to see if the
	// output is associated with a remote server.
	const FAvaBroadcastMediaOutputInfo& OutputInfo = GetMediaOutputInfo(InMediaOutput);
	if (OutputInfo.IsValid())
	{
		return OutputInfo.IsRemote();
	}

	// Fallback/Legacy, in case the output info is not valid,
	// if the playback client is started, we can try the legacy method
	// of checking the remote device providers. This is much less reliable.
	IAvaMediaModule& AvaMediaModule = IAvaMediaModule::Get();
	if (AvaMediaModule.IsPlaybackClientStarted())
	{
		return AvaMediaModule.GetPlaybackClient().IsMediaOutputRemoteFallback(InMediaOutput);
	}

	return false;
}

const FString& FAvaBroadcastOutputChannel::GetMediaOutputServerName(const UMediaOutput* InMediaOutput) const
{
	const FAvaBroadcastMediaOutputInfo& OutputInfo = GetMediaOutputInfo(InMediaOutput);
	if (OutputInfo.IsValid())
	{
		return OutputInfo.ServerName;
	}

	static const FString EmptyString;
	return EmptyString;
}

TArray<UMediaOutput*> FAvaBroadcastOutputChannel::GetRemoteMediaOutputs() const
{
	TArray<UMediaOutput*> RemoteMediaOutputs;
	RemoteMediaOutputs.Reserve(MediaOutputs.Num());
	for (UMediaOutput* MediaOutput : MediaOutputs)
	{
		if (IsValid(MediaOutput) && IsMediaOutputRemote(MediaOutput))
		{
			RemoteMediaOutputs.Add(MediaOutput);
		}
	}
	return RemoteMediaOutputs;
}

bool FAvaBroadcastOutputChannel::IsMediaOutputCompatible(UMediaOutput* InMediaOutput, UTextureRenderTarget2D* InRenderTarget) const
{
	// Check if this MediaOutput can start with the given render target
	if (IsValid(InRenderTarget) && IsValid(InMediaOutput))
	{
		// Size Match Check
		{
			const FIntPoint MediaOutputSize = InMediaOutput->GetRequestedSize();
			const FIntPoint RenderTargetSize = UE::AvaMediaRenderTargetUtils::GetRenderTargetSize(InRenderTarget);
			if (MediaOutputSize != UMediaOutput::RequestCaptureSourceSize && MediaOutputSize != RenderTargetSize)
			{
				FString& Message = EmplaceMessageForOutput(InMediaOutput, EAvaBroadcastIssueSeverity::Errors);
				Message = FString::Printf(
					TEXT("Media Output \"%s\" requested size (%d, %d) is not compatible with Channel \"%s\" Render Target \"%s\" size (%d, %d).")
					, *InMediaOutput->GetName(), MediaOutputSize.X, MediaOutputSize.Y
					, *GetChannelName().ToString()
					, *InRenderTarget->GetName(), RenderTargetSize.X, RenderTargetSize.Y);
				
				UE_LOGF(LogAvaOutputChannel, Error, "%ls", *Message);
				return false;
			}
		}

		// Format Match Check
		{
			const EPixelFormat MediaOutputFormat = InMediaOutput->GetRequestedPixelFormat();
			const EPixelFormat RenderTargetFormat = InRenderTarget->GetFormat();
			if (MediaOutputFormat != PF_Unknown && MediaOutputFormat != RenderTargetFormat)
			{
				const int32 TargetNumColorBits = UE::AvaMediaRenderTargetUtils::GetNumColorChannelBits(RenderTargetFormat);
				const int32 OutputNumColorBits = UE::AvaMediaRenderTargetUtils::GetNumColorChannelBits(MediaOutputFormat);
				if (OutputNumColorBits > TargetNumColorBits)
				{
					FString& Message = EmplaceMessageForOutput(InMediaOutput, EAvaBroadcastIssueSeverity::Errors);
					Message = FString::Printf(
						TEXT("Media Output \"%s\" requested format %s is not compatible with Channel \"%s\" Render Target \"%s\" format %s")
						TEXT(" because it has higher color precision (%d vs %d bits).")
						, *InMediaOutput->GetName(), GetPixelFormatString(MediaOutputFormat)
						, *GetChannelName().ToString()
						, *InRenderTarget->GetName(), GetPixelFormatString(RenderTargetFormat)
						, OutputNumColorBits, TargetNumColorBits);

					UE_LOGF(LogAvaOutputChannel, Error, "%ls", *Message);
					return false;
				}

				// RGBA8 vs RGB10A2, we will detect a conflict here.
				// RGB10A2 wins in color bits, but loose in alpha bits.
				const int32 TargetNumAlphaBits = UE::AvaMediaRenderTargetUtils::GetNumAlphaChannelBits(RenderTargetFormat);
				const int32 OutputNumAlphaBits = UE::AvaMediaRenderTargetUtils::GetNumAlphaChannelBits(MediaOutputFormat);
				if (OutputNumAlphaBits > TargetNumAlphaBits)
				{
					FString& Message = EmplaceMessageForOutput(InMediaOutput, EAvaBroadcastIssueSeverity::Warnings);
					Message = FString::Printf(
						TEXT("Media Output \"%s\" requested format %s is not compatible with Channel \"%s\" Render Target \"%s\" format %s")
						TEXT(" because it has higher alpha precision (%d vs %d bits).")
						, *InMediaOutput->GetName(), GetPixelFormatString(MediaOutputFormat)
						, *GetChannelName().ToString()
						, *InRenderTarget->GetName(), GetPixelFormatString(RenderTargetFormat)
						, OutputNumAlphaBits, TargetNumAlphaBits);

					UE_LOGF(LogAvaOutputChannel, Warning, "%ls", *Message);
				}
			}
		}
		return true;
	}
	return false;
}

void FAvaBroadcastOutputChannel::UpdateViewportTarget()
{
	//Only need to update when Broadcasting, since when starting Broadcast it will Capture and update it anyways
	if (GetState() == EAvaBroadcastChannelState::Live)
	{
		UTextureRenderTarget2D* const RenderTarget = GetCurrentRenderTarget(/*bInFallbackToPlaceholder*/true);

		bool bNeedStateRefresh = false;

		// Start all media outputs that have a compatible format 
		// and are not already started on another game instance.
		for (UMediaOutput* const MediaOutput : MediaOutputs)
		{
			if (!IsValid(MediaOutput))
			{
				continue;
			}

			if (IsMediaOutputRemote(MediaOutput))
			{
				continue;
			}

			// For now, we don't update devices that are not live.
			if (GetMediaOutputState(MediaOutput) != EAvaBroadcastOutputState::Live)
			{
				continue;
			}
		
			// Check if this MediaOutput can still run with the given render target.
			// Remark: this test maybe optional. It is normally done when channel start
			// and we don't allow configuration changes to the output while it is running.
			if (!IsMediaOutputCompatible(MediaOutput, RenderTarget))
			{
				// Ensure no stale capture is running for this output.
				StopCaptureForOutput(MediaOutput);
				bNeedStateRefresh = true;
				continue;
			}

			FAvaBroadcastCapture* AvaMediaCapture = MediaCaptures.Find(MediaOutput);
			UMediaCapture* const MediaCapture = AvaMediaCapture ? AvaMediaCapture->MediaCapture.Get() : nullptr;
			
			// Double Check and Skip Updating SceneViewport / RenderTarget if not Capturing
			// Remark: if the capture is an external/removable device, it can stop the capture.
			// We can the state to be refreshed at least.
			if (!UE::AvaMedia::Private::IsCapturing(MediaCapture))
			{
				bNeedStateRefresh = true;
				continue;
			}
			
			check(RenderTarget);

			// We go out of our way to avoid updating the render target (if it is the same)
			// because UpdateTextureRenderTarget2D has side effects even if we are updating with the same render target.
			if (AvaMediaCapture && AvaMediaCapture->RenderTarget.IsValid() && AvaMediaCapture->RenderTarget == RenderTarget)
			{
				continue;
			}
			
			if (AvaMediaCapture && MediaCapture->UpdateTextureRenderTarget2D(RenderTarget))
			{
				AvaMediaCapture->RenderTarget = RenderTarget;
			}
		}
		
		if (bNeedStateRefresh)
		{
			RefreshState();
		}
	}
	OnChannelChanged.Broadcast(*this, EAvaBroadcastChannelChange::RenderTarget);
}

UMediaCapture* FAvaBroadcastOutputChannel::GetMediaCaptureForOutput(const UMediaOutput* InMediaOutput) const
{
	const FAvaBroadcastCapture* AvaMediaCapture = MediaCaptures.Find(InMediaOutput);
	return AvaMediaCapture ? AvaMediaCapture->MediaCapture : nullptr;
}

void FAvaBroadcastOutputChannel::StopCaptureForOutput(const UMediaOutput* InMediaOutput)
{
	UMediaCapture* const MediaCapture = GetMediaCaptureForOutput(InMediaOutput);
	if (IsValid(MediaCapture) && UE::AvaMedia::Private::IsCapturing(MediaCapture))
	{
		MediaCapture->StopCapture(/*bAllowPendingFrameToBeProcess*/false);
	}
	MediaCaptures.Remove(InMediaOutput);
}

void FAvaBroadcastOutputChannel::TickPlaceholder(float)
{
	// Only Draw if Render Target is Invalid 
	if (GetState() == EAvaBroadcastChannelState::Live && !IsValid(GetCurrentRenderTarget(/*bInFallbackToPlaceholder*/false)))
	{
		DrawPlaceholderWidget();
	}
}

void FAvaBroadcastOutputChannel::StartPlaceholderTick()
{
	if (!PlaceholderTickHandle.IsValid() && FSlateApplication::IsInitialized())
	{
		PlaceholderTickHandle = FSlateApplication::Get().OnPreTick().AddRaw(this, &FAvaBroadcastOutputChannel::TickPlaceholder);
	}
}

void FAvaBroadcastOutputChannel::StopPlaceholderTick()
{
	if (PlaceholderTickHandle.IsValid() && FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().OnPreTick().Remove(PlaceholderTickHandle);
	}
	PlaceholderTickHandle.Reset();
	ReleasePlaceholderResources();
}

void FAvaBroadcastOutputChannel::OnAvaMediaCaptureStateChanged(const UMediaCapture* InMediaCapture, const UMediaOutput* InMediaOutput)
{
	RefreshState();

	// Note: the overall channel state may not change.
	// So we need to broadcast a specific event for the
	// media output state change.
	OnMediaOutputStateChanged.Broadcast(*this, InMediaOutput);
}

void FAvaBroadcastOutputChannel::UpdateChannelResources(bool bInIsProfileActive)
{
	ChannelNameText = FText::FromName(GetChannelName());
	
	if (bInIsProfileActive)
	{
		// Only update the placeholder resources (including render targets) if currently active profile.
		UpdatePlaceholder();
	}
	else
	{
		// Release all resources.
		ReleasePlaceholderResources();
		ReleasePlaceholderRenderTargets();
	}
}

FIntPoint FAvaBroadcastOutputChannel::DetermineRenderTargetSize() const
{
	// Just go through the list of media output in order and returns the first
	// one that specified a valid resolution.
	for (UMediaOutput* const MediaOutput : MediaOutputs)
	{
		if (IsValid(MediaOutput))
		{
			const FIntPoint TargetSize = MediaOutput->GetRequestedSize();
			if (TargetSize != UMediaOutput::RequestCaptureSourceSize && TargetSize.X > 0 && TargetSize.Y > 0)
			{
				return TargetSize;
			}
		}
	}
	return GetDefaultMediaOutputSize(GetChannelType());
}

EPixelFormat FAvaBroadcastOutputChannel::DetermineRenderTargetFormat() const
{
	// Determine render target format with the most bits per pixel.
	// Note: check for alpha bits (A2 vs A8).
	EPixelFormat HighestBitsPerColorChannelFormat = PF_Unknown;
	int32 HighestNumBitsPerColorChannel = 0;
	int32 HighestNumColorComponents = 0;
	EPixelFormat HighestAlphaBitsFormat = PF_Unknown;
	int32 HighestNumAlphaBits = 0;
	
	for (const UMediaOutput* const MediaOutput : MediaOutputs)
	{
		if (IsValid(MediaOutput))
		{
			const EPixelFormat TargetFormat = MediaOutput->GetRequestedPixelFormat();			
			if (TargetFormat != PF_Unknown)
			{
				const int32 NumBitsPerColorChannel = UE::AvaMediaRenderTargetUtils::GetNumColorChannelBits(TargetFormat);
				const int32 NumColorComponents = UE::AvaMediaRenderTargetUtils::GetNumColorComponents(TargetFormat);
				// Note: only select the format if it has at least as many color components.
				// Otherwise, we stick with the format that has less bits, but more components.
				if (NumBitsPerColorChannel > HighestNumBitsPerColorChannel && NumColorComponents >= HighestNumColorComponents)
				{
					HighestNumBitsPerColorChannel = NumBitsPerColorChannel;
					HighestNumColorComponents = NumColorComponents;
					HighestBitsPerColorChannelFormat = TargetFormat;
				}
				
				const int32 NumAlphaBits = UE::AvaMediaRenderTargetUtils::GetNumAlphaChannelBits(TargetFormat);
				if (NumAlphaBits > HighestNumAlphaBits)
				{
					HighestNumAlphaBits = NumAlphaBits;
					HighestAlphaBitsFormat = TargetFormat;
				}
			}
		}
	}

	// Note: this logic favors the number of color bits to the detriment of alpha bits.
	// In a RGBA8 vs RGB10A2 case, if we want 8 bits alpha, to the detriment of color bits,
	// the logic below will need to be changed.
	
	if (HighestNumAlphaBits > HighestNumBitsPerColorChannel && HighestAlphaBitsFormat != PF_Unknown)
	{
		return HighestAlphaBitsFormat;
	}
	
	if (HighestBitsPerColorChannelFormat != PF_Unknown)
	{
		return HighestBitsPerColorChannelFormat;
	}
	
	if (HighestAlphaBitsFormat != PF_Unknown)
	{
		return HighestAlphaBitsFormat;
	}
	
	return GetDefaultMediaOutputFormat();
}

FIntPoint FAvaBroadcastOutputChannel::GetDefaultMediaOutputSize(EAvaBroadcastChannelType InChannelType)
{
	// Special case for local preview channel.
	if (InChannelType == EAvaBroadcastChannelType::Preview)
	{
		return UAvaMediaSettings::Get().PreviewDefaultResolution;
	}

	// Default for program channel.
	return IAvaMediaModule::Get().GetBroadcastSettings().GetDefaultResolution();
}

EPixelFormat FAvaBroadcastOutputChannel::GetDefaultMediaOutputFormat()
{
	return IAvaMediaModule::Get().GetBroadcastSettings().GetDefaultPixelFormat();
}

void FAvaBroadcastOutputChannel::SetViewportQualitySettings(const FAvaViewportQualitySettings& InQualitySettings)
{
	QualitySettings = InQualitySettings;
	OnChannelChanged.Broadcast(*this, EAvaBroadcastChannelChange::Settings);
}

FString& FAvaBroadcastOutputChannel::EmplaceMessageForOutput(const UMediaOutput* InMediaOutput, EAvaBroadcastIssueSeverity InSeverity) const
{
	FLocalMediaOutputStatus& LocalMediaOutputStatus = LocalMediaOutputStatuses.Emplace(InMediaOutput);
	LocalMediaOutputStatus.Severity = InSeverity;
	return LocalMediaOutputStatus.Messages.Emplace_GetRef();
}

EAvaBroadcastIssueSeverity FAvaBroadcastOutputChannel::GetLocalSeverityForOutput(const UMediaOutput* InMediaOutput) const
{
	const FLocalMediaOutputStatus* LocalMediaOutputStatus = LocalMediaOutputStatuses.Find(InMediaOutput);
	return LocalMediaOutputStatus ? LocalMediaOutputStatus->Severity : EAvaBroadcastIssueSeverity::None;
}

void FAvaBroadcastOutputChannel::ResetLocalMediaOutputStatus(const UMediaOutput* InMediaOutput)
{
	if (IsValid(InMediaOutput))
	{
		LocalMediaOutputStatuses.Add(InMediaOutput,FLocalMediaOutputStatus());
	}
}