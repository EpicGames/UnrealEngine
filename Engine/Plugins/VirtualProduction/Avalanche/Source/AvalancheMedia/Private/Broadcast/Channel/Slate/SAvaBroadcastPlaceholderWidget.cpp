// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/Channel/Slate/SAvaBroadcastPlaceholderWidget.h"
#include "AvaMediaStyle.h"
#include "Internationalization/Text.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaBroadcastPlaceholderWidget"

void SAvaBroadcastPlaceholderWidget::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(0.1f)
		[
			SNew(SBox)
		]
		
		+ SVerticalBox::Slot()
		.FillHeight(0.2f)
		.VAlign(EVerticalAlignment::VAlign_Fill)
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		[
			SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFit)
			[
				SAssignNew(ChannelText, STextBlock)
				.Justification(ETextJustify::Center)
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(0.5f)
		.VAlign(EVerticalAlignment::VAlign_Fill)
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		.Padding(0.f, 25.f)
		[
			SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFit)
			[
				SNew(SImage)
				.Image(FAvaMediaStyle::Get().GetBrush(TEXT("AvaMedia.UnrealIcon")))
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight(0.1f)
		.VAlign(EVerticalAlignment::VAlign_Fill)
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		[
			SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFit)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(0.2f)
				.Padding(0.f, 5.f)
				[
					SNew(SScaleBox)
					.Stretch(EStretch::ScaleToFit)
					[
						SNew(SCircularThrobber)
						.Period(1.f)
						.NumPieces(3)
					]
				]
				+ SHorizontalBox::Slot()
				.FillWidth(0.8f)
				.Padding(5.f, 0.f)
				[
					SNew(SScaleBox)
					.Stretch(EStretch::ScaleToFit)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("WaitingForPlayback_Text", "Waiting for Playback"))
					]
				]
			]
		]
		
		+ SVerticalBox::Slot()
		.FillHeight(0.1f)
		[
			SNew(SBox)
		]
	];
}

void SAvaBroadcastPlaceholderWidget::SetChannelName(const FText& InChannelName)
{
	if (ChannelText.IsValid())
	{
		ChannelText->SetText(InChannelName);
	}
}


#undef LOCTEXT_NAMESPACE
