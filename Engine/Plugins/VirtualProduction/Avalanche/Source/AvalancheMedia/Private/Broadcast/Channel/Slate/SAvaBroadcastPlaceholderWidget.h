// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FText;
class STextBlock;

class SAvaBroadcastPlaceholderWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaBroadcastPlaceholderWidget) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

	void SetChannelName(const FText& InChannelName);
	
protected:
	TSharedPtr<STextBlock> ChannelText;
};
