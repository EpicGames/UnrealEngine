// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/OutputDevices/AvaBroadcastDeviceProviderData.h"

#include "Backends/JsonStructDeserializerBackend.h"
#include "Backends/JsonStructSerializerBackend.h"
#include "Backends/XmlStructSerializerBackend.h"
#include "HAL/FileManager.h"
#include "IMediaIOCoreModule.h"
#include "Misc/Paths.h"
#include "StructDeserializer.h"
#include "StructSerializer.h"
#include "UObject/UObjectIterator.h"

void FAvaBroadcastDeviceProviderData::PopulateFrom(const FName& InName, const IMediaIOCoreDeviceProvider* InDeviceProvider)
{
	Name = InName;
	Connections = InDeviceProvider->GetConnections();
	Configurations = InDeviceProvider->GetConfigurations();
	Devices = InDeviceProvider->GetDevices();
	//Modes = InDeviceProvider->GetModes(); TODO
	InputConfigurations = InDeviceProvider->GetInputConfigurations();
	OutputConfigurations = InDeviceProvider->GetOutputConfigurations();
	TimecodeConfigurations = InDeviceProvider->GetTimecodeConfigurations();
	DefaultConfiguration = InDeviceProvider->GetDefaultConfiguration();
	DefaultMode = InDeviceProvider->GetDefaultMode();
	DefaultInputConfiguration = InDeviceProvider->GetDefaultInputConfiguration();
	DefaultOutputConfiguration = InDeviceProvider->GetDefaultOutputConfiguration();
	DefaultTimecodeConfiguration = InDeviceProvider->GetDefaultTimecodeConfiguration();
	
#if WITH_EDITOR
	bShowInputTransportInSelector = InDeviceProvider->ShowInputTransportInSelector();
	bShowOutputTransportInSelector = InDeviceProvider->ShowOutputTransportInSelector();
	bShowInputKeyInSelector = InDeviceProvider->ShowInputKeyInSelector();
	bShowOutputKeyInSelector = InDeviceProvider->ShowOutputKeyInSelector();
	bShowReferenceInSelector = InDeviceProvider->ShowReferenceInSelector();
#endif
}
namespace FDeviceProviderDataHelper
{
	inline void ApplyServerName(const FString& InServerName, FMediaIODevice& OutDevice)
	{
		FString NewDeviceName = FString::Printf(TEXT("%s: %s"), *InServerName, * OutDevice.DeviceName.ToString());
		OutDevice.DeviceName = *NewDeviceName;
	}
	inline void ApplyServerName(const FString& InServerName, FMediaIOConnection& OutConnection)
	{
		ApplyServerName(InServerName, OutConnection.Device);
	}
	inline void ApplyServerName(const FString& InServerName, FMediaIOConfiguration& OutConfig)
	{
		ApplyServerName(InServerName, OutConfig.MediaConnection);
	}
	inline void ApplyServerName(const FString& InServerName, FMediaIOInputConfiguration& OutConfig)
	{
		ApplyServerName(InServerName, OutConfig.MediaConfiguration);
	}
	inline void ApplyServerName(const FString& InServerName, FMediaIOOutputConfiguration& OutConfig)
	{
		ApplyServerName(InServerName, OutConfig.MediaConfiguration);
	}
	inline void ApplyServerName(const FString& InServerName, FMediaIOVideoTimecodeConfiguration& OutConfig)
	{
		ApplyServerName(InServerName, OutConfig.MediaConfiguration);
	}
}

void FAvaBroadcastDeviceProviderData::ApplyServerName(const FString& InServerName)
{
	for (FMediaIOConnection& Connection : Connections)
	{
		FDeviceProviderDataHelper::ApplyServerName(InServerName,Connection);
	}
	for (FMediaIOConfiguration& Config : Configurations)
	{
		FDeviceProviderDataHelper::ApplyServerName(InServerName,Config);
	}
	for (FMediaIODevice& Device : Devices)
	{
		FDeviceProviderDataHelper::ApplyServerName(InServerName,Device);
	}
	for (FMediaIOInputConfiguration& Config : InputConfigurations)
	{
		FDeviceProviderDataHelper::ApplyServerName(InServerName,Config);
	}
	for (FMediaIOOutputConfiguration& Config : OutputConfigurations)
	{
		FDeviceProviderDataHelper::ApplyServerName(InServerName,Config);
	}
	for (FMediaIOVideoTimecodeConfiguration& Config : TimecodeConfigurations)
	{
		FDeviceProviderDataHelper::ApplyServerName(InServerName,Config);
	}

	FDeviceProviderDataHelper::ApplyServerName(InServerName,DefaultConfiguration);
	FDeviceProviderDataHelper::ApplyServerName(InServerName,DefaultInputConfiguration);
	FDeviceProviderDataHelper::ApplyServerName(InServerName,DefaultOutputConfiguration);
	FDeviceProviderDataHelper::ApplyServerName(InServerName,DefaultTimecodeConfiguration);
}

namespace FDeviceProviderDataListHelper
{
	inline const TCHAR* GetJsonSaveFilepath()
	{
		static const FString SaveFilepath = FPaths::ProjectConfigDir() / TEXT("DeviceProviders.json");
		return *SaveFilepath;
	}
	inline const TCHAR* GetXmlSaveFilepath()
	{
		static const FString SaveFilepath = FPaths::ProjectConfigDir() / TEXT("DeviceProviders.xml");
		return *SaveFilepath;
	}
}


bool FAvaBroadcastDeviceProviderDataList::SaveToJson()
{
	TUniquePtr<FArchive> FileWriter(
		IFileManager::Get().CreateFileWriter(FDeviceProviderDataListHelper::GetJsonSaveFilepath()));
	FJsonStructSerializerBackend Backend(*FileWriter, EStructSerializerBackendFlags::Default);
	FStructSerializer::Serialize(*this, Backend);

	FileWriter->Close();
	return !FileWriter->IsError();
}

bool FAvaBroadcastDeviceProviderDataList::SaveToXml()
{
	TUniquePtr<FArchive> FileWriter(
		IFileManager::Get().CreateFileWriter(FDeviceProviderDataListHelper::GetXmlSaveFilepath()));
	FXmlStructSerializerBackend Backend(*FileWriter, EStructSerializerBackendFlags::Default);
	FStructSerializer::Serialize(*this, Backend);
	Backend.SaveDocument();

	FileWriter->Close();
	return !FileWriter->IsError();
}

bool FAvaBroadcastDeviceProviderDataList::LoadFromJson()
{
	const TUniquePtr<FArchive> FileReader = TUniquePtr<FArchive>(
		IFileManager::Get().CreateFileReader(FDeviceProviderDataListHelper::GetJsonSaveFilepath()));
	FJsonStructDeserializerBackend Backend(*FileReader);
	FStructDeserializer::Deserialize(*this, Backend);

	FileReader->Close();
	return !FileReader->IsError();
}

void FAvaBroadcastDeviceProviderDataList::Populate(const FString& InAssignedServerName)
{
	ServerName = InAssignedServerName;
	
	if (!IMediaIOCoreModule::IsAvailable())
	{
		UE_LOGF(LogTemp, Error, "IMediaIOCoreModule is not available, sad face.");
		return;
	}
	
	const TConstArrayView<IMediaIOCoreDeviceProvider*> LocalDeviceProviders = IMediaIOCoreModule::Get().GetDeviceProviders();
	for (IMediaIOCoreDeviceProvider* DeviceProvider : LocalDeviceProviders)
	{
		FAvaBroadcastDeviceProviderData DeviceProviderData;
		DeviceProviderData.PopulateFrom(DeviceProvider->GetFName(), DeviceProvider);
		DeviceProviders.Add(DeviceProviderData);
	}
}