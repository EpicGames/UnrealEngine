// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/OutputDevices/AvaBroadcastDeviceProviderProxy.h"

#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputUtils.h"
#include "IAvaMediaModule.h"
#include "IMediaIOCoreDeviceProvider.h"
#include "IMediaIOCoreModule.h"
#include "MediaOutput.h"
#include "UObject/UObjectIterator.h"

//-----------------------------------------------------------------------------
//----- FAvaBroadcastDeviceProviderProxy -----------------------------------------------
//-----------------------------------------------------------------------------

FAvaBroadcastDeviceProviderProxy::FAvaBroadcastDeviceProviderProxy(FAvaBroadcastDeviceProviderProxyManager* InManager, const FAvaBroadcastDeviceProviderData& InData)
: Manager(InManager)
, Data(InData)
{
	
}

TArray<FMediaIOConfiguration> FAvaBroadcastDeviceProviderProxy::GetConfigurations(bool bAllowInput, bool bAllowOutput) const
{
	TArray<FMediaIOConfiguration> Out;
	for (const FMediaIOConfiguration& Config : Data.Configurations)
	{
		if ((Config.bIsInput && bAllowInput) || (!Config.bIsInput && bAllowOutput))
		{
			Out.Add(Config);
		}
	}
	return Out;
}

TArray<FMediaIOMode> FAvaBroadcastDeviceProviderProxy::GetModes(const FMediaIODevice& InDevice, bool bInOutput) const
{
		return Data.Modes; // TODO	
}

//-----------------------------------------------------------------------------
//----- FAvaBroadcastDeviceProviderWrapper ---------------------------------------------
//-----------------------------------------------------------------------------

FAvaBroadcastDeviceProviderWrapper::FAvaBroadcastDeviceProviderWrapper(FName InName, FAvaBroadcastDeviceProviderProxyManager* InManager)
	: Manager(InManager)
	, Name(InName)
	, DummyProxy(InManager, FAvaBroadcastDeviceProviderData())
{ }

void FAvaBroadcastDeviceProviderWrapper::AddProvider(const FString& InServerName, IMediaIOCoreDeviceProvider* InProvider, bool bInIsLocal)
{
	if (bInIsLocal)
	{
		LocalProvider = InProvider;
	}

	for (FProviderInfo& Info : Providers)
	{
		if (Info.ServerName == InServerName)
		{
			Info.Provider = InProvider;
			return;
		}
	}
	
	Providers.Add(FProviderInfo{InServerName, InProvider, nullptr});
}

void FAvaBroadcastDeviceProviderWrapper::AddProvider(const FString& InServerName, TUniquePtr<FAvaBroadcastDeviceProviderProxy>&& InProviderProxy)
{
	for (FProviderInfo& Info : Providers)
	{
		if (Info.ServerName == InServerName)
		{
			Info.Proxy = MoveTemp(InProviderProxy);
			Info.Provider = Info.Proxy.Get();
			return;
		}
	}

	Providers.Add(FProviderInfo{InServerName, InProviderProxy.Get(), MoveTemp(InProviderProxy)});
}

void FAvaBroadcastDeviceProviderWrapper::RemoveProvider(IMediaIOCoreDeviceProvider* InProvider)
{
	for (int32 i = 0; i < Providers.Num(); ++i)
	{
		if (Providers[i].Provider == InProvider)
		{
			OnRemoveProvider(InProvider);
			Providers.RemoveAt(i);
			return;
		}
	}
}

void FAvaBroadcastDeviceProviderWrapper::RemoveProvider(const FString& InServerName)
{
	for (int32 i = 0; i < Providers.Num(); ++i)
	{
		if (Providers[i].ServerName == InServerName)
		{
			OnRemoveProvider(Providers[i].Provider);
			Providers.RemoveAt(i);
			return;
		}
	}
}

const IMediaIOCoreDeviceProvider* FAvaBroadcastDeviceProviderWrapper::GetProviderForDeviceName(const FName& InDeviceName) const
{
	const FString DeviceName = InDeviceName.ToString();
	for (const FProviderInfo& ProviderInfo : Providers)
	{
		// For device from another server, the device name starts with the "ServerName:"
		// See FAvaBroadcastDeviceProviderData::ApplyServerName.
		if (DeviceName.StartsWith(ProviderInfo.ServerName))
		{
			return (ProviderInfo.Proxy.IsValid()) ? ProviderInfo.Proxy.Get() : ProviderInfo.Provider; 
		}
	}
	return GetLocalProvider();
}

const IMediaIOCoreDeviceProvider* FAvaBroadcastDeviceProviderWrapper::GetProviderForServer(const FString& InServerName) const
{
	for (const FProviderInfo& ProviderInfo : Providers)
	{
		// For device from another server, the device name starts with the "ServerName:"
		// See FAvaBroadcastDeviceProviderData::ApplyServerName.
		if (ProviderInfo.ServerName == InServerName)
		{
			return (ProviderInfo.Proxy.IsValid()) ? ProviderInfo.Proxy.Get() : ProviderInfo.Provider; 
		}
	}
	return nullptr;
}

FString FAvaBroadcastDeviceProviderWrapper::GetServerNameForDevice(const FName& InDeviceName) const
{
	// Search which provider proxy has the given device.
	for (const FProviderInfo& ProviderInfo : Providers)
	{
		if (FAvaBroadcastDeviceProviderProxyManager::IsDevicePresent(ProviderInfo.Provider, InDeviceName))
		{
			return ProviderInfo.ServerName;
		}
	}
	return FString();
}

bool FAvaBroadcastDeviceProviderWrapper::ShouldRemove() const
{
	// If it is either empty or just wrapping the local provider.
	return (Providers.IsEmpty() || (Providers.Num() == 1 && Providers[0].Provider == LocalProvider));
}

TArray<FMediaIOConnection> FAvaBroadcastDeviceProviderWrapper::GetConnections() const
{
	if (Providers.Num() == 1)
	{
		return Providers[0].Provider->GetConnections();
	}

	TArray<FMediaIOConnection> Connections;
	for (const FProviderInfo& Info : Providers)
	{
		Connections.Append(Info.Provider->GetConnections());
	}
	return Connections;
}

TArray<FMediaIOConfiguration> FAvaBroadcastDeviceProviderWrapper::GetConfigurations() const
{
	if (Providers.Num() == 1)
	{
		return Providers[0].Provider->GetConfigurations();
	}

	TArray<FMediaIOConfiguration> Configurations;
	for (const FProviderInfo& Info : Providers)
	{
		Configurations.Append(Info.Provider->GetConfigurations());
	}
	return Configurations;
}

TArray<FMediaIOConfiguration> FAvaBroadcastDeviceProviderWrapper::GetConfigurations(bool bInAllowInput, bool bInAllowOutput) const
{
	if (Providers.Num() == 1)
	{
		return Providers[0].Provider->GetConfigurations(bInAllowInput, bInAllowOutput);
	}

	TArray<FMediaIOConfiguration> Configurations;
	for (const FProviderInfo& Info : Providers)
	{
		Configurations.Append(Info.Provider->GetConfigurations(bInAllowInput, bInAllowOutput));
	}
	return Configurations;
}

TArray<FMediaIODevice> FAvaBroadcastDeviceProviderWrapper::GetDevices() const
{
	if (Providers.Num() == 1)
	{
		return Providers[0].Provider->GetDevices();
	}

	TArray<FMediaIODevice> Devices;
	for (const FProviderInfo& Info : Providers)
	{
		Devices.Append(Info.Provider->GetDevices());
	}
	return Devices;
}

TArray<FMediaIOMode> FAvaBroadcastDeviceProviderWrapper::GetModes(const FMediaIODevice& InDevice, bool bInOutput) const
{
	// SMELL: not sure this is correct.
	// Can you accumulate the modes for a device for different providers? Don't think so.
	if (Providers.Num() == 1)
	{
		return Providers[0].Provider->GetModes(InDevice, bInOutput);
	}

	TArray<FMediaIOMode> Modes;
	for (const FProviderInfo& Info : Providers)
	{
		Modes.Append(Info.Provider->GetModes(InDevice, bInOutput));
	}
	return Modes;
}

TArray<FMediaIOInputConfiguration> FAvaBroadcastDeviceProviderWrapper::GetInputConfigurations() const
{
	if (Providers.Num() == 1)
	{
		return Providers[0].Provider->GetInputConfigurations();
	}

	TArray<FMediaIOInputConfiguration> InputConfigurations;
	for (const FProviderInfo& Info : Providers)
	{
		InputConfigurations.Append(Info.Provider->GetInputConfigurations());
	}
	return InputConfigurations;
}

TArray<FMediaIOOutputConfiguration> FAvaBroadcastDeviceProviderWrapper::GetOutputConfigurations() const
{
	if (Providers.Num() == 1)
	{
		return Providers[0].Provider->GetOutputConfigurations();
	}

	TArray<FMediaIOOutputConfiguration> OutputConfigurations;
	for (const FProviderInfo& Info : Providers)
	{
		OutputConfigurations.Append(Info.Provider->GetOutputConfigurations());
	}
	return OutputConfigurations;
}

TArray<FMediaIOVideoTimecodeConfiguration> FAvaBroadcastDeviceProviderWrapper::GetTimecodeConfigurations() const
{
	if (Providers.Num() == 1)
	{
		return Providers[0].Provider->GetTimecodeConfigurations();
	}

	TArray<FMediaIOVideoTimecodeConfiguration> TimecodeConfigurations;
	for (const FProviderInfo& Info : Providers)
	{
		TimecodeConfigurations.Append(Info.Provider->GetTimecodeConfigurations());
	}
	return TimecodeConfigurations;
}

//-----------------------------------------------------------------------------
//----- FAvaBroadcastDeviceProviderProxyManager ----------------------------------------
//-----------------------------------------------------------------------------
FString FAvaBroadcastDeviceProviderProxyManager::LocalServerName(TEXT("Local (This Process)"));

const FAvaBroadcastDeviceProviderWrapper* FAvaBroadcastDeviceProviderProxyManager::GetDeviceProviderWrapper(FName InDeviceProviderName) const
{
	const TUniquePtr<FAvaBroadcastDeviceProviderWrapper>* DeviceProviderWrapper = DeviceProviderWrappers.Find(InDeviceProviderName);
	return DeviceProviderWrapper ? DeviceProviderWrapper->Get() : nullptr;
}

TArray<const IMediaIOCoreDeviceProvider*> FAvaBroadcastDeviceProviderProxyManager::GetDeviceProvidersForServer(const FString& InServerName) const
{
	TArray<const IMediaIOCoreDeviceProvider*> Providers;
	for (const TPair<FName, TUniquePtr<FAvaBroadcastDeviceProviderWrapper>>& WrapperEntry : DeviceProviderWrappers)
	{
		if (const IMediaIOCoreDeviceProvider* const Provider = WrapperEntry.Value->GetProviderForServer(InServerName))
		{
			Providers.Add(Provider);
		}
	}
	return Providers;
}

FString FAvaBroadcastDeviceProviderProxyManager::FindServerNameForDevice(const FName& InDeviceProviderName, const FName& InDeviceName) const
{
	const FAvaBroadcastDeviceProviderWrapper* DeviceProviderWrapper = GetDeviceProviderWrapper(InDeviceProviderName);

	if (DeviceProviderWrapper)
	{
		// If the device is local, i.e. on the current server, this will return "Local (This Process)".
		return DeviceProviderWrapper->GetServerNameForDevice(InDeviceName);
	}
	else
	{
		// There may not be a wrapper if the device provider is local, or there is no render node
		// connected. In this case, we just search the local provider directly.
		const IMediaIOCoreDeviceProvider* LocalProvider = GetLocalDeviceProvider(InDeviceProviderName);
		if (LocalProvider && IsDevicePresent(LocalProvider, InDeviceName))
		{
			return LocalServerName;
		}
	}

	// Return an empty string if the device couldn't be found anywhere.
	return FString();
}

bool FAvaBroadcastDeviceProviderProxyManager::IsLocalDevice(const FName& InDeviceProviderName, const FName& InDeviceName) const
{
	const FAvaBroadcastDeviceProviderWrapper* DeviceProviderWrapper = GetDeviceProviderWrapper(InDeviceProviderName);

	if (DeviceProviderWrapper && DeviceProviderWrapper->HasLocalProvider())
	{
		return FAvaBroadcastDeviceProviderProxyManager::IsDevicePresent(DeviceProviderWrapper->GetLocalProvider(), InDeviceName);
	}
	else if (const IMediaIOCoreDeviceProvider* LocalProvider = GetLocalDeviceProvider(InDeviceProviderName))
	{
		return FAvaBroadcastDeviceProviderProxyManager::IsDevicePresent(LocalProvider, InDeviceName);
	}
	return false;
}

void FAvaBroadcastDeviceProviderProxyManager::Install(const FString& InServerName, const FAvaBroadcastDeviceProviderDataList& InDeviceProviderDataList)
{
	if (!IMediaIOCoreModule::IsAvailable())
	{
		return;
	}

	DeviceProviderDataListByServer.Add(InServerName, MakeShared<FAvaBroadcastDeviceProviderDataList>(InDeviceProviderDataList));
	
	for (const FAvaBroadcastDeviceProviderData& ProviderData : InDeviceProviderDataList.DeviceProviders)
	{
		FAvaBroadcastDeviceProviderWrapper* Wrapper = nullptr;
		if (DeviceProviderWrappers.Contains(ProviderData.Name))
		{
			Wrapper = (*DeviceProviderWrappers.Find(ProviderData.Name)).Get();
		}
		else
		{
			// --- Need to make a new wrapper ---
			TUniquePtr<FAvaBroadcastDeviceProviderWrapper> DeviceProviderWrapper = MakeUnique<FAvaBroadcastDeviceProviderWrapper>(ProviderData.Name, this);

			// The local provider, if any, needs to be kept in the wrapper.
			IMediaIOCoreDeviceProvider* LocalProvider = IMediaIOCoreModule::Get().GetDeviceProvider(ProviderData.Name);
			if (LocalProvider)
			{
				DeviceProviderWrapper->AddProvider(LocalServerName, LocalProvider, true);
			}

			IMediaIOCoreModule::Get().RegisterDeviceProvider(DeviceProviderWrapper.Get());

			Wrapper = DeviceProviderWrapper.Get(); 
			
			DeviceProviderWrappers.Add(ProviderData.Name, MoveTemp(DeviceProviderWrapper));
		}

		if (Wrapper)
		{
			// Install a proxy for this server's devices in the wrapper.
			TUniquePtr<FAvaBroadcastDeviceProviderProxy> DeviceProviderProxy = MakeUnique<FAvaBroadcastDeviceProviderProxy>(this, ProviderData);
			DeviceProviderProxy->ApplyServerName(InServerName);
			Wrapper->AddProvider(InServerName, MoveTemp(DeviceProviderProxy));
		}
	}
	
	// Ensure all the wrapped local providers are at the end of IMediaIOCoreModule's array.
	ReorderWrappedLocalProviders();

	UAvaBroadcast::Get().QueueNotifyChange(EAvaBroadcastChange::OutputDevices);
}

void FAvaBroadcastDeviceProviderProxyManager::Uninstall(const FString& InServerName)
{
	if (!IMediaIOCoreModule::IsAvailable())
	{
		return;
	}

	TArray<FName> EntriesToRemove;
	
	// Remove the given server from all the wrappers.
	for (TPair<FName, TUniquePtr<FAvaBroadcastDeviceProviderWrapper>>& Entry : DeviceProviderWrappers )
	{
		Entry.Value->RemoveProvider(InServerName);
		if (Entry.Value->ShouldRemove())
		{
			IMediaIOCoreModule::Get().UnregisterDeviceProvider(Entry.Value.Get());
			EntriesToRemove.Add(Entry.Key);
		}
	}

	for (const FName& Key : EntriesToRemove)
	{
		DeviceProviderWrappers.Remove(Key);
	}

	DeviceProviderDataListByServer.Remove(InServerName);
	
	UAvaBroadcast::Get().QueueNotifyChange(EAvaBroadcastChange::OutputDevices);
}

void FAvaBroadcastDeviceProviderProxyManager::ListAllProviders()
{
	if (!IMediaIOCoreModule::IsAvailable())
	{
		UE_LOGF(LogAvaMedia, Error, "IMediaIOCoreModule is not available.");
		return;
	}

	// List all device providers.
	const TConstArrayView<IMediaIOCoreDeviceProvider*> DeviceProviders = IMediaIOCoreModule::Get().GetDeviceProviders();
	for (IMediaIOCoreDeviceProvider* DeviceProvider : DeviceProviders)
	{
		UE_LOGF(LogAvaMedia, Log, "Found DeviceProvider: \"%ls\".", *DeviceProvider->GetFName().ToString());
		// It seems the only way to get the output devices is to query all the output configurations and filter out the unique devices.
		const TArray<FMediaIOConfiguration> OutputConfigs = DeviceProvider->GetConfigurations(false, true);

		if (OutputConfigs.Num() > 0)
		{
			TSet<FString> OutputDeviceNames;
			for (const FMediaIOConfiguration& Config : OutputConfigs)
			{
				//UE_LOGF(LogAvaMedia, Log, "Found Output Config: \"%ls\".", *DeviceProvider->ToText(Config).ToString());
				OutputDeviceNames.Add(Config.MediaConnection.Device.DeviceName.ToString());
			}

			for (const FString& OutputDeviceName : OutputDeviceNames)
			{
				UE_LOGF(LogAvaMedia, Log, "Found Output Device: \"%ls\".", *OutputDeviceName);
			}
		}
		else
		{
			UE_LOGF(LogAvaMedia, Log, "No Output Devices Available for \"%ls\".", *DeviceProvider->GetFName().ToString());
		}	
	}

#if WITH_EDITORONLY_DATA
	for (const UClass* Class : TObjectRange<UClass>())
	{
		if (Class->IsChildOf(UMediaOutput::StaticClass()))
		{
			// Check that it has the ProviderName meta
			if (FName DeviceProviderName = UE::AvaBroadcastOutputUtils::GetDeviceProviderName(Class); !DeviceProviderName.IsNone())
			{
				UE_LOGF(LogAvaMedia, Log, "Found DeviceProviderName: \"%ls\" for class \"%ls\".", *DeviceProviderName.ToString(), *Class->GetName());

				if (IMediaIOCoreModule::Get().GetDeviceProvider(DeviceProviderName) == nullptr)
				{
					// This is possible, NDI has the meta data field, but it doesn't have a provider.
					UE_LOGF(LogAvaMedia, Log, "DeviceProvider \"%ls\" not found in IMediaIOCoreModule.", *DeviceProviderName.ToString());
				}
			}
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void FAvaBroadcastDeviceProviderProxyManager::TestInstall()
{
	FAvaBroadcastDeviceProviderDataList OutProviders;
	if (OutProviders.LoadFromJson())
	{
		Install(TEXT("RemoteTest"), OutProviders);
	}
}

void FAvaBroadcastDeviceProviderProxyManager::TestUninstall()
{
	Uninstall(TEXT("RemoteTest"));
}

bool FAvaBroadcastDeviceProviderProxyManager::IsDevicePresent(const IMediaIOCoreDeviceProvider* InProvider, const FName& InDeviceName)
{
	TArray<FMediaIODevice> Devices = InProvider->GetDevices();
	for (const FMediaIODevice& Device : Devices)
	{
		if (Device.DeviceName == InDeviceName)
		{
			return true;
		}
	}
	return false;
}

void FAvaBroadcastDeviceProviderProxyManager::ReorderWrappedLocalProviders()
{
	if (IMediaIOCoreModule::IsAvailable())
	{
		IMediaIOCoreModule& MediaIOCoreModule = IMediaIOCoreModule::Get();
		
		// Remote all local providers
		for (TPair<FName, TUniquePtr<FAvaBroadcastDeviceProviderWrapper>>& WrapperEntry : DeviceProviderWrappers)
		{
			if (WrapperEntry.Value->HasLocalProvider())
			{
				MediaIOCoreModule.UnregisterDeviceProvider(WrapperEntry.Value->GetLocalProvider());
			}
		}

		// Add them at the back of the array so they are ignored, but not lost.
		// Note: we keep them in there so we can detect if their respective module gets unloaded.
		// To test: "Module Unload BlackmagicMedia", "Module Load BlackmagicMedia".
		for (TPair<FName, TUniquePtr<FAvaBroadcastDeviceProviderWrapper>>& WrapperEntry : DeviceProviderWrappers)
		{
			if (WrapperEntry.Value->HasLocalProvider())
			{
				MediaIOCoreModule.RegisterDeviceProvider(WrapperEntry.Value->GetLocalProvider());
			}
		}
	}
}

const IMediaIOCoreDeviceProvider* FAvaBroadcastDeviceProviderProxyManager::GetLocalDeviceProvider(const FName& InDeviceProviderName)
{
	if (IMediaIOCoreModule::IsAvailable())
	{
		return IMediaIOCoreModule::Get().GetDeviceProvider(InDeviceProviderName);
	}
	return nullptr;
}

TSet<FString> FAvaBroadcastDeviceProviderProxyManager::GetServerNames() const
{
	TSet<FString> ServerNames;
	DeviceProviderDataListByServer.GetKeys(ServerNames);
	return (ServerNames);
}

const TSharedPtr<const FAvaBroadcastDeviceProviderDataList> FAvaBroadcastDeviceProviderProxyManager::GetDeviceProviderDataListForServer(const FString& InServerName) const
{
	const TSharedPtr<FAvaBroadcastDeviceProviderDataList>* FoundDeviceProviderDataList = DeviceProviderDataListByServer.Find(InServerName);
	return FoundDeviceProviderDataList ? *FoundDeviceProviderDataList : nullptr;
}
