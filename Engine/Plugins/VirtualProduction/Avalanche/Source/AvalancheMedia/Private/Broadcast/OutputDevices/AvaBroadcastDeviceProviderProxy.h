// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaIOCoreDeviceProvider.h"
#include "Broadcast/OutputDevices/AvaBroadcastDeviceProviderData.h"
#include "Broadcast/OutputDevices/IAvaBroadcastDeviceProviderProxyManager.h"

class FAvaBroadcastDeviceProviderProxyManager;

/**
 * Implements a proxy for the device provider of another system.
 */
class FAvaBroadcastDeviceProviderProxy : public IMediaIOCoreDeviceProvider
{
public:
	FAvaBroadcastDeviceProviderProxy(FAvaBroadcastDeviceProviderProxyManager* InManager, const FAvaBroadcastDeviceProviderData& InData);
	virtual ~FAvaBroadcastDeviceProviderProxy() = default;

	// This will add the server name to all device names.
	void ApplyServerName(const FString& InServerName) { Data.ApplyServerName(InServerName);}

	//IMediaIOCoreDeviceProvider
	virtual FName GetFName() override { return Data.Name; }

	virtual TArray<FMediaIOConnection> GetConnections() const override { return Data.Connections; }
	virtual TArray<FMediaIOConfiguration> GetConfigurations() const override { return Data.Configurations; }
	virtual TArray<FMediaIOConfiguration> GetConfigurations(bool bAllowInput, bool bAllowOutput) const override;
	virtual TArray<FMediaIODevice> GetDevices() const override { return Data.Devices; }
	virtual TArray<FMediaIOMode> GetModes(const FMediaIODevice& InDevice, bool bInOutput) const override;
	virtual TArray<FMediaIOInputConfiguration> GetInputConfigurations() const override { return Data.InputConfigurations; }
	virtual TArray<FMediaIOOutputConfiguration> GetOutputConfigurations() const override { return Data.OutputConfigurations; }
	virtual TArray<FMediaIOVideoTimecodeConfiguration> GetTimecodeConfigurations() const override { return Data.TimecodeConfigurations;}
	
	virtual FMediaIOConfiguration GetDefaultConfiguration() const override { return Data.DefaultConfiguration; }
	virtual FMediaIOMode GetDefaultMode() const override { return Data.DefaultMode; }
	virtual FMediaIOInputConfiguration GetDefaultInputConfiguration() const override { return Data.DefaultInputConfiguration; }
	virtual FMediaIOOutputConfiguration GetDefaultOutputConfiguration() const override { return Data.DefaultOutputConfiguration; }
	virtual FMediaIOVideoTimecodeConfiguration GetDefaultTimecodeConfiguration() const override { return Data.DefaultTimecodeConfiguration;}

#if WITH_EDITOR
	virtual bool ShowInputTransportInSelector() const override { return Data.bShowInputTransportInSelector; }
	virtual bool ShowOutputTransportInSelector() const override { return Data.bShowOutputTransportInSelector; }
	virtual bool ShowInputKeyInSelector() const override { return Data.bShowInputKeyInSelector; }
	virtual bool ShowOutputKeyInSelector() const override { return Data.bShowOutputKeyInSelector; }
	virtual bool ShowReferenceInSelector() const override { return Data.bShowReferenceInSelector; }
#endif
	//~IMediaIOCoreDeviceProvider
	
private:
	FAvaBroadcastDeviceProviderProxyManager* Manager;
	FAvaBroadcastDeviceProviderData Data;
};

/**
 *	Implements a wrapper of multiple other providers of the same device type from different systems.
 */
class FAvaBroadcastDeviceProviderWrapper : public IMediaIOCoreDeviceProvider
{
public:
	/**
	 *	@param InName Device Provider Name (ex: Blackmagic, AJA, etc).
	 *	@param InManager Parent Manager.
	 */
	FAvaBroadcastDeviceProviderWrapper(FName InName, FAvaBroadcastDeviceProviderProxyManager* InManager);
	virtual ~FAvaBroadcastDeviceProviderWrapper() = default;

	/** Add a provider for the given server name. */
	void AddProvider(const FString& InServerName, IMediaIOCoreDeviceProvider* InProvider, bool bInIsLocal);
	void AddProvider(const FString& InServerName, TUniquePtr<FAvaBroadcastDeviceProviderProxy>&& InProvider);
	
	/** Removes the provider from the list. */
	void RemoveProvider(IMediaIOCoreDeviceProvider* InProvider);
	void RemoveProvider(const FString& InServerName);

	bool HasLocalProvider() const { return LocalProvider != nullptr; }

	IMediaIOCoreDeviceProvider* GetLocalProvider() const { return LocalProvider; }
	const IMediaIOCoreDeviceProvider* GetProviderForDeviceName(const FName& InDeviceName) const;
	const IMediaIOCoreDeviceProvider* GetProviderForServer(const FString& InServerName) const;
	
	/** Search in the list of all wrapped proxies to find the device and returns the corresponding
	 * server name. If the provider is local, the server name is FAvaBroadcastDeviceProviderProxyManager::LocalServerName.
	 */
	FString GetServerNameForDevice(const FName& InDeviceName) const;
	
	/** Check if the wrapper is still wrapping something. */
	bool ShouldRemove() const;

	//IMediaIOCoreDeviceProvider
	virtual FName GetFName() override { return Name; }

	virtual TArray<FMediaIOConnection> GetConnections() const override;
	virtual TArray<FMediaIOConfiguration> GetConfigurations() const override;
	virtual TArray<FMediaIOConfiguration> GetConfigurations(bool bInAllowInput, bool bInAllowOutput) const override;
	virtual TArray<FMediaIODevice> GetDevices() const override;
	virtual TArray<FMediaIOMode> GetModes(const FMediaIODevice& InDevice, bool bInOutput) const override;
	virtual TArray<FMediaIOInputConfiguration> GetInputConfigurations() const override;
	virtual TArray<FMediaIOOutputConfiguration> GetOutputConfigurations() const override;
	virtual TArray<FMediaIOVideoTimecodeConfiguration> GetTimecodeConfigurations() const override;


	virtual FMediaIOConfiguration GetDefaultConfiguration() const override { return GetDefault()->GetDefaultConfiguration(); }
	virtual FMediaIOMode GetDefaultMode() const override { return GetDefault()->GetDefaultMode(); }
	virtual FMediaIOInputConfiguration GetDefaultInputConfiguration() const override { return GetDefault()->GetDefaultInputConfiguration(); }
	virtual FMediaIOOutputConfiguration GetDefaultOutputConfiguration() const override { return GetDefault()->GetDefaultOutputConfiguration(); }
	virtual FMediaIOVideoTimecodeConfiguration GetDefaultTimecodeConfiguration() const override { return GetDefault()->GetDefaultTimecodeConfiguration(); }

#if WITH_EDITOR
	virtual bool ShowInputTransportInSelector() const override { return GetDefault()->ShowInputTransportInSelector(); }
	virtual bool ShowOutputTransportInSelector() const override { return GetDefault()->ShowOutputTransportInSelector(); }
	virtual bool ShowInputKeyInSelector() const override { return GetDefault()->ShowInputKeyInSelector(); }
	virtual bool ShowOutputKeyInSelector() const override { return GetDefault()->ShowOutputKeyInSelector(); }
	virtual bool ShowReferenceInSelector() const override { return GetDefault()->ShowReferenceInSelector(); }
#endif
	//~IMediaIOCoreDeviceProvider

private:
	const IMediaIOCoreDeviceProvider* GetDefault() const
	{
		return !Providers.IsEmpty() ? Providers[0].Provider : &DummyProxy;
	}

	void OnRemoveProvider(IMediaIOCoreDeviceProvider* InProviderToRemove)
	{
		if (InProviderToRemove == LocalProvider)
		{
			LocalProvider = nullptr;
		}
	}
	
private:
	FAvaBroadcastDeviceProviderProxyManager* Manager = nullptr;

	FName Name;

	// We need to keep track of the local provider because it is a special snowflake.
	IMediaIOCoreDeviceProvider* LocalProvider = nullptr;
	
	FAvaBroadcastDeviceProviderProxy DummyProxy;
	
	struct FProviderInfo
	{
		FString ServerName;
		IMediaIOCoreDeviceProvider* Provider;
		TUniquePtr<FAvaBroadcastDeviceProviderProxy> Proxy;
	};
	TArray<FProviderInfo> Providers;
};

class FAvaBroadcastDeviceProviderProxyManager : public IAvaBroadcastDeviceProviderProxyManager
{
public:
	virtual ~FAvaBroadcastDeviceProviderProxyManager() {}

	//~ Begin IAvaBroadcastDeviceProviderProxyManager
	const FAvaBroadcastDeviceProviderWrapper* GetDeviceProviderWrapper(FName InDeviceProviderName) const override;
	TArray<const IMediaIOCoreDeviceProvider*> GetDeviceProvidersForServer(const FString& InServerName) const override;
	FString FindServerNameForDevice(const FName& InDeviceProviderName, const FName& InDeviceName) const override;
	bool IsLocalDevice(const FName& InDeviceProviderName, const FName& InDeviceName) const override;

	virtual void Install(const FString& InServerName, const FAvaBroadcastDeviceProviderDataList& InDeviceProviderDataList) override;
	virtual void Uninstall(const FString& InServerName) override;

	virtual TSet<FString> GetServerNames() const override;
	virtual const TSharedPtr<const FAvaBroadcastDeviceProviderDataList> GetDeviceProviderDataListForServer(const FString& InServerName) const override;
	virtual FString GetLocalServerName() const override { return LocalServerName; }
	//~ End IAvaBroadcastDeviceProviderProxyManager

	/** Lists all the devices from all device providers from all the media output classes. */
	static void ListAllProviders();

	void TestInstall();
	void TestUninstall();

	/** Returns true if the given device name can be found in the device provider. */
	static bool IsDevicePresent(const IMediaIOCoreDeviceProvider* InProvider, const FName& InDeviceName);

	static FString LocalServerName;
	
private:
	/** Ensure all the wrapped local providers are at the end of IMediaIOCoreModule's array. */
	void ReorderWrappedLocalProviders();

	/** Returns the given local provider. */
	static const IMediaIOCoreDeviceProvider* GetLocalDeviceProvider(const FName& InDeviceProviderName);

private:
	TMap<FName, TUniquePtr<FAvaBroadcastDeviceProviderWrapper>> DeviceProviderWrappers;

	/** Keep a copy of the device provider data list last received for each server. */
	TMap<FString, TSharedPtr<FAvaBroadcastDeviceProviderDataList>> DeviceProviderDataListByServer;
};
