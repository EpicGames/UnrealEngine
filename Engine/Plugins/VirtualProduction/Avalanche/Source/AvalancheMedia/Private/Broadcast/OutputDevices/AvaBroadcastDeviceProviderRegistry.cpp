// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaBroadcastDeviceProviderRegistry.h"

#include "Backends/JsonStructDeserializerBackend.h"
#include "Backends/JsonStructSerializerBackend.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputUtils.h"
#include "HAL/FileManager.h"
#include "MediaOutput.h"
#include "Misc/Paths.h"
#include "StructDeserializer.h"
#include "StructSerializer.h"
#include "UObject/UObjectIterator.h"

namespace UE::AvaMedia::BroadcastDeviceProviderRegistry::Private
{
	void InitializeData(FAvaBroadcastDeviceProviderRegistryData& OutData);
	bool SaveData(const FAvaBroadcastDeviceProviderRegistryData& OutData);
	bool LoadData(FAvaBroadcastDeviceProviderRegistryData& OutData);
}

const FAvaBroadcastDeviceProviderRegistry& FAvaBroadcastDeviceProviderRegistry::Get()
{
	static FAvaBroadcastDeviceProviderRegistry Instance;
	return Instance;
}

FAvaBroadcastDeviceProviderRegistry::FAvaBroadcastDeviceProviderRegistry()
{
	using namespace UE::AvaMedia::BroadcastDeviceProviderRegistry::Private;
#if WITH_EDITORONLY_DATA
	InitializeData(Data);
	SaveData(Data);
#else
	LoadData(Data);
#endif
}

bool FAvaBroadcastDeviceProviderRegistry::HasDeviceProviderName(const UClass* InMediaOutputClass) const
{
	if (InMediaOutputClass)
	{
		return Data.DeviceProviderNames.Contains(InMediaOutputClass->GetFName());
	}
	return false;
}

FName FAvaBroadcastDeviceProviderRegistry::GetDeviceProviderName(const UClass* InMediaOutputClass) const
{
	if (InMediaOutputClass)
	{
		if (const FName* FoundName = Data.DeviceProviderNames.Find(InMediaOutputClass->GetFName()))
		{
			return *FoundName;	
		}
	}
	return FName();
}

const FText& FAvaBroadcastDeviceProviderRegistry::GetOutputClassDisplayText(const UClass* InMediaOutputClass) const
{
	if (InMediaOutputClass)
	{
		if (const FText* FoundText = Data.OutputClassDisplayNames.Find(InMediaOutputClass->GetFName()))
		{
			return *FoundText;
		}
	}
	return FText::GetEmpty();
}


namespace UE::AvaMedia::BroadcastDeviceProviderRegistry::Private
{
	void InitializeData(FAvaBroadcastDeviceProviderRegistryData& OutData)
	{
		OutData.DeviceProviderNames.Reset();
		OutData.OutputClassDisplayNames.Reset();
		
#if WITH_EDITORONLY_DATA
		for (const UClass* const Class : TObjectRange<UClass>())
		{
			const bool bIsMediaOutputClass = Class->IsChildOf(UMediaOutput::StaticClass()) && Class != UMediaOutput::StaticClass();
			if (bIsMediaOutputClass)
			{
				OutData.OutputClassDisplayNames.Add(Class->GetFName(), Class->GetDisplayNameText());
				
				const FName DeviceProviderName = UE::AvaBroadcastOutputUtils::GetDeviceProviderName(Class);
				if (!DeviceProviderName.IsNone())
				{
					OutData.DeviceProviderNames.Add(Class->GetFName(), DeviceProviderName);
				}
			}
		}
#endif		
	}

	static FString GetRegistryFilepath()
	{
		const FString RegistryName = TEXT("BroadcastDeviceProviderRegistry.json");
		return FPaths::ProjectConfigDir() / RegistryName;
	}
	
	bool LoadData(FAvaBroadcastDeviceProviderRegistryData& OutData, FArchive& InArchive)
	{
		FJsonStructDeserializerBackend Backend(InArchive);
		return FStructDeserializer::Deserialize(&OutData, *FAvaBroadcastDeviceProviderRegistryData::StaticStruct(), Backend);
	}

	bool LoadData(FAvaBroadcastDeviceProviderRegistryData& OutData)
	{
		bool bLoaded = false;
		const TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*GetRegistryFilepath()));
		if (FileReader)
		{
			bLoaded =LoadData(OutData, *FileReader);
			FileReader->Close();
		}
		return bLoaded;
	}
	
	void SaveData(const FAvaBroadcastDeviceProviderRegistryData& OutData, FArchive& InArchive)
	{
		FJsonStructSerializerBackend Backend(InArchive, EStructSerializerBackendFlags::Default);
		FStructSerializer::Serialize(&OutData, *FAvaBroadcastDeviceProviderRegistryData::StaticStruct(), Backend);
	}

	bool SaveData(const FAvaBroadcastDeviceProviderRegistryData& OutData)
	{
		const TUniquePtr<FArchive> FileWriter(IFileManager::Get().CreateFileWriter(*GetRegistryFilepath()));
		if (FileWriter)
		{
			SaveData(OutData, *FileWriter);
			FileWriter->Close();
			return true;
		}
		return false;
	}
}
