// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"

#include "AvaBroadcastDeviceProviderRegistry.generated.h"

class UClass;

USTRUCT()
struct FAvaBroadcastDeviceProviderRegistryData
{
	GENERATED_BODY()

	/** Maps output class name to the device provider. */
	UPROPERTY()
	TMap<FName, FName> DeviceProviderNames;

	UPROPERTY()
	TMap<FName, FText> OutputClassDisplayNames;
};

class FAvaBroadcastDeviceProviderRegistry
{
public:
	static const FAvaBroadcastDeviceProviderRegistry& Get();

	bool HasDeviceProviderName(const UClass* InMediaOutputClass) const;
	FName GetDeviceProviderName(const UClass* InMediaOutputClass) const;
	const FText& GetOutputClassDisplayText(const UClass* InMediaOutputClass) const;

protected:
	FAvaBroadcastDeviceProviderRegistry();
	~FAvaBroadcastDeviceProviderRegistry() = default;

private:
	FAvaBroadcastDeviceProviderRegistryData Data;
};