// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/OutputDevices/AvaBroadcastDisplayDeviceManager.h"

#include "Templates/RefCounting.h"

DEFINE_LOG_CATEGORY_STATIC(LogAvaDisplayDeviceManager, Log, All);

#if PLATFORM_WINDOWS

THIRD_PARTY_INCLUDES_START
#include "Windows/AllowWindowsPlatformTypes.h"
#include "dxgi1_3.h"
#include "dxgi1_4.h"
#include "dxgi1_6.h"
#include "Windows/HideWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_END

namespace AvaDXGIHelper
{
	// I haven't found a way to get the actual pixel format of an output.
	// So we try to guess it.
	static DXGI_FORMAT GuessOutputPixelFormat(IDXGIOutput* InOutput)
	{
		DXGI_FORMAT OutputPixelFormat = DXGI_FORMAT_R8G8B8A8_UNORM;
		
		TRefCountPtr<IDXGIOutput6> Output6;
		if (SUCCEEDED(
			InOutput->QueryInterface(__uuidof(IDXGIOutput6), (void**)Output6.GetInitReference())))
		{
			DXGI_OUTPUT_DESC1 OutputDesc1;
			Output6->GetDesc1(&OutputDesc1);
			switch (OutputDesc1.BitsPerColor)
			{
			case 8: OutputPixelFormat = DXGI_FORMAT_R8G8B8A8_UNORM;
				break;
			case 10: OutputPixelFormat = DXGI_FORMAT_R10G10B10A2_UNORM;
				break;
			case 16: OutputPixelFormat = DXGI_FORMAT_R16G16B16A16_FLOAT;
				break;
			default: OutputPixelFormat = DXGI_FORMAT_R8G8B8A8_UNORM;
				break;
			}
		}
		return OutputPixelFormat;
	}
	
	static TArray<DXGI_MODE_DESC> GetOutputModesForFormat(IDXGIOutput* InOutput, DXGI_FORMAT InFormat)
	{
		TArray<DXGI_MODE_DESC> ModeDescriptors;
		UINT Num = 0;
		const UINT Flags = DXGI_ENUM_MODES_INTERLACED;
		InOutput->GetDisplayModeList(InFormat, Flags, &Num, 0);
		if (Num)
		{
			ModeDescriptors.SetNumUninitialized(Num);
			InOutput->GetDisplayModeList(InFormat, Flags, &Num, ModeDescriptors.GetData());
		}
		return ModeDescriptors;
	}

	inline FFrameRate ToFramerate(const DXGI_RATIONAL& InFrameRate)
	{
		return FFrameRate(InFrameRate.Numerator, InFrameRate.Denominator);
	}

	// We have an approximate frame rate in DevMode, but we can try to guess more
	// accurately by checking the modes returned by GetDisplayModeList.
	// Unfortunately, there is no way to get the current display mode directly, so again
	// we have to first guess the pixel format.
	static FFrameRate GuessFrameRate(IDXGIOutput* InOutput, const DEVMODE& InDevMode)
	{
		// Because this API returns an integer display frequency, it is not accurate.
		FFrameRate ApproxFrameRate(InDevMode.dmDisplayFrequency, 1);

		// Step 1 of the guessing game: guess the pixel format.
		const DXGI_FORMAT GuessedOutputPixelFormat = GuessOutputPixelFormat(InOutput);

		{
			DXGI_MODE_DESC ModeToMatch;
			ModeToMatch.Format = GuessedOutputPixelFormat;
			ModeToMatch.Height = InDevMode.dmPelsHeight;
			ModeToMatch.Width = InDevMode.dmPelsWidth;
			ModeToMatch.RefreshRate.Numerator = InDevMode.dmDisplayFrequency;
			ModeToMatch.RefreshRate.Denominator = 1;
			ModeToMatch.Scaling = DXGI_MODE_SCALING_UNSPECIFIED;
			DXGI_MODE_DESC ClosestMatch;
			if (SUCCEEDED(InOutput->FindClosestMatchingMode(&ModeToMatch, &ClosestMatch, nullptr)))
			{
				return ToFramerate(ClosestMatch.RefreshRate);
			}
		}
		
		TArray<DXGI_MODE_DESC> ModeDescriptors = GetOutputModesForFormat(InOutput, GuessedOutputPixelFormat);
		if (!ModeDescriptors.IsEmpty())
		{
			FFrameRate ClosestFrameRate(0,0);
			double ClosestDist = 0.0;

			for (DXGI_MODE_DESC& ModeDesc : ModeDescriptors)
			{
				if (ModeDesc.Width == InDevMode.dmPelsWidth && ModeDesc.Height == InDevMode.dmPelsHeight)
				{
					FFrameRate FrameRate = ToFramerate(ModeDesc.RefreshRate);
					double Dist = FGenericPlatformMath::Abs(ApproxFrameRate.AsDecimal() - FrameRate.AsDecimal());
					if (!ClosestFrameRate.IsValid() || Dist < ClosestDist)
					{
						ClosestFrameRate = FrameRate;
						ClosestDist = Dist;
					}
				}
			}

			if (ClosestFrameRate.IsValid())
			{
				ApproxFrameRate = ClosestFrameRate;
			}							
		}
		return ApproxFrameRate;
	}
	
	static void EnumMonitorsWindowsDXGI(TArray<FAvaBroadcastMonitorInfo>& OutMonitorInfo)
	{
		TRefCountPtr<IDXGIFactory1> DXGIFactory1;
		if (CreateDXGIFactory1(__uuidof(IDXGIFactory1), (void**)DXGIFactory1.GetInitReference()) != S_OK || !DXGIFactory1)
		{
			return;
		}

		OutMonitorInfo.Empty(2); // Reserve two slots, as that will be the most common maximum
	
		TRefCountPtr<IDXGIAdapter> TempAdapter;
		for (uint32 AdapterIndex = 0; DXGIFactory1->EnumAdapters(AdapterIndex, TempAdapter.GetInitReference()) != DXGI_ERROR_NOT_FOUND; ++AdapterIndex)
		{
			if (TempAdapter)
			{
				DXGI_ADAPTER_DESC AdapterDesc;
				TempAdapter->GetDesc(&AdapterDesc);

				FAvaBroadcastDisplayAdapterInfo AdapterInfo;
				AdapterInfo.Description = AdapterDesc.Description;
				AdapterInfo.VendorId = AdapterDesc.VendorId;
				AdapterInfo.DeviceId = AdapterDesc.DeviceId;
				AdapterInfo.SubSysId = AdapterDesc.SubSysId;
				AdapterInfo.Revision = AdapterDesc.Revision;
				AdapterInfo.DedicatedVideoMemory = AdapterDesc.DedicatedVideoMemory;
			
				TRefCountPtr<IDXGIOutput> TempOutput;
				for (uint32 OutputIndex =0; TempAdapter->EnumOutputs(OutputIndex, TempOutput.GetInitReference()) != DXGI_ERROR_NOT_FOUND; ++OutputIndex)
				{
					DXGI_OUTPUT_DESC OutputDesc;
					TempOutput->GetDesc(&OutputDesc);
				
					// Get Resolution and Frame rate
					DEVMODE DevMode;
					ZeroMemory(&DevMode, sizeof(DevMode));
					DevMode.dmSize = sizeof(DevMode);
					EnumDisplaySettings(OutputDesc.DeviceName, ENUM_CURRENT_SETTINGS, &DevMode);

					FFrameRate ApproxFrameRate = GuessFrameRate(TempOutput.GetReference(), DevMode);
					UE_LOGF(LogAvaDisplayDeviceManager, Verbose, "Mode: %d x %d @ %f Hz",
						DevMode.dmPelsWidth, DevMode.dmPelsHeight, ApproxFrameRate.AsDecimal());
					
					// Get Display and Working rect.
					MONITORINFOEX MonitorInfo;
					ZeroMemory(&MonitorInfo, sizeof(MonitorInfo));
					MonitorInfo.cbSize = sizeof(MonitorInfo);
					GetMonitorInfo(OutputDesc.Monitor, &MonitorInfo);
				
					// UE_LOGF(LogAvaDisplayDeviceManager, Log, "Monitor: %ls on %ls (%d x %d @ %d Hz)",
					// 	OutputDesc.DeviceName, AdapterDesc.Description,
					// 	DevMode.dmPelsWidth, DevMode.dmPelsHeight, DevMode.dmDisplayFrequency);

					FAvaBroadcastMonitorInfo AvaInfo;
					AvaInfo.Name = OutputDesc.DeviceName;
					AvaInfo.Name.RemoveFromStart(TEXT("\\\\.\\"));
					AvaInfo.AdapterInfo = AdapterInfo;
					AvaInfo.Width = DevMode.dmPelsWidth;
					AvaInfo.Height = DevMode.dmPelsHeight;
					AvaInfo.DisplayFrequency = ApproxFrameRate;
					AvaInfo.DisplayRect.Bottom = MonitorInfo.rcMonitor.bottom;
					AvaInfo.DisplayRect.Left = MonitorInfo.rcMonitor.left;
					AvaInfo.DisplayRect.Right = MonitorInfo.rcMonitor.right;
					AvaInfo.DisplayRect.Top = MonitorInfo.rcMonitor.top;
					AvaInfo.WorkArea.Bottom = MonitorInfo.rcWork.bottom;
					AvaInfo.WorkArea.Left = MonitorInfo.rcWork.left;
					AvaInfo.WorkArea.Right = MonitorInfo.rcWork.right;
					AvaInfo.WorkArea.Top = MonitorInfo.rcWork.top;
					AvaInfo.bIsPrimary = (MonitorInfo.dwFlags & MONITORINFOF_PRIMARY) > 0;
					OutMonitorInfo.Add(AvaInfo);
				}
			}
		}
	}
}
#endif

static void EnumMonitorsGeneric(TArray<FAvaBroadcastMonitorInfo>& OutMonitorInfo)
{
	FDisplayMetrics DisplayMetrics;

	FDisplayMetrics::RebuildDisplayMetrics(DisplayMetrics);

	OutMonitorInfo.Empty(DisplayMetrics.MonitorInfo.Num());
	
	for (const FMonitorInfo& Info : DisplayMetrics.MonitorInfo)
	{
		FAvaBroadcastMonitorInfo AvaInfo;
		AvaInfo.Name = Info.Name;
		//AvaInfo.AdapterDesc -- Don't have the adapter name.
		AvaInfo.Width = Info.DisplayRect.Right - Info.DisplayRect.Left;
		AvaInfo.Height = Info.DisplayRect.Bottom - Info.DisplayRect.Top;
		//AvaInfo.DisplayFrequency -- Don't have display frequency.
		AvaInfo.DisplayRect = Info.DisplayRect;
		AvaInfo.WorkArea = Info.WorkArea;
		AvaInfo.bIsPrimary = Info.bIsPrimary;
		OutMonitorInfo.Add(AvaInfo);
	}
}

void FAvaBroadcastDisplayDeviceManager::EnumMonitors(TArray<FAvaBroadcastMonitorInfo>& OutMonitorInfo)
{
#if PLATFORM_WINDOWS
	static bool bUseDXGIEnumeration = true;
	if (bUseDXGIEnumeration)
	{
		AvaDXGIHelper::EnumMonitorsWindowsDXGI(OutMonitorInfo);
		return;
	}
#endif
	EnumMonitorsGeneric(OutMonitorInfo);
}

TArray<FAvaBroadcastMonitorInfo> FAvaBroadcastDisplayDeviceManager::CachedMonitorInfo;

const TArray<FAvaBroadcastMonitorInfo>&  FAvaBroadcastDisplayDeviceManager::GetCachedMonitors(bool bInForceUpdate)
{
	if (bInForceUpdate || CachedMonitorInfo.Num() == 0)
	{
		EnumMonitors(CachedMonitorInfo);
	}
	return CachedMonitorInfo;
}

FString FAvaBroadcastDisplayDeviceManager::GetMonitorDisplayName(const FAvaBroadcastMonitorInfo& InMonitorInfo)
{
	FString DisplayName = FString::Format(TEXT("{0}: {1}x{2} @ {3},{4}"),
		{
			InMonitorInfo.Name,
			InMonitorInfo.Width,
			InMonitorInfo.Height,
			InMonitorInfo.DisplayRect.Left,
			InMonitorInfo.DisplayRect.Top
		});

	if (InMonitorInfo.bIsPrimary)
	{
		DisplayName += TEXT(" (Primary Monitor)");
	}
	return DisplayName;
}
