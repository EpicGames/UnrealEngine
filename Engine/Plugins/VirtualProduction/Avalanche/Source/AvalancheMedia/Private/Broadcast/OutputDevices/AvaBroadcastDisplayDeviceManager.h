// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericApplication.h"
#include "Misc/FrameRate.h"

/**
 * Contains information on physical display adapter,
 * a.k.a. graphics card.
 **/
struct FAvaBroadcastDisplayAdapterInfo
{
	FString Description;			// ex: "Nvidia GeForce RTX ..."
	uint32 VendorId = 0;
	uint32 DeviceId = 0;
	uint32 SubSysId = 0;
	uint32 Revision = 0;
	uint64 DedicatedVideoMemory = 0;
};

/**
 * Contains info on a physical monitor connected to the display device.
 * Similar to FMonitorInfo, except we have the adapter description
 * and display frequency.
 */
struct FAvaBroadcastMonitorInfo
{
	FAvaBroadcastDisplayAdapterInfo AdapterInfo;
	FString Name;					// ex: DISPLAY1, DISPLAY2, etc (not manufacturer name)
	int32 Width = 0;				// Horizontal resolution in pixels.
	int32 Height = 0;				// Vertical resolution in pixels.
	FFrameRate DisplayFrequency = FFrameRate(0, 0);	// Default invalid.
	FPlatformRect DisplayRect;
	FPlatformRect WorkArea;
	bool bIsPrimary = false;
};

/**
 * Provides additional information on the display devices that is not available
 * in FDisplayMetrics.
 */
class FAvaBroadcastDisplayDeviceManager
{
public:
	static void EnumMonitors(TArray<FAvaBroadcastMonitorInfo>& OutMonitorInfo);

	static const TArray<FAvaBroadcastMonitorInfo>&  GetCachedMonitors(bool bForceUpdate = false);

	/**
	 * Returns a string with a display name for the monitor.
	 **/
	static FString GetMonitorDisplayName(const FAvaBroadcastMonitorInfo& InMonitorInfo);
	
private:
	static TArray<FAvaBroadcastMonitorInfo> CachedMonitorInfo;
};
