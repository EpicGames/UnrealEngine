// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaBroadcastDisplayDeviceProvider.h"
#include "Broadcast/OutputDevices/AvaBroadcastDisplayDeviceManager.h"
#include "Internationalization/Text.h"
#include "MediaIOCoreCommonDisplayMode.h"
#include "MediaIOCoreDefinitions.h"

#define LOCTEXT_NAMESPACE "AvaBroadcastDisplayDeviceProvider"

FName FAvaBroadcastDisplayDeviceProvider::GetProviderName()
{
	static FName NAME_Provider = "AvaDisplay";
	return NAME_Provider;
}


FName FAvaBroadcastDisplayDeviceProvider::GetProtocolName()
{
	static FName NAME_Protocol = "avadisplay";
	return NAME_Protocol;
}

FName FAvaBroadcastDisplayDeviceProvider::GetFName()
{
	return GetProviderName();
}

TArray<FMediaIOConnection> FAvaBroadcastDisplayDeviceProvider::GetConnections() const
{
	return TArray<FMediaIOConnection>();
}

TArray<FMediaIOConfiguration> FAvaBroadcastDisplayDeviceProvider::GetConfigurations() const
{
	return GetConfigurations(true, true);
}

TArray<FMediaIOConfiguration> FAvaBroadcastDisplayDeviceProvider::GetConfigurations(bool /*bInAllowInput*/, bool bInAllowOutput) const
{
	TArray<FMediaIOConfiguration> Results;
	if (bInAllowOutput)
	{
		TArray<FAvaBroadcastMonitorInfo> MonitorInfos = FAvaBroadcastDisplayDeviceManager::GetCachedMonitors();

		for (int32 i = 0; i < MonitorInfos.Num(); ++i)
		{
			const FAvaBroadcastMonitorInfo& MonitorInfo = MonitorInfos[i];

			FMediaIOConfiguration MediaConfiguration = GetDefaultConfiguration();
			MediaConfiguration.bIsInput = false;
			MediaConfiguration.MediaMode.Resolution = FIntPoint(MonitorInfo.Width, MonitorInfo.Height);
			if (MonitorInfo.DisplayFrequency.IsValid())
			{
				MediaConfiguration.MediaMode.FrameRate = MonitorInfo.DisplayFrequency;
			}

			MediaConfiguration.MediaConnection.Device.DeviceName = *FAvaBroadcastDisplayDeviceManager::GetMonitorDisplayName(MonitorInfo);
			MediaConfiguration.MediaConnection.Device.DeviceIdentifier = i;
			MediaConfiguration.MediaConnection.Protocol = GetProtocolName();
			MediaConfiguration.MediaConnection.PortIdentifier = 0;

			Results.Add(MediaConfiguration);
		}
	}
	return Results;
}

TArray<FMediaIOInputConfiguration> FAvaBroadcastDisplayDeviceProvider::GetInputConfigurations() const
{
	return TArray<FMediaIOInputConfiguration>();
}

TArray<FMediaIOOutputConfiguration> FAvaBroadcastDisplayDeviceProvider::GetOutputConfigurations() const
{
	TArray<FMediaIOOutputConfiguration> Results;

	TArray<FMediaIOConfiguration> Configs = GetConfigurations(false, true);

	FMediaIOOutputConfiguration DefaultOutputConfiguration = GetDefaultOutputConfiguration();
	DefaultOutputConfiguration.KeyPortIdentifier = 0;
	DefaultOutputConfiguration.OutputType = EMediaIOOutputType::Fill;	// HDMI/Display port don't support alpha.
	DefaultOutputConfiguration.OutputReference = EMediaIOReferenceType::FreeRun;

	for (const FMediaIOConfiguration& Config : Configs)
	{
		DefaultOutputConfiguration.MediaConfiguration = Config;
		Results.Add(DefaultOutputConfiguration);
	}
	return Results;
}

TArray<FMediaIOVideoTimecodeConfiguration> FAvaBroadcastDisplayDeviceProvider::GetTimecodeConfigurations() const
{
	TArray<FMediaIOVideoTimecodeConfiguration> MediaConfigurations;
	return MediaConfigurations;
}

TArray<FMediaIODevice> FAvaBroadcastDisplayDeviceProvider::GetDevices() const
{
	TArray<FMediaIODevice> Results;

	TArray<FAvaBroadcastMonitorInfo> MonitorInfos = FAvaBroadcastDisplayDeviceManager::GetCachedMonitors();
	
	for (int32 i = 0; i < MonitorInfos.Num(); ++i)
	{
		const FAvaBroadcastMonitorInfo& MonitorInfo = MonitorInfos[i];
		FMediaIODevice Device;
		Device.DeviceName = *FAvaBroadcastDisplayDeviceManager::GetMonitorDisplayName(MonitorInfo);
		Device.DeviceIdentifier = i;
		Results.Add(Device);
	}

	return Results;
}


TArray<FMediaIOMode> FAvaBroadcastDisplayDeviceProvider::GetModes(const FMediaIODevice& InDevice, bool bInOutput) const
{
	TArray<FMediaIOMode> Results;
	return Results;
}

FMediaIOConfiguration FAvaBroadcastDisplayDeviceProvider::GetDefaultConfiguration() const
{
	FMediaIOConfiguration Configuration;
	Configuration.bIsInput = true;
	Configuration.MediaConnection.Device.DeviceIdentifier = 1;
	Configuration.MediaConnection.Protocol = GetProtocolName();
	Configuration.MediaConnection.PortIdentifier = 0;
	Configuration.MediaMode = GetDefaultMode();
	return Configuration;
}


FMediaIOMode FAvaBroadcastDisplayDeviceProvider::GetDefaultMode() const
{
	FMediaIOMode Mode;
	Mode.DeviceModeIdentifier = 0;	// Unused, but can't be invalid.
	Mode.FrameRate = FFrameRate(30, 1);
	Mode.Resolution = FIntPoint(1920, 1080);
	Mode.Standard = EMediaIOStandardType::Progressive;
	return Mode;
}


FMediaIOInputConfiguration FAvaBroadcastDisplayDeviceProvider::GetDefaultInputConfiguration() const
{
	FMediaIOInputConfiguration Configuration;
	Configuration.MediaConfiguration = GetDefaultConfiguration();
	Configuration.MediaConfiguration.bIsInput = true;
	Configuration.InputType = EMediaIOInputType::Fill;
	return Configuration;
}

FMediaIOOutputConfiguration FAvaBroadcastDisplayDeviceProvider::GetDefaultOutputConfiguration() const
{
	FMediaIOOutputConfiguration Configuration;
	Configuration.MediaConfiguration = GetDefaultConfiguration();
	Configuration.MediaConfiguration.bIsInput = false;
	Configuration.OutputReference = EMediaIOReferenceType::FreeRun;
	Configuration.OutputType = EMediaIOOutputType::Fill;
	return Configuration;
}

FMediaIOVideoTimecodeConfiguration FAvaBroadcastDisplayDeviceProvider::GetDefaultTimecodeConfiguration() const
{
	FMediaIOVideoTimecodeConfiguration Configuration;
	Configuration.MediaConfiguration = GetDefaultConfiguration();
	return Configuration;
}

FText FAvaBroadcastDisplayDeviceProvider::ToText(const FMediaIOConfiguration& InConfiguration, bool bInIsAutoDetected) const
{
	if (bInIsAutoDetected)
	{
		return FText::Format(LOCTEXT("FMediaIOAutoConfigurationToText", "{0} - {1} [device{2}/auto]")
				, InConfiguration.bIsInput ? LOCTEXT("In", "In") : LOCTEXT("Out", "Out")
				, FText::FromName(InConfiguration.MediaConnection.Device.DeviceName)
				, FText::AsNumber(InConfiguration.MediaConnection.Device.DeviceIdentifier)
				);
	}
	if (InConfiguration.IsValid())
	{
		return FText::Format(LOCTEXT("FMediaIOConfigurationToText", "[{0}] - {1} [device{2}/{3}]")
			, InConfiguration.bIsInput ? LOCTEXT("In", "In") : LOCTEXT("Out", "Out")
			, FText::FromName(InConfiguration.MediaConnection.Device.DeviceName)
			, FText::AsNumber(InConfiguration.MediaConnection.Device.DeviceIdentifier)
			, InConfiguration.MediaMode.GetModeName()
		);
	}
	return LOCTEXT("Invalid", "<Invalid>");
}


FText FAvaBroadcastDisplayDeviceProvider::ToText(const FMediaIOConnection& InConnection) const
{
	if (InConnection.IsValid())
	{
		return FText::Format(LOCTEXT("FMediaIOConnectionToText", "{0} [device{1}]")
			, FText::FromName(InConnection.Device.DeviceName)
			, LOCTEXT("Device", "device")
			, FText::AsNumber(InConnection.Device.DeviceIdentifier)
		);
	}
	return LOCTEXT("Invalid", "<Invalid>");
}

FText FAvaBroadcastDisplayDeviceProvider::ToText(const FMediaIOOutputConfiguration& InConfiguration) const
{
	if (InConfiguration.IsValid())
	{
		return FText::Format(LOCTEXT("FMediaIOOutputConfigurationToText", "{0} - {1} [device{2}/{3}/{4}]")
			, InConfiguration.OutputType == EMediaIOOutputType::Fill ? LOCTEXT("Fill", "Fill") : LOCTEXT("FillAndKey", "Fill&Key")
			, FText::FromName(InConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName)
			, FText::AsNumber(InConfiguration.MediaConfiguration.MediaConnection.Device.DeviceIdentifier)
			, GetTransportName(InConfiguration.MediaConfiguration.MediaConnection.TransportType, InConfiguration.MediaConfiguration.MediaConnection.QuadTransportType)
			, InConfiguration.MediaConfiguration.MediaMode.GetModeName()
		);
	}
	return LOCTEXT("Invalid", "<Invalid>");
}

#undef LOCTEXT_NAMESPACE
