// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/OutputDevices/AvaBroadcastDisplayMediaCapture.h"

#include "AvaMediaRenderTargetUtils.h"
#include "Broadcast/OutputDevices/AvaBroadcastDisplayDeviceManager.h"
#include "Broadcast/OutputDevices/AvaBroadcastDisplayMediaOutput.h"
#include "Broadcast/OutputDevices/AvaBroadcastRenderTargetMediaUtils.h"
#include "Broadcast/OutputDevices/Slate/SAvaBroadcastCaptureImage.h"
#include "Brushes/SlateImageBrush.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/GenericWindow.h"
#include "Slate/SceneViewport.h"
#include "TextureResource.h"
#include "Widgets/SWindow.h"

DEFINE_LOG_CATEGORY(LogAvaBroadcastDisplayMedia);

class UAvaBroadcastDisplayMediaCapture::FCaptureInstance
{
public:
	FCaptureInstance(UAvaBroadcastDisplayMediaOutput* InMediaOutput, UTextureRenderTarget2D* InRenderTarget)
	{
		if (!InRenderTarget)
		{
			return;
		}
		
		TArray<FAvaBroadcastMonitorInfo> MonitorInfos = FAvaBroadcastDisplayDeviceManager::GetCachedMonitors();

		FString DeviceName = InMediaOutput->OutputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName.ToString();

		FVector2D Size(InRenderTarget->SizeX, InRenderTarget->SizeY);
		FVector2D Position(0, 0);
		EAutoCenter CenterRule = EAutoCenter::PreferredWorkArea;

		for (int32 i = 0; i < MonitorInfos.Num(); ++i)
		{
			const FAvaBroadcastMonitorInfo& MonitorInfo = MonitorInfos[i];
	
			if (DeviceName.Contains(MonitorInfo.Name))
			{
				Size = FVector2D(MonitorInfo.Width, MonitorInfo.Height);
				Position = FVector2D(MonitorInfo.DisplayRect.Left, MonitorInfo.DisplayRect.Top);
				CenterRule = EAutoCenter::None;	// Since we found the display, don't try to auto center.
				break;
			}
		}

		ImageBrush = MakeShared<FSlateImageBrush>(InRenderTarget, FVector2D(InRenderTarget->SizeX, InRenderTarget->SizeY));

		// Source: FPIEPreviewDeviceModule

		static FWindowStyle BackgroundlessStyle = FCoreStyle::Get().GetWidgetStyle<FWindowStyle>("Window");
		BackgroundlessStyle.SetBackgroundBrush(FSlateNoResource());

		// Attempt at creating a borderless window.
		TSharedRef<SWindow> NewWindow = SNew(SWindow)
			.Style(&BackgroundlessStyle)
			.ScreenPosition(Position)
			.ClientSize(Size)
			.UseOSWindowBorder(false)
			.LayoutBorder(FMargin(0))
			.SizingRule(ESizingRule::FixedSize)
			.CreateTitleBar(false)
			.HasCloseButton(false)
			.SupportsMinimize(false)
			.SupportsMaximize(false)
			.AutoCenter(CenterRule)	// Don't mess with requested position/size
			.AdjustInitialSizeAndPositionForDPIScale(false)	// Don't mess with requested position/size
			[
				SNew(SAvaBroadcastCaptureImage)
				.ImageArgs(SImage::FArguments()
					.Image(ImageBrush.Get()))
				.EnableGammaCorrection(false)
				.EnableBlending(false)
			];

		// This will make the window and show it.
		FSlateApplication::Get().AddWindow(NewWindow, true);

		// Fullscreen support for Mac causes a lock up. Disabling until it gets fixed.
#if !PLATFORM_MAC
		NewWindow->SetWindowMode(EWindowMode::Type::WindowedFullscreen);
#endif

		Window = NewWindow;
	}

	~FCaptureInstance()
	{
		if (Window.IsValid() && FSlateApplicationBase::IsInitialized())
		{
			Window->RequestDestroyWindow();
		}
	}

public:
	TSharedPtr<SWindow> Window;

	TSharedPtr<FSlateImageBrush> ImageBrush;
};

UAvaBroadcastDisplayMediaCapture::~UAvaBroadcastDisplayMediaCapture()
{
	delete CaptureInstance;
}

void UAvaBroadcastDisplayMediaCapture::OnRHIResourceCaptured_RenderingThread(FRHICommandListImmediate& RHICmdList, const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FTextureRHIRef InTexture)
{
	FScopeLock ScopeLock(&CaptureInstanceCriticalSection);
	if (CaptureInstance)
	{
		FTextureRHIRef Target;

		// In the mean time, we use an intermediate render target that will be then rendered by Slate in the window.
		// This means an additional avoidable render pass so it is less optimal (but works).
		if (RenderTarget)
		{
			Target = RenderTarget->GetRenderTargetResource() ? RenderTarget->GetRenderTargetResource()->GetRenderTargetTexture() : nullptr;
		}

		if (Target.IsValid())
		{
			UE::AvaBroadcastRenderTargetMediaUtils::CopyTexture(RHICmdList, InTexture, Target);
		}
	}
}

bool UAvaBroadcastDisplayMediaCapture::InitializeCapture()
{
	return true;
}

bool UAvaBroadcastDisplayMediaCapture::PostInitializeCaptureViewport(TSharedPtr<FSceneViewport>& InSceneViewport)
{
	bool bSuccess = false;
	const FTextureRHIRef& BackBuffer = InSceneViewport->GetRenderTargetTexture();
	if (BackBuffer.IsValid())
	{
		const FRHITextureDesc& Desc = BackBuffer->GetDesc();
		bSuccess = StartNewCapture(Desc.Extent, Desc.Format);
	}
	else
	{
		UE_LOGF(LogAvaBroadcastDisplayMedia, Error, "Can't start capture of a scene viewport with no back buffer.");
	}
	return bSuccess;
}

bool UAvaBroadcastDisplayMediaCapture::PostInitializeCaptureRenderTarget(UTextureRenderTarget2D* InRenderTarget)
{
	bool bSuccess = StartNewCapture(FIntPoint(InRenderTarget->SizeX, InRenderTarget->SizeY), InRenderTarget->GetFormat());
	return bSuccess;
}

void UAvaBroadcastDisplayMediaCapture::StopCaptureImpl(bool /*bAllowPendingFrameToBeProcess*/)
{
	TRACE_BOOKMARK(TEXT("UAvaBroadcastDisplayMediaCapture::StopCapture"));

	FScopeLock ScopeLock(&CaptureInstanceCriticalSection);

	delete CaptureInstance;
	CaptureInstance = nullptr;
}

bool UAvaBroadcastDisplayMediaCapture::StartNewCapture(const FIntPoint& InSourceTargetSize, EPixelFormat InSourceTargetFormat)
{
	TRACE_BOOKMARK(TEXT("UAvaBroadcastDisplayMediaCapture::StartNewCapture"));
	{
		FScopeLock ScopeLock(&CaptureInstanceCriticalSection);

		delete CaptureInstance;
		CaptureInstance = nullptr;

		UAvaBroadcastDisplayMediaOutput* AvaDisplayMediaOutput = CastChecked<UAvaBroadcastDisplayMediaOutput>(MediaOutput);
		if (AvaDisplayMediaOutput)
		{
			static const FName RenderTargetBaseName = TEXT("AvaDisplayCapture_RenderTarget");
			RenderTarget = UE::AvaMediaRenderTargetUtils::CreateDefaultRenderTarget(RenderTargetBaseName);
			check(RenderTarget);

			FIntPoint TargetSize = AvaDisplayMediaOutput->GetRequestedSize();
			if (TargetSize == UMediaOutput::RequestCaptureSourceSize)
			{
				TargetSize = InSourceTargetSize;
			}

			EPixelFormat TargetFormat = AvaDisplayMediaOutput->GetRequestedPixelFormat();
			if (TargetFormat == EPixelFormat::PF_Unknown)
			{
				TargetFormat = InSourceTargetFormat;
			}

			RenderTarget->OverrideFormat = TargetFormat;
			RenderTarget->ResizeTarget(TargetSize.X, TargetSize.Y);

			CaptureInstance = new FCaptureInstance(AvaDisplayMediaOutput, RenderTarget);
		}
		else
		{
			UE_LOGF(LogAvaBroadcastDisplayMedia, Error, "Internal Error: Media Capture's associated Media Output is not of type \"UAvaBroadcastDisplayMediaOutput\".");
		}
	}

	SetState(EMediaCaptureState::Capturing);
	return true;
}
