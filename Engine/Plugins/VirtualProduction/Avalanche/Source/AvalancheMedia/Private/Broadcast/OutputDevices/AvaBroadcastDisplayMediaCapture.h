// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/CriticalSection.h"
#include "MediaCapture.h"
#include "AvaBroadcastDisplayMediaCapture.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAvaBroadcastDisplayMedia, Log, All);

UCLASS()
class UAvaBroadcastDisplayMediaCapture : public UMediaCapture
{
	GENERATED_BODY()

public:
	virtual ~UAvaBroadcastDisplayMediaCapture() override; // so we can pimpl.

protected:
	virtual bool ShouldCaptureRHIResource() const override { return true; }
	virtual void OnRHIResourceCaptured_RenderingThread(FRHICommandListImmediate& RHICmdList, const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FTextureRHIRef InTexture) override;

	virtual bool InitializeCapture() override;
	virtual bool PostInitializeCaptureViewport(TSharedPtr<FSceneViewport>& InSceneViewport) override;
	virtual bool PostInitializeCaptureRenderTarget(UTextureRenderTarget2D* InRenderTarget) override;
	virtual void StopCaptureImpl(bool bAllowPendingFrameToBeProcess) override;

	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> RenderTarget;

private:
	bool StartNewCapture(const FIntPoint& InSourceTargetSize, EPixelFormat InSourceTargetFormat);

	FCriticalSection CaptureInstanceCriticalSection;

	// Private capture instance
	class FCaptureInstance;
	FCaptureInstance* CaptureInstance = nullptr; // lazy pimpl
};
