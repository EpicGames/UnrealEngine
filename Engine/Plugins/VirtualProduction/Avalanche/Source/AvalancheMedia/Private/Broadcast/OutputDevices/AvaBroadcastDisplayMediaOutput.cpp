// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/OutputDevices/AvaBroadcastDisplayMediaOutput.h"

#include "Broadcast/OutputDevices/AvaBroadcastDisplayMediaCapture.h"

UAvaBroadcastDisplayMediaOutput::UAvaBroadcastDisplayMediaOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UAvaBroadcastDisplayMediaOutput::Validate(FString& OutFailureReason) const
{
	if (!Super::Validate(OutFailureReason))
	{
		return false;
	}

	return true;
}

FIntPoint UAvaBroadcastDisplayMediaOutput::GetRequestedSize() const
{
	return OutputConfiguration.MediaConfiguration.MediaMode.Resolution;
}

EPixelFormat UAvaBroadcastDisplayMediaOutput::GetRequestedPixelFormat() const
{
	return PF_A2B10G10R10;
}

EMediaCaptureConversionOperation UAvaBroadcastDisplayMediaOutput::GetConversionOperation(EMediaCaptureSourceType /*InSourceType*/) const
{
	return EMediaCaptureConversionOperation::NONE;
}

UMediaCapture* UAvaBroadcastDisplayMediaOutput::CreateMediaCaptureImpl()
{
	UMediaCapture* Result = NewObject<UAvaBroadcastDisplayMediaCapture>();
	if (Result)
	{
		UE_LOGF(LogAvaBroadcastDisplayMedia, Log, "Created Motion Design Display Media Capture");
		Result->SetMediaOutput(this);
	}
	return Result;
}
