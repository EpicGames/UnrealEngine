// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaBroadcastOutputClassItem.h"

#include "AvaBroadcastDeviceProviderRegistry.h"
#include "AvaBroadcastOutputDeviceItem.h"
#include "AvaBroadcastOutputServerItem.h"
#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/OutputDevices/AvaBroadcastDeviceProviderData.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputUtils.h"
#include "Broadcast/OutputDevices/IAvaBroadcastDeviceProviderProxyManager.h"
#include "IAvaMediaModule.h"
#include "IMediaIOCoreDeviceProvider.h"
#include "IMediaIOCoreModule.h"
#include "MediaIOCoreDefinitions.h"
#include "MediaOutput.h"
#include "Slate/SAvaBroadcastOutputTreeItem.h"
#include "Styling/SlateIconFinder.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#define LOCTEXT_NAMESPACE "AvaBroadcastOutputClassItem"

namespace UE::AvaOutputClassItem::Private
{
	IMediaIOCoreDeviceProvider* GetDeviceProvider(const UClass* InMediaOutputClass)
	{
		const FName DeviceProviderName = UE::AvaBroadcastOutputUtils::GetDeviceProviderName(InMediaOutputClass);
		if (!IMediaIOCoreModule::IsAvailable() || DeviceProviderName.IsNone())
		{
			return nullptr;
		}

		// Remark: even if the class has a device provider name, it is possible
		// that there is no corresponding device provider.
		return IMediaIOCoreModule::Get().GetDeviceProvider(DeviceProviderName);
	}

	void UpdateDeviceName(UAvaBroadcast& InBroadcast, const FName& InChannelName, const UMediaOutput* InMediaOutput, const FString& InDeviceName)
	{
		FAvaBroadcastOutputChannel& Channel = InBroadcast.GetCurrentProfile().GetChannelMutable(InChannelName);
		if (Channel.IsValidChannel())
		{
			if (FAvaBroadcastMediaOutputInfo* ExistingOutputInfo = Channel.GetMediaOutputInfoMutable(InMediaOutput))
			{
				ExistingOutputInfo->DeviceName = FName(InDeviceName);	
			}
		}
	}
}

FText FAvaBroadcastOutputClassItem::GetDisplayName() const
{
#if WITH_EDITOR
	check(OutputClass.IsValid());
	return OutputClass->GetDisplayNameText();
#else
	if (OutputClass.IsValid())
	{
		return FAvaBroadcastDeviceProviderRegistry::Get().GetOutputClassDisplayText(OutputClass.Get());
	}
	return LOCTEXT("DisplayName_NotAvailable", "NotAvailable");
#endif
}

const FSlateBrush* FAvaBroadcastOutputClassItem::GetIconBrush() const
{
	check(OutputClass.IsValid());
	return FSlateIconFinder::FindIconBrushForClass(OutputClass.Get());
}

void FAvaBroadcastOutputClassItem::RefreshChildren(const FRefreshChildrenParams& InParams)
{
	IMediaIOCoreDeviceProvider* DeviceProvider = UE::AvaOutputClassItem::Private::GetDeviceProvider(OutputClass.Get());

	if (!DeviceProvider)
	{
		Children.Reset();
		return;
	}

	TSharedPtr<FAvaBroadcastOutputTreeItem> Parent = ParentWeak.Pin();
	if (!Parent.IsValid())
	{
		return;
	}

	TSharedPtr<FAvaBroadcastOutputServerItem> ParentServerItem = StaticCastSharedPtr<FAvaBroadcastOutputServerItem>(Parent);
	check(ParentServerItem.IsValid());

	const FName DeviceProviderName = DeviceProvider->GetFName();
	const IAvaBroadcastDeviceProviderProxyManager& DeviceProviderProxyManager = IAvaMediaModule::Get().GetDeviceProviderProxyManager();

	TArray<FMediaIOConfiguration> OutputConfigs = DeviceProvider->GetConfigurations(false, true);

	// Fill current devices from output configs of this device provider.
	TSet<FAvaBroadcastDevice> CurrentDevices;
	CurrentDevices.Reserve(OutputConfigs.Num());

	for (const FMediaIOConfiguration& Config : OutputConfigs)
	{
		const FString ServerName = DeviceProviderProxyManager.FindServerNameForDevice(DeviceProviderName, Config.MediaConnection.Device.DeviceName);
		const bool bIsLocal = DeviceProviderProxyManager.IsLocalDevice(DeviceProviderName, Config.MediaConnection.Device.DeviceName);
		const FAvaBroadcastDeviceProviderData* DeviceProviderDataForServer = ParentServerItem->GetDeviceProviderData(DeviceProviderName);

		if ((!DeviceProviderDataForServer && bIsLocal)
			 || (DeviceProviderDataForServer && !bIsLocal && ServerName == ParentServerItem->GetServerName()))
		{
			CurrentDevices.Add({ Config.MediaConnection.Device, DeviceProviderName, ServerName });
		}
	}

	//A set to contain the Media Devices of the Current Children
	TSet<FAvaBroadcastDevice> SeenDevices;
	SeenDevices.Reserve(CurrentDevices.Num());
	
	//Remove Existing Children that are Invalid
	for (TArray<FAvaOutputTreeItemPtr>::TIterator ItemIt = Children.CreateIterator(); ItemIt; ++ItemIt)
	{
		FAvaOutputTreeItemPtr Item(*ItemIt);

		//Remove Invalid Pointers or Items that are not Device Items since a Class Item can only contain Device Items
		if (!Item.IsValid() || !Item->IsA<FAvaBroadcastOutputDeviceItem>())
		{
			ItemIt.RemoveCurrent();
			continue;
		}

		const TSharedPtr<FAvaBroadcastOutputDeviceItem> DeviceItem = StaticCastSharedPtr<FAvaBroadcastOutputDeviceItem>(Item);
		const FAvaBroadcastDevice& UnderlyingDevice = DeviceItem->GetDevice();

		if (UnderlyingDevice.IsValid() && CurrentDevices.Contains(UnderlyingDevice))
		{
			SeenDevices.Add(UnderlyingDevice);
		}
		else
		{
			//Remove if there's no valid Underlying Device or it's no longer in the set
			ItemIt.RemoveCurrent();
		}
	}
	
	//Append the Devices that are not already in the Original Children List
	{
		TSharedPtr<FAvaBroadcastOutputClassItem> This = SharedThis(this);
		TArray<FAvaBroadcastDevice> NewDevices = CurrentDevices.Difference(SeenDevices).Array();
		Children.Reserve(Children.Num() + NewDevices.Num());
		
		for (const FAvaBroadcastDevice& Device : NewDevices)
		{
			TSharedPtr<FAvaBroadcastOutputDeviceItem> DeviceItem = MakeShared<FAvaBroadcastOutputDeviceItem>(This, Device);
			Children.Add(DeviceItem);
		}
	}
}

TSharedPtr<SWidget> FAvaBroadcastOutputClassItem::GenerateRowWidget()
{
	return SNew(SAvaBroadcastOutputTreeItem, SharedThis(this));
}

UMediaOutput* FAvaBroadcastOutputClassItem::AddMediaOutputToChannel(FName InTargetChannel, const FAvaBroadcastMediaOutputInfo& InOutputInfo)
{
	check(OutputClass.IsValid());

	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();

#if WITH_EDITOR
	FScopedTransaction Transaction(LOCTEXT("AddMediaOutput", "Add Media Output"));
#endif
	Broadcast.Modify();

	FAvaBroadcastMediaOutputInfo OutputInfo = InOutputInfo;
	
	// Support non-enumerated devices (i.e. no device provider).
	if (!OutputInfo.IsValid() || !OutputInfo.Guid.IsValid())
	{
		if (TSharedPtr<FAvaBroadcastOutputTreeItem> Parent = ParentWeak.Pin())
		{
			const TSharedPtr<FAvaBroadcastOutputServerItem> ParentServerItem = StaticCastSharedPtr<FAvaBroadcastOutputServerItem>(Parent);
			check(ParentServerItem.IsValid());

			// Fill the device info with available data.
			OutputInfo.Guid = FGuid::NewGuid();
			OutputInfo.ServerName = ParentServerItem->GetServerName();

			// Fetch the device provider name (if there is one) from the class's meta data.
			OutputInfo.DeviceProviderName = UE::AvaBroadcastOutputUtils::GetDeviceProviderName(OutputClass.Get());

			// Device name:
			// There is no enumerated device name. It will be updated from the media output object below.
		}
	}
	
	UMediaOutput* const MediaOutput = Broadcast.GetCurrentProfile().AddChannelMediaOutput(InTargetChannel, OutputClass.Get(), OutputInfo);

	// If the device name was not provided as input, we need to update it from the Media Output object.
	if (MediaOutput && OutputInfo.DeviceName.IsNone())
	{
		const FString DeviceName = UE::AvaBroadcastOutputUtils::GetDeviceName(MediaOutput);
		if (!DeviceName.IsEmpty())
		{
			UE::AvaOutputClassItem::Private::UpdateDeviceName(Broadcast, InTargetChannel, MediaOutput, DeviceName);
		}
	}

#if WITH_EDITOR
	if (!MediaOutput)
	{
		Transaction.Cancel();
	}
#endif
	
	return MediaOutput;
}

#undef LOCTEXT_NAMESPACE
