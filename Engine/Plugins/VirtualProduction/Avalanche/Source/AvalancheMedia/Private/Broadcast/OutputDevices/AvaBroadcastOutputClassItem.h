// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Broadcast/OutputDevices/AvaBroadcastOutputTreeItem.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FAvaBroadcastOutputClassItem : public FAvaBroadcastOutputTreeItem
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaBroadcastOutputClassItem, FAvaBroadcastOutputTreeItem);

	FAvaBroadcastOutputClassItem(const TSharedPtr<FAvaBroadcastOutputTreeItem>& InParent, UClass* InOutputClass)
		: Super(InParent)
		, OutputClass(InOutputClass)
	{
	}

	UClass* GetOutputClass() const
	{
		return OutputClass.Get();
	}

	//~ Begin IAvaBroadcastOutputTreeItem
	virtual FText GetDisplayName() const override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual void RefreshChildren(const FRefreshChildrenParams& InParams) override;
	virtual TSharedPtr<SWidget> GenerateRowWidget() override;
	virtual UMediaOutput* AddMediaOutputToChannel(FName InTargetChannel, const FAvaBroadcastMediaOutputInfo& InOutputInfo) override;
	//~ End IAvaBroadcastOutputTreeItem

protected:
	TWeakObjectPtr<UClass> OutputClass;
};
