// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaBroadcastOutputDeviceItem.h"

#include "AvaBroadcastOutputClassItem.h"
#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/Channel/AvaBroadcastMediaOutputInfo.h"
#include "IMediaIOCoreDeviceProvider.h"
#include "IMediaIOCoreModule.h"
#include "MediaOutput.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AvaBroadcastOutputDeviceItem"

FText FAvaBroadcastOutputDeviceItem::GetDisplayName() const
{
	return Device.GetDisplayNameText();
}

const FSlateBrush* FAvaBroadcastOutputDeviceItem::GetIconBrush() const
{
	return nullptr;
}

void FAvaBroadcastOutputDeviceItem::RefreshChildren(const FRefreshChildrenParams& InParams)
{
	//There shouldn't be any Childrens here
	Children.Reset();
}

TSharedPtr<SWidget> FAvaBroadcastOutputDeviceItem::GenerateRowWidget()
{
	return SNew(STextBlock)
		.Text(GetDisplayName());
}

bool FAvaBroadcastOutputDeviceItem::IsValidToDropInChannel(FName InTargetChannelName)
{
	// It is invalid to drop a "remote" device in a preview channel.
	if (UAvaBroadcast::Get().GetChannelType(InTargetChannelName) == EAvaBroadcastChannelType::Preview)
	{
		return !FAvaBroadcastMediaOutputInfo::IsRemote(Device.GetServerName());
	}
	return true;
}

UMediaOutput* FAvaBroadcastOutputDeviceItem::AddMediaOutputToChannel(FName InTargetChannel, const FAvaBroadcastMediaOutputInfo& /*InOutputInfo*/)
{
	if (TSharedPtr<FAvaBroadcastOutputTreeItem> Parent = ParentWeak.Pin())
	{
		FAvaBroadcastMediaOutputInfo OutputInfo;
		OutputInfo.Guid = FGuid::NewGuid();
		OutputInfo.ServerName = Device.GetServerName();
		OutputInfo.DeviceProviderName = Device.GetDeviceProviderName();
		OutputInfo.DeviceName = Device.GetDevice().DeviceName;

		UMediaOutput* const MediaOutput = Parent->AddMediaOutputToChannel(InTargetChannel, OutputInfo);
		
		if (MediaOutput)
		{
			const UClass* const MediaOutputClass = MediaOutput->GetClass();
			const IMediaIOCoreDeviceProvider* const DeviceProvider = IMediaIOCoreModule::Get().GetDeviceProvider(Device.GetDeviceProviderName());
			
			if (DeviceProvider)
			{
				FStructProperty* const Property = FindFProperty<FStructProperty>(MediaOutputClass, TEXT("OutputConfiguration"));
				if (Property && Property->Struct->IsChildOf(FMediaIOOutputConfiguration::StaticStruct()))
				{
					if (FMediaIOOutputConfiguration* const OutputConfig = Property->ContainerPtrToValuePtr<FMediaIOOutputConfiguration>(MediaOutput))
					{
						{
							// Initial "default" setup. 
							// This leaves the OutputConfig object in a possibly invalid state
							// because it is very likely the selected device's config doesn't match the one returned by
							// GetDefaultOutputConfiguration().
							*OutputConfig = DeviceProvider->GetDefaultOutputConfiguration();
							OutputConfig->MediaConfiguration.MediaConnection.Device = Device.GetDevice();
						}

						// Fetch all possible output configurations.
						TArray<FMediaIOOutputConfiguration> OutputConfigs = DeviceProvider->GetOutputConfigurations();

						// Select one that match the device. 
						// For now just get the first one.
						for (const FMediaIOOutputConfiguration& Config : OutputConfigs)
						{
							if (Config.MediaConfiguration.MediaConnection.Device.DeviceName == Device.GetDevice().DeviceName)
							{
								*OutputConfig = Config;
								break; // get the first one.
							}
						}
#if WITH_EDITOR
						FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet);
						MediaOutput->PostEditChangeProperty(PropertyChangedEvent);
#endif
					}
				}
			}
		}
		return MediaOutput;
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
