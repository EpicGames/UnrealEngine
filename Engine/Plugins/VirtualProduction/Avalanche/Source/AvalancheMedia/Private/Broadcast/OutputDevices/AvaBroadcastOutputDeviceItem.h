// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Broadcast/OutputDevices/AvaBroadcastOutputTreeItem.h"
#include "Broadcast/AvaBroadcastDevice.h"

class FAvaBroadcastOutputClassItem;

class FAvaBroadcastOutputDeviceItem : public FAvaBroadcastOutputTreeItem
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaBroadcastOutputDeviceItem, FAvaBroadcastOutputTreeItem);

	FAvaBroadcastOutputDeviceItem(const TSharedPtr<FAvaBroadcastOutputTreeItem>& InParent, const FAvaBroadcastDevice& InDevice)
		: Super(InParent)
		, Device(InDevice)
	{
	}

	const FAvaBroadcastDevice& GetDevice() const
	{
		return Device;
	}

	//~ Begin IAvaBroadcastOutputTreeItem
	virtual FText GetDisplayName() const override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual void RefreshChildren(const FRefreshChildrenParams& InParams) override;
	virtual TSharedPtr<SWidget> GenerateRowWidget() override;
	virtual bool IsValidToDropInChannel(FName InTargetChannelName) override;
	virtual UMediaOutput* AddMediaOutputToChannel(FName InTargetChannel, const FAvaBroadcastMediaOutputInfo& InOutputInfo) override;
	//~ End IAvaBroadcastOutputTreeItem

protected:
	FAvaBroadcastDevice Device;
};
