// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/OutputDevices/AvaBroadcastOutputRootItem.h"

#include "Broadcast/OutputDevices/AvaBroadcastOutputServerItem.h"
#include "Broadcast/OutputDevices/IAvaBroadcastDeviceProviderProxyManager.h"
#include "IAvaMediaModule.h"
#include "Input/Reply.h"
#include "MediaOutput.h"
#include "UObject/UObjectIterator.h"

void FAvaBroadcastOutputRootItem::RefreshChildren(const FRefreshChildrenParams& InParams)
{
	const IAvaBroadcastDeviceProviderProxyManager& DeviceProviderProxyManager = IAvaMediaModule::Get().GetDeviceProviderProxyManager();

	const TSet<FString> CurrentServers = DeviceProviderProxyManager.GetServerNames();

	TSet<FString> SeenServerNames;
	SeenServerNames.Reserve(CurrentServers.Num());

	//Add local server item
	const TSharedPtr<FAvaBroadcastOutputServerItem> OutputLocalServerItem = MakeShared<FAvaBroadcastOutputServerItem>(DeviceProviderProxyManager.GetLocalServerName(), nullptr);
	Children.Add(OutputLocalServerItem);

	//Remove Existing Children that are Invalid
	for (TArray<FAvaOutputTreeItemPtr>::TIterator ItemIt = Children.CreateIterator(); ItemIt; ++ItemIt)
	{
		FAvaOutputTreeItemPtr Item(*ItemIt);

		//Always going to display local devices group
		if (Item == OutputLocalServerItem)
		{
			continue;
		}

		//Remove Invalid Pointers or Items that are not Output Server Items since Root can only have the Server Items as Top Level
		if (!Item.IsValid() || !Item->IsA<FAvaBroadcastOutputServerItem>())
		{
			ItemIt.RemoveCurrent();
			continue;
		}

		const TSharedPtr<FAvaBroadcastOutputServerItem> OutputServerItem = StaticCastSharedPtr<FAvaBroadcastOutputServerItem>(Item);
		const FString UnderlyingServerName = OutputServerItem->GetServerName();

		if (!UnderlyingServerName.IsEmpty() && CurrentServers.Contains(UnderlyingServerName))
		{
			SeenServerNames.Add(UnderlyingServerName);
		}
		else
		{
			//Remove if there's no valid Underlying OutputClass or it's no longer in the set
			ItemIt.RemoveCurrent();
		}
	}
	
	//Append the New Servers that are not already in the Original Children List
	{
		TSet<FString> NewServers = CurrentServers.Difference(SeenServerNames);
		Children.Reserve(Children.Num() + NewServers.Num());
		
		for (const FString& ServerName : NewServers)
		{
			if (const TSharedPtr<const FAvaBroadcastDeviceProviderDataList> ServerDeviceProviderDataList = DeviceProviderProxyManager.GetDeviceProviderDataListForServer(ServerName))
			{
				TSharedPtr<FAvaBroadcastOutputServerItem> OutputServerItem = MakeShared<FAvaBroadcastOutputServerItem>(ServerName, ServerDeviceProviderDataList);
				Children.Add(OutputServerItem);
			}
		}
	}
}

FText FAvaBroadcastOutputRootItem::GetDisplayName() const
{
	//Root shouldn't need to give a Display Name!
	check(0);
	return FText::GetEmpty();
}

const FSlateBrush* FAvaBroadcastOutputRootItem::GetIconBrush() const
{
	check(0);
	return nullptr;
}

TSharedPtr<SWidget> FAvaBroadcastOutputRootItem::GenerateRowWidget()
{
	//Root shouldn't need to Generate a Row Widget
	check(0);
	return nullptr;
}

FReply FAvaBroadcastOutputRootItem::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	check(0);
	return FReply::Unhandled();
}

UMediaOutput* FAvaBroadcastOutputRootItem::AddMediaOutputToChannel(FName InTargetChannel, const FAvaBroadcastMediaOutputInfo& InOutputInfo)
{
	check(0);
	return nullptr;
}
