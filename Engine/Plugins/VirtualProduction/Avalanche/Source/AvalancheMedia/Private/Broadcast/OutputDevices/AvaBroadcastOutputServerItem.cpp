// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaBroadcastOutputServerItem.h"

#include "AvaBroadcastOutputClassItem.h"
#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputUtils.h"
#include "IMediaIOCoreModule.h"
#include "MediaOutput.h"
#include "Slate/SAvaBroadcastOutputTreeItem.h"
#include "Styling/AppStyle.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#define LOCTEXT_NAMESPACE "AvaBroadcastOutputServerItem"

FString FAvaBroadcastOutputServerItem::GetServerName() const
{
	return ServerName;
}

const FAvaBroadcastDeviceProviderData* FAvaBroadcastOutputServerItem::GetDeviceProviderData(FName InDeviceProviderName) const
{
	if (!DeviceProviderDataList.IsValid())
	{
		return nullptr;
	}

	for (const FAvaBroadcastDeviceProviderData& DeviceProviderData : DeviceProviderDataList->DeviceProviders)
	{
		if (DeviceProviderData.Name == InDeviceProviderName)
		{
			return &DeviceProviderData;
		}
	}

	return nullptr;
}

FText FAvaBroadcastOutputServerItem::GetDisplayName() const
{
	return FText::FromString(ServerName);
}

const FSlateBrush* FAvaBroadcastOutputServerItem::GetIconBrush() const
{
	return FAppStyle::Get().GetBrush(TEXT("Icons.Server"));
}

void FAvaBroadcastOutputServerItem::RefreshChildren(const FRefreshChildrenParams& InParams)
{
	if (!IMediaIOCoreModule::IsAvailable())
	{
		Children.Reset();
		return;
	}

	// Make a set of all allowed Media Output Classes.
	// It may have all classes or be restricted to those with the MediaIOCustomLayout MetaData.
	TSet<UClass*> CurrentOutputClasses;
	for (UClass* const Class : TObjectRange<UClass>())
	{
		const bool bIsMediaOutputClass = Class->IsChildOf(UMediaOutput::StaticClass()) && Class != UMediaOutput::StaticClass();
		const bool bHasDeviceProvider = UE::AvaBroadcastOutputUtils::HasDeviceProviderName(Class);

		if (bIsMediaOutputClass && (InParams.bShowAllMediaOutputClasses || bHasDeviceProvider))
		{
			CurrentOutputClasses.Add(Class);
		}
	}

	TSet<UClass*> SeenOutputClasses;
	SeenOutputClasses.Reserve(CurrentOutputClasses.Num());

	//Remove Existing Children that are Invalid
	for (TArray<FAvaOutputTreeItemPtr>::TIterator ItemIt = Children.CreateIterator(); ItemIt; ++ItemIt)
	{
		FAvaOutputTreeItemPtr Item(*ItemIt);

		//Remove Invalid Pointers or Items that are not Output Class Items since Root can only have the Class Items as Top Level
		if (!Item.IsValid() || !Item->IsA<FAvaBroadcastOutputClassItem>())
		{
			ItemIt.RemoveCurrent();
			continue;
		}

		const TSharedPtr<FAvaBroadcastOutputClassItem> OutputClassItem = StaticCastSharedPtr<FAvaBroadcastOutputClassItem>(Item);
		UClass* const UnderlyingOutputClass = OutputClassItem->GetOutputClass();

		if (UnderlyingOutputClass && CurrentOutputClasses.Contains(UnderlyingOutputClass))
		{
			SeenOutputClasses.Add(UnderlyingOutputClass);
		}
		else
		{
			//Remove if there's no valid Underlying OutputClass or it's no longer in the set
			ItemIt.RemoveCurrent();
		}
	}

	//Append the New Output Classes that are not already in the Original Children List
	{
		TSharedPtr<FAvaBroadcastOutputServerItem> This = SharedThis(this);
		TArray<UClass*> NewOutputClasses = CurrentOutputClasses.Difference(SeenOutputClasses).Array();
		Children.Reserve(Children.Num() + NewOutputClasses.Num());

		for (UClass* const NewOutputClass : NewOutputClasses)
		{
			TSharedPtr<FAvaBroadcastOutputClassItem> OutputClassItem = MakeShared<FAvaBroadcastOutputClassItem>(This, NewOutputClass);
			Children.Add(OutputClassItem);
		}
	}
}

TSharedPtr<SWidget> FAvaBroadcastOutputServerItem::GenerateRowWidget()
{
	return SNew(SAvaBroadcastOutputTreeItem, SharedThis(this));
}

UMediaOutput* FAvaBroadcastOutputServerItem::AddMediaOutputToChannel(FName InTargetChannel, const FAvaBroadcastMediaOutputInfo& InOutputInfo)
{
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
