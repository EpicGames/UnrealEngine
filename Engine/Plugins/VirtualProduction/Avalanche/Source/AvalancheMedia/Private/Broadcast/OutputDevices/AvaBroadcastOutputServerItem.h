// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Broadcast/OutputDevices/AvaBroadcastOutputTreeItem.h"
#include "Broadcast/OutputDevices/AvaBroadcastDeviceProviderData.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FAvaBroadcastOutputServerItem : public FAvaBroadcastOutputTreeItem
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaBroadcastOutputServerItem, FAvaBroadcastOutputTreeItem);

	FAvaBroadcastOutputServerItem(const FString& InServerName, const TSharedPtr<const FAvaBroadcastDeviceProviderDataList>& InDeviceProviderDataList)
		: Super(nullptr)
		, ServerName(InServerName)
		, DeviceProviderDataList(InDeviceProviderDataList)
	{
	}

	FString GetServerName() const;
	const FAvaBroadcastDeviceProviderData* GetDeviceProviderData(FName InDeviceProviderName) const;

	//~ Begin IAvaBroadcastOutputTreeItem
	virtual FText GetDisplayName() const override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual void RefreshChildren(const FRefreshChildrenParams& InParams) override;
	virtual TSharedPtr<SWidget> GenerateRowWidget() override;
	virtual bool IsValidToDropInChannel(FName InTargetChannelName) override { return false; }
	virtual UMediaOutput* AddMediaOutputToChannel(FName InTargetChannel, const FAvaBroadcastMediaOutputInfo& InOutputInfo) override;
	//~ End IAvaBroadcastOutputTreeItem

protected:
	FString ServerName;
	const TSharedPtr<const FAvaBroadcastDeviceProviderDataList> DeviceProviderDataList;
};
