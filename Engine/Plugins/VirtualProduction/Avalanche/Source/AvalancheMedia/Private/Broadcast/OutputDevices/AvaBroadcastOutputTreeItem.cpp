// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/OutputDevices/AvaBroadcastOutputTreeItem.h"

#include "Input/Reply.h"

const TWeakPtr<FAvaBroadcastOutputTreeItem>& FAvaBroadcastOutputTreeItem::GetParent() const
{
	return ParentWeak;
}

const TArray<FAvaOutputTreeItemPtr>& FAvaBroadcastOutputTreeItem::GetChildren() const
{
	return Children;
}

FReply FAvaBroadcastOutputTreeItem::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		if (OnCreateDragDropOperationDelegate.IsBound())
		{
			return FReply::Handled().BeginDragDrop(OnCreateDragDropOperationDelegate.Execute(SharedThis(this)));
		}
	}
	return FReply::Unhandled();
}

void FAvaBroadcastOutputTreeItem::RefreshTree(const FAvaOutputTreeItemPtr& InItem, const FRefreshChildrenParams& InParams)
{
	TArray<FAvaOutputTreeItemPtr> ItemsRemainingToRefresh;
	ItemsRemainingToRefresh.Add(InItem);
		
	while (ItemsRemainingToRefresh.Num() > 0)
	{
		FAvaOutputTreeItemPtr Item = ItemsRemainingToRefresh.Pop();
		if (Item.IsValid())
		{
			Item->RefreshChildren(InParams);
			ItemsRemainingToRefresh.Append(Item->GetChildren());
		}
	}
}
