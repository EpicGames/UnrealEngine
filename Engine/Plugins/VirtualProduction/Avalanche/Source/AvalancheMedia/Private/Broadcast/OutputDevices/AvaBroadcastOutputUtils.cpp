// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/OutputDevices/AvaBroadcastOutputUtils.h"

#include "AvaBroadcastDeviceProviderRegistry.h"
#include "Broadcast/OutputDevices/AvaGameViewportMediaOutput.h"
#include "MediaIOCoreDefinitions.h"
#include "MediaOutput.h"
#include "Playback/AvaPlaybackMessages.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"

namespace UE::AvaBroadcastOutputUtils::Private
{
	// This is a meta data that is added to all Media Output classes (that have a device provider)
	// giving information on what DeviceProvider it is using.
	static const FName NAME_MediaIOCustomLayout("MediaIOCustomLayout");
}

FString UE::AvaBroadcastOutputUtils::GetDeviceName(const UMediaOutput* InMediaOutput)
{
	const UClass* const MediaOutputClass = InMediaOutput->GetClass();
	{
		const FStructProperty* const Property = FindFProperty<FStructProperty>(MediaOutputClass, TEXT("OutputConfiguration"));
		if (Property && Property->Struct->IsChildOf(FMediaIOOutputConfiguration::StaticStruct()))
		{
			const FMediaIOOutputConfiguration* const OutputConfig = Property->ContainerPtrToValuePtr<FMediaIOOutputConfiguration>(InMediaOutput);
			return OutputConfig->MediaConfiguration.MediaConnection.Device.DeviceName.ToString();
		}
	}
	
	if (const FProperty* const SourceNameProperty = FindFProperty<FProperty>(MediaOutputClass, TEXT("SourceName")))
	{
		return *SourceNameProperty->ContainerPtrToValuePtr<FString>(InMediaOutput);
	}

	if (const FProperty* const StreamerIdProperty = FindFProperty<FProperty>(MediaOutputClass, TEXT("StreamerId")))
	{
		return *StreamerIdProperty->ContainerPtrToValuePtr<FString>(InMediaOutput);
	}
	
	return TEXT("");
}

bool UE::AvaBroadcastOutputUtils::HasDeviceProviderName(const UClass* InMediaOutputClass)
{
#if WITH_EDITORONLY_DATA
	return IsValid(InMediaOutputClass)
		&& InMediaOutputClass->IsChildOf(UMediaOutput::StaticClass())
		&& InMediaOutputClass->HasMetaData(Private::NAME_MediaIOCustomLayout);
#else
	return FAvaBroadcastDeviceProviderRegistry::Get().HasDeviceProviderName(InMediaOutputClass);
#endif
}

FName UE::AvaBroadcastOutputUtils::GetDeviceProviderName(const UClass* InMediaOutputClass)
{
#if WITH_EDITORONLY_DATA
	if (HasDeviceProviderName(InMediaOutputClass))
	{
		return FName(InMediaOutputClass->GetMetaData(Private::NAME_MediaIOCustomLayout));
	}
	return FName();
#else
	return FAvaBroadcastDeviceProviderRegistry::Get().GetDeviceProviderName(InMediaOutputClass);
#endif
}

FName UE::AvaBroadcastOutputUtils::GetDeviceProviderName(const UMediaOutput* InMediaOutput)
{
	const UClass* const MediaOutputClass = IsValid(InMediaOutput) ? InMediaOutput->GetClass() : nullptr;
	if (MediaOutputClass && IsValid(MediaOutputClass))
	{
#if WITH_EDITORONLY_DATA
		if (MediaOutputClass->HasMetaData(Private::NAME_MediaIOCustomLayout))
		{
			return FName(MediaOutputClass->GetMetaData(Private::NAME_MediaIOCustomLayout));
		}
#else
		return FAvaBroadcastDeviceProviderRegistry::Get().GetDeviceProviderName(MediaOutputClass);
#endif
	}
	return FName(); 
}

bool UE::AvaBroadcastOutputUtils::IsGameViewportOutput(const UMediaOutput* InMediaOutput)
{
	return InMediaOutput->IsA<UAvaGameViewportMediaOutput>();
}

FAvaBroadcastOutputData UE::AvaBroadcastOutputUtils::CreateMediaOutputData(UMediaOutput* InMediaOutput)
{
	FAvaBroadcastOutputData MediaOutputData;
	MediaOutputData.MediaOutputClass = InMediaOutput->GetClass();
			
	TArray<uint8> SerializedData;
	FMemoryWriter MemoryWriter(SerializedData, true); 
	FObjectAndNameAsStringProxyArchive ObjectWriter = FObjectAndNameAsStringProxyArchive(MemoryWriter, false);
	InMediaOutput->Serialize(ObjectWriter);

	MediaOutputData.SerializedData = MoveTemp(SerializedData);
	MediaOutputData.ObjectFlags = InMediaOutput->GetFlags();

	return MediaOutputData;
}

UMediaOutput* UE::AvaBroadcastOutputUtils::CreateMediaOutput(const FAvaBroadcastOutputData& InMediaOutputData, UObject* InOuter)
{
	UClass* const MediaOutputClass = InMediaOutputData.MediaOutputClass.LoadSynchronous();
	UMediaOutput* MediaOutput = nullptr;
	if (MediaOutputClass)
	{
		MediaOutput = NewObject<UMediaOutput>(
			InOuter
			, MediaOutputClass
			, NAME_None
			, InMediaOutputData.GetObjectFlags());

		FMemoryReader MemoryReader(InMediaOutputData.SerializedData, true); 
		FObjectAndNameAsStringProxyArchive ObjectReader = FObjectAndNameAsStringProxyArchive(MemoryReader, false);
		MediaOutput->Serialize(ObjectReader);
		MediaOutput->PostLoad();
	}
	return MediaOutput;
}