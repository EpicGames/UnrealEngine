// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/OutputDevices/AvaBroadcastRenderTargetMediaCapture.h"

#include "AvaMediaRenderTargetUtils.h"
#include "Broadcast/OutputDevices/AvaBroadcastRenderTargetMediaOutput.h"
#include "Broadcast/OutputDevices/AvaBroadcastRenderTargetMediaUtils.h"
#include "Engine/Engine.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Slate/SceneViewport.h"
#include "TextureResource.h"

DEFINE_LOG_CATEGORY(LogAvaBroadcastRenderTargetMedia);

void UAvaBroadcastRenderTargetMediaCapture::OnRHIResourceCaptured_RenderingThread(FRHICommandListImmediate& RHICmdList, const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FTextureRHIRef InTexture)
{
	FScopeLock ScopeLock(&RenderTargetCriticalSection);

	if (!RenderTarget || !RenderTarget->GetRenderTargetResource())
	{
		return;
	}
	
	const FTextureRHIRef RenderTargetRHIRef = RenderTarget->GetRenderTargetResource()->GetRenderTargetTexture();

	if (RenderTargetRHIRef.IsValid())
	{
		// Note: we possibly don't want to undo this for RGB8_SRGB texture. Need to test.
		constexpr bool bSRGBToLinear = true;

		float Gamma = 1.0f;
		
		if (const FTextureRenderTarget2DResource* Resource = static_cast<FTextureRenderTarget2DResource*>(RenderTarget->GetResource()))
		{
			// The render targets for rendering the ava channel has bForceLinearGamma = false
			// and TargetGamma = 2.2 for float textures (to force output to gamma space).
			// Those parameters are used to determine the render target's display gamma, so
			// we need to match the same display gamma that was used for rendering in order to invert it.
			// See UAvaPlaybackGraph::UpdateRenderTarget.			
			const bool bBackupLinearGamma = RenderTarget->bForceLinearGamma;
			const float BackupTargetGamma = RenderTarget->TargetGamma;
			RenderTarget->bForceLinearGamma = false;
			if (UE::AvaMediaRenderTargetUtils::IsFloatFormat(RenderTarget))
			{
				RenderTarget->TargetGamma = GEngine ? GEngine->GetDisplayGamma() : 2.2f;
			}
			// Must invert the gamma done in post process.
			// See GetTonemapperOutputDeviceParameters (PostProcessTonemap.cpp).
			Gamma = Resource ? Resource->GetDisplayGamma()/2.2f : 1.0f;
			RenderTarget->bForceLinearGamma = bBackupLinearGamma;
			RenderTarget->TargetGamma = BackupTargetGamma;
		}

		// This will convert the color space from SCS_FinalColorSDR back to SCS_FinalToneCurveHDR.
		UE::AvaBroadcastRenderTargetMediaUtils::ConvertTextureRGBGamma(RHICmdList, InTexture, RenderTargetRHIRef, bSRGBToLinear, Gamma);
	}
}

bool UAvaBroadcastRenderTargetMediaCapture::InitializeCapture()
{
	return true;
}

bool UAvaBroadcastRenderTargetMediaCapture::PostInitializeCaptureViewport(TSharedPtr<FSceneViewport>& InSceneViewport)
{
	bool bSuccess = false;
	const FTextureRHIRef& BackBuffer = InSceneViewport->GetRenderTargetTexture();
	if (BackBuffer.IsValid())
	{
		const FRHITextureDesc& Desc = BackBuffer->GetDesc();
		bSuccess = StartNewCapture(Desc.Extent, Desc.Format);
	}
	else
	{
		UE_LOGF(LogAvaBroadcastRenderTargetMedia, Error, "Can't start capture of a scene viewport with no back buffer.");
	}
	return bSuccess;
}

bool UAvaBroadcastRenderTargetMediaCapture::PostInitializeCaptureRenderTarget(UTextureRenderTarget2D* InRenderTarget)
{
	return StartNewCapture(FIntPoint(InRenderTarget->SizeX, InRenderTarget->SizeY), InRenderTarget->GetFormat());
}

bool UAvaBroadcastRenderTargetMediaCapture::PostInitializeCaptureRHIResource(const FRHICaptureResourceDescription& InResourceDescription)
{
	return StartNewCapture(InResourceDescription.ResourceSize, InResourceDescription.PixelFormat);
}

void UAvaBroadcastRenderTargetMediaCapture::StopCaptureImpl(bool /*bAllowPendingFrameToBeProcess*/)
{
	TRACE_BOOKMARK(TEXT("UAvaBroadcastRenderTargetMediaCapture::StopCapture"));
}

bool UAvaBroadcastRenderTargetMediaCapture::StartNewCapture(const FIntPoint& InSourceTargetSize, EPixelFormat InSourceTargetFormat)
{
	TRACE_BOOKMARK(TEXT("UAvaBroadcastRenderTargetMediaCapture::StartNewCapture"));
	{
		FScopeLock ScopeLock(&RenderTargetCriticalSection);

		RenderTarget = nullptr;
		
		const UAvaBroadcastRenderTargetMediaOutput* AvaRenderTargetMediaOutput = CastChecked<UAvaBroadcastRenderTargetMediaOutput>(MediaOutput);
		if (AvaRenderTargetMediaOutput)
		{
			RenderTarget = AvaRenderTargetMediaOutput->RenderTarget.LoadSynchronous();
			if (RenderTarget)
			{
				SetState(EMediaCaptureState::Capturing);
				return true;
			}
			else
			{
				UE_LOGF(LogAvaBroadcastRenderTargetMedia, Error, "Missing render target object: \"%ls\".", *AvaRenderTargetMediaOutput->RenderTarget.ToString());
			}
		}
		else
		{
			UE_LOGF(LogAvaBroadcastRenderTargetMedia, Error, "Internal Error: Media Capture's associated Media Output is not of type \"UAvaBroadcastRenderTargetMediaOutput\".");
		}
	}
	SetState(EMediaCaptureState::Error);
	return false;
}
