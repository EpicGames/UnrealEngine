// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/CriticalSection.h"
#include "MediaCapture.h"
#include "AvaBroadcastRenderTargetMediaCapture.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAvaBroadcastRenderTargetMedia, Log, All);

UCLASS()
class UAvaBroadcastRenderTargetMediaCapture : public UMediaCapture
{
	GENERATED_BODY()

protected:
	virtual bool ShouldCaptureRHIResource() const override { return true; }
	virtual void OnRHIResourceCaptured_RenderingThread(FRHICommandListImmediate& RHICmdList, const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FTextureRHIRef InTexture) override;

	virtual bool InitializeCapture() override;
	virtual bool PostInitializeCaptureViewport(TSharedPtr<FSceneViewport>& InSceneViewport) override;
	virtual bool PostInitializeCaptureRenderTarget(UTextureRenderTarget2D* InRenderTarget) override;
	virtual bool PostInitializeCaptureRHIResource(const FRHICaptureResourceDescription& InResourceDescription) override;
	virtual void StopCaptureImpl(bool bAllowPendingFrameToBeProcess) override;

	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> RenderTarget;

private:
	bool StartNewCapture(const FIntPoint& InSourceTargetSize, EPixelFormat InSourceTargetFormat);
	
	FCriticalSection RenderTargetCriticalSection;
};
