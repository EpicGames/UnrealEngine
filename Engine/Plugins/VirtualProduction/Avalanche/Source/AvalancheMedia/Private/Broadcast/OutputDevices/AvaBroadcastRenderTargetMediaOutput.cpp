// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/OutputDevices/AvaBroadcastRenderTargetMediaOutput.h"

#include "Engine/TextureRenderTarget2D.h"
#include "Broadcast/OutputDevices/AvaBroadcastRenderTargetMediaCapture.h"

bool UAvaBroadcastRenderTargetMediaOutput::Validate(FString& OutFailureReason) const
{
	if (!Super::Validate(OutFailureReason))
	{
		return false;
	}

	const UTextureRenderTarget2D* LoadedRenderTarget = RenderTarget.LoadSynchronous();
	if (!LoadedRenderTarget)
	{
		OutFailureReason = FString::Printf(TEXT("Missing render target object: \"%s\"."), *RenderTarget.ToString());
		return false;
	}
	return true;
}

FIntPoint UAvaBroadcastRenderTargetMediaOutput::GetRequestedSize() const
{
	const UTextureRenderTarget2D* LoadedRenderTarget = RenderTarget.LoadSynchronous();
	return (LoadedRenderTarget) ? FIntPoint(LoadedRenderTarget->SizeX, LoadedRenderTarget->SizeY) : RequestCaptureSourceSize;
}


EPixelFormat UAvaBroadcastRenderTargetMediaOutput::GetRequestedPixelFormat() const
{
	const UTextureRenderTarget2D* LoadedRenderTarget = RenderTarget.LoadSynchronous();
	return (LoadedRenderTarget) ? LoadedRenderTarget->GetFormat() : EPixelFormat::PF_Unknown;
}

EMediaCaptureConversionOperation UAvaBroadcastRenderTargetMediaOutput::GetConversionOperation(EMediaCaptureSourceType /*InSourceType*/) const
{
	return bInvertKeyOutput ? EMediaCaptureConversionOperation::INVERT_ALPHA : EMediaCaptureConversionOperation::NONE;
}

#if WITH_EDITOR
void UAvaBroadcastRenderTargetMediaOutput::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	// The source name is used to display as the "device name" in the broadcast editor.
	SourceName = RenderTarget.GetAssetName();
	
	// Fallback in case the path is not set.
	if (SourceName.IsEmpty())
	{
		SourceName = GetFName().ToString();
	}
	
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

UMediaCapture* UAvaBroadcastRenderTargetMediaOutput::CreateMediaCaptureImpl()
{
	UMediaCapture* Result = NewObject<UAvaBroadcastRenderTargetMediaCapture>();
	if (Result)
	{
		UE_LOGF(LogAvaBroadcastRenderTargetMedia, Log, "Created Motion Design Render Target Media Capture");
		Result->SetMediaOutput(this);
	}
	return Result;
}
