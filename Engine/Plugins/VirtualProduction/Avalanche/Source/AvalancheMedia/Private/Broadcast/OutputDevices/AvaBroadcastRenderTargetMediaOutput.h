// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaOutput.h"
#include "UObject/SoftObjectPtr.h"
#include "AvaBroadcastRenderTargetMediaOutput.generated.h"

class UTextureRenderTarget2D;

/**
 * Capture a Media to a render target.
 */
UCLASS(BlueprintType, ClassGroup = "Motion Design Broadcast",
	meta = (DisplayName = "Motion Design Render Target Media Output", MediaIOCustomLayout = "AvaRenderTarget"))
class UAvaBroadcastRenderTargetMediaOutput : public UMediaOutput
{
	GENERATED_BODY()

public:
	/** Invert the key (alpha channel). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Media")
	bool bInvertKeyOutput = true;

	/**
	 * Specify the render target to be capturing to.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Media")
	TSoftObjectPtr<UTextureRenderTarget2D> RenderTarget;

	/**
	 * The source name is a property that exists to provide the "device name" for
	 * displaying in the broadcast editor. It is derived from the specified render target
	 * path. */
	UPROPERTY()
	FString SourceName;
	
	//~ UMediaOutput interface
public:
	virtual bool Validate(FString& FailureReason) const override;
	virtual FIntPoint GetRequestedSize() const override;
	virtual EPixelFormat GetRequestedPixelFormat() const override;
	virtual EMediaCaptureConversionOperation GetConversionOperation(EMediaCaptureSourceType InSourceType) const override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

protected:
	virtual UMediaCapture* CreateMediaCaptureImpl() override;
	//~ End UMediaOutput interface
};
