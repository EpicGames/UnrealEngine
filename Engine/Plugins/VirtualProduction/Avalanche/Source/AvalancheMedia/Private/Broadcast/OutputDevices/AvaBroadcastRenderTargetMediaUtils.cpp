// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/OutputDevices/AvaBroadcastRenderTargetMediaUtils.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/TextureRenderTarget2D.h"
#include "MediaShaders.h"
#include "RenderGraphUtils.h"
#include "RHIStaticStates.h"
#include "ScreenPass.h"
#include "TextureResource.h"
#include "SceneView.h"
#include "ShowFlags.h"

void UE::AvaBroadcastRenderTargetMediaUtils::ClearRenderTarget(UTextureRenderTarget2D* InRenderTarget)
{
	FLinearColor ClearColor = InRenderTarget->ClearColor;
	FTextureRenderTargetResource* RenderTargetResource = InRenderTarget->GameThread_GetRenderTargetResource();
	ENQUEUE_RENDER_COMMAND(FAvaClearRenderTarget)(
		[RenderTargetResource, ClearColor, RenderTargetName = InRenderTarget->GetName()](FRHICommandListImmediate& RHICmdList)
		{
			FRDGBuilder GraphBuilder(RHICmdList, RDG_EVENT_NAME("ClearRenderTarget(%s)", *RenderTargetName));
			const FRDGTextureRef OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(RenderTargetResource->TextureRHI, *RenderTargetName));
			AddClearRenderTargetPass(GraphBuilder, OutputTexture, ClearColor);
			GraphBuilder.Execute();
		});
}

void UE::AvaBroadcastRenderTargetMediaUtils::CopyTexture(FRHICommandListImmediate& InRHICmdList, FTextureRHIRef InSourceTexture, FTextureRHIRef InDestTarget)
{
	if (!InSourceTexture.IsValid() || !InDestTarget.IsValid())
	{
		return;
	}
	
	FRDGBuilder GraphBuilder(InRHICmdList);

	FRDGTexture* InputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(InSourceTexture, TEXT("SourceTexture")));
	FRDGTexture* OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(InDestTarget, TEXT("DestTarget")));
	const FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	
	if (InputTexture->Desc.Format == OutputTexture->Desc.Format &&
		InputTexture->Desc.Extent.X == OutputTexture->Desc.Extent.X &&
		InputTexture->Desc.Extent.Y == OutputTexture->Desc.Extent.Y)
	{
		// The formats are the same and size are the same. simple copy
		// Note: this deals with differing formats.
		// Note2: also deals with mipmaps if any.
		AddDrawTexturePass(GraphBuilder, GlobalShaderMap, InputTexture, OutputTexture, FRDGDrawTextureInfo());
	}
	else
	{
		// The formats or sizes differ to pixel shader stuff
		// Configure source/output viewport to get the right UV scaling from source texture to output texture
		const FScreenPassTextureViewport InputViewport(InputTexture);
		const FScreenPassTextureViewport OutputViewport(OutputTexture);

		const TShaderMapRef<FScreenPassVS> VertexShader(GlobalShaderMap);

		// In cases where texture is converted from a format that doesn't have A channel, we want to force set it to 1.
		constexpr int32 ConversionOperation = 0; // None
		FModifyAlphaSwizzleRgbaPS::FPermutationDomain PermutationVector;
		PermutationVector.Set<FModifyAlphaSwizzleRgbaPS::FConversionOp>(ConversionOperation);

		// Rectangle area to use from source
		const FIntRect ViewRect(FIntPoint(0, 0), InputTexture->Desc.Extent);

		//Dummy ViewFamily/ViewInfo created to use built in Draw Screen/Texture Pass
		const FSceneViewFamily ViewFamily(FSceneViewFamily::ConstructionValues(nullptr, nullptr, FEngineShowFlags(ESFIM_Game))
			.SetTime(FGameTime()));
		FSceneViewInitOptions ViewInitOptions;
		ViewInitOptions.ViewFamily = &ViewFamily;
		ViewInitOptions.SetViewRectangle(ViewRect);
		ViewInitOptions.ViewOrigin = FVector::ZeroVector;
		ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
		ViewInitOptions.ProjectionMatrix = FMatrix::Identity;
		FSceneView ViewInfo(ViewInitOptions);

		const TShaderMapRef<FModifyAlphaSwizzleRgbaPS> PixelShader(GlobalShaderMap, PermutationVector);
		FModifyAlphaSwizzleRgbaPS::FParameters* Parameters = PixelShader->AllocateAndSetParameters(GraphBuilder, InputTexture, OutputTexture);
		AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("AvaRenderTargetMediaSwizzle"), ViewInfo, OutputViewport, InputViewport, VertexShader, PixelShader, Parameters);
	}
	GraphBuilder.Execute();
}

namespace UE::AvaBroadcastRenderTargetMediaUtils::Private
{
	class FAvaRGBGammaConvertPS : public FGlobalShader
	{
		DECLARE_SHADER_TYPE(FAvaRGBGammaConvertPS, Global);
		SHADER_USE_PARAMETER_STRUCT(FAvaRGBGammaConvertPS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int, SrgbToLinear)
		SHADER_PARAMETER(float, Gamma)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()
	
	public:
		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
		{
			return IsFeatureLevelSupported(InParameters.Platform, ERHIFeatureLevel::ES3_1);
		}
		
		FParameters* AllocateAndSetParameters(FRDGBuilder& InGraphBuilder, FRDGTextureRef InRGBATexture, bool bInSrgbToLinear, float InGamma, FRDGTextureRef InOutputTexture)
		{
			FParameters* Parameters = InGraphBuilder.AllocParameters<FParameters>();
			Parameters->InputTexture = InRGBATexture;
			Parameters->InputSampler = TStaticSamplerState<SF_Bilinear>::GetRHI(); // or  SF_Point
			Parameters->SrgbToLinear = bInSrgbToLinear ? 1 : 0;
			Parameters->Gamma = InGamma;
			Parameters->RenderTargets[0] = FRenderTargetBinding{ InOutputTexture, ERenderTargetLoadAction::ENoAction };
			return Parameters;
		}
	};

	static const FString GammaConvertShaderPath = FString(VirtualShaderMountPoint) + TEXT("/AvaRGBGammaConvert.usf");
	IMPLEMENT_SHADER_TYPE(, FAvaRGBGammaConvertPS,  *GammaConvertShaderPath, TEXT("MainPS"), SF_Pixel);
}

void UE::AvaBroadcastRenderTargetMediaUtils::ConvertTextureRGBGamma(FRHICommandListImmediate& InRHICmdList, FTextureRHIRef InSourceTexture, FTextureRHIRef InDestTarget, bool bInSrgbToLinear, float InGamma)
{
	FRDGBuilder GraphBuilder(InRHICmdList);

	FRDGTexture* InputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(InSourceTexture, TEXT("SourceTexture")));
	FRDGTexture* OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(InDestTarget, TEXT("DestTarget")));
	const FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	
	// The formats or sizes differ to pixel shader stuff
	// Configure source/output viewport to get the right UV scaling from source texture to output texture
	const FScreenPassTextureViewport InputViewport(InputTexture);
	const FScreenPassTextureViewport OutputViewport(OutputTexture);

	const TShaderMapRef<FScreenPassVS> VertexShader(GlobalShaderMap);
	
	// Rectangle area to use from source
	const FIntRect ViewRect(FIntPoint(0, 0), InputTexture->Desc.Extent);

	//Dummy ViewFamily/ViewInfo created to use built in Draw Screen/Texture Pass
	const FSceneViewFamily ViewFamily(FSceneViewFamily::ConstructionValues(nullptr, nullptr, FEngineShowFlags(ESFIM_Game))
		.SetTime(FGameTime()));
	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.ViewFamily = &ViewFamily;
	ViewInitOptions.SetViewRectangle(ViewRect);
	ViewInitOptions.ViewOrigin = FVector::ZeroVector;
	ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
	ViewInitOptions.ProjectionMatrix = FMatrix::Identity;
	FSceneView ViewInfo(ViewInitOptions);

	const TShaderMapRef<Private::FAvaRGBGammaConvertPS> PixelShader(GlobalShaderMap);
	Private::FAvaRGBGammaConvertPS::FParameters* Parameters = PixelShader->AllocateAndSetParameters(GraphBuilder, InputTexture, bInSrgbToLinear, InGamma, OutputTexture);
	AddDrawScreenPass(GraphBuilder, RDG_EVENT_NAME("ConvertTextureRGBGamma"), ViewInfo, OutputViewport, InputViewport, VertexShader, PixelShader, Parameters);
	
	GraphBuilder.Execute();
}
