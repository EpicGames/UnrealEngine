// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaGameViewportMediaCapture.h"

#include "AvaGameViewportMediaOutput.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Slate/SceneViewport.h"
#include "TextureResource.h"

bool UAvaGameViewportMediaCapture::InitializeCapture()
{
	return true;
}

bool UAvaGameViewportMediaCapture::PostInitializeCaptureViewport(TSharedPtr<FSceneViewport>& InSceneViewport)
{
	SetState(EMediaCaptureState::Capturing);
	return true;
}

bool UAvaGameViewportMediaCapture::PostInitializeCaptureRenderTarget(UTextureRenderTarget2D* InRenderTarget)
{
	SetState(EMediaCaptureState::Capturing);
	return true;
}