// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/CriticalSection.h"
#include "MediaCapture.h"

#include "AvaGameViewportMediaCapture.generated.h"

UCLASS()
class UAvaGameViewportMediaCapture : public UMediaCapture
{
	GENERATED_BODY()

protected:
	//~ Begin UMediaCapture
	virtual bool ShouldCaptureRHIResource() const override { return true; }
	virtual bool InitializeCapture() override;
	virtual bool PostInitializeCaptureViewport(TSharedPtr<FSceneViewport>& InSceneViewport) override;
	virtual bool PostInitializeCaptureRenderTarget(UTextureRenderTarget2D* InRenderTarget) override;
	//~ End UMediaCapture
};
