// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaGameViewportMediaOutput.h"

#include "AvaGameViewportMediaCapture.h"

DEFINE_LOG_CATEGORY(LogAvaGameViewportMedia);

UMediaCapture* UAvaGameViewportMediaOutput::CreateMediaCaptureImpl()
{
	UMediaCapture* Result = NewObject<UAvaGameViewportMediaCapture>();
	if (Result)
	{
		Result->SetMediaOutput(this);
	}
	return Result;
}