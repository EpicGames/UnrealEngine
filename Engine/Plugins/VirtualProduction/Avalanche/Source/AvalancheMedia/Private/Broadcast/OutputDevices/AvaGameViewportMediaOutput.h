// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaCapture.h"
#include "AvaGameViewportMediaOutput.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAvaGameViewportMedia, Log, All);

UCLASS(BlueprintType, ClassGroup = "Motion Design Broadcast",
	meta = (DisplayName = "Motion Design Game Viewport Media Output", MediaIOCustomLayout = "AvaGameViewport"))
class UAvaGameViewportMediaOutput : public UMediaOutput
{
	GENERATED_BODY()

	//~ Begin UMediaOutput
	public:
	virtual FIntPoint GetRequestedSize() const override { return UMediaOutput::RequestCaptureSourceSize; }
	virtual EPixelFormat GetRequestedPixelFormat() const override { return EPixelFormat::PF_B8G8R8A8; }
	virtual EMediaCaptureConversionOperation GetConversionOperation(EMediaCaptureSourceType InSourceType) const override { return EMediaCaptureConversionOperation::NONE; }
protected:
	virtual UMediaCapture* CreateMediaCaptureImpl() override;
	//~ End UMediaOutput

public:
	/**
	 * The source name is a property that exists to provide the "device name" for
	 * displaying in the broadcast editor. 
	 */
	UPROPERTY()
	FString SourceName = TEXT("Game Viewport");
};