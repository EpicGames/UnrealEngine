// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/OutputDevices/Slate/SAvaBroadcastCaptureImage.h"
#include "Layout/FlowDirection.h"
#include "Layout/Geometry.h"
#include "Layout/SlateRect.h"
#include "Math/TransformCalculus2D.h"
#include "Rendering/DrawElements.h"
#include "Rendering/DrawElementTypes.h"
#include "Rendering/SlateRenderTransform.h"
#include "Styling/SlateBrush.h"
#include "Styling/WidgetStyle.h"
#include "Types/PaintArgs.h"
#include "UObject/NoExportTypes.h"

void SAvaBroadcastCaptureImage::Construct(const FArguments& InArgs)
{
	ShouldInvertAlpha = InArgs._ShouldInvertAlpha;
	bEnableGammaCorrection = InArgs._EnableGammaCorrection;
	EnableBlending = InArgs._EnableBlending;
	
	SImage::Construct(InArgs._ImageArgs);
}

int32 SAvaBroadcastCaptureImage::OnPaint(const FPaintArgs& Args
	, const FGeometry& AllottedGeometry
	, const FSlateRect& MyCullingRect
	, FSlateWindowElementList& OutDrawElements
	, int32 LayerId
	, const FWidgetStyle& InWidgetStyle
	, bool bParentEnabled) const
{
	const FSlateBrush* const ImageBrush = GetImageAttribute().Get();
	
	if (ImageBrush && ImageBrush->DrawAs != ESlateBrushDrawType::NoDrawType)
	{
		const bool bIsEnabled = ShouldBeEnabled(bParentEnabled);
		
		ESlateDrawEffect DrawEffects = bIsEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

		if (ShouldInvertAlpha.IsSet() && ShouldInvertAlpha.Get())
		{
			DrawEffects |= ESlateDrawEffect::InvertAlpha;
		}

		if (EnableBlending.IsSet() && EnableBlending.Get() == false)
		{
			DrawEffects |= ESlateDrawEffect::NoBlending;
		}

		if (!bEnableGammaCorrection)
		{
			DrawEffects |= ESlateDrawEffect::NoGamma;
		}
		
		const FLinearColor FinalColorAndOpacity(InWidgetStyle.GetColorAndOpacityTint()
			* GetColorAndOpacityAttribute().Get().GetColor(InWidgetStyle)
			* ImageBrush->GetTint(InWidgetStyle));

		if (bFlipForRightToLeftFlowDirection && GSlateFlowDirection == EFlowDirection::RightToLeft)
		{
			const FGeometry FlippedGeometry = AllottedGeometry.MakeChild(FSlateRenderTransform(FScale2D(-1, 1)));
			
			FSlateDrawElement::MakeBox(OutDrawElements
				, LayerId
				, FlippedGeometry.ToPaintGeometry()
				, ImageBrush
				, DrawEffects
				, FinalColorAndOpacity);
		}
		else
		{
			FSlateDrawElement::MakeBox(OutDrawElements
				, LayerId
				, AllottedGeometry.ToPaintGeometry()
				, ImageBrush
				, DrawEffects
				, FinalColorAndOpacity);
		}
	}

	return LayerId;
}

