// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaBroadcastOutputTreeItem.h"

#include "Broadcast/OutputDevices/AvaBroadcastOutputTreeItem.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

void SAvaBroadcastOutputTreeItem::Construct(const FArguments& InArgs, const TSharedPtr<FAvaBroadcastOutputTreeItem>& InOutputTreeItem)
{
	check(InOutputTreeItem.IsValid());
	OutputTreeItemWeak = InOutputTreeItem;

	ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SImage)
				.Image(InOutputTreeItem->GetIconBrush())
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.Padding(5.f, 0.f, 0.f, 0.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(InOutputTreeItem->GetDisplayName())
			]
		];
}
