// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FAvaBroadcastOutputTreeItem;

class SAvaBroadcastOutputTreeItem : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaBroadcastOutputTreeItem){}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedPtr<FAvaBroadcastOutputTreeItem>& InOutputTreeItem);
	
protected:
	TWeakPtr<FAvaBroadcastOutputTreeItem> OutputTreeItemWeak;
};
