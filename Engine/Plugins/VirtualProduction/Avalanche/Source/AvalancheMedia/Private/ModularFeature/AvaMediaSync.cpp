// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMediaSync.h"

#include "Features/IModularFeatures.h"
#include "IAvaMediaModule.h"
#include "ModularFeature/IAvaMediaSyncProvider.h"

FAvaMediaSync::FAvaMediaSync()
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	ModularFeatures.OnModularFeatureRegistered().AddRaw(this, &FAvaMediaSync::HandleModularFeatureRegistered);
	ModularFeatures.OnModularFeatureUnregistered().AddRaw(this, &FAvaMediaSync::HandleModularFeatureUnregistered);
	UseProvider(IAvaMediaSyncProvider::Get());
	RefreshFeatureAvailability();
}

FAvaMediaSync::~FAvaMediaSync()
{
	UseProvider(nullptr);
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	ModularFeatures.OnModularFeatureRegistered().RemoveAll(this);
	ModularFeatures.OnModularFeatureUnregistered().RemoveAll(this);
}

void FAvaMediaSync::SetFeatureEnabled(bool bInIsFeatureEnabled)
{
	bIsFeatureEnabled = bInIsFeatureEnabled;
	
	UE_LOGF(LogAvaMedia, Log, "Modular Feature %ls is %ls.",
		*IAvaMediaSyncProvider::GetModularFeatureName().ToString(),
		bInIsFeatureEnabled ? TEXT("enabled") : TEXT("disabled"));
}

IAvaMediaSyncProvider* FAvaMediaSync::GetCurrentProvider()
{
	if (!bIsFeatureEnabled)
	{
		return nullptr;
	}
	
	if (!CurrentProvider && IsFeatureAvailable())
	{
		UseProvider(IAvaMediaSyncProvider::Get());
	}
	return CurrentProvider;
}

int32 FAvaMediaSync::GetModularFeatureImplementationCount()
{
	IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;
	return IModularFeatures::Get().GetModularFeatureImplementationCount(IAvaMediaSyncProvider::GetModularFeatureName());
}

void FAvaMediaSync::UseProvider(IAvaMediaSyncProvider* InSyncProvider)
{
	if (CurrentProvider)
	{
		CurrentProvider->GetOnAvaSyncPackageModified().RemoveAll(this);
	}	
	if (InSyncProvider)
	{
		InSyncProvider->GetOnAvaSyncPackageModified().AddRaw(this, &FAvaMediaSync::HandleAvaSyncPackageModified);
	}
	if (InSyncProvider != CurrentProvider)
	{
		CurrentProvider = InSyncProvider;
		IAvaMediaModule::Get().GetOnAvaMediaSyncProviderChanged().Broadcast(CurrentProvider);
		
		if (CurrentProvider)
		{
			UE_LOGF(LogAvaMedia, Display, "Using Modular Feature %ls Implementation: \"%ls\".",
				*IAvaMediaSyncProvider::GetModularFeatureName().ToString(), *CurrentProvider->GetName().ToString());
		}
	}
}

void FAvaMediaSync::RefreshFeatureAvailability()
{
	bIsFeatureAvailable = GetModularFeatureImplementationCount() > 0 ? true : false;
	if (!bIsFeatureAvailable)
	{
		UE_LOGF(LogAvaMedia, Display, "No %ls Implementations available. Feature Disabled.",
			*IAvaMediaSyncProvider::GetModularFeatureName().ToString());
	}
}

void FAvaMediaSync::HandleModularFeatureRegistered(const FName& InFeatureName, IModularFeature* InFeature)
{
	if (InFeatureName == IAvaMediaSyncProvider::GetModularFeatureName())
	{
		UseProvider(static_cast<IAvaMediaSyncProvider*>(InFeature));
		bIsFeatureAvailable = true;
	}
}

void FAvaMediaSync::HandleModularFeatureUnregistered(const FName& InFeatureName, IModularFeature* InFeature)
{
	if (InFeatureName == IAvaMediaSyncProvider::GetModularFeatureName())
	{
		if (CurrentProvider == InFeature)
		{
			// Lazy deinit. Note that GetCurrentProvider() will try to acquire a new one on demand.
			UseProvider(nullptr);
		}		
		RefreshFeatureAvailability();
	}
}

void FAvaMediaSync::HandleAvaSyncPackageModified(const FName& InPackageName)
{
	IAvaMediaModule::Get().GetOnAvaMediaSyncPackageModified().Broadcast(CurrentProvider, InPackageName);
}