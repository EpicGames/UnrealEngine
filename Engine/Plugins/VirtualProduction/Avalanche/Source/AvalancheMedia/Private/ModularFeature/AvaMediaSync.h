// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class FName;
class IAvaMediaSyncProvider;
class IModularFeature;

/**
 * Helper class for modular feature IAvaMediaSyncProvider's events.
 * The purpose of this class is to maintain a relay with the currently
 * available IAvaMediaSyncProvider's events and the corresponding
 * IAvaMediaModule's events. The complexity of maintaining proper event
 * registration with IAvaMediaSyncProvider is encapsulated in this class.
 */
class FAvaMediaSync
{
public:
	FAvaMediaSync();
	~FAvaMediaSync();

	/**
	 * Returns true if the feature is enabled.
	 * This is independent than the feature available.
	 */
	bool IsFeatureEnabled() const { return bIsFeatureEnabled; }

	/**
	 * The feature is enabled by default, but can be disabled even if it is available.
	 * The use case of disabling the media sync feature is if a local server process is
	 * launched in game mode from the same folder as the client.
	 */
	void SetFeatureEnabled(bool bInIsFeatureEnabled);
	
	/**
	 *	Returns the currently available Media Sync Provider or null if feature is disabled.
	 */
	IAvaMediaSyncProvider* GetCurrentProvider();
	
	/**
	 *	Returns true if the IAvaMediaSyncProvider feature is available,
	 *	i.e. if it has at least one implementation. 
	 */
	bool IsFeatureAvailable() const { return bIsFeatureAvailable; }

	/**
	 * Returns the number of IAvaMediaSyncProvider implementations registered.
	 * remark: This call is thread safe.
	 */
	static int32 GetModularFeatureImplementationCount();

private:
	void UseProvider(IAvaMediaSyncProvider* InSyncProvider);
	void RefreshFeatureAvailability();
	
	// Delegate handlers
	void HandleModularFeatureRegistered(const FName& InFeatureName, IModularFeature* InFeature);
	void HandleModularFeatureUnregistered(const FName& InFeatureName, IModularFeature* InFeature);
	void HandleAvaSyncPackageModified(const FName& InPackageName);
	
	IAvaMediaSyncProvider* CurrentProvider = nullptr;
	bool bIsFeatureAvailable = false;
	bool bIsFeatureEnabled = true;
};