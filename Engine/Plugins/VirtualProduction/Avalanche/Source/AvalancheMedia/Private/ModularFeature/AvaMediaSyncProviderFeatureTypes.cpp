// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModularFeature/AvaMediaSyncProviderFeatureTypes.h"

#include "Engine/Engine.h"
#include "Misc/App.h"
#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "AvaMediaSyncProviderFeatureTypes"

FAvaMediaSyncConnectionInfo::FAvaMediaSyncConnectionInfo()
	: EngineVersion(FNetworkVersion::GetLocalNetworkVersion())
	, InstanceId(FApp::GetInstanceId())
	, SessionId(FApp::GetSessionId())
	, HostName(FPlatformProcess::ComputerName())
	, ProjectName(FApp::GetProjectName())
{
	// We only want the dirname, not absolute path
	ProjectDir = GetBasename(FPaths::ProjectDir());

	// Figure out which type of instance it is
	if (GEngine == nullptr)
	{
		InstanceType = EAvaMediaSyncEngineType::Unknown;
	}
	else if (IsRunningDedicatedServer())
	{
		InstanceType = EAvaMediaSyncEngineType::Server;
	}
	else if (IsRunningCommandlet())
	{
		InstanceType = EAvaMediaSyncEngineType::Commandlet;
	}
	else if (GEngine->IsEditor())
	{
		InstanceType = EAvaMediaSyncEngineType::Editor;
	}
	else if (IsRunningGame())
	{
		InstanceType = EAvaMediaSyncEngineType::Game;
	}
	else
	{
		InstanceType = EAvaMediaSyncEngineType::Other;
	}	
}

FString FAvaMediaSyncConnectionInfo::ToString() const
{
	return FString::Printf(
		TEXT("EngineVersion: %d, HostName: %s, InstanceId: %s, InstanceType: %s, SessionId: %s, ProjectName: %s, ProjectDir: %s"),
		EngineVersion,
		*HostName,
		*InstanceId.ToString(),
		*UEnum::GetValueAsString(InstanceType),
		*SessionId.ToString(),
		*ProjectName,
		*ProjectDir
	);
}

FString FAvaMediaSyncConnectionInfo::GetBasename(const FString& InPath)
{
	FString LocalPath = InPath;
	LocalPath.RemoveFromEnd(TEXT("/"));
	return FPaths::GetBaseFilename(LocalPath);
}

FText FAvaMediaSyncConnectionInfo::GetHumanReadableInstanceType() const
{
	FText Result;

	switch (InstanceType)
	{
	case EAvaMediaSyncEngineType::Server:
		Result = LOCTEXT("Instance_Type_Server", "Server");
		break;
	case EAvaMediaSyncEngineType::Commandlet:
		Result = LOCTEXT("Instance_Type_Commandlet", "Commandlet");
		break;
	case EAvaMediaSyncEngineType::Editor:
		Result = LOCTEXT("Instance_Type_Editor", "Editor");
		break;
	case EAvaMediaSyncEngineType::Game:
		Result = LOCTEXT("Instance_Type_Game", "Game");
		break;
	case EAvaMediaSyncEngineType::Other:
		Result = LOCTEXT("Instance_Type_Other", "Other");
		break;
	case EAvaMediaSyncEngineType::Unknown:
		Result = LOCTEXT("Instance_Type_Unknown", "Unknown");
		break;
	default: break;
	}
	
	return Result;
}

#undef LOCTEXT_NAMESPACE
