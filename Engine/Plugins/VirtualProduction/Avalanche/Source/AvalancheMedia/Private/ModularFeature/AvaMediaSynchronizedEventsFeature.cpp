// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModularFeature/AvaMediaSynchronizedEventsFeature.h"

#include "AvaMediaSettings.h"
#include "Features/IModularFeatures.h"
#include "HAL/IConsoleManager.h"
#include "IAvaMediaModule.h"
#include "ModularFeature/AvaMediaSynchronizedEvent.h"
#include "ModularFeature/IAvaMediaSynchronizedEventDispatcher.h"

#define LOCTEXT_NAMESPACE "AvaMediaSynchronizedEventsFeature"

namespace UE::AvaMedia::SynchronizedEvents::Private
{
	// Allow events to be dispatched as early as possible (at most 1 frame earlier).
	TAutoConsoleVariable<bool> CVarSyncEarlyDispatch(
		TEXT("AvaMediaSynchronizedEvent.NoSync.EarlyDispatch")
		, false
		, TEXT("If true, will dispatch events as soon as ready. if false, ready events are all batched on the next tick."), ECVF_Cheat);
}

/**
 * Provide a default no-sync implementation that fires the queued events on the next tick (*).
 * 
 * (*) Note: could fire the event either immediately, i.e. when it is made, or on the next dispatch update.
 * Sync events should tolerate to not be executed immediately (not all events are like that). So deferring the
 * invoke to next Tick helps identify such issues. Only events that the invoke can be deferred can be used with
 * this synchronisation mechanism.
 */
class FAvaMediaSynchronizedEventDispatcher : public IAvaMediaSynchronizedEventDispatcher
{
public:
	virtual bool PushEvent(FString&& InEventSignature, TUniqueFunction<void()> InFunction) override
	{
		check(IsInGameThread());
		
		if (!InFunction)
		{
			return false;
		}
		
		if (UE::AvaMedia::SynchronizedEvents::Private::CVarSyncEarlyDispatch.GetValueOnAnyThread())
		{
			InFunction();
		}
		else
		{
			Events.Add(MakeUnique<FAvaMediaSynchronizedEvent>(MoveTemp(InEventSignature), MoveTemp(InFunction)));	
		}
		return true;
	}

	virtual EAvaMediaSynchronizedEventState GetEventState(const FString& InEventSignature) const override
	{
		return EAvaMediaSynchronizedEventState::NotFound;
	}

	virtual void DispatchEvents() override
	{
		check(IsInGameThread());
		
		for (const TUniquePtr<FAvaMediaSynchronizedEvent>& Event : Events)
		{
			Event->Function();
		}
		Events.Reset();
	}

	TArray<TUniquePtr<FAvaMediaSynchronizedEvent>> Events;
};

class FAvaMediaSynchronizedEventsNoSync : public IAvaMediaSynchronizedEventsFeature
{
	virtual FName GetName() const override
	{
		static const FName ImplementationName(TEXT("NoSync"));
		return ImplementationName;
	}

	virtual FText GetDisplayName() const override
	{
		static const FText DisplayName = LOCTEXT("NoSyncDisplayName", "No Sync");
		return DisplayName;
	}

	virtual FText GetDisplayDescription() const override
	{
		static const FText DisplayDescription = LOCTEXT("NoSyncDisplayDescription",
			 "This implementation does not perform synchronisation. "
			 "Events are either executed immedially or on the next frame depending on the configuration.");
		return DisplayDescription;
	}

	virtual int32 GetPriority() const override
	{
		return 0; // So any other implementation will have priority.
	}
	
	virtual TSharedPtr<IAvaMediaSynchronizedEventDispatcher> CreateDispatcher(const FString& InSignature) override
	{
		return MakeShared<FAvaMediaSynchronizedEventDispatcher>();
	}
};

void FAvaMediaSynchronizedEventsFeature::Startup()
{
	IModularFeatures::Get().RegisterModularFeature(IAvaMediaSynchronizedEventsFeature::GetModularFeatureName(), GetInternalImplementation());
}

void FAvaMediaSynchronizedEventsFeature::Shutdown()
{
	IModularFeatures::Get().UnregisterModularFeature(IAvaMediaSynchronizedEventsFeature::GetModularFeatureName(), GetInternalImplementation());	
}

IAvaMediaSynchronizedEventsFeature* FAvaMediaSynchronizedEventsFeature::Get()
{
	const FName SelectedImplementation(IAvaMediaModule::Get().GetPlayableSettings().SynchronizedEventsFeature.Implementation);
	
	IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;
	return FindImplementation(SelectedImplementation);
}

TSharedPtr<IAvaMediaSynchronizedEventDispatcher> FAvaMediaSynchronizedEventsFeature::CreateDispatcher(const FString& InSignature)
{
	if (IAvaMediaSynchronizedEventsFeature* Feature = Get())
	{
		return Feature->CreateDispatcher(InSignature);
	}

	// Fallback returns the no-sync dispatcher implementation directly.
	return MakeShared<FAvaMediaSynchronizedEventDispatcher>();
}

void FAvaMediaSynchronizedEventsFeature::EnumerateImplementations(TFunctionRef<void(const IAvaMediaSynchronizedEventsFeature*)> InCallback)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	static const FName FeatureName = IAvaMediaSynchronizedEventsFeature::GetModularFeatureName();

	const int32 NumImplementations = ModularFeatures.GetModularFeatureImplementationCount(FeatureName);
	for (int32 ImplementationIndex = 0; ImplementationIndex < NumImplementations; ++ImplementationIndex)
	{
		const IAvaMediaSynchronizedEventsFeature* Implementation
			= static_cast<IAvaMediaSynchronizedEventsFeature*>(ModularFeatures.GetModularFeatureImplementation(FeatureName, ImplementationIndex));
		InCallback(Implementation);
	}
}

IAvaMediaSynchronizedEventsFeature* FAvaMediaSynchronizedEventsFeature::FindImplementation(FName InImplementation)
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	static const FName FeatureName = IAvaMediaSynchronizedEventsFeature::GetModularFeatureName();

	const int32 NumImplementations = ModularFeatures.GetModularFeatureImplementationCount(FeatureName);
	
	if (InImplementation == UAvaMediaSettings::SynchronizedEventsFeatureSelection_Default)
	{
		// Automatic selection of highest priority implementation.
		IAvaMediaSynchronizedEventsFeature* SelectedImplementation = nullptr;
		int SelectedPriority = 0;
		
		for (int32 ImplementationIndex = 0; ImplementationIndex < NumImplementations; ++ImplementationIndex)
		{
			IAvaMediaSynchronizedEventsFeature* Implementation
				= static_cast<IAvaMediaSynchronizedEventsFeature*>(ModularFeatures.GetModularFeatureImplementation(FeatureName, ImplementationIndex));
			if (Implementation && (!SelectedImplementation || Implementation->GetPriority() > SelectedPriority))
			{
				SelectedPriority = Implementation->GetPriority();
				SelectedImplementation = Implementation;
			}
		}

		if (SelectedImplementation)
		{
			return SelectedImplementation;
		}
	}
	else
	{
		// Selection by explicit name
		for (int32 ImplementationIndex = 0; ImplementationIndex < NumImplementations; ++ImplementationIndex)
		{
			IAvaMediaSynchronizedEventsFeature* Implementation
				= static_cast<IAvaMediaSynchronizedEventsFeature*>(ModularFeatures.GetModularFeatureImplementation(FeatureName, ImplementationIndex));
			if (Implementation && Implementation->GetName() == InImplementation)
			{
				return Implementation;
			}
		}
	}

	// Fallback to the no-sync implementation.
	return GetInternalImplementation();
}

IAvaMediaSynchronizedEventsFeature* FAvaMediaSynchronizedEventsFeature::GetInternalImplementation()
{
	static FAvaMediaSynchronizedEventsNoSync SynchronizedEventsNoSync;
	return &SynchronizedEventsNoSync;
}

#undef LOCTEXT_NAMESPACE