// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModularFeature/IAvaMediaSyncProvider.h"

#include "Features/IModularFeatures.h"

// TODO: As a future improvement, we might want to have a setting for a preferred provider.
// TODO: See: USourceCodeAccessSettings for example.
IAvaMediaSyncProvider* IAvaMediaSyncProvider::Get()
{
	IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;
	IModularFeatures& ModularFeatureManager = IModularFeatures::Get();
	if (ModularFeatureManager.GetModularFeatureImplementationCount(GetModularFeatureName()) > 0)
	{
		return &ModularFeatureManager.GetModularFeature<IAvaMediaSyncProvider>(GetModularFeatureName());
	}
	return nullptr;
}
