// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playable/AvaPlayable.h"

#include "AvaPlayableUtils.h"
#include "AvaSequence.h"
#include "AvaSequencePlayer.h"
#include "Broadcast/AvaBroadcast.h"
#include "Controller/RCCustomControllerUtilities.h"
#include "Engine/Engine.h"
#include "Framework/AvaSoftAssetPtr.h"
#include "IAvaSceneInterface.h"
#include "IAvaSequenceProvider.h"
#include "Playable/AvaPlayableGroup.h"
#include "Playable/AvaPlayableRemoteControl.h"
#include "Playable/AvaPlayableRemoteControlPresetInfo.h"
#include "Playable/AvaPlayableRemoteControlValues.h"
#include "Playable/Playables/AvaPlayableLevelStreaming.h"
#include "Playable/Playables/AvaPlayableRemoteProxy.h"
#include "Playback/AvaPlaybackUtils.h"
#include "RCVirtualProperty.h"
#include "RemoteControlPreset.h"

DEFINE_LOG_CATEGORY(LogAvaPlayable);

#define LOCTEXT_NAMESPACE "AvaPlayable"

// Transition logic and Scene State requires the RC controller values to be updated.
static TAutoConsoleVariable<bool> CVarUpdateRCControllerValues(
	TEXT("MotionDesignPlayable.UpdateRCControllerValues"),
	true,
	TEXT("Set the controller values without running the behaviors."),
	ECVF_Default);

// Some "Special" controllers, such as texture controller require the behavior to be executed.
static TAutoConsoleVariable<bool> CVarExecuteSpecialRCControllerBehavior(
	TEXT("MotionDesignPlayable.ExecuteSpecialRCControllerBehavior"),
	true,
	TEXT("Execute the special controller behaviors (such as texture controllers)."),
	ECVF_Default);

// In the pursuit of determinism, we want to restore the state of playables by restoring
// all the entity values, and if possible the controllers, as more and more controllers
// are needed to affect the playable state (texture controllers, scene state, etc).
// We still don't have a guarantee that behaviors are deterministic, but at least, we can know
// if they are overlapping. Controllers with overlapping controlled states can't be executed, but
// we can attempt to execute the non-overlapping ones. This is not yet good enough to be enabled by default,
// but it might be able to solve issues in specific cases. Thus why it is available as a cvar for now.
static TAutoConsoleVariable<bool> CVarExecuteNonOverlappingRCControllerBehavior(
	TEXT("MotionDesignPlayable.ExecuteNonOverlappingRCControllerBehavior"),
	false,
	TEXT("Execute all non-overlapping controller behaviors."),
	ECVF_Default);

#if WITH_EDITOR
// UActorComponent::PreEditChange calls FlushRenderingCommands() if GFlushRenderingCommandsOnPreEditChange is true (which it is by default)
// Allow Playables to turn it off to reduce FlushRenderingCommands() calls in play time.
static TAutoConsoleVariable<bool> CVarPlayableAllowFlushRenderingCommandsOnPreEditChange(
	TEXT("MotionDesignPlayable.AllowFlushRenderingCommandsOnPreEditChange"),
	false,
	TEXT("Editor only! Determines whether FlushRenderingCommands() is allowed to be called by an Actor Component's PreEditChange when updating RC values in the playable"),
	ECVF_Default);
#endif

namespace UE::AvaPlayable::Private
{
	FString GetPrettyPlayableInfo(const UAvaPlayable* InPlayable)
	{
		if (InPlayable)
		{
			return FString::Printf(TEXT("Id:%s, Asset:%s, Status:%s"),
				*InPlayable->GetInstanceId().ToString(),
				*InPlayable->GetSourceAssetPath().GetAssetName(),
				*AvaPlayback::Utils::StaticEnumToString(InPlayable->GetPlayableStatus()));
		}
		return TEXT("(nullptr)");
	}
	
	FString GetPrettySequenceInfo(const UAvaSequence* InSequence)
	{
		if (InSequence)
		{
			return FString::Printf(TEXT("Name:%s, Label:%s"),
				*InSequence->GetFName().ToString(),
				*InSequence->GetLabel().ToString());
		}
		return TEXT("(nullptr)");
	}		

	FString GetBriefFrameInfo()
	{
		return UE::AvaPlayback::Utils::GetBriefFrameInfo();
	}
	
	FString GetPrettySequenceCommandInfo(EAvaPlaybackAnimAction InAnimAction, const FAvaPlaybackAnimPlaySettings& InAnimPlaySettings)
	{
		return FString::Printf(TEXT("Action:%s, Name:%s"),
			*AvaPlayback::Utils::StaticEnumToString(InAnimAction),
			*InAnimPlaySettings.AnimationName.ToString());
	}
}

UAvaPlayable::FOnSequenceEvent UAvaPlayable::OnSequenceEventDelegate;
UAvaPlayable::FOnTransitionEvent UAvaPlayable::OnTransitionEventDelegate;

UAvaPlayable* UAvaPlayable::Create(UObject* InOuter, const FPlayableCreationInfo& InPlayableInfo)
{
	using namespace UE::AvaPlayable::Private;

	const UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	
	UAvaPlayable* NewPlayable;
	
	// Forked channels considerations:
	// - The case of forked remote channels is/will be handled internally to the RemoteProxy playable.
	// - The case of forked local and remote channels will lead to a local playable and a remote proxy playable.
	//   It would require wrapping the playables in a composite (or facade?) proxy. TODO
	
	if (UE::AvaMedia::PlayableUtils::HasLocalPlay(Broadcast, InPlayableInfo.ChannelName))
	{
		// For the moment, remote outputs will be ignored.
		if (UE::AvaMedia::PlayableUtils::HasRemotePlay(Broadcast, InPlayableInfo.ChannelName))
		{
			UE_LOGF(LogAvaPlayable, Error, "Forked Channels with both local and remote outputs are not supported in this version. Only local instance will be created.");
		}
		
		NewPlayable = CreateLocalPlayable(InOuter, InPlayableInfo);
	}
	else
	{
		// Purely remote channel.
		NewPlayable = CreateRemoteProxyPlayable(InOuter, InPlayableInfo);
	}

	// Finish the setup.
	if (NewPlayable && !NewPlayable->InitPlayable(InPlayableInfo))
	{
		// final setup may fail, in this case the playable is discarded.
		return nullptr;
	}

	return NewPlayable;
}

const FSoftObjectPath& UAvaPlayable::GetSourceAssetPath() const
{
	static const FSoftObjectPath Empty;
	return Empty;
}

EAvaPlayableCommandResult UAvaPlayable::ExecuteAnimationCommand(EAvaPlaybackAnimAction InAnimAction, const FAvaPlaybackAnimPlaySettings& InAnimPlaySettings)
{
	using namespace UE::AvaPlayable::Private;

	const EAvaPlayableStatus PlayableStatus = GetPlayableStatus();

	if (PlayableStatus == EAvaPlayableStatus::Unknown
		|| PlayableStatus == EAvaPlayableStatus::Error
		|| PlayableStatus == EAvaPlayableStatus::Unloaded)
	{
		UE_LOGF(LogAvaPlayable, Verbose,
			"%ls Playable {%ls} -> Discarding Sequence Command: {%ls}.",
			*GetBriefFrameInfo(), *GetPrettyPlayableInfo(this), *GetPrettySequenceCommandInfo(InAnimAction, InAnimPlaySettings));
	
		// Discard the command
		return EAvaPlayableCommandResult::ErrorDiscard;
	}

	// Asset status must be visible to run the animation commands.
	// If not visible, the components are not yet added to the world and animations won't execute.
	if (PlayableStatus != EAvaPlayableStatus::Visible)
	{
		UE_LOGF(LogAvaPlayable, Verbose,
			"%ls Playable {%ls} -> ReQueueing Sequence Command: {%ls}.",
			*GetBriefFrameInfo(), *GetPrettyPlayableInfo(this), *GetPrettySequenceCommandInfo(InAnimAction, InAnimPlaySettings));
		
		// Keep the command in the queue for next tick.
		return EAvaPlayableCommandResult::KeepPending;
	}

	const IAvaSceneInterface* Scene = GetSceneInterface();
	if (!Scene)
	{
		return EAvaPlayableCommandResult::ErrorDiscard;
	}
	
	IAvaSequencePlaybackObject* PlaybackObject = Scene->GetPlaybackObject();
	
	if (!PlaybackObject)
	{
		return EAvaPlayableCommandResult::ErrorDiscard;
	}

	const IAvaSequenceProvider* SequenceProvider = Scene->GetSequenceProvider();

	if (!SequenceProvider)
	{
		return EAvaPlayableCommandResult::ErrorDiscard;
	}

	UE_LOGF(LogAvaPlayable, Verbose,
		"%ls Playable {%ls} -> Executing Sequence Command: {%ls}.",
		*GetBriefFrameInfo(), *GetPrettyPlayableInfo(this), *GetPrettySequenceCommandInfo(InAnimAction, InAnimPlaySettings));

	bool bFoundPreviewMark = false;
	
	for (const TObjectPtr<UAvaSequence>& Sequence : SequenceProvider->GetSequences())
	{
		// Remark: if the command doesn't specify the sequence name, we run the command on all the sequences.
		if (Sequence && (Sequence->GetFName() == InAnimPlaySettings.AnimationName || InAnimPlaySettings.AnimationName.IsNone()))
		{
			switch (InAnimAction)
			{
			case EAvaPlaybackAnimAction::Play:
				PlaybackObject->PlaySequence(Sequence, InAnimPlaySettings.AsPlayParams());
				break;

			case EAvaPlaybackAnimAction::Continue:
				PlaybackObject->ContinueSequence(Sequence);
				break;

			case EAvaPlaybackAnimAction::Stop:
				PlaybackObject->StopSequence(Sequence);
				break;

			case EAvaPlaybackAnimAction::PreviewFrame:
				// If no animation is specified, delay the missing preview mark warning.
				if (Sequence->GetPreviewMark() || Sequence->GetFName() == InAnimPlaySettings.AnimationName)
				{
					bFoundPreviewMark = true;
					PlaybackObject->PreviewFrame(Sequence);
				}
				break;
			}
		}
	}

	// Log a warning if a PreviewFrame was requested but no marks where found in any of the sequences.
	if (InAnimPlaySettings.AnimationName.IsNone() && InAnimAction == EAvaPlaybackAnimAction::PreviewFrame && !bFoundPreviewMark)
	{
		const FString SequenceList = FString::JoinBy(SequenceProvider->GetSequences(), TEXT(", "), [](const TObjectPtr<UAvaSequence>& InSequence)
		{
			return FString::Printf(TEXT("%s' ('%s')"), *InSequence->GetLabel().ToString(), *InSequence->GetName());
		});		
		UE_LOGF(LogAvaPlayable, Warning , "Failed to Preview Sequence. Preview Mark was not found in any sequences: %ls." , *SequenceList);
	}
	
	return EAvaPlayableCommandResult::Executed;
}

EAvaPlayableCommandResult UAvaPlayable::UpdateRemoteControlCommand(const TSharedRef<FAvaPlayableRemoteControlValues>& InRemoteControlValues, EAvaPlayableRCUpdateFlags InFlags)
{
	const EAvaPlayableStatus PlayableStatus = GetPlayableStatus();

	if (PlayableStatus == EAvaPlayableStatus::Unknown
		|| PlayableStatus == EAvaPlayableStatus::Error
		|| PlayableStatus == EAvaPlayableStatus::Unloaded)
	{
		using namespace UE::AvaPlayable::Private;
		UE_LOGF(LogAvaPlayable, Verbose, "%ls Playable {%ls} -> Discarding RC Update.", *GetBriefFrameInfo(), *GetPrettyPlayableInfo(this));

		// Discard the command
		return EAvaPlayableCommandResult::ErrorDiscard;
	}

	// Asset status must be visible to run the command.
	// If not visible, the components are not yet added to the world.
	if (PlayableStatus != EAvaPlayableStatus::Visible)
	{
		using namespace UE::AvaPlayable::Private;
		UE_LOGF(LogAvaPlayable, Verbose, "%ls Playable {%ls} -> ReQueueing RC Update.", *GetBriefFrameInfo(), *GetPrettyPlayableInfo(this));

		// Keep the command in the queue for next tick.
		return EAvaPlayableCommandResult::KeepPending;
	}

	const IAvaSceneInterface* Scene = GetSceneInterface();
	if (!Scene)
	{
		return EAvaPlayableCommandResult::ErrorDiscard;
	}

	URemoteControlPreset* const RemoteControlPreset = Scene->GetRemoteControlPreset();
	if (!RemoteControlPreset)
	{
		const bool bIsRemoteControlPresetExpectedToBePresent = Scene->IsRemoteControlPresetExpectedToBePresent();

		UE_CLOGF(bIsRemoteControlPresetExpectedToBePresent, LogAvaPlayable, Error,
			"Remote Control command for asset \"%ls\": Remote Control Preset is null.",
			*GetSourceAssetPath().ToString());

		return bIsRemoteControlPresetExpectedToBePresent
			? EAvaPlayableCommandResult::ErrorDiscard
			: EAvaPlayableCommandResult::Executed;
	}

	using namespace UE::AvaPlayable::Private;
	UE_LOGF(LogAvaPlayable, Verbose, "%ls Playable {%ls} -> Executing RC Update.", *GetBriefFrameInfo(), *GetPrettyPlayableInfo(this));

#if WITH_EDITOR
	extern ENGINE_API bool GFlushRenderingCommandsOnPreEditChange;
	bool bFlushRenderingCommandsOnPreEditChange = GFlushRenderingCommandsOnPreEditChange;

	// If not allowing flush rendering commands on pre edit change, this will force the global to be set to false in this function scope.
	// NOTE: If allowing the flush, this won't force the global to be true in case callers have set the global to false.
	if (!CVarPlayableAllowFlushRenderingCommandsOnPreEditChange.GetValueOnGameThread())
	{
		bFlushRenderingCommandsOnPreEditChange = false;
	}

	TGuardValue<bool> ShouldFlushRenderingCommands(GFlushRenderingCommandsOnPreEditChange, bFlushRenderingCommandsOnPreEditChange);
#endif

	TSet<FGuid> AppliedOrIgnoredEntities;
	
	if (CVarUpdateRCControllerValues.GetValueOnGameThread())
	{
		using namespace UE::AvaPlayableRemoteControl;

		// Apply special controllers that don't work just with entity values.
		const bool bExecuteSpecialControllerBehavior = CVarExecuteSpecialRCControllerBehavior.GetValueOnGameThread();
		const bool bExecuteAllControllerBehaviors = EnumHasAnyFlags(InFlags, EAvaPlayableRCUpdateFlags::ExecuteControllerBehaviors);
		const bool bExecuteNonOverlappingControllers = CVarExecuteNonOverlappingRCControllerBehavior.GetValueOnGameThread();
		const bool bExecuteEventControllers = EnumHasAnyFlags(InFlags, EAvaPlayableRCUpdateFlags::ControllerValuesAsEvents);

		TSet<FGuid> ModifiedControllers;

		// For RC updates, the behaviors are executed only on the controller values that changed.
		// The expectation is that only a few controllers are changed in an update and there are no collisions
		// in the underlying entities (order independent update).
		if (bExecuteAllControllerBehaviors)
		{
			for (const TPair<FGuid, FAvaPlayableRemoteControlValue>& Controller : InRemoteControlValues->ControllerValues)
			{
				if (LatestRemoteControlValues)
				{
					const FAvaPlayableRemoteControlValue* FoundValue = LatestRemoteControlValues->ControllerValues.Find(Controller.Key);
					if (!FoundValue || !FoundValue->IsSameValueAs(Controller.Value))
					{
						ModifiedControllers.Add(Controller.Key);
					}
				}
				else
				{
					ModifiedControllers.Add(Controller.Key);
				}
			}
		}

		TSharedPtr<FAvaPlayableRemoteControlPresetInfo> RCPInfo =
			IAvaPlayableRemoteControlPresetInfoCache::Get().GetRemoteControlPresetInfo(GetSourceAssetPath(), RemoteControlPreset);
		
		using namespace UE::RCCustomControllers;

		TArray<URCVirtualPropertyBase*> Controllers = RemoteControlPreset->GetControllers();
		for (URCVirtualPropertyBase* Controller : Controllers)
		{
			if (!Controller)
			{
				continue;
			}

			// Skip the ignored controllers
			if (FAvaPlayableRemoteControlValues::ShouldIgnoreController(Controller))
			{
				GetEntitiesControlledByController(RemoteControlPreset, Controller, AppliedOrIgnoredEntities);
				continue;
			}
			
			if (const FAvaPlayableRemoteControlValue* ControllerValue = InRemoteControlValues->GetControllerValue(Controller->Id))
			{
				const bool bIsEventController = FAvaPlayableRemoteControlValues::IsRuntimeEventController(Controller);

				// Skip event controllers unless this is an event RC update.
				if (bIsEventController && !bExecuteEventControllers)
				{
					continue;
				}
				
				const bool bBehaviorsEnabled =
					// Texture Controller Require the Bind Behavior to be executed to setup the texture in external mode.
					(bExecuteSpecialControllerBehavior && GetCustomControllerTypeName(Controller) == CustomTextureControllerName)
					// Modified controller's behavior are executed when updating RC values
					|| (bExecuteAllControllerBehaviors && ModifiedControllers.Contains(Controller->Id))
					// Non-overlapping controller's behavior can be enabled (experimental)
					|| (bExecuteNonOverlappingControllers && RCPInfo.IsValid() && !RCPInfo->IsControllerOverlapping(Controller->Id))
					// Execute event controllers if this is an event update regardless of the controller value (i.e. modified or not)
					|| (bIsEventController && bExecuteEventControllers);

				SetValueOfController(Controller, ControllerValue->Value, bBehaviorsEnabled);

				// (Tentative) Update the payload of event controllers.
				// It is supposed to reflect the latest values as they are applied.
				if (bIsEventController && bExecuteEventControllers && LatestRemoteControlValues)
				{
					LatestRemoteControlValues->SetControllerValue(Controller->Id, *ControllerValue);
				}

				if (bBehaviorsEnabled)
				{
					GetEntitiesControlledByController(RemoteControlPreset, Controller, AppliedOrIgnoredEntities);
					UE_LOGF(LogAvaPlayable, Verbose, "%ls Playable {%ls} -> Updating Controller %ls (with behaviors).",
						*GetBriefFrameInfo(), *GetPrettyPlayableInfo(this), *Controller->DisplayName.ToString());
				}
			}
		}
	}

	// WYSIWYG (Solution): For the runtime/playback RCP, we don't apply the controllers.
	// We assume the controller actions are already executed in the rundown's managed RCP
	// during page edition and the resulting entity values are already captured.

	// Only apply entity values for non-event RC updates.
	if (!EnumHasAnyFlags(InFlags, EAvaPlayableRCUpdateFlags::ControllerValuesAsEvents))
	{
		InRemoteControlValues->ApplyEntityValuesToRemoteControlPreset(RemoteControlPreset, AppliedOrIgnoredEntities);
		LatestRemoteControlValues = InRemoteControlValues;
		OnRemoteControlValuesApplied();
	}

	return EAvaPlayableCommandResult::Executed;
}

void UAvaPlayable::BeginPlay(const FAvaInstancePlaySettings& InWorldPlaySettings)
{
	if (!PlayableGroup)
	{
		return;
	}

	const bool bGroupHasBegunPlay = PlayableGroup->ConditionalBeginPlay(InWorldPlaySettings);
	
	if (!bIsPlaying || bGroupHasBegunPlay)
	{
		bIsPlaying = true;
		
		// Playable events need to transit through playback events to reach the rundown for proper impl layer separation.
		UAvaSequencePlayer::OnSequenceStarted().AddUObject(this, &UAvaPlayable::HandleOnSequenceStarted);
		UAvaSequencePlayer::OnSequencePaused().AddUObject(this, &UAvaPlayable::HandleOnSequencePaused);
		UAvaSequencePlayer::OnSequenceFinished().AddUObject(this, &UAvaPlayable::HandleOnSequenceFinished);

		OnPlay();
	}
}

void UAvaPlayable::EndPlay(EAvaPlayableEndPlayOptions InOptions)
{
	if (!bIsPlaying)
	{
		return;
	}

	bIsPlaying = false;
	UAvaSequencePlayer::OnSequenceStarted().RemoveAll(this);
	UAvaSequencePlayer::OnSequencePaused().RemoveAll(this);
	UAvaSequencePlayer::OnSequenceFinished().RemoveAll(this);
	OnEndPlay();

	if (PlayableGroup && EnumHasAnyFlags(InOptions, EAvaPlayableEndPlayOptions::ConditionalEndPlayWorld) && !PlayableGroup->HasPlayingPlayables())
	{
		const bool bForceImmediate = EnumHasAnyFlags(InOptions, EAvaPlayableEndPlayOptions::ForceImmediate);
		PlayableGroup->RequestEndPlayWorld(bForceImmediate);
	}
}

bool UAvaPlayable::HasSequence(const UAvaSequence* InSequence) const
{
	const IAvaSceneInterface* SceneInterface = GetSceneInterface();
	if (!SceneInterface)
	{
		return false;
	}
	
	const IAvaSequenceProvider* SequenceProvider = SceneInterface->GetSequenceProvider();
	if (!SequenceProvider)
	{
		return false;
	}
	
	for (const TObjectPtr<UAvaSequence>& Sequence : SequenceProvider->GetSequences())
	{
		if (Sequence == InSequence)
		{
			return true;
		}
	}
	return false;
}

bool UAvaPlayable::InitPlayable(const FPlayableCreationInfo& InPlayableInfo)
{
	if (PlayableGroup)
	{
		// Register this playable in the instance group.
		// This is necessary to determine what is playing in what group.
		PlayableGroup->RegisterPlayable(this);
		return true;
	}
	
	// Currently, playables must have a playable group otherwise they are unplayable.
	UE_LOGF(LogAvaPlayable, Error, "Failed to create or acquire a playable group for \"%ls\". Playable will be discarded.", *InPlayableInfo.SourceAsset.ToSoftObjectPath().ToString());
	return false;
}

void UAvaPlayable::HandleOnSequenceStarted(UAvaSequencePlayer* InSequencePlayer, UAvaSequence* InSequence)
{
	if (HasSequence(InSequence))
	{
		using namespace UE::AvaPlayable::Private;
		UE_LOGF(LogAvaPlayable, Verbose, "%ls Playable {%ls}: Sequence {%ls} started.",
			*GetBriefFrameInfo(), *GetPrettyPlayableInfo(this),  *GetPrettySequenceInfo(InSequence));
		OnSequenceEventDelegate.Broadcast(this, InSequence->GetLabel(), EAvaPlayableSequenceEventType::Started);
	}
}

void UAvaPlayable::HandleOnSequencePaused(UAvaSequencePlayer* InSequencePlayer, UAvaSequence* InSequence)
{
	if (HasSequence(InSequence))
	{
		using namespace UE::AvaPlayable::Private;
		UE_LOGF(LogAvaPlayable, Verbose, "%ls Playable {%ls}: Sequence {%ls} paused.",
			*GetBriefFrameInfo(), *GetPrettyPlayableInfo(this),  *GetPrettySequenceInfo(InSequence));
		OnSequenceEventDelegate.Broadcast(this, InSequence->GetLabel(), EAvaPlayableSequenceEventType::Paused);
	}
}

void UAvaPlayable::HandleOnSequenceFinished(UAvaSequencePlayer* InSequencePlayer, UAvaSequence* InSequence)
{
	if (HasSequence(InSequence))
	{
		using namespace UE::AvaPlayable::Private;
		UE_LOGF(LogAvaPlayable, Verbose, "%ls Playable {%ls}: Sequence {%ls} finished.",
			*GetBriefFrameInfo(), *GetPrettyPlayableInfo(this),  *GetPrettySequenceInfo(InSequence));
		OnSequenceEventDelegate.Broadcast(this, InSequence->GetLabel(), EAvaPlayableSequenceEventType::Finished);
	}
}

UAvaPlayable* UAvaPlayable::CreateLocalPlayable(UObject* InOuter, const FPlayableCreationInfo& InPlayableInfo)
{
	switch (InPlayableInfo.SourceAsset.GetAssetType())
	{
	case EMotionDesignAssetType::World:
		return NewObject<UAvaPlayableLevelStreaming>(InOuter ? InOuter : GEngine);
	default:
		UE_LOGF(LogAvaPlayable, Error, "Asset \"%ls\" is an unsupported type.", *InPlayableInfo.SourceAsset.ToSoftObjectPath().ToString());
		return nullptr;
	}
}

UAvaPlayable* UAvaPlayable::CreateRemoteProxyPlayable(UObject* InOuter, const FPlayableCreationInfo& InPlayableInfo)
{
	return NewObject<UAvaPlayableRemoteProxy>(InOuter ? InOuter : GEngine);
}

#undef LOCTEXT_NAMESPACE
