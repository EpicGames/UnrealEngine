// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"
#include "AvaPlayableAssetUserData.generated.h"

class UAvaPlayable;

/**
 * AssetUserData for AvaPlayable.
 *
 * This is strategically injected in asset instances to be able to retrieve
 * the corresponding playable directly.
 */
UCLASS()
class UAvaPlayableAssetUserData : public UAssetUserData
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<UAvaPlayable> PlayableWeak;
};
