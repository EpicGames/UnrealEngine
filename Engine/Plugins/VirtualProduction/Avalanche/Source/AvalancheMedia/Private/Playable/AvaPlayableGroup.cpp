// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playable/AvaPlayableGroup.h"

#include "AvaPlayableGroupSubsystem.h"
#include "AvaPlayableUtils.h"
#include "Engine/GameInstance.h"
#include "Engine/Level.h"
#include "Engine/ViewportStatsSubsystem.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "GameFramework/Pawn.h"
#include "IAvaMediaModule.h"
#include "Playable/AvaPlayable.h"
#include "Playable/AvaPlayableGroupAssetUserData.h"
#include "Playable/AvaPlayableGroupManager.h"
#include "Playable/AvaPlayableSettings.h"
#include "Playable/PlayableGroups/AvaGameInstancePlayableGroup.h"
#include "Playable/PlayableGroups/AvaGameViewportPlayableGroup.h"
#include "Playable/PlayableGroups/AvaRemoteProxyPlayableGroup.h"
#include "Playable/Transition/AvaPlayableTransition.h"
#include "Playback/AvaPlaybackManager.h"
#include "Playback/AvaPlaybackUtils.h"
#include "Playback/IAvaPlaybackServer.h"
#include "SceneView.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "AvaPlayableGroup"

namespace UE::AvaMedia::PlayableGroup::Private
{
	template<typename InElementType>
	void CleanStaleKeys(TSet<TObjectKey<InElementType>>& InOutSetToClean)
	{
		for (typename TSet<TObjectKey<InElementType>>::TIterator It(InOutSetToClean); It; ++It)
		{
			if (!It->ResolveObjectPtr())
			{
				It.RemoveCurrent();
			}
		}
	}

	UAvaPlayableGroupAssetUserData* FindPlayableGroupAssetUserData(IInterface_AssetUserData* InAssetInstance)
	{
		return InAssetInstance->GetAssetUserData<UAvaPlayableGroupAssetUserData>();
	}

	UAvaPlayableGroupAssetUserData* FindPlayableGroupAssetUserDataSafe(IInterface_AssetUserData* InAssetInstance)
	{
		return InAssetInstance ? InAssetInstance->GetAssetUserData<UAvaPlayableGroupAssetUserData>() : nullptr;
	}

	UAvaPlayableGroupAssetUserData* FindOrAddPlayableGroupAssetUserData(IInterface_AssetUserData* InAssetInstance)
	{
		if (!InAssetInstance)
		{
			return nullptr;
		}
		
		UAvaPlayableGroupAssetUserData* PlayableGroupUserData = FindPlayableGroupAssetUserData(InAssetInstance);
		if (!PlayableGroupUserData)
		{
			PlayableGroupUserData = NewObject<UAvaPlayableGroupAssetUserData>();
			InAssetInstance->AddAssetUserData(PlayableGroupUserData);
		}
		return PlayableGroupUserData;
	}
}

UAvaPlayableGroup* UAvaPlayableGroup::MakePlayableGroup(UObject* InOuter, const FPlayableGroupCreationInfo& InPlayableGroupInfo)
{
	UObject* Outer = InOuter ? InOuter : GetTransientPackage();
	
	UAvaPlayableGroup* NewPlayableGroup = nullptr;
	if (InPlayableGroupInfo.bIsRemoteProxy)
	{
		// Remote Proxy group doesn't have a game instance.
		NewPlayableGroup = NewObject<UAvaRemoteProxyPlayableGroup>(Outer);
		NewPlayableGroup->ParentPlayableGroupManagerWeak = InPlayableGroupInfo.PlayableGroupManager;
	}
	else
	{
		if (InPlayableGroupInfo.GameInstance)
		{
			NewPlayableGroup = UAvaGameViewportPlayableGroup::Create(InOuter, InPlayableGroupInfo.GameInstance, InPlayableGroupInfo.PlayableGroupManager);
		}
		else
		{
			NewPlayableGroup = UAvaGameInstancePlayableGroup::Create(InOuter, InPlayableGroupInfo);
		}
	}

	if (NewPlayableGroup)
	{
		NewPlayableGroup->ChannelName = InPlayableGroupInfo.ChannelName;

		if (const UWorld* PlayWorld = NewPlayableGroup->GetPlayWorld())
		{
			using namespace UE::AvaMedia::PlayableGroup::Private;
			if(UAvaPlayableGroupAssetUserData* PlayableGroupUserData = FindOrAddPlayableGroupAssetUserData(PlayWorld->PersistentLevel))
			{
				PlayableGroupUserData->PlayableGroupsWeak.AddUnique(NewPlayableGroup);
			}
		}
	}
	
	return NewPlayableGroup;
}

void UAvaPlayableGroup::RegisterPlayable(UAvaPlayable* InPlayable)
{
	// Prevent accumulation of stale keys.
	UE::AvaMedia::PlayableGroup::Private::CleanStaleKeys(Playables);

	if (InPlayable)
	{
		InPlayable->OnPlayableStatusChanged().AddUObject(this, &UAvaPlayableGroup::OnPlayableStatusChanged);
		Playables.Add(InPlayable);
	}
}

/** Unregister a playable when it is about to be deleted. */
void UAvaPlayableGroup::UnregisterPlayable(UAvaPlayable* InPlayable)
{
	if (InPlayable)
	{
		InPlayable->OnPlayableStatusChanged().RemoveAll(this);
		Playables.Remove(InPlayable);
	}
}

bool UAvaPlayableGroup::HasPlayables() const
{
	for (const TObjectKey<UAvaPlayable>& PlayableKey : Playables)
	{
		if (PlayableKey.ResolveObjectPtr())
		{
			return true;
		}
	}
	return false;
}

bool UAvaPlayableGroup::HasPlayingPlayables() const
{
	for (const TObjectKey<UAvaPlayable>& PlayableKey : Playables)
	{
		const UAvaPlayable* Playable = PlayableKey.ResolveObjectPtr();
		if (Playable && Playable->IsPlaying())
		{
			return true;
		}
	}
	return false;
}

void UAvaPlayableGroup::FindPlayablesBySourceAssetPath(const FSoftObjectPath& InSourceAssetPath, TArray<UAvaPlayable*>& OutFoundPlayables) const
{
	for (const TObjectKey<UAvaPlayable>& PlayableKey : Playables)
	{
		UAvaPlayable* Playable = PlayableKey.ResolveObjectPtr();
		if (Playable && Playable->GetSourceAssetPath() == InSourceAssetPath)
		{
			OutFoundPlayables.Add(Playable);
		}
	}
}

void UAvaPlayableGroup::RegisterPlayableTransition(UAvaPlayableTransition* InPlayableTransition)
{
	if (InPlayableTransition)
	{
		// Protect PlayableTransitions iterator.
		if (bIsTickingTransitions)
		{
			PlayableTransitionsToRemove.Remove(InPlayableTransition);
			PlayableTransitionsToAdd.Add(InPlayableTransition);
			return;
		}

		// Prevent accumulation of stale keys.
		UE::AvaMedia::PlayableGroup::Private::CleanStaleKeys(PlayableTransitions);

		PlayableTransitions.Add(InPlayableTransition);
		
		if (UAvaPlayableGroupManager* PlayableGroupManager = GetPlayableGroupManager())
		{
			PlayableGroupManager->RegisterForTransitionTicking(this);
		}
	}
}
	
void UAvaPlayableGroup::UnregisterPlayableTransition(UAvaPlayableTransition* InPlayableTransition)
{
	if (InPlayableTransition)
	{
		// Protect PlayableTransitions iterator.
		if (bIsTickingTransitions)
		{
			PlayableTransitionsToAdd.Remove(InPlayableTransition);
			PlayableTransitionsToRemove.Add(InPlayableTransition);
			return;
		}
		
		PlayableTransitions.Remove(InPlayableTransition);

		UE::AvaMedia::PlayableGroup::Private::CleanStaleKeys(PlayableTransitions);
		
		if (PlayableTransitions.IsEmpty())
		{
			if (UAvaPlayableGroupManager* PlayableGroupManager = GetPlayableGroupManager())
			{
				PlayableGroupManager->UnregisterFromTransitionTicking(this);
			}
		}
	}
}

void UAvaPlayableGroup::TickTransitions(double InDeltaSeconds)
{
	// Ticking the transitions will lead to some transitions being removed.
	// So we need to protect the iterator with a scope and do the operations when iteration is done.
	{
		TGuardValue TickGuard(bIsTickingTransitions, true);
	
		for (TSet<TObjectKey<UAvaPlayableTransition>>::TIterator TickIt(PlayableTransitions); TickIt; ++TickIt)
		{
			if (UAvaPlayableTransition* TransitionToTick = TickIt->ResolveObjectPtr())
			{
				TransitionToTick->Tick(InDeltaSeconds);
			}
			else
			{
				// Prevent accumulation of stale keys.
				// Remark: if there is a stale key, it means the transition was not stopped properly.
				TickIt.RemoveCurrent();
			}
		}
	}
	
	for (const TObjectKey<UAvaPlayableTransition>& ToRemove : PlayableTransitionsToRemove)
	{
		UnregisterPlayableTransition(ToRemove.ResolveObjectPtr());
	}
	PlayableTransitionsToRemove.Reset();
	
	for (const TObjectKey<UAvaPlayableTransition>& ToAdd : PlayableTransitionsToAdd)
	{
		RegisterPlayableTransition(ToAdd.ResolveObjectPtr());
	}
	PlayableTransitionsToAdd.Reset();
}

bool UAvaPlayableGroup::HasTransitions() const
{
	return !PlayableTransitions.IsEmpty();
}

void UAvaPlayableGroup::PushSynchronizedEvent(FString&& InEventSignature, TUniqueFunction<void()> InFunction)
{
	if (UAvaPlayableGroupManager* Manager = GetPlayableGroupManager())
	{
		// Using one dispatcher for now, if the event signature needs to be scoped per playable group
		// we could have a dispatcher for each playable group with a unique signature.
		Manager->PushSynchronizedEvent(MoveTemp(InEventSignature), MoveTemp(InFunction));
	}
	else if (InFunction)
	{
		InFunction();
	}
}

bool UAvaPlayableGroup::IsSynchronizedEventPushed(const FString& InEventSignature) const
{
	if (const UAvaPlayableGroupManager* Manager = GetPlayableGroupManager())
	{
		return Manager->IsSynchronizedEventPushed(InEventSignature);
	}
	return false;	
}

void UAvaPlayableGroup::SetLastAppliedCameraPlayable(UAvaPlayable* InPlayable)
{
	LastAppliedCameraPlayableWeak = InPlayable;
}

UTextureRenderTarget2D* UAvaPlayableGroup::GetRenderTarget() const
{
	return ManagedRenderTarget.Get();
}

UTextureRenderTarget2D* UAvaPlayableGroup::GetManagedRenderTarget() const
{
	return ManagedRenderTarget.Get();
}


void UAvaPlayableGroup::SetManagedRenderTarget(UTextureRenderTarget2D* InManageRenderTarget)
{
	ManagedRenderTarget = InManageRenderTarget;
}

UWorld* UAvaPlayableGroup::GetPlayWorld() const
{
	return GameInstance ? GameInstance->GetWorld() : nullptr;
}

FName UAvaPlayableGroup::GetChannelName() const
{
	return ChannelName;
}

void UAvaPlayableGroup::NotifyLevelStreaming(UAvaPlayable* InPlayable)
{
	// If the world is not playing, we need to make sure the level streaming still updates.
	if (!IsWorldPlaying())
	{
		if (UAvaPlayableGroupManager* PlayableGroupManager = GetPlayableGroupManager())
		{
			PlayableGroupManager->RegisterForLevelStreamingUpdate(this);
		}
	}
}

void UAvaPlayableGroup::FVisibilityRequest::Execute(const UAvaPlayableGroup* InPlayableGroup) const
{
	UAvaPlayable* Playable = PlayableWeak.Get();
	
	if (!Playable)
	{
		using namespace UE::AvaPlayback::Utils;
		UE_LOGF(LogAvaPlayable, Error,
			"%ls Failed to Set Visibility to \"%ls\" because the playable has become stale. Playable Group: \"%ls\".",
			*GetBriefFrameInfo(), bShouldBeVisible ? TEXT("true") : TEXT("false"), *InPlayableGroup->GetFullName());
		return;
	}

	Playable->SetShouldBeVisible(bShouldBeVisible);
}

void UAvaPlayableGroup::RegisterVisibilityConstraint(const TWeakInterfacePtr<IAvaPlayableVisibilityConstraint>& InVisibilityConstraint)
{
	if (!VisibilityConstraints.Contains(InVisibilityConstraint))
	{
		VisibilityConstraints.RemoveAll([](const TWeakInterfacePtr<IAvaPlayableVisibilityConstraint>& InConstraint) { return InConstraint.IsStale();});
		VisibilityConstraints.Add(InVisibilityConstraint);
	}
}

void UAvaPlayableGroup::UnregisterVisibilityConstraint(const IAvaPlayableVisibilityConstraint* InVisibilityConstraint)
{
	VisibilityConstraints.RemoveAll([InVisibilityConstraint](const TWeakInterfacePtr<IAvaPlayableVisibilityConstraint>& InConstraint)
	{
		// also remove stale pointers.
		return InConstraint.IsStale() || InConstraint.Get() == InVisibilityConstraint;
	});
}

void UAvaPlayableGroup::RequestSetVisibility(UAvaPlayable* InPlayable, bool bInShouldBeVisible)
{
	FVisibilityRequest Request = { InPlayable, bInShouldBeVisible };

	if (IsVisibilityConstrained(InPlayable))
	{
		VisibilityRequests.Add(MoveTemp(Request));
	}
	else
	{
		Request.Execute(this);
	}
}

void UAvaPlayableGroup::SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView)
{
	const FAvaPlayableSettings& PlayableSettings = IAvaMediaModule::Get().GetPlayableSettings();
	if (PlayableSettings.bHidePawnActors)
	{
		HidePawnsForView(GetPlayWorld(), InView);
	}

	for (const TObjectKey<UAvaPlayable>& PlayableKey : Playables)
	{
		UAvaPlayable* Playable = PlayableKey.ResolveObjectPtr();
		
		if (Playable && Playable->IsPlaying())
		{
			Playable->SetupView(InViewFamily, InView);
		}
	}
}

bool UAvaPlayableGroup::IsVisibilityConstrained(const UAvaPlayable* InPlayable) const
{
	for (const TWeakInterfacePtr<IAvaPlayableVisibilityConstraint>& ConstraintWeak : VisibilityConstraints)
	{
		if (const IAvaPlayableVisibilityConstraint* Constraint = ConstraintWeak.Get())
		{
			if (Constraint->IsVisibilityConstrained(InPlayable))
			{
				return true;
			}
		}
	}
	return false;
}

void UAvaPlayableGroup::ForEachPlayable(TFunctionRef<bool(UAvaPlayable*)> InFunction)
{
	for (const TObjectKey<UAvaPlayable>& PlayableKey : Playables)
	{
		if (UAvaPlayable* Playable = PlayableKey.ResolveObjectPtr())
		{
			if (!InFunction(Playable))
			{
				return;
			}
		}
	}
}
	
void UAvaPlayableGroup::ForEachPlayableTransition(TFunctionRef<bool(UAvaPlayableTransition*)> InFunction)
{
	for (const TObjectKey<UAvaPlayableTransition>& TransitionKey : PlayableTransitions)
	{
		if (UAvaPlayableTransition* Transition = TransitionKey.ResolveObjectPtr())
		{
			if (!InFunction(Transition))
			{
				return;
			}
		}
	}
}

UAvaPlayableGroup* UAvaPlayableGroup::FindPlayableGroupForWorld(const UWorld* InWorld, bool bInFallbackToGlobalSearch)
{
	if (!InWorld || !InWorld->PersistentLevel)
	{
		return nullptr;
	}

	// Fast path: if a world is managed by a playable group, it should have an asset user data that
	// we can retrieve the corresponding playable group from.
	using namespace UE::AvaMedia::PlayableGroup::Private;
	if (UAvaPlayableGroupAssetUserData* PlayableGroupUserData = FindPlayableGroupAssetUserData(InWorld->PersistentLevel))
	{
		for (const TWeakObjectPtr<UAvaPlayableGroup>& PlayableGroupWeak : PlayableGroupUserData->PlayableGroupsWeak)
		{
			if (UAvaPlayableGroup* PlayableGroup = PlayableGroupWeak.Get())
			{
				return PlayableGroup;
			}
		}
	}

	if (!bInFallbackToGlobalSearch)
	{
		return nullptr;
	}
	
	// Global Search starting from the system's root playable group managers.
	const UAvaPlayableGroupManager* PlayableGroupManager = nullptr;

	// For Game Viewport output, the sub system will give us the playable group manager directly.
	if (const UGameInstance* GameInstance = InWorld->GetGameInstance())
	{
		if (const UAvaPlayableGroupSubsystem* PlayableGroupSubsystem = GameInstance->GetSubsystem<UAvaPlayableGroupSubsystem>())
		{
			PlayableGroupManager = PlayableGroupSubsystem->PlayableGroupManager;
		}
	}

	if (!PlayableGroupManager)
	{
		const FAvaPlaybackManager& PlaybackManager = IAvaMediaModule::Get().GetLocalPlaybackManager();
		PlayableGroupManager = PlaybackManager.GetPlayableGroupManager();
	}

	// Search in the local playable group manager for that world.
	UAvaPlayableGroup* PlayableGroup = PlayableGroupManager->FindPlayableGroupForWorld(InWorld);
	
	// If not found, search in the playback server's playback manager.
	if (!PlayableGroup && IAvaMediaModule::Get().IsPlaybackServerStarted())
	{
		PlayableGroupManager = IAvaMediaModule::Get().GetPlaybackServer()->GetPlaybackManager().GetPlayableGroupManager();
		PlayableGroup = PlayableGroupManager->FindPlayableGroupForWorld(InWorld);
	}

	return PlayableGroup;
}

void UAvaPlayableGroup::OnPlayableStatusChanged(UAvaPlayable* InPlayable)
{
	// Evaluate the playable visibility requests
	for (TArray<FVisibilityRequest>::TIterator RequestIt(VisibilityRequests); RequestIt; ++RequestIt)
	{
		const UAvaPlayable* Playable = RequestIt->PlayableWeak.Get();
		if (Playable && IsVisibilityConstrained(Playable))
		{
			continue;
		}
		
		RequestIt->Execute(this);
		RequestIt.RemoveCurrent();
	}
}

void UAvaPlayableGroup::ConditionalRegisterWorldDelegates(UWorld* InWorld)
{
	if (!DisplayDelegateIndices.IsEmpty() && LastWorldBoundToDisplayDelegates.Get() == InWorld)
	{
		return;
	}

	if (!DisplayDelegateIndices.IsEmpty() && LastWorldBoundToDisplayDelegates.IsValid())
	{
		UnregisterWorldDelegates(LastWorldBoundToDisplayDelegates.Get());
	}

	if (UViewportStatsSubsystem* ViewportSubsystem = InWorld->GetSubsystem<UViewportStatsSubsystem>())
	{
		DisplayDelegateIndices.Add(ViewportSubsystem->AddDisplayDelegate([this](FText& OutText, FLinearColor& OutColor)
		{
			return DisplayLoadedAssets(OutText, OutColor);
		}));
		DisplayDelegateIndices.Add(ViewportSubsystem->AddDisplayDelegate([this](FText& OutText, FLinearColor& OutColor)
		{
			return DisplayPlayingAssets(OutText, OutColor);
		}));
		DisplayDelegateIndices.Add(ViewportSubsystem->AddDisplayDelegate([this](FText& OutText, FLinearColor& OutColor)
		{
			return DisplayTransitions(OutText, OutColor);
		}));

		LastWorldBoundToDisplayDelegates = InWorld;
	}
}

void UAvaPlayableGroup::UnregisterWorldDelegates(UWorld* InWorld)
{
	if (!DisplayDelegateIndices.IsEmpty())
	{
		// Check that we are indeed registered to the world
		check(LastWorldBoundToDisplayDelegates.Get() == InWorld);
	
		if (UViewportStatsSubsystem* ViewportSubsystem = InWorld->GetSubsystem<UViewportStatsSubsystem>())
		{
			// Remark: removing them in the reverse order they where added. It is using RemoveAtSwap()
			// so removing the first one would change the index of the next one. Removing a delegate in the
			// middle of the array will invalidate the indices above.
			for (int32 Index = DisplayDelegateIndices.Num()-1; Index >= 0; --Index)
			{
				ViewportSubsystem->RemoveDisplayDelegate(DisplayDelegateIndices[Index]);
			}
		}

		DisplayDelegateIndices.Reset();
		LastWorldBoundToDisplayDelegates.Reset();
	}

	check(LastWorldBoundToDisplayDelegates.Get() == nullptr);
}

bool UAvaPlayableGroup::DisplayLoadedAssets(FText& OutText, FLinearColor& OutColor)
{
	FString AssetList;
	for (const TObjectKey<UAvaPlayable>& PlayableKey : Playables)
	{
		if (const UAvaPlayable* Playable = PlayableKey.ResolveObjectPtr())
		{
			AssetList += AssetList.IsEmpty() ? TEXT("") : TEXT(", ");
			AssetList += Playable->GetSourceAssetPath().GetAssetName();
		}
	}

	if (!AssetList.IsEmpty())
	{
		OutText = FText::Format(LOCTEXT("DisplayLoadedGraphics", "Loaded Graphic(s): {0}"), FText::FromString(AssetList));
		OutColor = FLinearColor::Red;
		return true;
	}
	return false;
}

bool UAvaPlayableGroup::DisplayPlayingAssets(FText& OutText, FLinearColor& OutColor)
{
	FString AssetList;
	for (const TObjectKey<UAvaPlayable>& PlayableKey : Playables)
	{
		const UAvaPlayable* Playable = PlayableKey.ResolveObjectPtr();
		if (Playable && Playable->IsPlaying())
		{
			AssetList += AssetList.IsEmpty() ? TEXT("") : TEXT(", ");
			AssetList += Playable->GetSourceAssetPath().GetAssetName();

			if (!Playable->GetUserData().IsEmpty())
			{
				AssetList += FString::Printf(TEXT(" (%s)"), *Playable->GetUserData());
			}
		}
	}

	if (!AssetList.IsEmpty())
	{
		OutText = FText::Format(LOCTEXT("DisplayPlayingGraphics", "Playing Graphic(s): {0}"), FText::FromString(AssetList));
		OutColor = FLinearColor::Green;
		return true;
	}
	return false;
}

bool UAvaPlayableGroup::DisplayTransitions(FText& OutText, FLinearColor& OutColor)
{
	FString TransitionList;
	for (const TObjectKey<UAvaPlayableTransition>& TransitionKey : PlayableTransitions)
	{
		const UAvaPlayableTransition* Transition = TransitionKey.ResolveObjectPtr();
		if (Transition && Transition->IsRunning())
		{
			TransitionList += TransitionList.IsEmpty() ? TEXT("") : TEXT(", ");
			TransitionList += Transition->GetPrettyInfo();
		}
	}

	if (!TransitionList.IsEmpty())
	{
		OutText = FText::Format(LOCTEXT("DisplayTransitions", "Transition(s): {0}"), FText::FromString(TransitionList));
		OutColor = FLinearColor::Green;
		return true;
	}
	return false;
}

void UAvaPlayableGroup::HidePawnsForView(const UWorld* InPlayWorld, FSceneView& InView) const
{
	if (!InPlayWorld)
	{
		return;
	}

	for (const APawn* Pawn : TActorRange<APawn>(InPlayWorld))
	{
		using namespace UE::AvaMedia::PlayableUtils;
		AddPrimitiveComponentIds(Pawn, InView.HiddenPrimitives);
	}
}

#undef LOCTEXT_NAMESPACE