// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"
#include "AvaPlayableGroupAssetUserData.generated.h"

class UAvaPlayableGroup;

/**
 * AssetUserData for AvaPlayableGroup.
 * 
 * This is strategically injected in asset instances to be able to retrieve
 * the corresponding playable group directly.
 */
UCLASS()
class UAvaPlayableGroupAssetUserData : public UAssetUserData
{
	GENERATED_BODY()

public:
	/**
	 * A playable group being loosely defined in the playable framework,
	 * an asset instance may be associated with more than one playable group.
	 * The intent is to be injecting this in asset instances that are associated with
	 * just one playable group.
	 */
	TArray<TWeakObjectPtr<UAvaPlayableGroup>> PlayableGroupsWeak;
};
