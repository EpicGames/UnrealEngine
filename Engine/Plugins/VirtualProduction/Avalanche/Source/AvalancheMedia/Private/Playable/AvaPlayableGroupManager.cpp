// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playable/AvaPlayableGroupManager.h"

#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputUtils.h"
#include "Broadcast/OutputDevices/AvaBroadcastRenderTargetMediaUtils.h"
#include "Engine/Engine.h"
#include "Framework/AvaGameInstance.h"
#include "Misc/CoreDelegates.h"
#include "Misc/TimeGuard.h"
#include "ModularFeature/AvaMediaSynchronizedEventsFeature.h"
#include "ModularFeature/IAvaMediaSynchronizedEventDispatcher.h"
#include "Playable/AvaPlayableGroup.h"
#include "Playable/PlayableGroups/AvaRemoteProxyPlayableGroup.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "AvaPlayableGroupManager"

namespace UE::AvaMedia::PlayableGroupManager::Private
{
	UGameViewportClient* FindGameViewportClient()
	{
		if (GEngine->GameViewport)
		{
			return GEngine->GameViewport;
		}
		const TIndirectArray<FWorldContext>& WorldContexts = GEngine->GetWorldContexts();
		for (const FWorldContext& Context : WorldContexts)
		{
			if ((Context.WorldType == EWorldType::PIE) && Context.World() != nullptr && Context.GameViewport != nullptr)
			{
				return Context.GameViewport;
			}
		}
		return nullptr;
	}

	bool HasLocalGameViewportOutput(const FAvaBroadcastOutputChannel& InBroadcastChannel)
	{
		if (!InBroadcastChannel.IsValidChannel())
		{
			return false;
		}
		
		for (const UMediaOutput* MediaOutput : InBroadcastChannel.GetMediaOutputs())
		{
			if (AvaBroadcastOutputUtils::IsGameViewportOutput(MediaOutput) && !InBroadcastChannel.IsMediaOutputRemote(MediaOutput))
			{
				return true;
			}
		}
		return false;
	}
}

UAvaPlayableGroup* UAvaPlayableGroupChannelManager::GetOrCreatePlayableGroup(UGameInstance* InExistingGameInstance, bool bInIsRemoteProxy)
{
	// See if there is a matching playable group.
	for (const TWeakObjectPtr<UAvaPlayableGroup>& PlayableGroupWeak : PlayableGroupsWeak)
	{
		if (UAvaPlayableGroup* PlayableGroup = PlayableGroupWeak.Get())
		{
			if (InExistingGameInstance && PlayableGroup->GetGameInstance() == InExistingGameInstance)
			{
				return PlayableGroup;
			}

			const bool bIsRemoteProxy = PlayableGroup->IsA<UAvaRemoteProxyPlayableGroup>();
			if (bInIsRemoteProxy && bIsRemoteProxy)
			{
				return PlayableGroup;
			}
			
			if (InExistingGameInstance == nullptr && !bInIsRemoteProxy && !bIsRemoteProxy)
			{
				return PlayableGroup;
			}
		}
	}
	
	UAvaPlayableGroup::FPlayableGroupCreationInfo PlayableGroupCreationInfo;
	PlayableGroupCreationInfo.PlayableGroupManager = GetPlayableGroupManager();
	PlayableGroupCreationInfo.ChannelName = ChannelName;
	PlayableGroupCreationInfo.bIsRemoteProxy = bInIsRemoteProxy;
	PlayableGroupCreationInfo.bIsSharedGroup = true;
	PlayableGroupCreationInfo.GameInstance = InExistingGameInstance;
	
	UAvaPlayableGroup* NewPlayableGroup = UAvaPlayableGroup::MakePlayableGroup(GetPlayableGroupManager(), PlayableGroupCreationInfo);

	// Keep track of the shared playable group.
	PlayableGroupsWeak.Add(NewPlayableGroup);

	return NewPlayableGroup;
}

UAvaPlayableGroupManager* UAvaPlayableGroupChannelManager::GetPlayableGroupManager() const
{
	return Cast<UAvaPlayableGroupManager>(GetOuter());	
}

void UAvaPlayableGroupChannelManager::GetPlayableGroups(TArray<TWeakObjectPtr<UAvaPlayableGroup>>& OutGroups) const
{
	OutGroups.Append(PlayableGroupsWeak);
}

UAvaPlayableGroup* UAvaPlayableGroupChannelManager::FindPlayableGroupForWorld(const UWorld* InWorld) const
{
	for (const TWeakObjectPtr<UAvaPlayableGroup>& PlayableGroupWeak : PlayableGroupsWeak)
	{
		if (UAvaPlayableGroup* PlayableGroup = PlayableGroupWeak.Get())
		{
			if (PlayableGroup->GetPlayWorld() == InWorld)
			{
				return PlayableGroup;	
			}
		}
	}
	return nullptr;
}

void UAvaPlayableGroupChannelManager::Shutdown()
{
	PlayableGroupsWeak.Reset();
}

void UAvaPlayableGroupChannelManager::BeginDestroy()
{
	Shutdown();
	Super::BeginDestroy();
}

void UAvaPlayableGroupManager::Init()
{
	if (!SynchronizedEventDispatcher)
	{
		// There is only a primary/default dispatcher for now.
		SynchronizedEventDispatcher = FAvaMediaSynchronizedEventsFeature::CreateDispatcher(TEXT("DefaultGroup"));
	}

	if (!UAvaGameInstance::GetOnEndPlay().IsBoundToObject(this))
	{
		UAvaGameInstance::GetOnEndPlay().AddUObject(this, &UAvaPlayableGroupManager::OnGameInstanceEndPlay);
	}
}

void UAvaPlayableGroupManager::Shutdown()
{
	for (TPair<FName, TObjectPtr<UAvaPlayableGroupChannelManager>>& Pair : ChannelManagers)
	{
		if (UAvaPlayableGroupChannelManager* const ChannelManager = Pair.Value)
		{
			ChannelManager->Shutdown();
		}
	}
	UAvaGameInstance::GetOnEndPlay().RemoveAll(this);
}

void UAvaPlayableGroupManager::Tick(double InDeltaSeconds)
{
	SCOPE_TIME_GUARD(TEXT("UAvaPlayableGroupManager::Tick"));
	UpdateLevelStreaming();
	TickTransitions(InDeltaSeconds);

	if (SynchronizedEventDispatcher)
	{
		SynchronizedEventDispatcher->DispatchEvents();
	}
}

void UAvaPlayableGroupManager::PushSynchronizedEvent(FString&& InEventSignature, TUniqueFunction<void()> InFunction)
{
	if (SynchronizedEventDispatcher)
	{
		SynchronizedEventDispatcher->PushEvent(MoveTemp(InEventSignature), MoveTemp(InFunction));
	}
	else if (InFunction)
	{
		InFunction();
	}
}

bool UAvaPlayableGroupManager::IsSynchronizedEventPushed(const FString& InEventSignature) const
{
	if (SynchronizedEventDispatcher)
	{
		const EAvaMediaSynchronizedEventState EventState = SynchronizedEventDispatcher->GetEventState(InEventSignature);
		return EventState == EAvaMediaSynchronizedEventState::Pending || EventState == EAvaMediaSynchronizedEventState::Ready;
	}
	return false;	
}

UAvaPlayableGroupChannelManager* UAvaPlayableGroupManager::FindOrAddChannelManager(const FName& InChannelName)
{
	UAvaPlayableGroupChannelManager* ChannelManager = FindChannelManager(InChannelName);
	if (!ChannelManager)
	{
		ChannelManager = NewObject<UAvaPlayableGroupChannelManager>(this);
		ChannelManager->ChannelName = InChannelName;
		ChannelManagers.Add(InChannelName, ChannelManager);
	}
	return ChannelManager;
}

UAvaPlayableGroup* UAvaPlayableGroupManager::GetOrCreateSharedPlayableGroup(const FName& InChannelName, bool bInIsRemoteProxy)
{
	using namespace UE::AvaMedia::PlayableGroupManager::Private;

	UGameInstance* ExistingGameInstance = nullptr;
	
	const FAvaBroadcastOutputChannel& BroadcastChannel =  UAvaBroadcast::Get().GetCurrentProfile().GetChannel(InChannelName);
	if (HasLocalGameViewportOutput(BroadcastChannel) && !bInIsRemoteProxy)
	{
		if (const UGameViewportClient* GameViewport = FindGameViewportClient())
		{
			ExistingGameInstance = GameViewport->GetGameInstance();
		}
	}

	UAvaPlayableGroupChannelManager* ChannelManager = FindOrAddChannelManager(InChannelName);
	return ChannelManager ? ChannelManager->GetOrCreatePlayableGroup(ExistingGameInstance, bInIsRemoteProxy) : nullptr;
}

void UAvaPlayableGroupManager::RegisterForLevelStreamingUpdate(UAvaPlayableGroup* InPlayableGroup)
{
	if (ensure(!bIsUpdatingStreaming))
	{
		GroupsToUpdateStreaming.Add(InPlayableGroup);
	}
}

void UAvaPlayableGroupManager::UnregisterFromLevelStreamingUpdate(UAvaPlayableGroup* InPlayableGroup)
{
	if (!bIsUpdatingStreaming)
	{
		GroupsToUpdateStreaming.Remove(InPlayableGroup);
	}
}

void UAvaPlayableGroupManager::RegisterForTransitionTicking(UAvaPlayableGroup* InPlayableGroup)
{
	if (ensure(!bIsTickingTransitions))
	{
		GroupsToTickTransitions.Add(InPlayableGroup);
	}
}

void UAvaPlayableGroupManager::UnregisterFromTransitionTicking(UAvaPlayableGroup* InPlayableGroup)
{
	if (!bIsTickingTransitions)
	{
		GroupsToTickTransitions.Remove(InPlayableGroup);
	}
}

TArray<TWeakObjectPtr<UAvaPlayableGroup>> UAvaPlayableGroupManager::GetPlayableGroups(FName InChannelName) const
{
	TArray<TWeakObjectPtr<UAvaPlayableGroup>> Groups;
	if (InChannelName.IsNone())
	{
		for (const TPair<FName, TObjectPtr<UAvaPlayableGroupChannelManager>>& ChannelManager : ChannelManagers)
		{
			if (ChannelManager.Value)
			{
				ChannelManager.Value->GetPlayableGroups(Groups);
			}
		}
	}
	else
	{
		if (const UAvaPlayableGroupChannelManager* ChannelManager = FindChannelManager(InChannelName))
		{
			ChannelManager->GetPlayableGroups(Groups);
		}
	}
	return Groups;
}

UAvaPlayableGroup* UAvaPlayableGroupManager::FindPlayableGroupForWorld(const UWorld* InWorld) const
{
	for (const TPair<FName, TObjectPtr<UAvaPlayableGroupChannelManager>>& ChannelManager : ChannelManagers)
	{
		if (ChannelManager.Value)
		{
			if (UAvaPlayableGroup* PlayableGroup = ChannelManager.Value->FindPlayableGroupForWorld(InWorld))
			{
				return PlayableGroup;
			}
		}
	}
	return nullptr;
}

void UAvaPlayableGroupManager::BeginDestroy()
{
	Shutdown();
	Super::BeginDestroy();
}

void UAvaPlayableGroupManager::OnGameInstanceEndPlay(UAvaGameInstance* InGameInstance, FName InChannelName)
{
	FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetChannelMutable(InChannelName);
	if (!Channel.IsValidChannel())
	{
		return;
	}

	// In the current design, there is only one playable group active on a channel, i.e.
	// the graph of playables and groups must end on a root "playable group".
	// This is resolved by the playback graph(s) running on that channel.
	//
	// When we receive an EndPlay event from a game instance for a given channel, we need to
	// check if that is the current root playable group, and if it is, clear out the
	// channels association with it.
	//
	// If the channel is live, it will then start rendering the placeholder graphic, if
	// configured to do so, on the next slate tick.
	
	if (const UAvaPlayableGroup* PlayableGroup = Channel.GetLastActivePlayableGroup())
	{
		if (PlayableGroup->GetGameInstance() == InGameInstance)
		{
			Channel.UpdateRenderTarget(nullptr, nullptr);
			Channel.UpdateAudioDevice(FAudioDeviceHandle());
		}
	}

	// If the channel is not live, the placeholder graphic doesn't render
	// so we need to explicitly clear the channel here.
	if (Channel.GetState() != EAvaBroadcastChannelState::Live)
	{
		if (UTextureRenderTarget2D* const RenderTarget = Channel.GetCurrentRenderTarget(true))
		{
			UE::AvaBroadcastRenderTargetMediaUtils::ClearRenderTarget(RenderTarget);
		}
	}
}

void UAvaPlayableGroupManager::UpdateLevelStreaming()
{
	TGuardValue TransitionsTickGuard(bIsUpdatingStreaming, true);		
	for (TSet<TWeakObjectPtr<UAvaPlayableGroup>>::TIterator GroupIterator(GroupsToUpdateStreaming); GroupIterator; ++GroupIterator)
	{
		const UAvaPlayableGroup* GroupToUpdate = GroupIterator->Get();
		// We only update streaming if the group is not playing.
		if (GroupToUpdate && !GroupToUpdate->IsWorldPlaying())
		{
			// World may not be created yet.
			if (UWorld* PlayWorld = GroupToUpdate->GetPlayWorld())
			{
				// (This is normally updated by the game viewport client when the world is playing.)
				PlayWorld->UpdateLevelStreaming();

				// Check if still has streaming. If not, from the list.
				if (!PlayWorld->HasStreamingLevelsToConsider())
				{
					GroupIterator.RemoveCurrent();
				}
			}
		}
		else
		{
			GroupIterator.RemoveCurrent();
		}
	}
}

void UAvaPlayableGroupManager::TickTransitions(double InDeltaSeconds)
{
	TGuardValue TransitionsTickGuard(bIsTickingTransitions, true);
	for (TSet<TWeakObjectPtr<UAvaPlayableGroup>>::TIterator GroupIterator(GroupsToTickTransitions); GroupIterator; ++GroupIterator)
	{
		bool bHasTransitions = false;
		if (UAvaPlayableGroup* GroupToTick = GroupIterator->Get())
		{
			GroupToTick->TickTransitions(InDeltaSeconds);
			bHasTransitions = GroupToTick->HasTransitions();
		}

		// Group automatically deregister if stale or they don't have active transitions.
		if (!bHasTransitions)
		{
			GroupIterator.RemoveCurrent();
		}
	}
}

#undef LOCTEXT_NAMESPACE