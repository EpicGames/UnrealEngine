// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"

#include "AvaPlayableGroupManager.generated.h"

class IAvaMediaSynchronizedEventDispatcher;
class UAvaGameInstance;
class UAvaPlayableGroup;
class UAvaPlayableGroupManager;
class UGameInstance;

/**
 * Manager for the shared playable groups per channel.
 */
UCLASS()
class UAvaPlayableGroupChannelManager : public UObject
{
	GENERATED_BODY()

public:
	UAvaPlayableGroup* GetOrCreatePlayableGroup(UGameInstance* InExistingGameInstance, bool bInIsRemoteProxy);

	UAvaPlayableGroupManager* GetPlayableGroupManager() const;

	void GetPlayableGroups(TArray<TWeakObjectPtr<UAvaPlayableGroup>>& OutGroups) const;

	UAvaPlayableGroup* FindPlayableGroupForWorld(const UWorld* InWorld) const;
	
protected:
	//~ Begin UObject
	virtual void BeginDestroy() override;
	//~ End UObject

	void Shutdown();
	
protected:
	FName ChannelName;

	/**
	 * Shared playable groups for this channel.
	 * The manager only keeps weak pointers.
	 * Groups are owned by their respective playables. 
	 */
	TArray<TWeakObjectPtr<UAvaPlayableGroup>> PlayableGroupsWeak;

	friend class UAvaPlayableGroupManager;
};

/**
 * Manager for the shared playable groups.
 * The scope of this manager is either global (in the global playback manager)
 * or for a given playback manager.
 */
UCLASS()
class UAvaPlayableGroupManager : public UObject
{
	GENERATED_BODY()
	
public:
	void Init();

	void Shutdown();

	void Tick(double InDeltaSeconds);

	void PushSynchronizedEvent(FString&& InEventSignature, TUniqueFunction<void()> InFunction);

	bool IsSynchronizedEventPushed(const FString& InEventSignature) const;

	UAvaPlayableGroupChannelManager* FindChannelManager(const FName& InChannelName) const
	{
		const TObjectPtr<UAvaPlayableGroupChannelManager>* ChannelManager = ChannelManagers.Find(InChannelName);
		return ChannelManager ? *ChannelManager : nullptr;
	}

	UAvaPlayableGroupChannelManager* FindOrAddChannelManager(const FName& InChannelName);

	/**
	 * @brief Returns the playable group for the given channel.
	 * @param InChannelName Channel name, should correspond to a configured broadcast channel.
	 * @param bInIsRemoteProxy Indicate if the implementation is local or remote.
	 * @return Playable group.
	 */
	UAvaPlayableGroup* GetOrCreateSharedPlayableGroup(const FName& InChannelName, bool bInIsRemoteProxy);

	void RegisterForLevelStreamingUpdate(UAvaPlayableGroup* InPlayableGroup);
	void UnregisterFromLevelStreamingUpdate(UAvaPlayableGroup* InPlayableGroup);

	void RegisterForTransitionTicking(UAvaPlayableGroup* InPlayableGroup);
	void UnregisterFromTransitionTicking(UAvaPlayableGroup* InPlayableGroup);

	/**
	 * Returns playable groups part of the given channel, or all playable groups if channel is None.
	 */
	TArray<TWeakObjectPtr<UAvaPlayableGroup>> GetPlayableGroups(FName InChannelName = NAME_None) const;

	/**
	 * Return the playable group corresponding to the given world.
	 */
	UAvaPlayableGroup* FindPlayableGroupForWorld(const UWorld* InWorld) const;

protected:
	//~ Begin UObject
	virtual void BeginDestroy() override;
	//~ End UObject

	// Move to UAvaGameInstancePlayableGroup
	void OnGameInstanceEndPlay(UAvaGameInstance* InGameInstance, FName InChannelName);

	void UpdateLevelStreaming();
	
	void TickTransitions(double InDeltaSeconds);

protected:
	UPROPERTY(Transient)
	TMap<FName, TObjectPtr<UAvaPlayableGroupChannelManager>> ChannelManagers;

	bool bIsUpdatingStreaming = false;
	TSet<TWeakObjectPtr<UAvaPlayableGroup>> GroupsToUpdateStreaming;

	bool bIsTickingTransitions = false;
	TSet<TWeakObjectPtr<UAvaPlayableGroup>> GroupsToTickTransitions;

	TArray<TWeakObjectPtr<UAvaPlayableGroup>> GroupsToTickEvents;
	
	TSharedPtr<IAvaMediaSynchronizedEventDispatcher> SynchronizedEventDispatcher;
};