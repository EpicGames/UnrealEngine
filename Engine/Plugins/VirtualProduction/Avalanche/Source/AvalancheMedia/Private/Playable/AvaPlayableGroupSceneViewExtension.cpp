// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPlayableGroupSceneViewExtension.h"

#include "Playable/AvaPlayableGroup.h"
#include "SceneView.h"

FAvaPlayableGroupSceneViewExtension::FAvaPlayableGroupSceneViewExtension(const FAutoRegister& InAutoReg)
	: FSceneViewExtensionBase(InAutoReg)
{
}

void FAvaPlayableGroupSceneViewExtension::SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView)
{
	if (!InViewFamily.Scene)
	{
		return;
	}
	
	const UWorld* ViewWorld = InViewFamily.Scene->GetWorld();
	if (!ViewWorld)
	{
		return;
	}

	if (UAvaPlayableGroup* ViewPlayableGroup = UAvaPlayableGroup::FindPlayableGroupForWorld(ViewWorld))
	{
		ViewPlayableGroup->SetupView(InViewFamily, InView);
	}
}