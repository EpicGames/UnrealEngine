// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "SceneTypes.h"
#include "SceneViewExtension.h"

class FAvaPlayableGroupSceneViewExtension final : public FSceneViewExtensionBase
{
public:
	FAvaPlayableGroupSceneViewExtension(const FAutoRegister& InAutoReg);
	virtual ~FAvaPlayableGroupSceneViewExtension() override = default;

	void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override;
};
