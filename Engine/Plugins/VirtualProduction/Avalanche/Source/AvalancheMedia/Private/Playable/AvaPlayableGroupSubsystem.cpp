// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPlayableGroupSubsystem.h"

#include "Playable/AvaPlayableGroupManager.h"
#include "Playable/PlayableGroups/AvaGameViewportPlayableGroup.h"

void UAvaPlayableGroupSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
}

void UAvaPlayableGroupSubsystem::Deinitialize()
{
	if (PlayableGroupManager)
	{
		TArray<TWeakObjectPtr<UAvaPlayableGroup>> PlayableGroupsWeak = PlayableGroupManager->GetPlayableGroups();
		for (const TWeakObjectPtr<UAvaPlayableGroup>& PlayableGroupWeak : PlayableGroupsWeak)
		{
			UAvaGameViewportPlayableGroup* GameViewportPlayableGroup = Cast<UAvaGameViewportPlayableGroup>(PlayableGroupWeak.Get());
	
			if (GameViewportPlayableGroup && GameViewportPlayableGroup->GetGameInstance() == GetGameInstance())
			{
				GameViewportPlayableGroup->DetachGameInstance();
			}
		}
	}
	
	PlayableGroupManager = nullptr;
	
	Super::Deinitialize();
}
