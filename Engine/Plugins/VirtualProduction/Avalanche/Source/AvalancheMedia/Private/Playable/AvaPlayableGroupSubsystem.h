// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/GameInstanceSubsystem.h"

#include "AvaPlayableGroupSubsystem.generated.h"

class UAvaPlayableGroupManager;

UCLASS()
class UAvaPlayableGroupSubsystem : public UGameInstanceSubsystem
{
	GENERATED_BODY()

public:
	//~ Begin UGameInstanceSubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End UGameInstanceSubsystem

	UPROPERTY()
	TObjectPtr<UAvaPlayableGroupManager> PlayableGroupManager;
};
