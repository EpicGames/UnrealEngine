// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playable/AvaPlayableLibrary.h"

#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Playable/AvaPlayable.h"
#include "Playable/AvaPlayableAssetUserData.h"
#include "Playable/AvaPlayableGroup.h"
#include "Playable/Playables/AvaPlayableLevelStreaming.h"
#include "Playable/Transition/AvaPlayableTransition.h"

namespace UE::AvaPlayableLibrary::Private
{
	ULevel* GetLevel(const UObject* InWorldContextObject)
	{
		if (!InWorldContextObject)
		{
			return nullptr;
		}

		ULevel* Level = InWorldContextObject->GetTypedOuter<ULevel>();
		if (!Level && GEngine)
		{
			if (const UWorld* World = GEngine->GetWorldFromContextObject(InWorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
			{
				Level = World->PersistentLevel;
			}
		}
		return Level;
	}
}


UAvaPlayable* UAvaPlayableLibrary::GetPlayable(const UObject* InWorldContextObject)
{
	using namespace UE::AvaPlayableLibrary::Private;
	ULevel* Level = GetLevel(InWorldContextObject);
	if (!Level)
	{
		return nullptr;
	}

	if (const UAvaPlayableAssetUserData* PlayableUserData = Level->GetAssetUserData<UAvaPlayableAssetUserData>())
	{
		if (UAvaPlayable* Playable = PlayableUserData->PlayableWeak.Get())
		{
			return Playable;
		}
	}

	return nullptr;
}

UAvaPlayableTransition* UAvaPlayableLibrary::GetPlayableTransition(const UAvaPlayable* InPlayable)
{
	if (!InPlayable)
	{
		return nullptr;
	}

	UAvaPlayableGroup* PlayableGroup = InPlayable->GetPlayableGroup();
	if (!PlayableGroup)
	{
		return nullptr;
	}

	UAvaPlayableTransition* FoundTransition = nullptr;
	PlayableGroup->ForEachPlayableTransition([&FoundTransition, InPlayable](UAvaPlayableTransition* InTransition)
	{
		if (InTransition->IsEnterPlayable(InPlayable)
			|| InTransition->IsPlayingPlayable(InPlayable)
			|| InTransition->IsExitPlayable(InPlayable))
		{
			FoundTransition = InTransition;
			return false;
		}		
		return true;
	});

	return FoundTransition;
}

bool UAvaPlayableLibrary::UpdatePlayableRemoteControlValues(const UObject* InWorldContextObject)
{
	if (UAvaPlayable* Playable = GetPlayable(InWorldContextObject))
	{
		if (UAvaPlayableTransition* Transition = GetPlayableTransition(Playable))
		{
			// Assume that if the RC Values need to be updated, it's because of an Enter Playable not yet having its Update RC called
			constexpr bool bIsEnterPlayable = true;

			if (const TSharedPtr<FAvaPlayableRemoteControlValues> RemoteControlValues = Transition->GetValuesForPlayable(Playable, bIsEnterPlayable))
			{
				if (Playable->UpdateRemoteControlCommand(RemoteControlValues.ToSharedRef(), EAvaPlayableRCUpdateFlags::ExecuteControllerBehaviors) == EAvaPlayableCommandResult::Executed)
				{
					return true;
				}
			}
		}
	}
	return false;
}

bool UAvaPlayableLibrary::IsPlayableHidden(const UObject* InWorldContextObject)
{
	const UAvaPlayableLevelStreaming* LevelStreamingPlayable = Cast<UAvaPlayableLevelStreaming>(GetPlayable(InWorldContextObject));
	return LevelStreamingPlayable ? LevelStreamingPlayable->GetShouldBeHidden() : false;
}

bool UAvaPlayableLibrary::SetPlayableHidden(const UObject* InWorldContextObject, bool bInShouldBeHidden)
{
	if (UAvaPlayableLevelStreaming* LevelStreamingPlayable = Cast<UAvaPlayableLevelStreaming>(GetPlayable(InWorldContextObject)))
	{
		LevelStreamingPlayable->SetShouldBeHidden(bInShouldBeHidden);
		return true;
	}
	return false;
}

/**
 * Pending Latent Action to push a sync event on the cluster.
 * The event signature includes the playable id as well.
 * The given signature need only be unique in terms of any sequence of synchronized events for the given playable.
 */
class FAvaPlayableSyncEventLatentAction : public FPendingLatentAction
{
public:
	FAvaPlayableSyncEventLatentAction(const FLatentActionInfo& InLatentInfo, UAvaPlayable* InPlayable, const FString& InEventSignature, bool& InSuccess)
		: ExecutionFunction(InLatentInfo.ExecutionFunction)
		, OutputLink(InLatentInfo.Linkage)
		, CallbackTarget(InLatentInfo.CallbackTarget)
		, TimeRemaining(10.0)
		, OutSuccess(InSuccess)
	{
		if (UAvaPlayableGroup* Group = (InPlayable ? InPlayable->GetPlayableGroup() : nullptr))
		{
			// Build a unique playable id string. 
			const FString PlayableId = InPlayable->GetInstanceId().IsValid() ? InPlayable->GetInstanceId().ToString() : InPlayable->GetSourceAssetPath().ToString();

			// Build unique signature for this event.
			FString PlayableEventSignature = FString(TEXT("PlayableSyncEvent_")) + PlayableId + FString(TEXT("_")) + InEventSignature;
	
			if (!Group->IsSynchronizedEventPushed(PlayableEventSignature))
			{
				SyncFence = MakeShared<bool>(false);
				TWeakPtr<bool> SyncFenceWeak = SyncFence;

				auto SyncEventHandler = [SyncFenceWeak]()
				{
					if (TSharedPtr<bool> SyncFencePtr = SyncFenceWeak.Pin())
					{
						*SyncFencePtr = true;
					}
				};

				// Keep a copy of the signature for the description and logs.
				EventSignature = PlayableEventSignature;

				Group->PushSynchronizedEvent(MoveTemp(PlayableEventSignature), MoveTemp(SyncEventHandler));
			}
		}
	}
	
	void FailedOperation(FLatentResponse& InResponse) const
	{
		OutSuccess = false;
		InResponse.FinishAndTriggerIf(true, ExecutionFunction, OutputLink, CallbackTarget);
	}

	//~ Begin FPendingLatentAction
	virtual void UpdateOperation(FLatentResponse& InResponse) override
	{
		if (!SyncFence)
		{
			FailedOperation(InResponse);
		}
		
		if (*SyncFence)
		{
			OutSuccess = true;
			InResponse.FinishAndTriggerIf(true, ExecutionFunction, OutputLink, CallbackTarget);
			return;
		}

		// Timed out
		TimeRemaining -= InResponse.ElapsedTime();
		if (TimeRemaining <= 0.0f)
		{
			UE_LOGF(LogAvaPlayable, Warning, "Playable Sync Event %ls: Timed out", *EventSignature);
			FailedOperation(InResponse);
		}
	}

#if WITH_EDITOR
	virtual FString GetDescription() const override
	{
		return FString::Printf(TEXT("Playable Sync Event: %s"), *EventSignature);
	}
#endif
	//~ End FPendingLatentAction

private:
	/** Latent Action Info - The function to execute. */
	FName ExecutionFunction;

	/** Latent Action Info - The resume point within the function to execute. */
	int32 OutputLink;

	/** Latent Action Info - Object to execute the function on. */
	FWeakObjectPtr CallbackTarget;

	/** Copy of the event signature for logs and description. */
	FString EventSignature;

	/** Keeps track of the remaining time, in seconds, before the operation times out. */
	float TimeRemaining;

	/** Output parameter - Indicate if the operation completed successfully. */
	bool& OutSuccess;
	
	/** Simple shared bool as a fence to know when the event is completed. */
	TSharedPtr<bool> SyncFence;
};

void UAvaPlayableLibrary::PlayableSyncEventLatent(const UObject* InWorldContextObject, FLatentActionInfo InLatentInfo, const FString& InEventSignature, bool& bInSuccess)
{
	bInSuccess = false;

	UWorld* World = GEngine->GetWorldFromContextObject(InWorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	UAvaPlayable* Playable = GetPlayable(InWorldContextObject);
	
	if (!World || !Playable)
	{
		return;
	}
	
	FLatentActionManager& LatentManager = World->GetLatentActionManager();
	if (LatentManager.FindExistingAction<FAvaPlayableSyncEventLatentAction>(InLatentInfo.CallbackTarget, InLatentInfo.UUID) == nullptr)
	{
		FAvaPlayableSyncEventLatentAction* NewAction = new FAvaPlayableSyncEventLatentAction(InLatentInfo, Playable, InEventSignature, bInSuccess);
		LatentManager.AddNewAction(InLatentInfo.CallbackTarget, InLatentInfo.UUID, NewAction);
	}
}