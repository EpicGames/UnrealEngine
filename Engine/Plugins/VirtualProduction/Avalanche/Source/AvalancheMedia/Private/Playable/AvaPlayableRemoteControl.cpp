// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playable/AvaPlayableRemoteControl.h"

#include "Action/RCAction.h"
#include "Action/RCActionContainer.h"
#include "Action/RCPropertyIdAction.h"
#include "AvaMediaSerializationUtils.h"
#include "Backends/JsonStructDeserializerBackend.h"
#include "Backends/JsonStructSerializerBackend.h"
#include "Behaviour/Builtin/Path/RCSetAssetByPathBehaviour.h"
#include "Behaviour/RCBehaviour.h"
#include "Controller/RCController.h"
#include "IRemoteControlModule.h"
#include "RCVirtualProperty.h"
#include "RemoteControlPropertyIdRegistry.h"
#include "Templates/CopyQualifiersFromTo.h"

DEFINE_LOG_CATEGORY(LogAvaPlayableRemoteControl);

namespace UE::AvaPlayableRemoteControl::Private
{
	/**
	 * Safely casts a Remote Control Entity from one Type to the other
	 * @param InEntity the entity to cast
	 * @return the entity casted to the desired type if it was derived from it. nullptr otherwise
	 */
	template<typename To, typename From>
	TSharedPtr<TCopyQualifiersFromTo_T<From, To>> CastEntity(const TSharedPtr<From>& InEntity)
	{
		const UScriptStruct* EntityStruct = InEntity.IsValid() ? InEntity->GetStruct() : nullptr;
		if (EntityStruct && EntityStruct->IsChildOf(To::StaticStruct()))
		{
			return StaticCastSharedPtr<TCopyQualifiersFromTo_T<From, To>>(InEntity);
		}
		return nullptr;
	}

	bool ResolveObjectPropertyForReadOnly(UObject* Object, FRCFieldPathInfo PropertyPath, FRCObjectReference& OutObjectRef, FString& OutErrorText)
	{
		if (!Object)
		{
			OutErrorText = FString::Printf(TEXT("Invalid object to resolve property '%s'"), *PropertyPath.GetFieldName().ToString());
			return false;
		}

		bool bSuccess = true;
		
		if (PropertyPath.GetSegmentCount() != 0)
		{
			if (PropertyPath.Resolve(Object))
			{
				OutObjectRef = FRCObjectReference{ERCAccess::READ_ACCESS, Object, MoveTemp(PropertyPath)};
			}
			else
			{
				OutErrorText = FString::Printf(TEXT("Object property: %s could not be resolved on object: %s"), *PropertyPath.GetFieldName().ToString(), *Object->GetPathName());
				bSuccess = false;
			}
		}
		else
		{
			OutObjectRef = FRCObjectReference{ERCAccess::READ_ACCESS, Object};
		}
		return bSuccess;
	}
	
	inline bool GetObjectRef(const TSharedPtr<const FRemoteControlProperty>& InField, const ERCAccess InAccess, FRCObjectReference& OutObjectRef)
	{
		if (!InField.IsValid())
		{
			return false;
		}
		if (UObject* FieldBoundObject = InField->GetBoundObject(); IsValid(FieldBoundObject))
		{
			FString ErrorText;
			bool bSuccess = false;
			
			if (InAccess == ERCAccess::READ_ACCESS)
			{
				// Fix for private/protected properties: we allow reading the property even if private/protected.
				// This is necessary for the case of private/protect props that have a setter.
				// RC doesn't have a path to allow reading those (it doesn't support getters).
				bSuccess = ResolveObjectPropertyForReadOnly(FieldBoundObject, InField->FieldPathInfo, OutObjectRef, ErrorText);
			}
			else
			{
				bSuccess = IRemoteControlModule::Get().ResolveObjectProperty(InAccess, FieldBoundObject, InField->FieldPathInfo, OutObjectRef, &ErrorText);
			}

			if (bSuccess)
			{
				return true;
			}
			
			UE_LOGF(LogAvaPlayableRemoteControl, Error,
				"Couldn\'t resolve object property \"%ls\" in object \"%ls\": %ls",
				*InField->FieldName.ToString(), *FieldBoundObject->GetPathName(), *ErrorText);
		}
		else
		{
			UE_LOGF(LogAvaPlayableRemoteControl, Error,
				"Couldn\'t resolve object property \"%ls\": Invalid Field Bound Object.",
				*InField->FieldName.ToString());
		}
		return false;
	}

	inline bool HasAccess(const TSharedPtr<const FRemoteControlEntity>& InRemoteControlEntity, const ERCAccess InAccess, FString& OutErrorText)
	{
		using namespace UE::AvaPlayableRemoteControl::Private;

		if (!InRemoteControlEntity.IsValid())
		{
			OutErrorText = TEXT("Entity is null"); 
			return false;
		}

		TSharedPtr<const FRemoteControlProperty> Field = CastEntity<FRemoteControlProperty>(InRemoteControlEntity);
		if (!Field.IsValid())
		{
			OutErrorText = FString::Printf(TEXT("Wrong Entity type \"%s\", expected \"%s\"."), *GetNameSafe(InRemoteControlEntity->GetStruct()), *GetNameSafe(FRemoteControlProperty::StaticStruct())); 
			return false;
		}
		
		if (UObject* FieldBoundObject = Field->GetBoundObject(); IsValid(FieldBoundObject))
		{
			FRCObjectReference DummyObjectRef;
			if (InAccess == ERCAccess::READ_ACCESS)
			{
				return	ResolveObjectPropertyForReadOnly(FieldBoundObject, Field->FieldPathInfo, DummyObjectRef, OutErrorText);
			}
			return IRemoteControlModule::Get().ResolveObjectProperty(InAccess, FieldBoundObject, Field->FieldPathInfo, DummyObjectRef, &OutErrorText);
		}
		
		OutErrorText = TEXT("Invalid bound object");
		return false;
	}
}

bool UE::AvaPlayableRemoteControl::HasReadAccess(const TSharedPtr<const FRemoteControlEntity>& InRemoteControlEntity, FString& OutErrorText)
{
	using namespace UE::AvaPlayableRemoteControl::Private;
	return HasAccess(InRemoteControlEntity, ERCAccess::READ_ACCESS, OutErrorText);
}

bool UE::AvaPlayableRemoteControl::HasWriteAccess(const TSharedPtr<const FRemoteControlEntity>& InRemoteControlEntity, FString& OutErrorText)
{
	using namespace UE::AvaPlayableRemoteControl::Private;
	return HasAccess(InRemoteControlEntity, ERCAccess::WRITE_ACCESS, OutErrorText);
}

EAvaPlayableRemoteControlResult UE::AvaPlayableRemoteControl::GetValueOfEntity(const TSharedPtr<const FRemoteControlEntity>& InRemoteControlEntity, TArray<uint8>& OutValue)
{
	using namespace UE::AvaPlayableRemoteControl::Private;
	OutValue.Reset();

	TSharedPtr<const FRemoteControlProperty> Field = CastEntity<FRemoteControlProperty>(InRemoteControlEntity);
	if (!Field.IsValid())
	{
		return EAvaPlayableRemoteControlResult::InvalidParameter;
	}

	FRCObjectReference ObjectRef;
	if (!GetObjectRef(Field, ERCAccess::READ_ACCESS, ObjectRef))
	{
		return EAvaPlayableRemoteControlResult::ReadAccessDenied;
	}
	FMemoryWriter Writer = FMemoryWriter(OutValue);
	FJsonStructSerializerBackend WriterBackend = FJsonStructSerializerBackend(Writer, EStructSerializerBackendFlags::Default);
	return IRemoteControlModule::Get().GetObjectProperties(ObjectRef, WriterBackend) ? EAvaPlayableRemoteControlResult::Completed : EAvaPlayableRemoteControlResult::ReadPropertyFailed;
}

EAvaPlayableRemoteControlResult UE::AvaPlayableRemoteControl::GetValueOfEntity(const TSharedPtr<const FRemoteControlEntity>& InRemoteControlEntity, FString& OutValue)
{
	TArray<uint8> ValueAsBytes;
	const EAvaPlayableRemoteControlResult Result = GetValueOfEntity(InRemoteControlEntity, ValueAsBytes);
	AvaMediaSerializationUtils::JsonValueConversion::BytesToString(ValueAsBytes, OutValue);
	return Result;
}

EAvaPlayableRemoteControlResult UE::AvaPlayableRemoteControl::SetValueOfEntity(const TSharedPtr<FRemoteControlEntity>& InRemoteControlEntity, const TArrayView<const uint8>& InValue)
{
	using namespace UE::AvaPlayableRemoteControl::Private;

	TSharedPtr<FRemoteControlProperty> Field = CastEntity<FRemoteControlProperty>(InRemoteControlEntity);
	if (!Field.IsValid())
	{
		return EAvaPlayableRemoteControlResult::InvalidParameter;
	}
	
	FRCObjectReference ObjectRefRead;
	if (GetObjectRef(Field, ERCAccess::READ_ACCESS, ObjectRefRead))
	{
		TArray<uint8> CurrentValueAsBytes;
		FMemoryWriter Writer(CurrentValueAsBytes);
		FJsonStructSerializerBackend WriterBackend = FJsonStructSerializerBackend(Writer, EStructSerializerBackendFlags::Default);
		if (IRemoteControlModule::Get().GetObjectProperties(ObjectRefRead, WriterBackend))
		{
			if (CurrentValueAsBytes == InValue)
			{
				// if the given value is already set, don't do anything
				return EAvaPlayableRemoteControlResult::UpToDate;
			}
		}
	}

	FRCObjectReference ObjectRefWrite;
	if (!GetObjectRef(Field, ERCAccess::WRITE_ACCESS, ObjectRefWrite))
	{
		return EAvaPlayableRemoteControlResult::WriteAccessDenied;
	}

	FMemoryReaderView Reader(InValue);
	FJsonStructDeserializerBackend ReaderBackend = FJsonStructDeserializerBackend(Reader);

	// Notes:
	// - if RemoteControl.EnableOngoingChangeOptimization is enabled, PostEditChangeProperty is not called right away
	// there might be a delay (of 0.2 seconds) before it is called.
	// - OnPropertyChangedDelegate (OnExposedPropertiesModified()) is a "per frame" event and is broadcast from URemoteControlPreset::OnEndFrame().
	const bool bDeserializationSucceeded = IRemoteControlModule::Get().SetObjectProperties(ObjectRefWrite, ReaderBackend, ERCPayloadType::Json);

	if (!bDeserializationSucceeded && !ReaderBackend.GetLastErrorMessage().IsEmpty())
	{
		UObject* FieldBoundObject = Field->GetBoundObject();
		UE_LOGF(LogAvaPlayableRemoteControl, Error,
			"Couldn\'t set object property \"%ls\" in object \"%ls\" - Deserializer Error: %ls",
			*Field->FieldName.ToString(), IsValid(FieldBoundObject) ? *FieldBoundObject->GetPathName() : TEXT("[InvalidFieldBoundObject]"),
			*ReaderBackend.GetLastErrorMessage());
	}

#if WITH_EDITOR
	UObject* const Object = ObjectRefWrite.Object.Get();
	if (bDeserializationSucceeded && IsValid(Object))
	{
		FEditPropertyChain EditPropertyChain;
		ObjectRefWrite.PropertyPathInfo.ToEditPropertyChain(EditPropertyChain);

		// Note: Only PostEditChangeChainProperty is called here because PostEditChangeProperty is already dealt with in RC.
		// Ideally, this should be in RC so that PostEditChangeProperty does not get called twice.
		if (!EditPropertyChain.IsEmpty())
		{
			FPropertyChangedEvent PropertyEvent = ObjectRefWrite.PropertyPathInfo.ToPropertyChangedEvent(EPropertyChangeType::ValueSet);

			FPropertyChangedChainEvent ChainEvent(EditPropertyChain, PropertyEvent);

			TArray<TMap<FString, int32>> ArrayIndicesPerObject;
			{
				TMap<FString, int32> ArrayIndices;
				ArrayIndices.Reserve(ObjectRefWrite.PropertyPathInfo.Segments.Num());

				for (FRCFieldPathSegment& Segment : ObjectRefWrite.PropertyPathInfo.Segments)
				{
					ArrayIndices.Add(Segment.Name.ToString(), Segment.ArrayIndex);
				}

				ArrayIndicesPerObject.Add(MoveTemp(ArrayIndices));
			}

			ChainEvent.ObjectIteratorIndex = 0;
			ChainEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);

			Object->PostEditChangeChainProperty(ChainEvent);
		}
	}
#endif

	return bDeserializationSucceeded ? EAvaPlayableRemoteControlResult::Completed : EAvaPlayableRemoteControlResult::WritePropertyFailed;
}

EAvaPlayableRemoteControlResult UE::AvaPlayableRemoteControl::SetValueOfEntity(const TSharedPtr<FRemoteControlEntity>& InRemoteControlEntity, const FString& InValue)
{
	return SetValueOfEntity(InRemoteControlEntity, AvaMediaSerializationUtils::JsonValueConversion::ValueToConstBytesView(InValue));
}

EAvaPlayableRemoteControlResult UE::AvaPlayableRemoteControl::GetValueOfController(URCVirtualPropertyBase* InController, TArray<uint8>& OutValue)
{
	OutValue.Reset();

	if (!InController)
	{
		return EAvaPlayableRemoteControlResult::InvalidParameter;
	}

	FMemoryWriter Writer = FMemoryWriter(OutValue);
	FJsonStructSerializerBackend WriterBackend = FJsonStructSerializerBackend(Writer, EStructSerializerBackendFlags::Default);
	InController->SerializeToBackend(WriterBackend);
	return EAvaPlayableRemoteControlResult::Completed;
}

EAvaPlayableRemoteControlResult UE::AvaPlayableRemoteControl::GetValueOfController(URCVirtualPropertyBase* InController, FString& OutValue)
{
	TArray<uint8> ValueAsBytes;
	const EAvaPlayableRemoteControlResult Result = GetValueOfController(InController, ValueAsBytes);

	AvaMediaSerializationUtils::JsonValueConversion::BytesToString(ValueAsBytes, OutValue);
	return Result;
}

EAvaPlayableRemoteControlResult UE::AvaPlayableRemoteControl::SetValueOfController(URCVirtualPropertyBase* InController, const TArrayView<const uint8>& InValue)
{
	if (!InController)
	{
		return EAvaPlayableRemoteControlResult::InvalidParameter;
	}

	// TODO: Use the RC error callback to catch errors in the controller actions.
	FMemoryReaderView Reader(InValue);
	FJsonStructDeserializerBackend ReaderBackend = FJsonStructDeserializerBackend(Reader);
	return InController->DeserializeFromBackend(ReaderBackend) ? EAvaPlayableRemoteControlResult::Completed : EAvaPlayableRemoteControlResult::WritePropertyFailed;
}

EAvaPlayableRemoteControlResult UE::AvaPlayableRemoteControl::SetValueOfController(URCVirtualPropertyBase* InController, const FString& InValue)
{
	return SetValueOfController(InController, AvaMediaSerializationUtils::JsonValueConversion::ValueToConstBytesView(InValue));
}

EAvaPlayableRemoteControlResult UE::AvaPlayableRemoteControl::SetValueOfController(URCVirtualPropertyBase* InController, const FString& InValue, bool bInBehaviorsEnabled)
{
	if (bInBehaviorsEnabled)
	{
		return SetValueOfController(InController, InValue);
	}
	
	FScopedPushControllerBehavioursEnable PushBehavioursEnable(InController, false);
	return SetValueOfController(InController, InValue);
}

FString UE::AvaPlayableRemoteControl::EnumToString(EAvaPlayableRemoteControlResult InValue)
{
	return StaticEnum<EAvaPlayableRemoteControlResult>()->GetNameStringByValue(static_cast<int64>(InValue));
}

namespace UE::AvaPlayableRemoteControl
{
	FScopedPushControllerBehavioursEnable::FScopedPushControllerBehavioursEnable(URCVirtualPropertyBase* InVirtualProperty, bool bInBehavioursEnabled)
		: VirtualProperty(InVirtualProperty)
	{
		if (URCController* const Controller = Cast<URCController>(InVirtualProperty))
		{
			PreviousBehavioursEnabled.Reserve(Controller->GetBehaviors().Num());
			for (URCBehaviour* const Behavior : Controller->GetBehaviors())
			{
				if (Behavior)
				{
					PreviousBehavioursEnabled.Add(Behavior->bIsEnabled);
					Behavior->bIsEnabled = bInBehavioursEnabled;
				}
			}
		}
	}

	FScopedPushControllerBehavioursEnable::~FScopedPushControllerBehavioursEnable()
	{
		if (URCController* const Controller = Cast<URCController>(VirtualProperty))
		{
			int32 BehaviourIndex = 0;
			for (URCBehaviour* const Behavior : Controller->GetBehaviors())
			{
				if (Behavior)
				{
					Behavior->bIsEnabled = PreviousBehavioursEnabled[BehaviourIndex++];
				}
			}
		}
	}
}