// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playable/AvaPlayableRemoteControlPresetInfo.h"

#include "Controller/RCCustomControllerUtilities.h"
#include "Playable/AvaPlayableRemoteControl.h"
#include "Playable/AvaPlayableRemoteControlValues.h"
#include "RCVirtualProperty.h"

void FAvaPlayableRemoteControlPresetInfo::Refresh(const URemoteControlPreset* InRemoteControlPreset)
{
	EntitiesControlledByController.Reset();
	OverlappingControllers.Reset();

	if (IsValid(InRemoteControlPreset))
	{
		PresetId = InRemoteControlPreset->GetPresetId();
		
		// Set of entities for the current controller only.
		TSet<FGuid> TmpControlledEntities;
		TmpControlledEntities.Reserve(EntitiesControlledByController.GetAllocatedSize());
		
		const TArray<URCVirtualPropertyBase*> Controllers = InRemoteControlPreset->GetControllers();
		for (const URCVirtualPropertyBase* Controller : Controllers)
		{
			if (!Controller)
			{
				continue;
			}
			
			TmpControlledEntities.Reset();
			
			if (!UE::RCCustomControllers::GetEntitiesControlledByController(InRemoteControlPreset, Controller, TmpControlledEntities))
			{
				UE_LOGF(LogAvaPlayableRemoteControl, Warning, "Failed to get controlled entities for controller \"%ls\" (id:%ls).",
					*Controller->DisplayName.ToString(), *Controller->Id.ToString());
			}

			if (!TmpControlledEntities.IsEmpty())
			{
				// Merge in the controlled entity info map
				for (const FGuid& EntityId : TmpControlledEntities)
				{
					auto& EntityInfo = EntitiesControlledByController.FindOrAdd(EntityId);

					// Remark: we don't consider overlap with ignored controllers.
					if (!FAvaPlayableRemoteControlValues::ShouldIgnoreController(Controller))
					{
						EntityInfo.ControlledBy.AddUnique(Controller->Id);	
						if (EntityInfo.ControlledBy.Num() > 1)
						{
							OverlappingControllers.Append(EntityInfo.ControlledBy);
						}
					}
				}
			}
		}
	}
	else
	{
		PresetId.Invalidate();
	}

	bDirty = false;
}