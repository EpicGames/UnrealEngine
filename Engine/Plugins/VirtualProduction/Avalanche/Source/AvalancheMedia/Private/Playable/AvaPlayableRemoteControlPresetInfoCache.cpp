// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPlayableRemoteControlPresetInfoCache.h"

#include "AssetRegistry/AssetData.h"
#include "IAvaMediaModule.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"

IAvaPlayableRemoteControlPresetInfoCache& IAvaPlayableRemoteControlPresetInfoCache::Get()
{
	return IAvaMediaModule::Get().GetPlayableRemoteControlPresetInfoCache();
}

TSharedPtr<FAvaPlayableRemoteControlPresetInfo> FAvaPlayableRemoteControlPresetInfoCache::GetRemoteControlPresetInfo(const FSoftObjectPath& InAssetPath, const URemoteControlPreset* InRemoteControlPreset)
{
	if (InAssetPath.IsNull())
	{
		return nullptr;
	}

	OrderQueue.Remove(InAssetPath); // Removes preserves the order. O(n)
	OrderQueue.Add(InAssetPath); // Most recent is at the end of the array.

	if (TSharedPtr<FAvaPlayableRemoteControlPresetInfo>* ExistingEntry = PresetInfoCache.Find(InAssetPath))
	{
		if (ExistingEntry->IsValid())
		{
			if ((*ExistingEntry)->IsDirty())
			{
				(*ExistingEntry)->Refresh(InRemoteControlPreset);
			}
			return *ExistingEntry;
		}
	}

	TSharedPtr<FAvaPlayableRemoteControlPresetInfo> NewPresetInfo = MakeShared<FAvaPlayableRemoteControlPresetInfo>();
	NewPresetInfo->Refresh(InRemoteControlPreset);

	PresetInfoCache.Add(InAssetPath, NewPresetInfo);
	TrimCache();
	
	return NewPresetInfo;
}

void FAvaPlayableRemoteControlPresetInfoCache::Flush(const FSoftObjectPath& InAssetPath)
{
	if (const TSharedPtr<FAvaPlayableRemoteControlPresetInfo>* ExistingEntry = PresetInfoCache.Find(InAssetPath))
	{
		if (ExistingEntry->GetSharedReferenceCount() <= 1)
		{
			RemoveEntry(InAssetPath);
		}
	}
}

void FAvaPlayableRemoteControlPresetInfoCache::Flush()
{
	RemoveEntries([](const FSoftObjectPath& InAssetPath, const TSharedPtr<FAvaPlayableRemoteControlPresetInfo>& InPresetInfo)
	{
		return InPresetInfo.GetSharedReferenceCount() <= 1;
	});
}

int32 FAvaPlayableRemoteControlPresetInfoCache::GetMaximumCacheSize()
{
	return 100;	// This is not very critical, we just don't want an infinite size cache.
}

void FAvaPlayableRemoteControlPresetInfoCache::TrimCache()
{
	const int32 MaximumCacheSize = GetMaximumCacheSize(); 
	if (MaximumCacheSize > 0)
	{
		while (OrderQueue.Num() > MaximumCacheSize)
		{
			// LRU: oldest is at the start of the array.
			PresetInfoCache.Remove(OrderQueue[0]);
			OrderQueue.RemoveAt(0);
		}
	}
}

void FAvaPlayableRemoteControlPresetInfoCache::RemoveEntry(const FSoftObjectPath& InAssetPath)
{
	OrderQueue.Remove(InAssetPath);
	PresetInfoCache.Remove(InAssetPath);
}

void FAvaPlayableRemoteControlPresetInfoCache::RemoveEntries(TFunctionRef<bool(const FSoftObjectPath&, const TSharedPtr<FAvaPlayableRemoteControlPresetInfo>&)> InRemovePredicate)
{
	for (TMap<FSoftObjectPath, TSharedPtr<FAvaPlayableRemoteControlPresetInfo>>::TIterator EntryIt = PresetInfoCache.CreateIterator(); EntryIt; ++EntryIt)
	{
		if (InRemovePredicate(EntryIt.Key(), EntryIt.Value()))
		{
			OrderQueue.Remove(EntryIt.Key());
			EntryIt.RemoveCurrent();
		}
	}
}

void FAvaPlayableRemoteControlPresetInfoCache::OnPackageSaved(const FString& InPackageFileName, UPackage* InPackage, FObjectPostSaveContext InObjectSaveContext)
{
	if (InObjectSaveContext.IsProceduralSave())
	{
		return;
	}

	OnPackageModified(InPackage->GetFName());
}

void FAvaPlayableRemoteControlPresetInfoCache::OnAvaSyncPackageModified(IAvaMediaSyncProvider* InAvaMediaSyncProvider, const FName& InPackageName)
{
	UE_LOGF(LogAvaMedia, Verbose,
		"A sync operation has touched the package \"%ls\" on disk. Remote Control Preset Info Cache notified.",
		*InPackageName.ToString());

	OnPackageModified(InPackageName);
}

void FAvaPlayableRemoteControlPresetInfoCache::OnAssetRemoved(const FAssetData& InAssetData)
{
	// Remove entry for the given package.
	RemoveEntries([PackageName = InAssetData.PackageName](const FSoftObjectPath& InAssetPath, const TSharedPtr<FAvaPlayableRemoteControlPresetInfo>& InPresetInfo)
	{
		return InAssetPath.GetLongPackageFName() == PackageName;
	});
}

void FAvaPlayableRemoteControlPresetInfoCache::OnPackageModified(const FName& InPackageName)
{
	// Invalidate corresponding assets from that package.
	for (const TPair<FSoftObjectPath, TSharedPtr<FAvaPlayableRemoteControlPresetInfo>>& Entry : PresetInfoCache)
	{
		if (Entry.Key.GetLongPackageFName() == InPackageName && Entry.Value.IsValid())
		{
			UE_LOGF(LogAvaMedia, Verbose,
				"Remote Control Preset Info Cache: Package \"%ls\" being touched caused asset \"%ls\" to be invalidated.",
				*InPackageName.ToString(), *Entry.Key.ToString());

			Entry.Value->MarkDirty();
		}
	}
}