// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playable/AvaPlayableRemoteControlPresetInfo.h"

class IAvaMediaSyncProvider;
class UPackage;

/**
 * Implementation of a RCP info cache.
 * Similar to FAvaRundownManagedInstanceCache in terms of invalidation.
 * Given this is relatively lightweight auxiliary data, we could mark as dirty instead of deleting.
 */
class FAvaPlayableRemoteControlPresetInfoCache final : public IAvaPlayableRemoteControlPresetInfoCache
{
public:
	//~ Begin IAvaPlayableRemoteControlPresetInfoCache
	virtual TSharedPtr<FAvaPlayableRemoteControlPresetInfo> GetRemoteControlPresetInfo(const FSoftObjectPath& InAssetPath, const URemoteControlPreset* InRemoteControlPreset) override;
	virtual void Flush(const FSoftObjectPath& InAssetPath) override;
	virtual void Flush() override;
	//~ End IAvaPlayableRemoteControlPresetInfoCache

	/**
	 * Get the maximum size of the cache beyond witch it will start flushing elements.
	 */
	static int32 GetMaximumCacheSize();
	
	/**
	 * Trim the cache elements that exceed the cache capacity according to LRU replacement policy.
	 */
	void TrimCache();

private:
	void RemoveEntry(const FSoftObjectPath& InAssetPath);
	void RemoveEntries(TFunctionRef<bool(const FSoftObjectPath&, const TSharedPtr<FAvaPlayableRemoteControlPresetInfo>&)> InRemovePredicate);

	void OnPackageSaved(const FString& InPackageFileName, UPackage* InPackage, FObjectPostSaveContext InObjectSaveContext);
	void OnAvaSyncPackageModified(IAvaMediaSyncProvider* InAvaMediaSyncProvider, const FName& InPackageName);
	void OnAssetRemoved(const FAssetData& InAssetData);
	void OnPackageModified(const FName& InPackageName);

private:
	TMap<FSoftObjectPath, TSharedPtr<FAvaPlayableRemoteControlPresetInfo>> PresetInfoCache;
	TArray<FSoftObjectPath> OrderQueue;
};