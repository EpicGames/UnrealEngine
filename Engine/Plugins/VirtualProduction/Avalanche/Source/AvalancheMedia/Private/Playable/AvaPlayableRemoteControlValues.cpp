// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playable/AvaPlayableRemoteControlValues.h"

#include "AvaMediaSerializationUtils.h"
#include "Controller/RCStatelessEventController.h"
#include "Dom/JsonObject.h"
#include "IAvaMediaModule.h"
#include "Misc/PackageName.h"
#include "Playable/AvaPlayableRemoteControl.h"
#include "Playable/AvaPlayableRemoteControlValuesPrivate.h"
#include "Playable/AvaPlayableSettings.h"
#include "RCVirtualProperty.h"
#include "Serialization/CustomVersion.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

namespace UE::AvaPlayableRemoteControlValues::Private
{
	bool PruneValues(const TMap<FGuid, FAvaPlayableRemoteControlValue>& InValues, TMap<FGuid, FAvaPlayableRemoteControlValue>& OutValues)
	{
		bool bModified = false;
		for (TMap<FGuid, FAvaPlayableRemoteControlValue>::TIterator ValueIterator(OutValues); ValueIterator; ++ValueIterator)
		{
			if (!InValues.Contains(ValueIterator->Key))
			{
				ValueIterator.RemoveCurrent();
				bModified = true;
			}
		}
		return bModified;
	}
	
	bool UpdateValues(const TMap<FGuid, FAvaPlayableRemoteControlValue>& InValues, TMap<FGuid, FAvaPlayableRemoteControlValue>& OutValues, bool bInUpdateDefaults)
	{
		// Remove property values that are no longer exposed.
		bool bModified = PruneValues(InValues, OutValues);
	
		// Add missing property values and optionally update the default values.
		for (const TPair<FGuid, FAvaPlayableRemoteControlValue>& SourceValue : InValues)
		{
			FAvaPlayableRemoteControlValue* ExistingValue = OutValues.Find(SourceValue.Key);
			if (!ExistingValue)
			{
				// Remark: IsDefault flag follows along.
				OutValues.Add(SourceValue.Key, SourceValue.Value);
				bModified = true;
			}
			else if (bInUpdateDefaults && ExistingValue->bIsDefault && !ExistingValue->IsSameValueAs(SourceValue.Value))
			{
				ExistingValue->SetValueFrom(SourceValue.Value);
				bModified = true;
			}
		}
		return bModified;
	}

	bool HasSameValues(const TMap<FGuid, FAvaPlayableRemoteControlValue>& InValues, const TMap<FGuid, FAvaPlayableRemoteControlValue>& InOtherValues)
	{
		// If values count differ, consider as different
		if (InValues.Num() != InOtherValues.Num())
		{
			return false;
		}

		// Both Value maps have the same count, so one cannot be a subset of another, therefore, a single find pass should determine equality 
		for (const TPair<FGuid, FAvaPlayableRemoteControlValue>& Pair : InValues)
		{
			const FAvaPlayableRemoteControlValue* FoundValue = InOtherValues.Find(Pair.Key);
			if (!FoundValue || !FoundValue->IsSameValueAs(Pair.Value))
			{
				// Other's Value wasn't found, or was different from the value of this
				return false;
			}
		}

		return true;
	}

	bool HasSameValueAndDefault(const FAvaPlayableRemoteControlValue& InValue, const FAvaPlayableRemoteControlValue& InOtherValue)
	{
		return InOtherValue.IsSameValueAs(InValue.Value) && InOtherValue.bIsDefault == InValue.bIsDefault;
	}

	/** Compares values and default status. */
	bool HasSameValuesAndDefaults(const TMap<FGuid, FAvaPlayableRemoteControlValue>& InValues, const TMap<FGuid, FAvaPlayableRemoteControlValue>& InOtherValues)
	{
		// If values count differ, consider as different
		if (InValues.Num() != InOtherValues.Num())
		{
			return false;
		}
		// Both Value maps have the same count, so one cannot be a subset of another, therefore, a single find pass should determine equality 
		for (const TPair<FGuid, FAvaPlayableRemoteControlValue>& Pair : InValues)
		{
			const FAvaPlayableRemoteControlValue* FoundValue = InOtherValues.Find(Pair.Key);
			if (!FoundValue || !HasSameValueAndDefault(Pair.Value, *FoundValue))
			{
				// Other's Value wasn't found, or was different from the value of this.
				return false;
			}
		}
		return true;
	}

	bool ResetValues(const TMap<FGuid, FAvaPlayableRemoteControlValue>& InValues, TMap<FGuid, FAvaPlayableRemoteControlValue>& OutValues, bool bInIsDefaults)
	{
		TMap<FGuid, FAvaPlayableRemoteControlValue> ResetValues = InValues;
		if (bInIsDefaults)
		{
			for (TPair<FGuid, FAvaPlayableRemoteControlValue>& Value : ResetValues)
			{
				Value.Value.bIsDefault = true;
			}
		}
		
		const bool bModified = !HasSameValuesAndDefaults(OutValues, ResetValues);
		OutValues = ResetValues;
	
		return bModified;
	}

	bool ResetValue(const FAvaPlayableRemoteControlValue& InValue, FAvaPlayableRemoteControlValue& OutValue, bool bInIsDefaults)
	{
		FAvaPlayableRemoteControlValue ResetValue = InValue;
		if (bInIsDefaults)
		{
			ResetValue.bIsDefault = true;
		}

		const bool bModified = !HasSameValueAndDefault(OutValue, ResetValue);
		OutValue = ResetValue;

		return bModified;
	}

	EAvaPlayableRemoteControlChanges ToRemoteControlChanges(bool bInModified, EAvaPlayableRemoteControlChanges InModifiedChanges)
	{
		return bInModified ? InModifiedChanges : EAvaPlayableRemoteControlChanges::None;
	}

	void CollectReferencedAssetPaths(const TSharedPtr<FJsonValue>& InJsonValue, TSet<FSoftObjectPath>& OutReferencedPaths)
	{
		if (!InJsonValue.IsValid())
		{
			return;
		}

		switch (InJsonValue->Type)
		{
		case EJson::String:
			{
				// See if that is a valid package or asset path.
				const FString ValueAsString = InJsonValue->AsString();
				if ((FPackageName::IsValidTextForLongPackageName(ValueAsString) || FPackageName::IsValidObjectPath(ValueAsString))
					&& FPackageName::DoesPackageExist(ValueAsString))
				{
					OutReferencedPaths.Add(FSoftObjectPath(*ValueAsString));
				}
			}
			break;
		case EJson::Array:
			for(const TSharedPtr<FJsonValue>& JsonValue : InJsonValue->AsArray())
			{
				CollectReferencedAssetPaths(JsonValue, OutReferencedPaths);
			}
			break;
		case EJson::Object:
			if (const TSharedPtr<FJsonObject>& JsonObject = InJsonValue->AsObject())
			{
				for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& JsonValue : JsonObject->Values)
				{
					CollectReferencedAssetPaths(JsonValue.Value, OutReferencedPaths);
				}
			}
			break;
		default:
			break;	// don't care about number and bool.
		}
	}
}

const FGuid FAvaPlayableRemoteControlValueCustomVersion::Key(0x85218F83, 0xEDF141CA, 0x800EF947, 0x2F14CB06);
FCustomVersionRegistration GRegisterAvaPlayableRemoteControlValueCustomVersion(FAvaPlayableRemoteControlValueCustomVersion::Key
	, FAvaPlayableRemoteControlValueCustomVersion::LatestVersion
	, TEXT("AvaPlayableRemoteControlValueVersion"));

bool FAvaPlayableRemoteControlValue::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FAvaPlayableRemoteControlValueCustomVersion::Key);
	
	if (Ar.CustomVer(FAvaPlayableRemoteControlValueCustomVersion::Key) >= FAvaPlayableRemoteControlValueCustomVersion::ValueAsString)
	{
		UScriptStruct* Struct = FAvaPlayableRemoteControlValue::StaticStruct();
		Struct->SerializeTaggedProperties(Ar, reinterpret_cast<uint8*>(this), Struct, nullptr);
	}
	else
	{
		FAvaPlayableRemoteControlValueAsBytes_Legacy LegacyValue;
		UScriptStruct* Struct = FAvaPlayableRemoteControlValueAsBytes_Legacy::StaticStruct();
		Struct->SerializeTaggedProperties(Ar, reinterpret_cast<uint8*>(&LegacyValue), Struct, nullptr);

		UE::AvaMediaSerializationUtils::JsonValueConversion::BytesToString(LegacyValue.Bytes, Value);
		bIsDefault = LegacyValue.bIsDefault;
	}

	return true;
}

void FAvaPlayableRemoteControlValues::CopyFrom(const URemoteControlPreset* InRemoteControlPreset, bool bInIsDefault)
{
	using namespace UE::AvaPlayableRemoteControl;

	EntityValues.Reset();
	ControllerValues.Reset();

	if (IsValid(InRemoteControlPreset))
	{
		FString ValueAsString;
		for (const TWeakPtr<const FRemoteControlEntity>& EntityWeakPtr : InRemoteControlPreset->GetExposedEntities<FRemoteControlEntity>())
		{
			const TSharedPtr<const FRemoteControlEntity> Entity = EntityWeakPtr.Pin();
			if (!Entity.IsValid())
			{
				continue;
			}

			const EAvaPlayableRemoteControlResult Result = GetValueOfEntity(Entity, ValueAsString);

			if (Failed(Result))
			{
				UE_LOGF(LogAvaPlayableRemoteControl, Error,
					"Failed to read value of entity \"%ls\" (id:%ls) from RemoteControlPreset \"%ls\": %ls.",
					*Entity->GetLabel().ToString(), *Entity->GetId().ToString(), *InRemoteControlPreset->GetName(), *EnumToString(Result));
				continue;
			}
			
			EntityValues.Add(Entity->GetId(), FAvaPlayableRemoteControlValue(ValueAsString, bInIsDefault));
		}

		TArray<URCVirtualPropertyBase*> Controllers = InRemoteControlPreset->GetControllers();
		for (URCVirtualPropertyBase* Controller : Controllers)
		{
			// Skip ignored controllers
			if (ShouldIgnoreController(Controller))
			{
				continue;
			}

			const EAvaPlayableRemoteControlResult Result = GetValueOfController(Controller, ValueAsString);

			if (Failed(Result))
			{
				UE_LOGF(LogAvaPlayableRemoteControl, Error,
					"Failed to read value of controller \"%ls\" (id:%ls) from RemoteControlPreset \"%ls\": %ls.",
					*Controller->DisplayName.ToString(), *Controller->Id.ToString(), *InRemoteControlPreset->GetName(), *EnumToString(Result));
				continue;
			}

			ControllerValues.Add(Controller->Id, FAvaPlayableRemoteControlValue(ValueAsString, bInIsDefault));
		}
	}
}

bool FAvaPlayableRemoteControlValues::HasSameEntityValues(const FAvaPlayableRemoteControlValues& InOther) const
{
	return UE::AvaPlayableRemoteControlValues::Private::HasSameValues(EntityValues, InOther.EntityValues);
}

bool FAvaPlayableRemoteControlValues::HasSameControllerValues(const FAvaPlayableRemoteControlValues& InOther) const
{
	return UE::AvaPlayableRemoteControlValues::Private::HasSameValues(ControllerValues, InOther.ControllerValues);
}

EAvaPlayableRemoteControlChanges FAvaPlayableRemoteControlValues::PruneRemoteControlValues(const FAvaPlayableRemoteControlValues& InRemoteControlValues)
{
	using namespace UE::AvaPlayableRemoteControlValues::Private;
	return ToRemoteControlChanges(PruneValues(InRemoteControlValues.EntityValues, EntityValues), EAvaPlayableRemoteControlChanges::EntityValues) 
		| ToRemoteControlChanges(PruneValues(InRemoteControlValues.ControllerValues, ControllerValues), EAvaPlayableRemoteControlChanges::ControllerValues); 
}

EAvaPlayableRemoteControlChanges FAvaPlayableRemoteControlValues::UpdateRemoteControlValues(const FAvaPlayableRemoteControlValues& InRemoteControlValues, bool bInUpdateDefaults)
{
	using namespace UE::AvaPlayableRemoteControlValues::Private;
	return ToRemoteControlChanges(UpdateValues(InRemoteControlValues.EntityValues, EntityValues, bInUpdateDefaults), EAvaPlayableRemoteControlChanges::EntityValues) 
		| ToRemoteControlChanges(UpdateValues(InRemoteControlValues.ControllerValues, ControllerValues, bInUpdateDefaults), EAvaPlayableRemoteControlChanges::ControllerValues); 
}

EAvaPlayableRemoteControlChanges FAvaPlayableRemoteControlValues::ResetRemoteControlValues(const FAvaPlayableRemoteControlValues& InReferenceValues, bool bInIsDefaults)
{
	using namespace UE::AvaPlayableRemoteControlValues::Private;
	return ToRemoteControlChanges(ResetValues(InReferenceValues.EntityValues, EntityValues, bInIsDefaults), EAvaPlayableRemoteControlChanges::EntityValues)
		| ToRemoteControlChanges(ResetValues(InReferenceValues.ControllerValues, ControllerValues, bInIsDefaults), EAvaPlayableRemoteControlChanges::ControllerValues);
}

EAvaPlayableRemoteControlChanges FAvaPlayableRemoteControlValues::ResetRemoteControlEntityValue(const FGuid& InId
	, const FAvaPlayableRemoteControlValue& InReferenceValue, bool bInIsDefaults)
{
	if (EntityValues.Contains(InId))
	{
		using namespace UE::AvaPlayableRemoteControlValues::Private;
		return ToRemoteControlChanges(ResetValue(InReferenceValue, EntityValues[InId], bInIsDefaults), EAvaPlayableRemoteControlChanges::EntityValues);
	}
	return EAvaPlayableRemoteControlChanges::None;
}

EAvaPlayableRemoteControlChanges FAvaPlayableRemoteControlValues::ResetRemoteControlControllerValue(const FGuid& InId
	, const FAvaPlayableRemoteControlValue& InReferenceValue, bool bInIsDefaults)
{
	if (ControllerValues.Contains(InId))
	{
		using namespace UE::AvaPlayableRemoteControlValues::Private;
		return ToRemoteControlChanges(ResetValue(InReferenceValue, ControllerValues[InId], bInIsDefaults), EAvaPlayableRemoteControlChanges::ControllerValues);
	}
	return EAvaPlayableRemoteControlChanges::None;
}

bool FAvaPlayableRemoteControlValues::SetEntityValue(const FGuid& InId, const URemoteControlPreset* InRemoteControlPreset, bool bInIsDefault)
{
	const TSharedPtr<const FRemoteControlEntity> Entity = InRemoteControlPreset->GetExposedEntity<FRemoteControlEntity>(InId).Pin();

	if (!Entity)
	{
		UE_LOGF(LogAvaPlayableRemoteControl, Error,
			"Requested entity id \"%ls\" was not found in RemoteControlPreset \"%ls\".",
			*InId.ToString(), *InRemoteControlPreset->GetName());
		return false;
	}

	using namespace UE::AvaPlayableRemoteControl;
	FAvaPlayableRemoteControlValue Value;
	Value.bIsDefault = bInIsDefault;

	const EAvaPlayableRemoteControlResult Result = GetValueOfEntity(Entity, Value.Value);

	if (Failed(Result))
	{
		UE_LOGF(LogAvaPlayableRemoteControl, Error,
			"Failed to read value of entity \"%ls\" (id:%ls) from RemoteControlPreset \"%ls\": %ls.",
			*Entity->GetLabel().ToString(), *InId.ToString(), *InRemoteControlPreset->GetName(), *EnumToString(Result));
		return false;
	}

	EntityValues.Add(Entity->GetId(), MoveTemp(Value));
	return true;
}

bool FAvaPlayableRemoteControlValues::SetControllerValue(const FGuid& InId, const URemoteControlPreset* InRemoteControlPreset, bool bInIsDefault)
{
	URCVirtualPropertyBase* Controller = InRemoteControlPreset->GetController(InId);

	if (!Controller)
	{
		UE_LOGF(LogAvaPlayableRemoteControl, Error,
			"Requested controller id \"%ls\" was not found in RemoteControlPreset \"%ls\".",
			*InId.ToString(), *InRemoteControlPreset->GetName());
		return false;
	}

	using namespace UE::AvaPlayableRemoteControl;
	FAvaPlayableRemoteControlValue Value;
	Value.bIsDefault = bInIsDefault;

	const EAvaPlayableRemoteControlResult Result = GetValueOfController(Controller, Value.Value);
	
	if (Failed(Result))
	{
		UE_LOGF(LogAvaPlayableRemoteControl, Error,
			"Failed to read value of controller \"%ls\" (id:%ls) from RemoteControlPreset \"%ls\": %ls.",
			*Controller->DisplayName.ToString(), *InId.ToString(), *InRemoteControlPreset->GetName(), *EnumToString(Result));
		return false;
	}
		
	ControllerValues.Add(InId, MoveTemp(Value));
	return true;
}

void FAvaPlayableRemoteControlValues::ApplyEntityValuesToRemoteControlPreset(URemoteControlPreset* InRemoteControlPreset, const TSet<FGuid>& InSkipEntities) const
{
	if (!InRemoteControlPreset)
	{
		return;
	}
	using namespace UE::AvaPlayableRemoteControl;
	for (const TWeakPtr<FRemoteControlEntity>& EntityWeakPtr : InRemoteControlPreset->GetExposedEntities<FRemoteControlEntity>())
	{
		if (const TSharedPtr<FRemoteControlEntity> Entity = EntityWeakPtr.Pin())
		{
			if (InSkipEntities.Contains(Entity->GetId()))
			{
				UE_LOGF(LogAvaPlayableRemoteControl, Verbose, "Skipping exposed entity \"%ls\" (id:%ls).",
					*Entity->GetLabel().ToString(), *Entity->GetId().ToString());
				continue;
			}

			if (const FAvaPlayableRemoteControlValue* Value = GetEntityValue(Entity->GetId()))
			{
				const EAvaPlayableRemoteControlResult Result = SetValueOfEntity(Entity, Value->Value); 
				if (Failed(Result))
				{
					UE_LOGF(LogAvaPlayableRemoteControl, Error, "Failed to set value of exposed entity \"%ls\" (id:%ls): %ls.",
						*Entity->GetLabel().ToString(), *Entity->GetId().ToString(), *EnumToString(Result));
				}
			}
			else
			{
				FString AccessError;
				if (!HasReadAccess(Entity, AccessError))
				{
					UE_LOGF(LogAvaPlayableRemoteControl, Error, "Exposed entity \"%ls\" (id:%ls): value not found in page. Reason: %ls.",
						*Entity->GetLabel().ToString(), *Entity->GetId().ToString(), *AccessError);
				}
				else
				{
					UE_LOGF(LogAvaPlayableRemoteControl, Error, "Exposed entity \"%ls\" (id:%ls): value not found in page.",
						*Entity->GetLabel().ToString(), *Entity->GetId().ToString());
				}
			}
		}
	}
}

void FAvaPlayableRemoteControlValues::ApplyControllerValuesToRemoteControlPreset(URemoteControlPreset* InRemoteControlPreset, bool bInForceDisableBehaviors) const
{
	if (!InRemoteControlPreset)
	{
		return;
	}
	TArray<URCVirtualPropertyBase*> Controllers = InRemoteControlPreset->GetControllers();
	for (URCVirtualPropertyBase* Controller : Controllers)
	{
		if (const FAvaPlayableRemoteControlValue* Value = GetControllerValue(Controller->Id))
		{
			using namespace UE::AvaPlayableRemoteControl;
			EAvaPlayableRemoteControlResult Result = SetValueOfController(Controller, Value->Value, /*bInBehaviorsEnabled*/ !bInForceDisableBehaviors);
			if (Failed(Result))
			{
				UE_LOGF(LogAvaPlayableRemoteControl, Error, "Failed to set virtual value of controller \"%ls\" (id:%ls): %ls.",
					*Controller->DisplayName.ToString(), *Controller->Id.ToString(), *EnumToString(Result));
			}
		}
		else
		{
			UE_LOGF(LogAvaPlayableRemoteControl, Error, "Controller \"%ls\" (id:%ls): value not found in page.",
				*Controller->DisplayName.ToString(), *Controller->Id.ToString());
		}
	}
}

bool FAvaPlayableRemoteControlValues::HasIdCollisions(const FAvaPlayableRemoteControlValues& InOtherValues) const
{
	const bool bHasControllerIdCollisions = HasIdCollisions(ControllerValues, InOtherValues.ControllerValues);
	const bool bHasEntityIdCollisions = HasIdCollisions(EntityValues, InOtherValues.EntityValues);
	return bHasControllerIdCollisions || bHasEntityIdCollisions; 
}

bool FAvaPlayableRemoteControlValues::Merge(const FAvaPlayableRemoteControlValues& InOtherValues)
{
	const bool bHasIdCollisions = HasIdCollisions(InOtherValues);

	ControllerValues.Append(InOtherValues.ControllerValues);
	EntityValues.Append(InOtherValues.EntityValues);

	return !bHasIdCollisions;
}

bool FAvaPlayableRemoteControlValues::HasIdCollisions(const TMap<FGuid, FAvaPlayableRemoteControlValue>& InValues, const TMap<FGuid, FAvaPlayableRemoteControlValue>& InOtherValues)
{
	for (const TPair<FGuid, FAvaPlayableRemoteControlValue>& ValueEntry : InValues)
	{
		if (InOtherValues.Contains(ValueEntry.Key))
		{
			return true;
		}
	}
	return false;
}

const FAvaPlayableRemoteControlValues& FAvaPlayableRemoteControlValues::GetDefaultEmpty()
{
	const static FAvaPlayableRemoteControlValues Empty;
	return Empty;
}

void FAvaPlayableRemoteControlValues::CollectReferencedAssetPaths(const TMap<FGuid, FAvaPlayableRemoteControlValue>& InValues,TSet<FSoftObjectPath>& OutReferencedPaths)
{
	// Try to extract a package reference from the value
	for (const TPair<FGuid, FAvaPlayableRemoteControlValue>& Value : InValues)
	{
		TSharedPtr<FJsonObject> ValueObject;
		if (!FJsonSerializer::Deserialize(TJsonReaderFactory<>::Create(Value.Value.Value), ValueObject) || !ValueObject.IsValid())
		{
			UE_LOGF(LogAvaPlayableRemoteControl, Warning, "CollectReferencedPaths: Unable to parse json '%ls'", *Value.Value.Value);
			continue;
		}

		// For now, we don't care about the field name.
		// Just trying to find values that could possibly be an existing asset reference.
		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& JsonValue : ValueObject->Values)
		{
			UE::AvaPlayableRemoteControlValues::Private::CollectReferencedAssetPaths(JsonValue.Value, OutReferencedPaths);
		}
	}
}

bool FAvaPlayableRemoteControlValues::ShouldIgnoreController(const URCVirtualPropertyBase* InController)
{
	if (InController)
	{
		const FAvaPlayableSettings& PlayableSettings = IAvaMediaModule::Get().GetPlayableSettings();
		// Current way to identify an ignored controller is by using a postfix in the name.
		// Other options would be to check meta data. (todo)
		const FString ControllerName = InController->DisplayName.ToString(); 
		for (const FString& PostFixToIgnore : PlayableSettings.IgnoredControllerPostfix)
		{
			if (ControllerName.EndsWith(PostFixToIgnore))
			{
				return true;
			}
		}
	}
	return false;
}

bool FAvaPlayableRemoteControlValues::IsRuntimeEventController(const URCVirtualPropertyBase* InController)
{
	// For now, we only consider the "stateless" event controllers.
	// todo: event controllers with a payload.
	return FRCStatelessEventController::IsStatelessEventController(InController);
}