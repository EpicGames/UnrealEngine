// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "AvaPlayableRemoteControlValuesPrivate.generated.h"

/** Custom serialization version for FAvaPlayableRemoteControlValue */
struct FAvaPlayableRemoteControlValueCustomVersion
{
	enum Type
	{
		// Initial version had the values stored as raw bytes.
		BeforeCustomVersionWasAdded = 0,

		// Values are stored as strings (json formatted).
		ValueAsString,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	static const FGuid Key;
	
private:
	FAvaPlayableRemoteControlValueCustomVersion() = delete;
};

USTRUCT()
struct FAvaPlayableRemoteControlValueAsBytes_Legacy
{
	GENERATED_BODY()
	
	UPROPERTY()
	TArray<uint8> Bytes;

	UPROPERTY()
	bool bIsDefault = false;
};