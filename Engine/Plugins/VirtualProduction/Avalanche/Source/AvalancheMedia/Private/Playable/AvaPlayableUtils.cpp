// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPlayableUtils.h"

#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/Channel/AvaBroadcastOutputChannel.h"
#include "Components/PrimitiveComponent.h"

namespace UE::AvaMedia::PlayableUtils
{
	void AddPrimitiveComponentIds(const AActor* InActor, TSet<FPrimitiveComponentId>& InComponentIds)
	{
		TInlineComponentArray<UPrimitiveComponent*> Components;
		InActor->GetComponents(Components);
		for (int32 ComponentIndex = 0; ComponentIndex < Components.Num(); ComponentIndex++)
		{
			const UPrimitiveComponent* PrimitiveComponent = Components[ComponentIndex];
			if (PrimitiveComponent->IsRegistered())
			{
				InComponentIds.Add(PrimitiveComponent->GetPrimitiveSceneId());

				for (USceneComponent* AttachedChild : PrimitiveComponent->GetAttachChildren())
				{						
					const UPrimitiveComponent* AttachChildPC = Cast<UPrimitiveComponent>(AttachedChild);
					if (AttachChildPC && AttachChildPC->IsRegistered())
					{
						InComponentIds.Add(AttachChildPC->GetPrimitiveSceneId());
					}
				}
			}
		}
	}

	bool HasLocalPlay(const UAvaBroadcast& InBroadcast, FName InChannelName)
	{
		const FAvaBroadcastOutputChannel& Channel = InBroadcast.GetCurrentProfile().GetChannel(InChannelName);

		// If there is no broadcast channel defined, like for preview (by default), then this is for local play.
		if (!Channel.IsValidChannel())
		{
			return true;
		}

		// For non-preview, the commands will be executed locally if the channel has at least one local outputs or no outputs.
		// The "no outputs" condition is considered valid. Empty channels run locally.
		if (Channel.HasAnyLocalMediaOutputs() || Channel.GetMediaOutputs().IsEmpty())
		{
			return true;
		}
		
		return false;
	}

	bool HasRemotePlay(const UAvaBroadcast& InBroadcast, FName InChannelName)
	{
		const FAvaBroadcastOutputChannel& Channel = InBroadcast.GetCurrentProfile().GetChannel(InChannelName);
		return Channel.IsValidChannel() && Channel.HasAnyRemoteMediaOutputs();
	}
}
