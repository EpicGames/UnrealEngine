// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"

class AActor;
class FPrimitiveComponentId;
class UAvaBroadcast;

namespace UE::AvaMedia::PlayableUtils
{
	void AddPrimitiveComponentIds(const AActor* InActor, TSet<FPrimitiveComponentId>& InComponentIds);

	/** Returns true if the broadcast's current profile for the given channel is configured to have local play */
	bool HasLocalPlay(const UAvaBroadcast& InBroadcast, FName InChannelName);

	/** Returns true if the broadcast's current profile for the given channel is configured to have remote play */
	bool HasRemotePlay(const UAvaBroadcast& InBroadcast, FName InChannelName);
}