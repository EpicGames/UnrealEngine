// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaGameInstancePlayableGroup.h"

#include "Framework/AvaGameInstance.h"
#include "Playable/AvaPlayable.h"
#include "Playable/AvaPlayableGroupManager.h"
#include "UObject/Package.h"

namespace UE::AvaMedia::PlayableGroup::Private
{
	UPackage* MakeInstancePackage(const FString& InInstancePackageName)
	{
		UPackage* InstancePackage = CreatePackage(*InInstancePackageName);
		if (InstancePackage)
		{
			InstancePackage->SetFlags(RF_Transient);
		}
		else
		{
			// Note: The outer will fallback to GEngine in that case.
			UE_LOGF(LogAvaPlayable, Error, "Unable to create package \"%ls\" for Motion Design Game Instance.", *InInstancePackageName);
		}
		return InstancePackage;
	}

	UPackage* MakeGameInstancePackage(const FSoftObjectPath& InSourceAssetPath, const FName& InChannelName)
	{
		// Remote Control Preset will be registered with the package name.
		// We want a package name unique to the game instance and that will be
		// human readable since it will show up in the Web Remote Control page.
		
		FString InstancePackageName = TEXT("/Temp/");	// Keep it short for web page.

		// Using channel name, since we should have one instance per channel.
		InstancePackageName += InChannelName.ToString();

		// In order to keep things short, we remove "/Game" since we added the channel name instead.
		FString InstanceSubPath;
		if (!InSourceAssetPath.GetLongPackageName().Split(TEXT("/Game"), nullptr, &InstanceSubPath))
		{
			InstanceSubPath = InSourceAssetPath.GetLongPackageName();
		}

		// This may happen if the original asset path is not specified.
		if (InstanceSubPath.IsEmpty())
		{
			// Add something to get a valid path at least.
			InstanceSubPath = TEXT("InvalidAssetName");
		}
		
		InstancePackageName += InstanceSubPath;

		return MakeInstancePackage(InstancePackageName);
	}

	UPackage* MakeSharedInstancePackage(const FName& InChannelName)
	{
		// Remote Control Preset will be registered with the package name.
		// We want a package name unique to the game instance and that will be
		// human readable since it will show up in the Web Remote Control page.
		
		FString SharedPackageName = TEXT("/Temp/");	// Keep it short for web page.

		// Using channel name, since we should have one instance per channel.
		SharedPackageName += InChannelName.ToString();

		// Shared for all levels.
		SharedPackageName += TEXT("/SharedLevels");
		
		return MakeInstancePackage(SharedPackageName);
	}
}

UAvaGameInstancePlayableGroup* UAvaGameInstancePlayableGroup::Create(UObject* InOuter, const FPlayableGroupCreationInfo& InPlayableGroupInfo)
{
	using namespace UE::AvaMedia::PlayableGroup::Private;
	
	UAvaGameInstancePlayableGroup* GameInstanceGroup = NewObject<UAvaGameInstancePlayableGroup>(InOuter);
	GameInstanceGroup->ParentPlayableGroupManagerWeak = InPlayableGroupInfo.PlayableGroupManager;
		
	if (InPlayableGroupInfo.bIsSharedGroup)
	{
		GameInstanceGroup->GameInstancePackage = MakeSharedInstancePackage(InPlayableGroupInfo.ChannelName);
	}
	else
	{
		// We can create the package even if the name is null, it will have a generic name. But that will be considered an error.
		if (!InPlayableGroupInfo.SourceAssetPath.IsNull())
		{
			UE_LOGF(LogAvaPlayable, Error, "Creating game instance package for asset with unspecified name.");
		}
		GameInstanceGroup->GameInstancePackage = MakeGameInstancePackage(InPlayableGroupInfo.SourceAssetPath, InPlayableGroupInfo.ChannelName);
	}
		
	GameInstanceGroup->GameInstance = UAvaGameInstance::Create(GameInstanceGroup->GameInstancePackage);

	return GameInstanceGroup;
}

UAvaGameInstance* UAvaGameInstancePlayableGroup::GetAvaGameInstance() const
{
	return Cast<UAvaGameInstance>(GameInstance);
}

bool UAvaGameInstancePlayableGroup::ConditionalCreateWorld()
{
	UAvaGameInstance* AvaGameInstance = GetAvaGameInstance();
	
	if (!AvaGameInstance)
	{
		return false;
	}

	bool bWorldWasCreated = false;

	if (!AvaGameInstance->IsWorldCreated())
	{
		bWorldWasCreated = AvaGameInstance->CreateWorld();
	}
	
	// Make sure we register our delegates to this world.
	if (AvaGameInstance->GetPlayWorld())
	{
		ConditionalRegisterWorldDelegates(AvaGameInstance->GetPlayWorld());
	}
	
	return bWorldWasCreated;
}

bool UAvaGameInstancePlayableGroup::ConditionalBeginPlay(const FAvaInstancePlaySettings& InWorldPlaySettings)
{
	UAvaGameInstance* AvaGameInstance = GetAvaGameInstance();
	
	bool bHasBegunPlay = false;
	if (!AvaGameInstance)
	{
		return bHasBegunPlay;
	}
	
	// Make sure we don't have pending unload or stop requests left over in the game instance.
	AvaGameInstance->CancelWorldRequests();

	if (!AvaGameInstance->IsWorldPlaying())
	{
		bHasBegunPlay = AvaGameInstance->BeginPlayWorld(InWorldPlaySettings);
	}
	else
	{
		AvaGameInstance->UpdateRenderTarget(InWorldPlaySettings.RenderTarget);
		AvaGameInstance->UpdateSceneViewportSize(InWorldPlaySettings.ViewportSize);
	}
	return bHasBegunPlay;
}

void UAvaGameInstancePlayableGroup::RequestEndPlayWorld(bool bInForceImmediate)
{
	if (UAvaGameInstance* AvaGameInstance = GetAvaGameInstance())
	{
		AvaGameInstance->RequestEndPlayWorld(bInForceImmediate);
	}
}

bool UAvaGameInstancePlayableGroup::IsWorldPlaying() const
{
	const UAvaGameInstance* AvaGameInstance = GetAvaGameInstance();
	return AvaGameInstance ? AvaGameInstance->IsWorldPlaying() : false;
}

bool UAvaGameInstancePlayableGroup::IsRenderTargetReady() const
{
	const UAvaGameInstance* AvaGameInstance = GetAvaGameInstance();
	return AvaGameInstance ? AvaGameInstance->IsRenderTargetReady() : false;
}

UTextureRenderTarget2D* UAvaGameInstancePlayableGroup::GetRenderTarget() const
{
	const UAvaGameInstance* AvaGameInstance = GetAvaGameInstance();
	return AvaGameInstance ? AvaGameInstance->GetRenderTarget() : ManagedRenderTarget.Get();
}

UWorld* UAvaGameInstancePlayableGroup::GetPlayWorld() const
{
	const UAvaGameInstance* AvaGameInstance = GetAvaGameInstance();
	return AvaGameInstance ? AvaGameInstance->GetPlayWorld() : nullptr;
}

bool UAvaGameInstancePlayableGroup::ConditionalRequestUnloadWorld(bool bForceImmediate)
{
	UAvaGameInstance* AvaGameInstance = GetAvaGameInstance();
	
	if (!AvaGameInstance)
	{
		return false;
	}
	
	if (!HasPlayables())
	{
		UnregisterWorldDelegates(AvaGameInstance->GetPlayWorld());
		AvaGameInstance->RequestUnloadWorld(bForceImmediate);
		return true;
	}
	return false;
}
