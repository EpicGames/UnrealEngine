// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playable/AvaPlayableGroup.h"
#include "AvaGameInstancePlayableGroup.generated.h"

class UAvaGameInstance;

/**
 * This playable group implements a container that owns a
 * motion design game instance. It is intended to be used for
 * broadcast only self contained world simulation and rendering.
 *
 * Render Target:
 * The render target provided as argument of BeginPlay
 * is used to override the game viewport's canvas. So the game will render
 * in the provided render target directly.
 */
UCLASS()
class UAvaGameInstancePlayableGroup : public UAvaPlayableGroup
{
	GENERATED_BODY()

public:
	static UAvaGameInstancePlayableGroup* Create(UObject* InOuter, const FPlayableGroupCreationInfo& InPlayableGroupInfo);
	
	UAvaGameInstance* GetAvaGameInstance() const;

	//~ Begin UAvaPlayableGroup
	virtual bool ConditionalCreateWorld() override;
	virtual bool ConditionalBeginPlay(const FAvaInstancePlaySettings& InWorldPlaySettings) override;
	virtual void RequestEndPlayWorld(bool bInForceImmediate) override;
	virtual bool ConditionalRequestUnloadWorld(bool bForceImmediate) override;
	virtual bool IsWorldPlaying() const override;
	virtual bool IsRenderTargetReady() const override;
	virtual UTextureRenderTarget2D* GetRenderTarget() const override;
	virtual UWorld* GetPlayWorld() const override;
	//~ End UAvaPlayableGroup

	UPROPERTY(Transient)
	TObjectPtr<UPackage> GameInstancePackage;
};
