// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaGameViewportPlayableGroup.h"

#include "Broadcast/OutputDevices/AvaBroadcastRenderTargetMediaUtils.h"
#include "Engine/GameInstance.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/AvaGameInstance.h"
#include "Playable/AvaPlayableGroupManager.h"
#include "Playable/AvaPlayableGroupSubsystem.h"
#include "TextureResource.h"
#if WITH_EDITOR
#include "LevelEditorViewport.h"
#endif

UAvaGameViewportPlayableGroup* UAvaGameViewportPlayableGroup::Create(
	UObject* InOuter,
	UGameInstance* InGameInstance,
	UAvaPlayableGroupManager* InPlayableGroupManager)
{
	if (InGameInstance)
	{
		UAvaGameViewportPlayableGroup* PlayableGroup = NewObject<UAvaGameViewportPlayableGroup>(InOuter);
		PlayableGroup->ParentPlayableGroupManagerWeak = InPlayableGroupManager;
		PlayableGroup->GameInstance = InGameInstance;
		if (UAvaPlayableGroupSubsystem* PlayableGroupSubsystem = InGameInstance->GetSubsystem<UAvaPlayableGroupSubsystem>())
		{
			PlayableGroupSubsystem->PlayableGroupManager = InPlayableGroupManager;
		}
		return PlayableGroup;
	}
	return nullptr;
}

UAvaGameViewportPlayableGroup::~UAvaGameViewportPlayableGroup()
{
	UnregisterViewportDelegates();
}

void UAvaGameViewportPlayableGroup::DetachGameInstance()
{
	UnregisterViewportDelegates();
	GameInstance = nullptr;
	bIsPlaying = false;
}

bool UAvaGameViewportPlayableGroup::ConditionalBeginPlay(const FAvaInstancePlaySettings& InWorldPlaySettings)
{
	ViewportRenderTargetWeak = InWorldPlaySettings.RenderTarget;

	if (!bIsPlaying)
	{
		RegisterViewportDelegates();
		bIsPlaying = true;
		return true;
	}
	return false;
}

void UAvaGameViewportPlayableGroup::RequestEndPlayWorld(bool bInForceImmediate)
{
	UnregisterViewportDelegates();
	bIsPlaying = false;
}

UTextureRenderTarget2D* UAvaGameViewportPlayableGroup::GetRenderTarget() const
{
	if (UTextureRenderTarget2D* ViewportRenderTarget = ViewportRenderTargetWeak.Get())
	{
		return ViewportRenderTarget;
	}
	return ManagedRenderTarget;
}

void UAvaGameViewportPlayableGroup::RegisterViewportDelegates()
{
	if (!UGameViewportClient::OnViewportRendered().IsBoundToObject(this))
	{
		UGameViewportClient::OnViewportRendered().AddUObject(this, &UAvaGameViewportPlayableGroup::OnViewportRendered);
	}
}

void UAvaGameViewportPlayableGroup::UnregisterViewportDelegates()
{
	UGameViewportClient::OnViewportRendered().RemoveAll(this);
}

void UAvaGameViewportPlayableGroup::OnViewportRendered(FViewport* InViewport)
{
	// Main path for capturing the viewport
	if (GameInstance)
	{
		if (const UGameViewportClient* GameViewportClient = GameInstance->GetGameViewportClient())
		{
			if (GameViewportClient->Viewport == InViewport)
			{
				CopyViewportRenderTarget(InViewport);
				return;
			}
		}
	}

#if WITH_EDITOR
	// Fallback for "in editor" PIE (i.e. when PIE window is part of level editor).
	if (GCurrentLevelEditingViewportClient && GCurrentLevelEditingViewportClient->Viewport == InViewport)
	{
		CopyViewportRenderTarget(InViewport);
	}
#endif
}

void UAvaGameViewportPlayableGroup::CopyViewportRenderTarget(FViewport* InViewport) const
{
	if (!ViewportRenderTargetWeak.IsValid())
	{
		return;
	}

	TWeakObjectPtr<UTextureRenderTarget2D> RenderTargetWeak = ViewportRenderTargetWeak;
	
	const FTextureRHIRef SourceRef = InViewport->GetRenderTargetTexture();
	if (!SourceRef.IsValid())
	{
		return;
	}
	
	ENQUEUE_RENDER_COMMAND(CopyViewportRenderTarget)([SourceRef, RenderTargetWeak](FRHICommandListImmediate& RHICmdList) 
	{
		UTextureRenderTarget2D* Target = RenderTargetWeak.Get();
		if (!Target)
		{
			return;
		}

		const FTextureRenderTargetResource* TargetResource = Target->GetRenderTargetResource();
		if (!TargetResource)
		{
			return;
		}

		const FTextureRHIRef TargetRef = TargetResource->GetRenderTargetTexture();
		if (!TargetRef.IsValid())
		{
			return;
		}

		UE::AvaBroadcastRenderTargetMediaUtils::CopyTexture(RHICmdList, SourceRef, TargetRef);
	});
}