// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playable/AvaPlayableGroup.h"
#include "AvaGameViewportPlayableGroup.generated.h"

class FViewport;

/**
 * This playable group implements a wrapper for a pre-existing game instance.
 * It is intended to be used for injecting motion design's playable in an existing
 * game instance that is not specifically implemented for motion design playback.
 *
 * Render Target:
 * The render target provided as argument of BeginPlay
 * is used to copy the viewport render in when the viewport is rendered,
 * but before EndFrame, so it can still be used with media captures for
 * chaining outputs to other devices.
 */
UCLASS()
class UAvaGameViewportPlayableGroup : public UAvaPlayableGroup
{
	GENERATED_BODY()

public:
	static UAvaGameViewportPlayableGroup* Create(
		UObject* InOuter,
		UGameInstance* InGameInstance,
		UAvaPlayableGroupManager* InPlayableGroupManager);

	virtual ~UAvaGameViewportPlayableGroup() override;

	void DetachGameInstance();
	
	//~ Begin UAvaPlayableGroup
	virtual bool ConditionalBeginPlay(const FAvaInstancePlaySettings& InWorldPlaySettings) override;
	virtual void RequestEndPlayWorld(bool bInForceImmediate) override;
	virtual UTextureRenderTarget2D* GetRenderTarget() const override;
	//~ End UAvaPlayableGroup
	
private:
	void RegisterViewportDelegates();
	void UnregisterViewportDelegates();
	void CopyViewportRenderTarget(FViewport* InViewport) const;
	void OnViewportRendered(FViewport* InViewport);
	
private:
	/** Internal copy of the viewport render target. */
	TWeakObjectPtr<UTextureRenderTarget2D> ViewportRenderTargetWeak;

	/**
	 * Keep track if the playable group is "playing".
	 * ConditionalBeginPlay will return true only if the group was not already playing. 
	 */
	bool bIsPlaying = false;
};