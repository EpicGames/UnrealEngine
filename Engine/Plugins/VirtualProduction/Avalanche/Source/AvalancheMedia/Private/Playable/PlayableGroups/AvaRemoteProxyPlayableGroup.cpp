// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRemoteProxyPlayableGroup.h"

bool UAvaRemoteProxyPlayableGroup::ConditionalBeginPlay(const FAvaInstancePlaySettings& InWorldPlaySettings)
{
	if (!bIsPlaying)
	{
		bIsPlaying = true;
		return true;
	}
	return false;
}

void UAvaRemoteProxyPlayableGroup::RequestEndPlayWorld(bool bInForceImmediate)
{
	bIsPlaying = false;
}