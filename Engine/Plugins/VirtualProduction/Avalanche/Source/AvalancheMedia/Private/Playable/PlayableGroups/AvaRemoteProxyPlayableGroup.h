// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playable/AvaPlayableGroup.h"
#include "AvaRemoteProxyPlayableGroup.generated.h"

/**
 *	Remote Proxy Playable Group doesn't have a game instance because it is executed remotely.
 */
UCLASS()
class UAvaRemoteProxyPlayableGroup : public UAvaPlayableGroup
{
	GENERATED_BODY()
	
public:
	//~ Begin UAvaPlayableGroup
	virtual bool ConditionalBeginPlay(const FAvaInstancePlaySettings& InWorldPlaySettings) override;
	virtual void RequestEndPlayWorld(bool bInForceImmediate) override;
	//~ End UAvaPlayableGroup
	
	bool bIsPlaying = false;
};