// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playable/Playables/AvaPlayableLevelStreaming.h"
#include "AvaRemoteControlRebind.h"
#include "AvaRemoteControlUtils.h"
#include "AvaScene.h"
#include "AvaSceneSubsystem.h"
#include "Engine/Level.h"
#include "Engine/LevelStreamingDynamic.h"
#include "Engine/LocalPlayer.h"
#include "EngineUtils.h"
#include "Framework/AvaInstanceSettings.h"
#include "Framework/AvaNullActor.h"
#include "Framework/AvaSoftAssetPtr.h"
#include "IAvaMediaModule.h"
#include "IAvaRemoteControlInterface.h"
#include "MaterialCache/AvaMaterialCacheHelper.h"
#include "MaterialCache/AvaMaterialCacheSettings.h"
#include "Playable/AvaPlayableAssetUserData.h"
#include "Playable/AvaPlayableGroup.h"
#include "Playable/AvaPlayableGroupManager.h"
#include "Playable/AvaPlayableUtils.h"
#include "Playback/AvaPlaybackUtils.h"
#include "SceneView.h"
#include "Streaming/LevelStreamingDelegates.h"
#include "UObject/Package.h"
#include "UnrealEngine.h"

#define LOCTEXT_NAMESPACE "AvaPlayableLevelStreaming"

namespace UE::AvaMedia::LevelStreamingPlayable::Private
{
	static TAutoConsoleVariable<bool> CVarRuntimePrecacheShaders(
		TEXT("MotionDesignPlayable.RuntimePrecacheShaders"),
		false, // False by default to avoid the extra cost of having to try cache materials at runtime/playtime.
		TEXT("Caches any remaining shaders from materials pending completion found in the playable while preparing to start play.")
	);

	static TAutoConsoleVariable<int32> CVarCacheShadersOnLevelLoadMethod(
		TEXT("MotionDesignPlayable.CacheShadersOnLevelLoadMethod"),
		1,
		TEXT("Determines how shader should be cached on level load.\n")
		TEXT(" 0: No Caching. Shaders will not be cached on level load. They could still be cached via MotionDesignPlayable.RuntimePrecacheShaders.\n")
		TEXT(" 1: FirstLoad. Shaders will be cached only on the first level load. Subsequent level loads will not cache shaders\n")
		TEXT(" 2: EveryLoad. Shaders will be cached on every level load.")
	);

	UAvaPlayableAssetUserData* FindOrAddPlayableAssetUserData(ULevel* InLevel)
	{
		if (!InLevel)
		{
			return nullptr;
		}

		UAvaPlayableAssetUserData* PlayableUserData = InLevel->GetAssetUserData<UAvaPlayableAssetUserData>();
		if (!PlayableUserData)
		{
			PlayableUserData = NewObject<UAvaPlayableAssetUserData>();
			InLevel->AddAssetUserData(PlayableUserData);
		}
		return PlayableUserData;
	}

	AAvaScene* FindAvaScene(const ULevel* InLevel)
	{
		AAvaScene* AvaScene = nullptr;
		InLevel->Actors.FindItemByClass(&AvaScene);
		return AvaScene;
	}

	/**
	 * Utility function to extract the level transform from load options.
	 * returns true if the transform was specified, false otherwise.
	 */
	bool GetTransformFromOptions(FTransform& OutTransform, const FString& InOptions, const UAvaPlayableGroup* InPlayableGroup, const FAvaSoftAssetPtr& InSourceAsset)
	{
		// Read the transform from load options if available
		FString TransformString;
		if (FParse::Value(*InOptions,TEXT("Transform="), TransformString))
		{
			if (OutTransform.InitFromString(TransformString))
			{
				return true;
			}
		}

		FString SpawnPointString;
		if (FParse::Value(*InOptions,TEXT("SpawnPointTag="), SpawnPointString))
		{
			if (InPlayableGroup)
			{
				FName SpawnPointName = FName(*SpawnPointString);
				if (UWorld* World = InPlayableGroup->GetPlayWorld())
				{
					for (AAvaNullActor* Actor : TActorRange<AAvaNullActor>(World))
					{
						if (Actor && Actor->ActorHasTag(SpawnPointName))
						{
							OutTransform = Actor->GetTransform();
							return true;
						}
					}
					// To help diagnose problems
					UE_LOGF(LogAvaPlayable, Verbose, "Loading Level [%ls]: NullActor with tag \"%ls\" was not found.",
						*InSourceAsset.ToSoftObjectPath().ToString(), *SpawnPointString);
				}
			}
			else
			{
				UE_LOGF(LogAvaPlayable, Error,
					"Loading Level [%ls]: \"SpawnPoint\" option was specified but the playable doesn't have a valid playable group.",
					*InSourceAsset.ToSoftObjectPath().ToString());
			}
		}

		OutTransform = FTransform::Identity;
		return false;
	}

	/**
	 * Utility function to determine how to cache shaders from load options.
	 * returns how shaders should be cached on load.
	 */
	UAvaPlayableLevelStreaming::ECacheShadersOnLevelLoadMethod GetCacheShadersOnLevelLoadMethod(const FString& InOptions)
	{
		if (FParse::Param(*InOptions, TEXT("CacheShaders")))
		{
			constexpr UAvaPlayableLevelStreaming::ECacheShadersOnLevelLoadMethod CacheShadersOnLevelLoadMethods[] = 
			{
				UAvaPlayableLevelStreaming::ECacheShadersOnLevelLoadMethod::None,
				UAvaPlayableLevelStreaming::ECacheShadersOnLevelLoadMethod::FirstLoad,
				UAvaPlayableLevelStreaming::ECacheShadersOnLevelLoadMethod::EveryLoad,
			};
			constexpr int32 LastElementIndex = static_cast<int32>(UE_ARRAY_COUNT(CacheShadersOnLevelLoadMethods)) - 1;

			const int32 Index = FMath::Clamp(CVarCacheShadersOnLevelLoadMethod.GetValueOnGameThread(), 0, LastElementIndex);
			return CacheShadersOnLevelLoadMethods[Index];
		}
		return UAvaPlayableLevelStreaming::ECacheShadersOnLevelLoadMethod::None;
	}

	/**
	 * Version of FLevelUtils::ApplyLevelTransform specialized to work with animated objects with sequencer.
	 * All root actors get added to a root null and the root is moved.
	 * Only actors that can't be attached get moved.
	 * todo: We could try to exclude cameras from the transform.
	 * In the standard setup, the camera under DefaultScene, along with other things we can to transform. 
	 */
	AActor* ApplyLevelTransform(ULevel* InLevel, const FTransform& InTransform, AActor* InExistingPivotActor)
	{
		// Create a root actor and put everything under it.
		UWorld* const World = InLevel->GetWorld();
		if (!IsValid(World))
		{
			return nullptr;
		}

		AActor* PivotActor = InExistingPivotActor;
		if (!PivotActor)
		{
			FActorSpawnParameters SpawnParams;
			SpawnParams.OverrideLevel = InLevel;
			SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
			SpawnParams.bNoFail = true;
			PivotActor = World->SpawnActor<AAvaNullActor>(SpawnParams);

			if (!PivotActor)
			{
				UE_LOGF(LogAvaPlayable, Error, "Failed to create a pivot actor to transform the level.");
				return nullptr;
			}
		}

		for (AActor* Actor : InLevel->Actors)
		{
			// Exclude the cameras from the transformation
			if (Actor && Actor != PivotActor)
			{
				// Only attach root actors.
				if (Actor->GetAttachParentActor())
				{
					continue;
				}

				// TODO: Have a generic way to tag actors we want to exclude from the level transform (like camera).

				if (!Actor->AttachToActor(PivotActor, FAttachmentTransformRules::KeepRelativeTransform))
				{
					// In case we fail to attach, apply the transformation (as in FLevelUtils::ApplyLevelTransform)
					if (USceneComponent* RootComponent = Actor->GetRootComponent())
					{
						RootComponent->SetRelativeLocation_Direct(InTransform.TransformPosition(RootComponent->GetRelativeLocation()));
						RootComponent->SetRelativeRotation_Direct(InTransform.TransformRotation(RootComponent->GetRelativeRotation().Quaternion()).Rotator());
						RootComponent->SetRelativeScale3D_Direct(InTransform.GetScale3D() * RootComponent->GetRelativeScale3D());

						// Any components which have cached their bounds will not be accurate after a level transform is applied. Force them to recompute the bounds once more.
						Actor->MarkNeedsRecomputeBoundsOnceForGame();
					}
				}
			}
		}

		PivotActor->SetActorTransform(InTransform);

		return PivotActor;
	}

	EAvaPlayableStatus GetPlayableStatusFromLevelStreamingState(ELevelStreamingState InState, bool bInShouldBeLoaded)
	{
		switch (InState)
		{
			case ELevelStreamingState::Removed:
				return EAvaPlayableStatus::Unloaded;
			
			case ELevelStreamingState::Unloaded:
				// If the LevelStreaming was not loaded and has just been made to be loading, the status will still be "unloaded" but
				// we consider it is loading.
				return bInShouldBeLoaded ? EAvaPlayableStatus::Loading : EAvaPlayableStatus::Unloaded;
			
			case ELevelStreamingState::FailedToLoad:
				return EAvaPlayableStatus::Error;

			case ELevelStreamingState::Loading:
				return EAvaPlayableStatus::Loading;
			
			case ELevelStreamingState::LoadedNotVisible:
			case ELevelStreamingState::MakingVisible:
			case ELevelStreamingState::MakingInvisible:
				return EAvaPlayableStatus::Loaded;
			
			case ELevelStreamingState::LoadedVisible:
				return EAvaPlayableStatus::Visible;
			
			default:
				return EAvaPlayableStatus::Error;
		}
	}

	EAvaPlayableStatus GetPlayableStatusFromLevelStreaming(const ULevelStreaming* InLevelStreaming)
	{
		if (!InLevelStreaming)
		{
			return EAvaPlayableStatus::Unloaded;
		}

		return GetPlayableStatusFromLevelStreamingState(InLevelStreaming->GetLevelStreamingState(), InLevelStreaming->ShouldBeLoaded());
	}

	ELevelStreamingState GetLevelStreamingState(const ULevelStreaming* InLevelStreaming)
	{
		return InLevelStreaming ? InLevelStreaming->GetLevelStreamingState() : ELevelStreamingState::Unloaded;
	}

	static int32 GetPlayableStatusRelevanceShouldBeUnloaded(EAvaPlayableStatus InStatus)
	{
		switch (InStatus)
		{
			case EAvaPlayableStatus::Unknown:
				return 0;
			case EAvaPlayableStatus::Error:
				return 1;
			case EAvaPlayableStatus::Unloaded:
				return 2; // Weakest (desired)
			case EAvaPlayableStatus::Loading:
				return 3;
			case EAvaPlayableStatus::Loaded:
				return 4;
			case EAvaPlayableStatus::Visible:
				return 5;
			default:
				return 0;
		}
	}

	static int32 GetPlayableStatusRelevanceShouldBeLoadedOrVisible(EAvaPlayableStatus InStatus)
	{
		switch (InStatus)
		{
			case EAvaPlayableStatus::Unknown:
				return 0;
			case EAvaPlayableStatus::Error:
				return 1;
			case EAvaPlayableStatus::Unloaded:
				return 5;
			case EAvaPlayableStatus::Loading:
				return 4;
			case EAvaPlayableStatus::Loaded:
				return 3;
			case EAvaPlayableStatus::Visible:
				return 2; // Weakest (desired)
			default:
				return 0;
		}
	}
	
	/**
	 * Returns the relevance of the status for comparison with another status
	 * in order to determine the combined status of a playable according to what it should be.
	 */
	static int32 GetPlayableStatusRelevance(EAvaPlayableStatus InStatus, bool bInShouldBeLoaded, bool bInShouldBeVisible)
	{
		// The "desired" status is the weakest of the valid statues.
		if (bInShouldBeLoaded || bInShouldBeVisible)
		{
			return GetPlayableStatusRelevanceShouldBeLoadedOrVisible(InStatus);
		}
		
		return GetPlayableStatusRelevanceShouldBeUnloaded(InStatus);
	}
}

bool UAvaPlayableLevelStreaming::LoadAsset(const FAvaSoftAssetPtr& InSourceAsset, bool bInInitiallyVisible, const FString& InLoadOptions)
{
	if (!PlayableGroup)
	{
		return false;
	}

	// Ensure world is created. Does nothing if already created.
	PlayableGroup->ConditionalCreateWorld();

	// Setup the level transform from the load options.
	bHasTransform = UE::AvaMedia::LevelStreamingPlayable::Private::GetTransformFromOptions(LevelTransform, InLoadOptions, PlayableGroup, InSourceAsset);

	// Determine how shaders should be cached on load
	CacheShadersOnLevelLoadMethod = UE::AvaMedia::LevelStreamingPlayable::Private::GetCacheShadersOnLevelLoadMethod(InLoadOptions);

	const FAvaInstanceSettings& PlaybackInstanceSettings = IAvaMediaModule::Get().GetAvaInstanceSettings();
	bLoadSubPlayables = PlaybackInstanceSettings.bEnableLoadSubPlayables;
	
	check(InSourceAsset.GetAssetType() == EMotionDesignAssetType::World);
	// Remark: We are not using the level streaming transform because it doesn't work with animated objects.
	const bool bAssetLoading = LoadLevel(TSoftObjectPtr<UWorld>(InSourceAsset.ToSoftObjectPath()), FTransform::Identity, bInInitiallyVisible);
	if (bAssetLoading)
	{
		// Refresh status immediately (assumes it won't be loaded nor visible immediately, so shouldn't affect TL)
		SynchronizedLevelStreamingState = UE::AvaMedia::LevelStreamingPlayable::Private::GetLevelStreamingState(LevelStreaming);
		UpdatePlayableStatus(SynchronizedLevelStreamingState);
		PlayableGroup->NotifyLevelStreaming(this);
	}
	return bAssetLoading;
}

bool UAvaPlayableLevelStreaming::UnloadAsset()
{
	if (Scene)
	{
		FAvaRemoteControlUtils::UnregisterRemoteControlPreset(Scene->GetRemoteControlPreset());
	}

	UnloadSubPlayables();
	
	// Ripped from UPocketLevelInstance::BeginDestroy()
	if (LevelStreaming)
	{
		LevelStreaming->bShouldBlockOnUnload = false;
		LevelStreaming->SetShouldBeVisible(false);
		LevelStreaming->SetShouldBeLoaded(false);
		LevelStreaming->SetIsRequestingUnloadAndRemoval(true);
		LevelStreaming->OnLevelShown.RemoveAll(this);
		LevelStreaming->OnLevelLoaded.RemoveAll(this);

		const ULevel* const Level = LevelStreaming->GetLoadedLevel();
		if (IsValid(Level) && Level->GetPackage())
		{
			// Hack so that FLevelStreamingGCHelper::PrepareStreamedOutLevelForGC unloads the level.
			Level->GetPackage()->SetPackageFlags(PKG_PlayInEditor);
		}
	}
	LevelStreaming = nullptr;
	Scene = nullptr;
	SourceLevel.Reset();

	// Refresh status immediately. (not sure if level streaming is going to be called)
	SynchronizedLevelStreamingState = UE::AvaMedia::LevelStreamingPlayable::Private::GetLevelStreamingState(LevelStreaming);
	UpdatePlayableStatus(SynchronizedLevelStreamingState);

	return true;
}

void UAvaPlayableLevelStreaming::UpdatePlayableStatus(ELevelStreamingState InNewState)
{
	using namespace UE::AvaMedia::LevelStreamingPlayable::Private;

	bool bShouldBeLoaded = LevelStreaming ? LevelStreaming->ShouldBeLoaded() : false;
	bool bShouldBeVisible = LevelStreaming ? LevelStreaming->ShouldBeVisible() : false;

	EAvaPlayableStatus MostRelevantStatus = EAvaPlayableStatus::Unknown;
	int32 HighestRelevance = -1;

	auto CompareAndSetMostRelevantStatus = [&MostRelevantStatus, &HighestRelevance, bShouldBeLoaded, bShouldBeVisible](EAvaPlayableStatus InStatus)
	{
		const int32 CurrentRelevance = GetPlayableStatusRelevance(InStatus, bShouldBeLoaded, bShouldBeVisible);
		if (CurrentRelevance > HighestRelevance)
		{
			MostRelevantStatus = InStatus;
			HighestRelevance = CurrentRelevance;
		}
	};
	
	// Check sub playables.
	for (const UAvaPlayableLevelStreaming* SubPlayable : SubPlayables)
	{
		CompareAndSetMostRelevantStatus(SubPlayable->GetPlayableStatus());
	}

	CompareAndSetMostRelevantStatus(GetPlayableStatusFromLevelStreamingState(InNewState, bShouldBeLoaded));

	if (PlayableStatus != MostRelevantStatus)
	{
		PlayableStatus = MostRelevantStatus;
		NotifyPlayableStatusChanged();
	}
}

bool UAvaPlayableLevelStreaming::IsReadyToPlay() const
{
	if (bCachingShaders)
	{
		return false;
	}

	if (const ULevel* LoadedLevel = GetLoadedLevel())
	{
		if (UWorld* const World = LoadedLevel->GetWorld())
		{
			const UAvaSceneSubsystem* const SceneSubsystem = World->GetSubsystem<UAvaSceneSubsystem>();
			if (SceneSubsystem && !SceneSubsystem->IsReadyToPlay(LoadedLevel))
			{
				return false;
			}
		}
	}

	for (const UAvaPlayableLevelStreaming* SubPlayable : SubPlayables)
	{
		if (SubPlayable && !SubPlayable->IsReadyToPlay())
		{
			return false;
		}
	}
	return true;
}

IAvaSceneInterface* UAvaPlayableLevelStreaming::GetSceneInterface() const
{
	if (Scene)
	{
		return static_cast<IAvaSceneInterface*>(Scene);
	}
	return nullptr;
}

bool UAvaPlayableLevelStreaming::GetShouldBeVisible() const
{
	return LevelStreaming ? LevelStreaming->GetShouldBeVisibleFlag() : false;
}

void UAvaPlayableLevelStreaming::SetShouldBeVisible(bool bInShouldBeVisible)
{
	if (LevelStreaming)
	{
		LevelStreaming->SetShouldBeVisible(bInShouldBeVisible);
	}

	for (UAvaPlayableLevelStreaming* SubPlayable : SubPlayables)
	{
		SubPlayable->UpdateVisibilityFromParents();
	}
}

void UAvaPlayableLevelStreaming::SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView)
{
	if (!LevelStreaming)
	{
		return;
	}
	
	const ULevel* const Level = LevelStreaming->GetLoadedLevel();
	if (!IsValid(Level))
	{
		return;
	}

	// Not using synchronized state here. We want to react to the actual state and compensate.
	const ELevelStreamingState StreamingState = LevelStreaming->GetLevelStreamingState();
	const bool bIsVisible = StreamingState == ELevelStreamingState::MakingVisible || StreamingState == ELevelStreamingState::LoadedVisible;
	
	if (bIsVisible && (bShouldBeHidden || bWaitingForShowPlayable))
	{
		TSet<FPrimitiveComponentId> HiddenPrimitives;	// Todo(opt): cache this?

		using namespace UE::AvaMedia::PlayableUtils;

		for (TObjectPtr<AActor> Actor : Level->Actors)
		{
			if (IsValid(Actor))
			{
				AddPrimitiveComponentIds(Actor, HiddenPrimitives);
			}
		}

		if (!HiddenPrimitives.IsEmpty())
		{
			InView.HiddenPrimitives.Append(HiddenPrimitives);
		}
	}
}

bool UAvaPlayableLevelStreaming::LoadLevel(const TSoftObjectPtr<UWorld>& InSourceLevel, const FTransform& InTransform, bool bInInitiallyVisible)
{
	if (SourceLevel == InSourceLevel)
	{
		// Already started loading.
		return false;
	}

	if (!PlayableGroup)
	{
		return false;
	}
	
	bool bSuccess = false;

	ULevelStreamingDynamic::FLoadLevelInstanceParams Params(PlayableGroup->GetPlayWorld(), InSourceLevel.GetLongPackageName(), InTransform);
	Params.bLoadAsTempPackage = true;
	Params.bInitiallyVisible  = bInInitiallyVisible;

	LevelStreaming = ULevelStreamingDynamic::LoadLevelInstance(Params, bSuccess);

	if (!bSuccess || !LevelStreaming)
	{
		UE_LOGF(LogAvaPlayable, Error, "[%ls]: Failed to load level instance `%ls`.", *GetPathNameSafe(this), *InSourceLevel.ToString());
		return false;
	}

	SourceLevel = InSourceLevel;

	// Reset has ever cached shaders if the level asset has changed
	bHasEverCachedShaders = false;

	LevelStreaming->SetShouldBeLoaded(true);
	LevelStreaming->SetShouldBeVisible(bInInitiallyVisible);
	return true;
}

void UAvaPlayableLevelStreaming::HandleTransitionEvent(UAvaPlayable* InPlayable, UAvaPlayableTransition* InTransition, EAvaPlayableTransitionEventFlags InTransitionFlags)
{
	if (InPlayable == this && EnumHasAnyFlags(InTransitionFlags, EAvaPlayableTransitionEventFlags::ShowPlayable))
	{
		bWaitingForShowPlayable = false;
	}
}

void UAvaPlayableLevelStreaming::OnLevelStreamingStateChanged(UWorld* InWorld
	, const ULevelStreaming* InLevelStreaming
	, ULevel* InLevelIfLoaded
	, ELevelStreamingState InPreviousState
	, ELevelStreamingState InNewState)
{
	// Filter out levels we don't care about.
	if (InLevelStreaming != LevelStreaming)
	{
		return;
	}

	using namespace UE::AvaMedia::LevelStreamingPlayable::Private;
	
	// Inject Playable user data in streamed level.
	if (UAvaPlayableAssetUserData* PlayableUserData = FindOrAddPlayableAssetUserData(InLevelIfLoaded))
	{
		PlayableUserData->PlayableWeak = this;
	}

	// Package the event handler for queueing.
	TWeakObjectPtr<UAvaPlayableLevelStreaming> ThisPlayableWeak(this);
	auto SyncEventHandler = [ThisPlayableWeak, InNewState]()
	{
		if (UAvaPlayableLevelStreaming* ThisPlayable = ThisPlayableWeak.Get())
		{
			ThisPlayable->OnLevelStreamingStateChanged_Synchronized(InNewState);
		}
	};

	// Sub-playables don't have an instanceId. But they are supposed to be unique in the playable group, so the
	// source asset path should uniquely identify them.
	const FString InstanceIdString = GetInstanceId().IsValid() ? GetInstanceId().ToString() : GetSourceAssetPath().ToString();

	// Build unique signature for this event.
	FString SyncEventSignature = FString(TEXT("Playable_")) + InstanceIdString + FString(TEXT("_LevelStreaming_")) + EnumToString(InNewState);

	// The same level streaming events are usually sent twice, but we only want to push it once.
	// This avoids generating a warning in the sync event logs. 
	if (!GetPlayableGroup()->IsSynchronizedEventPushed(SyncEventSignature))
	{
		GetPlayableGroup()->PushSynchronizedEvent(MoveTemp(SyncEventSignature), MoveTemp(SyncEventHandler));
	}
}

void UAvaPlayableLevelStreaming::OnLevelStreamingStateChanged_Synchronized(ELevelStreamingState InNewState)
{
	if (!LevelStreaming)
	{
		return;
	}

	// Using state from synchronized event for status updates.
	// The one from LevelStreaming is not synchronized and may cause transitions to start early on some nodes.
	SynchronizedLevelStreamingState = InNewState;

	if (InNewState == ELevelStreamingState::FailedToLoad)
	{
		UE_LOGF(LogAvaPlayable, Error, "Level \"%ls\" failed to load.", *LevelStreaming->PackageNameToLoad.ToString());		
	}
	else if (InNewState == ELevelStreamingState::LoadedNotVisible || InNewState == ELevelStreamingState::LoadedVisible)
	{
		ULevel* Level = LevelStreaming->GetLoadedLevel();
		if (!IsValid(Level))
		{
			return;
		}

		if (UWorld* OuterWorld = Level->GetTypedOuter<UWorld>())
		{
			// Workaround to avoid UEditorEngine::CheckForWorldGCLeaks killing the editor.
			// Change the sub-level's world to be a "persistent" world type.
			OuterWorld->WorldType = EWorldType::GamePreview;

			if (UPackage* Package = OuterWorld->GetPackage())
			{
				// Mark package as transient. Prevents SaveDirtyPackages from trying to save this package.
				Package->SetFlags(RF_Transient);
			}

			if (bLoadSubPlayables)
			{
				LoadSubPlayables(OuterWorld);
			}
		}

		// Workaround to destroy the Linker Load so that it does not keep the underlying File Opened
		if (Level->GetPackage())
		{
			FAvaPlaybackUtils::FlushPackageLoading(Level->GetPackage());
		}

		if (bHasTransform && !bTransformApplied)
		{
			PivotActorForTransform = UE::AvaMedia::LevelStreamingPlayable::Private::ApplyLevelTransform(Level, LevelTransform, PivotActorForTransform.Get());						
			bTransformApplied = true;
		}
		
		// Resolve the ava scene for the other operations.
		ResolveScene(Level);

		FName ShaderProfile;
		bool bShouldCacheShaders = false;

		// Caching happens in two scenarios:
		// - When the user has requested shaders to be cached on load and the level has just been loaded (not visible yet).
		// - When the RuntimePrecacheShaders cvar is set to true and the level just became visible, prior to any transition logic starting, etc.
		// Both scenarios can happen for one playable, and it's expected since these two states can be at completely different times.
		// However, the shader profile will be different in these two scenarios
		if (InNewState == ELevelStreamingState::LoadedNotVisible && CacheShadersOnLevelLoadMethod != ECacheShadersOnLevelLoadMethod::None)
		{
			// Only cache shaders if it has never done so, or the method is to cache every load.
			if (!bHasEverCachedShaders || CacheShadersOnLevelLoadMethod == ECacheShadersOnLevelLoadMethod::EveryLoad)
			{
				ShaderProfile = GetDefault<UAvaMaterialCacheSettings>()->GetOfflineProfile();
				bShouldCacheShaders = true;
			}
		}
		else if (InNewState == ELevelStreamingState::LoadedVisible && UE::AvaMedia::LevelStreamingPlayable::Private::CVarRuntimePrecacheShaders.GetValueOnGameThread())
		{
			ShaderProfile = GetDefault<UAvaMaterialCacheSettings>()->GetRealtimeProfile();
			bShouldCacheShaders = true;
		}

		// Cache materials for the level incrementally, only if not already caching shaders
		if (bShouldCacheShaders && !bCachingShaders)
		{
			UE::Ava::FMaterialCacheHelper& MaterialCacheHelper = UE::Ava::FMaterialCacheHelper::Get();
			MaterialCacheHelper.RequestCacheMaterials(Level, ShaderProfile);
			bCachingShaders = MaterialCacheHelper.IsCaching(Level);
			bHasEverCachedShaders = true;

			if (bCachingShaders)
			{
				FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateWeakLambda(this,
					[this](float)
					{
						const ULevel* const Level = LevelStreaming ? LevelStreaming->GetLoadedLevel() : nullptr;
						if (Level)
						{
							bCachingShaders = UE::Ava::FMaterialCacheHelper::Get().IsCaching(Level);
						}
						else
						{
							bCachingShaders = false;
						}
						// Continue ticking until caching shaders stops
						return bCachingShaders;
					}));
			}
		}
	}
	
	// Important: playable status gets updated in the synchronized event handler.
	UpdatePlayableStatus(SynchronizedLevelStreamingState);
}

void UAvaPlayableLevelStreaming::NotifyPlayableStatusChanged()
{
	using namespace UE::AvaPlayback::Utils;

	UE_LOGF(LogAvaPlayable, Verbose, "%ls Playable \"%ls\" (id:%ls) Status Changed: %ls",
		*GetBriefFrameInfo(),
		*GetSourceAssetPath().GetAssetName(), *GetInstanceId().ToString(),
		*StaticEnumToString(PlayableStatus));

	// OnPlay (camera setup, animations, etc) can only be done when the level is visible (components must be active).
	// With camera rig, we also need to make sure the rig level is loaded and visible.

	if (GetPlayableStatus() == EAvaPlayableStatus::Visible)
	{
		if (bOnPlayQueued)
		{
			bOnPlayQueued = false;
			OnPlay();
		}
	}
	
	OnPlayableStatusChanged().Broadcast(this);

	// Parent playables must be informed of the status change too.
	for (const TObjectKey<UAvaPlayableLevelStreaming>& ParentPlayableKey : ParentPlayables)
	{
		if (UAvaPlayableLevelStreaming* ParentPlayable = ParentPlayableKey.ResolveObjectPtr())
		{
			// Note: using the synchronized streaming state to avoid spurious states.
			ParentPlayable->UpdatePlayableStatus(ParentPlayable->SynchronizedLevelStreamingState);
		}
	}	
}

void UAvaPlayableLevelStreaming::BindDelegates()
{
	if (!FLevelStreamingDelegates::OnLevelStreamingStateChanged.IsBoundToObject(this))
	{
		FLevelStreamingDelegates::OnLevelStreamingStateChanged.AddUObject(this, &UAvaPlayableLevelStreaming::OnLevelStreamingStateChanged);
	}
	if (!UAvaPlayable::OnTransitionEvent().IsBoundToObject(this))
	{
		UAvaPlayable::OnTransitionEvent().AddUObject(this, &UAvaPlayableLevelStreaming::HandleTransitionEvent);
	}
}

void UAvaPlayableLevelStreaming::UnbindDelegates()
{
	FLevelStreamingDelegates::OnLevelStreamingStateChanged.RemoveAll(this);
	UAvaPlayable::OnTransitionEvent().RemoveAll(this);
}

ULevel* UAvaPlayableLevelStreaming::GetLoadedLevel() const
{
	if (!LevelStreaming)
	{
		return nullptr;	
	}
	
	ULevel* Level = LevelStreaming->GetLoadedLevel();
	return IsValid(Level) ? Level : nullptr;
}

void UAvaPlayableLevelStreaming::ResolveScene(const ULevel* InLevel)
{
	if (!IsValid(Scene))
	{
		Scene = UE::AvaMedia::LevelStreamingPlayable::Private::FindAvaScene((InLevel));
		if (Scene)
		{
			FAvaRemoteControlUtils::RegisterRemoteControlPreset(Scene->GetRemoteControlPreset(), /*bInEnsureUniqueId*/ true);
			FAvaRemoteControlRebind::RebindUnboundEntities(Scene->GetRemoteControlPreset(), InLevel);
		}
		else
		{
			UE_LOGF(LogAvaPlayable, Error, "Loaded level \"%ls\" is not an Motion Design level.", *LevelStreaming->PackageNameToLoad.ToString());
		}
	}
}

bool UAvaPlayableLevelStreaming::InitPlayable(const FPlayableCreationInfo& InPlayableInfo)
{
	// For now, we share all the levels in the same instance group. We may do sub-grouping later.
	PlayableGroup = InPlayableInfo.PlayableGroup ?
		InPlayableInfo.PlayableGroup : InPlayableInfo.PlayableGroupManager->GetOrCreateSharedPlayableGroup(InPlayableInfo.ChannelName, false);

	const bool bInitSuccess = Super::InitPlayable(InPlayableInfo);
	
	if (bInitSuccess)
	{
		BindDelegates();
	}
	return bInitSuccess;
}

void UAvaPlayableLevelStreaming::OnPlay()
{
	if (!LevelStreaming || !GetPlayableGroup())
	{
		return;
	}
	
	if (!LevelStreaming->GetShouldBeVisibleFlag())
	{
		// Can't make visible immediately if part of a transition with other playables and the others are not ready.
		GetPlayableGroup()->RequestSetVisibility(this, true);
	}

	const ULevel* const Level = LevelStreaming->GetLoadedLevel();
	if (!IsValid(Level))
	{
		// Level is not yet loaded, queue the action when it gets loaded.
		// Remark: we could either do this when the event is received or poll on the next tick.
		const ELevelStreamingState LevelStreamingState = LevelStreaming->GetLevelStreamingState();
		ensure(LevelStreaming->ShouldBeLoaded());
		if (LevelStreamingState != ELevelStreamingState::FailedToLoad)
		{
			bOnPlayQueued = true;
		}
		else
		{
			UE_LOGF(LogAvaPlayable, Error, "Level \"%ls\" is not loading. Current Streaming State: \"%ls\".",
				*LevelStreaming->PackageNameToLoad.ToString(), EnumToString(LevelStreamingState));			
		}
		return;
	}

	// Check if the level is visible. We can't do the actual camera setup, or animation
	// if the level is not yet visible as the components are inactive.
	const ELevelStreamingState LevelStreamingState = LevelStreaming->GetLevelStreamingState();
	if (LevelStreamingState != ELevelStreamingState::LoadedVisible)
	{
		bOnPlayQueued = true;
		return;
	}

	// Ensure scene is resolved.
	ResolveScene(Level);
}

void UAvaPlayableLevelStreaming::OnEndPlay()
{
	// Ensure the level is hidden and clear dirty flag because those are transient and shouldn't be saved.
	if (LevelStreaming)
	{
		const ULevel* const Level = LevelStreaming->GetLoadedLevel();
		if (IsValid(Level))
		{
			if (UPackage* LevelPackage = Level->GetPackage())
			{
				LevelPackage->ClearDirtyFlag();
			}
		}

		LevelStreaming->SetShouldBeVisible(false);
	}
}

void UAvaPlayableLevelStreaming::OnRemoteControlValuesApplied()
{
	const ULevel* const Level = LevelStreaming->GetLoadedLevel();
	if (!IsValid(Level))
	{
		return;
	}

	auto NotifyRemoteControlValuesApplied = 
		[](UObject* InObject)
		{
			check(InObject);
			if (InObject->GetClass()->ImplementsInterface(UAvaRemoteControlInterface::StaticClass()))
			{
				IAvaRemoteControlInterface::Execute_OnValuesApplied(InObject);
			}
		};

	for (AActor* Actor : Level->Actors)
	{
		if (IsValid(Actor))
		{
			NotifyRemoteControlValuesApplied(Actor);
			Actor->ForEachComponent(/*bNestedComps*/false, NotifyRemoteControlValuesApplied);
		}
	}
}

void UAvaPlayableLevelStreaming::BeginDestroy()
{
	UnbindDelegates();
	Super::BeginDestroy();
}

void UAvaPlayableLevelStreaming::LoadSubPlayables(const UWorld* InLevelInstanceWorld)
{
	check(InLevelInstanceWorld);
	
	for (const ULevelStreaming* SubLevelStreaming : InLevelInstanceWorld->GetStreamingLevels())
	{
		if (SubLevelStreaming)
		{
			GetOrLoadSubPlayable(SubLevelStreaming);
		}
	}
}

void UAvaPlayableLevelStreaming::UnloadSubPlayables()
{
	for (UAvaPlayableLevelStreaming* SubPlayable : SubPlayables)
	{
		if (SubPlayable)
		{
			SubPlayable->ParentPlayables.Remove(this);

			// Shared Sub-Playables will be unloaded if they no longer have any parent
			// playables to keep them alive.
			if (!SubPlayable->HasParentPlayables())
			{
				SubPlayable->UnloadAsset();
				if (UAvaPlayableGroup* ParentPlayableGroup = SubPlayable->GetPlayableGroup())
				{
					ParentPlayableGroup->UnregisterPlayable(SubPlayable);
				}
			}
		}
	}
	
	SubPlayables.Reset();
}

void UAvaPlayableLevelStreaming::GetOrLoadSubPlayable(const ULevelStreaming* InLevelStreaming)
{
	if (!PlayableGroup)
	{
		return;
	}

	const FSoftObjectPath SourceAssetPath = InLevelStreaming->GetWorldAsset().ToSoftObjectPath();

	// Check already loaded sub-playables
	for (const TObjectPtr<UAvaPlayableLevelStreaming>& SubPlayable : SubPlayables)
	{
		if (SubPlayable->GetSourceAssetPath() == SourceAssetPath)
		{
			return; // Already loaded.
		}
	}
	
	// For now, sub playables are shared globally, i.e. unique instance per group.
	// Todo: We could support instancing scope (i.e. global vs local). Would require additional asset/scene info.
	TArray<UAvaPlayable*> FoundPlayables;
	PlayableGroup->FindPlayablesBySourceAssetPath(SourceAssetPath, FoundPlayables);
	for (UAvaPlayable* FoundPlayable : FoundPlayables)
	{
		if (UAvaPlayableLevelStreaming* ExistingPlayable = Cast<UAvaPlayableLevelStreaming>(FoundPlayable))
		{
			// Only use the existing playable if it is a sub-playable already.
			if (ExistingPlayable->HasParentPlayables())
			{
				AddSubPlayable(ExistingPlayable);
				return;
			}
		}
	}

	if (UAvaPlayableLevelStreaming* NewPlayable = CreateSubPlayable(PlayableGroup, SourceAssetPath))
	{
		// TODO: Propagate more stuff from InLevelStreaming. Needs to reach LoadLevel.
		const FAvaSoftAssetPtr AssetPtr = { UWorld::StaticClass(), TSoftObjectPtr<UObject>(SourceAssetPath)};
		if (NewPlayable->LoadAsset(AssetPtr, GetShouldBeVisible(), FString()))
		{
			AddSubPlayable(NewPlayable);
		}
		else
		{
			PlayableGroup->UnregisterPlayable(NewPlayable);
		}
	}
}

UAvaPlayableLevelStreaming* UAvaPlayableLevelStreaming::CreateSubPlayable(UAvaPlayableGroup* InPlayableGroup, const FSoftObjectPath& InSourceAssetPath)
{
	if (!InPlayableGroup)
	{
		return nullptr;
	}

	UAvaPlayableLevelStreaming* NewPlayable = NewObject<UAvaPlayableLevelStreaming>(GEngine);
	
	const FPlayableCreationInfo PlayableCreationInfo =
		{
			InPlayableGroup->GetPlayableGroupManager(),
			{ UWorld::StaticClass(), TSoftObjectPtr<UObject>(InSourceAssetPath) },
			FName(),
			InPlayableGroup
		};
	
	if (NewPlayable && !NewPlayable->InitPlayable(PlayableCreationInfo))
	{
		// final setup may fail, in this case the playable is discarded.
		return nullptr;
	}

	return NewPlayable;
}

void UAvaPlayableLevelStreaming::AddSubPlayable(UAvaPlayableLevelStreaming* InSubPlayable)
{
	if (InSubPlayable)
	{
		SubPlayables.AddUnique(InSubPlayable);
		InSubPlayable->ParentPlayables.Add(this);
	}
}

void UAvaPlayableLevelStreaming::RemoveSubPlayable(UAvaPlayableLevelStreaming* InSubPlayable)
{
	if (InSubPlayable)
	{
		SubPlayables.Remove(InSubPlayable);
		InSubPlayable->ParentPlayables.Remove(this);
	}
}

bool UAvaPlayableLevelStreaming::HasParentPlayables() const
{
	for (const TObjectKey<UAvaPlayableLevelStreaming>& ParentPlayableKey : ParentPlayables)
	{
		if (ParentPlayableKey.ResolveObjectPtr())
		{
			return true;
		}
	}
	return false;
}

void UAvaPlayableLevelStreaming::UpdateVisibilityFromParents()
{
	bool bShouldBeVisible = false;
	
	for (const TObjectKey<UAvaPlayableLevelStreaming>& ParentPlayableKey : ParentPlayables)
	{
		if (const UAvaPlayableLevelStreaming* ParentPlayable = ParentPlayableKey.ResolveObjectPtr())
		{
			if (ParentPlayable->GetShouldBeVisible())
			{
				bShouldBeVisible = true;
				break;
			}
		}
	}

	SetShouldBeVisible(bShouldBeVisible);
}

#undef LOCTEXT_NAMESPACE
