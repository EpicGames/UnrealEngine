// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "Engine/LevelStreaming.h"
#include "Playable/AvaPlayable.h"
#include "AvaPlayableLevelStreaming.generated.h"

class AActor;
class AAvaScene;
class UAvaPlayableTransition;
class ULevelStreamingDynamic;

UCLASS(NotBlueprintable, BlueprintType, ClassGroup = "Motion Design Playable",
	meta = (DisplayName = "Motion Design Level Streaming Playable"))
class UAvaPlayableLevelStreaming : public UAvaPlayable
{
	GENERATED_BODY()
public:
	enum class ECacheShadersOnLevelLoadMethod : uint8
	{
		/** No shaders will be cached on load */
		None,
		/** Shaders will be cached only on the first level load. Subsequent level loads will not cache shaders */
		FirstLoad,
		/** Shaders will be cached every load */
		EveryLoad,
	};

	//~ Begin UAvaPlayable
	virtual bool LoadAsset(const FAvaSoftAssetPtr& InSourceAsset, bool bInInitiallyVisible, const FString& InLoadOptions) override;
	virtual bool UnloadAsset() override;
	virtual const FSoftObjectPath& GetSourceAssetPath() const override { return SourceLevel.ToSoftObjectPath(); }
	virtual EAvaPlayableStatus GetPlayableStatus() const override { return PlayableStatus; }
	virtual IAvaSceneInterface* GetSceneInterface() const override;
	virtual bool GetShouldBeVisible() const override;
	virtual void SetShouldBeVisible(bool bInShouldBeVisible) override;
	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override;
	virtual bool IsReadyToPlay() const override;

protected:
	virtual bool InitPlayable(const FPlayableCreationInfo& InPlayableInfo) override;
	virtual void OnPlay() override;
	virtual void OnEndPlay() override;
	virtual void OnRemoteControlValuesApplied() override;
	//~ End UAvaPlayable

	//~ Begin UObject
	virtual void BeginDestroy() override;
	//~ End UObject

public:
	ULevelStreamingDynamic* GetLevelStreaming() const { return LevelStreaming; }

	void SetShouldBeHidden(bool bInShouldBeHidden) { bShouldBeHidden = bInShouldBeHidden; }
	bool GetShouldBeHidden() const { return bShouldBeHidden; }

protected:
	bool LoadLevel(const TSoftObjectPtr<UWorld>& InSourceLevel, const FTransform& InTransform, bool bInInitiallyVisible);

	void HandleTransitionEvent(UAvaPlayable* InPlayable, UAvaPlayableTransition* InTransition, EAvaPlayableTransitionEventFlags InTransitionFlags);

	void OnLevelStreamingStateChanged(UWorld* InWorld
		, const ULevelStreaming* InLevelStreaming
		, ULevel* InLevelIfLoaded
		, ELevelStreamingState InPreviousState
		, ELevelStreamingState InNewState);

	void OnLevelStreamingStateChanged_Synchronized(ELevelStreamingState InNewState);	

	void UpdatePlayableStatus(ELevelStreamingState InNewState);
	void NotifyPlayableStatusChanged();
	
	void BindDelegates();
	void UnbindDelegates();
	ULevel* GetLoadedLevel() const;
	void ResolveScene(const ULevel* InLevel);

	void LoadSubPlayables(const UWorld* InLevelInstance);
	void UnloadSubPlayables();
	
	void GetOrLoadSubPlayable(const ULevelStreaming* InLevelStreaming);

	/**
	 * Creates a level streaming playable from the given level streaming information.
	 * A new level streaming object is created, wrapped in the returned playable.
	 */
	static UAvaPlayableLevelStreaming* CreateSubPlayable(UAvaPlayableGroup* InPlayableGroup, const FSoftObjectPath& InSourceAssetPath);

	void AddSubPlayable(UAvaPlayableLevelStreaming* InSubPlayable);
	void RemoveSubPlayable(UAvaPlayableLevelStreaming* InSubPlayable);

	/** For shared playables (loaded through streaming dependencies), returns true if the playable is still part of other dependencies. */
	bool HasParentPlayables() const;

	void UpdateVisibilityFromParents();

protected:
	UPROPERTY(Transient)
	TSoftObjectPtr<UWorld> SourceLevel;
	
	UPROPERTY(Transient)
	TObjectPtr<ULevelStreamingDynamic> LevelStreaming;

	UPROPERTY(Transient)
	TObjectPtr<AAvaScene> Scene; 

	bool bLoadSubPlayables = false;

	EAvaPlayableStatus PlayableStatus = EAvaPlayableStatus::Unloaded;

	/** Keep track of the synchronized level streaming state. */
	ELevelStreamingState SynchronizedLevelStreamingState = ELevelStreamingState::Unloaded;
	
	/**
	 * Dependent playables loaded from secondary streaming levels.
	 * Those playables are shared by the parent playable(s) and will be unloaded
	 * when the last parent playable is unloaded.
	 */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UAvaPlayableLevelStreaming>> SubPlayables;

	/**
	 * Keep track of the dependencies this playable is part of.
	 * This helps determine when the playable should be unloaded.
	 */
	TSet<TObjectKey<UAvaPlayableLevelStreaming>> ParentPlayables;

	/** The method of caching shaders when loading the level */
	ECacheShadersOnLevelLoadMethod CacheShadersOnLevelLoadMethod = ECacheShadersOnLevelLoadMethod::None;

	bool bOnPlayQueued = false;
	
	/** Enter Playables should be hidden until the transition has started. */
	bool bWaitingForShowPlayable = true;

	/** If true, all primitives from the playable will hidden. */
	bool bShouldBeHidden = false;

	/** True if shaders are currently being compiled */
	bool bCachingShaders = false;

	/** True if shaders have been cached for the level at some point in the past */
	bool bHasEverCachedShaders = false;

	/** True if the load command specified a transform. */
	bool bHasTransform = false;

	/** Keep track if the transform was applied to avoid doing it multiple times during level streaming. */
	bool bTransformApplied = false;

	/** Keep a pointer to the pivot actor used for transform in case we want to change the transform later on (for reuse). */
	TWeakObjectPtr<AActor> PivotActorForTransform;

	/** Transform to apply to the actors at the end of level streaming. */
	FTransform LevelTransform;
};
