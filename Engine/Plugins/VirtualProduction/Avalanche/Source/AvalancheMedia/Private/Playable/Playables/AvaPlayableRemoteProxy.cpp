// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playable/Playables/AvaPlayableRemoteProxy.h"

#include "Broadcast/AvaBroadcast.h"
#include "Framework/AvaSoftAssetPtr.h"
#include "IAvaMediaModule.h"
#include "Playable/AvaPlayableGroupManager.h"
#include "Playback/AvaPlaybackClientDelegates.h"
#include "Playback/AvaPlaybackUtils.h"
#include "Playback/IAvaPlaybackClient.h"

#define LOCTEXT_NAMESPACE "AvaPlayableRemoteProxy"

bool UAvaPlayableRemoteProxy::LoadAsset(const FAvaSoftAssetPtr& InSourceAsset, bool bInInitiallyVisible, const FString& InLoadOptions)
{
	if (!PlayableGroup)
	{
		return false;
	}
	
	SourceAssetPath = InSourceAsset.ToSoftObjectPath();
	IAvaPlaybackClient& PlaybackClient = IAvaMediaModule::Get().GetPlaybackClient();	
	const TArray<FString> OnlineServers = PlaybackClient.GetOnlineServersForChannel(PlayingChannelFName);
	
	bool bShouldRequestLoad = false;

	// Reconcile the status per-server.
	for (const FString& Server : OnlineServers)
	{
		const TOptional<EAvaPlaybackStatus> RemoteStatusOpt =
			PlaybackClient.GetRemotePlaybackStatus(InstanceId, SourceAssetPath, PlayingChannelName, Server);

		if (!RemoteStatusOpt.IsSet())
		{
			PlaybackClient.RequestPlaybackAssetStatus(SourceAssetPath, Server, /*bInForceRefresh*/ false);
		}
		
		const EAvaPlaybackStatus RemoteStatus = RemoteStatusOpt.IsSet() ? RemoteStatusOpt.GetValue() : EAvaPlaybackStatus::Unknown;

		const bool bCanLoad = RemoteStatus == EAvaPlaybackStatus::Available
			|| RemoteStatus == EAvaPlaybackStatus::Unknown;
		
		const bool bIsLoaded = RemoteStatus == EAvaPlaybackStatus::Loading
			|| RemoteStatus == EAvaPlaybackStatus::Loaded
			|| RemoteStatus == EAvaPlaybackStatus::Starting
			|| RemoteStatus == EAvaPlaybackStatus::Started;

		if (bCanLoad && !bIsLoaded)
		{
			bShouldRequestLoad = true;
		}
	}

	// We have at least one server that was not in the proper state, so we issue the load request.
	if (bShouldRequestLoad)
	{
		// Todo(opt): Combine actions: LoadWithUserData
		// Server-side requirement: a load request on an already loaded/playing asset will not do anything.
		PlaybackClient.RequestPlayback(InstanceId, SourceAssetPath, PlayingChannelName, EAvaPlaybackAction::Load, InLoadOptions);
		PlaybackClient.RequestPlayback(InstanceId, SourceAssetPath, PlayingChannelName, EAvaPlaybackAction::SetUserData, UserData);
	}

	// After this, we should expect the asset to be loaded.
	bShouldBeLoaded = true;	
	return true;
}

bool UAvaPlayableRemoteProxy::UnloadAsset()
{
	IAvaPlaybackClient& PlaybackClient = IAvaMediaModule::Get().GetPlaybackClient();
	if (PlaybackClient.HasAnyServerOnlineForChannel(PlayingChannelFName))
	{
		PlaybackClient.RequestPlayback(InstanceId, SourceAssetPath, PlayingChannelName, EAvaPlaybackAction::Unload);
	}
	bShouldBeLoaded = false;
	return true;
}

namespace UE::AvaMedIaRemoteProxyPlayable::Private
{
	EAvaPlayableStatus GetPlayableStatus(EAvaPlaybackStatus InPlaybackStatus)
	{
		switch (InPlaybackStatus)
		{
		case EAvaPlaybackStatus::Unknown:
			return EAvaPlayableStatus::Unknown;
		case EAvaPlaybackStatus::Missing:
		case EAvaPlaybackStatus::Syncing:
		case EAvaPlaybackStatus::Available:
			return EAvaPlayableStatus::Unloaded;
		case EAvaPlaybackStatus::Loading:
			return EAvaPlayableStatus::Loading;
		case EAvaPlaybackStatus::Loaded:
			return EAvaPlayableStatus::Loaded;
		case EAvaPlaybackStatus::Starting:
			return EAvaPlayableStatus::Loaded;
		case EAvaPlaybackStatus::Started:
			return EAvaPlayableStatus::Visible;
		case EAvaPlaybackStatus::Stopping:
			return EAvaPlayableStatus::Loaded;
		case EAvaPlaybackStatus::Unloading:
			return EAvaPlayableStatus::Unloaded;
		case EAvaPlaybackStatus::Error:
		default:
			return EAvaPlayableStatus::Error;
		}
	}

	// During the loading process, going towards "loaded/visible" is only
	// reached when all replicated playables are in that state.
	int32 PlayableStatusPriorityForLoaded(EAvaPlayableStatus InStatus)
	{
		switch (InStatus)
		{
		case EAvaPlayableStatus::Unknown: return 0; // ignore unknown status
		case EAvaPlayableStatus::Error: return 5;	// wins over everything
		case EAvaPlayableStatus::Unloaded: return 4;
		case EAvaPlayableStatus::Loading: return 3;
		case EAvaPlayableStatus::Loaded: return 2;
		case EAvaPlayableStatus::Visible: return 1;	// destination state is the "weakest"
		default:
			return -1;
		}
	}

	// During the unloading process, going towards "unloaded" is only
	// reached when all replicated playables are in that state.
	int32 PlayableStatusPriorityForUnloaded(EAvaPlayableStatus InStatus)
	{
		switch (InStatus)
		{
		case EAvaPlayableStatus::Unknown: return 0; // ignore unknown status
		case EAvaPlayableStatus::Error: return 5;	// wins over everything
		case EAvaPlayableStatus::Unloaded: return 1; // destination state is the "weakest"
		case EAvaPlayableStatus::Loading: return 2;
		case EAvaPlayableStatus::Loaded: return 3;
		case EAvaPlayableStatus::Visible: return 4;
		default:
			return -1;
		}
	}

	int32 PlayableStatusPriority(EAvaPlayableStatus InStatus, bool bInShouldBeLoaded)
	{
		return bInShouldBeLoaded ? PlayableStatusPriorityForLoaded(InStatus) : PlayableStatusPriorityForUnloaded(InStatus);
	}

	// Contextually reconcile the playable statuses
	EAvaPlayableStatus ReconcilePlayableStatus(EAvaPlayableStatus InStatus, EAvaPlayableStatus InOtherStatus, bool bInShouldBeLoaded)
	{
		if (PlayableStatusPriority(InStatus, bInShouldBeLoaded) > PlayableStatusPriority(InOtherStatus, bInShouldBeLoaded))
		{
			return InStatus;
		}
		return InOtherStatus;
	}
}

EAvaPlayableStatus UAvaPlayableRemoteProxy::GetPlayableStatus() const
{
	using namespace UE::AvaMedIaRemoteProxyPlayable;
	IAvaPlaybackClient& PlaybackClient = IAvaMediaModule::Get().GetPlaybackClient();
	const TArray<FString> OnlineServers = PlaybackClient.GetOnlineServersForChannel(PlayingChannelFName);

	EAvaPlayableStatus PlayableStatus = EAvaPlayableStatus::Unknown;
	
	for (const FString& Server : OnlineServers)
	{
		TOptional<EAvaPlaybackStatus> PlaybackStatus = PlaybackClient.GetRemotePlaybackStatus(InstanceId, SourceAssetPath, PlayingChannelName, Server);

		if (PlaybackStatus.IsSet())
		{
			PlayableStatus = Private::ReconcilePlayableStatus(PlayableStatus, Private::GetPlayableStatus(PlaybackStatus.GetValue()), bShouldBeLoaded);
		}
		else
		{
			PlaybackClient.RequestPlayback(InstanceId, SourceAssetPath, PlayingChannelName, EAvaPlaybackAction::Status);
		}
	}
	
	return PlayableStatus;
}

IAvaSceneInterface* UAvaPlayableRemoteProxy::GetSceneInterface() const
{
	return nullptr;
}

EAvaPlayableCommandResult UAvaPlayableRemoteProxy::ExecuteAnimationCommand(EAvaPlaybackAnimAction InAnimAction, const FAvaPlaybackAnimPlaySettings& InAnimPlaySettings)
{
	IAvaPlaybackClient& PlaybackClient = IAvaMediaModule::Get().GetPlaybackClient();
	
	// If an animation event was locally scheduled on a remote playable,
	// we need to propagate the event.
	if (PlaybackClient.HasAnyServerOnlineForChannel(PlayingChannelFName))
	{
		switch (InAnimAction)
		{
		case EAvaPlaybackAnimAction::None:
			break;
		case EAvaPlaybackAnimAction::Play:
		case EAvaPlaybackAnimAction::PreviewFrame:
			PlaybackClient.RequestAnimPlayback(InstanceId, SourceAssetPath, PlayingChannelName, InAnimPlaySettings);
			break;

		case EAvaPlaybackAnimAction::Continue:
		case EAvaPlaybackAnimAction::Stop:
		case EAvaPlaybackAnimAction::CameraCut:
			PlaybackClient.RequestAnimAction(InstanceId, SourceAssetPath, PlayingChannelName, InAnimPlaySettings.AnimationName.ToString(), InAnimAction);
			break;

		default:
			using namespace UE::AvaPlayback::Utils;
			UE_LOGF(LogAvaPlayable, Warning,
				"%ls Animation command action \"%ls\" for asset \"%ls\" on channel \"%ls\" is not implemented.",
				*GetBriefFrameInfo(), *StaticEnumToString(InAnimAction), *SourceAssetPath.ToString(), *PlayingChannelName);
			break;
		}
	}
	return EAvaPlayableCommandResult::Executed;
}

EAvaPlayableCommandResult UAvaPlayableRemoteProxy::UpdateRemoteControlCommand(const TSharedRef<FAvaPlayableRemoteControlValues>& InRemoteControlValues, EAvaPlayableRCUpdateFlags InFlags)
{
	IAvaPlaybackClient& PlaybackClient = IAvaMediaModule::Get().GetPlaybackClient();
	if (PlaybackClient.HasAnyServerOnlineForChannel(PlayingChannelFName))
	{
		PlaybackClient.RequestRemoteControlUpdate(InstanceId, SourceAssetPath, PlayingChannelName, *InRemoteControlValues, InFlags);
	}
	return EAvaPlayableCommandResult::Executed;
}

void UAvaPlayableRemoteProxy::SetUserData(const FString& InUserData)
{
	if (UserData != InUserData)
	{
		// Setting user data is only allowed if source path is specified (even if InstanceId is ok).
		if (!GetSourceAssetPath().IsNull())	// Should be set in LoadAsset.
		{
			IAvaPlaybackClient& PlaybackClient = IAvaMediaModule::Get().GetPlaybackClient();
			if (!GetSourceAssetPath().IsNull() && PlaybackClient.HasAnyServerOnlineForChannel(PlayingChannelFName))
			{
				PlaybackClient.RequestPlayback(InstanceId, GetSourceAssetPath(), PlayingChannelName, EAvaPlaybackAction::SetUserData, InUserData);
			}
		}
		else
		{
			// If LoadAsset hasn't been called yet, user data is going to be sent along with the Load command.
			
			if (bShouldBeLoaded)	// LoadAsset already called.
			{
				using namespace UE::AvaPlayback::Utils;
				UE_LOGF(LogAvaPlayable, Warning,
					"%ls Failed to set user data. Source Asset Path is not specified (it should be) for instance (id:%ls) on channel \"%ls\".",
					*GetBriefFrameInfo(), *InstanceId.ToString(), *PlayingChannelName);
			}
		}
	}
	
	Super::SetUserData(InUserData);
}

bool UAvaPlayableRemoteProxy::InitPlayable(const FPlayableCreationInfo& InPlayableInfo)
{
	// We keep track of the channel this playable is part of.
	PlayingChannelFName = InPlayableInfo.ChannelName;
	PlayingChannelName = InPlayableInfo.ChannelName.ToString();
	
	constexpr bool bIsRemoteProxy = true;

	// Remote playables have proxy playable groups imitating the same setup as local ones.
	switch (InPlayableInfo.SourceAsset.GetAssetType())
	{
	case EMotionDesignAssetType::World:
		PlayableGroup = InPlayableInfo.PlayableGroupManager->GetOrCreateSharedPlayableGroup(InPlayableInfo.ChannelName, bIsRemoteProxy);
		break;
	default:
		UE_LOGF(LogAvaPlayable, Error, "Asset \"%ls\" is an unsupported type.", *InPlayableInfo.SourceAsset.ToSoftObjectPath().ToString());
		break;
	}
	
	const bool bInitSucceeded = Super::InitPlayable(InPlayableInfo);
	if (bInitSucceeded)
	{
		RegisterClientEventHandlers();
	}
	return bInitSucceeded;
}

void UAvaPlayableRemoteProxy::OnPlay()
{
	IAvaMediaModule& AvaMediaModule = IAvaMediaModule::Get();
	if (!AvaMediaModule.IsPlaybackClientStarted())
	{
		return;
	}
	
	IAvaPlaybackClient& Client = AvaMediaModule.GetPlaybackClient();
	
	if (Client.HasAnyServerOnlineForChannel(PlayingChannelFName))
	{
		const FString ChannelName = PlayingChannelName;
		const TOptional<EAvaPlaybackStatus> RemoteStatusOpt = Client.GetRemotePlaybackStatus(InstanceId, SourceAssetPath, ChannelName);
		const EAvaPlaybackStatus RemoteStatus = RemoteStatusOpt.IsSet() ? RemoteStatusOpt.GetValue() : EAvaPlaybackStatus::Unknown;
		// TODO: rework this logic.
		if (RemoteStatus == EAvaPlaybackStatus::Available
			|| RemoteStatus == EAvaPlaybackStatus::Loading || RemoteStatus == EAvaPlaybackStatus::Loaded
			|| RemoteStatus == EAvaPlaybackStatus::Unknown || RemoteStatus == EAvaPlaybackStatus::Stopping
			|| RemoteStatus == EAvaPlaybackStatus::Unloading)
		{
			// Todo: Combine actions: StartWithUserData
			Client.RequestPlayback(InstanceId, SourceAssetPath, ChannelName, EAvaPlaybackAction::Start);
		}
	}
}

void UAvaPlayableRemoteProxy::OnEndPlay()
{
	IAvaPlaybackClient& PlaybackClient = IAvaMediaModule::Get().GetPlaybackClient();
	if (PlaybackClient.HasAnyServerOnlineForChannel(PlayingChannelFName))
	{
		PlaybackClient.RequestPlayback(InstanceId, SourceAssetPath, PlayingChannelName, EAvaPlaybackAction::Stop);
	}
}

void UAvaPlayableRemoteProxy::BeginDestroy()
{
	UnregisterClientEventHandlers();
	Super::BeginDestroy();
}

void UAvaPlayableRemoteProxy::RegisterClientEventHandlers()
{
	using namespace UE::AvaPlaybackClient::Delegates;
	GetOnPlaybackSequenceEvent().RemoveAll(this);
	GetOnPlaybackSequenceEvent().AddUObject(this, &UAvaPlayableRemoteProxy::HandlePlaybackSequenceEvent);
	GetOnPlaybackStatusChanged().RemoveAll(this);
	GetOnPlaybackStatusChanged().AddUObject(this, &UAvaPlayableRemoteProxy::HandlePlaybackStatusChanged);
}

void UAvaPlayableRemoteProxy::UnregisterClientEventHandlers() const
{
	using namespace UE::AvaPlaybackClient::Delegates;
	GetOnPlaybackSequenceEvent().RemoveAll(this);
	GetOnPlaybackStatusChanged().RemoveAll(this);
}

void UAvaPlayableRemoteProxy::HandlePlaybackSequenceEvent(IAvaPlaybackClient& InPlaybackClient,
	const UE::AvaPlaybackClient::Delegates::FPlaybackSequenceEventArgs& InEventArgs)
{
	if (InEventArgs.InstanceId == InstanceId && InEventArgs.ChannelName == PlayingChannelName)
	{
		const FName SequenceName(InEventArgs.SequenceName);
		OnSequenceEventDelegate.Broadcast(this, SequenceName, InEventArgs.EventType);
	}
}

void UAvaPlayableRemoteProxy::HandlePlaybackStatusChanged(IAvaPlaybackClient& InPlaybackClient,
	const UE::AvaPlaybackClient::Delegates::FPlaybackStatusChangedArgs& InEventArgs)
{
	if (InEventArgs.InstanceId == InstanceId && InEventArgs.ChannelName == PlayingChannelName)
	{
		OnPlayableStatusChanged().Broadcast(this);
	}
}


#undef LOCTEXT_NAMESPACE
