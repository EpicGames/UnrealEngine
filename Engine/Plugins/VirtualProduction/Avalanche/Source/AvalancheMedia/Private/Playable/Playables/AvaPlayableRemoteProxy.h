// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playable/AvaPlayable.h"
#include "AvaPlayableRemoteProxy.generated.h"

class IAvaPlaybackClient;

namespace UE::AvaPlaybackClient::Delegates
{
	struct FPlaybackSequenceEventArgs;
	struct FPlaybackStatusChangedArgs;
}

UCLASS(NotBlueprintable, BlueprintType, ClassGroup = "Motion Design Playable",
	meta = (DisplayName = "Motion Design Remote Proxy Playable"))
class UAvaPlayableRemoteProxy : public UAvaPlayable
{
	GENERATED_BODY()
public:
	FName GetPlayingChannelFName() const { return PlayingChannelFName; }
	
	//~ Begin UAvaPlayable
	virtual bool LoadAsset(const FAvaSoftAssetPtr& InSourceAsset, bool bInInitiallyVisible, const FString& InLoadOptions) override;
	virtual bool UnloadAsset() override;
	virtual const FSoftObjectPath& GetSourceAssetPath() const override { return SourceAssetPath; }
	virtual EAvaPlayableStatus GetPlayableStatus() const override;
	virtual IAvaSceneInterface* GetSceneInterface() const override;
	virtual EAvaPlayableCommandResult ExecuteAnimationCommand(EAvaPlaybackAnimAction InAnimAction, const FAvaPlaybackAnimPlaySettings& InAnimPlaySettings) override;
	virtual EAvaPlayableCommandResult UpdateRemoteControlCommand(const TSharedRef<FAvaPlayableRemoteControlValues>& InRemoteControlValues, EAvaPlayableRCUpdateFlags InFlags) override;
	virtual bool IsRemoteProxy() const override { return true; }
	virtual void SetUserData(const FString& InUserData) override;

protected:
	virtual bool InitPlayable(const FPlayableCreationInfo& InPlayableInfo) override;
	virtual void OnPlay() override;
	virtual void OnEndPlay() override;
	//~ End UAvaPlayable

	//~ Begin UObject
	virtual void BeginDestroy() override;
	//~ End UObject

	void RegisterClientEventHandlers();
	void UnregisterClientEventHandlers() const;

	void HandlePlaybackSequenceEvent(IAvaPlaybackClient& InPlaybackClient,
		const UE::AvaPlaybackClient::Delegates::FPlaybackSequenceEventArgs& InEventArgs);

	void HandlePlaybackStatusChanged(IAvaPlaybackClient& InPlaybackClient,
		const UE::AvaPlaybackClient::Delegates::FPlaybackStatusChangedArgs& InEventArgs);

protected:
	/** Channel name this playable is playing on. */
	FName PlayingChannelFName;
	FString PlayingChannelName;

	FSoftObjectPath SourceAssetPath;

	/**
	 * Internal forked channel's status reconciling needs to keep track of the expected series of events.
	 * If should be loaded, expected sequence is: loading, loaded, making visible, visible.
	 * If it should be unloaded the sequence is inverted.
	 * The reconciling logic will select the "slowest" node. Example: for loading process, the status
	 * is loading as long as there is one node still loading.
	 */
	bool bShouldBeLoaded = false;
};
