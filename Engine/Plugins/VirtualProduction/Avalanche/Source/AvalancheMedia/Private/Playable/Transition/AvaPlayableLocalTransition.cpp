// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playable/Transition/AvaPlayableLocalTransition.h"

#include "AvaSequencePlayer.h"
#include "AvaTransitionSubsystem.h"
#include "AvaTransitionTree.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "Execution/AvaTransitionExecutorBuilder.h"
#include "Execution/IAvaTransitionExecutor.h"
#include "Framework/AvaInstanceSettings.h"
#include "IAvaMediaModule.h"
#include "IAvaSceneInterface.h"
#include "IAvaSequenceProvider.h"
#include "Playable/AvaPlayable.h"
#include "Playable/Transition/AvaPlayableTransitionPrivate.h"
#include "Playable/Transition/AvaPlayableTransitionScene.h"
#include "Playback/AvaPlaybackUtils.h"

namespace UE::AvaPlayableTransition::Private
{
	UAvaTransitionSubsystem* GetTransitionSubsystem(const UAvaPlayable& InPlayable)
	{
		if (const UAvaPlayableGroup* PlayableGroup = InPlayable.GetPlayableGroup())
		{
			if (const UWorld* World = PlayableGroup->GetPlayWorld())
			{
				return World->GetSubsystem<UAvaTransitionSubsystem>();
			}
		}
		return nullptr;
	}

	IAvaTransitionBehavior* GetTransitionBehavior(const UAvaPlayable& InPlayable, const UAvaTransitionSubsystem& InTransitionSubsystem)
	{
		if (const IAvaSceneInterface* SceneInterface = InPlayable.GetSceneInterface())
		{
			if (ULevel* const Level = SceneInterface->GetSceneLevel())
			{
				return InTransitionSubsystem.GetTransitionBehavior(Level);
			}
		}
		return nullptr;		
	}

	IAvaTransitionBehavior* GetTransitionBehavior(const UAvaPlayable& InPlayable)
	{
		if (const UAvaTransitionSubsystem* TransitionSubsystem = GetTransitionSubsystem(InPlayable))
		{
			return GetTransitionBehavior(InPlayable, *TransitionSubsystem);
		}
		return nullptr;		
	}

	bool MakeNullTransitionBehaviorInstance(UAvaPlayableTransition* InPlayableTransition, const FAvaTagHandle& InTransitionLayer, FAvaTransitionBehaviorInstance& OutBehaviorInstance)
	{
		IAvaTransitionBehavior* const TransitionBehavior = nullptr;
		OutBehaviorInstance = FAvaTransitionBehaviorInstance();
		OutBehaviorInstance.SetBehavior(TransitionBehavior);
		OutBehaviorInstance.CreateScene<FAvaPlayableTransitionScene>(InPlayableTransition, InTransitionLayer, InPlayableTransition);
		return true;
	}

	FAvaTagHandle GetTransitionLayer(const IAvaTransitionBehavior* InBehavior)
	{
		if (InBehavior)
		{
			return InBehavior->GetTransitionLayer();
		}
		return FAvaTagHandle();
	}

	FAvaTagHandle GetTransitionLayer(const FAvaTransitionBehaviorInstance& InBehaviorInstance)
	{
		// Remark: InBehaviorInstance.GetTransitionLayer() doesn't return a valid layer.
		return GetTransitionLayer(InBehaviorInstance.GetBehavior());
	}

	FAvaTagHandle GetTransitionLayer(const UAvaPlayable& InPlayable)
	{
		return GetTransitionLayer(GetTransitionBehavior(InPlayable));
	}

	class FBuilderHelper
	{
	public:
		FBuilderHelper(FString&& InContextName, UAvaPlayableTransition* InPlayableTransition)
			: ContextName(MoveTemp(InContextName))
			, PlayableTransition(InPlayableTransition)
		{
			ExecutorBuilder.SetContextName(ContextName);
		}

		const FString& GetContextName() const { return ContextName; }

		bool HasBehaviorInstances() const
		{
			return NumEnterInstance > 0 || NumExitInstance > 0;
		}

		bool AddTransitionBehaviorInstance(UAvaPlayable* InPlayable, EAvaPlayableTransitionEntryRole InPlayableRole)
		{
			if (!InPlayable)
			{
				UE_LOGF(LogAvaPlayable, Error, "Playable Transition \"%ls\" setup error: Invalid playable.", *ContextName);
				return false;
			}

			UAvaTransitionSubsystem* TransitionSubsystem = GetTransitionSubsystem(*InPlayable); 

			if (!TransitionSubsystem)
			{
				UE_LOGF(LogAvaPlayable, Error,
					"Playable Transition \"%ls\" setup error: Can't retrieve transition subsystem for playable {%ls}.",
					*ContextName, *GetPrettyPlayableInfo(InPlayable));
				return false;
			}

			if (LastTransitionSubsystem != nullptr && LastTransitionSubsystem != TransitionSubsystem)
			{
				// Todo: if we hit this error, we need to create multiple playable transitions batched per subsystem.
				UE_LOGF(LogAvaPlayable, Error,
					"Playable Transition \"%ls\" setup error: Playable {%ls} is in a different subsystem.",
					*ContextName, *GetPrettyPlayableInfo(InPlayable));
				return false;
			}
			
			LastTransitionSubsystem = TransitionSubsystem;

			FAvaTransitionBehaviorInstance BehaviorInstance;
			if (MakeTransitionBehaviorInstance(InPlayable, TransitionSubsystem, BehaviorInstance))
			{
				if (InPlayableRole == EAvaPlayableTransitionEntryRole::Enter)
				{
					ExecutorBuilder.AddEnterInstance(BehaviorInstance);
					++NumEnterInstance;
				}
				else
				{
					// Either "playing" or "exit" are considered Exit in that implementation layer.
					ExecutorBuilder.AddExitInstance(BehaviorInstance);
					++NumExitInstance;
					
					// For exit playables, we need to add a "take out" dummy enter instance.
					if (InPlayableRole == EAvaPlayableTransitionEntryRole::Exit)
					{
						FAvaTransitionBehaviorInstance NullEnterInstance;
						if (MakeNullTransitionBehaviorInstance(PlayableTransition, GetTransitionLayer(BehaviorInstance), NullEnterInstance))
						{
							ExecutorBuilder.AddEnterInstance(NullEnterInstance);
						}
					}
				}
				return true;
			}
			return false;
		}
		
		bool MakeTransitionBehaviorInstance(UAvaPlayable* InPlayable, const UAvaTransitionSubsystem* InTransitionSubsystem, FAvaTransitionBehaviorInstance& OutBehaviorInstance) const
		{
			if (IAvaTransitionBehavior* TransitionBehavior = GetTransitionBehavior(*InPlayable, *InTransitionSubsystem))
			{
				if (TransitionBehavior->IsEnabled() && TransitionBehavior->GetTransitionTree())
				{
					OutBehaviorInstance = FAvaTransitionBehaviorInstance();
					OutBehaviorInstance.SetBehavior(TransitionBehavior);
					OutBehaviorInstance.CreateScene<FAvaPlayableTransitionScene>(InPlayable, InPlayable, PlayableTransition);
					return true;
				}
			}
			return false;
		}

	private:
		FString ContextName;
		UAvaPlayableTransition* PlayableTransition = nullptr;
		int32 NumEnterInstance = 0;
		int32 NumExitInstance = 0;

	public:
		UAvaTransitionSubsystem* LastTransitionSubsystem = nullptr;
		FAvaTransitionExecutorBuilder ExecutorBuilder;
	};

	struct FSequenceHelper
	{
		const IAvaSequenceProvider* SequenceProvider = nullptr;
		IAvaSequencePlaybackObject* SequencePlayback = nullptr;

		FSequenceHelper(const UAvaPlayable* InPlayable)
		{
			if (InPlayable)
			{
				if (IAvaSceneInterface* SceneInterface = InPlayable->GetSceneInterface())
				{
					SequenceProvider = SceneInterface->GetSequenceProvider();
					SequencePlayback = SceneInterface->GetPlaybackObject();
				}
			}
		}

		bool IsValid() const
		{
			return SequenceProvider && SequencePlayback;
		}
	};

	// Source: FAvaTransitionInitializeSequence::ExecuteSequenceTask
	void InitializeSequences(const UAvaPlayable* InPlayable)
	{
		const FSequenceHelper Helper(InPlayable);
		if (Helper.IsValid())
		{
			FAvaSequencePlayParams PlaySettings;
			PlaySettings.Start = PlaySettings.End = FAvaSequenceTime(0.0);
			PlaySettings.PlayMode = EAvaSequencePlayMode::Forward;

			for (const TObjectPtr<UAvaSequence>& Sequence : Helper.SequenceProvider->GetSequences())
			{
				Helper.SequencePlayback->PlaySequence(Sequence.Get(), PlaySettings);
			}
		}
	}	

	// Source: FAvaTransitionSequenceUtils::UpdatePlayerRunStatus
	bool HasActiveSequences(const UAvaPlayable* InPlayable)
	{
		const FSequenceHelper Helper(InPlayable);
		if (Helper.IsValid())
		{
			for (const TObjectPtr<UAvaSequence>& Sequence : Helper.SequenceProvider->GetSequences())
			{
				if (const UAvaSequencePlayer* SequencePlayer = Helper.SequencePlayback->GetSequencePlayer(Sequence.Get()))
				{
					// Considering paused as not active.
					if (SequencePlayer->GetPlaybackStatus() != EMovieScenePlayerStatus::Paused)
					{
						return true;
					}
				}
			}
		}
		return false;
	}

	bool HasActiveSequences(const TArray<TWeakObjectPtr<UAvaPlayable>>& InPlayablesWeak)
	{
		for (const TWeakObjectPtr<UAvaPlayable> PlayableWeak : InPlayablesWeak)
		{
			if (HasActiveSequences(PlayableWeak.Get()))
			{
				return true;
			}
		}
		return false;
	}

	/**
	 * Determines if a playing playable should be kicked out based on the transition layer.
	 */
	bool ShouldKickOutPlayable(bool bInIsPreviewFrameLogic, const UAvaPlayable* InPlayable, const TArray<FAvaTagHandle>& InTransitionLayersToKickOut)
	{
		if (bInIsPreviewFrameLogic && InPlayable)
		{
			const FAvaTagHandle TransitionLayer = GetTransitionLayer(*InPlayable);

			auto IsTransitionLayerOverlap = [&TransitionLayer](const FAvaTagHandle& InOtherLayer)
			{
				// Note: Invalid layer (No TL) means everything is kicked out.
				return !InOtherLayer.IsValid() || InOtherLayer.Overlaps(TransitionLayer);
			};
			
			if (!TransitionLayer.IsValid()	// No TL gets kicked out regardless.
				|| InTransitionLayersToKickOut.ContainsByPredicate(IsTransitionLayerOverlap))
			{
				return true;
			}
		}
		return false;
	}
}

bool UAvaPlayableLocalTransition::Start()
{
	if (!Super::Start())
	{
		return false;
	}

	using namespace UE::AvaPlayableTransition::Private;

	FBuilderHelper Helper(GetInstanceName(), this);

	TArray<UAvaPlayable*> EnterPlayables = Pin(EnterPlayablesWeak);
	TArray<UAvaPlayable*> PlayingPlayables = Pin(PlayingPlayablesWeak);
	TArray<UAvaPlayable*> ExitPlayables = Pin(ExitPlayablesWeak);
	
	if (EnterPlayables.IsEmpty() && PlayingPlayables.IsEmpty() && ExitPlayables.IsEmpty())
	{
		UE_LOGF(LogAvaPlayable, Error,
			"Playable Transition \"%ls\" setup error: no playables specified, either as enter or exit. Nothing to do transition on.",
			*Helper.GetContextName());
		return false;
	}

	// Store the Latest Remote Control Values for Playables that are not Enter Playables
	// as the Enter Playable ones are stored in EnterPlayableValues
	OtherPlayableValues.Empty(PlayingPlayables.Num() + ExitPlayables.Num());

	for (UAvaPlayable* Playable : PlayingPlayables)
	{
		OtherPlayableValues.Add(Playable, Playable->GetLatestRemoteControlValues());
	}

	for (UAvaPlayable* Playable : ExitPlayables)
	{
		OtherPlayableValues.Add(Playable, Playable->GetLatestRemoteControlValues());
	}

	// PreviewFrame will bypass the transition behaviors.
	const bool bIsPreviewFrameTransition = EnumHasAnyFlags(TransitionFlags, EAvaPlayableTransitionFlags::PlayEnterPlayablesAtPreviewFrame);
	TArray<FAvaTagHandle> TransitionLayersToKickOut;

	int32 ArrayIndex = 0;
	for (UAvaPlayable* Playable : EnterPlayables)
	{
		if (EnterPlayableValues.IsValidIndex(ArrayIndex) && EnterPlayableValues[ArrayIndex].IsValid() && !IsPlayingPlayable(Playable))
		{
			// Note: we are still not running the controller behaviors when updating RC from a transition.
			Playable->UpdateRemoteControlCommand(EnterPlayableValues[ArrayIndex].ToSharedRef(), EAvaPlayableRCUpdateFlags::None);
		}

		if (bIsPreviewFrameTransition || !Helper.AddTransitionBehaviorInstance(Playable, EAvaPlayableTransitionEntryRole::Enter))
		{
			// Handle Playables with no transition tree (or PreviewFrame override).
			InitializeSequences(Playable);
			PostExecutorSequencePlayablesWeak.Add(Playable);

			// For preview frame, we keep track of the transition layer to kick out all other playables on that layer.
			// Normally, the transition tree would do that, but we are bypassing it in this mode.
			if (bIsPreviewFrameTransition)
			{
				// Note: adding potentially invalid layer on purpose, indicates a no TL enter page and that will kick everything else.
				TransitionLayersToKickOut.Add(GetTransitionLayer(*Playable));
			}
		}

		// Signal that this playable can be shown now that sequences are initialized and RC is injected.
		UAvaPlayable::OnTransitionEvent().Broadcast(Playable, this, EAvaPlayableTransitionEventFlags::ShowPlayable);
		++ArrayIndex;
	}

	const EAvaPlayableTransitionEntryRole PlayingPlayablesRole = EnumHasAnyFlags(TransitionFlags, EAvaPlayableTransitionFlags::TreatPlayingAsExiting) ?
		EAvaPlayableTransitionEntryRole::Exit : EAvaPlayableTransitionEntryRole::Playing;

	for (UAvaPlayable* Playable : PlayingPlayables)
	{
		if (ShouldKickOutPlayable(bIsPreviewFrameTransition, Playable, TransitionLayersToKickOut) || !Helper.AddTransitionBehaviorInstance(Playable, PlayingPlayablesRole))
		{
			// No Transition Tree (or kick out logic):
			// Stop exit/playing playable without any transition.
			// We can't integrate that with a transition tree for enter pages atm.
			UAvaPlayable::OnTransitionEvent().Broadcast(Playable, this, EAvaPlayableTransitionEventFlags::StopPlayable);
		}
	}

	for (UAvaPlayable* Playable : ExitPlayables)
	{
		if (ShouldKickOutPlayable(bIsPreviewFrameTransition, Playable, TransitionLayersToKickOut) || !Helper.AddTransitionBehaviorInstance(Playable, EAvaPlayableTransitionEntryRole::Exit))
		{
			// No Transition Tree (or kick out logic):
			// Stop exit/playing playable without any transition.
			// We can't integrate that with a transition tree for enter pages atm.
			UAvaPlayable::OnTransitionEvent().Broadcast(Playable, this, EAvaPlayableTransitionEventFlags::StopPlayable);
		}
	}

	bool bTransitionExecutorStarted = false;
	if (Helper.HasBehaviorInstances() && Helper.LastTransitionSubsystem)
	{
		using namespace UE::AvaPlayback::Utils;
		UE_LOGF(LogAvaPlayable, Verbose, "%ls Playable Transition \"%ls\" starting.", *GetBriefFrameInfo(), *Helper.GetContextName());

		Helper.ExecutorBuilder.SetOnFinished(FSimpleDelegate::CreateUObject(this, &UAvaPlayableLocalTransition::OnTransitionExecutorEnded));
		
		TransitionExecutor = Helper.ExecutorBuilder.Build(*Helper.LastTransitionSubsystem);
		if (TransitionExecutor)
		{
			TransitionExecutor->Start();
			bTransitionExecutorStarted = true;
		}
	}

	// If no transition tree, skip to next phase of the transition.
	if (!bTransitionExecutorStarted)
	{
		PostTransitionExecutorPhase();
	}

	// Allows client-side to implement forked/clustered channel state reconciling.
	UAvaPlayable::OnTransitionEvent().Broadcast(nullptr, this, EAvaPlayableTransitionEventFlags::Starting);

	return true;
}

void UAvaPlayableLocalTransition::Stop()
{
	if (TransitionExecutor.IsValid())
	{
		TransitionExecutor->Stop();
		
		// Stop should've called OnTransitionTreeEnded, which resets the ptr.
		ensure(!TransitionExecutor.IsValid());
	}

	UAvaPlayable::OnSequenceEvent().RemoveAll(this);

	Super::Stop();
}

bool UAvaPlayableLocalTransition::IsRunning() const
{
	// The transition can be considered "running" either if it is executing
	// a transition tree or if it has "post transition tree" sequences going.
	return TransitionExecutor.IsValid() || bWaitOnPostExecutorSequences;
}

void UAvaPlayableLocalTransition::Tick(double InDeltaSeconds)
{
	using namespace UE::AvaPlayableTransition::Private;
	
	// We need to poll the sequences. Sequence Events are not reliable.
	if (bWaitOnPostExecutorSequences)
	{
		if (!HasActiveSequences(PostExecutorSequencePlayablesWeak))
		{
			FinishWaitOnPostExecutorSequences();
			NotifyTransitionFinished();
		}
	}
}

void UAvaPlayableLocalTransition::OnTransitionExecutorEnded()
{
	using namespace UE::AvaPlayback::Utils;
	
	if (TransitionExecutor.IsValid())
	{
		// Notify all the playable that have been discarded.
		auto StopDiscardedPlayables = [this](const TArray<TWeakObjectPtr<UAvaPlayable>>& InPlayablesWeak)
		{
			for (const TWeakObjectPtr<UAvaPlayable>& PlayableWeak : InPlayablesWeak)
			{
				if (UAvaPlayable* Playable = PlayableWeak.Get())
				{
					if (DiscardPlayablesWeak.Contains(Playable))
					{
						UAvaPlayable::OnTransitionEvent().Broadcast(Playable, this, EAvaPlayableTransitionEventFlags::StopPlayable);
					}
				}
			}
		};

		StopDiscardedPlayables(PlayingPlayablesWeak);
		StopDiscardedPlayables(ExitPlayablesWeak);

		TransitionExecutor.Reset();
		
		UE_LOGF(LogAvaPlayable, Verbose, "%ls Transition Executor \"%ls\" ended.", *GetBriefFrameInfo(), *GetInstanceName());
	}

	PostTransitionExecutorPhase();
}

void UAvaPlayableLocalTransition::PostTransitionExecutorPhase()
{
	bool bSequenceStarted = false;
	// Deal with non-TL enter playables. After the TL pages have been taken out,
	// we need to start the sequences.
	for (const TWeakObjectPtr<UAvaPlayable>& EnterPlayableWeak : PostExecutorSequencePlayablesWeak)
	{
		if (UAvaPlayable* Playable = EnterPlayableWeak.Get())
		{
			FAvaPlaybackAnimPlaySettings AnimSettings;
			if (EnumHasAnyFlags(TransitionFlags, EAvaPlayableTransitionFlags::PlayEnterPlayablesAtPreviewFrame))
			{
				AnimSettings.Action = EAvaPlaybackAnimAction::PreviewFrame;	
			}
			else
			{
				AnimSettings.Action = EAvaPlaybackAnimAction::Play;
			}

			// Default settings will play all sequences.
			if (Playable->ExecuteAnimationCommand(AnimSettings.Action, AnimSettings) == EAvaPlayableCommandResult::Executed)
			{
				bSequenceStarted = true;
			}
		}
	}

	const FAvaInstanceSettings& PlaybackInstanceSettings = IAvaMediaModule::Get().GetAvaInstanceSettings();

	if (bSequenceStarted && PlaybackInstanceSettings.bDefaultPlayableTransitionWaitForSequences)
	{
		StartWaitOnPostExecutorSequences();
	}
	else
	{
		NotifyTransitionFinished();
	}
}

void UAvaPlayableLocalTransition::NotifyTransitionFinished()
{
	using namespace UE::AvaPlayback::Utils;
	UE_LOGF(LogAvaPlayable, Verbose, "%ls Playable Transition \"%ls\" ended.", *GetBriefFrameInfo(), *GetInstanceName());

	// This will indicate the playable transition is completed and can be cleaned up.
	// Combo templates break the one page one playable rule, so we need a dedicated event to signal the end of the playable transition.
	UAvaPlayable::OnTransitionEvent().Broadcast(nullptr, this, EAvaPlayableTransitionEventFlags::Finished);
}

void UAvaPlayableLocalTransition::StartWaitOnPostExecutorSequences()
{
	bWaitOnPostExecutorSequences = true;

	// Make sure we listen to sequence events.
	if (UAvaPlayable::OnSequenceEvent().IsBoundToObject(this))
	{
		UAvaPlayable::OnSequenceEvent().AddUObject(this, &UAvaPlayableLocalTransition::OnPlayableSequenceEvent);
	}
}

void UAvaPlayableLocalTransition::FinishWaitOnPostExecutorSequences()
{
	bWaitOnPostExecutorSequences = false;
	
	// Remark: This is the only task of playable transition requiring to listen to sequence events, *for now*.
	UAvaPlayable::OnSequenceEvent().RemoveAll(this);
}

void UAvaPlayableLocalTransition::OnPlayableSequenceEvent(UAvaPlayable* InPlayable, FName InSequenceLabel, EAvaPlayableSequenceEventType InEventType)
{
	// Remark: the sequence events are not entirely reliable.
	// The transitions are also "ticked" to poll this condition.
	using namespace UE::AvaPlayableTransition::Private;

	if (PostExecutorSequencePlayablesWeak.Contains(InPlayable) && InEventType == EAvaPlayableSequenceEventType::Finished)
	{
		// Check if this was the last active sequence of this transition.
		if (!HasActiveSequences(PostExecutorSequencePlayablesWeak))
		{
			FinishWaitOnPostExecutorSequences();
			NotifyTransitionFinished();
		}
	}
}

FString UAvaPlayableLocalTransition::GetInstanceName() const
{
	return TransitionId.IsValid() ? TransitionId.ToString() : GetFullName();
}
