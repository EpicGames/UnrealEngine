// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "Playable/Transition/AvaPlayableTransition.h"
#include "AvaPlayableLocalTransition.generated.h"

class IAvaTransitionExecutor;

/**
 * Implements the transition tree execution for all playables with one.
 * It also implements the "hardcoded" TL for playables with no transition tree.
 * The hardcoded TL rules are:
 * - "exit" playables are stopped with "out" sequences defines in transition tree
 *   if there is one, otherwise nothing is done.
 * - "enter" playables have their animation triggered after the transition tree is finished, letting the
 *  exit pages animated out as they should.
 */
UCLASS()
class UAvaPlayableLocalTransition : public UAvaPlayableTransition
{
	GENERATED_BODY()
	
public:
	//~ Begin UAvaPlayableTransition
	virtual bool Start() override;
	virtual void Stop() override;
	virtual bool IsRunning() const override;
	virtual void Tick(double InDeltaSeconds) override;
	//~ End UAvaPlayableTransition

protected:
	void OnTransitionExecutorEnded();
	
	void PostTransitionExecutorPhase();

	void OnPlayableSequenceEvent(UAvaPlayable* InPlayable, FName InSequenceLabel, EAvaPlayableSequenceEventType InEventType);
	
	void NotifyTransitionFinished();

	void StartWaitOnPostExecutorSequences();
	void FinishWaitOnPostExecutorSequences();

	FString GetInstanceName() const;
	
protected:
	/** Transition tree executor. */
	TSharedPtr<IAvaTransitionExecutor> TransitionExecutor;

	/** List of enter playables that have no TL and need to play the animations after the transition tree. */
	TArray<TWeakObjectPtr<UAvaPlayable>> PostExecutorSequencePlayablesWeak;

	/** Indicate the transition is waiting on sequences. */
	bool bWaitOnPostExecutorSequences = false;
};