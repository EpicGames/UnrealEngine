// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playable/Transition/AvaPlayableRemoteTransition.h"

#include "IAvaMediaModule.h"
#include "Playable/AvaPlayable.h"
#include "Playable/Transition/AvaPlayableTransitionPrivate.h"
#include "Playback/AvaPlaybackClientDelegates.h"
#include "Playback/AvaPlaybackUtils.h"
#include "Playback/IAvaPlaybackClient.h"

UAvaPlayableRemoteTransition::~UAvaPlayableRemoteTransition()
{
	UnregisterFromPlaybackClientDelegates();
}

bool UAvaPlayableRemoteTransition::Start()
{
	IAvaMediaModule& AvaMediaModule = IAvaMediaModule::Get();
	
	if (!AvaMediaModule.IsPlaybackClientStarted())
	{
		return false;
	}

	if (!Super::Start())
	{
		return false;
	}

	if (!TransitionId.IsValid())
	{
		TransitionId = FGuid::NewGuid();
	}
	
	using namespace UE::AvaPlayableTransition::Private;
	TArray<FGuid> EnterInstanceIds = GetInstanceIds(EnterPlayablesWeak);
	TArray<FGuid> PlayingInstanceIds = GetInstanceIds(PlayingPlayablesWeak);
	TArray<FGuid> ExitInstanceIds = GetInstanceIds(ExitPlayablesWeak);
	TArray<FAvaPlayableRemoteControlValues> EnterValues;
	EnterValues.Reserve(EnterPlayableValues.Num());
	for (const TSharedPtr<FAvaPlayableRemoteControlValues>& Values : EnterPlayableValues)
	{
		EnterValues.Add(Values.IsValid() ? *Values : FAvaPlayableRemoteControlValues::GetDefaultEmpty());
	}
	
	IAvaPlaybackClient& PlaybackClient = AvaMediaModule.GetPlaybackClient();
	PlaybackClient.RequestPlayableTransitionStart(TransitionId, MoveTemp(EnterInstanceIds), MoveTemp(PlayingInstanceIds), MoveTemp(ExitInstanceIds), MoveTemp(EnterValues), ChannelName, TransitionFlags);

	TArray<FString> Servers = PlaybackClient.GetOnlineServersForChannel(ChannelName);
	for (const FString& Server : Servers)
	{
		FRemoteStatusInfo& StatusInfo = RemoteStatusPerServer.Add(Server);
		StatusInfo.Status = ERemoteStatus::StartRequest;
	}

	RegisterToPlaybackClientDelegates();	// to get the transition events from the server side.

	using namespace UE::AvaPlayback::Utils;
	UE_LOGF(LogAvaPlayable, Verbose, "%ls Remote Playable Transition \"%ls\" starting.", *GetBriefFrameInfo(), *GetInstanceName());
	return true;
}

void UAvaPlayableRemoteTransition::Stop()
{
	IAvaMediaModule& AvaMediaModule = IAvaMediaModule::Get();

	// There is no need to request a stop if we have been notified the transition is already ended on all servers (avoids server side warning).
	if (AvaMediaModule.IsPlaybackClientStarted() && !IsTransitionFinishedOnAllServers())
	{
		AvaMediaModule.GetPlaybackClient().RequestPlayableTransitionStop(TransitionId, ChannelName);
	}

	// Forked/Clustered channels: Check for desyncs.
	if (RemoteStatusPerServer.Num() > 1 && IsTransitionFinishedOnAllServers())
	{
		auto CheckForDesync = [this](const FString& InValueName, const TFunction<int32(const FRemoteStatusInfo& InStatusInfo)>& InGetValueFunction)
		{
			bool bDesyncDetected = false;
			int32 DesyncDeltaFrame = 0;
			TOptional<int32> PreviousValue;
			for (const TPair<FString, FRemoteStatusInfo>& StatusInfo : RemoteStatusPerServer)
			{
				const int32 CurrentValue = InGetValueFunction(StatusInfo.Value);
				if (PreviousValue.IsSet() && PreviousValue.GetValue() != CurrentValue)
				{
					DesyncDeltaFrame = FMath::Abs(PreviousValue.GetValue() - CurrentValue);
					bDesyncDetected = true;
					break;
				}
				PreviousValue = CurrentValue;
			}

			// Report desync
			if (bDesyncDetected)
			{
				using namespace UE::AvaPlayback::Utils;
				UE_LOGF(LogAvaPlayable, Warning, "%ls Remote Playable Transition \"%ls\" %ls Desync Detected: Delta Frames: %d.",
					*GetBriefFrameInfo(), *GetInstanceName(), *InValueName, DesyncDeltaFrame);
			}
		};

		CheckForDesync(TEXT("Start"), [](const FRemoteStatusInfo& InStatusInfo)
		{
			return InStatusInfo.StartFrameNumber;
		});
		CheckForDesync(TEXT("Finish"), [](const FRemoteStatusInfo& InStatusInfo)
		{
			return InStatusInfo.FinishFrameNumber;
		});
	}

	Super::Stop();
}

void UAvaPlayableRemoteTransition::RegisterToPlaybackClientDelegates()
{
	UE::AvaPlaybackClient::Delegates::GetOnPlaybackTransitionEvent().RemoveAll(this);
	UE::AvaPlaybackClient::Delegates::GetOnPlaybackTransitionEvent().AddUObject(this, &UAvaPlayableRemoteTransition::HandlePlaybackTransitionEvent);
	UE::AvaPlaybackClient::Delegates::GetOnConnectionEvent().RemoveAll(this);
	UE::AvaPlaybackClient::Delegates::GetOnConnectionEvent().AddUObject(this, &UAvaPlayableRemoteTransition::HandleRemoteConnectionEvent);
}

void UAvaPlayableRemoteTransition::UnregisterFromPlaybackClientDelegates() const
{
	UE::AvaPlaybackClient::Delegates::GetOnPlaybackTransitionEvent().RemoveAll(this);
	UE::AvaPlaybackClient::Delegates::GetOnConnectionEvent().RemoveAll(this);
}

bool UAvaPlayableRemoteTransition::IsRunning() const
{
	return !RemoteStatusPerServer.IsEmpty() && !IsTransitionFinishedOnAllServers();
}

void UAvaPlayableRemoteTransition::HandlePlaybackTransitionEvent(IAvaPlaybackClient& InPlaybackClient,
	const UE::AvaPlaybackClient::Delegates::FPlaybackTransitionEventArgs& InArgs)
{
	using namespace UE::AvaPlayback::Utils;

	if (InArgs.TransitionId != TransitionId)
	{ 
		return;
	}

	const bool bIsTransitionStartingEvent = EnumHasAnyFlags(InArgs.EventFlags, EAvaPlayableTransitionEventFlags::Starting);
	const bool bIsTransitionFinishedEvent = EnumHasAnyFlags(InArgs.EventFlags, EAvaPlayableTransitionEventFlags::Finished);
	
	if (bIsTransitionStartingEvent)
	{
		// Don't stomp finished state.
		if (GetRemoteStatus(InArgs.ServerName) != ERemoteStatus::Finished)
		{
			SetRemoteStatus(InArgs.ServerName, ERemoteStatus::Started);
		}
		else
		{
			UE_LOGF(LogAvaPlayable, Warning,
				"%ls Remote Playable Transition \"%ls\" discarding \"Starting\" event from server \"%ls\" because it is already finished.",
				*GetBriefFrameInfo(), *GetInstanceName(), *InArgs.ServerName);
		}

		SetRemoteStartFrame(InArgs.ServerName, InArgs.ServerFrameNumber);
	}

	if (bIsTransitionFinishedEvent)	// Finished has priority over all other states.
	{
		SetRemoteStatus(InArgs.ServerName, ERemoteStatus::Finished);
		SetRemoteFinishFrame(InArgs.ServerName, InArgs.ServerFrameNumber);

		UE_LOGF(LogAvaPlayable, Verbose, "%ls Remote Playable Transition \"%ls\" ended on server \"%ls\" at frame [%d].",
			*GetBriefFrameInfo(), *GetInstanceName(), *InArgs.ServerName, InArgs.ServerFrameNumber);
	}

	if (InArgs.InstanceId.IsValid())
	{
		if (UAvaPlayable* Playable = FindPlayable(InArgs.InstanceId))
		{
			const bool bIsStopPlayableEvent = EnumHasAnyFlags(InArgs.EventFlags, EAvaPlayableTransitionEventFlags::StopPlayable);

			// Reconciling stop playable event for forked channels.
			// We only propagate the stop event if all forked channels have stopped.
			if (bIsStopPlayableEvent)
			{
				UE_LOGF(LogAvaPlayable, Verbose,
					"%ls Remote Playable Transition \"%ls\": Marking playable instance Id \"%ls\" for stop on server \"%ls\".",
					*GetBriefFrameInfo(), *GetInstanceName(), *InArgs.InstanceId.ToString(), *InArgs.ServerName);

				MarkPlayableForStop(InArgs.ServerName, InArgs.InstanceId);
			}

			// Relay locally through playable event.
			if (!bIsStopPlayableEvent || IsPlayableMarkedForStopOnAllServers(InArgs.InstanceId))
			{
				// Prevent locally broadcasting the instance event more than once.
				if (!HasEventBeenBroadcast(InArgs.InstanceId, InArgs.EventFlags))
				{
					UE_LOGF(LogAvaPlayable, Verbose,
						"%ls Remote Playable Transition \"%ls\": Playable instance Id \"%ls\" was marked for stop on all servers. Broadcasting locally.",
						*GetBriefFrameInfo(), *GetInstanceName(), *InArgs.InstanceId.ToString());

					UAvaPlayable::OnTransitionEvent().Broadcast(Playable, this, InArgs.EventFlags);
					MarkEventsBroadcast(InArgs.InstanceId, InArgs.EventFlags);
				}
			}
		}
		else
		{
			UE_LOGF(LogAvaPlayable, Error,
				"%ls Remote Playable Transition \"%ls\" doesn't have playable instance Id \"%ls\".",
				*GetBriefFrameInfo(), *GetInstanceName(), *InArgs.InstanceId.ToString());
		}
	}
	else
	{
		// Event propagation reconciling for forked channels.
		// Only propagate "finish" event when all remote transitions are finished.
		if ((bIsTransitionFinishedEvent && IsTransitionFinishedOnAllServers()) || bIsTransitionStartingEvent)
		{
			UE_LOGF(LogAvaPlayable, Verbose,
				"%ls Remote Playable Transition \"%ls\": Finish event received from on all servers. Broadcasting locally.",
				*GetBriefFrameInfo(), *GetInstanceName());

			UAvaPlayable::OnTransitionEvent().Broadcast(nullptr, this, InArgs.EventFlags);
		}
	}
}

void UAvaPlayableRemoteTransition::HandleRemoteConnectionEvent(IAvaPlaybackClient& InPlaybackClient,
	const UE::AvaPlaybackClient::Delegates::FConnectionEventArgs& InArgs)
{
	using namespace UE::AvaPlaybackClient::Delegates;
	if (InArgs.Event == EConnectionEvent::ServerDisconnected)
	{
		RemoteStatusPerServer.Remove(InArgs.ServerName);
		if (IsTransitionFinishedOnAllServers())
		{
			UAvaPlayable::OnTransitionEvent().Broadcast(nullptr, this, EAvaPlayableTransitionEventFlags::Finished);
		}
	}
}

UAvaPlayableRemoteTransition::ERemoteStatus UAvaPlayableRemoteTransition::GetRemoteStatus(const FString& InServer) const
{
	const FRemoteStatusInfo* FoundRemoteStatus = RemoteStatusPerServer.Find(InServer);
	return FoundRemoteStatus ? FoundRemoteStatus->Status : ERemoteStatus::Unknown;
}

void UAvaPlayableRemoteTransition::SetRemoteStatus(const FString& InServer, ERemoteStatus InStatus)
{
	RemoteStatusPerServer.FindOrAdd(InServer).Status = InStatus;
}

void UAvaPlayableRemoteTransition::SetRemoteStartFrame(const FString& InServer, int32 InFrameNumber)
{
	RemoteStatusPerServer.FindOrAdd(InServer).StartFrameNumber = InFrameNumber;
}

void UAvaPlayableRemoteTransition::SetRemoteFinishFrame(const FString& InServer, int32 InFrameNumber)
{
	RemoteStatusPerServer.FindOrAdd(InServer).FinishFrameNumber = InFrameNumber;
}

bool UAvaPlayableRemoteTransition::IsTransitionFinishedOnAllServers() const
{
	for (const TPair<FString, FRemoteStatusInfo>& StatusInfo : RemoteStatusPerServer)
	{
		if (StatusInfo.Value.Status != ERemoteStatus::Finished)
		{
			return false;
		}
	}
	return true;
}

void UAvaPlayableRemoteTransition::MarkPlayableForStop(const FString& InServer, const FGuid& InInstanceId)
{
	if (FRemoteStatusInfo* FoundRemoteStatus = RemoteStatusPerServer.Find(InServer))
	{
		FoundRemoteStatus->PlayablesMarkedForStop.Add(InInstanceId);
	}
}

bool UAvaPlayableRemoteTransition::IsPlayableMarkedForStopOnAllServers(const FGuid& InInstanceId) const
{
	for (const TPair<FString, FRemoteStatusInfo>& StatusInfo : RemoteStatusPerServer)
	{
		if (!StatusInfo.Value.PlayablesMarkedForStop.Contains(InInstanceId))
		{
			return false;
		}
	}
	return true; // was marked on all servers.
}

FString UAvaPlayableRemoteTransition::GetInstanceName() const
{
	return TransitionId.ToString();
}
