// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playable/Transition/AvaPlayableTransition.h"
#include "AvaPlayableRemoteTransition.generated.h"

class IAvaPlaybackClient;

namespace UE::AvaPlaybackClient::Delegates
{
	struct FPlaybackTransitionEventArgs;
	struct FConnectionEventArgs;
}

UCLASS()
class UAvaPlayableRemoteTransition : public UAvaPlayableTransition
{
	GENERATED_BODY()
	
public:
	virtual ~UAvaPlayableRemoteTransition() override;

	void SetChannelName(const FName& InChannelName) { ChannelName = InChannelName; }
	
	//~ Begin UAvaPlayableTransition
	virtual bool Start() override;
	virtual void Stop() override;
	virtual bool IsRunning() const override;
	//~ End UAvaPlayableTransition

protected:
	void HandlePlaybackTransitionEvent(IAvaPlaybackClient& InPlaybackClient,
		const UE::AvaPlaybackClient::Delegates::FPlaybackTransitionEventArgs& InArgs);

	void HandleRemoteConnectionEvent(IAvaPlaybackClient& InPlaybackClient,
		const UE::AvaPlaybackClient::Delegates::FConnectionEventArgs& InArgs);

	void RegisterToPlaybackClientDelegates();
	void UnregisterFromPlaybackClientDelegates() const;

	enum class ERemoteStatus : uint8
	{
		Unknown,
		StartRequest,
		Started,
		Finished
	};

	ERemoteStatus GetRemoteStatus(const FString& InServer) const;
	void SetRemoteStatus(const FString& InServer, ERemoteStatus InStatus);
	void SetRemoteStartFrame(const FString& InServer, int32 InFrameNumber);
	void SetRemoteFinishFrame(const FString& InServer, int32 InFrameNumber);

	bool IsTransitionFinishedOnAllServers() const;

	void MarkPlayableForStop(const FString& InServer, const FGuid& InInstanceId);
	bool IsPlayableMarkedForStopOnAllServers(const FGuid& InInstanceId) const;
	
	FString GetInstanceName() const;

	/** Mark the given events as broadcast for the given instance. */
	void MarkEventsBroadcast(const FGuid& InInstanceId, EAvaPlayableTransitionEventFlags InFlags)
	{
		if (EAvaPlayableTransitionEventFlags* Flags = InstanceBroadcastEvent.Find(InInstanceId))
		{
			*Flags |= InFlags;
		}
		else
		{
			InstanceBroadcastEvent.Add(InInstanceId, InFlags);
		}
	}

	/** Returns true if the given events were marked as broadcast for the given instance. */
	bool HasEventBeenBroadcast(const FGuid& InInstanceId, EAvaPlayableTransitionEventFlags InFlags) const
	{
		if (const EAvaPlayableTransitionEventFlags* Flags = InstanceBroadcastEvent.Find(InInstanceId))
		{
			return EnumHasAllFlags(*Flags, InFlags);
		}
		return false;
	}
	
protected:
	FName ChannelName;

	/**
	 * Forked channel support: tracking the status of the transition per
	 * server to be able to reconcile local event propagation.
	 */
	struct FRemoteStatusInfo
	{
		ERemoteStatus Status = ERemoteStatus::Unknown;
		int32 StartFrameNumber = 0;
		int32 FinishFrameNumber = 0;
		TSet<FGuid> PlayablesMarkedForStop;
	};
	
	TMap<FString, FRemoteStatusInfo> RemoteStatusPerServer;

	// Keep track of the events that were locally broadcast to avoid sending them more than once.
	TMap<FGuid, EAvaPlayableTransitionEventFlags> InstanceBroadcastEvent;
};