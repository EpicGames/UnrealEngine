// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playable/Transition/AvaPlayableTransition.h"

#include "Algo/Accumulate.h"
#include "Playable/AvaPlayable.h"
#include "Playable/Transition/AvaPlayableLocalTransition.h"
#include "Playable/Transition/AvaPlayableRemoteTransition.h"
#include "Playable/Transition/AvaPlayableTransitionPrivate.h"

bool UAvaPlayableTransition::Start()
{
	using namespace UE::AvaPlayableTransition::Private;

	// Accumulate all the playable groups this transition is part of.
	PlayableGroupsWeak.Reset();
	auto AccumulatePlayableGroups = [this](const TArray<TWeakObjectPtr<UAvaPlayable>>& InPlayablesWeak)
	{
		for (const TWeakObjectPtr<UAvaPlayable>& PlayableWeak : InPlayablesWeak)
		{
			if (const UAvaPlayable* Playable = PlayableWeak.Get())
			{
				PlayableGroupsWeak.Add(Playable->GetPlayableGroup());
			}
		}
	};

	AccumulatePlayableGroups(EnterPlayablesWeak);
	AccumulatePlayableGroups(ExitPlayablesWeak);

	for (const TWeakObjectPtr<UAvaPlayableGroup>& PlayableGroupWeak : PlayableGroupsWeak)
	{
		if (UAvaPlayableGroup* PlayableGroup = PlayableGroupWeak.Get())
		{
			PlayableGroup->RegisterPlayableTransition(this);
		}
	}
	return true;
}

void UAvaPlayableTransition::Stop()
{
	for (const TWeakObjectPtr<UAvaPlayableGroup>& PlayableGroupWeak : PlayableGroupsWeak)
	{
		if (UAvaPlayableGroup* PlayableGroup = PlayableGroupWeak.Get())
		{
			PlayableGroup->UnregisterPlayableTransition(this);
		}
	}
}

void UAvaPlayableTransition::SetTransitionId(const FGuid& InTransitionId)
{
	TransitionId = InTransitionId;
}

void UAvaPlayableTransition::SetTransitionFlags(EAvaPlayableTransitionFlags InFlags)
{
	TransitionFlags = InFlags;
}

void UAvaPlayableTransition::SetEnterPlayables(TArray<TWeakObjectPtr<UAvaPlayable>>&& InPlayablesWeak)
{
	EnterPlayablesWeak = MoveTemp(InPlayablesWeak);
}

void UAvaPlayableTransition::SetPlayingPlayables(TArray<TWeakObjectPtr<UAvaPlayable>>&& InPlayablesWeak)
{
	PlayingPlayablesWeak = MoveTemp(InPlayablesWeak);
}

void UAvaPlayableTransition::SetExitPlayables(TArray<TWeakObjectPtr<UAvaPlayable>>&& InPlayablesWeak)
{
	ExitPlayablesWeak = MoveTemp(InPlayablesWeak);
}

bool UAvaPlayableTransition::IsEnterPlayable(const UAvaPlayable* InPlayable) const
{
	return EnterPlayablesWeak.Contains(InPlayable);
}

bool UAvaPlayableTransition::IsPlayingPlayable(const UAvaPlayable* InPlayable) const
{
	return PlayingPlayablesWeak.Contains(InPlayable);
}

bool UAvaPlayableTransition::IsExitPlayable(const UAvaPlayable* InPlayable) const
{
	return ExitPlayablesWeak.Contains(InPlayable);
}

void UAvaPlayableTransition::SetEnterPlayableValues(TArray<TSharedPtr<FAvaPlayableRemoteControlValues>>&& InPlayableValues)
{
	EnterPlayableValues = MoveTemp(InPlayableValues);
}

TSharedPtr<FAvaPlayableRemoteControlValues> UAvaPlayableTransition::GetValuesForPlayable(const UAvaPlayable* InPlayable, bool bInIsEnterPlayable)
{
	if (bInIsEnterPlayable)
	{
		int32 ArrayIndex = 0;
		for (const TWeakObjectPtr<UAvaPlayable> PlayableWeak : EnterPlayablesWeak)
		{
			const UAvaPlayable* Playable = PlayableWeak.Get();
			if (Playable == InPlayable && EnterPlayableValues.IsValidIndex(ArrayIndex))
			{
				return EnterPlayableValues[ArrayIndex];
			}
			++ArrayIndex;
		}
	}
	else if (TSharedPtr<FAvaPlayableRemoteControlValues>* FoundValues = OtherPlayableValues.Find(InPlayable))
	{
		return *FoundValues;
	}

	return nullptr;
}

void UAvaPlayableTransition::MarkPlayableAsDiscard(UAvaPlayable* InPlayable)
{
	// Hack: Intercept "in-place" playable, don't mark them for discard.
	if (IsEnterPlayable(InPlayable) && IsPlayingPlayable(InPlayable))
	{
		return;
	}
	
	DiscardPlayablesWeak.AddUnique(InPlayable);
	UAvaPlayable::OnTransitionEvent().Broadcast(InPlayable, this, EAvaPlayableTransitionEventFlags::MarkPlayableDiscard);
}

FString UAvaPlayableTransition::GetPrettyInfo() const
{
	auto MakePrettyPlayableList = [](const TArray<TWeakObjectPtr<UAvaPlayable>>& InPlayablesWeak) -> FString
	{
		FString List;
		for (const TWeakObjectPtr<UAvaPlayable>& PlayableWeak : InPlayablesWeak)
		{
			if (const UAvaPlayable* Playable = PlayableWeak.Get())
			{
				List += List.IsEmpty() ? TEXT("") : TEXT(", ");
				List += Playable->GetUserData();
			}
		}
		return List;
	};

	const FString InList = MakePrettyPlayableList(EnterPlayablesWeak);
	const FString OutList = MakePrettyPlayableList(ExitPlayablesWeak);

	if (InList.IsEmpty())
	{
		return FString::Printf(TEXT("{Out:{%s}}"), *OutList);
	}

	if (OutList.IsEmpty())
	{
		return FString::Printf(TEXT("{In:{%s}}"), *InList);
	}

	return FString::Printf(TEXT("{In:{%s}, Out:{%s}}"), *InList, *OutList);
}

UAvaPlayable* UAvaPlayableTransition::FindPlayable(const FGuid& InInstanceId) const
{
	auto IsPlayablePredicate = [&InInstanceId](const TWeakObjectPtr<UAvaPlayable>& InPlayableWeak) -> bool
	{
		if (const UAvaPlayable* Playable = InPlayableWeak.Get())
		{
			return Playable->GetInstanceId() == InInstanceId;		
		}
		return false;
	};
	
	const TWeakObjectPtr<UAvaPlayable>* PlayableWeak = EnterPlayablesWeak.FindByPredicate(IsPlayablePredicate);
	if (PlayableWeak && PlayableWeak->IsValid())
	{
		return PlayableWeak->Get();
	}

	PlayableWeak = PlayingPlayablesWeak.FindByPredicate(IsPlayablePredicate);
	if (PlayableWeak && PlayableWeak->IsValid())
	{
		return PlayableWeak->Get();
	}

	PlayableWeak = ExitPlayablesWeak.FindByPredicate(IsPlayablePredicate);
	if (PlayableWeak && PlayableWeak->IsValid())
	{
		return PlayableWeak->Get();
	}
	
	return nullptr;
}

FAvaPlayableTransitionBuilder::FAvaPlayableTransitionBuilder() = default;

void FAvaPlayableTransitionBuilder::AddEnterPlayableValues(const TSharedPtr<FAvaPlayableRemoteControlValues>& InValues)
{
	EnterPlayableValues.Add(InValues);
}

bool FAvaPlayableTransitionBuilder::AddEnterPlayable(UAvaPlayable* InPlayable, bool bInAllowMultipleAdd)
{
	if (!bInAllowMultipleAdd && ExitPlayablesWeak.Contains(InPlayable))
	{
		using namespace UE::AvaPlayableTransition::Private;
		UE_LOGF(LogAvaPlayable, Error,
			"Playable Transition setup error: Playable {%ls} can't added as an \"enter\" playable because it is already in the \"exit\" list.",
			*GetPrettyPlayableInfo(InPlayable));
		return false;
	}

	if (!bInAllowMultipleAdd && PlayingPlayablesWeak.Contains(InPlayable))
	{
		using namespace UE::AvaPlayableTransition::Private;
		UE_LOGF(LogAvaPlayable, Error,
			"Playable Transition setup error: Playable {%ls} can't added as an \"enter\" playable because it is already in the \"playing\" list.",
			*GetPrettyPlayableInfo(InPlayable));
		return false;
	}
	
	EnterPlayablesWeak.AddUnique(InPlayable);
	return true;
}

bool FAvaPlayableTransitionBuilder::AddPlayingPlayable(UAvaPlayable* InPlayable, bool bInAllowMultipleAdd)
{
	if (!bInAllowMultipleAdd && EnterPlayablesWeak.Contains(InPlayable))
	{
		using namespace UE::AvaPlayableTransition::Private;
		UE_LOGF(LogAvaPlayable, Error,
			"Playable Transition setup error: Playable {%ls} can't added as an \"exit\" playable because it is already in the \"enter\" list.",
			*GetPrettyPlayableInfo(InPlayable));
		return false;
	}

	if (!bInAllowMultipleAdd && ExitPlayablesWeak.Contains(InPlayable))
	{
		using namespace UE::AvaPlayableTransition::Private;
		UE_LOGF(LogAvaPlayable, Error,
			"Playable Transition setup error: Playable {%ls} can't added as an \"exit\" playable because it is already in the \"exit\" list.",
			*GetPrettyPlayableInfo(InPlayable));
		return false;
	}

	
	PlayingPlayablesWeak.AddUnique(InPlayable);
	return true;
}

bool FAvaPlayableTransitionBuilder::AddExitPlayable(UAvaPlayable* InPlayable, bool bInAllowMultipleAdd)
{
	if (!bInAllowMultipleAdd && EnterPlayablesWeak.Contains(InPlayable))
	{
		using namespace UE::AvaPlayableTransition::Private;
		UE_LOGF(LogAvaPlayable, Error,
			"Playable Transition setup error: Playable {%ls} can't added as an \"exit\" playable because it is already in the \"enter\" list.",
			*GetPrettyPlayableInfo(InPlayable));
		return false;
	}

	if (!bInAllowMultipleAdd && PlayingPlayablesWeak.Contains(InPlayable))
	{
		using namespace UE::AvaPlayableTransition::Private;
		UE_LOGF(LogAvaPlayable, Error,
			"Playable Transition setup error: Playable {%ls} can't added as an \"enter\" playable because it is already in the \"playing\" list.",
			*GetPrettyPlayableInfo(InPlayable));
		return false;
	}

	ExitPlayablesWeak.AddUnique(InPlayable);
	return true;
}

UAvaPlayableTransition* FAvaPlayableTransitionBuilder::MakeTransition(UObject* InOuter, const FGuid& InTransitionId)
{
	// Determine if we have remote playables.
	using namespace UE::AvaPlayableTransition::Private;

	// A transition that has only "playing" playables does nothing.
	if (EnterPlayablesWeak.IsEmpty() && ExitPlayablesWeak.IsEmpty())
	{
		return nullptr;
	}

	// A remote transition is created only if all playables are remote.
	const bool bCreateRemoteTransition = AreAllPlayablesRemote(EnterPlayablesWeak)
		&& AreAllPlayablesRemote(ExitPlayablesWeak)
		&& AreAllPlayablesRemote(PlayingPlayablesWeak);
	
	UAvaPlayableTransition* PlayableTransition;
	if (bCreateRemoteTransition)
	{
		UAvaPlayableRemoteTransition* RemotePlayableTransition = NewObject<UAvaPlayableRemoteTransition>(InOuter);
		TArray<FName> ChannelNames;
		ChannelNames.Reserve(1);
		GetChannelNamesFromPlayables(EnterPlayablesWeak, ChannelNames);
		GetChannelNamesFromPlayables(ExitPlayablesWeak, ChannelNames);
		GetChannelNamesFromPlayables(PlayingPlayablesWeak, ChannelNames);
		if (ChannelNames.Num() > 1)
		{
			const FString ChannelNameList = Algo::Accumulate(ChannelNames, FString(), [](FString InResult, const FName& InName)
			{
				InResult = InResult.IsEmpty() ? InName.ToString() : InResult + ", " + InName.ToString();
				return InResult;
			});
		
			UE_LOGF(LogAvaPlayable, Warning,
				"Playable Transition setup warning: Playables from different channels (%ls) are in the same transition.",
				*ChannelNameList);
		}
		
		if (ChannelNames.Num() > 0)
		{
			RemotePlayableTransition->SetChannelName(ChannelNames[0]);
		}
		else
		{
			UE_LOGF(LogAvaPlayable, Error,
				"Playable Transition setup error: Couldn't get a channel name from the list of playables.");
		}
		
		PlayableTransition = RemotePlayableTransition;
	}
	else
	{
		PlayableTransition = NewObject<UAvaPlayableLocalTransition>(InOuter);
	}

	// There must be the same number of entries then there are playables.
	if (ensure(EnterPlayableValues.Num() == EnterPlayablesWeak.Num()))
	{
		PlayableTransition->SetEnterPlayableValues(MoveTemp(EnterPlayableValues));
	}

	PlayableTransition->SetEnterPlayables(MoveTemp(EnterPlayablesWeak));
	PlayableTransition->SetPlayingPlayables(MoveTemp(PlayingPlayablesWeak));
	PlayableTransition->SetExitPlayables(MoveTemp(ExitPlayablesWeak));
	PlayableTransition->SetTransitionId(InTransitionId);
	
	return PlayableTransition;
}
