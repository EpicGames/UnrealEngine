// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playable/Transition/AvaPlayableTransitionPrivate.h"

#include "Playable/AvaPlayable.h"
#include "Playable/Playables/AvaPlayableRemoteProxy.h"
#include "Playback/AvaPlaybackUtils.h"

namespace UE::AvaPlayableTransition::Private
{
	/**
	 * Convert the array of weak objects to an array of objects than can be used (hasn't expired).
	 */
	TArray<UAvaPlayable*> Pin(const TArray<TWeakObjectPtr<UAvaPlayable>>& InPlayablesWeak)
	{
		TArray<UAvaPlayable*> Playables;
		Playables.Reserve(InPlayablesWeak.Num());
		for (const TWeakObjectPtr<UAvaPlayable>& PlayableWeak : InPlayablesWeak)
		{
			if (UAvaPlayable* Playable = PlayableWeak.Get())
			{
				Playables.Add(Playable);
			}
		}
		return Playables;
	}

	TArray<FGuid> GetInstanceIds(const TArray<TWeakObjectPtr<UAvaPlayable>>& InPlayablesWeak)
	{
		TArray<FGuid> InstanceIds;
		InstanceIds.Reserve(InPlayablesWeak.Num());
		for (const TWeakObjectPtr<UAvaPlayable>& PlayableWeak : InPlayablesWeak)
		{
			if (const UAvaPlayable* Playable = PlayableWeak.Get())
			{
				InstanceIds.Add(Playable->GetInstanceId());
			}
		}
		return InstanceIds;
	}

	bool AreAllPlayablesRemote(const TArray<TWeakObjectPtr<UAvaPlayable>>& InPlayablesWeak)
	{
		for (const TWeakObjectPtr<UAvaPlayable>& PlayableWeak : InPlayablesWeak)
		{
			if (const UAvaPlayable* Playable = PlayableWeak.Get())
			{
				if (!Playable->IsA<UAvaPlayableRemoteProxy>())
				{
					return false;
				}
			}
		}
		return true;
	}

	void GetChannelNamesFromPlayables(const TArray<TWeakObjectPtr<UAvaPlayable>>& InPlayablesWeak, TArray<FName>& OutChannelNames)
	{
		for (const TWeakObjectPtr<UAvaPlayable>& PlayableWeak : InPlayablesWeak)
		{
			if (const UAvaPlayable* Playable = PlayableWeak.Get())
			{
				if (const UAvaPlayableRemoteProxy* RemotePlayable = Cast<UAvaPlayableRemoteProxy>(Playable))
				{
					OutChannelNames.AddUnique(RemotePlayable->GetPlayingChannelFName());
				}
			}
		}
	}

	FString GetPrettyPlayableInfo(const UAvaPlayable* InPlayable)
	{
		if (InPlayable)
		{
			return FString::Printf(TEXT("Id:%s, Asset:%s, Status:%s"),
				*InPlayable->GetInstanceId().ToString(),
				*InPlayable->GetSourceAssetPath().ToString(),
				*AvaPlayback::Utils::StaticEnumToString(InPlayable->GetPlayableStatus()));
		}
		return TEXT("(nullptr)");
	}
}
