// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playable/AvaPlayable.h"

namespace UE::AvaPlayableTransition::Private
{
	/** Convert the array of weak objects to an array of objects than can be used (hasn't expired). */
	TArray<UAvaPlayable*> Pin(const TArray<TWeakObjectPtr<UAvaPlayable>>& InPlayablesWeak);

	/** Returns the array of playable instance ids. */
	TArray<FGuid> GetInstanceIds(const TArray<TWeakObjectPtr<UAvaPlayable>>& InPlayablesWeak);

	/** Returns true if all the given playables are remote. */
	bool AreAllPlayablesRemote(const TArray<TWeakObjectPtr<UAvaPlayable>>& InPlayablesWeak);

	/** Collects all the channel names from the given playables. Only remote playables have channel. */
	void GetChannelNamesFromPlayables(const TArray<TWeakObjectPtr<UAvaPlayable>>& InPlayablesWeak, TArray<FName>& OutChannelNames);
	
	/** Returns a string with prettified information about the playable suitable for logging. */
	FString GetPrettyPlayableInfo(const UAvaPlayable* InPlayable);
}