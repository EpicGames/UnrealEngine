// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playable/Transition/AvaPlayableTransitionScene.h"

#include "AvaTransitionContext.h"
#include "IAvaSceneInterface.h"
#include "Playable/AvaPlayable.h"
#include "Playable/Transition/AvaPlayableTransition.h"
#include "Transition/Extensions/IAvaTransitionRCExtension.h"

namespace UE::AvaMedia::Private
{
	/** Controllers aren't applied to the Preset, instead, this compares the latest remote control values for a given playable */
	class FAvaRCTransitionPlayableExtension : public IAvaRCTransitionExtension
	{
		virtual EAvaTransitionComparisonResult CompareControllers(const FGuid& InControllerId, const FAvaTransitionContext& InMyContext, const FAvaTransitionContext& InOtherContext) const override
		{
			const FAvaTransitionScene* MyScene = InMyContext.GetTransitionScene();
			const FAvaTransitionScene* OtherScene = InOtherContext.GetTransitionScene();

			if (!MyScene || !OtherScene)
			{
				return EAvaTransitionComparisonResult::None;
			}

			const UAvaPlayable* MyPlayable = MyScene->GetDataView().GetPtr<UAvaPlayable>();
			const UAvaPlayable* OtherPlayable = OtherScene->GetDataView().GetPtr<UAvaPlayable>();

			if (!MyPlayable || !OtherPlayable)
			{
				return EAvaTransitionComparisonResult::None;
			}

			UAvaPlayableTransition* MyTransition = static_cast<const FAvaPlayableTransitionScene*>(MyScene)->PlayableTransitionWeak.Get();
			UAvaPlayableTransition* OtherTransition = static_cast<const FAvaPlayableTransitionScene*>(OtherScene)->PlayableTransitionWeak.Get();

			if (!MyTransition || !OtherTransition)
			{
				return EAvaTransitionComparisonResult::None;
			}

			const bool bIsMyPlayableEnter = InMyContext.GetTransitionType() == EAvaTransitionType::In;
			const bool bIsOtherPlayableEnter = InOtherContext.GetTransitionType() == EAvaTransitionType::In;

			TSharedPtr<const FAvaPlayableRemoteControlValues> MyValues = MyTransition->GetValuesForPlayable(MyPlayable, bIsMyPlayableEnter);
			TSharedPtr<const FAvaPlayableRemoteControlValues> OtherValues = OtherTransition->GetValuesForPlayable(OtherPlayable, bIsOtherPlayableEnter);

			if (!MyValues.IsValid() || !OtherValues.IsValid())
			{
				return EAvaTransitionComparisonResult::None;
			}

			const FAvaPlayableRemoteControlValue* MyValue = MyValues->ControllerValues.Find(InControllerId);
			const FAvaPlayableRemoteControlValue* OtherValue = OtherValues->ControllerValues.Find(InControllerId);

			if (!MyValue || !OtherValue)
			{
				return EAvaTransitionComparisonResult::None;
			}

			return MyValue->IsSameValueAs(*OtherValue)
				? EAvaTransitionComparisonResult::Same
				: EAvaTransitionComparisonResult::Different;
		}
	};
}

FAvaPlayableTransitionScene::FAvaPlayableTransitionScene(UAvaPlayable* InPlayable, UAvaPlayableTransition* InPlayableTransition)
	: FAvaTransitionScene(InPlayable)
	, PlayableTransitionWeak(InPlayableTransition)
{
	AddExtension<UE::AvaMedia::Private::FAvaRCTransitionPlayableExtension>();
}

FAvaPlayableTransitionScene::FAvaPlayableTransitionScene(const FAvaTagHandle& InTransitionLayer, UAvaPlayableTransition* InPlayableTransition)
	: FAvaPlayableTransitionScene(nullptr, InPlayableTransition)
{
	OverrideTransitionLayer = InTransitionLayer;
}

EAvaTransitionComparisonResult FAvaPlayableTransitionScene::Compare(const FAvaTransitionScene& InOther) const
{
	const UAvaPlayable* MyPlayable    = GetDataView().GetPtr<UAvaPlayable>();
	const UAvaPlayable* OtherPlayable = InOther.GetDataView().GetPtr<UAvaPlayable>();

	if (!MyPlayable || !OtherPlayable)
	{
		return EAvaTransitionComparisonResult::None;
	}

	// Determine if Template is the same via the Package Name To Load (i.e. Source Level)
	if (MyPlayable->GetSourceAssetPath() == OtherPlayable->GetSourceAssetPath())
	{
		return EAvaTransitionComparisonResult::Same;
	}

	return EAvaTransitionComparisonResult::Different;
}

ULevel* FAvaPlayableTransitionScene::GetLevel() const
{
	const UAvaPlayable* Playable = GetDataView().GetPtr<UAvaPlayable>();
	const IAvaSceneInterface* SceneInterface = Playable ? Playable->GetSceneInterface() : nullptr;
	return SceneInterface ? SceneInterface->GetSceneLevel() : nullptr;
}

void FAvaPlayableTransitionScene::GetOverrideTransitionLayer(FAvaTagHandle& OutTransitionLayer) const
{
	if (OverrideTransitionLayer.IsSet())
	{
		OutTransitionLayer = *OverrideTransitionLayer;
	}
}

void FAvaPlayableTransitionScene::OnFlagsChanged()
{
	UAvaPlayable* Playable = GetDataView().GetMutablePtr<UAvaPlayable>();
	UAvaPlayableTransition* PlayableTransition = PlayableTransitionWeak.Get();
	if (!Playable || !PlayableTransition)
	{
		return;
	}

	// Event received when the playable can be discarded/recycled.
	if (HasAnyFlags(EAvaTransitionSceneFlags::NeedsDiscard))
	{
		// Do some error checking.
		if (PlayableTransition->IsEnterPlayable(Playable))
		{
			UE_LOGF(LogAvaPlayable, Error, "Playable Transition \"%ls\" Error: An \"enter\" playable is being discarded.", *PlayableTransition->GetFullName());
		}
		PlayableTransition->MarkPlayableAsDiscard(Playable);
	}
}
