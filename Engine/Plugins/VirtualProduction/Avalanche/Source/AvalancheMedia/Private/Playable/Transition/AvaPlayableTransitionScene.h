// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagHandle.h"
#include "AvaTransitionScene.h"
#include "AvaPlayableTransitionScene.generated.h"

class UAvaPlayable;
class UAvaPlayableTransition;

USTRUCT()
struct FAvaPlayableTransitionScene : public FAvaTransitionScene
{
	GENERATED_BODY()

	FAvaPlayableTransitionScene() = default;

	explicit FAvaPlayableTransitionScene(UAvaPlayable* InPlayable, UAvaPlayableTransition* InPlayableTransition);

	explicit FAvaPlayableTransitionScene(const FAvaTagHandle& InTransitionLayer, UAvaPlayableTransition* InPlayableTransition);

	//~ Begin FAvaTransitionScene
	virtual EAvaTransitionComparisonResult Compare(const FAvaTransitionScene& InOther) const override;
	virtual ULevel* GetLevel() const override;
	virtual void GetOverrideTransitionLayer(FAvaTagHandle& OutTransitionLayer) const override;
	virtual void OnFlagsChanged() override;
	//~ End FAvaTransitionScene

	TWeakObjectPtr<UAvaPlayableTransition> PlayableTransitionWeak;

	TOptional<FAvaTagHandle> OverrideTransitionLayer;
};
