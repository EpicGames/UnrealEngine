// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/AvaPlaybackClient.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AvaMediaMessageUtils.h"
#include "AvaMediaSettings.h"
#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/IAvaBroadcastSettings.h"
#include "Broadcast/OutputDevices/AvaBroadcastDeviceProviderProxy.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputUtils.h"
#include "IAvaMediaModule.h"
#include "IAvaModule.h"
#include "IMediaIOCoreModule.h"
#include "MediaOutput.h"
#include "MessageEndpointBuilder.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "Playback/AvaPlaybackClientDelegates.h"
#include "Playback/AvaPlaybackUtils.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"

DEFINE_LOG_CATEGORY_STATIC(LogAvaPlaybackClient, Log, All);

namespace UE::AvaPlaybackClient::Private
{
	bool IsPlaybackActionAllowedForNullAsset(EAvaPlaybackAction InAction)
	{
		switch (InAction)
		{
		case EAvaPlaybackAction::None:
			return false;
		case EAvaPlaybackAction::Load:
		case EAvaPlaybackAction::Start:
		case EAvaPlaybackAction::Stop:
		case EAvaPlaybackAction::Unload:
			return true;
		case EAvaPlaybackAction::Status:
		case EAvaPlaybackAction::SetUserData:
		case EAvaPlaybackAction::GetUserData:
	default:
		return false;
		}
	}

	bool IsAssetMissing(EAvaPlaybackAssetStatus InAssetStatus)
	{
		return InAssetStatus == EAvaPlaybackAssetStatus::Missing;
	}

	bool IsAssetAvailable(EAvaPlaybackAssetStatus InAssetStatus)
	{
		return InAssetStatus == EAvaPlaybackAssetStatus::Available
		|| InAssetStatus == EAvaPlaybackAssetStatus::MissingDependencies;
	}

	FString GetCommandActionString(EAvaPlaybackAction InAction, const FString& InArguments)
	{
		FString ActionString = AvaPlayback::Utils::StaticEnumToString(InAction); 
		if (InArguments.IsEmpty())
		{
			return ActionString;
		}

		return FString::Printf(TEXT("%s \"%s\""), *ActionString, *InArguments);
	};
}

FAvaPlaybackClient::FAvaPlaybackClient(IAvaMediaModule* InParentModule)
	: ParentModule(InParentModule)
{
	UPackage::PreSavePackageWithContextEvent.AddRaw(this, &FAvaPlaybackClient::OnPreSavePackage);
	UPackage::PackageSavedWithContextEvent.AddRaw(this, &FAvaPlaybackClient::OnPackageSaved);
	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		AssetRegistry->OnAssetRemoved().AddRaw(this, &FAvaPlaybackClient::OnAssetRemoved);
	}
}

FAvaPlaybackClient::~FAvaPlaybackClient()
{
	if (BroadcastChangedHandle.IsValid())
	{
		UAvaBroadcast::Get().RemoveChangeListener(BroadcastChangedHandle);
	}

	FAvaBroadcastOutputChannel::GetOnChannelChanged().RemoveAll(this);
	
	UPackage::PreSavePackageWithContextEvent.RemoveAll(this);
	UPackage::PackageSavedWithContextEvent.RemoveAll(this);
	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		AssetRegistry->OnAssetRemoved().RemoveAll(this);
	}
	
	FTSTicker::RemoveTicker(PingTickDelegateHandle);
	FCoreDelegates::OnEndFrame.RemoveAll(this);
	FMessageEndpoint::SafeRelease(MessageEndpoint);

	for (IConsoleObject* ConsoleCmd : ConsoleCommands)
	{
		IConsoleManager::Get().UnregisterConsoleObject(ConsoleCmd);
	}
	ConsoleCommands.Empty();

#if WITH_EDITOR
	if (UObjectInitialized())
	{
		UAvaMediaSettings& AvaMediaSettings = UAvaMediaSettings::GetMutable();
		AvaMediaSettings.OnSettingChanged().RemoveAll(this);
	}
#endif
}

void FAvaPlaybackClient::Init()
{
	ComputerName = FPlatformProcess::ComputerName();
	ProjectContentPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir());
	ProcessId = FPlatformProcess::GetCurrentProcessId();
	
	RegisterCommands();

	// Create our end point.
	MessageEndpoint = FMessageEndpoint::Builder("AvaPlaybackClient")
		.Handling<FAvaPlaybackPong>(this, &FAvaPlaybackClient::HandlePlaybackPongMessage)
		.Handling<FAvaPlaybackLog>(this, &FAvaPlaybackClient::HandlePlaybackLogMessage)
		.Handling<FAvaPlaybackUpdateServerUserData>(this, &FAvaPlaybackClient::HandleUpdateServerUserData)
		.Handling<FAvaPlaybackStatStatus>(this, &FAvaPlaybackClient::HandleStatStatus)
		.Handling<FAvaBroadcastDeviceProviderDataList>(this, &FAvaPlaybackClient::HandleDeviceProviderDataListMessage)
		.Handling<FAvaBroadcastStatus>(this, &FAvaPlaybackClient::HandleBroadcastStatusMessage)
		.Handling<FAvaPlaybackAssetStatus>(this, &FAvaPlaybackClient::HandlePlaybackAssetStatusMessage)
		.Handling<FAvaPlaybackStatus>(this, &FAvaPlaybackClient::HandlePlaybackStatusMessage)
		.Handling<FAvaPlaybackStatuses>(this, &FAvaPlaybackClient::HandlePlaybackStatusesMessage)
		.Handling<FAvaPlaybackSequenceEvent>(this, &FAvaPlaybackClient::HandlePlaybackSequenceEventMessage)
		.Handling<FAvaPlaybackTransitionEvent>(this, &FAvaPlaybackClient::HandlePlaybackTransitionEventMessage)
		.Handling<FAvaPlaybackTransitionEventMulti>(this, &FAvaPlaybackClient::HandlePlaybackTransitionEventMultiMessage);
	
	if (MessageEndpoint.IsValid())
	{
		PingTickDelegate = FTickerDelegate::CreateRaw(this, &FAvaPlaybackClient::HandlePingTicker);
		PingTickDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(PingTickDelegate, UAvaMediaSettings::Get().PingInterval);
		FCoreDelegates::OnEndFrame.AddRaw(this, &FAvaPlaybackClient::Tick);
		
		UE_LOGF(LogAvaPlaybackClient, Log, "Motion Design Playback Client \"%ls\" Started.", *ComputerName);
	}

	BroadcastChangedHandle = UAvaBroadcast::Get().AddChangeListener(
		FOnAvaBroadcastChanged::FDelegate::CreateRaw(this, &FAvaPlaybackClient::OnBroadcastChanged));

	FAvaBroadcastOutputChannel::GetOnChannelChanged().AddRaw(this, &FAvaPlaybackClient::OnChannelChanged);
	
#if WITH_EDITOR
	UAvaMediaSettings& AvaMediaSettings = UAvaMediaSettings::GetMutable();
	AvaMediaSettings.OnSettingChanged().AddRaw(this, &FAvaPlaybackClient::OnAvaMediaSettingsChanged);
#endif

	ApplyAvaMediaSettings();
}

int32 FAvaPlaybackClient::GetNumConnectedServers() const
{
	return Servers.Num();
}

TArray<FString> FAvaPlaybackClient::GetServerNames() const
{
	TArray<FString> ServerNames;
	ServerNames.Empty(Servers.Num());
	ForAllServers([&ServerNames](const FServerInfo& InServerInfo)
	{
		ServerNames.Add(InServerInfo.ServerName);
	});
	return ServerNames;
}

FMessageAddress FAvaPlaybackClient::GetServerAddress(const FString& InServerName) const
{
	if (const FServerInfo* ServerInfo = GetServerInfo(InServerName))
	{
		return ServerInfo->Address;
	}
	FMessageAddress InvalidAddress;
	InvalidAddress.Invalidate();
	return InvalidAddress;
}

FString FAvaPlaybackClient::GetServerProjectContentPath(const FString& InServerName) const
{
	if (const FServerInfo* ServerInfo = GetServerInfo(InServerName))
	{
		return ServerInfo->ProjectContentPath;
	}
	return FString();
}

uint32 FAvaPlaybackClient::GetServerProcessId(const FString& InServerName) const
{
	if (const FServerInfo* ServerInfo = GetServerInfo(InServerName))
	{
		return ServerInfo->ProcessId;
	}
	return 0;
}

bool FAvaPlaybackClient::HasServerUserData(const FString& InServerName, const FString& InKey) const
{
	if (const FServerInfo* ServerInfo = GetServerInfo(InServerName))
	{
		return ServerInfo->UserDataEntries.Contains(InKey);
	}
	return false;
}

const FString& FAvaPlaybackClient::GetServerUserData(const FString& InServerName, const FString& InKey) const
{
	if (const FServerInfo* ServerInfo = GetServerInfo(InServerName))
	{
		if (const FString* Data = ServerInfo->UserDataEntries.Find(InKey))
		{
			return *Data;
		}
	}
	static FString EmptyData;
	return EmptyData;
}

bool FAvaPlaybackClient::HasUserData(const FString& InKey) const
{
	return UserDataEntries.Contains(InKey);
}

const FString& FAvaPlaybackClient::GetUserData(const FString& InKey) const
{
	if (const FString* Data = UserDataEntries.Find(InKey))
	{
		return *Data;
	}
	static FString EmptyString;
	return EmptyString;
}

void FAvaPlaybackClient::SetUserData(const FString& InKey, const FString& InData)
{
	UserDataEntries.Add(InKey, InData);
	SendUserDataUpdate(AllServerAddresses);
}

void FAvaPlaybackClient::RemoveUserData(const FString& InKey)
{
	UserDataEntries.Remove(InKey);
	SendUserDataUpdate(AllServerAddresses);
}

void FAvaPlaybackClient::BroadcastStatCommand(const FString& InCommand, bool bInBroadcastLocalState)
{
	SendStatCommand(InCommand, bInBroadcastLocalState, AllServerAddresses);
}

void FAvaPlaybackClient::RequestPlaybackAssetStatus(const FSoftObjectPath& InAssetPath, const FString& InChannelOrServerName, bool bInForceRefresh)
{
	if (!InAssetPath.IsValid())
	{
		UE_LOGF(LogAvaPlaybackClient, Warning, "Invalid asset path for Playback Asset Status Request.");
		return;
	}
	
	TArray<FString> ServerNames = Servers.Contains(InChannelOrServerName) ? TArray<FString>{InChannelOrServerName} : GetServerNamesForChannel(FName(InChannelOrServerName));
	
	for (const FString& ServerName : ServerNames)
	{
		RequestPlaybackAssetStatusForServer(InAssetPath, ServerName, bInForceRefresh);
	}
}

void FAvaPlaybackClient::RequestPlayback(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName,
	EAvaPlaybackAction InAction, const FString& InArguments)
{
	using namespace UE::AvaPlayback::Utils;
	using namespace UE::AvaPlaybackClient::Private;

	if (InAssetPath.IsNull() && !IsPlaybackActionAllowedForNullAsset(InAction))
	{
		UE_LOGF(LogAvaPlaybackClient, Error,
		   "Playback request [%ls] on channel \"%ls\" with no asset specified is not allowed.",
		   *GetCommandActionString(InAction, InArguments), *InChannelName);
		return;
	}
	
	// Possibly TMP. Maybe find a generic way to deal with pending requests.
	// Status requests can be spammed every frame by the UI so we block them
	// here to avoid spamming the message bus and servers.
	if (InAction == EAvaPlaybackAction::Status)
	{
		const FDateTime CurrentTime = FDateTime::UtcNow();
		const FString RequestKey = GetPlaybackStatusRequestKey(InInstanceId, InAssetPath, InChannelName);
		const FDateTime* PendingRequest = PendingPlaybackStatusRequests.Find(RequestKey);
		// Check if we already have a pending request and that it hasn't expired.
		if (PendingRequest && CurrentTime < *PendingRequest)
		{
			return;
		}

		// Keep track of the request and the time it was made so we have a timeout.
		const FDateTime ExpirationTime = CurrentTime + FTimespan::FromSeconds(UAvaMediaSettings::Get().ClientPendingStatusRequestTimeout);
		PendingPlaybackStatusRequests.Add(RequestKey, ExpirationTime);
	}

	// Special case where the command is sent on all the assets and channels on all servers.
	if (InChannelName.IsEmpty() || InAssetPath.IsNull())
	{
		FAvaPlaybackRequest* Request = FMessageEndpoint::MakeMessage<FAvaPlaybackRequest>();
		Request->Commands.Reserve(1);
		Request->Commands.Add({InInstanceId, InAssetPath, InChannelName, InAction, InArguments});
		SendRequest(Request, AllServerAddresses);
		return;
	}
	
	if (InAssetPath.IsValid())
	{
		// Update the status of the playback for each server that we made a request to.
		TArray<FString> ServerNames = GetServerNamesForChannel(FName(InChannelName));
		for (const FString& ServerName : ServerNames)
		{
			if (FServerInfo* ServerInfo = GetServerInfo(ServerName))
			{
				// The command will be sent in a batch on the next Tick.
				ServerInfo->PendingPlaybackCommands.Add({InInstanceId, InAssetPath, InChannelName, InAction, InArguments});

				// Todo: Ideally, don't override the remote status. Need another way to know if a request has been issued.
				switch (InAction)
				{
				case EAvaPlaybackAction::None:
				case EAvaPlaybackAction::Status:
				case EAvaPlaybackAction::GetUserData:
				case EAvaPlaybackAction::SetUserData:
					break;
				case EAvaPlaybackAction::Load:
					ServerInfo->SetInstanceStatus(InChannelName, InInstanceId, InAssetPath, EAvaPlaybackStatus::Loading);
					break;
				case EAvaPlaybackAction::Start:
					ServerInfo->SetInstanceStatus(InChannelName, InInstanceId, InAssetPath, EAvaPlaybackStatus::Starting);
					break;
				case EAvaPlaybackAction::Stop:
					ServerInfo->SetInstanceStatus(InChannelName, InInstanceId, InAssetPath, EAvaPlaybackStatus::Stopping);
					break;
				case EAvaPlaybackAction::Unload:
					ServerInfo->SetInstanceStatus(InChannelName, InInstanceId, InAssetPath, EAvaPlaybackStatus::Unloading);
					break;
				}
			}
		}
	}
	else
	{
		// We will wait for feedback from server, if it managed to play the asset correctly.
		UE_LOGF(LogAvaPlaybackClient, Warning,
			   "Playback request \"%ls\" on channel \"%ls\" with no asset specified. Unable to update playback status.",
			   *StaticEnumToString(InAction), *InChannelName);
	}
}

void FAvaPlaybackClient::RequestAnimPlayback(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName,
												  const FAvaPlaybackAnimPlaySettings& InAnimSettings)
{
	FAvaPlaybackAnimPlaybackRequest* Request = FMessageEndpoint::MakeMessage<FAvaPlaybackAnimPlaybackRequest>();
	Request->InstanceId = InInstanceId;
	Request->AssetPath = InAssetPath;
	Request->ChannelName = InChannelName;
	Request->AnimPlaySettings.Add(InAnimSettings);

	if (!InAssetPath.IsValid())
	{
		UE_LOGF(LogAvaPlaybackClient, Warning, "Animation \"%ls\" request with invalid asset path on channel \"%ls\".",
			*InAnimSettings.AnimationName.ToString(), *InChannelName);
	}
		
	SendRequest(Request, GetServerAddressesForChannel(InChannelName));
}

void FAvaPlaybackClient::RequestAnimAction(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName,
												  const FString& InAnimationName, EAvaPlaybackAnimAction InAction)
{
	using namespace UE::AvaPlayback::Utils;
	FAvaPlaybackAnimPlaybackRequest* Request = FMessageEndpoint::MakeMessage<FAvaPlaybackAnimPlaybackRequest>();
	Request->InstanceId = InInstanceId;
	Request->AssetPath = InAssetPath;
	Request->ChannelName = InChannelName;
	Request->AnimActionInfos.Add({InAnimationName, InAction});

	if (!InAssetPath.IsValid())
	{
		UE_LOGF(LogAvaPlaybackClient, Warning, "Animation \"%ls\" %ls request with invalid asset path on channel \"%ls\".",
			*InAnimationName, *StaticEnumToString(InAction), *InChannelName);
	}
	
	SendRequest(Request, GetServerAddressesForChannel(InChannelName));
}

void FAvaPlaybackClient::RequestRemoteControlUpdate(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath,
														 const FString& InChannelName,
														 const FAvaPlayableRemoteControlValues& InRemoteControlValues,
														 EAvaPlayableRCUpdateFlags InFlags)
{
	FAvaPlaybackRemoteControlUpdateRequest* Request = FMessageEndpoint::MakeMessage<FAvaPlaybackRemoteControlUpdateRequest>();
	Request->InstanceId = InInstanceId;
	Request->AssetPath = InAssetPath;
	Request->ChannelName = InChannelName;
	Request->RemoteControlValues = InRemoteControlValues;
	Request->UpdateFlags = InFlags;

	if (!InAssetPath.IsValid())
	{
		UE_LOGF(LogAvaPlaybackClient, Warning, "Remote Control Values Update request with invalid asset path on channel \"%ls\".", *InChannelName);
	}
	
	SendRequest(Request, GetServerAddressesForChannel(InChannelName));
}

void FAvaPlaybackClient::RequestPlayableTransitionStart(const FGuid& InTransitionId, TArray<FGuid>&& InEnterInstanceIds, TArray<FGuid>&& InPlayingInstanceIds, TArray<FGuid>&& InExitInstanceIds, TArray<FAvaPlayableRemoteControlValues>&& InEnterValues, const FName& InChannelName, EAvaPlayableTransitionFlags InTransitionFlags)
{
	FAvaPlaybackTransitionStartRequest* Request = FMessageEndpoint::MakeMessage<FAvaPlaybackTransitionStartRequest>();
	Request->TransitionId = InTransitionId;
	Request->ChannelName = InChannelName.ToString();
	Request->EnterInstanceIds = MoveTemp(InEnterInstanceIds);
	Request->PlayingInstanceIds = MoveTemp(InPlayingInstanceIds);
	Request->ExitInstanceIds = MoveTemp(InExitInstanceIds);
	Request->EnterValues = MoveTemp(InEnterValues);
	Request->bUnloadDiscardedInstances = !UAvaMediaSettings::Get().bKeepPagesLoaded;
	Request->SetTransitionFlags(InTransitionFlags);
	SendRequest(Request, GetServerAddressesForChannel(InChannelName));
}

void FAvaPlaybackClient::RequestPlayableTransitionStop(const FGuid& InTransitionId, const FName& InChannelName)
{
	FAvaPlaybackTransitionStopRequest* Request = FMessageEndpoint::MakeMessage<FAvaPlaybackTransitionStopRequest>();
	Request->TransitionId = InTransitionId;
	Request->ChannelName = InChannelName.ToString();
	Request->bUnloadDiscardedInstances = !UAvaMediaSettings::Get().bKeepPagesLoaded;
	SendRequest(Request, GetServerAddressesForChannel(InChannelName));
}

void FAvaPlaybackClient::RequestBroadcast(const FString& InProfile, const FName& InChannel,
											   const TArray<UMediaOutput*>& InRemoteMediaOutputs,
											   EAvaBroadcastAction InAction, const FString& InServerName)
{
	FAvaBroadcastRequest* Request = FMessageEndpoint::MakeMessage<FAvaBroadcastRequest>();
	Request->Profile = InProfile;
	Request->Channel = InChannel.ToString();
	Request->Action = InAction;

	bool bNeedMediaOutputs = (InAction == EAvaBroadcastAction::Start || InAction == EAvaBroadcastAction::UpdateConfig);
	if (Request->Channel.IsEmpty())
	{
		// If no channel is specified, no need to send outputs. 
		bNeedMediaOutputs = false;
		
		if (InAction == EAvaBroadcastAction::UpdateConfig)
		{
			UE_LOGF(LogAvaPlaybackClient, Error, "Invalid Broadcast Request: UpdateConfig requires a channel to be specified.");
			return;
		}
	}
	
	if (bNeedMediaOutputs)
	{
		// For now, we send the media output objects in the request.
		uint32 TotalOutputDataSize = 0;
		for (UMediaOutput* const MediaOutput : InRemoteMediaOutputs)
		{
			if (IsValid(MediaOutput))
			{
				FAvaBroadcastOutputData MediaOutputData = UE::AvaBroadcastOutputUtils::CreateMediaOutputData(MediaOutput);
				
				// Also propagate output info. Necessary to send the Guid and Server.
				MediaOutputData.OutputInfo
					= UAvaBroadcast::Get().GetCurrentProfile().GetChannel(InChannel).GetMediaOutputInfo(MediaOutput);
				
				if (!MediaOutputData.OutputInfo.IsValid())
				{
					UE_LOGF(LogAvaPlaybackClient, Warning, "MediaOutput information is not valid for channel \"%ls\".",
						   *InChannel.ToString());
				}
				TotalOutputDataSize += MediaOutputData.SerializedData.Num();
				Request->MediaOutputs.Add(MoveTemp(MediaOutputData));
			}
		}
		
		// Adding a warning here, if we hit this warning, it may be necessary
		// to send the data through some other transport.
		// Data size is about 1k per output. It is unlikely we will hit this limit.
		const uint32 SafeMessageSizeLimit = UE::AvaMediaMessageUtils::GetSafeMessageSizeLimit();	
		if (TotalOutputDataSize > SafeMessageSizeLimit)
		{
			UE_LOGF(LogAvaPlaybackClient, Warning,
				"The broadcast request (DataSize: %d) is larger that the safe message size limit (%d).",
				TotalOutputDataSize, SafeMessageSizeLimit);
		}
	}

	const TArray<FMessageAddress> ServerAddresses = InServerName.IsEmpty() ?
		GetServerAddressesForChannel(InChannel) : TArray<FMessageAddress>({ GetServerAddress(InServerName) });

	// Also update the channel settings.
	if (InAction == EAvaBroadcastAction::Start || InAction == EAvaBroadcastAction::UpdateConfig)
	{
		SendBroadcastChannelSettingsUpdate(ServerAddresses, UAvaBroadcast::Get().GetCurrentProfile().GetChannel(InChannel));
	}

	// Send to server(s) that have output for this channel.
	SendRequest(Request, ServerAddresses);
}

bool FAvaPlaybackClient::IsMediaOutputRemoteFallback(const UMediaOutput* InMediaOutput)
{
	const FString DeviceName = UE::AvaBroadcastOutputUtils::GetDeviceName(InMediaOutput);
	if (!DeviceName.IsEmpty())
	{
		// See if it begins with any of the remote server names that are connected.
		// See FAvaBroadcastDeviceProviderData::ApplyServerName. All MediaOutput from replicated Device Provider
		// Have the server name in the beginning of the device name.
		for (const TPair<FString, TSharedPtr<FServerInfo>>& Server : Servers)
		{
			if (DeviceName.StartsWith(Server.Key))
			{
				return true;
			}
		}
	}

	// We may fall in this case if the device is from a remote server that is offline
	// right now. So we still want to double check if the device is a local one.

	// Search in the list of device providers for the local server only.
	const FName DeviceProviderName = UE::AvaBroadcastOutputUtils::GetDeviceProviderName(InMediaOutput);
	if (!DeviceProviderName.IsNone() && !DeviceName.IsEmpty())
	{
		// First check if we have a wrapper for this provider. This means
		// we have some remote devices currently online on that provider.
		const FAvaBroadcastDeviceProviderWrapper* DeviceProviderWrapper =
			ParentModule->GetDeviceProviderProxyManager().GetDeviceProviderWrapper(DeviceProviderName);

		IMediaIOCoreDeviceProvider* LocalDeviceProvider;

		if (DeviceProviderWrapper && DeviceProviderWrapper->HasLocalProvider())
		{
			LocalDeviceProvider = DeviceProviderWrapper->GetLocalProvider();
		}
		else
		{
			// If we don't have a wrapper, then directly fetch the concrete local device provider.
			LocalDeviceProvider = IMediaIOCoreModule::Get().GetDeviceProvider(DeviceProviderName);
		}

		if (LocalDeviceProvider)
		{
			const FName DeviceFName(DeviceName);
			TArray<FMediaIODevice> LocalDevices = LocalDeviceProvider->GetDevices();
			for (const FMediaIODevice& LocalDevice : LocalDevices)
			{
				if (DeviceFName == LocalDevice.DeviceName)
				{
					return false; // found as local device, so not remote.
				}
			}
			// If we didn't find the device in local provider (and it has a local provider),
			// then we can safely consider it a remote device, but it's server is currently offline.
			return true;
		}
	}

	// By default the device will be local. But this is not a good default.
	return false;
}

EAvaBroadcastIssueSeverity FAvaPlaybackClient::GetMediaOutputIssueSeverity(
	const FString& InServerName, const FString& InChannelName, const FGuid& InOutputGuid) const
{
	if (const FBroadcastChannelInfo* ChannelInfo = GetChannelInfo(InServerName, InChannelName))
	{
		if (const FAvaBroadcastOutputStatus* MediaOutputStatus = ChannelInfo->MediaOutputStatuses.Find(InOutputGuid))
		{
			return MediaOutputStatus->MediaIssueSeverity;
		}
	}
	return EAvaBroadcastIssueSeverity::None;
}

const TArray<FString>& FAvaPlaybackClient::GetMediaOutputIssueMessages(
	const FString& InServerName, const FString& InChannelName, const FGuid& InOutputGuid) const
{
	if (const FBroadcastChannelInfo* ChannelInfo = GetChannelInfo(InServerName, InChannelName))
	{
		if (const FAvaBroadcastOutputStatus* MediaOutputStatus = ChannelInfo->MediaOutputStatuses.Find(InOutputGuid))
		{
			return MediaOutputStatus->MediaIssueMessages;
		}
	}
	static const TArray<FString> EmptyStringArray;
	return EmptyStringArray;
}

EAvaBroadcastOutputState FAvaPlaybackClient::GetMediaOutputState(
	const FString& InServerName, const FString& InChannelName, const FGuid& InOutputGuid) const
{
	if (const FBroadcastChannelInfo* ChannelInfo = GetChannelInfo(InServerName, InChannelName))
	{
		if (const FAvaBroadcastOutputStatus* MediaOutputStatus = ChannelInfo->MediaOutputStatuses.Find(InOutputGuid))
		{
			return MediaOutputStatus->MediaOutputState;
		}
		else
		{
			// If the server is connected but doesn't have a status yet, we consider it "idle".
			return EAvaBroadcastOutputState::Idle;
		}
	}
	// If we couldn't find the channel state in the list of servers,
	// then consider this output as offline.
	return EAvaBroadcastOutputState::Offline;
}

// Note: same logic as GetServerNamesForChannel.
bool FAvaPlaybackClient::HasAnyServerOnlineForChannel(const FName& InChannelName) const
{
	if (InChannelName.IsNone())
	{
		return false;
	}
	
	const FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetChannel(InChannelName);
	if (!Channel.IsValidChannel())
	{
		return false;
	}

	const TArray<UMediaOutput*>& RemoteOutputs = Channel.GetRemoteMediaOutputs();
	
	for (const UMediaOutput* RemoteOutput : RemoteOutputs)
	{
		const FAvaBroadcastMediaOutputInfo& OutputInfo = Channel.GetMediaOutputInfo(RemoteOutput);
		if (OutputInfo.IsValid())
		{
			if (Servers.Contains(OutputInfo.ServerName))
			{
				return true;
			}
		}
		else
		{
			UE_LOGF(LogAvaPlaybackClient, Warning, "MediaOutputInfo invalid for channel \"%ls\".", *InChannelName.ToString());
			
			// Try to find the server name from the device name.
			const FString ServerName = GetServerNameForMediaOutputFallback(RemoteOutput);
			if (!ServerName.IsEmpty())
			{
				return true;
			}
		}
	}
	
	return false;
}

TArray<FString> FAvaPlaybackClient::GetOnlineServersForChannel(const FName& InChannelName) const
{
	return GetServerNamesForChannel(InChannelName, /*bInOnlineOnly*/ true);
}

TOptional<EAvaPlaybackStatus> FAvaPlaybackClient::GetRemotePlaybackStatus(
	const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FString& InServerName) const
{
	TOptional<EAvaPlaybackStatus> PlaybackStatus;

	if (!InServerName.IsEmpty())
	{
		PlaybackStatus = GetPlaybackStatusForServer(InServerName, InChannelName, InInstanceId, InAssetPath);
	}
	else
	{
		// Because of "forked" channels, we could actually have more than one server playing this channel.
		TArray<FString> ServerNames = GetServerNamesForChannel(FName(InChannelName));

		// In this case, we return the first one found. It is not accurate though.
		// Todo: Ideally, we should retire this code path.
		for (const FString& ServerName : ServerNames)
		{
			PlaybackStatus = GetPlaybackStatusForServer(ServerName, InChannelName, InInstanceId, InAssetPath);
			if (PlaybackStatus.IsSet())
			{
				break;
			}
		}
	}

	return PlaybackStatus;
}

const FString* FAvaPlaybackClient::GetRemotePlaybackUserData(
	const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FString& InServerName) const
{
	if (!InServerName.IsEmpty())
	{
		return GetPlaybackUserDataForServer(InServerName, InChannelName, InInstanceId, InAssetPath);
	}

	// Because of "forked" channels, we could actually have more than one server playing this channel.
	TArray<FString> ServerNames = GetServerNamesForChannel(FName(InChannelName));

	// In this case, we return the first one found. It is not accurate though.
	// Todo: Ideally, we should retire this code path.
	for (const FString& ServerName : ServerNames)
	{
		if (const FString* FoundUserData = GetPlaybackUserDataForServer(ServerName, InChannelName, InInstanceId, InAssetPath))
		{
			return FoundUserData;
		}
	}

	return nullptr;
}

TOptional<EAvaPlaybackAssetStatus> FAvaPlaybackClient::GetRemotePlaybackAssetStatus(
	const FSoftObjectPath& InAssetPath, const FString& InServerName) const
{
	if (const FServerInfo* ServerInfo = GetServerInfo(InServerName))
	{
		return ServerInfo->GetPlaybackAssetStatus(InAssetPath);
	}
	return TOptional<EAvaPlaybackAssetStatus>();
}

TOptional<EAvaPlaybackStatus> FAvaPlaybackClient::GetPlaybackStatusForServer(
	const FString& InServerName, const FString& InChannelName, const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath) const
{
	if (const FServerInfo* ServerInfo = GetServerInfo(InServerName))
	{
		return ServerInfo->GetInstanceStatus(InChannelName, InInstanceId, InAssetPath);
	}
	return TOptional<EAvaPlaybackStatus>();
}

const FString* FAvaPlaybackClient::GetPlaybackUserDataForServer(
	const FString& InServerName, const FString& InChannelName, const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath) const
{
	if (const FServerInfo* ServerInfo = GetServerInfo(InServerName))
	{
		return ServerInfo->GetInstanceUserData(InChannelName, InInstanceId, InAssetPath);
	}
	return nullptr;
}

void FAvaPlaybackClient::RequestPlaybackAssetStatusForServer(const FSoftObjectPath& InAssetPath, const FString& InServerName, bool bInForceRefresh)
{
	FServerInfo* ServerInfo = GetServerInfo(InServerName);
	if (!ServerInfo)
	{
		UE_LOGF(LogAvaPlaybackClient, Warning, "Specified server \"%ls\" not found for Playback Asset Status Request.", *InServerName);
		return;
	}
	
	// Status requests can be spammed every frame by the UI so we block them
	// here to avoid spamming the message bus and servers.
	const FDateTime CurrentTime = FDateTime::UtcNow();
	if (const FPendingPlaybackAssetStatusRequest* PendingRequest = ServerInfo->PendingPlaybackAssetStatusRequests.Find(InAssetPath))
	{
		// Check if the current request hasn't expired.
		if (CurrentTime < PendingRequest->ExpirationTime)
		{
			// A "force refresh" request will override a non-"force refresh" one.
			if (PendingRequest->bForceRefresh >= bInForceRefresh)
			{
				return;
			}
		}
	}

	// Keep track of the request and the time it was made so we have a timeout.
	const FDateTime ExpirationTime = CurrentTime + FTimespan::FromSeconds(UAvaMediaSettings::Get().ClientPendingStatusRequestTimeout);
	ServerInfo->PendingPlaybackAssetStatusRequests.Add(InAssetPath, {ExpirationTime, bInForceRefresh} );

	FAvaPlaybackAssetStatusRequest* Request = FMessageEndpoint::MakeMessage<FAvaPlaybackAssetStatusRequest>();
	Request->AssetPath = InAssetPath;
	Request->bForceRefresh = bInForceRefresh;
	SendRequest(Request, ServerInfo->Address);
}

void FAvaPlaybackClient::Tick()
{
	for (const TPair<FString, TSharedPtr<FServerInfo>>& Server : Servers)
	{
		if (!Server.Value->PendingPlaybackCommands.IsEmpty())
		{
			// Batched per server for now.
			FAvaPlaybackRequest* Request = FMessageEndpoint::MakeMessage<FAvaPlaybackRequest>();
			Request->Commands = MoveTemp(Server.Value->PendingPlaybackCommands);
			Server.Value->PendingPlaybackCommands.Reset();
			SendRequest(Request, Server.Value->Address);
		}
	}
}

bool FAvaPlaybackClient::HandlePingTicker(float InDeltaTime)
{
	const FDateTime CurrentTime = FDateTime::UtcNow();
	PublishPlaybackPing(CurrentTime, true);
	RemoveDeadServers(CurrentTime);
	return true;
}

void FAvaPlaybackClient::HandlePlaybackPongMessage(const FAvaPlaybackPong& InMessage,
														const TSharedRef<IMessageContext, ESPMode::ThreadSafe>&	InContext)
{
	// Remark: the server will send a user data update before sending the pong message.
	// The server info will have been created already by this point. But it won't have all the information
	// complete yet.
	
	FServerInfo& ServerInfo = GetOrCreateServerInfo(InMessage.ServerName, InContext->GetSender());
	ServerInfo.ResetPingTimeout();

	// If the server info is new, the process id will not be set yet.
	const bool bIsNewServer = ServerInfo.ProcessId == 0 ? true : false;
	
	if (!bIsNewServer && ServerInfo.ProcessId != InMessage.ProcessId)
	{
		UE_LOGF(LogAvaPlaybackClient, Warning, "Received server \"%ls\" process Id has changed from %d to %d.",
			*InMessage.ServerName, ServerInfo.ProcessId, InMessage.ProcessId);
	}
	
	ServerInfo.ProcessId = InMessage.ProcessId;
	ServerInfo.ProjectContentPath = InMessage.ProjectContentPath;
	
	// The server may have requested the client info, and it has to be sent unless
	// this is a new server, in which case it has already been sent by OnServerAdded().
	if (InMessage.bRequestClientInfo && !bIsNewServer)
	{
		SendClientInfo(InContext->GetSender());
	}

	// We want to propagate the connection event after all the information has been set
	// in the server info.
	if (bIsNewServer)
	{
		using namespace UE::AvaPlaybackClient::Delegates;
		GetOnConnectionEvent().Broadcast(*this, {InMessage.ServerName, EConnectionEvent::ServerConnected});
	}
}

void FAvaPlaybackClient::HandlePlaybackLogMessage(const FAvaPlaybackLog& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	if (GLog && InMessage.Verbosity != 0)
	{
		// Basic message formatting: adding the server name to identify the origin of the message.
		const FString FormattedText = InMessage.ServerName + TEXT(" >> ") + InMessage.Text;

		// Relay to local GLog.
		GLog->Serialize(*FormattedText, static_cast<ELogVerbosity::Type>(InMessage.Verbosity), InMessage.Category, InMessage.Time);
	}
}

void FAvaPlaybackClient::HandleUpdateServerUserData(const FAvaPlaybackUpdateServerUserData& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	FServerInfo& ServerInfo = GetOrCreateServerInfo(InMessage.ServerName, InContext->GetSender());
	ServerInfo.ResetPingTimeout();
	ServerInfo.UserDataEntries = InMessage.UserDataEntries;

	// Logging when user data is updated (for debugging).
	UE_LOGF(LogAvaPlaybackClient, Verbose, "Received new user data for server \"%ls\".", *InMessage.ServerName);
	for (const TPair<FString, FString>& UserData : ServerInfo.UserDataEntries)
	{
		UE_LOGF(LogAvaPlaybackClient, Verbose, "User data \"%ls\":\"%ls\".", *UserData.Key, *UserData.Value);
	}
}

void FAvaPlaybackClient::HandleStatStatus(const FAvaPlaybackStatStatus& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	// If the client couldn't run the command, but the serve did, we are going
	// to replicate the stats state on the client instead.
	if (InMessage.bClientStateReliable == false && InMessage.bCommandSucceeded)
	{
		UE_LOGF(LogAvaPlaybackClient, Verbose,
			"Received reliable enabled runtime stats from server \"%ls\".", *InMessage.ServerName);
		IAvaModule::Get().OverwriteEnabledRuntimeStats(InMessage.EnabledRuntimeStats);
	}
}

void FAvaPlaybackClient::HandleDeviceProviderDataListMessage(const FAvaBroadcastDeviceProviderDataList& InMessage,
																  const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UE_LOGF(LogAvaPlaybackClient, Verbose, "Received Device Provider Data for server \"%ls\".",
		   *InMessage.ServerName);
	IAvaBroadcastDeviceProviderProxyManager& Manager = ParentModule->GetDeviceProviderProxyManager();
	Manager.Install(InMessage.ServerName, InMessage);

	{
		FServerInfo& ServerInfo = GetOrCreateServerInfo(InMessage.ServerName, InContext->GetSender());
		ServerInfo.bDeviceProviderInstalled = true;
	}

	// Check if we have installed all the connected server's device providers.
	bool bAllDeviceProvidersInstalled = true;
	{
		ForAllServers([&bAllDeviceProvidersInstalled](const FServerInfo& InServerInfo)
		{
			if (InServerInfo.bDeviceProviderInstalled == false)
			{
				UE_LOGF(LogAvaPlaybackClient, Verbose,
					   "Device Provider Data for server \"%ls\" hasn't been received yet.", *InServerInfo.ServerName);
				bAllDeviceProvidersInstalled = false;
			}
		});
	}

	if (bAllDeviceProvidersInstalled)
	{
		UE_LOGF(LogAvaPlaybackClient, Verbose,
			   "All Device Providers Proxies are installed. Requesting broadcast status update...");
		// Remark: We are waiting for all the device providers from all online servers to be
		// installed before requesting a broadcast status.
		// This is because handling the broadcast status update will load the broadcast
		// configuration. When the broadcast object is loaded, there is the potential for
		// legacy code to convert the config and it needs the device provider proxies to
		// be installed to resolve the device names and corresponding servers.
		ForAllServers([this](const FServerInfo& InServerInfo)
		{
			UE_LOGF(LogAvaPlaybackClient, Verbose, "Requesting full broadcast status update for server \"%ls\".",
				   *InServerInfo.ServerName);
			// Also request broadcast channels status to update state of local channels if required.
			FAvaBroadcastStatusRequest* StatusRequest = FMessageEndpoint::MakeMessage<
				FAvaBroadcastStatusRequest>();
			StatusRequest->bIncludeMediaOutputData = true;
			// We will want to ensure our local config is the same as the servers.

			SendRequest(StatusRequest, InServerInfo.Address);
		});
	}
}

void FAvaPlaybackClient::HandleBroadcastStatusMessage(const FAvaBroadcastStatus& InMessage,
														   const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	using namespace UE::AvaPlayback::Utils;
	UE_LOGF(LogAvaPlaybackClient, Verbose,
		   "Received broadcast update from server \"%ls\" for channel \"%ls\": Status: \"%ls\".",
		   *InMessage.ServerName, *InMessage.ChannelName,
		   *StaticEnumToString(InMessage.ChannelState));

	// Log details for the output status.
	for (const TPair<FGuid, FAvaBroadcastOutputStatus>& OutputStatus : InMessage.MediaOutputStatuses)
	{
		UE_LOGF(LogAvaPlaybackClient, Verbose,
			   "Playback Server: \"%ls\" Channel: \"%ls\" OutputId \"%ls\" Status: \"%ls\" Severity: \"%ls\".",
			   *InMessage.ServerName, *InMessage.ChannelName, *OutputStatus.Key.ToString(),
			   *StaticEnumToString(OutputStatus.Value.MediaOutputState),
			   *StaticEnumToString(OutputStatus.Value.MediaIssueSeverity));
		
		for (const FString& Message : OutputStatus.Value.MediaIssueMessages)
		{
			UE_LOGF(LogAvaPlaybackClient, Verbose, "Output Issue Message: \"%ls\".", *Message);
		}
	}

	FServerInfo& ServerInfo = GetOrCreateServerInfo(InMessage.ServerName, InContext->GetSender());
	ServerInfo.ResetPingTimeout();

	const FString CurrentProfileName = UAvaBroadcast::Get().GetCurrentProfileName().ToString();

	{
		// Keep a back store of this locally, mainly so we can respond to UI requests
		// for the status of the media output objects.
		FBroadcastChannelInfo& ChannelInfo = ServerInfo.GetOrCreateBroadcastChannelInfo(InMessage.ChannelName);
		ChannelInfo.ChannelState = InMessage.ChannelState;
		ChannelInfo.ChannelIssueSeverity = InMessage.ChannelIssueSeverity;		
		ChannelInfo.MediaOutputStatuses = InMessage.MediaOutputStatuses;
	}

	{
		const FName ChannelName = *InMessage.ChannelName;
		FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetChannelMutable(ChannelName);
		if (Channel.IsValidChannel())
		{
			// Purpose of this section is to ensure the server and the client have
			// the same configuration. We want to perform this in the less intrusive
			// way possible, i.e. if the server is running and the client connects to it
			// we would like the server to be left completely undisturbed if it's configuration
			// is already synced.

			// Verify if the server has the corresponding outputs already.
			bool bServerIsMissingOutputs = false;
			int32 NumOutputsForThisServer = 0;
			TArray<UMediaOutput*> RemoteOutputs = Channel.GetRemoteMediaOutputs();
			for (const UMediaOutput* RemoteOutput : RemoteOutputs)
			{
				const FAvaBroadcastMediaOutputInfo& OutputInfo = Channel.GetMediaOutputInfo(RemoteOutput);
				if (OutputInfo.IsValid() && OutputInfo.ServerName == InMessage.ServerName)
				{
					++NumOutputsForThisServer;
					if (!InMessage.MediaOutputStatuses.Contains(OutputInfo.Guid))
					{
						bServerIsMissingOutputs = true;
					}
					else if (InMessage.bIncludeMediaOutputData)
					{
						// The server already has this output, but we need to check
						// if it is the same. TODO.
					}
				}
			}

			if ((InMessage.bIncludeMediaOutputData && NumOutputsForThisServer > 0 && InMessage.MediaOutputs.Num() == 0)
				|| bServerIsMissingOutputs)
			{
				UE_LOGF(LogAvaPlaybackClient, Log,
					   "Playback Server: \"%ls\" Channel: \"%ls\" is missing outputs. Requesting configuration update.",
					   *InMessage.ServerName, *InMessage.ChannelName);

				RequestBroadcast(CurrentProfileName, ChannelName, RemoteOutputs, EAvaBroadcastAction::UpdateConfig, InMessage.ServerName);
			}

			// Note: this will broadcast to delegates which may then request states of media outputs.
			// So the back store needs to be updated before calling this.
			Channel.RefreshState();

			// Explicitly call the broadcast of the channel state change to propagate the status of the output,
			// which is not fully reflected in the channel state.
			FAvaBroadcastOutputChannel::GetOnChannelChanged().Broadcast(Channel, EAvaBroadcastChannelChange::State);
		}
		else
		{
			UE_LOGF(LogAvaPlaybackClient, Error,
				   "Received broadcast update from server \"%ls\" for locally invalid channel \"%ls\".",
				   *InMessage.ServerName, *InMessage.ChannelName);

			// Request this channel be deleted.
			RequestBroadcast(CurrentProfileName, ChannelName, {}, EAvaBroadcastAction::DeleteChannel, InMessage.ServerName);
		}
	}

	// Check for missing channels (is this logic valid?)
	if (InMessage.ChannelIndex == InMessage.NumChannels - 1)
	{
		// We have received the last channel this server has, so we can check in the server state
		// if it has all the channels it is supposed to have.
		const TArray<FAvaBroadcastOutputChannel*>& Channels = UAvaBroadcast::Get().GetCurrentProfile().GetChannels();
		for (const FAvaBroadcastOutputChannel* Channel : Channels)
		{
			bool bHasOutputsForThisServer = false;
			TArray<UMediaOutput*> RemoteOutputs = Channel->GetRemoteMediaOutputs();
			for (const UMediaOutput* RemoteOutput : RemoteOutputs)
			{
				const FAvaBroadcastMediaOutputInfo& OutputInfo = Channel->GetMediaOutputInfo(RemoteOutput);
				if (OutputInfo.IsValid() && OutputInfo.ServerName == InMessage.ServerName)
				{
					bHasOutputsForThisServer = true;
					break;
				}
			}
			if (bHasOutputsForThisServer)
			{
				if (ServerInfo.GetBroadcastChannelInfo(Channel->GetChannelName().ToString()) == nullptr)
				{
					// The server is missing this channel, we need to update it's config.
					RequestBroadcast(CurrentProfileName, Channel->GetChannelName(), RemoteOutputs, EAvaBroadcastAction::UpdateConfig, InMessage.ServerName);
				}
			}
		}
	}
}

void FAvaPlaybackClient::HandlePlaybackAssetStatusMessage(
	const FAvaPlaybackAssetStatus& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	using namespace UE::AvaPlayback::Utils;
	using namespace UE::AvaPlaybackClient::Delegates;

	if (!InMessage.AssetPath.IsValid())
	{
		UE_LOGF(LogAvaPlaybackClient, Error,
		   "%ls Received Remote Playback Asset \"%ls\" from \"%ls\" Status: %ls. Invalid asset path, can't update status.",
		   *GetBriefFrameInfo(), *InMessage.AssetPath.GetAssetName(), *InMessage.ServerName, *StaticEnumToString(InMessage.Status));
		return;
	}
	
	FServerInfo& ServerInfo = GetOrCreateServerInfo(InMessage.ServerName, InContext->GetSender());
	ServerInfo.ResetPingTimeout();	
	ServerInfo.SetPlaybackAssetStatus(InMessage.AssetPath, InMessage.Status);
		
	// Clear the pending request (if any).
	ServerInfo.PendingPlaybackAssetStatusRequests.Remove(InMessage.AssetPath);

	UE_LOGF(LogAvaPlaybackClient, Verbose,
	   "%ls Received Remote Playback Asset \"%ls\" from \"%ls\" Status Changed: %ls.",
	   *GetBriefFrameInfo(), *InMessage.AssetPath.GetAssetName(), *InMessage.ServerName, *StaticEnumToString(InMessage.Status));
	
	GetOnPlaybackAssetStatusChanged().Broadcast(*this,{InMessage.AssetPath, InMessage.ServerName, InMessage.Status});

	// TODO - Further refactoring needed to untangle playback state and asset state. 
	// Because some of the states of the playback status reflect the state of the asset on disk,
	// we need to make sure the playback state properly reflects the asset state for those cases.
	using namespace UE::AvaPlaybackClient::Private;
	const bool bAssetIsMissing = IsAssetMissing(InMessage.Status);
	const bool bAssetIsAvailable = IsAssetAvailable(InMessage.Status);

	for (const TPair<FString, TUniquePtr<FPlaybackChannelInfo>>& ChannelInfo : ServerInfo.PlaybackChannelInfosByName)
	{
		check(ChannelInfo.Value.IsValid());
		if (FPlaybackAssetInfo* AssetInfo = ChannelInfo.Value->GetAssetInfo(InMessage.AssetPath))
		{
			auto SetPlaybackInstanceStatus = [this, AssetInfo, &InMessage, &ChannelInfo](const FGuid& InInstanceId, EAvaPlaybackStatus InNewStatus)
			{
				const TOptional<EAvaPlaybackStatus> PreviousStatusOpt = AssetInfo->GetInstanceStatus(InInstanceId);
				const EAvaPlaybackStatus PreviousStatus = PreviousStatusOpt.IsSet() ? PreviousStatusOpt.GetValue() : EAvaPlaybackStatus::Unknown;
				
				AssetInfo->SetInstanceStatus(InInstanceId, InNewStatus);

				UE_LOGF(LogAvaPlaybackClient, Verbose,
				   "%ls Triggered Remote Playback \"%ls\" (id:%ls) for \"%ls\" (%ls) Status Changed: %ls -> %ls because of asset status change.",
				   *GetBriefFrameInfo(), 
				   *InMessage.AssetPath.GetAssetName(), *InInstanceId.ToString(), *InMessage.ServerName, *ChannelInfo.Key,
				   *StaticEnumToString(PreviousStatus), *StaticEnumToString(InNewStatus));

				const FPlaybackStatusChangedArgs Args =
				{
					InInstanceId,
					InMessage.AssetPath,
					ChannelInfo.Key,
					InMessage.ServerName,
					PreviousStatus,
					InNewStatus
				};
				GetOnPlaybackStatusChanged().Broadcast(*this, Args);
			};

			for (TPair<FGuid, FPlaybackInstanceInfo>& InstanceInfo : AssetInfo->InstanceByIds)
			{
				// A missing asset leads to a missing playback.
				if (InstanceInfo.Value.Status == EAvaPlaybackStatus::Available && bAssetIsMissing)
				{
					SetPlaybackInstanceStatus(InstanceInfo.Key, EAvaPlaybackStatus::Missing);
				}
				// If the playback was missing, and the asset becomes available, update the playback to available too.
				else if (InstanceInfo.Value.Status == EAvaPlaybackStatus::Missing && bAssetIsAvailable)
				{
					SetPlaybackInstanceStatus(InstanceInfo.Key, EAvaPlaybackStatus::Available);
				}
			}
		}
	}
}

void FAvaPlaybackClient::HandlePlaybackStatus(FServerInfo& InServerInfo,
	const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName,
	EAvaPlaybackStatus InStatus, const FString& InUserData, bool bInValidUserData)
{
	using namespace UE::AvaPlayback::Utils;
	
	if (!InAssetPath.IsValid())
	{
		UE_LOGF(LogAvaPlaybackClient, Error,
		   "%ls Received Remote Playback \"%ls\" (id:%ls) from \"%ls\" (%ls) Status: %ls. Invalid asset path, can't update status.",
		   *GetBriefFrameInfo(), *InAssetPath.GetAssetName(), *InInstanceId.ToString(), *InServerInfo.ServerName, *InChannelName, *StaticEnumToString(InStatus));
		return;
	}

	TOptional<EAvaPlaybackStatus> PrevPlaybackStatus = InServerInfo.GetInstanceStatus(InChannelName, InInstanceId, InAssetPath);
	if (!PrevPlaybackStatus.IsSet())
	{
		PrevPlaybackStatus = EAvaPlaybackStatus::Unknown;
	}

	if (InStatus == EAvaPlaybackStatus::Available && InInstanceId.IsValid())
	{
		// We need to clear the playback status cache when an actual instance returns to "available".
		// This prevents the cache from accumulating an ever-growing list of stale instance statuses.
		InServerInfo.RemoveInstanceInfo(InChannelName, InInstanceId, InAssetPath);
	}
	else
	{
		InServerInfo.SetInstanceStatus(InChannelName, InInstanceId, InAssetPath, InStatus);
	
		if (bInValidUserData)
		{
			InServerInfo.SetInstanceUserData(InChannelName, InInstanceId, InAssetPath, InUserData);

			UE_LOGF(LogAvaPlaybackClient, Verbose,
			   "%ls Received Remote Playback \"%ls\" (id:%ls) from \"%ls\" (%ls) User Data: %ls.",
			   *GetBriefFrameInfo(), *InAssetPath.GetAssetName(), *InInstanceId.ToString(), *InServerInfo.ServerName, *InChannelName, *InUserData);
		}
	}

	// Clear the pending request (if any).
	PendingPlaybackStatusRequests.Remove(GetPlaybackStatusRequestKey(InInstanceId, InAssetPath, InChannelName));

	UE_LOGF(LogAvaPlaybackClient, Verbose,
		   "%ls Received Remote Playback \"%ls\" (id:%ls) from \"%ls\" (%ls) Status Change: %ls -> %ls.",
		   *GetBriefFrameInfo(), 
		   *InAssetPath.GetAssetName(), *InInstanceId.ToString(), *InServerInfo.ServerName, *InChannelName,
		   *StaticEnumToString(PrevPlaybackStatus.GetValue()), *StaticEnumToString(InStatus));

	using namespace UE::AvaPlaybackClient::Delegates;
	const FPlaybackStatusChangedArgs Args =
	{
		InInstanceId,
		InAssetPath,
		InChannelName,
		InServerInfo.ServerName,
		PrevPlaybackStatus.GetValue(),
		InStatus
	};	
	GetOnPlaybackStatusChanged().Broadcast(*this, Args);
}

void FAvaPlaybackClient::HandlePlaybackStatusMessage(const FAvaPlaybackStatus& InMessage,
														  const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	FServerInfo& ServerInfo = GetOrCreateServerInfo(InMessage.ServerName, InContext->GetSender());
	ServerInfo.ResetPingTimeout();

	HandlePlaybackStatus(ServerInfo, InMessage.InstanceId, InMessage.AssetPath, InMessage.ChannelName, InMessage.Status, InMessage.UserData, InMessage.bValidUserData);
}

void FAvaPlaybackClient::HandlePlaybackStatusesMessage(const FAvaPlaybackStatuses& InMessage,
															const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	FServerInfo& ServerInfo = GetOrCreateServerInfo(InMessage.ServerName, InContext->GetSender());
	ServerInfo.ResetPingTimeout();

	const int32 NumStatuses = FMath::Max(InMessage.AssetPaths.Num(), InMessage.InstanceIds.Num());
	for (int32 StatusIndex = 0; StatusIndex < NumStatuses; StatusIndex++)
	{
		const FSoftObjectPath AssetPath = InMessage.AssetPaths.IsValidIndex(StatusIndex) ? InMessage.AssetPaths[StatusIndex] : FSoftObjectPath();
		const FGuid InstanceId = InMessage.InstanceIds.IsValidIndex(StatusIndex) ? InMessage.InstanceIds[StatusIndex] : FGuid();

		HandlePlaybackStatus(ServerInfo, InstanceId, AssetPath, InMessage.ChannelName, InMessage.Status);
	}
}

void FAvaPlaybackClient::HandlePlaybackSequenceEventMessage(const FAvaPlaybackSequenceEvent& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	using namespace UE::AvaPlaybackClient::Delegates;
	const FPlaybackSequenceEventArgs Args =
	{
		InMessage.InstanceId,
		InMessage.AssetPath,
		InMessage.ChannelName,
		InMessage.ServerName,
		InMessage.SequenceLabel,
		InMessage.EventType,
		InMessage.FrameNumber
	};
	GetOnPlaybackSequenceEvent().Broadcast(*this, Args);
}

void FAvaPlaybackClient::HandlePlaybackTransitionEventMessage(const FAvaPlaybackTransitionEvent& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	using namespace UE::AvaPlaybackClient::Delegates;
	const FPlaybackTransitionEventArgs Args =
	{
		InMessage.TransitionId,
		InMessage.Entry.InstanceId,
		InMessage.ChannelName,
		InMessage.ServerName,
		InMessage.Entry.GetEventFlags(),
		InMessage.FrameNumber
	};
	GetOnPlaybackTransitionEvent().Broadcast(*this, Args);
}

void FAvaPlaybackClient::HandlePlaybackTransitionEventMultiMessage(const FAvaPlaybackTransitionEventMulti& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	using namespace UE::AvaPlaybackClient::Delegates;
	for (const FAvaPlaybackTransitionEventEntry& Entry : InMessage.Entries)
	{
		const FPlaybackTransitionEventArgs Args =
		{
			InMessage.TransitionId,
			Entry.InstanceId,
			InMessage.ChannelName,
			InMessage.ServerName,
			Entry.GetEventFlags(),
			InMessage.FrameNumber
		};
		GetOnPlaybackTransitionEvent().Broadcast(*this, Args);
	}
}

void FAvaPlaybackClient::RegisterCommands()
{
	if (ConsoleCommands.Num() != 0)
	{
		return;
	}

	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("MotionDesignPlaybackClient.PingServers"),
		TEXT("Requests servers to give their information."),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaPlaybackClient::PingServersCommand),
		ECVF_Default
	));
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("MotionDesignPlaybackClient.RefreshAssetStatus"),
		TEXT("Request a refresh of the asset status."),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaPlaybackClient::RequestPlaybackAssetStatusCommand),
		ECVF_Default
	));
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("MotionDesignPlaybackClient.PlaybackRequest"),
		TEXT("Request Connected Servers to execute a playback request."),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaPlaybackClient::RequestPlaybackCommand),
		ECVF_Default
	));
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("MotionDesignPlaybackClient.BroadcastRequest"),
		TEXT("Request Connected Servers to execute a broadcast request."),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaPlaybackClient::RequestBroadcastCommand),
		ECVF_Default
	));
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("MotionDesignPlaybackClient.SetUserData"),
		TEXT("Set Replicated User Data Entry (Key, Value)."),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaPlaybackClient::SetUserDataCommand),
		ECVF_Default
	));
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("MotionDesignPlaybackClient.Status"),
		TEXT("Display current status of all server info."),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaPlaybackClient::ShowStatusCommand),
		ECVF_Default
	));
}

namespace UE::AvaPlaybackClient::Private
{
	inline FString GetCommaSeparatedList(const UEnum* InEnum)
	{
		FString List;
		const int32 NumEnums = InEnum->ContainsExistingMax() ? InEnum->NumEnums() - 1 : InEnum->NumEnums();
		for (int32 EnumIndex = 0; EnumIndex < NumEnums; ++EnumIndex)
		{
			if (!List.IsEmpty())
			{
				List += TEXT(", ");
			}
			List += InEnum->GetNameStringByIndex(EnumIndex);
		}
		return List;
	}
}

void FAvaPlaybackClient::RequestPlaybackAssetStatusCommand(const TArray<FString>& InArgs)
{
	using namespace UE::AvaPlaybackClient::Private;
	if (Servers.Num() == 0)
	{
		UE_LOGF(LogAvaPlaybackClient, Log, "No servers available. Do AvaMediaClient.PingServers.");
		return;
	}

	if (InArgs.Num() == 0)
	{
		UE_LOGF(LogAvaPlaybackClient, Log,
			   "Arguments: [AssetPath] [ChannelOrServer]. Ex: \"/Game/AvaPlayback.AvaPlayback Channel0\"");
	}

	FString ChannelOrServerName;
	FSoftObjectPath AssetPath;

	if (InArgs.Num() > 0)
	{
		AssetPath = FSoftObjectPath(FTopLevelAssetPath(InArgs[0]));
	}
	
	if (InArgs.Num() > 1)
	{
		ChannelOrServerName = InArgs[1];
	}
	else
	{
		ChannelOrServerName = UAvaBroadcast::Get().GetChannelName(0).ToString();
		UE_LOGF(LogAvaPlaybackClient, Log, "No Channel specified, using \"%ls\".", *ChannelOrServerName);
	}

	if (AssetPath.IsValid())
	{
		RequestPlaybackAssetStatus(AssetPath, ChannelOrServerName, true);
	}
	else
	{
		// Request a refresh of all cached assets for the specified server/channel.
		ForAllServers([&ChannelOrServerName, this](const FServerInfo& InServerInfo)
		{			
			if (InServerInfo.ServerName ==  ChannelOrServerName || InServerInfo.BroadcastChannelInfosByName.Contains(ChannelOrServerName))
			{
				for (const TPair<FSoftObjectPath, EAvaPlaybackAssetStatus>& AssetStatus : InServerInfo.PlaybackAssetStatuses)
				{
					RequestPlaybackAssetStatus(AssetStatus.Key, ChannelOrServerName, true);
				}
			}
		});
	}
}

void FAvaPlaybackClient::RequestPlaybackCommand(const TArray<FString>& InArgs)
{
	using namespace UE::AvaPlaybackClient::Private;
	if (Servers.Num() == 0)
	{
		UE_LOGF(LogAvaPlaybackClient, Log, "No servers available. Do AvaMediaClient.PingServers.");
		return;
	}

	if (InArgs.Num() == 0)
	{
		UE_LOGF(LogAvaPlaybackClient, Log,
			   "Arguments: Action [Package] [AssetName]. Ex: \"Start /Game/AvaPlayback AvaPlayback\"");
		UE_LOGF(LogAvaPlaybackClient, Log, "Action: %ls", *GetCommaSeparatedList(StaticEnum<EAvaPlaybackAction>()));
		return;
	}

	const int64 PlaybackActionValue = StaticEnum<EAvaPlaybackAction>()->GetValueByNameString(InArgs[0]);
	if (PlaybackActionValue == INDEX_NONE)
	{
		UE_LOGF(LogAvaPlaybackClient, Log, "Action: \"%ls\" is not valid. Possible actions: %ls",
			   *InArgs[0], *GetCommaSeparatedList(StaticEnum<EAvaPlaybackAction>()));
		return;
	}

	const EAvaPlaybackAction PlaybackAction = static_cast<EAvaPlaybackAction>(PlaybackActionValue);

	if (InArgs.Num() >= 3)
	{
		//Concatenate all Args (starting from the 3rd) into one String with spaces in between each arg
		FString ConcatenatedCommands;
		for (int32 Index = 3; Index < InArgs.Num(); ++Index)
		{
			ConcatenatedCommands += InArgs[Index] + TEXT(" ");
		}

		FString ChannelName;
		FParse::Value(*ConcatenatedCommands, TEXT("Channel="), ChannelName);

		FString InstanceId;
		FParse::Value(*ConcatenatedCommands, TEXT("InstandId="), InstanceId);
		
		const FSoftObjectPath AssetPath(InArgs[1] + TEXT(".") + InArgs[2]);
		RequestPlayback(FGuid(InstanceId), AssetPath, ChannelName, PlaybackAction);
	}
	else
	{
		RequestPlayback(FGuid(), FSoftObjectPath(), TEXT(""), PlaybackAction);
	}
}

void FAvaPlaybackClient::RequestBroadcastCommand(const TArray<FString>& InArgs)
{
	using namespace UE::AvaPlaybackClient::Private;

	if (Servers.Num() == 0)
	{
		UE_LOGF(LogAvaPlaybackClient, Log, "No servers available.");
		return;
	}

	if (InArgs.Num() == 0 || InArgs.Num() > 1)
	{
		UE_LOGF(LogAvaPlaybackClient, Log, "Arguments: Action.  Possible actions: %ls",
			   *GetCommaSeparatedList(StaticEnum<EAvaBroadcastAction>()));
		return;
	}

	const int64 BroadcastActionValue = StaticEnum<EAvaBroadcastAction>()->GetValueByNameString(InArgs[0]);
	if (BroadcastActionValue == INDEX_NONE)
	{
		UE_LOGF(LogAvaPlaybackClient, Log, "Action: \"%ls\" is not valid. Possible actions: %ls",
			   *InArgs[0], *GetCommaSeparatedList(StaticEnum<EAvaBroadcastAction>()));
		return;
	}
	
	RequestBroadcast(TEXT(""), FName(), TArray<UMediaOutput*>(), static_cast<EAvaBroadcastAction>(BroadcastActionValue));
}

void FAvaPlaybackClient::SetUserDataCommand(const TArray<FString>& InArgs)
{
	if (InArgs.Num() >= 2)
	{
		UE_LOGF(LogAvaPlaybackClient, Log, "Setting User Data Key \"%ls\" to Value: \"%ls\".", *InArgs[0], *InArgs[1]);
		SetUserData(InArgs[0], InArgs[1]);
	}
	else if (InArgs.Num() == 1)
	{
		// One argument means to remove that user data entry.
		if (HasUserData(InArgs[0]))
		{
			UE_LOGF(LogAvaPlaybackClient, Log, "Removing User Data Key \"%ls\".", *InArgs[0]);
			RemoveUserData(InArgs[0]);
		}
		else
		{
			UE_LOGF(LogAvaPlaybackClient, Error, "User Data Key \"%ls\" not found.", *InArgs[0]);
		}
	}
}

void FAvaPlaybackClient::ShowStatusCommand(const TArray<FString>& InArgs)
{
	using namespace UE::AvaPlayback::Utils;
	UE_LOGF(LogAvaPlaybackClient, Display, "Playback Client: \"%ls\"", *ComputerName);
	UE_LOGF(LogAvaPlaybackClient, Display, "- Endpoint Bus Address: \"%ls\"", MessageEndpoint.IsValid() ? *MessageEndpoint->GetAddress().ToString() : TEXT("Invalid"));
	UE_LOGF(LogAvaPlaybackClient, Display, "- ProcessId: %d", ProcessId);
	UE_LOGF(LogAvaPlaybackClient, Display, "- Content Path: \"%ls\"", *ProjectContentPath);

	for (const TPair<FString, FString>& UserData : UserDataEntries)
	{
		UE_LOGF(LogAvaPlaybackClient, Display, "- User data \"%ls\":\"%ls\".", *UserData.Key, *UserData.Value);
	}
	
	for (const TPair<FString, TSharedPtr<FServerInfo>>& Server : Servers)
	{
		check(Server.Value.IsValid());
		const FServerInfo& ServerInfo = *Server.Value;
		UE_LOGF(LogAvaPlaybackClient, Display, "Connected Server: \"%ls\"", *ServerInfo.ServerName);
		
		UE_LOGF(LogAvaPlaybackClient, Display, "   - Endpoint Bus Address: \"%ls\"", *ServerInfo.Address.ToString());
		UE_LOGF(LogAvaPlaybackClient, Display, "   - ProcessId: %d", ServerInfo.ProcessId);
		UE_LOGF(LogAvaPlaybackClient, Display, "   - Content Path: \"%ls\"", *ServerInfo.ProjectContentPath);
		
		for (const TPair<FString, FString>& UserData : ServerInfo.UserDataEntries)
		{
			UE_LOGF(LogAvaPlaybackClient, Display, "   - User data \"%ls\":\"%ls\".", *UserData.Key, *UserData.Value);
		}
		for (const TPair<FString, TUniquePtr<FBroadcastChannelInfo>>& ChannelInfo : ServerInfo.BroadcastChannelInfosByName)
		{
			check(ChannelInfo.Value.IsValid());
			UE_LOGF(LogAvaPlaybackClient, Display, "   - Channel(\"%ls\") Channel State: \"%ls\".",
				*ChannelInfo.Key,
				*StaticEnumToString(ChannelInfo.Value->ChannelState));
			UE_LOGF(LogAvaPlaybackClient, Display, "   - Channel(\"%ls\") Channel Issue Severity: \"%ls\".",
				*ChannelInfo.Key,
				*StaticEnumToString(ChannelInfo.Value->ChannelIssueSeverity));
			for (const TPair<FGuid, FAvaBroadcastOutputStatus>& MediaOutputStatus : ChannelInfo.Value->MediaOutputStatuses)
			{
				UE_LOGF(LogAvaPlaybackClient, Display, "   - Channel(\"%ls\") Media Output \"%ls\" State: \"%ls\".",
					*ChannelInfo.Key, *MediaOutputStatus.Key.ToString(),
					*StaticEnumToString(MediaOutputStatus.Value.MediaOutputState));
				UE_LOGF(LogAvaPlaybackClient, Display, "   - Channel(\"%ls\") Media Output \"%ls\" Issue Severity: \"%ls\".",
					*ChannelInfo.Key, *MediaOutputStatus.Key.ToString(),
					*StaticEnumToString(MediaOutputStatus.Value.MediaIssueSeverity));
				for (const FString& MediaIssueMessage : MediaOutputStatus.Value.MediaIssueMessages)
				{
					UE_LOGF(LogAvaPlaybackClient, Display, "   - Channel(\"%ls\") Media Output \"%ls\" Message: \"%ls\".",
						*ChannelInfo.Key, *MediaOutputStatus.Key.ToString(), *MediaIssueMessage);
				}
			}
		}
		for (const TPair<FString, TUniquePtr<FPlaybackChannelInfo>>& ChannelInfoEntry : ServerInfo.PlaybackChannelInfosByName)
		{
			check(ChannelInfoEntry.Value.IsValid());
			const FPlaybackChannelInfo& ChannelInfo = *ChannelInfoEntry.Value;
			for (const TPair<FSoftObjectPath, TUniquePtr<FPlaybackAssetInfo>>& AssetInfo : ChannelInfo.AssetInfoByPaths)
			{
				// Dump per instance statuses.
				for (const TPair<FGuid, FPlaybackInstanceInfo>& InstanceInfo : AssetInfo.Value->InstanceByIds)
				{
					// Check if we have user data.
					FString PrettyPrintUserData = InstanceInfo.Value.UserData.IsSet() ?
						FString::Printf(TEXT(" - UserData: %s"), *InstanceInfo.Value.UserData.GetValue()) : FString();
					
					UE_LOGF(LogAvaPlaybackClient, Display, "   - Channel(\"%ls\") Playback \"%ls\"(instance \"%ls\") Status:\"%ls\"%ls.",
						*ChannelInfoEntry.Key,
						*AssetInfo.Key.ToString(),
						*InstanceInfo.Key.ToString(),
						*StaticEnumToString(InstanceInfo.Value.Status),
						*PrettyPrintUserData);
				}
			}
		}
		for (const TPair<FSoftObjectPath, EAvaPlaybackAssetStatus>& AssetStatus : ServerInfo.PlaybackAssetStatuses)
		{
			UE_LOGF(LogAvaPlaybackClient, Display, "   - Asset (\"%ls\") Status \"%ls\".",
				*AssetStatus.Key.ToString(),
				*StaticEnumToString(AssetStatus.Value));
		}
	}
}

void FAvaPlaybackClient::OnBroadcastChanged(EAvaBroadcastChange InChange)
{
	if (EnumHasAnyFlags(InChange, EAvaBroadcastChange::CurrentProfile))
	{
		// Propagate the new output configuration to the connected servers of this channel.
		// We can do this because the profile can only be switched if channels are idle.
		FAvaBroadcastProfile& CurrentProfile = UAvaBroadcast::Get().GetCurrentProfile();
		const TArray<FAvaBroadcastOutputChannel*>& Channels = CurrentProfile.GetChannels();
		for (const FAvaBroadcastOutputChannel* Channel : Channels)
		{
			TArray<UMediaOutput*> RemoteOutputs = Channel->GetRemoteMediaOutputs();
			if (!RemoteOutputs.IsEmpty())
			{
				const FString ProfileName = CurrentProfile.GetName().ToString();
				RequestBroadcast(ProfileName, Channel->GetChannelName(), RemoteOutputs, EAvaBroadcastAction::UpdateConfig);
			}
		}
	}
}

void FAvaPlaybackClient::OnChannelChanged(const FAvaBroadcastOutputChannel& InChannel, EAvaBroadcastChannelChange InChange)
{
	if (EnumHasAnyFlags(InChange, EAvaBroadcastChannelChange::Settings))
	{
		SendBroadcastChannelSettingsUpdate(GetServerAddressesForChannel(InChannel.GetChannelName()), InChannel);
	}
}

void FAvaPlaybackClient::OnAvaMediaSettingsChanged(UObject*, struct FPropertyChangedEvent&)
{
	ApplyAvaMediaSettings();
	SendBroadcastSettingsUpdate(AllServerAddresses);
	SendAvaInstanceSettingsUpdate(AllServerAddresses);
	SendPlayableSettingsUpdate(AllServerAddresses);
}

void FAvaPlaybackClient::OnPreSavePackage(UPackage* InPackage, FObjectPreSaveContext InObjectSaveContext)
{
	// Only execute if this is a user save
	if (InObjectSaveContext.IsProceduralSave())
	{
		return;
	}

	// Early return is no servers are connected.
	if (Servers.IsEmpty())
	{
		return;
	}

	// Propagate this to inform local servers to flush loaders (and unlock the files).
	// Remark: The server may already flush the playback asset's package loaders (if enabled) but this path will
	// cover any other assets (not playback) that might have been loaded by both the server and client. 
	SendPackageEvent(AllServerAddresses, InPackage->GetFName(), EAvaPlaybackPackageEvent::PreSave);
}

void FAvaPlaybackClient::OnPackageSaved(const FString& InPackageFileName, UPackage* InPackage, FObjectPostSaveContext InObjectSaveContext)
{
	// Only execute if this is a user save
	if (InObjectSaveContext.IsProceduralSave())
	{
		return;
	}

	// Early return is no servers are connected.
	if (Servers.IsEmpty())
	{
		return;
	}

	// Propagate this to inform local servers to reload the package.
	SendPackageEvent(AllServerAddresses, InPackage->GetFName(), EAvaPlaybackPackageEvent::PostSave);
	
	TArray<FAssetData> AssetsInPackage;
	if (const IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		AssetsInPackage.Reserve(16);
		AssetRegistry->GetAssetsByPackageName(InPackage->GetFName(), AssetsInPackage);
		
		if (AssetsInPackage.IsEmpty())
		{
			UE_LOGF(LogAvaPlaybackClient, Warning, "Asset Registry returns no asset data for package \"%ls\".", *InPackage->GetName());
		}
	}
	
	// If a local package (of a playback asset) was saved, we will refresh the status of the assets with the servers.
	ForAllServers([&AssetsInPackage, this](const FServerInfo& InServerInfo)
	{
		for (const FAssetData& Assets : AssetsInPackage)
		{
			if (InServerInfo.PlaybackAssetStatuses.Contains(Assets.ToSoftObjectPath()))
			{
				// Note: we force a refresh (i.e. disregard the server's cached value) because the asset has changed on client side.
				RequestPlaybackAssetStatusForServer(Assets.ToSoftObjectPath(), InServerInfo.ServerName, /*bInForceRefresh*/ true);
			}
		}
	});
}

void FAvaPlaybackClient::OnAssetRemoved(const FAssetData& InAssetData)
{
	// Early return is no servers are connected.
	if (Servers.IsEmpty())
	{
		return;
	}

	// Propagate this to inform local servers to purge the package.
	SendPackageEvent(AllServerAddresses, InAssetData.PackageName, EAvaPlaybackPackageEvent::AssetDeleted);
}

void FAvaPlaybackClient::ApplyAvaMediaSettings()
{
	const UAvaMediaSettings& Settings = UAvaMediaSettings::Get();
#if !NO_LOGGING
	if (Settings.bVerbosePlaybackClientLogging)
	{
		LogAvaPlaybackClient.SetVerbosity(ELogVerbosity::Verbose);
	}
	else
	{
		LogAvaPlaybackClient.SetVerbosity(ELogVerbosity::Log);
	}
#endif
}

void FAvaPlaybackClient::PublishPlaybackPing(const FDateTime& InCurrentTime, bool bInAutoPing)
{
	if (MessageEndpoint.IsValid())
	{
		const UAvaMediaSettings& AvaMediaSettings = UAvaMediaSettings::Get();
		// Add a timeout to all known servers
		const FDateTime Timeout = InCurrentTime + FTimespan::FromSeconds(AvaMediaSettings.PingTimeoutInterval);
		ForAllServers([&Timeout](FServerInfo& InServerInfo)
		{
			InServerInfo.AddTimeout(Timeout);
		});

		FAvaPlaybackPing* PlaybackPingMessage = FMessageEndpoint::MakeMessage<FAvaPlaybackPing>();
		PlaybackPingMessage->bAutoPing = bInAutoPing;
		PlaybackPingMessage->PingIntervalSeconds = AvaMediaSettings.PingInterval;
		PublishRequest(PlaybackPingMessage);
	}
}

void FAvaPlaybackClient::SendUserDataUpdate(const TArray<FMessageAddress>& InRecipients)
{
	FAvaPlaybackUpdateClientUserData* UserDataUpdate = FMessageEndpoint::MakeMessage<FAvaPlaybackUpdateClientUserData>();
	UserDataUpdate->UserDataEntries = UserDataEntries;
	SendRequest(UserDataUpdate, InRecipients, EMessageFlags::Reliable);
}

void FAvaPlaybackClient::SendBroadcastSettingsUpdate(const TArray<FMessageAddress>& InRecipients)
{
	const IAvaBroadcastSettings& BroadcastSettings = IAvaMediaModule::Get().GetBroadcastSettings();
	FAvaBroadcastSettingsUpdate* BroadcastSettingsUpdate = FMessageEndpoint::MakeMessage<FAvaBroadcastSettingsUpdate>();
	BroadcastSettingsUpdate->BroadcastSettings.ChannelClearColor = BroadcastSettings.GetChannelClearColor();
	BroadcastSettingsUpdate->BroadcastSettings.ChannelDefaultPixelFormat = BroadcastSettings.GetDefaultPixelFormat();
	BroadcastSettingsUpdate->BroadcastSettings.ChannelDefaultResolution = BroadcastSettings.GetDefaultResolution();
	BroadcastSettingsUpdate->BroadcastSettings.bDrawPlaceholderWidget = BroadcastSettings.IsDrawPlaceholderWidget();
	BroadcastSettingsUpdate->BroadcastSettings.PlaceholderWidgetClass = BroadcastSettings.GetPlaceholderWidgetClass();
	SendRequest(BroadcastSettingsUpdate, InRecipients, EMessageFlags::Reliable);
}

void FAvaPlaybackClient::SendAvaInstanceSettingsUpdate(const TArray<FMessageAddress>& InRecipients)
{
	FAvaPlaybackInstanceSettingsUpdate* InstanceSettingsUpdate = FMessageEndpoint::MakeMessage<FAvaPlaybackInstanceSettingsUpdate>();
	InstanceSettingsUpdate->InstanceSettings = UAvaMediaSettings::Get().AvaInstanceSettings;
	SendRequest(InstanceSettingsUpdate, InRecipients, EMessageFlags::Reliable);
}

void FAvaPlaybackClient::SendPlayableSettingsUpdate(const TArray<FMessageAddress>& InRecipients)
{
	FAvaPlaybackPlayableSettingsUpdate* PlayableSettingsUpdate = FMessageEndpoint::MakeMessage<FAvaPlaybackPlayableSettingsUpdate>();
	PlayableSettingsUpdate->PlayableSettings = UAvaMediaSettings::Get().PlayableSettings;
	SendRequest(PlayableSettingsUpdate, InRecipients, EMessageFlags::Reliable);
}

void FAvaPlaybackClient::SendBroadcastChannelSettingsUpdate(const TArray<FMessageAddress>& InRecipients, const FAvaBroadcastOutputChannel& InChannel)
{
	FAvaBroadcastChannelSettingsUpdate* ChannelSettingsUpdate = FMessageEndpoint::MakeMessage<FAvaBroadcastChannelSettingsUpdate>();
	ChannelSettingsUpdate->Profile = InChannel.GetProfileName().ToString();
	ChannelSettingsUpdate->Channel = InChannel.GetChannelName().ToString();
	ChannelSettingsUpdate->QualitySettings = InChannel.GetViewportQualitySettings();
	SendRequest(ChannelSettingsUpdate, InRecipients, EMessageFlags::Reliable);
}

void FAvaPlaybackClient::SendPackageEvent(const TArray<FMessageAddress>& InRecipients, const FName& InPackageName, EAvaPlaybackPackageEvent InEvent)
{
	FAvaPlaybackPackageEvent* PackageEvent = FMessageEndpoint::MakeMessage<FAvaPlaybackPackageEvent>();
	PackageEvent->PackageName = InPackageName;
	PackageEvent->Event = InEvent;
	SendRequest(PackageEvent, InRecipients, EMessageFlags::Reliable);
}

void FAvaPlaybackClient::SendStatCommand(const FString& InCommand, bool bInBroadcastLocalState, const TArray<FMessageAddress>& InRecipients)
{
	FAvaPlaybackStatCommand* Request = FMessageEndpoint::MakeMessage<FAvaPlaybackStatCommand>();
	Request->Command = InCommand;
	Request->bClientStateReliable = bInBroadcastLocalState;
	Request->ClientEnabledRuntimeStats = IAvaModule::Get().GetEnabledRuntimeStats();
	SendRequest(Request, InRecipients);
}

void FAvaPlaybackClient::SendClientInfo(const FMessageAddress& InRecipient)
{
	FAvaPlaybackUpdateClientInfo* ClientInfo = FMessageEndpoint::MakeMessage<FAvaPlaybackUpdateClientInfo>();
	ClientInfo->ComputerName = ComputerName;
	ClientInfo->ProcessId = ProcessId;
	ClientInfo->ProjectContentPath = ProjectContentPath;
	SendRequest(ClientInfo, InRecipient);

	const TArray<FMessageAddress> Recipients = {InRecipient}; 
	SendUserDataUpdate(Recipients);
	SendBroadcastSettingsUpdate(Recipients);
	SendAvaInstanceSettingsUpdate(Recipients);
	SendPlayableSettingsUpdate(Recipients);
	SendStatCommand(FString(), true, Recipients);	// Send empty stat command, will just send current states.
}

void FAvaPlaybackClient::RemoveDeadServers(const FDateTime& InCurrentTime)
{
	bool bServerRemoved = false;

	for (TMap<FString, TSharedPtr<FServerInfo>>::TIterator ServerIter(Servers); ServerIter; ++ServerIter)
	{
		check(ServerIter.Value().IsValid());
		if (ServerIter.Value()->HasTimedOut(InCurrentTime))
		{
			UE_LOGF(LogAvaPlaybackClient, Log, "Server \"%ls\" is not longer responding to pings. Removing.",
				   *ServerIter.Key());
			const TSharedPtr<FServerInfo> RemovedServer = ServerIter.Value();
			ServerIter.RemoveCurrent();
			OnServerRemoved(*RemovedServer);
			bServerRemoved = true;
		}
	}

	if (bServerRemoved)
	{
		UpdateServerAddresses();
	}
}

void FAvaPlaybackClient::UpdateServerAddresses()
{
	AllServerAddresses.Empty(Servers.Num());
	ForAllServers([this](const FServerInfo& InServerInfo)
	{
		AllServerAddresses.Add(InServerInfo.Address);
	});
}

TArray<FMessageAddress> FAvaPlaybackClient::GetServerAddressesForChannel(const FName& InChannelName) const
{
	if (!InChannelName.IsNone())
	{
		TArray<FString> ServerNames = GetServerNamesForChannel(InChannelName);
		TArray<FMessageAddress> ServerAddresses;
		ServerAddresses.Reserve(ServerNames.Num());
		for (const FString& ServerName : ServerNames)
		{
			if (const FServerInfo* ServerInfo = GetServerInfo(ServerName))
			{
				ServerAddresses.Add(ServerInfo->Address);
			}
		}
		return ServerAddresses;
	}

	// If no channel is specified, we will return all the servers that are connected.
	return AllServerAddresses;
}

TArray<FString> FAvaPlaybackClient::GetServerNamesForChannel(const FName& InChannelName, bool bInOnlineOnly) const
{
	if (InChannelName.IsNone())
	{
		// If no channel is specified, we will return all the servers that are connected.
		return GetServerNames();
	}
	
	TArray<FString> OutServerNames;

	const FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetChannel(InChannelName);
	const TArray<UMediaOutput*>& RemoteOutputs = Channel.GetRemoteMediaOutputs();
	OutServerNames.Reserve(RemoteOutputs.Num());
	for (const UMediaOutput* RemoteOutput : RemoteOutputs)
	{
		const FAvaBroadcastMediaOutputInfo& OutputInfo = Channel.GetMediaOutputInfo(RemoteOutput);
		if (OutputInfo.IsValid())
		{
			if (!bInOnlineOnly || Servers.Contains(OutputInfo.ServerName))
			{
				OutServerNames.AddUnique(OutputInfo.ServerName);
			}
		}
		else
		{
			UE_LOGF(LogAvaPlaybackClient, Warning, "MediaOutputInfo invalid for channel \"%ls\".",
				   *InChannelName.ToString());

			// Try to find the server name from the device name. The server has to be online for this to work.
			const FString ServerName = GetServerNameForMediaOutputFallback(RemoteOutput);
			if (!ServerName.IsEmpty())
			{
				OutServerNames.AddUnique(ServerName);
			}
			else
			{
				UE_LOGF(LogAvaPlaybackClient, Error,
					   "Unable to find server name for remote MediaOutput for channel \"%ls\".",
					   *InChannelName.ToString());
			}
		}
	}
	return OutServerNames;
}

FString FAvaPlaybackClient::GetServerNameForMediaOutputFallback(const UMediaOutput* InMediaOutput) const
{
	const FString DeviceName = UE::AvaBroadcastOutputUtils::GetDeviceName(InMediaOutput);
	if (!DeviceName.IsEmpty())
	{
		// See if it begins with any of the remote server names that are connected.
		// See FAvaBroadcastDeviceProviderData::ApplyServerName. All MediaOutput from replicated Device Provider
		// Have the server name in the beginning of the device name.
		for (const TPair<FString, TSharedPtr<FServerInfo>>& Server : Servers)
		{
			if (DeviceName.StartsWith(Server.Key))
			{
				return Server.Key;
			}
		}
	}
	return FString();
}

FAvaPlaybackClient::FServerInfo& FAvaPlaybackClient::GetOrCreateServerInfo(
	const FString& InServerName, const FMessageAddress& InSenderAddress, bool* bOutCreated)
{
	if (FServerInfo* ServerInfo = GetServerInfo(InServerName))
	{
		if (ServerInfo->Address != InSenderAddress)
		{
			// This is suspicious though. It may also indicate a collision with multiple servers
			// with the same name on the same computer host.
			UE_LOGF(LogAvaPlaybackClient, Warning, "Server \"%ls\" Address changed.", *InServerName);
			ServerInfo->Address = InSenderAddress;
		}
		return *ServerInfo;
	}

	const TSharedPtr<FServerInfo> ServerInfo = MakeShared<FServerInfo>();
	ServerInfo->Address = InSenderAddress;
	ServerInfo->ServerName = InServerName;
	Servers.Add(InServerName, ServerInfo);
	
	UpdateServerAddresses();
	OnServerAdded(*ServerInfo);

	if (bOutCreated)
	{
		*bOutCreated = true;
	}
	return *ServerInfo;
}

void FAvaPlaybackClient::ForAllServers(TFunctionRef<void(FServerInfo& /*InServerInfo*/)> InFunction)
{
	for (const TPair<FString, TSharedPtr<FServerInfo>>& ServerInfo : Servers)
	{
		check(ServerInfo.Value.IsValid());
		InFunction(*ServerInfo.Value);
	}
}

void FAvaPlaybackClient::ForAllServers(TFunctionRef<void(const FServerInfo& /*InServerInfo*/)> InFunction) const
{
	for (const TPair<FString, TSharedPtr<FServerInfo>>& ServerInfo : Servers)
	{
		check(ServerInfo.Value.IsValid());
		InFunction(*ServerInfo.Value);
	}
}

void FAvaPlaybackClient::OnServerAdded(const FServerInfo& InServerInfo)
{
	UE_LOGF(LogAvaPlaybackClient, Log, "Registering new playback server \"%ls\".", *InServerInfo.ServerName);
		
	FAvaPlaybackDeviceProviderDataRequest* DataRequest = FMessageEndpoint::MakeMessage<FAvaPlaybackDeviceProviderDataRequest>();
	SendRequest(DataRequest, InServerInfo.Address);
	
	SendClientInfo(InServerInfo.Address);
}

void FAvaPlaybackClient::OnServerRemoved(const FServerInfo& InRemovedServer)
{
	if (ParentModule)
	{
		IAvaBroadcastDeviceProviderProxyManager& Manager = ParentModule->GetDeviceProviderProxyManager();
		Manager.Uninstall(InRemovedServer.ServerName);
	}

	// Update the status (i.e. go offline) of channels for this server.
	TArray<FString> AffectedChannelNames;
	for (const TPair<FString, TUniquePtr<FBroadcastChannelInfo>>& ChannelInfo : InRemovedServer.BroadcastChannelInfosByName)
	{
		check(ChannelInfo.Value.IsValid());
		AffectedChannelNames.Add(ChannelInfo.Key);
	}

	FAvaBroadcastProfile& CurrentProfile = UAvaBroadcast::Get().GetCurrentProfile();
	
	// Try to reconcile channel state from remaining outputs (if any), i.e. channel is still partially online.
	for (const FString& AffectedChannelName : AffectedChannelNames)
	{
		FAvaBroadcastOutputChannel& Channel = CurrentProfile.GetChannelMutable(FName(AffectedChannelName));
		if (Channel.IsValidChannel())
		{
			// Note: this will broadcast to delegates which may then request states of media outputs.
			// So the back store needs to be updated before calling this.
			Channel.RefreshState();
		}
	}

	using namespace UE::AvaPlaybackClient::Delegates;
	GetOnConnectionEvent().Broadcast(*this, {InRemovedServer.ServerName, EConnectionEvent::ServerDisconnected});
}

const FAvaPlaybackClient::FBroadcastChannelInfo* FAvaPlaybackClient::GetChannelInfo(
	const FString& InServerName, const FString& InChannelName) const
{
	if (!InServerName.IsEmpty())
	{
		if (const FServerInfo* ServerInfo = GetServerInfo(InServerName))
		{
			return ServerInfo->GetBroadcastChannelInfo(InChannelName);
		}
	}
	else
	{
		// Legacy support: If the server name is not specified,
		// Find the first server with the given channel name.
		// Note: This doesn't work for "forked" channels.	
		for (const TPair<FString, TSharedPtr<FServerInfo>>& Server : Servers)
		{
			check(Server.Value.IsValid());
			if (const FBroadcastChannelInfo* ChannelInfo = Server.Value->GetBroadcastChannelInfo(InChannelName))
			{
				return ChannelInfo;
			}
		}
	}
	return nullptr;
}
