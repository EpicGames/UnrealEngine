// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Broadcast/OutputDevices/AvaBroadcastDeviceProviderData.h"
#include "Containers/Ticker.h"
#include "IMessageContext.h"
#include "MessageEndpoint.h"
#include "Playback/AvaPlaybackMessages.h"
#include "Playback/IAvaPlaybackClient.h"

class IAvaMediaModule;
struct FAvaBroadcastOutputChannel;

// Listens to command to play and broadcast
class FAvaPlaybackClient : public IAvaPlaybackClient
{
public:
	FAvaPlaybackClient(IAvaMediaModule* InParentModule);
	virtual ~FAvaPlaybackClient() override;

	void Init();

	//IAvaPlaybackClient
	virtual int32 GetNumConnectedServers() const override;
	virtual TArray<FString> GetServerNames() const override;
	virtual FMessageAddress GetServerAddress(const FString& InServerName) const override;
	virtual bool HasServerUserData(const FString& InServerName, const FString& InKey) const override;
	virtual const FString& GetServerUserData(const FString& InServerName, const FString& InKey) const override;
	virtual bool HasUserData(const FString& InKey) const override;
	virtual const FString& GetUserData(const FString& InKey) const override;
	virtual void SetUserData(const FString& InKey, const FString& InData) override;
	virtual void RemoveUserData(const FString& InKey) override;
	virtual void BroadcastStatCommand(const FString& InCommand, bool bInBroadcastLocalState) override;
	virtual void RequestPlaybackAssetStatus(const FSoftObjectPath& InAssetPath, const FString& InChannelOrServerName, bool bInForceRefresh) override;
	virtual void RequestPlayback(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, EAvaPlaybackAction InAction, const FString& InArguments = FString()) override;
	virtual void RequestAnimPlayback(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FAvaPlaybackAnimPlaySettings& InAnimSettings) override;
	virtual void RequestAnimAction(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FString& InAnimationName, EAvaPlaybackAnimAction InAction) override;
	virtual void RequestRemoteControlUpdate(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FAvaPlayableRemoteControlValues& InRemoteControlValues, EAvaPlayableRCUpdateFlags InFlags) override;
	virtual void RequestPlayableTransitionStart(const FGuid& InTransitionId, TArray<FGuid>&& InEnterInstanceIds, TArray<FGuid>&& InPlayingInstanceIds, TArray<FGuid>&& InExitInstanceIds, TArray<FAvaPlayableRemoteControlValues>&& InEnterValues, const FName& InChannelName, EAvaPlayableTransitionFlags InTransitionFlags) override;
	virtual void RequestPlayableTransitionStop(const FGuid& InTransitionId, const FName& InChannelName) override;
	virtual void RequestBroadcast(const FString& InProfile, const FName& InChannel, const TArray<UMediaOutput*>& InRemoteMediaOutputs, EAvaBroadcastAction InAction, const FString& InServerName = FString()) override;
	virtual bool IsMediaOutputRemoteFallback(const UMediaOutput* InMediaOutput) override;
	virtual EAvaBroadcastIssueSeverity GetMediaOutputIssueSeverity(const FString& InServerName, const FString& InChannelName, const FGuid& InOutputGuid) const override;
	virtual const TArray<FString>& GetMediaOutputIssueMessages(const FString& InServerName, const FString& InChannelName, const FGuid& InOutputGuid) const override;
	virtual EAvaBroadcastOutputState GetMediaOutputState(const FString& InServerName, const FString& InChannelName, const FGuid& InOutputGuid) const override;
	virtual bool HasAnyServerOnlineForChannel(const FName& InChannelName) const override;
	virtual TArray<FString> GetOnlineServersForChannel(const FName& InChannelName) const override;
	virtual TOptional<EAvaPlaybackStatus> GetRemotePlaybackStatus(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FString& InServerName) const override;
	virtual const FString* GetRemotePlaybackUserData(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FString& InServerName) const override;
	virtual TOptional<EAvaPlaybackAssetStatus> GetRemotePlaybackAssetStatus(const FSoftObjectPath& InAssetPath, const FString& InServerName) const override;
	//~IAvaPlaybackClient

	FString GetServerProjectContentPath(const FString& InServerName) const;
	uint32 GetServerProcessId(const FString& InServerName) const;

	FString GetProjectContentPath() const { return ProjectContentPath; }

protected:
	TOptional<EAvaPlaybackStatus> GetPlaybackStatusForServer(const FString& InServerName, const FString& InChannelName, const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath) const;
	const FString* GetPlaybackUserDataForServer(const FString& InServerName, const FString& InChannelName, const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath) const;
	void RequestPlaybackAssetStatusForServer(const FSoftObjectPath& InAssetPath, const FString& InServerName, bool bInForceRefresh);

	void Tick();

	bool HandlePingTicker(float InDeltaTime);

	// Message handlers
	void HandlePlaybackPongMessage(const FAvaPlaybackPong& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandlePlaybackLogMessage(const FAvaPlaybackLog& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleUpdateServerUserData(const FAvaPlaybackUpdateServerUserData& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleStatStatus(const FAvaPlaybackStatStatus& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleDeviceProviderDataListMessage(const FAvaBroadcastDeviceProviderDataList& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleBroadcastStatusMessage(const FAvaBroadcastStatus& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandlePlaybackAssetStatusMessage(const FAvaPlaybackAssetStatus& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandlePlaybackStatusMessage(const FAvaPlaybackStatus& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandlePlaybackStatusesMessage(const FAvaPlaybackStatuses& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandlePlaybackSequenceEventMessage(const FAvaPlaybackSequenceEvent& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandlePlaybackTransitionEventMessage(const FAvaPlaybackTransitionEvent& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandlePlaybackTransitionEventMultiMessage(const FAvaPlaybackTransitionEventMulti& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);

private:
	class FServerInfo;
	
	void HandlePlaybackStatus(FServerInfo& InServerInfo,
		const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName,
		EAvaPlaybackStatus InStatus, const FString& InUserData = FString(), bool bInValidUserData = false);

protected:	
	void RegisterCommands();
	
	// Command handlers
	void PingServersCommand(const TArray<FString>& InArgs) { PublishPlaybackPing(FDateTime::UtcNow(), false);}
	void RequestPlaybackAssetStatusCommand(const TArray<FString>& InArgs);
	void RequestPlaybackCommand(const TArray<FString>& InArgs);
	void RequestBroadcastCommand(const TArray<FString>& InArgs);
	void SetUserDataCommand(const TArray<FString>& InArgs);
	void ShowStatusCommand(const TArray<FString>& InArgs);

	// Event handlers
	void OnBroadcastChanged(EAvaBroadcastChange InChange);
	void OnChannelChanged(const FAvaBroadcastOutputChannel& InChannel, EAvaBroadcastChannelChange InChange);
	void OnAvaMediaSettingsChanged(UObject*, struct FPropertyChangedEvent&);
	void OnPreSavePackage(UPackage* InPackage, FObjectPreSaveContext InObjectSaveContext);
	void OnPackageSaved(const FString& InPackageFileName, UPackage* InPackage, FObjectPostSaveContext InObjectSaveContext);
	void OnAssetRemoved(const FAssetData& InAssetData);
	
	void ApplyAvaMediaSettings();

protected:
	void FillClientInfo(FAvaPlaybackClientMessageBase& InMessage)
	{
		// Remark: for now, client name and computer name are the same.
		// We don't support having many clients per computer at this point.
		InMessage.ClientName =  ComputerName;
	}

	template<typename MessageType>
	void PublishRequest(MessageType* InMessage)
	{
		FillClientInfo(*InMessage);
		MessageEndpoint->Publish(InMessage);
	}

	template<typename MessageType>
	void SendRequest(MessageType* InMessage, const FMessageAddress& InRecipient, EMessageFlags flags = EMessageFlags::None)
	{
		FillClientInfo(*InMessage);
		MessageEndpoint->Send(InMessage, MessageType::StaticStruct(), flags, nullptr,
			TArrayBuilder<FMessageAddress>().Add(InRecipient), FTimespan::Zero(), FDateTime::MaxValue());
	}
	
	template<typename MessageType>
	void SendRequest(MessageType* InMessage, const TArray<FMessageAddress>& InRecipients, EMessageFlags flags = EMessageFlags::None)
	{
		FillClientInfo(*InMessage);
		MessageEndpoint->Send(InMessage, MessageType::StaticStruct(), flags, nullptr,
			InRecipients, FTimespan::Zero(), FDateTime::MaxValue());
	}

	void PublishPlaybackPing(const FDateTime& InCurrentTime, bool bInAutoPing);
	void SendUserDataUpdate(const TArray<FMessageAddress>& InRecipients);
	void SendBroadcastSettingsUpdate(const TArray<FMessageAddress>& InRecipients);
	void SendAvaInstanceSettingsUpdate(const TArray<FMessageAddress>& InRecipients);
	void SendPlayableSettingsUpdate(const TArray<FMessageAddress>& InRecipients);
	void SendBroadcastChannelSettingsUpdate(const TArray<FMessageAddress>& InRecipients, const FAvaBroadcastOutputChannel& InChannel);
	void SendPackageEvent(const TArray<FMessageAddress>& InRecipients, const FName& InPackageName, EAvaPlaybackPackageEvent InEvent);
	void SendStatCommand(const FString& InCommand, bool bInBroadcastLocalState, const TArray<FMessageAddress>& InRecipients);
	void SendClientInfo(const FMessageAddress& InRecipient);	
	void RemoveDeadServers(const FDateTime& InCurrentTime);
	void UpdateServerAddresses();
	
	/**
	 * Get the server addresses for the given channel in the current broadcast profile.
	 * This is based on the channel's MediaOutputInfos. It will properly work even
	 * if the channel is idle (but not offline).
	 */
	TArray<FMessageAddress> GetServerAddressesForChannel(const FName& InChannelName) const;
	TArray<FMessageAddress> GetServerAddressesForChannel(const FString& InChannelName) const
	{
		return GetServerAddressesForChannel(FName(InChannelName));
	}

	/**
	 * Get the server names for the given channel in the current broadcast profile.
	 * This is based on the channel's MediaOutputInfos. It will properly work
	 * regardless of the channel's status (idle or even offline).
	 */
	TArray<FString> GetServerNamesForChannel(const FName& InChannelName, bool bInOnlineOnly = false) const;

	FString GetServerNameForMediaOutputFallback(const UMediaOutput* InMediaOutput) const;

private:
	IAvaMediaModule* ParentModule;
	
	TArray<IConsoleObject*> ConsoleCommands;
	
	/** Holds the messaging endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;
	
	/** Holds a delegate to be invoked when the widget ticks. */
	FTickerDelegate PingTickDelegate;

	/** Handle to the registered TickDelegate. */
	FTSTicker::FDelegateHandle PingTickDelegateHandle;
	
	FDelegateHandle BroadcastChangedHandle;
	
	FString ComputerName;
	FString ProjectContentPath;
	uint32 ProcessId = 0;
	TMap<FString, FString> UserDataEntries;
	
	/** Structure holding information on pending playback asset status requests.*/
	struct FPendingPlaybackAssetStatusRequest
	{
		FDateTime ExpirationTime;
		bool bForceRefresh = false;
	};
	
	/** Remote Broadcast Channel Info */
	struct FBroadcastChannelInfo
	{
		EAvaBroadcastChannelState ChannelState;
		EAvaBroadcastIssueSeverity ChannelIssueSeverity;
		TMap<FGuid, FAvaBroadcastOutputStatus> MediaOutputStatuses;
	};

	/** Remote Playback Instance Info */
	struct FPlaybackInstanceInfo
	{
		EAvaPlaybackStatus  Status;
		TOptional<FString> UserData;
	};
	
	/** Remote playback for multiple instances of a given asset. */
	struct FPlaybackAssetInfo
	{
		TMap<FGuid, FPlaybackInstanceInfo> InstanceByIds;

		TOptional<EAvaPlaybackStatus> GetInstanceStatus(const FGuid& InInstanceId) const
		{
			const FPlaybackInstanceInfo* Instance = InstanceByIds.Find(InInstanceId);
			return Instance ? TOptional(Instance->Status) : TOptional<EAvaPlaybackStatus>();
		}

		const FPlaybackInstanceInfo* GetInstanceInfo(const FGuid& InInstanceId) const
		{
			return InInstanceId.IsValid() ? InstanceByIds.Find(InInstanceId) : nullptr;
		}

		void RemoveInstanceInfo(const FGuid& InInstanceId)
		{
			InstanceByIds.Remove(InInstanceId);
		}

		FPlaybackInstanceInfo* GetOrAddInstanceInfo(const FGuid& InInstanceId)
		{
			FPlaybackInstanceInfo* Instance = InstanceByIds.Find(InInstanceId);
			return Instance ? Instance : &InstanceByIds.Emplace(InInstanceId);
		}
		
		void SetInstanceStatus(const FGuid& InInstanceId, EAvaPlaybackStatus InStatus)
		{
			if (FPlaybackInstanceInfo* Instance = GetOrAddInstanceInfo(InInstanceId))
			{
				Instance->Status = InStatus;
			}
		}

		const FString* GetInstanceUserData(const FGuid& InInstanceId) const
		{
			// If the request is for a specific instance, check if we have it.
			if (const FPlaybackInstanceInfo* Instance = GetInstanceInfo(InInstanceId))
			{
				return Instance->UserData.IsSet() ? &Instance->UserData.GetValue() : nullptr;
			}
			return nullptr;
		}

		void SetInstanceUserData(const FGuid& InInstanceId, const FString& InUserData)
		{
			if (FPlaybackInstanceInfo* Instance = GetOrAddInstanceInfo(InInstanceId))
			{
				Instance->UserData = InUserData;
			}
		}
	};

	/** Playback Asset Info for a given channel.  */
	struct FPlaybackChannelInfo
	{
		TMap<FSoftObjectPath, TUniquePtr<FPlaybackAssetInfo>> AssetInfoByPaths;

		FPlaybackAssetInfo* GetAssetInfo(const FSoftObjectPath& InAssetPath) const
		{
			const TUniquePtr<FPlaybackAssetInfo>* AssetInfo = AssetInfoByPaths.Find(InAssetPath);
			return AssetInfo ? (*AssetInfo).Get() : nullptr;
		}

		FPlaybackAssetInfo* GetOrAddAssetInfo(const FSoftObjectPath& InAssetPath)
		{
			if (const TUniquePtr<FPlaybackAssetInfo>* AssetInfo = AssetInfoByPaths.Find(InAssetPath))
			{
				return (*AssetInfo).Get();
			}
			return AssetInfoByPaths.Add(InAssetPath, MakeUnique<FPlaybackAssetInfo>()).Get();
		}

		void RemoveInstanceInfo(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath)
		{
			if (FPlaybackAssetInfo* AssetInfo = GetAssetInfo(InAssetPath))
			{
				AssetInfo->RemoveInstanceInfo(InInstanceId);
			}
		}
		
		TOptional<EAvaPlaybackStatus> GetInstanceStatus(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath) const
		{
			const FPlaybackAssetInfo* AssetInfo = GetAssetInfo(InAssetPath);
			return AssetInfo ? AssetInfo->GetInstanceStatus(InInstanceId) : TOptional<EAvaPlaybackStatus>();
		}
		
		void SetInstanceStatus(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, EAvaPlaybackStatus InStatus)
		{
			if (FPlaybackAssetInfo* AssetInfo = GetOrAddAssetInfo(InAssetPath))
			{
				AssetInfo->SetInstanceStatus(InInstanceId, InStatus);
			}
		}

		const FString* GetInstanceUserData(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath) const
		{
			const FPlaybackAssetInfo* AssetInfo = GetAssetInfo(InAssetPath);
			return AssetInfo ? AssetInfo->GetInstanceUserData(InInstanceId) : nullptr;
		}
		
		void SetInstanceUserData(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InUserData)
		{
			if (FPlaybackAssetInfo* PlaybackInfos = GetOrAddAssetInfo(InAssetPath))
			{
				PlaybackInfos->SetInstanceUserData(InInstanceId, InUserData);
			}
		}
	};
	
	/** Remote Server Info */
	class FServerInfo
	{
	public:
		FMessageAddress Address;
		FString ServerName;
		FString ProjectContentPath;
		uint32 ProcessId = 0;
		TMap<FString, FString> UserDataEntries;
		TOptional<FDateTime> PingTimeout;
		bool bDeviceProviderInstalled = false;

		// Key ChannelName.
		TMap<FString, TUniquePtr<FBroadcastChannelInfo>> BroadcastChannelInfosByName;
		TMap<FString, TUniquePtr<FPlaybackChannelInfo>> PlaybackChannelInfosByName;

		// Asset statuses (on disk) are not per-channel.
		TMap<FSoftObjectPath, EAvaPlaybackAssetStatus> PlaybackAssetStatuses;

		/** Map using "playback asset status request" key to keep track of pending playback asset status requests. */
		TMap<FSoftObjectPath, FPendingPlaybackAssetStatusRequest> PendingPlaybackAssetStatusRequests;

		/** Accumulate all the playback commands and send them all in one message on the next tick. */
		TArray<FAvaPlaybackCommand> PendingPlaybackCommands;
		
		FBroadcastChannelInfo& GetOrCreateBroadcastChannelInfo(const FString& InChannelName)
		{
			const TUniquePtr<FBroadcastChannelInfo>* ChannelInfo = BroadcastChannelInfosByName.Find(InChannelName);
			return ChannelInfo ? *(*ChannelInfo).Get() : *BroadcastChannelInfosByName.Add(InChannelName, MakeUnique<FBroadcastChannelInfo>()).Get();
		}

		const FBroadcastChannelInfo* GetBroadcastChannelInfo(const FString& InChannelName) const
		{
			const TUniquePtr<FBroadcastChannelInfo>* ChannelInfo = BroadcastChannelInfosByName.Find(InChannelName);
			return ChannelInfo ? ChannelInfo->Get() : nullptr;
		}

		FPlaybackChannelInfo* GetOrCreatePlaybackChannelInfo(const FString& InChannelName)
		{
			const TUniquePtr<FPlaybackChannelInfo>* ChannelInfo = PlaybackChannelInfosByName.Find(InChannelName);
			return ChannelInfo ? (*ChannelInfo).Get() : PlaybackChannelInfosByName.Add(InChannelName, MakeUnique<FPlaybackChannelInfo>()).Get();
		}

		const FPlaybackChannelInfo* GetPlaybackChannelInfo(const FString& InChannelName) const
		{
			const TUniquePtr<FPlaybackChannelInfo>* ChannelInfo = PlaybackChannelInfosByName.Find(InChannelName);
			return ChannelInfo ? (*ChannelInfo).Get() : nullptr;
		}
		
		FPlaybackChannelInfo* GetPlaybackChannelInfo(const FString& InChannelName)
		{
			const TUniquePtr<FPlaybackChannelInfo>* ChannelInfo = PlaybackChannelInfosByName.Find(InChannelName);
			return ChannelInfo ? (*ChannelInfo).Get() : nullptr;
		}

		
		TOptional<EAvaPlaybackStatus> GetInstanceStatus(
			const FString& InChannelName, const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath) const
		{
			const FPlaybackChannelInfo* ChannelInfo = GetPlaybackChannelInfo(InChannelName);
			return ChannelInfo ? ChannelInfo->GetInstanceStatus(InInstanceId, InAssetPath) : TOptional<EAvaPlaybackStatus>();
		}

		void RemoveInstanceInfo(const FString& InChannelName, const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath)
		{
			if (FPlaybackChannelInfo* ChannelInfo = GetPlaybackChannelInfo(InChannelName))
			{
				ChannelInfo->RemoveInstanceInfo(InInstanceId, InAssetPath);
			}
		}
		
		void SetInstanceStatus(
			const FString& InChannelName, const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, EAvaPlaybackStatus InStatus)
		{
			if (FPlaybackChannelInfo* ChannelInfo = GetOrCreatePlaybackChannelInfo(InChannelName))
			{
				ChannelInfo->SetInstanceStatus(InInstanceId, InAssetPath, InStatus);
			}
		}

		const FString* GetInstanceUserData(const FString& InChannelName, const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath) const
		{
			const FPlaybackChannelInfo* ChannelInfo = GetPlaybackChannelInfo(InChannelName);
			return ChannelInfo ? ChannelInfo->GetInstanceUserData(InInstanceId, InAssetPath) : nullptr;
		}

		void SetInstanceUserData(const FString& InChannelName, const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InUserData)
		{
			if (FPlaybackChannelInfo* ChannelInfo = GetOrCreatePlaybackChannelInfo(InChannelName))
			{
				ChannelInfo->SetInstanceUserData(InInstanceId, InAssetPath, InUserData);
			}
		}
		
		TOptional<EAvaPlaybackAssetStatus> GetPlaybackAssetStatus(const FSoftObjectPath& InAssetPath) const
		{
			const EAvaPlaybackAssetStatus* FoundStatus = PlaybackAssetStatuses.Find(InAssetPath);
			return FoundStatus ? TOptional(*FoundStatus) : TOptional<EAvaPlaybackAssetStatus>();
		}
		
		void SetPlaybackAssetStatus(const FSoftObjectPath& InAssetPath, EAvaPlaybackAssetStatus InStatus)
		{
			PlaybackAssetStatuses.Add(InAssetPath, InStatus);
		}
		
		/**
		* Add a time by which a ping response should arrive
		*/
		void AddTimeout(const FDateTime& InNewTimeout)
		{
			if (!PingTimeout || InNewTimeout < PingTimeout.GetValue())
			{
				PingTimeout = InNewTimeout;
			}
		}

		/**
		 * Check if ping response arrived on time
		*/
		bool HasTimedOut(const FDateTime& InNow) const
		{
			return PingTimeout && InNow > PingTimeout.GetValue();
		}

		/**
		 * Resets the ping timeout.
		 * This is done when the ping response is received.
		 */
		void ResetPingTimeout()
		{
			PingTimeout.Reset();
		}
	};
	TMap<FString, TSharedPtr<FServerInfo>> Servers;	// Key is ServerName
	TArray<FMessageAddress> AllServerAddresses;

	FServerInfo& GetOrCreateServerInfo(const FString& InServerName, const FMessageAddress& InSenderAddress, bool* bOutCreated = nullptr);
	FServerInfo* GetServerInfo(const FString& InServerName)
	{
		const TSharedPtr<FServerInfo>* ServerInfo = Servers.Find(InServerName);
		return ServerInfo ? (*ServerInfo).Get() : nullptr;
	}
	const FServerInfo* GetServerInfo(const FString& InServerName) const
	{
		const TSharedPtr<FServerInfo>* ServerInfo = Servers.Find(InServerName);
		return ServerInfo ? (*ServerInfo).Get() : nullptr;
	}
	
	void ForAllServers(TFunctionRef<void(FServerInfo& /*InServerInfo*/)> InFunction);
	void ForAllServers(TFunctionRef<void(const FServerInfo& /*InServerInfo*/)> InFunction) const;
	void OnServerAdded(const FServerInfo& InServerInfo);
	void OnServerRemoved(const FServerInfo& InRemovedServer);
	const FBroadcastChannelInfo* GetChannelInfo(const FString& InServerName, const FString& InChannelName) const;
	
	/** Map using "playback status request" key to keep track of pending playback status requests. */
	TMap<FString, FDateTime> PendingPlaybackStatusRequests;
	
	static FString GetPlaybackStatusRequestKey(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName)
	{
		return InChannelName + InInstanceId.ToString() + InAssetPath.ToString();
	}
};
