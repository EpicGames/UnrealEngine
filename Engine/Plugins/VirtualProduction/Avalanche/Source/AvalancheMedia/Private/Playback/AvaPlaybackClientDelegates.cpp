// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/AvaPlaybackClientDelegates.h"

namespace UE::AvaPlaybackClient::Delegates
{
	namespace Private
	{
		static FOnConnectionEvent OnConnectionEvent;
		static FOnPlaybackStatusChanged OnPlaybackStatusChanged;
		static FOnPlaybackAssetStatusChanged OnPlaybackAssetStatusChanged;
		static FOnPlaybackSequenceEvent OnPlaybackSequenceEvent;
		static FOnPlaybackTransitionEvent OnPlaybackTransitionEvent;
	}
	
	FOnConnectionEvent& GetOnConnectionEvent()
	{
		return Private::OnConnectionEvent;
	}
	
	FOnPlaybackStatusChanged& GetOnPlaybackStatusChanged()
	{
		return Private::OnPlaybackStatusChanged;
	}
	
	FOnPlaybackAssetStatusChanged& GetOnPlaybackAssetStatusChanged()
	{
		return Private::OnPlaybackAssetStatusChanged;	
	}
	
	FOnPlaybackSequenceEvent& GetOnPlaybackSequenceEvent()
	{
		return Private::OnPlaybackSequenceEvent;
	}

	FOnPlaybackTransitionEvent& GetOnPlaybackTransitionEvent()
	{
		return Private::OnPlaybackTransitionEvent;
	}
}
