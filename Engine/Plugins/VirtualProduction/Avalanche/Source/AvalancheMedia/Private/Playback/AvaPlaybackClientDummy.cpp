// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPlaybackClientDummy.h"

const FString FAvaPlaybackClientDummy::EmptyString;
const TArray<FString> FAvaPlaybackClientDummy::EmptyStringArray;
