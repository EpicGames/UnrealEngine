// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playback/IAvaPlaybackClient.h"

class FAvaPlaybackClientDummy : public IAvaPlaybackClient
{
	static const FString EmptyString;
	static const TArray<FString> EmptyStringArray;
public:
	
	//~ Begin IAvaPlaybackClient Interface
	virtual int32 GetNumConnectedServers() const override { return 0; }
	virtual TArray<FString> GetServerNames() const override { return TArray<FString>();}
	virtual FMessageAddress GetServerAddress(const FString& InServerName) const override
	{
		FMessageAddress InvalidAddress;
		InvalidAddress.Invalidate();
		return InvalidAddress;
	}
	virtual bool HasServerUserData(const FString& InServerName, const FString& InKey) const override { return false;}
	virtual const FString& GetServerUserData(const FString& InServerName, const FString& InKey) const override { return EmptyString; }
	virtual bool HasUserData(const FString& InKey) const override { return false;}
	virtual const FString& GetUserData(const FString& InKey) const override { return EmptyString; }
	virtual void SetUserData(const FString& InKey, const FString& InData) override {}
	virtual void RemoveUserData(const FString& InKey) override {}
	virtual void BroadcastStatCommand(const FString& InCommand, bool bInBroadcastLocalState) override {}
	virtual void RequestPlaybackAssetStatus(const FSoftObjectPath& InAssetPath, const FString& InChannelOrServerName, bool bInForceRefresh) override {}
	virtual void RequestPlayback(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, EAvaPlaybackAction InAction, const FString& InArguments) override {}
	virtual void RequestAnimPlayback(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FAvaPlaybackAnimPlaySettings& InAnimSettings) override {};
	virtual void RequestAnimAction(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FString& InAnimationName, EAvaPlaybackAnimAction InAction) override {};
	virtual void RequestRemoteControlUpdate(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FAvaPlayableRemoteControlValues& InRemoteControlValues, EAvaPlayableRCUpdateFlags InFlags) override {};
	virtual void RequestPlayableTransitionStart(const FGuid& InTransitionId, TArray<FGuid>&& InEnterInstanceIds, TArray<FGuid>&& InPlayingInstanceIds, TArray<FGuid>&& InExitInstanceIds, TArray<FAvaPlayableRemoteControlValues>&& InEnterValues, const FName& InChannelName, EAvaPlayableTransitionFlags InTransitionFlags) override {}
	virtual void RequestPlayableTransitionStop(const FGuid& InTransitionId, const FName& InChannelName) override {}
	virtual void RequestBroadcast(const FString& InProfile, const FName& InChannel, const TArray<UMediaOutput*>& InRemoteMediaOutputs, EAvaBroadcastAction InAction, const FString& InServerName) override {}
	virtual bool IsMediaOutputRemoteFallback(const UMediaOutput* InMediaOutput) override { return false;}
	virtual EAvaBroadcastIssueSeverity GetMediaOutputIssueSeverity(const FString& InServerName, const FString& InChannelName, const FGuid& InOutputGuid) const override { return EAvaBroadcastIssueSeverity::None;}
	virtual const TArray<FString>& GetMediaOutputIssueMessages(const FString& InServerName, const FString& InChannelName, const FGuid& InOutputGuid) const override { return EmptyStringArray; }
	virtual EAvaBroadcastOutputState GetMediaOutputState(const FString& InServerName, const FString& InChannelName, const FGuid& InOutputGuid) const override { return EAvaBroadcastOutputState::Error; }
	virtual bool HasAnyServerOnlineForChannel(const FName& InChannelName) const override { return false;}
	virtual TArray<FString> GetOnlineServersForChannel(const FName& InChannelName) const override { return TArray<FString>(); };
   	virtual TOptional<EAvaPlaybackStatus> GetRemotePlaybackStatus(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FString& InServerName) const override
	{
		return EAvaPlaybackStatus::Unknown;
	}
	virtual const FString* GetRemotePlaybackUserData(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FString& InServerName) const override
	{
		return nullptr;
	}
	virtual TOptional<EAvaPlaybackAssetStatus> GetRemotePlaybackAssetStatus(const FSoftObjectPath& InAssetPath, const FString& InServerName) const override
	{
		return EAvaPlaybackAssetStatus::Unknown;
	}
	//~ End IAvaPlaybackClient Interface
};
