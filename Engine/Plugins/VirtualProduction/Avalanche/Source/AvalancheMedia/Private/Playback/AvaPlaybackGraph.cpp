// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/AvaPlaybackGraph.h"

#include "Async/Async.h"
#include "AvaMediaRenderTargetUtils.h"
#include "AvaMediaSettings.h"
#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/IAvaBroadcastSettings.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/AvaGameInstance.h"
#include "IAvaMediaModule.h"
#include "Misc/CoreDelegates.h"
#include "Playable/AvaPlayable.h"
#include "Playable/AvaPlayableGroup.h"
#include "Playback/AvaPlaybackManager.h"
#include "Playback/AvaPlaybackUtils.h"
#include "Playback/IAvaPlaybackGraphEditor.h"
#include "Playback/Nodes/AvaPlaybackNode.h"

#if WITH_EDITOR
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#endif

DEFINE_LOG_CATEGORY(LogAvaPlayback);

namespace UE::AvaPlaybackGraph::Private
{
	const FString& GetLoadOptions(UAvaPlaybackNode* InPlaybackNode)
	{
		if (UAvaPlaybackNodePlayer* PlayerNode = Cast<UAvaPlaybackNodePlayer>(InPlaybackNode))
		{
			return PlayerNode->GetLoadOptions();
		}
		static FString Empty;
		return Empty;
	}
}

UAvaPlaybackGraph::~UAvaPlaybackGraph()
{
	FAvaBroadcastOutputChannel::GetOnChannelChanged().RemoveAll(this);
}

void UAvaPlaybackGraph::Play()
{
	if (!FAvaBroadcastOutputChannel::GetOnChannelChanged().IsBoundToObject(this))
	{
		FAvaBroadcastOutputChannel::GetOnChannelChanged().AddUObject(this, &UAvaPlaybackGraph::OnChannelChanged);
	}

	if (!IsPlaying())
	{
		SetIsPlaying(true);
	}	
}

void UAvaPlaybackGraph::LoadInstances()
{
	// Gather the asset references from the player nodes.
	ConditionalResolvePlaybackSettings();

	for (TPair<FName, FAvaPlaybackChannelParameters>& Playback : PlaybackSettings)
	{
		// Load the assets
		for (const FAvaPlaybackPlayerParameters& Player : Playback.Value.Players)
		{
			FindOrLoadPlayable(Player.Asset, ResolveChannelName(Playback.Key), UE::AvaPlaybackGraph::Private::GetLoadOptions(Player.Node.Get()));
		}
	}
}

// Unload playables.
void UAvaPlaybackGraph::UnloadInstances(EAvaPlaybackUnloadOptions InUnloadOptions)
{
	// We need to unload all the playables, not just the ones currently traversed by the graph.
	// We keep track of all the local playables in ChannelPlayableGroups. This now includes the
	// remote playables.
	
	const bool bIsForceImmediate = EnumHasAnyFlags(InUnloadOptions, EAvaPlaybackUnloadOptions::ForceImmediate);

	for (const TPair<FName, FAvaPlaybackPlayableGroup>& ChannelPlayableGroup : ChannelPlayableGroups)
	{
		TArray<UAvaPlayable*> Playables;
		ChannelPlayableGroup.Value.GetAllPlayables(Playables);
		
		for (UAvaPlayable* Playable : Playables)
		{
			FSoftObjectPath SourceAsset = Playable->GetSourceAssetPath();
			UnloadAndRemovePlayable(Playable, SourceAsset, ChannelPlayableGroup.Key, bIsForceImmediate);
		}
	}
}

void UAvaPlaybackGraph::Stop(EAvaPlaybackStopOptions InStopOptions)
{
	const bool bIsForceImmediate = EnumHasAnyFlags(InStopOptions, EAvaPlaybackStopOptions::ForceImmediate);
	const bool bIsUnload = EnumHasAnyFlags(InStopOptions, EAvaPlaybackStopOptions::Unload);
	bool bIsShuttingDown = false;
	if (const TSharedPtr<FAvaPlaybackManager> PlaybackManager = PlaybackManagerWeak.Pin())
	{
		bIsShuttingDown = PlaybackManager->IsShuttingDown();
	}
	
	// We need to unload all the playables, not just the ones currently traversed by the graph.
	// We keep track of all the local playables in ChannelPlayableGroups. This now includes the
	// remote playables.

	for (const TPair<FName, FAvaPlaybackPlayableGroup>& ChannelPlayableGroup : ChannelPlayableGroups)
	{
		TArray<UAvaPlayable*> Playables;
		ChannelPlayableGroup.Value.GetAllPlayables(Playables);
		
		for (UAvaPlayable* Playable : Playables)
		{
			// EndPlayWorld is no longer issued here (unless shutting down) as some of the engine systems don't seem
			// to support a EndPlay, BeginPlay sequence anymore. It is unknown if this situation
			// will change in the future, so we keep the option.
			Playable->EndPlay(EAvaPlayableEndPlayOptions::None); //bIsShuttingDown ? EAvaPlayableEndPlayOptions::ConditionalEndPlayWorld : EAvaPlayableEndPlayOptions::None);

			if (bIsUnload)
			{
				FSoftObjectPath SourceAsset = Playable->GetSourceAssetPath();
				UnloadAndRemovePlayable(Playable, SourceAsset, ChannelPlayableGroup.Key, bIsForceImmediate);
			}
		}
	}

	PlaybackSettings.Reset();
	PreviousPlaybackSettings.Reset();
	SetIsPlaying(false);
}

void UAvaPlaybackGraph::SetPlaybackManager(const TSharedPtr<FAvaPlaybackManager>& InPlaybackManager)
{
	PlaybackManagerWeak = InPlaybackManager;

	if (IsManaged())
	{
		// Since the playback manager is going to manage ticking,
		// make sure to unregister the tick delegate.
		UnregisterTickDelegate();
	}
	else if (IsPlaying() && !TickDelegateHandle.IsValid())
	{
		RegisterTickDelegate();
	}
}

UAvaPlayable* UAvaPlaybackGraph::GetFirstPlayable() const
{
	for (const TPair<FName, FAvaPlaybackPlayableGroup>& ChannelPlayableGroup : ChannelPlayableGroups)
	{
		if (UAvaPlayable* Playable = ChannelPlayableGroup.Value.GetFirstPlayable())
		{
			return Playable;
		}
	}
	return nullptr;
}

void UAvaPlaybackGraph::GetAllPlayables(TArray<UAvaPlayable*>& OutPlayables) const
{
	for (const TPair<FName, FAvaPlaybackPlayableGroup>& ChannelPlayableGroup : ChannelPlayableGroups)
	{
		ChannelPlayableGroup.Value.GetAllPlayables(OutPlayables);
	}
}

void UAvaPlaybackGraph::ForEachPlayable(TFunctionRef<void(const UAvaPlayable*)> InFunction) const
{
	for (const TPair<FName, FAvaPlaybackPlayableGroup>& ChannelPlayableGroup : ChannelPlayableGroups)
	{
		ChannelPlayableGroup.Value.ForEachPlayable(InFunction);
	}
}

bool UAvaPlaybackGraph::HasPlayable(const UAvaPlayable* InPlayable) const
{
	for (const TPair<FName, FAvaPlaybackPlayableGroup>& ChannelPlayableGroup : ChannelPlayableGroups)
	{
		if (ChannelPlayableGroup.Value.HasPlayable(InPlayable))
		{
			return true;
		}
	}
	return false;
}

UAvaPlayable* UAvaPlaybackGraph::FindPlayable(const FSoftObjectPath& InSourceAssetPath, const FName& InChannelName) const
{
	const FAvaPlaybackPlayableGroup* ChannelPlayableGroup = ChannelPlayableGroups.Find(InChannelName);
	return ChannelPlayableGroup ? ChannelPlayableGroup->FindPlayable(InSourceAssetPath) : nullptr;
}

UAvaPlayable* UAvaPlaybackGraph::FindOrLoadPlayable(const FAvaSoftAssetPtr& InSourceAsset, const FName& InChannelName, const FString& InLoadOptions)
{
	FAvaPlaybackPlayableGroup* ChannelPlayableGroup = ChannelPlayableGroups.Find(InChannelName);
	if (!ChannelPlayableGroup)
	{
		ChannelPlayableGroup = &ChannelPlayableGroups.Add(InChannelName, FAvaPlaybackPlayableGroup());
	}
	
	if (const TObjectPtr<UAvaPlayable>* FoundPlayable = ChannelPlayableGroup->Playables.Find(InSourceAsset.ToSoftObjectPath()))
	{
		return *FoundPlayable;
	}

	// TODO: Extension point IAvaPlayableFactory -- Example: IMediaPlayerFactory. Using URL (> FAvaSoftAssetPath).
	UAvaPlayable* NewPlayable = UAvaPlayable::Create(this, {GetPlayableGroupManager(), InSourceAsset, InChannelName});
	if (NewPlayable)
	{
		ChannelPlayableGroup->Playables.Add(InSourceAsset.ToSoftObjectPath(), NewPlayable);
		OnPlayableCreated.Broadcast(this, NewPlayable);
		NewPlayable->LoadAsset(InSourceAsset, false, InLoadOptions);
	}
	return NewPlayable;
}

void UAvaPlaybackGraph::RemovePlayable(const FSoftObjectPath& InSourceAssetPath, const FName& InChannelName)
{
	if (FAvaPlaybackPlayableGroup* ChannelPlayableGroup = ChannelPlayableGroups.Find(InChannelName))
	{
		ChannelPlayableGroup->Playables.Remove(InSourceAssetPath);
	}
}

bool UAvaPlaybackGraph::UnloadAndRemovePlayable(UAvaPlayable* InPlayable, const FSoftObjectPath& InSourceAssetPath, const FName& InChannelName, bool bInForceImmediate)
{
	bool bWorldUnloaded = false;
	if (InPlayable)
	{
		InPlayable->UnloadAsset(); // bIsForceImmediate is not used anymore.
		InPlayable->GetPlayableGroup()->UnregisterPlayable(InPlayable);
		// If no more playables in the group, we can request unload of the world too.
		bWorldUnloaded = InPlayable->GetPlayableGroup()->ConditionalRequestUnloadWorld(bInForceImmediate);
	}	
	RemovePlayable(InSourceAssetPath, InChannelName);
	return bWorldUnloaded;
}

bool UAvaPlaybackGraph::UnloadAndRemovePlayable(const FSoftObjectPath& InSourceAssetPath, const FName& InChannelName, bool bInForceImmediate)
{
	return UnloadAndRemovePlayable(FindPlayable(InSourceAssetPath, InChannelName), InSourceAssetPath, InChannelName, bInForceImmediate);
}

TArray<UAvaGameInstance*> UAvaPlaybackGraph::GetActiveGameInstances() const
{
	TArray<UAvaGameInstance*> OutGameInstances;

	if (IsPlaying())
	{
		// TODO: revisit this. We have Active and Loaded.
		for (const TPair<FName, FAvaPlaybackChannelParameters>& Playback : PlaybackSettings)
		{
			FName ChannelName = ResolveChannelName(Playback.Key);
			for (const FAvaPlaybackPlayerParameters& Player : Playback.Value.Players)
			{
				const UAvaPlayable* Playable = FindPlayable(Player.Asset.ToSoftObjectPath(), ChannelName);
				if (Playable && Playable->GetPlayableGroup())
				{
					if (UAvaGameInstance* AvaGameInstance = Cast<UAvaGameInstance>(Playable->GetPlayableGroup()->GetGameInstance()))
					{
						OutGameInstances.AddUnique(AvaGameInstance);
					}
				}
			}
		}
	}
	return OutGameInstances;
}

bool UAvaPlaybackGraph::HasPlayerNodeForSourceAsset(const FSoftObjectPath& InSourceAssetPath) const
{
	for (const TObjectPtr<UAvaPlaybackNodePlayer>& PlayerNode : PlayerNodes)
	{
		if (PlayerNode && PlayerNode->GetAssetPath() == InSourceAssetPath)
		{
			return true;
		}
	}
	return false;
}

void UAvaPlaybackGraph::SetPlayableGroupManager(UAvaPlayableGroupManager* InPlayableGroupManager)
{
	PlayableGroupManager = InPlayableGroupManager;
}

UAvaPlayableGroupManager* UAvaPlaybackGraph::GetGlobalPlayableGroupManager()
{
	return IAvaMediaModule::Get().GetLocalPlaybackManager().GetPlayableGroupManager();	
}

void UAvaPlaybackGraph::DryRunGraph(bool bDeferredExecution)
{
	if (bDeferredExecution)
	{
		//Only async dry run if we have not requested it yet, and it's not dry running already.
		if (!bAsyncDryRunRequested && !bIsDryRunningGraph)
		{
			bAsyncDryRunRequested = true;
			
			TWeakObjectPtr<UAvaPlaybackGraph> ThisWeak(this);
			AsyncTask(ENamedThreads::GameThread, [ThisWeak]()
			{
				if (ThisWeak.IsValid())
				{
					ThisWeak->DryRunGraphInternal();
				}
			});
		}
	}
	else
	{
		DryRunGraphInternal();
	}
}

TArray<FName> UAvaPlaybackGraph::GetChannelNamesForIndices(const TArray<int32>& InChannelIndices) const
{
	TArray<FName> OutChannelNames;
	if (IsPreviewOnly())
	{
		OutChannelNames.Reserve(1);
		OutChannelNames.Add(PreviewChannelName);
	}
	else if (InChannelIndices.Num() > 0)
	{
		const UAvaBroadcast& AvaBroadcast = UAvaBroadcast::Get();
		OutChannelNames.Reserve(InChannelIndices.Num());
		for (const int32 ChannelIndex : InChannelIndices)
		{
			const FName ChannelFName = AvaBroadcast.GetChannelName(ChannelIndex);
			if (!ChannelFName.IsNone())
			{
				OutChannelNames.Add(ChannelFName);
			}
		}
	}
	return OutChannelNames;
}

void UAvaPlaybackGraph::DryRunGraphInternal()
{
	//Everytime we run this, the Task has been complete.
	bAsyncDryRunRequested = false;
	
	if (!bIsDryRunningGraph)
	{
		TGuardValue<bool> Guard(bIsDryRunningGraph, true);
		if (RootNode)
		{
			TSet<UAvaPlaybackNode*> SeenNodes;
			TArray<UAvaPlaybackNode*> RemainingNodes;
			
			auto NotifyDryRun = [this, &SeenNodes, &RemainingNodes](void(UAvaPlaybackNode::*ExecFunc)())
				{
					//Reset but not deallocate the Memory that was used when previous NotifyDryRuns were called
					RemainingNodes.Reset();
					SeenNodes.Reset();
					
					RemainingNodes.Add(RootNode);
					
					while (RemainingNodes.Num() > 0)
					{
						UAvaPlaybackNode* const Node = RemainingNodes.Pop();
						if (Node && !SeenNodes.Contains(Node))
						{
							SeenNodes.Add(Node);
							(*Node.*ExecFunc)();
							RemainingNodes.Append(Node->GetChildNodes());
						}
					}
				};

			TArray<UAvaPlaybackNode*> Ancestors;
			
			NotifyDryRun(&UAvaPlaybackNode::PreDryRun);
			RootNode->DryRunNode(Ancestors);
			NotifyDryRun(&UAvaPlaybackNode::PostDryRun);
		}
	}
}

void UAvaPlaybackGraph::RegisterTickDelegate()
{
	if (ensure(!TickDelegateHandle.IsValid()))
	{
		TickDelegateHandle = FCoreDelegates::OnEndFrame.AddUObject(this, &UAvaPlaybackGraph::OnEndFrameTick);
	}
}

void UAvaPlaybackGraph::UnregisterTickDelegate()
{
	if (TickDelegateHandle.IsValid())
	{
		FCoreDelegates::OnEndFrame.Remove(TickDelegateHandle);
		TickDelegateHandle.Reset();
	}
}

void UAvaPlaybackGraph::OnEndFrameTick()
{
	Tick(FApp::GetDeltaTime());
}

void UAvaPlaybackGraph::ConditionalResolvePlaybackSettings()
{
	if (PlaybackSettings.IsEmpty() && IsValid(RootNode))
	{
		RootNode->TickRoot(0.016f, PlaybackSettings);
	}
}

void UAvaPlaybackGraph::SetRootNode(UAvaPlaybackNodeRoot* InRoot)
{
	if (IsValid(InRoot) && !IsValid(RootNode))
	{
		RootNode = InRoot;
	}
}

void UAvaPlaybackGraph::AddPlayerNode(UAvaPlaybackNodePlayer* InPlayer)
{
	if (IsValid(InPlayer))
	{
		PlayerNodes.AddUnique(InPlayer);
	}
}

UAvaPlaybackNodeRoot* UAvaPlaybackGraph::GetRootNode() const
{
	return RootNode;
}

void UAvaPlaybackGraph::AddPlaybackNode(UAvaPlaybackNode* Node)
{
	//Note: Ensure that all the Nodes added here have Outer set to this Playback Object.
	Node->Rename(nullptr, this);

#if WITH_EDITORONLY_DATA
	PlaybackNodes.Add(Node);
#endif
}

void UAvaPlaybackGraph::RemovePlaybackNode(UAvaPlaybackNode* Node)
{
#if WITH_EDITORONLY_DATA
	PlaybackNodes.Remove(Node);
#endif
}

void UAvaPlaybackGraph::StopPlayable(const FSoftObjectPath& InSourceAssetPath, const FName& InChannelName)
{
	if (UAvaPlayable* const Playable = FindPlayable(InSourceAssetPath, InChannelName))
	{
		Playable->EndPlay(EAvaPlayableEndPlayOptions::None);
	}
}

void UAvaPlaybackGraph::LoadAsset(const FAvaSoftAssetPtr& InSourceAsset, const FName& InChannelName, const FString& InLoadOptions)
{
	FindOrLoadPlayable(InSourceAsset, InChannelName, InLoadOptions);
}

void UAvaPlaybackGraph::Tick(float DeltaTime)
{
	if (bIsTicking)
	{
		return;
	}
	TGuardValue TickGuard(bIsTicking, true);

	if (!IsValid(RootNode))
	{
		return;
	}

	// Backup previous settings so we can detect if an asset has switched and needs to be stopped.
	PreviousPlaybackSettings = PlaybackSettings;
	
	//Pass 1: Tick from Root (Channels Node) up to the Player Nodes.
	for (TPair<FName, FAvaPlaybackChannelParameters>& Playback : PlaybackSettings)
	{
		Playback.Value.Players.Reset();
	}
	RootNode->TickRoot(DeltaTime, PlaybackSettings);

	//Post Pass 1: Refresh any Playback from the Settings gotten from pass 1
	for (const TPair<FName, FAvaPlaybackChannelParameters>& Playback : PlaybackSettings)
	{
		const FName ChannelName = ResolveChannelName(Playback.Key);
		FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetChannelMutable(ChannelName);

		for (const FAvaPlaybackPlayerParameters& Player : Playback.Value.Players)
		{
			if (IsPreviewOnly())
			{
				RefreshPreview(Player.Asset, UE::AvaPlaybackGraph::Private::GetLoadOptions(Player.Node.Get()), Channel, ChannelName);
			}
			else if (Channel.IsValidChannel())
			{
				RefreshPlayback(Player.Asset, UE::AvaPlaybackGraph::Private::GetLoadOptions(Player.Node.Get()), Channel, ChannelName);
			}
		}

		// Check if the asset playing has changed (this may be the result of a switch/combiner node).
		if (const FAvaPlaybackChannelParameters* PreviousParameters = PreviousPlaybackSettings.Find(Playback.Key))
		{
			for (const FAvaPlaybackPlayerParameters& PreviousPlayer : PreviousParameters->Players)
			{
				// We still don't support multi-instance within a playback graph. The asset remains the key.
				auto IsSameAsset = [&PreviousPlayer](const FAvaPlaybackPlayerParameters& InPlayer)
				{
					return PreviousPlayer.Asset == InPlayer.Asset;
				};
				
				if (!Playback.Value.Players.FindByPredicate(IsSameAsset))
				{
					StopPlayable(PreviousPlayer.Asset.ToSoftObjectPath(), ChannelName);
				}
			}
		}
	}

	PreviousPlaybackSettings.Reset();

	//Pass 2: Tick Events from each Player Node to the Connected Event Triggers/Actions 
	for (TArray<TObjectPtr<UAvaPlaybackNodePlayer>>::TIterator Iter(PlayerNodes); Iter; ++Iter)
	{
		TObjectPtr<UAvaPlaybackNodePlayer> PlayerNode = *Iter;
		if (IsValid(PlayerNode))
		{
			PlayerNode->TickEventFeed(DeltaTime);
		}
		else
		{
			Iter.RemoveCurrent();
		}
	}

	//Post Pass 2: Reset the Event Triggers 
	for (UAvaPlaybackNodePlayer* PlayerNode : PlayerNodes)
	{
		if (IsValid(PlayerNode))
		{
			PlayerNode->ResetEvents();
		}
	}

	ExecutePendingAnimationCommands();
	ExecutePendingRemoteControlCommands();
}

void UAvaPlaybackGraph::PushAnimationCommand(const FSoftObjectPath& InSourceAssetPath, const FString& InChannelName, EAvaPlaybackAnimAction InAnimAction, const FAvaPlaybackAnimPlaySettings& InAnimSettings)
{
	const FDateTime Timeout = FDateTime::UtcNow() + FTimespan::FromSeconds(10);
	PendingAnimationCommands.Add({{Timeout, InSourceAssetPath, InChannelName, FName(InChannelName)}, InAnimAction, InAnimSettings});
}

void UAvaPlaybackGraph::PushRemoteControlValues(const FSoftObjectPath& InSourceAssetPath, const FString& InChannelName, const TSharedRef<FAvaPlayableRemoteControlValues>& InRemoteControlValues, EAvaPlayableRCUpdateFlags InFlags)
{
	const FDateTime Timeout = FDateTime::UtcNow() + FTimespan::FromSeconds(10);
	PendingRemoteControlCommands.Add({{Timeout, InSourceAssetPath, InChannelName, FName(InChannelName)}, InRemoteControlValues, InFlags});
}

void UAvaPlaybackGraph::SetupPlaybackNode(UAvaPlaybackNode* InPlaybackNode, bool bSelectNewNode)
{
	check(InPlaybackNode);

	InPlaybackNode->CreateStartingConnectors();

#if WITH_EDITOR
	// Create the graph node
	check(InPlaybackNode->GetGraphNode() == nullptr);
	if (TSharedPtr<IAvaPlaybackGraphEditor> GraphEditor = GetGraphEditor())
	{
		GraphEditor->SetupPlaybackNode(EdGraph, InPlaybackNode, bSelectNewNode);
	}
#endif
}

#if WITH_EDITOR
const TArray<TObjectPtr<UAvaPlaybackNode>>& UAvaPlaybackGraph::GetPlaybackNodes() const
{
	return PlaybackNodes;
}

void UAvaPlaybackGraph::CreateGraph()
{
	TSharedPtr<IAvaPlaybackGraphEditor> GraphEditor = GetGraphEditor();
	if (EdGraph == nullptr && GraphEditor.IsValid())
	{
		EdGraph = GraphEditor->CreatePlaybackGraph(this);
        EdGraph->bAllowDeletion = false;
        
        // Give the schema a chance to fill out any required nodes (like the results node)
        const UEdGraphSchema* const Schema = EdGraph->GetSchema();
        Schema->CreateDefaultNodesForGraph(*EdGraph);
	}
}

void UAvaPlaybackGraph::ClearGraph()
{
	if (EdGraph)
	{
		EdGraph->Nodes.Empty();
		
		// Give the schema a chance to fill out any required nodes (like the results node)
		const UEdGraphSchema* const Schema = EdGraph->GetSchema();
		Schema->CreateDefaultNodesForGraph(*EdGraph);
	}
}

void UAvaPlaybackGraph::RefreshPlaybackNode(UAvaPlaybackNode* InPlaybackNode)
{
	if (TSharedPtr<IAvaPlaybackGraphEditor> GraphEditor = GetGraphEditor())
	{
		if (InPlaybackNode && InPlaybackNode->GetGraphNode())
		{
			GraphEditor->RefreshNode(*InPlaybackNode->GetGraphNode());
		}
	}
}

void UAvaPlaybackGraph::CompilePlaybackNodesFromGraphNodes()
{
	if (TSharedPtr<IAvaPlaybackGraphEditor> GraphEditor = GetGraphEditor())
	{
		GraphEditor->CompilePlaybackNodesFromGraphNodes(this);
	}
}

UEdGraph* UAvaPlaybackGraph::GetGraph()
{
	return EdGraph;
}

void UAvaPlaybackGraph::ResetGraph()
{
	for (const UAvaPlaybackNode* const PlaybackNode : PlaybackNodes)
	{
		EdGraph->RemoveNode(PlaybackNode->GetGraphNode());
	}

	PlaybackNodes.Reset();
}

void UAvaPlaybackGraph::SetGraphEditor(TSharedPtr<IAvaPlaybackGraphEditor> InGraphEditor)
{
	GraphEditorWeak = InGraphEditor;
}

TSharedPtr<IAvaPlaybackGraphEditor> UAvaPlaybackGraph::GetGraphEditor() const
{
	return GraphEditorWeak.Pin();
}
#endif

void UAvaPlaybackGraph::ExecutePendingAnimationCommands()
{
	if (PendingAnimationCommands.IsEmpty())
	{
		return;
	}

	const UAvaBroadcast& AvaBroadcast = UAvaBroadcast::Get();

	// If commands can't be executed because the assets are still
	// streaming, we will accumulate them in this array for next time.
	// Note: preserves the order of the commands (ie. not a map or set).
	TArray<FAnimationCommand> StillPendingAnimationCommands;

	const FDateTime CurrentTime = FDateTime::UtcNow();

	for (FAnimationCommand& AnimCommand : PendingAnimationCommands)
	{
		if (AnimCommand.HasTimedOut(CurrentTime))
		{
			UE_LOGF(LogAvaPlayback, Warning,
				"%ls Animation command for asset \"%ls\" on channel \"%ls\" timed out and is discarded.",
				*UE::AvaPlayback::Utils::GetBriefFrameInfo(), *AnimCommand.SourcePath.ToString(), *AnimCommand.ChannelName);
			continue;
		}

		FAvaPlaybackAnimPlaySettings& AnimSettings = AnimCommand.AnimPlaySettings;
		AnimSettings.Action = AnimCommand.AnimAction; // TODO: get rid of the AnimAction in 2 places.
		
		UAvaPlayable* Playable = FindPlayable(AnimCommand.SourcePath, AnimCommand.ChannelFName);
		if (!Playable)
		{
			continue;
		}
		
		const EAvaPlayableCommandResult Result = Playable->ExecuteAnimationCommand(AnimCommand.AnimAction, AnimCommand.AnimPlaySettings);
		if (Result == EAvaPlayableCommandResult::KeepPending)
		{
			StillPendingAnimationCommands.Add(AnimCommand);
		}
	}
	PendingAnimationCommands = StillPendingAnimationCommands;
}

void UAvaPlaybackGraph::ExecutePendingRemoteControlCommands()
{
	if (PendingRemoteControlCommands.IsEmpty())
	{
		return;
	}

	// If commands can't be executed because the assets are still
	// streaming, we will accumulate them in this array for next time.
	// Note: preserves the order of the commands (ie. not a map or set).
	TArray<FRemoteControlCommand> StillPendingRemoteControlCommands;

	const FDateTime CurrentTime = FDateTime::UtcNow();
	
	for (const FRemoteControlCommand& RemoteControlCommand : PendingRemoteControlCommands)
	{
		if (RemoteControlCommand.HasTimedOut(CurrentTime))
		{
			UE_LOGF(LogAvaPlayback, Warning,
				"%ls Remote Control command for asset \"%ls\" on channel \"%ls\" timed out and is discarded.",
				*UE::AvaPlayback::Utils::GetBriefFrameInfo(), *RemoteControlCommand.SourcePath.ToString(), *RemoteControlCommand.ChannelName);
			continue;
		}
		
		UAvaPlayable* Playable = FindPlayable(RemoteControlCommand.SourcePath, RemoteControlCommand.ChannelFName);
		if (!Playable)
		{
			continue;
		}

		const EAvaPlayableCommandResult Result = Playable->UpdateRemoteControlCommand(RemoteControlCommand.Values, RemoteControlCommand.Flags);
		if (Result == EAvaPlayableCommandResult::KeepPending)
		{
			StillPendingRemoteControlCommands.Add(RemoteControlCommand);
		}
	}
	PendingRemoteControlCommands = StillPendingRemoteControlCommands;
}

void UAvaPlaybackGraph::OnChannelChanged(const FAvaBroadcastOutputChannel& InChannel, EAvaBroadcastChannelChange InChange)
{
	if (EnumHasAnyFlags(InChange, EAvaBroadcastChannelChange::State))
	{
		OnChannelBroadcastStateChanged(InChannel);
	}
	
	if (EnumHasAnyFlags(InChange, EAvaBroadcastChannelChange::Settings))
	{
		// Need to propagate the channel settings to loaded/playing game instances.
		if (const FAvaPlaybackPlayableGroup* ChannelPlayableGroup = ChannelPlayableGroups.Find(InChannel.GetChannelName()))
		{
			TSet<const UAvaPlayableGroup*> PlayableGroups; 
			ChannelPlayableGroup->ForEachPlayable([&PlayableGroups](const UAvaPlayable* InPlayable)
			{
				if (const UAvaPlayableGroup* PlayableGroup = InPlayable->GetPlayableGroup())
				{
					PlayableGroups.Add(PlayableGroup);
				}
			});
			for (const UAvaPlayableGroup* PlayableGroup : PlayableGroups)
			{
				if (const UAvaGameInstance* GameInstance = Cast<UAvaGameInstance>(PlayableGroup->GetGameInstance()))
				{
					if (UAvaGameViewportClient* ViewportClient = GameInstance->GetAvaGameViewportClient())
					{
						FAvaViewportQualitySettings QualitySettingsMutable = InChannel.GetViewportQualitySettings(); 
						QualitySettingsMutable.Apply(ViewportClient->EngineShowFlags);
					}
				}
			}
		}
	}
}

void UAvaPlaybackGraph::OnChannelBroadcastStateChanged(const FAvaBroadcastOutputChannel& InChannel)
{
	if (IsPlaying() && InChannel.IsValidChannel() && InChannel.GetState() == EAvaBroadcastChannelState::Live)
	{
		const FName ChangedChannelName = InChannel.GetChannelName();
		FAvaBroadcastProfile& Profile = UAvaBroadcast::Get().GetCurrentProfile();
		for (const TPair<FName, FAvaPlaybackChannelParameters>& Playback : PlaybackSettings)
		{
			const FName ChannelName = ResolveChannelName(Playback.Key);
			
			// Ensure to route the preview into it's channel regardless of pin connections.
			// The preview channel(s) don't have corresponding pins.
			if (ChannelName == ChangedChannelName)
			{
				FAvaBroadcastOutputChannel& Channel = Profile.GetChannelMutable(ChannelName);
				for (const FAvaPlaybackPlayerParameters& Player : Playback.Value.Players)
				{
					RefreshPlayback(Player.Asset, UE::AvaPlaybackGraph::Private::GetLoadOptions(Player.Node.Get()), Channel, ChannelName);
				}
			}
		}
	}
}

void UAvaPlaybackGraph::SetIsPlaying(bool bInIsPlaying)
{
	if (bIsPlaying != bInIsPlaying)
	{
		bIsPlaying = bInIsPlaying;
		if (bInIsPlaying && !IsManaged())
		{
			RegisterTickDelegate();
		}
#if WITH_EDITOR
		for (const TObjectPtr<UAvaPlaybackNode>& Node : PlaybackNodes)
		{
			if (Node)
			{
				Node->NotifyPlaybackStateChanged(bInIsPlaying);
			}
		}
#else
		RootNode->NotifyPlaybackStateChanged(bInIsPlaying);
#endif
		
		OnPlaybackStateChanged.Broadcast(bInIsPlaying);
	}

	if (!bInIsPlaying)
	{
		UnregisterTickDelegate();
	}
}

// Remark: very similar to FAvaBroadcastOutputChannel::IsChannelAvailableForPlayback, could we possibly merge the logic?
bool UAvaPlaybackGraph::CanRefreshPlayback(const FAvaSoftAssetPtr& InSourceAsset, const FAvaBroadcastOutputChannel& InChannel) const
{
	return IsPlaying() && InChannel.IsValidChannel() && !InSourceAsset.IsNull();
}

bool UAvaPlaybackGraph::RefreshPlayback(const FAvaSoftAssetPtr& InSourceAsset, const FString& InLoadOptions, FAvaBroadcastOutputChannel& InChannel, const FName& InChannelName)
{
	if (!CanRefreshPlayback(InSourceAsset, InChannel))
	{
		InChannel.UpdateRenderTarget(nullptr, nullptr);
		InChannel.UpdateAudioDevice(FAudioDeviceHandle());
		return false;
	}
	
	if (UAvaPlayable* const Playable = FindOrLoadPlayable(InSourceAsset, InChannelName, InLoadOptions))
	{
		UTextureRenderTarget2D* const RenderTarget = UpdatePlaybackRenderTarget(Playable, InChannel);

		const FIntPoint ViewportSize = RenderTarget
			? UE::AvaMediaRenderTargetUtils::GetRenderTargetSize(RenderTarget)
			: InChannel.DetermineRenderTargetSize();

		const FAvaInstancePlaySettings WorldPlaySettings =
		{ IAvaMediaModule::Get().GetAvaInstanceSettings(), InChannelName, RenderTarget, ViewportSize, InChannel.GetViewportQualitySettings() };

		Playable->BeginPlay(WorldPlaySettings);

		InChannel.UpdateAudioDevice(Playable->GetPlayWorld() ? Playable->GetPlayWorld()->GetAudioDevice() : FAudioDeviceHandle());
		InChannel.UpdateRenderTarget(Playable->GetPlayableGroup(), RenderTarget);

		return Playable->IsPlaying();
	}

	// Fallback
	InChannel.UpdateRenderTarget(nullptr, nullptr);
	InChannel.UpdateAudioDevice(FAudioDeviceHandle());
	return false;
}

bool UAvaPlaybackGraph::CanRefreshPreview(const FAvaSoftAssetPtr& InSourceAsset) const
{
	return IsPlaying() && !InSourceAsset.IsNull();
}

bool UAvaPlaybackGraph::RefreshPreview(const FAvaSoftAssetPtr& InSourceAsset, const FString& InLoadOptions, FAvaBroadcastOutputChannel& InChannel, const FName& InChannelName)
{
	// If the preview channel has been setup, go through the normal channel refresh.
	if (InChannel.IsValidChannel())
	{
		return RefreshPlayback(InSourceAsset, InLoadOptions, InChannel, InChannelName);
	}

	if (!CanRefreshPreview(InSourceAsset))
	{
		return false;
	}

	UAvaPlayable* const Playable = FindOrLoadPlayable(InSourceAsset, InChannelName, InLoadOptions);

	if (!Playable || !Playable->GetPlayableGroup())
	{
		return false;
	}
	
	UTextureRenderTarget2D* RenderTarget = Playable->GetPlayableGroup()->GetManagedRenderTarget();

	if (!RenderTarget)
	{
		// The render target is created here for now.
		// TODO: adjust to preview window's size.
		static const FName PreviewRenderTargetBaseName = TEXT("AvaPlayback_PreviewRenderTarget");
		RenderTarget = UE::AvaMediaRenderTargetUtils::CreateDefaultRenderTarget(PreviewRenderTargetBaseName);
		UE::AvaMediaRenderTargetUtils::UpdateRenderTarget(RenderTarget,
			UAvaMediaSettings::Get().PreviewDefaultResolution,
			FAvaBroadcastOutputChannel::GetDefaultMediaOutputFormat(),
			IAvaMediaModule::Get().GetBroadcastSettings().GetChannelClearColor());

		Playable->GetPlayableGroup()->SetManagedRenderTarget(RenderTarget);
	}
	
	const FIntPoint ViewportSize = UE::AvaMediaRenderTargetUtils::GetRenderTargetSize(RenderTarget);

	static const FAvaViewportQualitySettings DefaultPreviewQualitySettings;
	const FAvaInstancePlaySettings WorldPlaySettings =
		{ IAvaMediaModule::Get().GetAvaInstanceSettings(), InChannelName, RenderTarget, ViewportSize, DefaultPreviewQualitySettings };

	Playable->BeginPlay(WorldPlaySettings);
	
	return Playable->IsPlaying();
}

UTextureRenderTarget2D* UAvaPlaybackGraph::UpdatePlaybackRenderTarget(const UAvaPlayable* InPlayable
	, const FAvaBroadcastOutputChannel& InChannel)
{
	UTextureRenderTarget2D* RenderTarget = nullptr;
	
	if (InChannel.IsValidChannel())
	{
		// Preferably use the channel's placeholder RT to avoid having the MediaCapture
		// switch and possibly cause rendering glitches.
		RenderTarget = InChannel.GetPlaceholderRenderTarget();

		// Only fallback to internal RTs if we can't get one from the channels.
		if (!IsValid(RenderTarget))
		{
			const FName ChannelName = InChannel.GetChannelName();

			// See if we have a managed render target for this asset in this channel.
			RenderTarget = InPlayable->GetPlayableGroup()->GetManagedRenderTarget();
			
			if (!IsValid(RenderTarget))
			{
				static const FName PlaybackRenderTargetBaseName = TEXT("AvaPlayback_RenderTarget");
				RenderTarget = UE::AvaMediaRenderTargetUtils::CreateDefaultRenderTarget(PlaybackRenderTargetBaseName);

				// This render target is now managed by the playable's instance group.
				InPlayable->GetPlayableGroup()->SetManagedRenderTarget(RenderTarget);
			}
		}
		
		check(RenderTarget);

		UE::AvaMediaRenderTargetUtils::UpdateRenderTarget(RenderTarget,
			InChannel.DetermineRenderTargetSize(),
			InChannel.DetermineRenderTargetFormat(),
			IAvaMediaModule::Get().GetBroadcastSettings().GetChannelClearColor());
	}
	
	return RenderTarget;
}

FAvaPlaybackGraphBuilder::FAvaPlaybackGraphBuilder(UAvaPlayableGroupManager* InPlayableGroupManager)
{
	Playback = NewObject<UAvaPlaybackGraph>();
	Playback->SetPlayableGroupManager(InPlayableGroupManager);
	
	// Construct Root
	RootNode = Playback->ConstructPlaybackNode<UAvaPlaybackNodeRoot>();
}

FAvaPlaybackGraphBuilder::~FAvaPlaybackGraphBuilder()
{
	FinishBuilding();
}

bool FAvaPlaybackGraphBuilder::ConnectToRoot(const FString& InChannelName, UAvaPlaybackNode* InNodeToConnect)
{
	if (!InNodeToConnect)
	{
		return false;
	}
	
	// Connect Player Node to Root Node
	TArray<UAvaPlaybackNode*> NewRootChildNodes;
	NewRootChildNodes.AddZeroed(RootNode->GetMaxChildNodes());

	// Copy existing children.
	const TArray<TObjectPtr<UAvaPlaybackNode>>& ExistingChildren = RootNode->GetChildNodes();
	for (int32 PinIndex = 0; PinIndex < ExistingChildren.Num(); ++PinIndex)
	{
		if (NewRootChildNodes.IsValidIndex(PinIndex))
		{
			NewRootChildNodes[PinIndex] = ExistingChildren[PinIndex];
		}
	}

	// Get the pin index corresponding to the given channel name.
	const int32 ConnectionPinIndex = GetPinIndexForChannel(InChannelName);

	// Check that Index is valid. RootNode Max Child Nodes should be the Channel Names Count,
	// and the Connection Pin Index should be >= 0.
	if (NewRootChildNodes.IsValidIndex(ConnectionPinIndex))
	{
		NewRootChildNodes[ConnectionPinIndex] = InNodeToConnect;
		RootNode->SetChildNodes(MoveTemp(NewRootChildNodes));
		bFinished = false;
		return true;
	}
	else
	{
		UE_LOGF(LogAvaPlayback, Error
			, "Using index %d for channel \"%ls\" is invalid. Root node has %d children. The node \"%ls\" could not be added."
			, ConnectionPinIndex, *InChannelName, NewRootChildNodes.Num(), *InNodeToConnect->GetNodeDisplayNameText().ToString());
	}
	return false;	
}

int32 FAvaPlaybackGraphBuilder::GetPinIndexForChannel(const FString& InChannelName) const
{
	int32 ConnectionPinIndex = UAvaBroadcast::Get().GetChannelIndex(*InChannelName);

	// We could return nullptr here, but given that this is a "GetOrCreate",
	// the most sensible thing is to just connect to first index and continue
	if (ConnectionPinIndex == INDEX_NONE)
	{
		// For the default channel key, we expect it to not have a broadcast channel, so we don't give the warning.
		if (InChannelName != UE::AvaMedia::Playback::DefaultPreviewChannelName.ToString())
		{
			UE_LOGF(LogAvaPlayback, Warning
				, "No channel with name (%ls) was found. Connecting player node to first channel available"
				, *InChannelName);
		}
		ConnectionPinIndex = 0;
	}
	return ConnectionPinIndex;
}

UAvaPlaybackGraph* FAvaPlaybackGraphBuilder::FinishBuilding()
{
	if (Playback && !bFinished)
	{
		Playback->DryRunGraph();
		bFinished = true;
	}
	return Playback;
}
