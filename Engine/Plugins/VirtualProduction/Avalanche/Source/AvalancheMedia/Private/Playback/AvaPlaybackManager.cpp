// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/AvaPlaybackManager.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Broadcast/AvaBroadcast.h"
#include "Engine/Engine.h"
#include "Framework/AvaGameInstance.h"
#include "IAvaMediaModule.h"
#include "IAvaModule.h"
#include "Playable/AvaPlayable.h"
#include "Playable/AvaPlayableGroup.h"
#include "Playable/AvaPlayableGroupManager.h"
#include "Playback/AvaPlaybackUtils.h"
#include "Playback/Nodes/AvaPlaybackNodeLevelPlayer.h"
#include "Playback/Transition/AvaPlaybackTransition.h"
#include "UnrealClient.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"

DEFINE_LOG_CATEGORY_STATIC(LogAvaPlaybackManager, Log, All);

namespace UE::AvaPlaybackManager::Private
{
	inline bool ShouldForcePurgePackage(EAvaPlaybackPackageEventFlags InEventFlags)
	{
		// There is one asset per package, if it is deleted, the package will be deleted too,
		// eventually. When the event is received, the package is unlikely to be deleted yet.
		// If the only cause for this event is an asset deletion, we will force unload the package
		// even if the file may still be there.
		return EnumHasAnyFlags(InEventFlags, EAvaPlaybackPackageEventFlags::AssetDeleted)
		 && !EnumHasAnyFlags(InEventFlags, EAvaPlaybackPackageEventFlags::Saved);
	}
	
	// Used to check validity of FAvaPlaybackInstance's playback object to ensure
	// it is not pending kill or destroyed (might happen in Engine Shutdown).
	inline bool IsPlaybackValid(const TObjectPtr<UAvaPlaybackGraph>& InPlayback)
	{
		return IsValid(InPlayback.Get())
				&& InPlayback->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed) == false;
	}

	static bool IsPackageIgnored(const FString& InPackageName)
	{
		// Filter out any references outside of /Game.
		return !InPackageName.StartsWith(TEXT("/Game"));
	}

	/** Utility to walk a package's dependencies. */
	struct FPackageDependencyWalker
	{
		int MaxRecursion;
		bool bTrackMissingDependencies;
		TSet<FName> AllDependencies;
		TArray<FName> MissingDependencies;
		
		FPackageDependencyWalker(int InMaxRecursion, bool bInTrackMissingDependencies)
			: MaxRecursion(InMaxRecursion)
			, bTrackMissingDependencies(bInTrackMissingDependencies)
		{
			AllDependencies.Reserve(64);
			if (bInTrackMissingDependencies)
			{
				MissingDependencies.Reserve(8);
			}
		}

		/** Returns true if the asset has all of it's (non-ignored) dependencies. */
		bool CheckDependencies(const IAssetRegistry& InAssetRegistry, const FName& InPackageName)
		{
			AllDependencies.Reset();
			MissingDependencies.Reset();
			return CheckDependencies(InAssetRegistry, InPackageName, 0);
		}

	private:
		/** Returns true if the asset has all of it's (non-ignored) dependencies. */
		bool CheckDependencies(const IAssetRegistry& InAssetRegistry, const FName& InPackageName, int InRecursion)
		{
			TArray<FName> Dependencies;
			InAssetRegistry.GetDependencies(InPackageName, Dependencies);
		
			// We do a breath first pass, determine if any of the direct dependencies are missing.
			for (const FName& Dependency : Dependencies)
			{
				if (!IsPackageIgnored(Dependency.ToString()) && !FPackageName::DoesPackageExist(Dependency.ToString()))
				{
					if (bTrackMissingDependencies)
					{
						MissingDependencies.AddUnique(Dependency);
					}
					
					UE_LOGF(LogAvaPlaybackManager, Verbose, "Package \"%ls\" doesn't exist (dependency of \"%ls\").",
						*Dependency.ToString(), *InPackageName.ToString());

					return false;
				}
			}

			if (InRecursion < MaxRecursion)
			{
				// Depth pass
				for (const FName& Dependency : Dependencies)
				{
					// Note: we keep track of the dependencies we already visited and don't check them again.
					// Not doing this can lead to infinite recursion because assets have circular dependencies (apparently).
					if (!IsPackageIgnored(Dependency.ToString()) && !AllDependencies.Contains(Dependency))
					{
						AllDependencies.Add(Dependency);
						if (!CheckDependencies(InAssetRegistry, Dependency, InRecursion + 1))
						{
							UE_LOGF(LogAvaPlaybackManager, Verbose, "Package \"%ls\" (dependency of \"%ls\") has missing dependencies.",
								*Dependency.ToString(), *InPackageName.ToString());
							return false;
						}
					}
				}
			}
			else
			{
				if (MaxRecursion != 0)
				{
					UE_LOGF(LogAvaPlaybackManager, Warning, "Reached maximum recursion (%d) while evaluating dependencies on package \"%ls\".",
						InRecursion, *InPackageName.ToString());
				}
			}
			return true;
		}
	};

	FString GetPrettyPlaybackInstanceInfo(const FAvaPlaybackInstance* InPlaybackInstance)
	{
		if (InPlaybackInstance)
		{
			return FString::Printf(TEXT("Id:%s, Asset:%s, Channel:%s, UserData:\"%s\""),
				*InPlaybackInstance->GetInstanceId().ToString(),
				*InPlaybackInstance->GetSourcePath().GetAssetName(),
				*InPlaybackInstance->GetChannelName(),
				*InPlaybackInstance->GetInstanceUserData());
		}
		return TEXT("");
	}
}

FAvaPlaybackManager::FAvaPlaybackManager()
	: PlayableGroupManager(NewObject<UAvaPlayableGroupManager>())
{
	PlayableGroupManager->Init();
	UPackage::PackageSavedWithContextEvent.AddRaw(this, &FAvaPlaybackManager::OnPackageSaved);
	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		AssetRegistry->OnAssetRemoved().AddRaw(this, &FAvaPlaybackManager::OnAssetRemoved);
	}
	IAvaMediaModule::Get().GetOnAvaMediaSyncPackageModified().AddRaw(this, &FAvaPlaybackManager::OnAvaSyncPackageModified);
	FCoreDelegates::OnEndFrame.AddRaw(this, &FAvaPlaybackManager::Tick);
}

FAvaPlaybackManager::~FAvaPlaybackManager()
{
	UPackage::PackageSavedWithContextEvent.RemoveAll(this);

	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		AssetRegistry->OnAssetRemoved().RemoveAll(this);
	}
	IAvaMediaModule::Get().GetOnAvaMediaSyncPackageModified().RemoveAll(this);
	FCoreDelegates::OnEndFrame.RemoveAll(this);
	
	StopAllPlaybacks(true);
	PlayableGroupManager->Shutdown();
}

void FAvaPlaybackManager::Tick()
{
	double DeltaSeconds = FApp::GetDeltaTime();

	OnBeginTick.Broadcast(DeltaSeconds);
	
	// Update the status of the playback objects, propagate any changed status to listeners.
	ForAllPlaybackInstances([DeltaSeconds, this](FAvaPlaybackInstance& InPlaybackInstance)
	{
		if (InPlaybackInstance.IsPlaying())
		{
			InPlaybackInstance.GetPlayback()->Tick(DeltaSeconds);
		}
		
		if (InPlaybackInstance.UpdateStatus())
		{
			OnPlaybackInstanceStatusChanged.Broadcast(InPlaybackInstance);
		}
	});

	// Tick the playable groups after the playback graphs so that playable groups are created for this tick.
	if (GetPlayableGroupManager())
	{
		GetPlayableGroupManager()->Tick(DeltaSeconds);
	}

	if (PendingStartTransitions.Num())
	{
		TArray<TWeakObjectPtr<UAvaPlaybackTransition>> StillPendingStartTransitions;
		StillPendingStartTransitions.Reserve(PendingStartTransitions.Num());

		for (TWeakObjectPtr<UAvaPlaybackTransition>& TransitionToStart : PendingStartTransitions)
		{
			if (TransitionToStart.IsValid())
			{
				bool bShouldDiscard = false;
				if (TransitionToStart->CanStart(bShouldDiscard))
				{
					TransitionToStart->Start();
				}
				else if (!bShouldDiscard)
				{
					StillPendingStartTransitions.Add(TransitionToStart);
				}
			}
		}
		PendingStartTransitions = StillPendingStartTransitions;
	}
	
}

TSharedPtr<FAvaPlaybackInstance> FAvaPlaybackManager::AcquirePlaybackInstance(const FSoftObjectPath& InAssetPath, const FString& InChannelName) const
{
	const TSharedPtr<FAvaPlaybackSourceAssetEntry>* AssetEntry = PlaybackAssetEntries.Find(InAssetPath);
	if (AssetEntry && *AssetEntry)
	{
		return (*AssetEntry)->AcquirePlaybackInstance(InChannelName);
	}
	return TSharedPtr<FAvaPlaybackInstance>();
}

TSharedPtr<FAvaPlaybackInstance> FAvaPlaybackManager::LoadPlaybackInstance(const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FString& InLoadOptions)
{
	TSharedPtr<FAvaPlaybackInstance> PlaybackInstance;
	
	if (UAvaPlaybackGraph* Playback = LoadPlaybackObject(InAssetPath, InChannelName, InLoadOptions))
	{
		FGuid NewInstanceId = FGuid::NewGuid();
		
		PlaybackInstance = MakeShared<FAvaPlaybackInstance>(NewInstanceId, InAssetPath, InChannelName, Playback);
		if (const TSharedPtr<FAvaPlaybackSourceAssetEntry> AssetEntry = GetPlaybackAssetEntry(InAssetPath))
		{
			Playback->SetPlaybackManager(SharedThis(this));
			PlaybackInstance->AssetEntryWeak = AssetEntry.ToWeakPtr();
			AssetEntry->UsedInstances.Add(PlaybackInstance);
		}
	}
	return PlaybackInstance;
}

TSharedPtr<FAvaPlaybackInstance> FAvaPlaybackManager::AcquireOrLoadPlaybackInstance(const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FString& InLoadOptions)
{
	TSharedPtr<FAvaPlaybackInstance> PlaybackInstance = AcquirePlaybackInstance(InAssetPath, InChannelName);
	if (!PlaybackInstance)
	{
		PlaybackInstance = LoadPlaybackInstance(InAssetPath, InChannelName, InLoadOptions);
	}
	return PlaybackInstance;
}

TSharedPtr<FAvaPlaybackInstance> FAvaPlaybackManager::FindPlaybackInstance(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName) const
{
	if (const TSharedPtr<FAvaPlaybackSourceAssetEntry> AssetEntry = FindPlaybackAssetEntry(InAssetPath))
	{
		if (InInstanceId.IsValid())
		{
			return AssetEntry->FindPlaybackInstance(InInstanceId);
		}
		// Only use the channel if the instance id is not specified.
		return AssetEntry->FindPlaybackInstanceForChannel(InChannelName);
	}
	return TSharedPtr<FAvaPlaybackInstance>();
}

bool FAvaPlaybackManager::UnloadPlaybackInstances(const FSoftObjectPath& InAssetPath, const FString& InChannelName)
{
	bool bFoundInstance = false;
	if (const TSharedPtr<FAvaPlaybackSourceAssetEntry> AssetEntry = FindPlaybackAssetEntry(InAssetPath))
	{
		for (TArray<TSharedPtr<FAvaPlaybackInstance>>::TIterator InstanceIt(AssetEntry->AvailableInstances); InstanceIt; ++InstanceIt)
		{
			if ( (*InstanceIt)->GetChannelName() == InChannelName || InChannelName.IsEmpty())
			{
				(*InstanceIt)->AssetEntryWeak.Reset();
				(*InstanceIt)->Unload();
				InstanceIt.RemoveCurrent();
				bFoundInstance = true;
			}
		}
	}
	return bFoundInstance;
}

void FAvaPlaybackManager::ForAllPlaybackInstances(TFunctionRef<void(FAvaPlaybackInstance& /*InPlaybackInstance*/)> InFunction)
{
	using namespace UE::AvaPlaybackManager::Private;
	for (TPair<FSoftObjectPath, TSharedPtr<FAvaPlaybackSourceAssetEntry>>& AssetEntry : PlaybackAssetEntries)
	{
		if (AssetEntry.Value)
		{
			AssetEntry.Value->ForAllPlaybackInstances(InFunction);
		}
	}
}

EAvaPlaybackAssetStatus FAvaPlaybackManager::GetLocalAssetStatus(const FName& InPackageName)
{
	// Fast check in cached results
	if (const EAvaPlaybackAssetStatus* CachedStatusEntry = CachedAssetStatus.Find(InPackageName))
	{
		return (*CachedStatusEntry);
	}

	EAvaPlaybackAssetStatus AssetStatus = EAvaPlaybackAssetStatus::Missing;
	
	if (FPackageName::DoesPackageExist(InPackageName.ToString()))
	{
		// Remark: Checking for dependencies might be optional. It is not particularly reliable.
		if (const IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
		{
			constexpr int MaxRecursion = 4;						// Todo: expose in config?
			constexpr bool bTrackMissingDependencies = false;	// This is used for debugging only (for now).
			UE::AvaPlaybackManager::Private::FPackageDependencyWalker DependencyWalker(MaxRecursion, bTrackMissingDependencies);
			const bool bHasAllDependencies = DependencyWalker.CheckDependencies(*AssetRegistry, InPackageName);
			AssetStatus = bHasAllDependencies ? EAvaPlaybackAssetStatus::Available : EAvaPlaybackAssetStatus::MissingDependencies;
		}
		else
		{
			AssetStatus = EAvaPlaybackAssetStatus::Available;
		}
	}

	CachedAssetStatus.Add(InPackageName, AssetStatus);
	return AssetStatus;
}

void FAvaPlaybackManager::InvalidateCachedLocalAssetStatus(const FName& InPackageName)
{
	CachedAssetStatus.Remove(InPackageName);
}

void FAvaPlaybackManager::InvalidatePlaybackAssetEntry(const FSoftObjectPath& InAssetPath)
{
	if (const TSharedPtr<FAvaPlaybackSourceAssetEntry> AssetEntry = GetPlaybackAssetEntry(InAssetPath))
	{
		AssetEntry->ForAllPlaybackInstances([this](FAvaPlaybackInstance& InPlaybackInstance)
		{
			OnPlaybackInstanceInvalidated.Broadcast(InPlaybackInstance);
		});
	}
	PlaybackAssetEntries.Remove(InAssetPath);
}

UAvaPlaybackGraph* FAvaPlaybackManager::LoadPlaybackObject(const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FString& InLoadOptions) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAvaPlaybackManager::LoadPlaybackObject);
	
	const FString PackageName = InAssetPath.GetLongPackageName();

	// Fast path: avoid a sync package load for maps.
	// The generated playback graph will load it async using a level streaming playable.
	if (FAvaPlaybackUtils::IsMapAsset(PackageName))
	{
		UAvaPlaybackGraph* const AvaPlayback = BuildPlaybackFromWorld(TSoftObjectPtr<UWorld>(InAssetPath), InChannelName, InLoadOptions);
		check(AvaPlayback);
		return AvaPlayback;
	}
	
	UObject* LoadedAsset = InAssetPath.ResolveObject();
	if (!LoadedAsset)
	{
		LoadedAsset = InAssetPath.TryLoad();
		if (!LoadedAsset)
		{
			UE_LOGF(LogAvaPlaybackManager, Error, "Failed to load asset \"%ls\"", *InAssetPath.ToString());
			return nullptr;
		}
	}
	
	// When the asset is a Playback Graph, it is used directly.
	// Warning: this is not an instance, but the asset itself. Will not work for multi-instancing (TL).
	if (UAvaPlaybackGraph* const AvaPlayback = Cast<UAvaPlaybackGraph>(LoadedAsset))
	{
		return AvaPlayback;
	}

	if (UWorld* const World = Cast<UWorld>(LoadedAsset))
	{
		UAvaPlaybackGraph* const AvaPlayback = BuildPlaybackFromWorld(World, InChannelName, InLoadOptions);
		check(AvaPlayback);
		return AvaPlayback;
	}

	UE_LOGF(LogAvaPlaybackManager, Error,
		"Asset \"%ls\" is not a supported Motion Design playback asset (\"%ls\").",
		*InAssetPath.ToString(), *LoadedAsset->GetClass()->GetFullName());
	
	return nullptr;
}

UAvaPlaybackGraph* FAvaPlaybackManager::BuildPlaybackFromWorld(const TSoftObjectPtr<UWorld>& InWorld, const FString& InChannelName, const FString& InLoadOptions) const
{
	using namespace UE::AvaPlaybackManager::Private;
	FAvaPlaybackGraphBuilder GraphBuilder(GetPlayableGroupManager());

	// Construct Player Node and assign the World that we passed in
	UAvaPlaybackNodeLevelPlayer* const PlayerNode = GraphBuilder.ConstructPlaybackNode<UAvaPlaybackNodeLevelPlayer>();
	PlayerNode->SetAsset(InWorld);
	PlayerNode->SetLoadOptions(InLoadOptions);
	
	GraphBuilder.ConnectToRoot(InChannelName, PlayerNode);
	return GraphBuilder.FinishBuilding();
}

TArray<FSoftObjectPath> FAvaPlaybackManager::StopAllPlaybacks(bool bInUnload)
{
	using namespace UE::AvaPlaybackManager::Private;
	TArray<FSoftObjectPath> StoppedPlaybackObjectPaths;
	
	if (UObjectInitialized())
	{
		const EAvaPlaybackStopOptions PlaybackStopOptions = GetPlaybackStopOptions(bInUnload);
		const EAvaPlaybackUnloadOptions PlaybackUnloadOptions = GetPlaybackUnloadOptions();
		for (TPair<FSoftObjectPath, TSharedPtr<FAvaPlaybackSourceAssetEntry>> AssetEntry : PlaybackAssetEntries)
		{
			if (AssetEntry.Value)
			{
				StoppedPlaybackObjectPaths.Add(AssetEntry.Key);
				AssetEntry.Value->ForAllPlaybackInstances([PlaybackStopOptions, PlaybackUnloadOptions, bInUnload](FAvaPlaybackInstance& InPlaybackInstance)
				{
					if (InPlaybackInstance.IsPlaying())
					{
						InPlaybackInstance.Playback->Stop(PlaybackStopOptions);
						InPlaybackInstance.Status = EAvaPlaybackStatus::Loaded;
					}
					else if (bInUnload)
					{
						InPlaybackInstance.Playback->UnloadInstances(PlaybackUnloadOptions);
						InPlaybackInstance.Status = EAvaPlaybackStatus::Available;
					}
				});
			}
		}
	}

	if (bInUnload)
	{
		PlaybackAssetEntries.Empty();
	}
	return StoppedPlaybackObjectPaths;
}

bool FAvaPlaybackManager::PushAnimationCommand(const FGuid& InInstanceId, const FSoftObjectPath& InSourcePath, const FString& InChannelName, EAvaPlaybackAnimAction InAction, const FAvaPlaybackAnimPlaySettings& InAnimSettings)
{
	bool bAnimationCommandPushed = false;
	if (InInstanceId.IsValid())
	{
		// Check if we have the entry directly in the map. Rundown path.
		if (const TSharedPtr<FAvaPlaybackInstance> PlaybackInstance = FindPlaybackInstance(InInstanceId, InSourcePath, InChannelName))
		{
			PlaybackInstance->GetPlayback()->PushAnimationCommand(InSourcePath, InChannelName, InAction, InAnimSettings);
			bAnimationCommandPushed = true;
		}
	}
	else
	{
		// If we had a playback asset (not from the rundown path), check if it is present in one of the playback instances.
		ForAllPlaybackInstances([InSourcePath, InChannelName, InAction, InAnimSettings, &bAnimationCommandPushed](FAvaPlaybackInstance& InPlaybackInstance)
		{
			if (InPlaybackInstance.GetPlayback()->HasPlayerNodeForSourceAsset(InSourcePath))
			{
				InPlaybackInstance.GetPlayback()->PushAnimationCommand(InSourcePath, InChannelName, InAction, InAnimSettings);
				bAnimationCommandPushed = true;
			}
		});
	}
	
	if (!bAnimationCommandPushed && bEnablePlaybackCommandsBuffering)
	{
		FString CommandBufferKey = MakeCommandBufferKey(InInstanceId, InSourcePath, InChannelName);
		GetOrCreatePlaybackCommandBuffers(MoveTemp(CommandBufferKey)).AnimationCommands.Add({InAction, InAnimSettings});
	}
	return bAnimationCommandPushed;
}

bool FAvaPlaybackManager::PushRemoteControlCommand(const FGuid& InInstanceId, const FSoftObjectPath& InSourcePath, const FString& InChannelName, const TSharedRef<FAvaPlayableRemoteControlValues>& InRemoteControlValues, EAvaPlayableRCUpdateFlags InFlags)
{
	bool bRemoteControlValuesPushed = false;
	if (InInstanceId.IsValid())
	{
		// Check if we have the entry directly in the map. Rundown path.
		if (const TSharedPtr<FAvaPlaybackInstance> PlaybackInstance = FindPlaybackInstance(InInstanceId, InSourcePath, InChannelName))
		{
			PlaybackInstance->GetPlayback()->PushRemoteControlValues(InSourcePath, InChannelName, InRemoteControlValues, InFlags);
			bRemoteControlValuesPushed = true;
		}
	}
	else
	{
		// If we had a playback asset (not from the rundown path), check if it is present in one of the playback instances.
		ForAllPlaybackInstances([InSourcePath, InChannelName, InRemoteControlValues, &bRemoteControlValuesPushed, InFlags](FAvaPlaybackInstance& InPlaybackInstance)
		{
			if (InPlaybackInstance.GetPlayback()->HasPlayerNodeForSourceAsset(InSourcePath))
			{
				InPlaybackInstance.GetPlayback()->PushRemoteControlValues(InSourcePath, InChannelName, InRemoteControlValues, InFlags);
				bRemoteControlValuesPushed = true;
			}
		});
	}
	
	if (!bRemoteControlValuesPushed && bEnablePlaybackCommandsBuffering)
	{
		FString CommandBufferKey = MakeCommandBufferKey(InInstanceId, InSourcePath, InChannelName);
		GetOrCreatePlaybackCommandBuffers(MoveTemp(CommandBufferKey)).RemoteControlCommands.Add({InRemoteControlValues, InFlags});
	}
	return bRemoteControlValuesPushed;
}

bool FAvaPlaybackManager::PushPlaybackTransitionStartCommand(UAvaPlaybackTransition* InTransitionToStart)
{
	PendingStartTransitions.AddUnique(InTransitionToStart);
	return true;
}

void FAvaPlaybackManager::ApplyPendingCommands(UAvaPlaybackGraph* InPlaybackObject, const FGuid& InInstanceId, const FSoftObjectPath& InSourcePath, const FString& InChannelName)
{
	if (!InPlaybackObject || PlaybackObjectCommandBuffers.IsEmpty())
	{
		return;
	}
	
	const FString CommandBufferKey = MakeCommandBufferKey(InInstanceId, InSourcePath, InChannelName);
	
	if (const FPlaybackObjectCommandBuffers* Commands = GetPlaybackCommandBuffers(CommandBufferKey))
	{
		for (const FPlaybackObjectCommandBuffers::FAnimationCommand& AnimCommand : Commands->AnimationCommands)
		{
			InPlaybackObject->PushAnimationCommand(InSourcePath, InChannelName, AnimCommand.AnimAction, AnimCommand.AnimPlaySettings);
		}
		for (const FPlaybackObjectCommandBuffers::FRemoteControlCommand& RCCommand : Commands->RemoteControlCommands)
		{
			InPlaybackObject->PushRemoteControlValues(InSourcePath, InChannelName, RCCommand.Values, RCCommand.UpdateFlags);
		}
		PlaybackObjectCommandBuffers.Remove(CommandBufferKey);
	}
}

void FAvaPlaybackManager::OnPackageModified(const FName& InPackageName, EAvaPlaybackPackageEventFlags InFlags)
{
	InvalidateCachedLocalAssetStatus(InPackageName);
	
	TArray<UAvaPlayableGroup*> PlayableGroupsToFlush;
	
	// Invalidate corresponding assets from that package.
	for (TPair<FSoftObjectPath, TSharedPtr<FAvaPlaybackSourceAssetEntry>>& AssetEntry : PlaybackAssetEntries)
	{
		if (AssetEntry.Value)
		{
			const FName SourcePackageFName = AssetEntry.Key.GetLongPackageFName();
			if (SourcePackageFName == InPackageName)
			{
				AssetEntry.Value->ForAllPlaybackInstances([this, SourcePackageFName](FAvaPlaybackInstance& InPlaybackInstance)
				{
					OnPlaybackInstanceInvalidated.Broadcast(InPlaybackInstance);
					UE_LOGF(LogAvaPlaybackManager, Log,
						"Package \"%ls\" being touched caused asset \"%ls\" to be invalidated.",
						*SourcePackageFName.ToString(), *InPlaybackInstance.SourcePath.ToString());

					// Disconnect from the cache so it doesn't get recycled.
					InPlaybackInstance.AssetEntryWeak.Reset();
				});
				
				// Get rid of all "available" instances. Instances in use will not be flushed yet.
				AssetEntry.Value->ForAllPlaybackInstances([this, SourcePackageFName, &PlayableGroupsToFlush](FAvaPlaybackInstance& InPlaybackInstance)
				{
					InPlaybackInstance.AssetEntryWeak.Reset();
					if (InPlaybackInstance.GetPlayback())
					{
						TArray<UAvaPlayable*> Playables;
						InPlaybackInstance.GetPlayback()->GetAllPlayables(Playables);
						for (const UAvaPlayable* Playable : Playables)
						{
							PlayableGroupsToFlush.AddUnique(Playable->GetPlayableGroup());
						}
					}
					InPlaybackInstance.Unload();	// this should lead to calling UnloadAsset().
				}, true, false);
				AssetEntry.Value->AvailableInstances.Reset();
			}
		}
	}

	// The following steps are necessary to get rid of the level streaming in time
	// for the corresponding level package to properly unload. In the "available" pool,
	// playables and their group may not be ticking, thus the need to do this here.
	for (const UAvaPlayableGroup* PlayableGroup : PlayableGroupsToFlush)
	{
		// Unloading the playables should set the level streaming to "should unload", but
		// we need to update parent world's level streaming in order for that to happen.
		if (PlayableGroup->GetPlayWorld())
		{
			PlayableGroup->GetPlayWorld()->UpdateLevelStreaming();	// Attempt at getting the levels unloaded.
		}
	}

	if (PlayableGroupsToFlush.Num() > 0)
	{
		// This will call FLevelStreamingGCHelper::PrepareStreamedOutLevelForGC to really get rid of the level instances.
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}

	// Modified package by external process will need to be reloaded.
	if (EnumHasAnyFlags(InFlags, EAvaPlaybackPackageEventFlags::External))
	{		
		if (UPackage* ExistingPackage = FindPackage(nullptr, *InPackageName.ToString()))
		{
			// TODO: Investigate if it is better to batch the package purges/reloads to reduce overhead (gc once).
			
			using namespace UE::AvaPlaybackManager::Private;
			
			// Check if the package was deleted, if not, reload it.
			if (FAvaPlaybackUtils::IsPackageDeleted(ExistingPackage) || ShouldForcePurgePackage(InFlags))
			{
				UE_LOGF(LogAvaPlaybackManager, Log, "Loaded package \"%ls\" being deleted on an external process requires a purge.", *InPackageName.ToString());
				FAvaPlaybackUtils::PurgePackages({ExistingPackage});
			}
			else
			{
				UE_LOGF(LogAvaPlaybackManager, Log, "Loaded package \"%ls\" being touched on an external process requires a reload.", *InPackageName.ToString());
				FAvaPlaybackUtils::ReloadPackages({ExistingPackage});
			}
		}
	}	
}

void FAvaPlaybackManager::OnParentWorldBeginTearDown()
{
	ForAllPlaybackInstances([](FAvaPlaybackInstance& InPlaybackInstance)
	{
		if (InPlaybackInstance.Playback->IsPlaying())
		{
			InPlaybackInstance.Playback->Stop(EAvaPlaybackStopOptions::ForceImmediate | EAvaPlaybackStopOptions::Unload);
		}
		else
		{
			InPlaybackInstance.Playback->UnloadInstances(EAvaPlaybackUnloadOptions::ForceImmediate);
		}
	});
	
	PlaybackAssetEntries.Empty();
	PlaybackObjectCommandBuffers.Empty();
	PlayableGroupManager->Shutdown();
}

bool FAvaPlaybackManager::HandleStatCommand(const TArray<FString>& InArgs)
{
	check(!InArgs.IsEmpty());

	// We want to replicate the normal viewport client logic, only fallback to Motion Design if nothing else is available.
	FCommonViewportClient* ViewportClient = GStatProcessingViewportClient ? GStatProcessingViewportClient : GEngine->GameViewport.Get();
	// Use a delegate to fetch the editor viewport from the corresponding editor module (if present).
	IAvaMediaModule::Get().GetEditorViewportClientDelegate().ExecuteIfBound(&ViewportClient);

	bool bPreviousAvaModuleRuntimeStatProcessingEnabled = false;
	
	// Fallback to a viewport from Motion Design playback.
	if (!ViewportClient)
	{
		bool bFound = false;
		// Look in the current playback objects.
		for (TPair<FSoftObjectPath, TSharedPtr<FAvaPlaybackSourceAssetEntry>>& AssetEntry : PlaybackAssetEntries)
		{
			ForAllPlaybackInstances([&ViewportClient](FAvaPlaybackInstance& InPlaybackInstance)
			{
				const TArray<UAvaGameInstance*> ActiveGameInstances = InPlaybackInstance.Playback->GetActiveGameInstances();
				for (const UAvaGameInstance* GameInstance : ActiveGameInstances)
				{
					if (IsValid(GameInstance->GetWorld()) && IsValid(GameInstance->GetAvaGameViewportClient()))
					{
						ViewportClient = GameInstance->GetAvaGameViewportClient();
						break;						
					}
				}
			});
			if (ViewportClient)
			{
				// Indicate that the stats are managed by the AvaModule since it is an Motion Design viewport client.
				bPreviousAvaModuleRuntimeStatProcessingEnabled = IAvaModule::Get().SetRuntimeStatProcessingEnabled(true);
				bFound = true;
				break;
			}
		}
		if (!bFound)
		{
			UE_LOGF(LogAvaPlaybackManager, Warning, "No Active Motion Design Game Instances found to apply the stat '%ls' command on.", *InArgs[0]);
		}
	}
	
	if (ViewportClient)
	{
		FCommonViewportClient* PreviousStatProcessingViewportClient = GStatProcessingViewportClient;
		GStatProcessingViewportClient = ViewportClient;

		FString StatCommand = TEXT("STAT ");
		StatCommand += InArgs[0];
		GEngine->Exec(ViewportClient->GetWorld(), *StatCommand, *GLog);

		GStatProcessingViewportClient = PreviousStatProcessingViewportClient;
		IAvaModule::Get().SetRuntimeStatProcessingEnabled(bPreviousAvaModuleRuntimeStatProcessingEnabled);
		return true;
	}

	// This is going to work for simple stats, but not stat groups such as "detailed".
	IAvaModule& AvaModule = IAvaModule::Get();
	AvaModule.SetRuntimeStatEnabled(*InArgs[0], !AvaModule.IsRuntimeStatEnabled(InArgs[0]));

	// Indicate we didn't find a viewport client to execute the stat command.
	// The current enabled stats is thus probably not the desired state and deemed unreliable.
	// If a connected server can run the command with an active viewport, the results will
	// be back propagated to the client and will be used instead,
	// see FAvaPlaybackClient::HandleStatStatus.
	return false;
}

bool FAvaPlaybackManager::IsPlaybackAsset(const FAssetData& InAssetData)
{
	return FAvaPlaybackUtils::IsPlayableAsset(InAssetData) || InAssetData.IsInstanceOf(StaticClass<UAvaPlaybackGraph>());
}

void FAvaPlaybackManager::OnPackageSaved(const FString& InPackageFileName, UPackage* InPackage, FObjectPostSaveContext InObjectSaveContext)
{
	// Only execute if this is a user save
	if (InObjectSaveContext.IsProceduralSave())
	{
		return;
	}

	OnPackageModified(InPackage->GetFName());
}

void FAvaPlaybackManager::OnAvaSyncPackageModified(IAvaMediaSyncProvider* InAvaMediaSyncProvider, const FName& InPackageName)
{
	UE_LOGF(LogAvaPlaybackManager, Verbose,
		"A sync operation has touched the package \"%ls\" on disk. Playback manager notified.",
		*InPackageName.ToString());

	OnPackageModified(InPackageName);
}

void FAvaPlaybackManager::OnAssetRemoved(const FAssetData& InAssetData)
{
	// Invalidate the internal cache for the given package.
	OnPackageModified(InAssetData.PackageName);

	// If the asset removed is a playback asset, broadcast event.
	if (IsPlaybackAsset(InAssetData))
	{
		OnLocalPlaybackAssetRemoved.Broadcast(InAssetData.ToSoftObjectPath());
	}
}

FAvaPlaybackManager::FPlaybackObjectCommandBuffers& FAvaPlaybackManager::GetOrCreatePlaybackCommandBuffers(FString&& InPlaybackKey)
{
	if (FPlaybackObjectCommandBuffers* ObjectCommands = PlaybackObjectCommandBuffers.Find(InPlaybackKey))
	{
		return *ObjectCommands;
	}
	return PlaybackObjectCommandBuffers.Emplace(MoveTemp(InPlaybackKey));
}

const FAvaPlaybackManager::FPlaybackObjectCommandBuffers* FAvaPlaybackManager::GetPlaybackCommandBuffers(const FString& InPlaybackKey) const
{
	return PlaybackObjectCommandBuffers.Find(InPlaybackKey);
}

FAvaPlaybackInstance::FAvaPlaybackInstance(const FGuid& InInstanceId, const FSoftObjectPath& InSourcePath, const FString& InChannelName, UAvaPlaybackGraph* InPlayback)
	: InstanceId(InInstanceId)
	, ChannelName(InChannelName)
	, ChannelFName(InChannelName)
	, SourcePath(InSourcePath)
	, Playback(InPlayback)
	, Status(EAvaPlaybackStatus::Loaded)
{
	if (Playback)
	{
		// Playables may not be created yet, in which case, InstanceId will be propagated by OnPlayableCreated. 
		if (UAvaPlayable* Playable = Playback->FindPlayable(SourcePath, ChannelFName))
		{
			Playable->SetInstanceId(InstanceId);
		}
		Playback->OnPlayableCreated.AddRaw(this, &FAvaPlaybackInstance::OnPlayableCreated);
	}
}

FAvaPlaybackInstance::~FAvaPlaybackInstance()
{
	if (Playback)
	{
		Playback->OnPlayableCreated.RemoveAll(this);
	}
}

void FAvaPlaybackInstance::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject( Playback );
}

FString FAvaPlaybackInstance::GetReferencerName() const
{
	return TEXT("FAvaPlaybackInstance");
}

void FAvaPlaybackInstance::SetInstanceId(const FGuid& InInstanceId)
{
	InstanceId = InInstanceId;
	if (Playback)
	{
		// Playables may not be created yet, in which case, InstanceId will be propagated by OnPlayableCreated. 
		if (UAvaPlayable* Playable = Playback->FindPlayable(SourcePath, ChannelFName))
		{
			Playable->SetInstanceId(InstanceId);
		}
	}
}

void FAvaPlaybackInstance::SetInstanceUserData(const FString& InUserData)
{
	InstanceUserData = InUserData;
	if (Playback)
	{
		// Playables may not be created yet, in which case, InstanceUserData will be propagated by OnPlayableCreated. 
		if (UAvaPlayable* Playable = Playback->FindPlayable(SourcePath, ChannelFName))
		{
			Playable->SetUserData(InUserData);
		}
	}
}

bool FAvaPlaybackInstance::UpdateStatus()
{
	using namespace UE::AvaPlaybackManager::Private;
	using namespace UE::AvaPlayback::Utils;
	
	if (!IsPlaybackValid(Playback))
	{
		return false;
	}
	
	EAvaPlaybackStatus NewStatus = Status;

	if (Playback->IsPlaying())
	{
		if (const UAvaPlayable* Playable = Playback->FindPlayable(SourcePath, ChannelFName))
		{
			switch (Playable->GetPlayableStatus())
			{
			case EAvaPlayableStatus::Unloaded:
				NewStatus = EAvaPlaybackStatus::Available;
				break;
			case EAvaPlayableStatus::Loading:
				NewStatus = EAvaPlaybackStatus::Loading;
				break;
			case EAvaPlayableStatus::Loaded:
				NewStatus = EAvaPlaybackStatus::Loaded;
				break;
			case EAvaPlayableStatus::Visible:
				NewStatus = Playable->GetPlayableGroup()->IsRenderTargetReady() ? EAvaPlaybackStatus::Started : EAvaPlaybackStatus::Starting;
				break;
			}
		}
		else
		{
			// The game instance may not be created yet. The RefreshPlayback is done on the next tick.
			NewStatus = EAvaPlaybackStatus::Loading;

			// Reporting possible edge case:
			if (Status != NewStatus)
			{
				UE_LOGF(LogAvaPlaybackManager, Verbose,
					"%ls Playback Instance {%ls} Status Changed: %ls -> %ls because Playable \"%ls\" (%ls) was not found in PLAYING instance.",
					*GetBriefFrameInfo(), *GetPrettyPlaybackInstanceInfo(this), *StaticEnumToString(Status), *StaticEnumToString(NewStatus),
					*SourcePath.GetAssetName(), *ChannelName);
			}
		}
	}
	else
	{
		// Even if not playing, we could have a game instance already, it can be preloaded now.
		if (const UAvaPlayable* Playable = Playback->FindPlayable(SourcePath, ChannelFName))
		{
			switch (Playable->GetPlayableStatus())
			{
			case EAvaPlayableStatus::Unloaded:
				NewStatus = EAvaPlaybackStatus::Available;
				break;
			case EAvaPlayableStatus::Loading:
				NewStatus = EAvaPlaybackStatus::Loading;
				break;
			case EAvaPlayableStatus::Loaded:
			case EAvaPlayableStatus::Visible:
				NewStatus = EAvaPlaybackStatus::Loaded;
				break;
			}
		}
		else
		{
			NewStatus = EAvaPlaybackStatus::Loading;
			
			// Reporting possible edge case:
			if (Status != NewStatus)
			{
				UE_LOGF(LogAvaPlaybackManager, Verbose,
					"%ls Playback Instance {%ls} Status Changed: %ls -> %ls because Playable \"%ls\" (%ls) was not found in instance.",
					*GetBriefFrameInfo(), *GetPrettyPlaybackInstanceInfo(this), *StaticEnumToString(Status), *StaticEnumToString(NewStatus),
					*SourcePath.GetAssetName(), *ChannelName);
			}
		}
	}

	if (NewStatus != Status)
	{
		Status = NewStatus;
		return true;
	}
	
	return false;
}

void FAvaPlaybackInstance::Unload()
{
	const TSharedPtr<FAvaPlaybackManager> Manager = GetManager();

	if (Playback->IsPlaying())
	{
		Playback->Stop(Manager ? Manager->GetPlaybackStopOptions(true) : EAvaPlaybackStopOptions::Unload);
	}
	else
	{
		Playback->UnloadInstances(Manager ? Manager->GetPlaybackUnloadOptions() : EAvaPlaybackUnloadOptions::None);
	}

	// Detach from cache.
	if (const TSharedPtr<FAvaPlaybackSourceAssetEntry> AssetEntry = AssetEntryWeak.Pin())
	{
		AssetEntry->DiscardInstance(this);
	}
}

void FAvaPlaybackInstance::Recycle()
{
	if (const TSharedPtr<FAvaPlaybackSourceAssetEntry> AssetEntry = AssetEntryWeak.Pin())
	{
		AssetEntry->RecycleInstance(this);
	}
}


TSharedPtr<FAvaPlaybackManager> FAvaPlaybackInstance::GetManager() const
{
	const TSharedPtr<FAvaPlaybackSourceAssetEntry> AssetEntry = AssetEntryWeak.Pin();
	return AssetEntry ? AssetEntry->GetManager() :  TSharedPtr<FAvaPlaybackManager>();
}

void FAvaPlaybackInstance::OnPlayableCreated(UAvaPlaybackGraph* InPlayback, UAvaPlayable* InPlayable)
{
	// Remark: The playable's source asset is not yet set at this point.
	if (InPlayable)
	{
		InPlayable->SetInstanceId(InstanceId);
		InPlayable->SetUserData(InstanceUserData);
	}
}

TSharedPtr<FAvaPlaybackInstance> FAvaPlaybackSourceAssetEntry::FindPlaybackInstance(const FGuid& InInstanceId,
	bool bInAvailableInstances, bool bInUsedInstances) const
{
	return FindPlaybackInstanceByPredicate([InInstanceId](const FAvaPlaybackInstance& InPlaybackInstance) ->bool
	{
		return InPlaybackInstance.GetInstanceId() == InInstanceId;
	}, bInAvailableInstances, bInUsedInstances);
}

TSharedPtr<FAvaPlaybackInstance> FAvaPlaybackSourceAssetEntry::FindPlaybackInstanceForChannel(const FString& InChannelName,
	bool bInAvailableInstances, bool bInUsedInstances) const
{
	return FindPlaybackInstanceByPredicate([InChannelName](const FAvaPlaybackInstance& InPlaybackInstance) ->bool
	{
		return InPlaybackInstance.GetChannelName() == InChannelName;
	}, bInAvailableInstances, bInUsedInstances);
}

TSharedPtr<FAvaPlaybackInstance> FAvaPlaybackSourceAssetEntry::FindPlaybackInstanceByPredicate(TFunctionRef<bool(const FAvaPlaybackInstance& /*InPlaybackInstance*/)> InPredicate,
	bool bInAvailableInstances, bool bInUsedInstances) const
{
	using namespace UE::AvaPlaybackManager::Private;
	if (bInAvailableInstances)
	{
		for (const TSharedPtr<FAvaPlaybackInstance>& PlaybackInstance : AvailableInstances)
		{
			if (PlaybackInstance && IsPlaybackValid(PlaybackInstance->GetPlayback()))
			{
				if (InPredicate(*PlaybackInstance))
				{
					return PlaybackInstance;
				}
			}
		}
	}
	if (bInUsedInstances)
	{
		for (const TWeakPtr<FAvaPlaybackInstance>& PlaybackInstanceWeak : UsedInstances)
		{
			TSharedPtr<FAvaPlaybackInstance> PlaybackInstance = PlaybackInstanceWeak.Pin();
			if (PlaybackInstance && IsPlaybackValid(PlaybackInstance->GetPlayback()))
			{
				if (InPredicate(*PlaybackInstance))
				{
					return PlaybackInstance;
				}
			}
		}
	}
	return TSharedPtr<FAvaPlaybackInstance>();
}

void FAvaPlaybackSourceAssetEntry::ForAllPlaybackInstances(TFunctionRef<void(FAvaPlaybackInstance& /*InPlaybackInstance*/)> InFunction,
	bool bInAvailableInstances, bool bInUsedInstances)
{
	using namespace UE::AvaPlaybackManager::Private;
	if (bInAvailableInstances)
	{
		for (TSharedPtr<FAvaPlaybackInstance>& PlaybackInstance : AvailableInstances)
		{
			if (PlaybackInstance && IsPlaybackValid(PlaybackInstance->GetPlayback()))
			{
				InFunction(*PlaybackInstance);
			}
		}
	}
	if (bInUsedInstances)
	{
		for (TWeakPtr<FAvaPlaybackInstance>& PlaybackInstanceWeak : UsedInstances)
		{
			TSharedPtr<FAvaPlaybackInstance> PlaybackInstance = PlaybackInstanceWeak.Pin();
			if (PlaybackInstance && IsPlaybackValid(PlaybackInstance->GetPlayback()))
			{
				InFunction(*PlaybackInstance);
			}
		}
	}
}

void FAvaPlaybackSourceAssetEntry::DiscardInstance(FAvaPlaybackInstance* InInstanceToRemove)
{
	{
		const int32 Index = AvailableInstances.IndexOfByPredicate([InInstanceToRemove](const TSharedPtr<FAvaPlaybackInstance>& InInstance)
		{
			return InInstance.Get() == InInstanceToRemove;
		});
		if (Index != INDEX_NONE)
		{
			AvailableInstances.RemoveAtSwap(Index);
			return;
		}
	}
	{
		const int32 Index = UsedInstances.IndexOfByPredicate([InInstanceToRemove](const TWeakPtr<FAvaPlaybackInstance>& InInstanceWeak)
		{
			const TSharedPtr<FAvaPlaybackInstance> Instance = InInstanceWeak.Pin();
			return Instance.Get() == InInstanceToRemove;
		});
		if (Index != INDEX_NONE)
		{
			UsedInstances.RemoveAtSwap(Index);
		}
	}
}

void FAvaPlaybackSourceAssetEntry::RecycleInstance(FAvaPlaybackInstance* InInstanceToRecycle)
{
	const int32 Index = UsedInstances.IndexOfByPredicate([InInstanceToRecycle](const TWeakPtr<FAvaPlaybackInstance>& InInstanceWeak)
	{
		const TSharedPtr<FAvaPlaybackInstance> Instance = InInstanceWeak.Pin();
		return Instance.Get() == InInstanceToRecycle;
	});
	if (Index != INDEX_NONE)
	{
		const TSharedPtr<FAvaPlaybackInstance> InstanceToRecycle = UsedInstances[Index].Pin();
		UsedInstances.RemoveAtSwap(Index);
		AvailableInstances.Add(InstanceToRecycle);
	}
}

