// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "Broadcast/AvaBroadcastDefines.h"
#include "Broadcast/Channel/AvaBroadcastMediaOutputInfo.h"
#include "Framework/AvaInstanceSettings.h"
#include "PixelFormat.h"
#include "Playable/AvaPlayableRemoteControlValues.h"
#include "Playable/AvaPlayableSettings.h"
#include "Playback/Nodes/Events/Actions/AvaPlaybackAnimations.h"
#include "Viewport/AvaViewportQualitySettings.h"
#include "AvaPlaybackMessages.generated.h"

USTRUCT()
struct FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	FString ClientName;
};

/**
 * Notes:
 * - ServerName vs HostName
 * The Host name is the physical computer device the playback server runs on.
 * A Server name is what the playback server is called and it is not necessarily the Host name.
 * If there are multiple servers on the same host, then each server must have a different name.
 * The Server name is a unique key to identify a server, while the host name is not.
 */
USTRUCT()
struct FAvaPlaybackServerMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	FString ServerName;
};

/**
 * Request published by client to discover servers.
 */
USTRUCT()
struct FAvaPlaybackPing : public FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()

	/** Indicate if the ping was generated on a timer (auto) or manually sent through a user command. */
	UPROPERTY()
	bool bAutoPing = true;

	/** Defines the interval in seconds between each client's pings. */
	UPROPERTY()
	float PingIntervalSeconds = 0.0f;
};

/**
 *	Response sent by server to client to be discovered.
 */
USTRUCT()
struct FAvaPlaybackPong : public FAvaPlaybackServerMessageBase
{
	GENERATED_BODY()

	/** Indicate if this is a reply from an auto ping. */
	UPROPERTY()
	bool bAutoPong = true;

	/** Server may request client info in case of a reconnection event. */
	UPROPERTY()
	bool bRequestClientInfo = false;

	/** Server's project content path. */
	UPROPERTY()
	FString ProjectContentPath;

	/** Server's process id on the current host. */
	UPROPERTY()
	uint32 ProcessId = 0;
};

/**
 *	Replicate server's log messages.
 */
USTRUCT()
struct FAvaPlaybackLog : public FAvaPlaybackServerMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	FString Text;

	UPROPERTY()
	uint8 Verbosity = 0;

	UPROPERTY()
	FName Category;

	UPROPERTY()
	double Time = 0.0;
};

/**
 * Request sent by client to replicate it's information on the destination server.
 */
USTRUCT()
struct FAvaPlaybackUpdateClientInfo : public FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	FString ComputerName;

	UPROPERTY()
	FString ProjectContentPath;

	UPROPERTY()
	uint32 ProcessId = 0;
};

/**
 * Request sent by client to replicate it's user data on the destination server.
 */
USTRUCT()
struct FAvaPlaybackUpdateClientUserData : public FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FString, FString> UserDataEntries;
};

/**
 * Request sent by server to replicate it's user data to the client.
 */
USTRUCT()
struct FAvaPlaybackUpdateServerUserData : public FAvaPlaybackServerMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FString, FString> UserDataEntries;
};

USTRUCT()
struct FAvaPlaybackStatCommand : public FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	FString Command;

	/**
	 * If the local client couldn't get a viewport client to run the stat command, the
	 * local state will not be reliable. In this case, the server state will be used.
	 */
	UPROPERTY()
	bool bClientStateReliable = false;
	
	/**
	 * Because the commands are toggles, there may be desync of states between client and server(s).
	 * In order to correct that we also send a list of enabled states on the client so that servers can
	 * ensure they have the same states enabled.
	 */
	UPROPERTY()
	TArray<FString> ClientEnabledRuntimeStats;
};

/**
 * Server's response when a stat command is received.
 */
USTRUCT()
struct FAvaPlaybackStatStatus : public FAvaPlaybackServerMessageBase
{
	GENERATED_BODY()

	/**
	 * Replication of the same field from the command.
	 * This is used by the client to replicate the server state in case
	 * the command succeeded on the server and it had an unreliable state.
	 */
	UPROPERTY()
	bool bClientStateReliable = false;
	
	/** Indicate if the server could execute the command. */
	UPROPERTY()
	bool bCommandSucceeded = false;

	/** Resulting enabled stats from the command. */
	UPROPERTY()
	TArray<FString> EnabledRuntimeStats;
};

/**
 * Request sent by client to obtain the device provider data from destination server.
 * The device provider data contains all available devices and their configuration
 * installed on the server.
 */
USTRUCT()
struct FAvaPlaybackDeviceProviderDataRequest : public FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()
};

/**
 *	Request for the client to replicate it's Motion Design instance settings to the server.
 **/
USTRUCT()
struct FAvaPlaybackInstanceSettingsUpdate : public FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	FAvaInstanceSettings InstanceSettings;
};

/**
 *	Request for the client to replicate it's Motion Design instance settings to the server.
 **/
USTRUCT()
struct FAvaPlaybackPlayableSettingsUpdate : public FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	FAvaPlayableSettings PlayableSettings;
};

UENUM()
enum class EAvaPlaybackPackageEvent
{
	None,
	PreSave,
	PostSave,
	AssetDeleted
};

/**
 *	Message sent by the client to inform servers that a local package has been modified by an event.
 **/
USTRUCT()
struct FAvaPlaybackPackageEvent : public FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	FName PackageName;

	UPROPERTY()
	EAvaPlaybackPackageEvent Event = EAvaPlaybackPackageEvent::None;
};

/**
 * Request by a client to obtain the status of an asset with the given path.
 * Note: since this is an asset on disk, there is no channel name.
 */
USTRUCT()
struct FAvaPlaybackAssetStatusRequest : public FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()
	
	UPROPERTY()
	FSoftObjectPath AssetPath;
	
	/**
	 * If this is true, the server will perform a full asset status compare with the client.
	 * Otherwise, the server may use locally cached status for the given asset.
	 */
	UPROPERTY()
	bool bForceRefresh = false;
};

/** Response from the server to a playback asset status request. */
USTRUCT()
struct FAvaPlaybackAssetStatus : public FAvaPlaybackServerMessageBase
{
	GENERATED_BODY()
	
	UPROPERTY()
	FSoftObjectPath AssetPath;
	
	UPROPERTY()
	EAvaPlaybackAssetStatus Status = EAvaPlaybackAssetStatus::Unknown;
};

/**
 * Playback command for a given asset.
 */
USTRUCT()
struct FAvaPlaybackCommand
{
	GENERATED_BODY()

	/** Playback's instance Id. */
	UPROPERTY()
	FGuid InstanceId;

	/** Reference to the asset to playback. */
	UPROPERTY()
	FSoftObjectPath AssetPath;

	/** Broadcast channel the action will apply to. */
	UPROPERTY()
	FString ChannelName;
	
	/** Action to do: Load, Play, Stop, Unload, etc. */
	UPROPERTY()
	EAvaPlaybackAction Action = EAvaPlaybackAction::None;

	/** Command additional arguments. */
	UPROPERTY()
	FString Arguments;
};

/**
 * Request by a client to execute a batch playback of commands on the connected servers.
 */
USTRUCT()
struct FAvaPlaybackRequest : public FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()
	
	UPROPERTY()
	TArray<FAvaPlaybackCommand> Commands;
};

/** Response from the server to a playback request. */
USTRUCT()
struct FAvaPlaybackStatus : public FAvaPlaybackServerMessageBase
{
	GENERATED_BODY()

	/** Playback's instance Id. */
	UPROPERTY()
	FGuid InstanceId;
	
	UPROPERTY()
	FSoftObjectPath AssetPath;

	UPROPERTY()
	FString ChannelName;
	
	UPROPERTY()
	EAvaPlaybackStatus Status = EAvaPlaybackStatus::Unknown;
	
	UPROPERTY()
	bool bValidUserData = false;
	
	UPROPERTY()
	FString UserData;
};

/**
 * Message batching the status of many assets per channel/host.
 * Used to reduce message overhead.
 */
USTRUCT()
struct FAvaPlaybackStatuses : public FAvaPlaybackServerMessageBase
{
	GENERATED_BODY()
	
	UPROPERTY()
	FString ChannelName;

	/** Playback's instance Id. */
	UPROPERTY()
	TArray<FGuid> InstanceIds;
	
	UPROPERTY()
	TArray<FSoftObjectPath> AssetPaths;

	/** Batches all assets with the same status. */
	UPROPERTY()
	EAvaPlaybackStatus Status = EAvaPlaybackStatus::Unknown;
};

USTRUCT()
struct FAvaPlaybackAnimActionInfo
{
	GENERATED_BODY()
    
	UPROPERTY()
	FString AnimationName;

	UPROPERTY()
	EAvaPlaybackAnimAction AnimationAction = EAvaPlaybackAnimAction::None;
};

/**
 * Request by a client to execute an animation on a playback asset
 * on a connected server.
 */
USTRUCT()
struct FAvaPlaybackAnimPlaybackRequest : public FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()

	/** Playback's instance Id. */
	UPROPERTY()
	FGuid InstanceId;
	
	UPROPERTY()
	FSoftObjectPath AssetPath;
	
	UPROPERTY()
	FString ChannelName;
	
	UPROPERTY()
	TArray<FAvaPlaybackAnimPlaySettings> AnimPlaySettings;

	UPROPERTY()
	TArray<FAvaPlaybackAnimActionInfo> AnimActionInfos;
};

/** Server replication of playback sequence events. */
USTRUCT()
struct FAvaPlaybackSequenceEvent : public FAvaPlaybackServerMessageBase
{
	GENERATED_BODY()

	/** Playback's instance Id. */
	UPROPERTY()
	FGuid InstanceId;
	
	UPROPERTY()
	FSoftObjectPath AssetPath;

	UPROPERTY()
	FString ChannelName;
	
	UPROPERTY()
	FString SequenceLabel;

	UPROPERTY()
	EAvaPlayableSequenceEventType EventType = EAvaPlayableSequenceEventType::None;

	UPROPERTY()
	int32 FrameNumber = 0;
};

USTRUCT()
struct FAvaPlaybackRemoteControlUpdateRequest : public FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()

	/** Playback's instance Id. */
	UPROPERTY()
	FGuid InstanceId;
	
	UPROPERTY()
	FSoftObjectPath AssetPath;

	UPROPERTY()
	FString ChannelName;
	
	UPROPERTY()
	FAvaPlayableRemoteControlValues RemoteControlValues;
	
	UPROPERTY()
	EAvaPlayableRCUpdateFlags UpdateFlags = EAvaPlayableRCUpdateFlags::None;
};

USTRUCT()
struct FAvaPlaybackTransitionStartRequest : public FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	FString ChannelName;

	UPROPERTY()
	FGuid TransitionId;
	
	UPROPERTY()
	TArray<FGuid> EnterInstanceIds;

	UPROPERTY()
	TArray<FGuid> PlayingInstanceIds;

	UPROPERTY()
	TArray<FGuid> ExitInstanceIds;

	UPROPERTY()
	TArray<FAvaPlayableRemoteControlValues> EnterValues;

	UPROPERTY()
	bool bUnloadDiscardedInstances = false;

	/** See EAvaPlayableTransitionFlags. */
	UPROPERTY()
	uint8 TransitionFlags = 0;

	EAvaPlayableTransitionFlags GetTransitionFlags() const { return static_cast<EAvaPlayableTransitionFlags>(TransitionFlags);}
	void SetTransitionFlags(EAvaPlayableTransitionFlags InTransitionFlags) { TransitionFlags = static_cast<uint8>(InTransitionFlags);}

};

USTRUCT()
struct FAvaPlaybackTransitionStopRequest : public FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	FString ChannelName;

	UPROPERTY()
	FGuid TransitionId;
	
	UPROPERTY()
	bool bUnloadDiscardedInstances = false;
};

/**
 * Defines a single transition event entry.
 * This is the payload for the transition events messages.
 */
USTRUCT()
struct FAvaPlaybackTransitionEventEntry
{
	GENERATED_BODY()
	
	UPROPERTY()
	FGuid InstanceId;

	UPROPERTY()
	uint8 EventFlags = static_cast<uint8>(EAvaPlayableTransitionEventFlags::None);

	FAvaPlaybackTransitionEventEntry() = default;
	
	FAvaPlaybackTransitionEventEntry(const FGuid& InInstanceId, EAvaPlayableTransitionEventFlags InFlags)
		: InstanceId(InInstanceId)
		, EventFlags(static_cast<uint8>(InFlags))
	{
	}

	// Because the enum is used as flags, we need to convert to uint8 manually. (Can't use TEnumAsByte for this apparently.)
	EAvaPlayableTransitionEventFlags GetEventFlags() const { return static_cast<EAvaPlayableTransitionEventFlags>(EventFlags);}
	void SetEventFlags(EAvaPlayableTransitionEventFlags InFlags) { EventFlags = static_cast<uint8>(InFlags);}
};

/**
 * Single entry transition event message.
 */
USTRUCT()
struct FAvaPlaybackTransitionEvent : public FAvaPlaybackServerMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	FString ChannelName;

	UPROPERTY()
	FGuid TransitionId;

	UPROPERTY()
	FAvaPlaybackTransitionEventEntry Entry;

	UPROPERTY()
	int32 FrameNumber = 0;
};

/**
 * Multi-entries transition event message.
 * We use this to group the events in a one message to ensure correct order on reception.
 */
USTRUCT()
struct FAvaPlaybackTransitionEventMulti : public FAvaPlaybackServerMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	FString ChannelName;

	UPROPERTY()
	FGuid TransitionId;
	
	UPROPERTY()
	TArray<FAvaPlaybackTransitionEventEntry> Entries;

	UPROPERTY()
	int32 FrameNumber = 0;
};

class UMediaOutput;

/**
 * Encapsulates a UMediaOutput object in binary form to be able to serialize it
 * nested within the broadcast request.
 */
USTRUCT()
struct FAvaBroadcastOutputData
{
	GENERATED_BODY()

	UPROPERTY()
	FAvaBroadcastMediaOutputInfo OutputInfo;

	UPROPERTY()
	TSoftClassPtr<UMediaOutput> MediaOutputClass;
	
	UPROPERTY()
	TArray<uint8> SerializedData;
	
	UPROPERTY()
	uint64 ObjectFlags = 0;

	EObjectFlags GetObjectFlags() const { return static_cast<EObjectFlags>(ObjectFlags); }
};

/** Request by a client to execute an action on broadcast channel(s). */
USTRUCT()
struct FAvaBroadcastRequest : public FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()
	
	/** Profile to select for broadcast. */
	UPROPERTY()
	FString Profile;

	/** Channel name. If empty, command is for all channels. */
	UPROPERTY()
	FString Channel;

	UPROPERTY()
	TArray<FAvaBroadcastOutputData> MediaOutputs;

	/** Action to do on channel: Start, Stop, Disable, etc. */
	UPROPERTY()
	EAvaBroadcastAction Action = EAvaBroadcastAction::None;
};

/** Request by a client to update a broadcast channel's settings. */
USTRUCT()
struct FAvaBroadcastChannelSettingsUpdate : public FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()
	
	/** Profile to select for broadcast. */
	UPROPERTY()
	FString Profile;

	/** Channel name. If empty, command is for all channels. */
	UPROPERTY()
	FString Channel;

	UPROPERTY(meta = (HideHeader, ShowPresets))
	FAvaViewportQualitySettings QualitySettings;
};

/**
 * Client sends this request to have a status of broadcast.
 *
 * This is used by the client when it connects to a server which
 * might be already broadcasting. This happens if the client
 * disconnects and reconnects. It needs to query the full
 * status of the server upon connection.
 */
USTRUCT()
struct FAvaBroadcastStatusRequest : public FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	bool bIncludeMediaOutputData = false;
};

USTRUCT()
struct FAvaBroadcastSettings
{
	GENERATED_BODY()

	/** Specifies the background clear color for the channel. */
	UPROPERTY()
	FLinearColor ChannelClearColor = FLinearColor::Black;

	/** Pixel format used if no media output has specific format requirement. */
	UPROPERTY()
	TEnumAsByte<EPixelFormat> ChannelDefaultPixelFormat = EPixelFormat::PF_B8G8R8A8;

	/** Resolution used if no media output has specific resolution requirement. */
	UPROPERTY()
	FIntPoint ChannelDefaultResolution = FIntPoint(1920, 1080);

	/**
	 * Enables drawing the placeholder widget when there is no Motion Design asset playing.
	 * If false, the channel is cleared to the background color.
	 */
	UPROPERTY()
	bool bDrawPlaceholderWidget = false;
	
	/** Specify a place holder widget to render when no Motion Design asset is playing. */
	UPROPERTY()
	FSoftObjectPath PlaceholderWidgetClass;
};

/**
 *	Request for the client to replicate it's broadcast settings to the server.
 **/
USTRUCT()
struct FAvaBroadcastSettingsUpdate : public FAvaPlaybackClientMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	FAvaBroadcastSettings BroadcastSettings;
};

/**
 * Encapsulate the complete status of a media output.
 */
USTRUCT()
struct FAvaBroadcastOutputStatus
{
	GENERATED_BODY()

	UPROPERTY()
	EAvaBroadcastOutputState MediaOutputState = EAvaBroadcastOutputState::Invalid;
	
	UPROPERTY()
	EAvaBroadcastIssueSeverity MediaIssueSeverity = EAvaBroadcastIssueSeverity::None;

	UPROPERTY()
	TArray<FString> MediaIssueMessages;
};

/**
 * Server's response when channel's broadcast status changes
 */
USTRUCT()
struct FAvaBroadcastStatus : public FAvaPlaybackServerMessageBase
{
	GENERATED_BODY()
	
	/** Profile to select for broadcast. */
	UPROPERTY()
	FString Profile;
	
	/** Channel name. If empty, command is for all channels of the profile. */
	UPROPERTY()
	FString ChannelName;

	/** Index of the channel in the profile. */
	UPROPERTY()
	int32 ChannelIndex = 0;

	/** Number of channels in the profile. */
	UPROPERTY()
	int32 NumChannels = 0;

	UPROPERTY()
	EAvaBroadcastChannelState ChannelState = EAvaBroadcastChannelState::Idle;

	UPROPERTY()
	EAvaBroadcastIssueSeverity ChannelIssueSeverity = EAvaBroadcastIssueSeverity::None;

	UPROPERTY()
	TMap<FGuid, FAvaBroadcastOutputStatus> MediaOutputStatuses;
	
	/**
	 * This is required to know if the MediaOutputs is empty because the data
	 * was not included or if it is because the data was included and there is
	 * actually no outputs defined on that channel.
	 */
	UPROPERTY()
	bool bIncludeMediaOutputData = false;
	
	/** Remark: this is only included if bIncludeMediaOutputData == true in the request. */
	UPROPERTY()
	TArray<FAvaBroadcastOutputData> MediaOutputs;
};
