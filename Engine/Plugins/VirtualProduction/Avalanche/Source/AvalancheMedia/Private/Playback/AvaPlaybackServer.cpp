// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/AvaPlaybackServer.h"

#include "AvaMediaMessageUtils.h"
#include "AvaMediaSettings.h"
#include "AvaPlaybackSyncManager.h"
#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/OutputDevices/AvaBroadcastDeviceProviderData.h"
#include "Broadcast/OutputDevices/AvaBroadcastDeviceProviderProxy.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputUtils.h"
#include "Containers/Ticker.h"
#include "IAvaModule.h"
#include "MediaOutput.h"
#include "MessageEndpointBuilder.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "Playable/AvaPlayable.h"
#include "Playback/AvaPlaybackUtils.h"
#include "Playback/Transition/AvaPlaybackServerTransition.h"

DEFINE_LOG_CATEGORY(LogAvaPlaybackServer);

namespace UE::AvaPlaybackServer::Private
{
	FAvaPlaybackStatus* MakePlaybackStatusMessage(const FGuid& InInstanceId, const FString& InChannelName,
		const FSoftObjectPath& InAssetPath, EAvaPlaybackStatus InStatus, const FString& InUserData, const bool bInUserDataValid = true)
	{
		FAvaPlaybackStatus* Message = FMessageEndpoint::MakeMessage<FAvaPlaybackStatus>();
		Message->InstanceId = InInstanceId;
		Message->ChannelName = InChannelName;
		Message->AssetPath = InAssetPath;
		Message->Status = InStatus;
		Message->UserData = InUserData;
		Message->bValidUserData = bInUserDataValid;
		return Message;
	}

	FAvaPlaybackStatus* MakePlaybackStatusMessage(const FGuid& InInstanceId, const FString& InChannelName,
		const FSoftObjectPath& InAssetPath, EAvaPlaybackStatus InStatus)
	{
		return MakePlaybackStatusMessage(InInstanceId, InChannelName, InAssetPath, InStatus, FString(), /*bInUserDataValid*/ false);
	}

	FString GetCommandActionString(const FAvaPlaybackCommand& InCommand)
	{
		FString ActionString = AvaPlayback::Utils::StaticEnumToString(InCommand.Action); 
		if (InCommand.Arguments.IsEmpty())
		{
			return ActionString;
		}

		return FString::Printf(TEXT("%s \"%s\""), *ActionString, *InCommand.Arguments);
	};

	/** Returns the command priority order of execution. */
	int32 GetCommandActionPriority(EAvaPlaybackAction InAction)
	{
		switch (InAction)
		{
		case EAvaPlaybackAction::None:			return 10;
		case EAvaPlaybackAction::Load:			return 1;
		case EAvaPlaybackAction::Start:			return 2;
		case EAvaPlaybackAction::Stop:			return 5;
		case EAvaPlaybackAction::Unload:		return 6;
		case EAvaPlaybackAction::Status:		return 7;		// We want the status after all other commands have been executed.
		case EAvaPlaybackAction::SetUserData:	return 3;		// Should be after Load and Start.
		case EAvaPlaybackAction::GetUserData:	return 4;		// Should be after "set user data". 
		default:
			return 10;
		}
	}

	// Simulate network latency by delaying playback commands a random amount.
	// This is used to cause desynchronization between the nodes on clustered rendering
	// and see how well the synchronization handles it.
	TAutoConsoleVariable<float> CVarTestMaxRandomWaitForPlaybackCommands(
		TEXT("MotionDesignPlaybackServer.Test.MaxRandomWaitForPlaybackCommands")
		, 0.0f
		, TEXT("if not zero, the server will wait a random duration between 0 and the specified delay before executing playback commands. Unit: seconds"), ECVF_Default);

	FRandomStream GPlaybackServerRandomStream((int32)FPlatformTime::Cycles());
}

/**
 * Container for the active playback instance transitions.
 * We use composition for the FGCObject interface.
 */
class FAvaPlaybackServer::FServerPlaybackInstanceTransitionCollection : public FGCObject
{
public:
	FServerPlaybackInstanceTransitionCollection() = default;
	virtual ~FServerPlaybackInstanceTransitionCollection() override = default;
		
	//~ Begin FGCObject
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
	{
		Collector.AddReferencedObjects(Transitions);
	}
		
	virtual FString GetReferencerName() const override { return TEXT("FServerPlaybackInstanceTransitions"); }
	//~ End FGCObject
	
	/** Map Key: TransitionId. */
	TMap<FGuid, TObjectPtr<UAvaPlaybackServerTransition>> Transitions;

	/** Transitions marked for stop and discard. */
	TSet<FGuid> MarkedForStopAndDiscard;

	UAvaPlaybackServerTransition* FindTransition(const FGuid& InTransitionId) const
	{
		const TObjectPtr<UAvaPlaybackServerTransition>* FoundTransition = Transitions.Find(InTransitionId);
		return FoundTransition ? (*FoundTransition).Get() : nullptr;
	}
};

FAvaPlaybackServer::FAvaPlaybackServer()
	: Manager(MakeShared<FAvaPlaybackManager>())
	, PlaybackInstanceTransitions(MakeUnique<FServerPlaybackInstanceTransitionCollection>())
{
}

FAvaPlaybackServer::~FAvaPlaybackServer()
{
	FAvaBroadcastOutputChannel::GetOnMediaOutputStateChanged().RemoveAll(this);
	FAvaBroadcastOutputChannel::GetOnChannelChanged().RemoveAll(this);
	UAvaPlayable::OnSequenceEvent().RemoveAll(this);

	Manager->OnPlaybackInstanceInvalidated.RemoveAll(this);
	Manager->OnPlaybackInstanceStatusChanged.RemoveAll(this);
	Manager->OnLocalPlaybackAssetRemoved.RemoveAll(this);

	FMessageEndpoint::SafeRelease(MessageEndpoint);
	
	StopPlaybacks();
	
	for (IConsoleObject* ConsoleCommand : ConsoleCommands)
	{
		IConsoleManager::Get().UnregisterConsoleObject(ConsoleCommand);
	}
	ConsoleCommands.Empty();

	FCoreDelegates::OnEndFrame.RemoveAll(this);

#if WITH_EDITOR
	if (UObjectInitialized())
	{
		UAvaMediaSettings* AvaMediaSettings = GetMutableDefault<UAvaMediaSettings>();
		AvaMediaSettings->OnSettingChanged().RemoveAll(this);
	}
#endif
}

void FAvaPlaybackServer::Init(const FString& InAssignedServerName)
{
	Manager->SetEnablePlaybackCommandsBuffering(true);
	Manager->OnPlaybackInstanceInvalidated.AddRaw(this, &FAvaPlaybackServer::OnPlaybackInstanceInvalidated);
	Manager->OnPlaybackInstanceStatusChanged.AddRaw(this, &FAvaPlaybackServer::OnPlaybackInstanceStatusChanged);
	Manager->OnLocalPlaybackAssetRemoved.AddRaw(this, &FAvaPlaybackServer::OnPlaybackAssetRemoved);

	ComputerName = FPlatformProcess::ComputerName();
	ProjectContentPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir());
	ProcessId = FPlatformProcess::GetCurrentProcessId();
	ServerName = InAssignedServerName.IsEmpty() ? ComputerName : InAssignedServerName;
	RegisterCommands();
	
	// Create our end point. Note that these handlers are used by other services and the Context may not be valid
	MessageEndpoint = FMessageEndpoint::Builder("AvaPlaybackServer")
	.Handling<FAvaPlaybackPing>(this, &FAvaPlaybackServer::HandlePlaybackPing)
	.Handling<FAvaPlaybackUpdateClientUserData>(this, &FAvaPlaybackServer::HandleUpdateClientUserData)
	.Handling<FAvaPlaybackStatCommand>(this, &FAvaPlaybackServer::HandleStatCommand)
	.Handling<FAvaPlaybackDeviceProviderDataRequest>(this, &FAvaPlaybackServer::HandleDeviceProviderDataRequest)
	.Handling<FAvaPlaybackUpdateClientInfo>(this, &FAvaPlaybackServer::HandleUpdateClientInfo)
	.Handling<FAvaPlaybackInstanceSettingsUpdate>(this, &FAvaPlaybackServer::HandleAvaInstanceSettingsUpdate)
	.Handling<FAvaPlaybackPlayableSettingsUpdate>(this, &FAvaPlaybackServer::HandlePlayableSettingsUpdate)
	.Handling<FAvaPlaybackPackageEvent>(this, &FAvaPlaybackServer::HandlePackageEvent)
	.Handling<FAvaPlaybackAssetStatusRequest>(this, &FAvaPlaybackServer::HandlePlaybackAssetStatusRequest)
	.Handling<FAvaPlaybackRequest>(this, &FAvaPlaybackServer::HandlePlaybackRequest)
	.Handling<FAvaPlaybackAnimPlaybackRequest>(this, &FAvaPlaybackServer::HandleAnimPlaybackRequest)
	.Handling<FAvaPlaybackRemoteControlUpdateRequest>(this, &FAvaPlaybackServer::HandleRemoteControlUpdateRequest)
	.Handling<FAvaPlaybackTransitionStartRequest>(this, &FAvaPlaybackServer::HandlePlayableTransitionStartRequest)
	.Handling<FAvaPlaybackTransitionStopRequest>(this, &FAvaPlaybackServer::HandlePlayableTransitionStopRequest)
	.Handling<FAvaBroadcastSettingsUpdate>(this, &FAvaPlaybackServer::HandleBroadcastSettingsUpdate)
	.Handling<FAvaBroadcastRequest>(this, &FAvaPlaybackServer::HandleBroadcastRequest)
	.Handling<FAvaBroadcastChannelSettingsUpdate>(this, &FAvaPlaybackServer::HandleBroadcastChannelSettingsUpdate)
	.Handling<FAvaBroadcastStatusRequest>(this, &FAvaPlaybackServer::HandleBroadcastStatusRequest);

	if (MessageEndpoint.IsValid())
	{
		// Subscribe to the server listing requests
		MessageEndpoint->Subscribe<FAvaPlaybackPing>();

		UE_LOGF(LogAvaPlaybackServer, Log, "Motion Design Playback Server \"%ls\" Started.", *ServerName);
	}

	// Prevent throttling and idling.	
	if (IConsoleVariable* IdleWhenNotForeground = IConsoleManager::Get().FindConsoleVariable(TEXT("t.IdleWhenNotForeground")))
	{
		IdleWhenNotForeground->Set(0);
	}

	FCoreDelegates::OnEndFrame.AddSP(this, &FAvaPlaybackServer::Tick);

	{
		FString LogReplicationVerbosity;
		if (FParse::Value(FCommandLine::Get(),TEXT("MotionDesignPlaybackServerLogReplication="), LogReplicationVerbosity))
		{
			LogReplicationVerbosityFromCommandLine = ParseLogVerbosityFromString(LogReplicationVerbosity);
		}
	}

#if WITH_EDITOR
	UAvaMediaSettings* AvaMediaSettings = GetMutableDefault<UAvaMediaSettings>();
	AvaMediaSettings->OnSettingChanged().AddSP(this, &FAvaPlaybackServer::OnAvaMediaSettingsChanged);
#endif
	ApplyAvaMediaSettings();

	FAvaBroadcastOutputChannel::GetOnMediaOutputStateChanged().AddSP(this, &FAvaPlaybackServer::OnMediaOutputStateChanged);
	FAvaBroadcastOutputChannel::GetOnChannelChanged().AddSP(this, &FAvaPlaybackServer::OnChannelChanged);
	UAvaPlayable::OnSequenceEvent().AddSP(this, &FAvaPlaybackServer::OnPlayableSequenceEvent);
}

TArray<FAvaPlaybackServer::FPlaybackInstanceReference> FAvaPlaybackServer::StopPlaybacks(const FString& InChannelName, const FSoftObjectPath& InAssetPath, bool bInUnload)
{
	TArray<FPlaybackInstanceReference> StoppedPlaybackInstances;
	
	if (UObjectInitialized())
	{
		const EAvaPlaybackStopOptions PlaybackStopOptions = Manager->GetPlaybackStopOptions(bInUnload);
		StoppedPlaybackInstances.Reserve(ActivePlaybackInstances.Num());
		
		for (const TPair<FGuid, TSharedPtr<FAvaPlaybackInstance>>& PlaybackInstance : ActivePlaybackInstances)
		{
			if (!PlaybackInstance.Value)
			{
				continue;
			}

			// channel filtering.
			if (!InChannelName.IsEmpty() && PlaybackInstance.Value->GetChannelName() != InChannelName)
			{
				continue;
			}

			// Asset path filtering
			if (!InAssetPath.IsNull() && PlaybackInstance.Value->GetSourcePath() != InAssetPath)
			{
				continue;
			}

			// If we just stopping and the playback is already stopped, skip.
			if (!bInUnload && PlaybackInstance.Value->IsPlaying() == false)
			{
				continue;
			}
			
			StoppedPlaybackInstances.Add({PlaybackInstance.Key, PlaybackInstance.Value->GetSourcePath()});
			PlaybackInstance.Value->GetPlayback()->Stop(PlaybackStopOptions);
			PlaybackInstance.Value->SetStatus(EAvaPlaybackStatus::Loaded);

			if (bInUnload)
			{
				PlaybackInstance.Value->Unload();
			}
			else
			{
				PlaybackInstance.Value->Recycle();
			}
		}
	}
	
	if (bInUnload)
	{
		for (const FPlaybackInstanceReference& StoppedInstance : StoppedPlaybackInstances)
		{
			ActivePlaybackInstances.Remove(StoppedInstance.Id);
		}
	}
	return StoppedPlaybackInstances;
}

TArray<FAvaPlaybackServer::FPlaybackInstanceReference> FAvaPlaybackServer::StartPlaybacks()
{
	TArray<FPlaybackInstanceReference> Instances;
	Instances.Reserve(ActivePlaybackInstances.Num());
	
	// Start all the loaded playback.
	for (const TPair<FGuid, TSharedPtr<FAvaPlaybackInstance>>& PlaybackInstance : ActivePlaybackInstances)
	{
		if (PlaybackInstance.Value->GetPlayback() && PlaybackInstance.Value->GetPlayback()->IsPlaying() == false)
		{
			Instances.Add({PlaybackInstance.Value->GetInstanceId(), PlaybackInstance.Value->GetSourcePath()});
			PlaybackInstance.Value->GetPlayback()->Play();
			PlaybackInstance.Value->SetStatus(EAvaPlaybackStatus::Starting);
		}
	}
	return Instances;
}

TArray<FString> FAvaPlaybackServer::GetAllChannelsFromPlayingPlaybacks(const FSoftObjectPath& InAssetPath) const
{
	TSet<FString> Channels;
	for (const TPair<FGuid, TSharedPtr<FAvaPlaybackInstance>>& PlaybackInstance : ActivePlaybackInstances)
	{
		if (!PlaybackInstance.Value)
		{
			continue;
		}

		// filter with asset path.
		if (!InAssetPath.IsNull() && PlaybackInstance.Value->GetSourcePath() == InAssetPath)
		{
			continue;
		}

		// Ignore stopped instances.
		if (!PlaybackInstance.Value->IsPlaying())
		{
			continue;
		}
		
		Channels.Add(PlaybackInstance.Value->GetChannelName());
	}
	return Channels.Array();
}

void FAvaPlaybackServer::StartShuttingDown()
{
	Manager->StartShuttingDown();
}

void FAvaPlaybackServer::StartBroadcast()
{
	UAvaBroadcast::Get().StartBroadcast();
}

void FAvaPlaybackServer::StopBroadcast()
{
	UAvaBroadcast::Get().StopBroadcast();
}

const FString& FAvaPlaybackServer::GetUserData(const FString& InKey) const
{
	if (const FString* Data = UserDataEntries.Find(InKey))
	{
		return *Data;
	}
	static FString EmptyString;
	return EmptyString;
}

void FAvaPlaybackServer::SetUserData(const FString& InKey, const FString& InData)
{
	UserDataEntries.Add(InKey, InData);
	SendUserDataUpdate(GetAllClientAddresses());
}

void FAvaPlaybackServer::RemoveUserData(const FString& InKey)
{
	UserDataEntries.Remove(InKey);
	SendUserDataUpdate(GetAllClientAddresses());
}

TArray<FString> FAvaPlaybackServer::GetClientNames() const
{
	TArray<FString> ClientNames;
	ClientNames.Empty(Clients.Num());
	for (const TPair<FString, TSharedPtr<FClientInfo>>& ClientInfo : Clients)
	{
		ClientNames.Add(ClientInfo.Value->ClientName);
	}
	return ClientNames;
}

FMessageAddress FAvaPlaybackServer::GetClientAddress(const FString& InClientName) const
{
	if (const TSharedPtr<FClientInfo>* ClientInfo = Clients.Find(InClientName))
	{
		return (*ClientInfo)->Address;
	}
	FMessageAddress InvalidAddress;
	InvalidAddress.Invalidate();
	return InvalidAddress;
}

bool FAvaPlaybackServer::HasClientUserData(const FString& InClientName, const FString& InKey) const
{
	if (const TSharedPtr<FClientInfo>* ClientInfo = Clients.Find(InClientName))
	{
		return (*ClientInfo)->UserDataEntries.Contains(InKey);
	}
	return false;
}

const FString& FAvaPlaybackServer::GetClientUserData(const FString& InClientName, const FString& InKey) const
{
	if (const TSharedPtr<FClientInfo>* ClientInfo = Clients.Find(InClientName))
	{
		if (const FString* Data = (*ClientInfo)->UserDataEntries.Find(InKey))
		{
			return *Data;
		}
	}
	static FString EmptyData;
	return EmptyData;
}

const IAvaBroadcastSettings* FAvaPlaybackServer::GetBroadcastSettings() const
{
	// Returns the first client we have.
	// Todo: In case we have multiple clients, we will need a smarter way to handle this.
	if (!Clients.IsEmpty())
	{
		return &Clients.begin()->Value->BroadcastSettings;
	}
	return nullptr;
}

const FAvaInstanceSettings* FAvaPlaybackServer::GetAvaInstanceSettings() const
{
	// Returns the first client we have.
	// Todo: In case we have multiple clients, we will need a smarter way to handle this.
	if (!Clients.IsEmpty())
	{
		return &Clients.begin()->Value->AvaInstanceSettings;
	}
	return nullptr;
}

const FAvaPlayableSettings* FAvaPlaybackServer::GetPlayableSettings() const
{
	// Returns the first client we have.
	// Todo: In case we have multiple clients, we will need a smarter way to handle this.
	if (!Clients.IsEmpty())
	{
		return &Clients.begin()->Value->PlayableSettings;
	}
	return nullptr;
}

bool FAvaPlaybackServer::RemovePlaybackInstanceTransition(const FGuid& InTransitionId)
{
	if (PlaybackInstanceTransitions)
	{
		return PlaybackInstanceTransitions->Transitions.Remove(InTransitionId) > 0;
	}
	return false;
}

void FAvaPlaybackServer::SendPlayableTransitionEvent(
	const FGuid& InTransitionId, const FAvaPlaybackTransitionEventEntry& InEntry,
	const FName& InChannelName, const FString& InClientName)
{
	FAvaPlaybackTransitionEvent* Message = FMessageEndpoint::MakeMessage<FAvaPlaybackTransitionEvent>();
	Message->ChannelName = InChannelName.ToString();
	Message->TransitionId = InTransitionId;
	Message->Entry = InEntry;
	Message->FrameNumber = GFrameNumber;

	// Transition events are sent reliably to avoid transitions getting stuck.
	SendResponse(Message, GetClientAddressSafe(InClientName), EMessageFlags::Reliable);
}

void FAvaPlaybackServer::SendPlayableTransitionEventMulti(
	const FGuid& InTransitionId, TArray<FAvaPlaybackTransitionEventEntry>&& InEntries,
	const FName& InChannelName, const FString& InClientName)
{
	FAvaPlaybackTransitionEventMulti* Message = FMessageEndpoint::MakeMessage<FAvaPlaybackTransitionEventMulti>();
	Message->ChannelName = InChannelName.ToString();
	Message->TransitionId = InTransitionId;
	Message->Entries = MoveTemp(InEntries);
	Message->FrameNumber = GFrameNumber;

	// Transition events are sent reliably to avoid transitions getting stuck.
	SendResponse(Message, GetClientAddressSafe(InClientName), EMessageFlags::Reliable);
}

void FAvaPlaybackServer::HandlePlaybackPing(const FAvaPlaybackPing& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	if (!InMessage.bAutoPing)
	{
		UE_LOGF(LogAvaPlaybackServer, Log, "Received Manual Ping from %ls", *InContext->GetSender().ToString());
	}
	
	FClientInfo& ClientInfo = GetOrCreateClientInfo(InMessage.ClientName, InContext->GetSender());
	ClientInfo.ResetPingTimeout();
	// The client announce it's ping interval. We can expect a new ping around that interval. For tolerance
	// we allow up to 3 ping intervals before declaring the client non responsive.
	ClientInfo.AddTimeout(FDateTime::UtcNow() + FTimespan::FromSeconds(3 * InMessage.PingIntervalSeconds));

	// Reply to the ping.
	{
		FAvaPlaybackPong* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaPlaybackPong>();
		ReplyMessage->bAutoPong = InMessage.bAutoPing;
		ReplyMessage->bRequestClientInfo = !ClientInfo.bClientInfoReceived;
		ReplyMessage->ProjectContentPath = ProjectContentPath;
		ReplyMessage->ProcessId = ProcessId; 
		SendResponse(ReplyMessage, InContext->GetSender());
	}
}

void FAvaPlaybackServer::HandleUpdateClientUserData(const FAvaPlaybackUpdateClientUserData& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	FClientInfo& ClientInfo = GetOrCreateClientInfo(InMessage.ClientName, InContext->GetSender());
	ClientInfo.ResetPingTimeout();
	ClientInfo.UserDataEntries = InMessage.UserDataEntries;

	// Logging when user data is updated (for debugging).
	UE_LOGF(LogAvaPlaybackServer, Verbose, "Received new user data for client \"%ls\".", *InMessage.ClientName);
	for (const TPair<FString, FString>& UserData : ClientInfo.UserDataEntries)
	{
		UE_LOGF(LogAvaPlaybackServer, Verbose, "User data \"%ls\":\"%ls\".", *UserData.Key, *UserData.Value);
	}
}

void FAvaPlaybackServer::HandleStatCommand(const FAvaPlaybackStatCommand& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	IAvaModule& AvaModule = IAvaModule::Get();

	bool bLocalCommandSucceeded = false;

	// Note: the client sends an empty command to sync it's state with server in connection handshake.
	if (!InMessage.Command.IsEmpty())
	{
		bLocalCommandSucceeded = Manager->HandleStatCommand({InMessage.Command});
	}

	// If the enabled state from the client where reliable, we ensure
	// that the server's state is sync'd to it.
	if (InMessage.bClientStateReliable)
	{
		AvaModule.OverwriteEnabledRuntimeStats(InMessage.ClientEnabledRuntimeStats);
	}

	// The server replies with it's current status.
	FAvaPlaybackStatStatus* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaPlaybackStatStatus>();
	ReplyMessage->bClientStateReliable = InMessage.bClientStateReliable;
	ReplyMessage->bCommandSucceeded = bLocalCommandSucceeded;
	ReplyMessage->EnabledRuntimeStats = AvaModule.GetEnabledRuntimeStats();
	SendResponse(ReplyMessage, InContext->GetSender());
}

void FAvaPlaybackServer::HandleDeviceProviderDataRequest(const FAvaPlaybackDeviceProviderDataRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	FAvaBroadcastDeviceProviderDataList* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaBroadcastDeviceProviderDataList>();
	ReplyMessage->Populate(ServerName);
	SendResponse(ReplyMessage, InContext->GetSender());
}

void FAvaPlaybackServer::HandleUpdateClientInfo(const FAvaPlaybackUpdateClientInfo& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	FClientInfo& ClientInfo = GetOrCreateClientInfo(InMessage.ClientName, InContext->GetSender());
	ClientInfo.ComputerName = InMessage.ComputerName;
	ClientInfo.ProjectContentPath = InMessage.ProjectContentPath;
	ClientInfo.ProcessId = InMessage.ProcessId;
	ClientInfo.bClientInfoReceived = true;

	// Sync Manager is not needed if the server instance is a local instance from the same project directory.
	const bool bShouldEnableSyncManager = !IsLocalClient(ClientInfo);
	ClientInfo.MediaSyncManager->SetEnable(bShouldEnableSyncManager);
}

void FAvaPlaybackServer::HandleAvaInstanceSettingsUpdate(const FAvaPlaybackInstanceSettingsUpdate& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	FClientInfo& ClientInfo = GetOrCreateClientInfo(InMessage.ClientName, InContext->GetSender());
	ClientInfo.AvaInstanceSettings = InMessage.InstanceSettings;
	
	UE_LOGF(LogAvaPlaybackServer, Verbose, "Received new instance settings from client \"%ls\".", *InMessage.ClientName);
}

void FAvaPlaybackServer::HandlePlayableSettingsUpdate(const FAvaPlaybackPlayableSettingsUpdate& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	FClientInfo& ClientInfo = GetOrCreateClientInfo(InMessage.ClientName, InContext->GetSender());
	ClientInfo.PlayableSettings = InMessage.PlayableSettings;
	
	UE_LOGF(LogAvaPlaybackServer, Verbose, "Received new playable settings from client \"%ls\".", *InMessage.ClientName);
}

void FAvaPlaybackServer::HandlePackageEvent(const FAvaPlaybackPackageEvent& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	const FClientInfo* ClientInfo = GetClientInfo(InContext->GetSender());
	
	// Ignore package events if the client is not local.
	// Also ignore package events if the client is in the same process.
	if (ClientInfo && (!IsLocalClient(*ClientInfo) || IsClientOnLocalProcess(*ClientInfo)))
	{
		return;
	}
		
	switch (InMessage.Event)
	{
	case EAvaPlaybackPackageEvent::None:
		break;
	case EAvaPlaybackPackageEvent::PostSave:
		UE_LOGF(LogAvaPlaybackServer, Verbose, "Client modified package \"%ls\".", *InMessage.PackageName.ToString());
		// External event will trigger a reload.
		Manager->OnPackageModified(InMessage.PackageName, EAvaPlaybackPackageEventFlags::External | EAvaPlaybackPackageEventFlags::Saved);
		break;
	case EAvaPlaybackPackageEvent::PreSave:
		// On demand flush package loading.
		if (!UE::IsSavingPackage(nullptr) && !IsGarbageCollectingAndLockingUObjectHashTables())
		{
			if (UPackage* ExistingPackage = FindPackage(nullptr, *InMessage.PackageName.ToString()))
			{
				FAvaPlaybackUtils::FlushPackageLoading(ExistingPackage);
			}
		}
		break;
	case EAvaPlaybackPackageEvent::AssetDeleted:
		UE_LOGF(LogAvaPlaybackServer, Verbose, "Client deleted asset in package \"%ls\".", *InMessage.PackageName.ToString());
		Manager->OnPackageModified(InMessage.PackageName, EAvaPlaybackPackageEventFlags::External | EAvaPlaybackPackageEventFlags::AssetDeleted);
		break;
	}
}

void FAvaPlaybackServer::HandlePlaybackAssetStatusRequest(const FAvaPlaybackAssetStatusRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	if (!InMessage.AssetPath.IsValid())
	{
		return;
	}

	EAvaPlaybackAssetStatus PlaybackAssetStatus = EAvaPlaybackAssetStatus::Missing;

	if (Manager->IsLocalAssetAvailable(InMessage.AssetPath))
	{
		// Consider the asset available, unless we find that it is out of date below.
		PlaybackAssetStatus = EAvaPlaybackAssetStatus::Available;

		// If there is a corresponding client info, we need to compare the asset to make sure it is up to date.
		if (const FClientInfo* ClientInfo = GetClientInfo(InContext->GetSender()))
		{
			// Note: if the value is not available, we still send a reply with an "available" status, it is better than
			// no status. The status will be updated again if OnAvaAssetSyncStatusReceived is called.
			const TOptional<bool> NeedsSync = ClientInfo->MediaSyncManager->GetAssetSyncStatus(InMessage.AssetPath, InMessage.bForceRefresh);
			if (NeedsSync.IsSet() && NeedsSync.GetValue())
			{
				PlaybackAssetStatus = EAvaPlaybackAssetStatus::NeedsSync;
			}
		}
	}
	
	SendPlaybackAssetStatus(InContext->GetSender(), InMessage.AssetPath, PlaybackAssetStatus);
}

void FAvaPlaybackServer::HandlePlaybackRequest(const FAvaPlaybackRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	using namespace UE::AvaPlaybackServer::Private;
	const float MaxRandomWait = CVarTestMaxRandomWaitForPlaybackCommands.GetValueOnAnyThread();

	const FDateTime UtcNow = FDateTime::UtcNow();
	
	for (const FAvaPlaybackCommand& Command : InMessage.Commands)
	{
		TSharedPtr<FPendingPlaybackCommand> PendingCommand
			= MakeShared<FPendingPlaybackCommand>(UtcNow, GFrameNumber, GetCommandActionPriority(Command.Action), InContext->GetSender(), Command);

		if (MaxRandomWait > 0.0f && (Command.Action == EAvaPlaybackAction::Load || Command.Action == EAvaPlaybackAction::Start))
		{
			FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSPLambda(this, [this, PendingCommand] (float) 
			{
				PendingPlaybackCommands.Add(PendingCommand);	
				return false;
			}), GPlaybackServerRandomStream.GetFraction() * MaxRandomWait);
		}
		else
		{
			PendingPlaybackCommands.Add(MoveTemp(PendingCommand));
		}
	}
}

void FAvaPlaybackServer::HandleAnimPlaybackRequest(const FAvaPlaybackAnimPlaybackRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	for (const FAvaPlaybackAnimPlaySettings& AnimSettings : InMessage.AnimPlaySettings)
	{
		Manager->PushAnimationCommand(InMessage.InstanceId, InMessage.AssetPath, InMessage.ChannelName, AnimSettings.Action, AnimSettings);
	}

	for (const FAvaPlaybackAnimActionInfo& ActionInfo : InMessage.AnimActionInfos)
	{
		FAvaPlaybackAnimPlaySettings AnimSettings;
		AnimSettings.AnimationName = !ActionInfo.AnimationName.IsEmpty() ? *ActionInfo.AnimationName : FName();
		Manager->PushAnimationCommand(InMessage.InstanceId, InMessage.AssetPath, InMessage.ChannelName, ActionInfo.AnimationAction, AnimSettings);
	}
}

void FAvaPlaybackServer::HandleRemoteControlUpdateRequest(const FAvaPlaybackRemoteControlUpdateRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	Manager->PushRemoteControlCommand(InMessage.InstanceId, InMessage.AssetPath, InMessage.ChannelName,
		MakeShared<FAvaPlayableRemoteControlValues>(InMessage.RemoteControlValues), InMessage.UpdateFlags);
}

void FAvaPlaybackServer::HandlePlayableTransitionStartRequest(const FAvaPlaybackTransitionStartRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	check(PlaybackInstanceTransitions);
	
	UAvaPlaybackServerTransition* Transition = UAvaPlaybackServerTransition::MakeNew(AsShared());
	Transition->SetTransitionId(InMessage.TransitionId);
	Transition->SetChannelName(FName(InMessage.ChannelName));
	Transition->SetClientName(GetClientNameSafe(InContext->GetSender()));
	Transition->SetUnloadDiscardedInstances(InMessage.bUnloadDiscardedInstances);
	Transition->SetTransitionFlags(InMessage.GetTransitionFlags());

	// Enter Instances are likely not loaded yet.
	Transition->AddPendingEnterInstanceIds(InMessage.EnterInstanceIds);
	Transition->SetEnterValues(InMessage.EnterValues);

	// We try to resolve the playing instances since they should be loaded (unless delayed).
	for (const FGuid& PlayingInstanceId : InMessage.PlayingInstanceIds)
	{
		if (TSharedPtr<FAvaPlaybackInstance> Instance = FindActivePlaybackInstance(PlayingInstanceId))
		{
			Transition->AddPlayingInstance(Instance);
		}
		else
		{
			// TODO: have a time out on this.
			// Avoid having a transition be stuck because it is waiting on an instance that will never come.
			Transition->AddPendingPlayingInstanceId(PlayingInstanceId);	// Will be resolved later.

			UE_LOGF(LogAvaPlaybackServer, Warning,
				"Transition \"%ls\" from client \"%ls\": \"Playing\" Instance Id \"%ls\" was not found in active playback instances.",
				*InMessage.TransitionId.ToString(), *GetClientNameSafe(InContext->GetSender()), *PlayingInstanceId.ToString());
		}
	}
	
	// We try to resolve the exit instances since they should be loaded (unless delayed).
	for (const FGuid& ExitInstanceId : InMessage.ExitInstanceIds)
	{
		if (TSharedPtr<FAvaPlaybackInstance> Instance = FindActivePlaybackInstance(ExitInstanceId))
		{
			Transition->AddExitInstance(Instance);
		}
		else
		{
			Transition->AddPendingExitInstanceId(ExitInstanceId);	// Will be resolved later.
			
			UE_LOGF(LogAvaPlaybackServer, Warning,
				"Transition \"%ls\" from client \"%ls\": \"Exit\" Instance Id \"%ls\" was not found in active playback instances.",
				*InMessage.TransitionId.ToString(), *GetClientNameSafe(InContext->GetSender()), *ExitInstanceId.ToString());
		}
	}
	
	PlaybackInstanceTransitions->Transitions.Add(InMessage.TransitionId, Transition);
	GetPlaybackManager().PushPlaybackTransitionStartCommand(Transition);
}

void FAvaPlaybackServer::HandlePlayableTransitionStopRequest(const FAvaPlaybackTransitionStopRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	if (!PlaybackInstanceTransitions)
	{
		UE_LOGF(LogAvaPlaybackServer, Error, 
			"Stopping Transition \"%ls\" failed because transition collection is not available.", *InMessage.TransitionId.ToString());
		// todo: should reply with an error message to client. (need a request id)
		return;
	}

	static FString None = TEXT("None");

	// Stop all transitions.
	if (!InMessage.TransitionId.IsValid() && InMessage.ChannelName.Equals(None))
	{
		PlaybackInstanceTransitions->Transitions.GetKeys(PlaybackInstanceTransitions->MarkedForStopAndDiscard);
		return;
	}
	
	if (PlaybackInstanceTransitions->Transitions.Contains(InMessage.TransitionId))
	{
		PlaybackInstanceTransitions->MarkedForStopAndDiscard.Add(InMessage.TransitionId);
	}
	else
	{
		UE_LOGF(LogAvaPlaybackServer, Warning,
			"Stop Transition request: Transition \"%ls\" was not found in the active list.", *InMessage.TransitionId.ToString());
	}
}

void FAvaPlaybackServer::HandleBroadcastSettingsUpdate(const FAvaBroadcastSettingsUpdate& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	FClientInfo& ClientInfo = GetOrCreateClientInfo(InMessage.ClientName, InContext->GetSender());
	ClientInfo.ResetPingTimeout();

	// TODO: check if assets are required, if so request a sync. Need the DataSync API for this.
	ClientInfo.BroadcastSettings.Settings = InMessage.BroadcastSettings;
	
	UE_LOGF(LogAvaPlaybackServer, Verbose, "Received new broadcast settings from client \"%ls\".", *InMessage.ClientName);
}

void FAvaPlaybackServer::HandleBroadcastChannelSettingsUpdate(const FAvaBroadcastChannelSettingsUpdate& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	const FName ChannelName = *InMessage.Channel;
	if (!InMessage.Channel.IsEmpty())
	{
		FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetOrAddChannel(ChannelName);
		Channel.SetViewportQualitySettings(InMessage.QualitySettings);
	}
}

void FAvaPlaybackServer::HandleBroadcastRequest(const FAvaBroadcastRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	const FName ChannelName = *InMessage.Channel;
	if (InMessage.Action == EAvaBroadcastAction::Start)
	{
		if (InMessage.Channel.IsEmpty())
		{
			StartBroadcast();
		}
		else
		{
			FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetOrAddChannel(ChannelName);
			if (UpdateChannelOutputConfig(Channel, InMessage.MediaOutputs, false))
			{
				Channel.StartChannelBroadcast();
			}
		}
	}
	else if (InMessage.Action == EAvaBroadcastAction::UpdateConfig && !InMessage.Channel.IsEmpty())
	{
		FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetOrAddChannel(ChannelName);
		UpdateChannelOutputConfig(Channel, InMessage.MediaOutputs, true);
	}
	else if (InMessage.Action == EAvaBroadcastAction::Stop)
	{
		if (ChannelName.IsNone())
		{
			StopBroadcast();
		}
		else
		{
			FAvaBroadcastProfile& Profile = UAvaBroadcast::Get().GetCurrentProfile();
			FAvaBroadcastOutputChannel& Channel = Profile.GetChannelMutable(ChannelName);
			if (Channel.IsValidChannel())
			{
				Channel.StopChannelBroadcast();
				SendChannelStatusUpdate(InMessage.Channel, Channel, InContext->GetSender());
			}
		}
	}
	else if (InMessage.Action == EAvaBroadcastAction::DeleteChannel)
	{
		if (!InMessage.Channel.IsEmpty())
		{
			FAvaBroadcastProfile& Profile = UAvaBroadcast::Get().GetCurrentProfile();
			if (Profile.RemoveChannel(FName(InMessage.Channel)))
			{
				SendAllChannelStatusUpdate(InContext->GetSender(), false);
			}
			else
			{
				// TODO: need to inform client that operation failed.
				UE_LOGF(LogAvaPlaybackServer, Error, "Failed to remove channel \"%ls\".", *InMessage.Channel);
			}
		}
		else
		{
			UE_LOGF(LogAvaPlaybackServer, Error, "Received a \"Delete Channel\" command with an empty channel name.");
		}
	}
}

void FAvaPlaybackServer::HandleBroadcastStatusRequest(const FAvaBroadcastStatusRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UE_LOGF(LogAvaPlaybackServer, Verbose, "Received a broadcast status request from \"%ls\"", *InMessage.ClientName);
	// Make sure all the channel status are refreshed
	{
		// Block channel status update while we refresh, we want to send one clean update at the end.
		TGuardValue<bool> BlockChannelStatusUpdate(bBlockChannelStatusUpdate, true);
		for (FAvaBroadcastOutputChannel* Channel : UAvaBroadcast::Get().GetCurrentProfile().GetChannels())
		{
			Channel->RefreshState();
		}
	}
	
	SendAllChannelStatusUpdate(InContext->GetSender(), InMessage.bIncludeMediaOutputData);
}

FString FAvaPlaybackServer::GetMessageEndpointAddressId() const
{
	return MessageEndpoint.IsValid() && MessageEndpoint->IsEnabled() ? MessageEndpoint->GetAddress().ToString() : TEXT("");
}

void FAvaPlaybackServer::Tick()
{
	const FDateTime CurrentTimeUtc = FDateTime::UtcNow();
	RemoveDeadClients(CurrentTimeUtc);

	for (const TPair<FString, TSharedPtr<FClientInfo>>& Client : Clients)
	{
		if (Client.Value->MediaSyncManager.IsValid() && Client.Value->MediaSyncManager->IsEnabled())
		{
			Client.Value->MediaSyncManager->Tick();
		}
	}

	// Execute the pending commands in batch for this tick.
	ExecutePendingPlaybackCommands(CurrentTimeUtc);

	if (PlaybackInstanceTransitions)
	{
		for (const FGuid& TransitionId : PlaybackInstanceTransitions->MarkedForStopAndDiscard)
		{
			if (UAvaPlaybackServerTransition* Transition = PlaybackInstanceTransitions->FindTransition(TransitionId))
			{
				Transition->Stop();	// Should remove from active list.

				if (PlaybackInstanceTransitions->Transitions.Contains(TransitionId))
				{
					UE_LOGF(LogAvaPlaybackServer, Error, "Playback Transition %ls is still in the list after being stopped.", *TransitionId.ToString());
					PlaybackInstanceTransitions->Transitions.Remove(TransitionId);	// Remove anyway.
				}
			}
		}
		
		PlaybackInstanceTransitions->MarkedForStopAndDiscard.Reset();

		// Try to resolve the instance for loaded transitions.
		for (const TPair<FGuid, TObjectPtr<UAvaPlaybackServerTransition>>& Transition : PlaybackInstanceTransitions->Transitions)
		{
			Transition.Value->TryResolveInstances(*this);
		}
	}
	
	// TODO:
	// - Check status of outputs and send to client(s). In particular, watch the media capture's transient states.
	// - Send telemetry if client subscribed to the stream.
}

void FAvaPlaybackServer::RegisterCommands()
{
	if (ConsoleCommands.Num() != 0)
	{
		return;
	}
		
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("MotionDesignPlaybackServer.StartPlayback"),
			TEXT("Starts the playback of the given playback object."),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaPlaybackServer::StartPlaybackCommand),
			ECVF_Default
			));
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("MotionDesignPlaybackServer.StopPlayback"),
			TEXT("Stops the playback of the given playback object."),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaPlaybackServer::StopPlaybackCommand),
			ECVF_Default
			));
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("MotionDesignPlaybackServer.StartBroacast"),
			TEXT("Starts the broacast on specified (or all) channel(s)."),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaPlaybackServer::StartBroadcastCommand),
			ECVF_Default
			));
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("MotionDesignPlaybackServer.StopBroadcast"),
			TEXT("Stops the broadcast of the specified (or all) channel(s)."),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaPlaybackServer::StopBroadcastCommand),
			ECVF_Default
			));
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("MotionDesignPlaybackServer.SetUserData"),
			TEXT("Set Replicated User Data Entry (Key, Value)."),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaPlaybackServer::SetUserDataCommand),
			ECVF_Default
			));
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("MotionDesignPlaybackServer.Status"),
			TEXT("Display current status of all server info."),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaPlaybackServer::ShowStatusCommand),
			ECVF_Default
			));
}

void FAvaPlaybackServer::StartPlaybackCommand(const TArray<FString>& InArgs)
{
	if (InArgs.Num() >= 1)
	{
		//Concatenate all Args (starting from the 2) into one String with spaces in between each arg
		FString ConcatenatedCommands;
		for (int32 Index = 1; Index < InArgs.Num(); ++Index)
		{
			ConcatenatedCommands += InArgs[Index] + TEXT(" ");
		}

		FString ChannelName;
		FParse::Value(*ConcatenatedCommands, TEXT("Channel="), ChannelName);

		const FSoftObjectPath AssetPath(InArgs[0]);

		// Uncached load.
		if (UAvaPlaybackGraph* PlaybackObject = Manager->LoadPlaybackObject(AssetPath, ChannelName, FString()))
		{
			PlaybackObject->Play();
			StartBroadcast();
		}
	}
	else
	{
		UE_LOGF(LogAvaPlaybackServer, Log, "Arguments: Package.AssetName. Ex: \"/Game/AvaPlayback.AvaPlayback\"");
	}
}

void FAvaPlaybackServer::StopPlaybackCommand(const TArray<FString>& InArgs)
{
	StopPlaybacks();
}

void FAvaPlaybackServer::StartBroadcastCommand(const TArray<FString>& InArgs)
{
	StartBroadcast();
}

void FAvaPlaybackServer::StopBroadcastCommand(const TArray<FString>& InArgs)
{
	StopBroadcast();
}

void FAvaPlaybackServer::SetUserDataCommand(const TArray<FString>& InArgs)
{
	if (InArgs.Num() >= 2)
	{
		UE_LOGF(LogAvaPlaybackServer, Log, "Setting User Data Key \"%ls\" to Value: \"%ls\".", *InArgs[0], *InArgs[1]);
		SetUserData(InArgs[0], InArgs[1]);
	}
	else if (InArgs.Num() == 1)
	{
		// One argument means to remove that user data entry.
		if (HasUserData(InArgs[0]))
		{
			UE_LOGF(LogAvaPlaybackServer, Log, "Removing User Data Key \"%ls\".", *InArgs[0]);
			RemoveUserData(InArgs[0]);
		}
		else
		{
			UE_LOGF(LogAvaPlaybackServer, Error, "User Data Key \"%ls\" not found.", *InArgs[0]);
		}
	}
}

void FAvaPlaybackServer::ShowStatusCommand(const TArray<FString>& InArgs)
{
	UE_LOGF(LogAvaPlaybackServer, Display, "Playback Server: \"%ls\"", *ServerName);
	UE_LOGF(LogAvaPlaybackServer, Display, "- Endpoint Bus Address: \"%ls\"", MessageEndpoint.IsValid() ? *MessageEndpoint->GetAddress().ToString() : TEXT("Invalid"));
	UE_LOGF(LogAvaPlaybackServer, Display, "- Computer: \"%ls\"", *ComputerName);
	UE_LOGF(LogAvaPlaybackServer, Display, "- ProcessId: %d", ProcessId);
	UE_LOGF(LogAvaPlaybackServer, Display, "- Content Path: \"%ls\"", *ProjectContentPath);

	for (const TPair<FString, FString>& UserData : UserDataEntries)
	{
		UE_LOGF(LogAvaPlaybackServer, Display, "- User data \"%ls\":\"%ls\".", *UserData.Key, *UserData.Value);
	}
	
	for (const TPair<FString, TSharedPtr<FClientInfo>>& Client : Clients)
	{
		const FClientInfo& ClientInfo = *Client.Value;
		UE_LOGF(LogAvaPlaybackServer, Display, "Connected Client: \"%ls\"", *ClientInfo.ClientName);
		UE_LOGF(LogAvaPlaybackServer, Display, "   - Endpoint Bus Address: \"%ls\"", *ClientInfo.Address.ToString());
		UE_LOGF(LogAvaPlaybackServer, Display, "   - Computer: \"%ls\"", *ClientInfo.ComputerName);
		UE_LOGF(LogAvaPlaybackServer, Display, "   - ProcessId: %d", ClientInfo.ProcessId);
		UE_LOGF(LogAvaPlaybackServer, Display, "   - Content Path: \"%ls\"", *ClientInfo.ProjectContentPath);

		for (const TPair<FString, FString>& UserData : ClientInfo.UserDataEntries)
		{
			UE_LOGF(LogAvaPlaybackServer, Display, "   - User data \"%ls\":\"%ls\".", *UserData.Key, *UserData.Value);
		}
		
		UE_LOGF(LogAvaPlaybackServer, Display, "   - BroadcastSettings.ChannelClearColor: (%f, %f, %f, %f)",
			ClientInfo.BroadcastSettings.Settings.ChannelClearColor.R,
			ClientInfo.BroadcastSettings.Settings.ChannelClearColor.G,
			ClientInfo.BroadcastSettings.Settings.ChannelClearColor.B,
			ClientInfo.BroadcastSettings.Settings.ChannelClearColor.A);
		UE_LOGF(LogAvaPlaybackServer, Display, "   - BroadcastSettings.ChannelDefaultPixelFormat: (%ls)",
			GetPixelFormatString(ClientInfo.BroadcastSettings.Settings.ChannelDefaultPixelFormat));
		UE_LOGF(LogAvaPlaybackServer, Display, "   - BroadcastSettings.ChannelDefaultResolution: (%d, %d)",
			ClientInfo.BroadcastSettings.Settings.ChannelDefaultResolution.X,
			ClientInfo.BroadcastSettings.Settings.ChannelDefaultResolution.Y);
		UE_LOGF(LogAvaPlaybackServer, Display, "   - BroadcastSettings.bDrawPlaceholderWidget: %ls",
			ClientInfo.BroadcastSettings.Settings.bDrawPlaceholderWidget ? TEXT("true") : TEXT("false"));
		UE_LOGF(LogAvaPlaybackServer, Display, "   - BroadcastSettings.PlaceholderWidgetClass: %ls",
			*ClientInfo.BroadcastSettings.Settings.PlaceholderWidgetClass.ToString());

		UE_LOGF(LogAvaPlaybackServer, Display, "   - MediaSyncManager: %ls.",
			ClientInfo.MediaSyncManager->IsEnabled() ? TEXT("enabled") : TEXT("disabled"));		
		ClientInfo.MediaSyncManager->EnumerateAllTrackedPackages([](const FName& InPackageName, const TOptional<bool>& bInNeedSync)
		{
			if (bInNeedSync.IsSet())
			{
				UE_LOGF(LogAvaPlaybackServer, Display, "   - Package Sync Status \"%ls\":\"%ls\".",
					*InPackageName.ToString(), bInNeedSync.GetValue() ? TEXT("Need Sync") : TEXT("Up To Date"));
			}
		});
		for (const FName& PendingPackage : ClientInfo.MediaSyncManager->GetPendingRequests())
		{
			UE_LOGF(LogAvaPlaybackServer, Display, "   - Pending Sync Status Request \"%ls\".", *PendingPackage.ToString());
		}
	}

	UE_LOGF(LogAvaPlaybackServer, Display, "Active Playback Instances:");
	for (const TPair<FGuid, TSharedPtr<FAvaPlaybackInstance>>& ActivePlaybackInstance : ActivePlaybackInstances)
	{
		const TSharedPtr<FAvaPlaybackInstance>& Instance = ActivePlaybackInstance.Value;
		UE_LOGF(LogAvaPlaybackServer, Display, "   - Id:%ls, Channel: %ls, Asset: %ls, Status: %ls, UserData: %ls .",
			*Instance->GetInstanceId().ToString(), *Instance->GetChannelName(), *Instance->GetSourcePath().ToString(),
			*UE::AvaPlayback::Utils::StaticEnumToString(Instance->GetStatus()),
			*Instance->GetInstanceUserData());
	}
	
	UE_LOGF(LogAvaPlaybackServer, Display, "Active Playback Transitions:");
	for (const TPair<FGuid, TObjectPtr<UAvaPlaybackServerTransition>>& TransitionEntry : PlaybackInstanceTransitions->Transitions)
	{
		if (const UAvaPlaybackServerTransition* Transition = TransitionEntry.Value)
		{
			UE_LOGF(LogAvaPlaybackServer, Display, "   - %ls: %ls",
				*Transition->GetPrettyTransitionInfo(), *Transition->GetBriefTransitionDescription());
		}
	}
}

void FAvaPlaybackServer::OnAvaMediaSettingsChanged(UObject*, struct FPropertyChangedEvent&)
{
	ApplyAvaMediaSettings();
}

void FAvaPlaybackServer::OnChannelChanged(const FAvaBroadcastOutputChannel& InChannel, EAvaBroadcastChannelChange InChange)
{
	// Propagate channel state change event to all clients (unless blocked).
	if (!bBlockChannelStatusUpdate && EnumHasAnyFlags(InChange, EAvaBroadcastChannelChange::State))
	{
		const FString ChannelName = InChannel.GetChannelName().ToString();
		for (const TPair<FString, TSharedPtr<FClientInfo>>& Client : Clients)
		{
			SendChannelStatusUpdate(ChannelName, InChannel, Client.Value->Address);
		}
	}
}

void FAvaPlaybackServer::OnMediaOutputStateChanged(const FAvaBroadcastOutputChannel& InChannel, const UMediaOutput* InMediaOutput)
{
	// Remark: the channel's state has already been refreshed.
	const FString ChannelName = InChannel.GetChannelName().ToString();
	for (const TPair<FString, TSharedPtr<FClientInfo>>& Client : Clients)
	{
		SendChannelStatusUpdate(ChannelName, InChannel, Client.Value->Address);
	}
}

void FAvaPlaybackServer::OnAvaAssetSyncStatusReceived(const FAvaPlaybackAssetSyncStatusReceivedParams& InParams)
{
	if (const FClientInfo* ClientInfo = GetClientInfo(InParams.RemoteName))
	{
		// If it needs sync, the status becomes "NeedSync" otherwise, we consider it available.
		const EAvaPlaybackAssetStatus PlaybackAssetStatus = (InParams.bNeedsSync) ? EAvaPlaybackAssetStatus::NeedsSync : EAvaPlaybackAssetStatus::Available;
		SendPlaybackAssetStatus(ClientInfo->Address, InParams.AssetPath, PlaybackAssetStatus);
	}
}

void FAvaPlaybackServer::OnPlaybackInstanceInvalidated(const FAvaPlaybackInstance& InPlaybackInstance)
{
	// If the entry was not playing (i.e. just loaded), we update it's status to one of the
	// unloaded status. When a non-playing playback entry is invalidated, it is as if it is unloaded.
	if (!InPlaybackInstance.GetPlayback()->IsPlaying())
	{
		SendPlaybackStatus(GetAllClientAddresses(), InPlaybackInstance.GetInstanceId(), InPlaybackInstance.GetChannelName(), InPlaybackInstance.GetSourcePath(), GetUnloadedPlaybackStatus(InPlaybackInstance.GetSourcePath()));
	}
}

void FAvaPlaybackServer::OnPlaybackInstanceStatusChanged(const FAvaPlaybackInstance& InPlaybackInstance)
{
	SendPlaybackStatus(GetAllClientAddresses(), InPlaybackInstance.GetInstanceId(), InPlaybackInstance.GetChannelName(), InPlaybackInstance.GetSourcePath(), InPlaybackInstance.GetStatus());
}

void FAvaPlaybackServer::OnPlaybackAssetRemoved(const FSoftObjectPath& InAssetPath)
{
	for (const FMessageAddress& ClientAddress : GetAllClientAddresses())
	{
		SendPlaybackAssetStatus(ClientAddress, InAssetPath, EAvaPlaybackAssetStatus::Missing);
	}
}

void FAvaPlaybackServer::OnPlayableSequenceEvent(UAvaPlayable* InPlayable, FName InSequenceLabel, EAvaPlayableSequenceEventType InEventType)
{
	if (!InPlayable)
	{
		return;
	}

	// Use the instance id to trace back which playback instance this event belongs to.
	const TSharedPtr<FAvaPlaybackInstance> PlaybackInstance = FindActivePlaybackInstance(InPlayable->GetInstanceId());
	if (!PlaybackInstance)
	{
		return;
	}

	// Filter out clients on the same process, there is no need to replicate playable events in that case.
	constexpr bool bExcludeClientOnLocalProcess = true;
	const TArray<FMessageAddress> ClientAddresses = GetAllClientAddresses(bExcludeClientOnLocalProcess);

	if (ClientAddresses.IsEmpty())
	{
		return;
	}
	
	FAvaPlaybackSequenceEvent* Message = FMessageEndpoint::MakeMessage<FAvaPlaybackSequenceEvent>();
	Message->InstanceId = InPlayable->GetInstanceId();
	Message->AssetPath = PlaybackInstance->GetSourcePath();
	Message->ChannelName = PlaybackInstance->GetChannelName();
	Message->SequenceLabel = InSequenceLabel.ToString();
	Message->EventType = InEventType;
	Message->FrameNumber = GFrameNumber;
	SendResponse(Message, ClientAddresses);
}

void FAvaPlaybackServer::ApplyAvaMediaSettings()
{
	const UAvaMediaSettings& Settings = UAvaMediaSettings::Get();

	const ELogVerbosity::Type LogReplicationVerbosity = LogReplicationVerbosityFromCommandLine.IsSet() ?
		LogReplicationVerbosityFromCommandLine.GetValue() : UAvaMediaSettings::ToLogVerbosity(Settings.PlaybackServerLogReplicationVerbosity);
	
	if (LogReplicationVerbosity > ELogVerbosity::NoLogging && LogReplicationVerbosity < ELogVerbosity::NumVerbosity)
	{
		if (!ReplicationOutputDevice.IsValid())
		{
			ReplicationOutputDevice = MakeUnique<FReplicationOutputDevice>(this);
		}
		ReplicationOutputDevice->SetVerbosityThreshold(LogReplicationVerbosity);
	}
	else
	{
		ReplicationOutputDevice.Reset();
	}
}

void FAvaPlaybackServer::SendUserDataUpdate(const TArray<FMessageAddress>& InRecipients)
{
	FAvaPlaybackUpdateServerUserData* UserDataUpdate = FMessageEndpoint::MakeMessage<FAvaPlaybackUpdateServerUserData>();
	UserDataUpdate->UserDataEntries = UserDataEntries;
	SendResponse(UserDataUpdate, InRecipients, EMessageFlags::Reliable);
}

void FAvaPlaybackServer::SendChannelStatusUpdate(const FString& InChannelName, const FAvaBroadcastOutputChannel& InChannel, const FMessageAddress& InSender, bool bInIncludeOutputData)
{
	FAvaBroadcastStatus* Response = FMessageEndpoint::MakeMessage<FAvaBroadcastStatus>();
	Response->ChannelName = InChannelName;
	// Remark: The channel index and number of channels is used to know if the client has received all the channel's statuses
	// for the current profile. Since the profile may have a sub-set of all channels, we use the index of the channel
	// in the profile itself and the number of channels in the profile.
	Response->ChannelIndex = UAvaBroadcast::Get().GetCurrentProfile().GetChannelIndexInProfile(InChannel.GetChannelName());
	Response->NumChannels = UAvaBroadcast::Get().GetCurrentProfile().GetChannels().Num();
	Response->ChannelState = InChannel.GetState();	// Assumes state has already been refreshed.
	Response->ChannelIssueSeverity = InChannel.GetIssueSeverity();
	Response->bIncludeMediaOutputData = bInIncludeOutputData;

	uint32 TotalOutputDataSize = 0;
	const TArray<UMediaOutput*>& MediaOutputs = InChannel.GetMediaOutputs();
	for (UMediaOutput* MediaOutput : MediaOutputs)
	{
		const FAvaBroadcastMediaOutputInfo& OutputInfo = InChannel.GetMediaOutputInfo(MediaOutput);
		FAvaBroadcastOutputStatus& OutputStatus = Response->MediaOutputStatuses.Emplace(OutputInfo.Guid);
		const EAvaBroadcastOutputState OutputState = InChannel.GetMediaOutputState(MediaOutput);
		OutputStatus.MediaOutputState = OutputState;
		OutputStatus.MediaIssueSeverity = InChannel.GetMediaOutputIssueSeverity(OutputState, MediaOutput);
		OutputStatus.MediaIssueMessages = InChannel.GetMediaOutputIssueMessages(MediaOutput);
		
		if (bInIncludeOutputData)
		{
			FAvaBroadcastOutputData MediaOutputData = UE::AvaBroadcastOutputUtils::CreateMediaOutputData(MediaOutput);
			MediaOutputData.OutputInfo = InChannel.GetMediaOutputInfo(MediaOutput);
			MediaOutputData.OutputInfo.ServerName = ServerName;	// Restore server name (was Local).
			TotalOutputDataSize += MediaOutputData.SerializedData.Num();
			Response->MediaOutputs.Add(MoveTemp(MediaOutputData));
		}
	}

	// Adding a warning here, if we hit this warning, it may be necessary to send the
	// data through some other transport.
	const uint32 SafeMessageSizeLimit = UE::AvaMediaMessageUtils::GetSafeMessageSizeLimit();	
	if (TotalOutputDataSize > SafeMessageSizeLimit)
	{
		UE_LOGF(LogAvaPlaybackServer, Warning,
			"The requested channel status update (DataSize: %d) is larger that the safe message size limit (%d).",
			TotalOutputDataSize, SafeMessageSizeLimit);
	}
	
	SendResponse(Response, InSender);
}

void FAvaPlaybackServer::SendAllChannelStatusUpdate(const FMessageAddress& InSender, bool bInIncludeOutputData)
{
	for (const FAvaBroadcastOutputChannel* Channel : UAvaBroadcast::Get().GetCurrentProfile().GetChannels())
	{
		SendChannelStatusUpdate(Channel->GetChannelName().ToString(), *Channel, InSender, bInIncludeOutputData);
	}
}

void FAvaPlaybackServer::SendLogMessage(const TCHAR* InText, ELogVerbosity::Type InVerbosity, const FName& InCategory, double InTime)
{
	// Filter out clients on the same process, there is no need to replicate logs in that case.
	constexpr bool bExcludeClientOnLocalProcess = true;
	const TArray<FMessageAddress> ClientAddresses = GetAllClientAddresses(bExcludeClientOnLocalProcess);

	if (ClientAddresses.IsEmpty())
	{
		return;
	}
	
	FAvaPlaybackLog* Message = FMessageEndpoint::MakeMessage<FAvaPlaybackLog>();
	Message->Text = InText;
	Message->Verbosity = InVerbosity;
	Message->Category = InCategory;
	Message->Time = InTime;
	SendResponse(Message, ClientAddresses);
}

void FAvaPlaybackServer::ExecutePendingPlaybackCommands(const FDateTime& InUtcNow)
{
	if (PendingPlaybackCommands.IsEmpty())
	{
		return;
	}

	// Sort pending commands according to priority of execution. (Should minimize the amount of re-scheduling)
	Algo::Sort(PendingPlaybackCommands, [](const TSharedPtr<FPendingPlaybackCommand>& InA, const TSharedPtr<FPendingPlaybackCommand>& InB)
	{
		return InA->Priority < InB->Priority;
	});

	for (TArray<TSharedPtr<FPendingPlaybackCommand>>::TIterator PendingCommandIt(PendingPlaybackCommands); PendingCommandIt; ++PendingCommandIt)
	{
		const FAvaPlaybackCommand& Command = (*PendingCommandIt)->Command;
		const FMessageAddress& ReplyTo = (*PendingCommandIt)->ReplyTo;
		bool bReschedule = false;
		
		switch (Command.Action)
		{
		case EAvaPlaybackAction::None:
			break;
			
		case EAvaPlaybackAction::Load:
			LoadPlayback(ReplyTo, Command.InstanceId, Command.ChannelName, Command.AssetPath, Command.Arguments);
			break;
			
		case EAvaPlaybackAction::Start:
			StartPlayback(ReplyTo, Command.InstanceId, Command.ChannelName, Command.AssetPath, Command.Arguments);
			break;
			
		case EAvaPlaybackAction::Stop:
			StopPlayback(ReplyTo, Command.InstanceId, Command.ChannelName, Command.AssetPath);
			break;
			
		case EAvaPlaybackAction::Unload:
			UnloadPlayback(ReplyTo, Command.InstanceId, Command.ChannelName, Command.AssetPath);
			break;
			
		case EAvaPlaybackAction::Status:
			SendPlaybackStatus(ReplyTo, Command.InstanceId, Command.ChannelName, Command.AssetPath);
			break;
			
		case EAvaPlaybackAction::SetUserData:	
			bReschedule = !SetPlaybackUserData(ReplyTo, Command.InstanceId, Command.Arguments);
			break;

		case EAvaPlaybackAction::GetUserData:
			bReschedule = !SendPlaybackUserData(ReplyTo, Command.InstanceId);
			break;
		}
		using namespace UE::AvaPlaybackServer::Private;
		using namespace UE::AvaPlayback::Utils;
		
		if (bReschedule)
		{
			const UAvaMediaSettings& Settings = UAvaMediaSettings::Get();
			const float CommandWaitTime = (InUtcNow - (*PendingCommandIt)->ReceivedUtc).GetTotalSeconds();
			
			if (CommandWaitTime > Settings.ServerPendingPlaybackCommandTimeout)
			{
				UE_LOGF(LogAvaPlaybackServer, Warning, "%ls Discarding Playback Command [%ls] (Timed out after %f seconds) for asset: \"%ls\" (id:%ls) on channel %ls",
					*GetBriefFrameInfo(), *GetCommandActionString(Command), CommandWaitTime,
					*Command.AssetPath.GetAssetName(), *Command.InstanceId.ToString(), *Command.ChannelName);

				PendingCommandIt.RemoveCurrent();
			}
		}
		else
		{
			UE_LOGF(LogAvaPlaybackServer, Verbose, "%ls Playback Command [%ls] Executed for asset: \"%ls\" (id:%ls) on channel \"%ls\", received frame [%d], wait time: %.2f ms",
				*GetBriefFrameInfo(), *GetCommandActionString(Command),
				*Command.AssetPath.GetAssetName(), *Command.InstanceId.ToString(), *Command.ChannelName,
				(*PendingCommandIt)->ReceivedFrameNumber, (InUtcNow - (*PendingCommandIt)->ReceivedUtc).GetTotalMilliseconds());

			PendingCommandIt.RemoveCurrent();
		}
	}
}

TSharedPtr<FAvaPlaybackInstance> FAvaPlaybackServer::GetOrLoadPlaybackInstance(const FGuid& InInstanceId, const FString& InChannelName, const FSoftObjectPath& InAssetPath, const FString& InLoadOptions)
{
	// Check if loaded locally by the server under the same Id.
	TSharedPtr<FAvaPlaybackInstance> PlaybackInstance = FindActivePlaybackInstance(InInstanceId);
	if (PlaybackInstance.IsValid())
	{
		bool bInstanceIsValid = true;
		
		// Validate channel name and asset path.
		// If client has reassigned the Id for some reason, it will cause a reload of a new instance. 
		if (PlaybackInstance->GetChannelName() != InChannelName)
		{
			UE_LOGF(LogAvaPlaybackServer, Error
				, "Existing Playback InstanceId \"%ls\" for asset \"%ls\" has the wrong channel \"%ls\", requested \"%ls\"."
				, *InInstanceId.ToString(), *InAssetPath.ToString(), *PlaybackInstance->GetChannelName(), *InChannelName);
			bInstanceIsValid = false;
		}
		
		if (PlaybackInstance->GetSourcePath() != InAssetPath)
		{
			UE_LOGF(LogAvaPlaybackServer, Error
				, "Existing Playback InstanceId \"%ls\" has wrong source asset path \"%ls\", requested \"%ls\"."
				, *InInstanceId.ToString(), *PlaybackInstance->GetSourcePath().ToString(), *InAssetPath.ToString());
			bInstanceIsValid = false;
		}

		if (bInstanceIsValid)
		{
			return PlaybackInstance;
		}
	}

	// Load it or acquire a cached recycled asset.
	PlaybackInstance = Manager->AcquireOrLoadPlaybackInstance(InAssetPath, InChannelName, InLoadOptions);
	
	if (PlaybackInstance.IsValid())
	{
		PlaybackInstance->SetInstanceId(InInstanceId); // set the instance id provided by the client.
		PlaybackInstance->SetStatus(EAvaPlaybackStatus::Loading);
		Manager->ApplyPendingCommands(PlaybackInstance->GetPlayback(), InInstanceId, InAssetPath, InChannelName);
		ActivePlaybackInstances.Add(InInstanceId, PlaybackInstance);
	}
	
	return PlaybackInstance;	
}

void FAvaPlaybackServer::LoadPlayback(const FMessageAddress& InReplyToAddress, const FGuid& InInstanceId, const FString& InChannelName, const FSoftObjectPath& InAssetPath, const FString& InLoadOptions)
{
	if (!InAssetPath.IsValid())
	{
		// Not supported.
		UE_LOGF(LogAvaPlaybackServer, Error, "Specifying invalid path for load command is not supported.");
	}
	else
	{
		if (const TSharedPtr<FAvaPlaybackInstance> PlaybackInstance = GetOrLoadPlaybackInstance(InInstanceId, InChannelName, InAssetPath, InLoadOptions))
		{
			if (PlaybackInstance->GetPlayback())
			{
				PlaybackInstance->GetPlayback()->LoadInstances();
			}
			PlaybackInstance->UpdateStatus();
			SendPlaybackStatus(InReplyToAddress, InInstanceId, InChannelName, InAssetPath, PlaybackInstance->GetStatus());
		}
		else
		{
			// There was an error loading, we could either send Error or Missing as status. Sending missing for now.
			SendPlaybackStatus(InReplyToAddress, InInstanceId, InChannelName, InAssetPath, EAvaPlaybackStatus::Missing);
		}
	}
}

void FAvaPlaybackServer::StartPlayback(const FMessageAddress& InReplyToAddress, const FGuid& InInstanceId, const FString& InChannelName, const FSoftObjectPath& InAssetPath, const FString& InLoadOptions)
{
	if (!InAssetPath.IsValid())
	{
		// Start all loaded playback
		const TArray<FPlaybackInstanceReference> StartedInstances = StartPlaybacks();
		SendPlaybackStatuses(InReplyToAddress, InChannelName, StartedInstances, EAvaPlaybackStatus::Starting);
	}
	else
	{
		if (const TSharedPtr<FAvaPlaybackInstance> PlaybackInstance = GetOrLoadPlaybackInstance(InInstanceId, InChannelName, InAssetPath, InLoadOptions))
		{
			PlaybackInstance->GetPlayback()->Play();
			PlaybackInstance->SetStatus(EAvaPlaybackStatus::Starting);
			SendPlaybackStatus(InReplyToAddress, InInstanceId, InChannelName, InAssetPath, EAvaPlaybackStatus::Starting);
		}
		else
		{
			// There was an error loading, we could either send Error or Missing as status. Sending missing for now.
			SendPlaybackStatus(InReplyToAddress, InInstanceId, InChannelName, InAssetPath, EAvaPlaybackStatus::Missing);
		}	
	}
}

void FAvaPlaybackServer::StopPlayback(const FMessageAddress& InReplyToAddress, const FGuid& InInstanceId, const FString& InChannelName, const FSoftObjectPath& InAssetPath)
{
	// Instance Id is specified.
	if (InInstanceId.IsValid())
	{
		if (const TSharedPtr<FAvaPlaybackInstance> PlaybackInstance = FindActivePlaybackInstance(InInstanceId))
		{
			if (PlaybackInstance->GetPlayback()->IsPlaying())
			{
				PlaybackInstance->GetPlayback()->Stop(EAvaPlaybackStopOptions::Default);
			}
			PlaybackInstance->SetStatus(EAvaPlaybackStatus::Loaded);
			SendPlaybackStatus(InReplyToAddress, InInstanceId, PlaybackInstance->GetChannelName(), PlaybackInstance->GetSourcePath(), EAvaPlaybackStatus::Loaded);
		}
		else
		{
			SendPlaybackStatus(InReplyToAddress, InInstanceId, InChannelName, InAssetPath, GetUnloadedPlaybackStatus(InAssetPath));
		}
		return;
	}

	constexpr bool bUnload = false; // don't unload.

	// Channel is specified.
	if (!InChannelName.IsEmpty())
	{
		const TArray<FPlaybackInstanceReference> StoppedInstances = StopPlaybacks(InChannelName, InAssetPath, bUnload);
		SendPlaybackStatuses(InReplyToAddress, InChannelName, StoppedInstances, EAvaPlaybackStatus::Loaded);
	}
	else
	{
		// If there is no channel specified, we want to stop all playbacks,
		// but group them by channel because that is how reply messages are grouped.
		const TArray<FString> ChannelNames = GetAllChannelsFromPlayingPlaybacks(InAssetPath);
		for (const FString& ChannelName : ChannelNames)
		{
			const TArray<FPlaybackInstanceReference> StoppedInstances = StopPlaybacks(ChannelName, InAssetPath, bUnload);
			if (!StoppedInstances.IsEmpty())
			{
				SendPlaybackStatuses(InReplyToAddress, ChannelName, StoppedInstances, EAvaPlaybackStatus::Loaded);
			}
		}
	}
}

void FAvaPlaybackServer::UnloadPlayback(const FMessageAddress& InReplyToAddress, const FGuid& InInstanceId, const FString& InChannelName, const FSoftObjectPath& InAssetPath)
{
	// Instance Id is specified.
	if (InInstanceId.IsValid())
	{
		if (const TSharedPtr<FAvaPlaybackInstance> Instance = FindActivePlaybackInstance(InInstanceId))
		{
			// Validation of the operation.
			// Unloading an instance that is part of a transition is an error state.
			if (PlaybackInstanceTransitions)
			{
				for (const TPair<FGuid, TObjectPtr<UAvaPlaybackServerTransition>>& Transition : PlaybackInstanceTransitions->Transitions)
				{
					if (Transition.Value && Transition.Value->ContainsInstance(InInstanceId))
					{
						// Force Take out should also force stop the transition.
						PlaybackInstanceTransitions->MarkedForStopAndDiscard.Add(Transition.Value->GetTransitionId());
						
						UE_LOGF(LogAvaPlaybackServer, Error, "%ls Unloading instance \"%ls\" (id:%ls) while it is part of transition %ls.",
							*UE::AvaPlayback::Utils::GetBriefFrameInfo(), *Instance->GetSourcePath().GetAssetName(), *InInstanceId.ToString(), *Transition.Value->GetTransitionId().ToString());
					}
				}
			}
			
			Instance->Unload();
			ActivePlaybackInstances.Remove(InInstanceId);
			SendPlaybackStatus(InReplyToAddress, InInstanceId, Instance->GetChannelName(), Instance->GetSourcePath(), GetUnloadedPlaybackStatus(Instance->GetSourcePath()));
		}
		else
		{
			SendPlaybackStatus(InReplyToAddress, InInstanceId, InChannelName, InAssetPath, GetUnloadedPlaybackStatus(InAssetPath));
		}
		return;
	}

	constexpr bool bUnload = true; // Stop and unload.

	// Channel is specified.
	if (!InChannelName.IsEmpty())
	{
		// Will filter on asset path if specified.
		const TArray<FPlaybackInstanceReference> StoppedInstances = StopPlaybacks(InChannelName, InAssetPath, bUnload);
		SendPlaybackStatuses(InReplyToAddress, InChannelName, StoppedInstances, EAvaPlaybackStatus::Available);
	}
	else
	{
		// If there is no channel specified, we want to unload all playbacks,
		// but group them by channel because that is how reply messages are grouped.
		const FAvaBroadcastProfile& Profile = UAvaBroadcast::Get().GetCurrentProfile();
		for (const FAvaBroadcastOutputChannel* Channel : Profile.GetChannels())	// Note: using all channels, not just playing ones.
		{
			const FString ChannelName = Channel->GetChannelName().ToString();
			// Will filter on asset path if specified.
			const TArray<FPlaybackInstanceReference> StoppedInstances = StopPlaybacks(ChannelName, InAssetPath, bUnload);
			if (!StoppedInstances.IsEmpty())
			{
				SendPlaybackStatuses(InReplyToAddress, ChannelName, StoppedInstances, EAvaPlaybackStatus::Available);
			}
		}
	}
}

bool FAvaPlaybackServer::SetPlaybackUserData(const FMessageAddress& InReplyToAddress, const FGuid& InInstanceId, const FString& InUserData)
{
	if (InInstanceId.IsValid())
	{
		const TSharedPtr<FAvaPlaybackInstance> Instance = FindActivePlaybackInstance(InInstanceId);

		// Instance may not be loaded yet.
		if (!Instance)
		{
			return false;
		}
		
		Instance->SetInstanceUserData(InUserData);
			
		using namespace UE::AvaPlaybackServer::Private;
		SendResponse( MakePlaybackStatusMessage(InInstanceId,
			Instance->GetChannelName(), Instance->GetSourcePath(),
			Instance->GetStatus(), Instance->GetInstanceUserData()), InReplyToAddress);
	}
	return true;
}

bool FAvaPlaybackServer::SendPlaybackUserData(const FMessageAddress& InReplyToAddress, const FGuid& InInstanceId)
{
	if (InInstanceId.IsValid())
	{
		const TSharedPtr<FAvaPlaybackInstance> Instance = FindActivePlaybackInstance(InInstanceId);

		// Instance may not be loaded yet.
		if (!Instance)
		{
			return false;
		}

		using namespace UE::AvaPlaybackServer::Private;
		SendResponse( MakePlaybackStatusMessage(InInstanceId,
			Instance->GetChannelName(), Instance->GetSourcePath(),
			Instance->GetStatus(), Instance->GetInstanceUserData()), InReplyToAddress);
	}
	return true;
}

void FAvaPlaybackServer::SendPlaybackStatus(const FMessageAddress& InReplyToAddress, const FGuid& InInstanceId, const FString& InChannelName, const FSoftObjectPath& InAssetPath)
{
	if (!InAssetPath.IsValid())
	{
		// Send all loaded/playing objects.
		if (!InChannelName.IsEmpty())
		{
			SendAllPlaybackStatusesForChannelAndAssetPath(InReplyToAddress, InChannelName, FSoftObjectPath());
		}
		else
		{
			const FAvaBroadcastProfile& Profile = UAvaBroadcast::Get().GetCurrentProfile();
			for (const FAvaBroadcastOutputChannel* Channel : Profile.GetChannels())
			{
				SendAllPlaybackStatusesForChannelAndAssetPath(InReplyToAddress, Channel->GetChannelName().ToString(), FSoftObjectPath());
			}
		}
	}
	else
	{
		if (InInstanceId.IsValid())
		{
			// Possibilities: Missing, Syncing, Available, Loaded, Started.
			// TODO: For syncing asset, we need to query the status of the transfer from StormSync, but we probably can do that on the client.
			if (const TSharedPtr<FAvaPlaybackInstance> PlaybackInstance = FindActivePlaybackInstance(InInstanceId))
			{
				PlaybackInstance->UpdateStatus();
				SendPlaybackStatus(InReplyToAddress, InInstanceId, InChannelName, InAssetPath, PlaybackInstance->GetStatus());
			}
			else
			{
				SendPlaybackStatus(InReplyToAddress, InInstanceId, InChannelName, InAssetPath, GetUnloadedPlaybackStatus(InAssetPath));
			}
		}
		else
		{
			// In case the instance id is not specified, we send the status of any playback we have for the given asset.
			SendAllPlaybackStatusesForChannelAndAssetPath(InReplyToAddress, InChannelName, InAssetPath);
		}
	}
}

void FAvaPlaybackServer::SendPlaybackStatus(const FMessageAddress& InSendTo, const FGuid& InInstanceId, const FString& InChannelName, const FSoftObjectPath& InAssetPath, EAvaPlaybackStatus InStatus)
{
	using namespace UE::AvaPlaybackServer::Private;
	SendResponse(MakePlaybackStatusMessage(InInstanceId, InChannelName, InAssetPath, InStatus), InSendTo);
}

void FAvaPlaybackServer::SendPlaybackStatus(const TArray<FMessageAddress>& InRecipients, const FGuid& InInstanceId, const FString& InChannelName, const FSoftObjectPath& InAssetPath, EAvaPlaybackStatus InStatus)
{
	using namespace UE::AvaPlaybackServer::Private;
	SendResponse(MakePlaybackStatusMessage(InInstanceId, InChannelName, InAssetPath, InStatus), InRecipients);
}

void FAvaPlaybackServer::SendPlaybackStatuses(const FMessageAddress& InSendTo, const FString& InChannelName, const TArray<FPlaybackInstanceReference>& InInstances, EAvaPlaybackStatus InStatus)
{
	FAvaPlaybackStatuses* Response = FMessageEndpoint::MakeMessage<FAvaPlaybackStatuses>();
	Response->ChannelName = InChannelName;
	Response->AssetPaths.Reserve(InInstances.Num());
	Response->InstanceIds.Reserve(InInstances.Num());
	for (const FPlaybackInstanceReference& Instance : InInstances)
	{
		Response->AssetPaths.Add(Instance.Path);
		Response->InstanceIds.Add(Instance.Id);
	}
	Response->Status = InStatus;
	SendResponse(Response, InSendTo);
}

void FAvaPlaybackServer::SendAllPlaybackStatusesForChannelAndAssetPath(const FMessageAddress& InSendTo, const FString& InChannelName, const FSoftObjectPath& InAssetPath)
{
	// Group all the playback objects per status.
	TMap<EAvaPlaybackStatus, TArray<FPlaybackInstanceReference>> InstancesPerStatus;

	for (const TPair<FGuid, TSharedPtr<FAvaPlaybackInstance>>& InstanceEntry : ActivePlaybackInstances)
	{
		if (const TSharedPtr<FAvaPlaybackInstance> Instance = InstanceEntry.Value)
		{
			if (!InChannelName.IsEmpty() && Instance->GetChannelName() != InChannelName)
			{
				continue;
			}

			if (!InAssetPath.IsNull() && Instance->GetSourcePath() != InAssetPath)
			{
				continue;
			}

			TArray<FPlaybackInstanceReference>* Instances = InstancesPerStatus.Find(Instance->GetStatus());
			if (!Instances)
			{
				Instances = &InstancesPerStatus.Emplace(Instance->GetStatus());
				Instances->Reserve(32);
			}
			Instances->Add({Instance->GetInstanceId(), Instance->GetSourcePath()});
		}
	}

	for (TPair<EAvaPlaybackStatus, TArray<FPlaybackInstanceReference>>& StatusEntry :  InstancesPerStatus)
	{
		SendPlaybackStatuses(InSendTo, InChannelName, StatusEntry.Value, StatusEntry.Key);
	}

	// In the case there is no playing instances, return the status of the unloaded asset as playback status.
	if (InstancesPerStatus.IsEmpty())
	{
		SendPlaybackStatus(InSendTo, FGuid(), InChannelName, InAssetPath, GetUnloadedPlaybackStatus(InAssetPath));
	}
}

void FAvaPlaybackServer::SendPlaybackAssetStatus(const FMessageAddress& InSendTo, const FSoftObjectPath& InAssetPath, EAvaPlaybackAssetStatus InStatus)
{
	FAvaPlaybackAssetStatus* Response = FMessageEndpoint::MakeMessage<FAvaPlaybackAssetStatus>();
	Response->AssetPath = InAssetPath;
	Response->Status = InStatus;
	SendResponse(Response, InSendTo);
}

bool FAvaPlaybackServer::UpdateChannelOutputConfig(FAvaBroadcastOutputChannel& InChannel,
	const TArray<FAvaBroadcastOutputData>& InMediaOutputs, bool bInRefreshState)
{
	if (InChannel.GetState() == EAvaBroadcastChannelState::Live)
	{
		UE_LOGF(LogAvaPlaybackServer, Error, "Failed to update output config on channel \"%ls\". Channel is live.",
			   *InChannel.GetChannelName().ToString());
		return false;
	}

	if (!InMediaOutputs.IsEmpty())
	{
		TArray<TStrongObjectPtr<UMediaOutput>> NewOutputs;
		NewOutputs.Reserve(InMediaOutputs.Num());
		TArray<FAvaBroadcastMediaOutputInfo> NewOutputInfos;
		NewOutputInfos.Reserve(InMediaOutputs.Num());

		for (const FAvaBroadcastOutputData& MediaOutputData : InMediaOutputs)
		{
			// Important: don't add outputs not destined for this server.
			if (MediaOutputData.OutputInfo.ServerName == ServerName)
			{
				NewOutputs.Emplace_GetRef().Reset(
					UE::AvaBroadcastOutputUtils::CreateMediaOutput(MediaOutputData, &UAvaBroadcast::Get()));
				NewOutputInfos.Add(MediaOutputData.OutputInfo);
			}
			else
			{
				UE_LOGF(LogAvaPlaybackServer, Warning,
					   "Channel \"%ls\" received an output for another server (\"%ls\")",
					   *InChannel.GetChannelName().ToString(), *MediaOutputData.OutputInfo.ServerName);
			}
		}

		if (!NewOutputs.IsEmpty())
		{
			{
				// Both RemoveMediaOutput and AddMediaOutput will broadcast channel events
				// we don't want those temporary states to propagate to the playback client.
				TGuardValue<bool> BlockChannelStatusUpdate(bBlockChannelStatusUpdate, true);
			
				TArray<UMediaOutput*> MediaOutputs = InChannel.GetMediaOutputs();
				for (UMediaOutput* MediaOutput : MediaOutputs)
				{
					InChannel.RemoveMediaOutput(MediaOutput);
				}

				for (int32 Index = 0; Index < NewOutputs.Num(); ++Index)
				{
					// Make the device info "local" for this server.
					NewOutputInfos[Index].ServerName = FAvaBroadcastDeviceProviderProxyManager::LocalServerName;
					InChannel.AddMediaOutput(NewOutputs[Index].Get(), NewOutputInfos[Index]);
				}
			}

			// We may not desired refresh state here to avoid spurious states if
			// we are in the middle of a series of commands.
			if (bInRefreshState)
			{
				InChannel.RefreshState();
			}
			return true;
		}
	}
	return false;
}

FAvaPlaybackServer::FClientInfo& FAvaPlaybackServer::GetOrCreateClientInfo(
	const FString& InClientName, const FMessageAddress& InClientAddress)
{
	if (const TSharedPtr<FClientInfo>* ClientInfo = Clients.Find(InClientName))
	{
		if ((*ClientInfo)->Address != InClientAddress)
		{
			// This is suspicious though. It may also indicate a collision with multiple clients
			// with the same name on the same computer host. This is a case we don't support for now.
			UE_LOGF(LogAvaPlaybackServer, Warning, "Client \"%ls\" Address changed, possible collision between clients with same name.", *InClientName);
			(*ClientInfo)->Address = InClientAddress;
		}
		return *(ClientInfo->Get());
	}

	const TSharedPtr<FClientInfo> ClientInfo = MakeShared<FClientInfo>(InClientAddress, InClientName);
	ClientInfo->MediaSyncManager->OnAvaAssetSyncStatusReceived.AddRaw(this, &FAvaPlaybackServer::OnAvaAssetSyncStatusReceived);
	Clients.Add(InClientName, ClientInfo);

	OnClientAdded(*ClientInfo);
	
	return *ClientInfo;
}

FAvaPlaybackServer::FClientInfo* FAvaPlaybackServer::GetClientInfo(const FMessageAddress& InClientAddress) const
{
	for (const TPair<FString, TSharedPtr<FClientInfo>>& Client : Clients )
	{
		if (Client.Value->Address == InClientAddress)
		{
			return Client.Value.Get();
		}
	}
	return nullptr;
}

const FString& FAvaPlaybackServer::GetClientNameSafe(const FMessageAddress& InClientAddress) const
{
	if (const FClientInfo* ClientInfo = GetClientInfo(InClientAddress))
	{
		return ClientInfo->ClientName;
	}
	static FString ClientNotFoundString(TEXT("[ClientNotFound]"));
	return ClientNotFoundString;
}

const FMessageAddress& FAvaPlaybackServer::GetClientAddressSafe(const FString& InClientName) const
{
	if (const FClientInfo* ClientInfo = GetClientInfo(InClientName))
	{
		return ClientInfo->Address;
	}
	static FMessageAddress Invalid;
	return Invalid;
}


TArray<FMessageAddress> FAvaPlaybackServer::GetAllClientAddresses(bool bInExcludeClientOnLocalProcess) const
{
	TArray<FMessageAddress> OutAddresses;
	OutAddresses.Reserve(Clients.Num());
	for (const TPair<FString, TSharedPtr<FClientInfo>>& Client : Clients )
	{
		if (bInExcludeClientOnLocalProcess && IsClientOnLocalProcess(*Client.Value))
		{
			continue;
		}
		
		OutAddresses.Add(Client.Value->Address);
	}
	return OutAddresses;
}

void FAvaPlaybackServer::RemoveDeadClients(const FDateTime& InCurrentTime)
{
	for (TMap<FString, TSharedPtr<FClientInfo>>::TIterator ServerIter(Clients); ServerIter; ++ServerIter)
	{
		if (ServerIter.Value()->HasTimedOut(InCurrentTime))
		{
			UE_LOGF(LogAvaPlaybackServer, Log, "Client \"%ls\" is not longer sending pings. Removing.",
				   *ServerIter.Key());
			const TSharedPtr<FClientInfo> RemovedClient = ServerIter.Value();
			ServerIter.RemoveCurrent();
			OnClientRemoved(*RemovedClient);
		}
	}
}

void FAvaPlaybackServer::OnClientAdded(const FClientInfo& InClientInfo)
{
	UE_LOGF(LogAvaPlaybackServer, Log, "Registering new playback client \"%ls\".", *InClientInfo.ClientName);
	// We send user data update on connection only (reliable send).
	SendUserDataUpdate({InClientInfo.Address});
}

void FAvaPlaybackServer::OnClientRemoved(const FClientInfo& InRemovedClient)
{
	// TODO	
}

bool FAvaPlaybackServer::IsLocalClient(const FClientInfo& ClientInfo) const
{
	return ClientInfo.ComputerName == ComputerName && ClientInfo.ProjectContentPath == ProjectContentPath;
}

FAvaPlaybackServer::FClientInfo::FClientInfo(const FMessageAddress& InClientAddress, const FString& InClientName)
	: Address(InClientAddress)
	, ClientName(InClientName)
{
	MediaSyncManager = MakeShared<FAvaPlaybackSyncManager>(InClientName);
}

FAvaPlaybackServer::FClientInfo::~FClientInfo() = default;

FAvaPlaybackServer::FReplicationOutputDevice::FReplicationOutputDevice(FAvaPlaybackServer* InServer)
		: Server(InServer)
{
	GLog->AddOutputDevice(this);
}
		
FAvaPlaybackServer::FReplicationOutputDevice::~FReplicationOutputDevice()
{
	// At shutdown, GLog may already be null
	if (GLog != nullptr)
	{
		GLog->RemoveOutputDevice(this);
	}
}

void FAvaPlaybackServer::FReplicationOutputDevice::SetVerbosityThreshold(ELogVerbosity::Type InVerbosityThreshold)
{
	VerbosityThreshold = InVerbosityThreshold;
}

void FAvaPlaybackServer::FReplicationOutputDevice::Serialize(const TCHAR* InText, ELogVerbosity::Type InVerbosity, const FName& InCategory)
{
	Serialize(InText, InVerbosity, InCategory, {});
}
	
void FAvaPlaybackServer::FReplicationOutputDevice::Serialize(const TCHAR* InText, ELogVerbosity::Type InVerbosity, const FName& InCategory, double InTime)
{
	if (Server && InVerbosity <= VerbosityThreshold)
	{
		Server->SendLogMessage(InText, InVerbosity, InCategory, InTime);
	}
}