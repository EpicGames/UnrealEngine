// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Broadcast/IAvaBroadcastSettings.h"
#include "Framework/AvaInstanceSettings.h"
#include "Playback/IAvaPlaybackServer.h"
#include "IMessageContext.h"
#include "MessageEndpoint.h"
#include "Playable/AvaPlayableSettings.h"
#include "Playback/AvaPlaybackManager.h"
#include "Playback/AvaPlaybackMessages.h"
#include "Templates/SharedPointer.h"

class FAvaPlaybackInstance;
class FAvaPlaybackSyncManager;
class UAvaPlaybackServerTransition;
struct FAvaPlaybackAssetSyncStatusReceivedParams;

DECLARE_LOG_CATEGORY_EXTERN(LogAvaPlaybackServer, Log, All);

/**
 * Playback (and Broadcast) Server
 *
 * The playback server implements the commands for broadcast (channels and outputs) and playback.
 * Playback assets are either "playables" or "playback graphs", however the playback server will
 * create playback graphs for everything. If the asset to play is a "playable" (e.g. a level),
 * it will create a transient playback graph for it.
 *
 * A lot of the playback commands are geared toward running a playback graph with a single playable node,
 * mostly because this system is only used with rundowns or playback graphs run on the client side.
 * So, it will emulate a client-side playable with a local transient playback graph with one player node.
 * The use case of running a more complex playback graph asset on the server side has not occured yet.
 */
class FAvaPlaybackServer : public TSharedFromThis<FAvaPlaybackServer>, public IAvaPlaybackServer
{
public:
	FAvaPlaybackServer();
	virtual ~FAvaPlaybackServer() override;

	void Init(const FString& InAssignedServerName);
	
	/**
	 * Returns a list of channel names from all the playing playback instances.
	 * Optionally filter for a given asset.
	 */
	TArray<FString> GetAllChannelsFromPlayingPlaybacks(const FSoftObjectPath& InAssetPath = FSoftObjectPath()) const;
	
	/**
	 *	Indicate the manager is in a shutdown sequence and will force game instances to destroy worlds right away.
	*/
	void StartShuttingDown();
	
	//~ Begin IAvaPlaybackServer Interface
	
	virtual TArray<FPlaybackInstanceReference> StopPlaybacks(const FString& InChannelName = FString(), const FSoftObjectPath& InAssetPath = FSoftObjectPath(), bool bInUnload = true) override;
	virtual TArray<FPlaybackInstanceReference> StartPlaybacks() override;
	
	virtual void StartBroadcast() override;
	virtual void StopBroadcast() override;
	
	virtual const FString& GetName() const override { return ServerName;}

	virtual bool HasUserData(const FString& InKey) const override { return UserDataEntries.Contains(InKey);}
	
	virtual const FString& GetUserData(const FString& InKey) const override;

	virtual void SetUserData(const FString& InKey, const FString& InData) override;

	virtual void RemoveUserData(const FString& InKey) override;

	virtual TArray<FString> GetClientNames() const override;

	virtual FMessageAddress GetClientAddress(const FString& InClientName) const override;

	virtual bool HasClientUserData(const FString& InClientName, const FString& InKey) const override;

	virtual const FString& GetClientUserData(const FString& InClientName, const FString& InKey) const override;

	virtual const IAvaBroadcastSettings* GetBroadcastSettings() const override;

	virtual const FAvaInstanceSettings* GetAvaInstanceSettings() const override;

	virtual const FAvaPlayableSettings* GetPlayableSettings() const override;
	
	virtual const FAvaPlaybackManager& GetPlaybackManager() const override { check(Manager); return *Manager; }
	virtual FAvaPlaybackManager& GetPlaybackManager() override { check(Manager); return *Manager; }

	//~ End IAvaPlaybackServer Interface
	
	TSharedPtr<FAvaPlaybackInstance> FindActivePlaybackInstance(const FGuid& InInstanceId) const
	{
		const TSharedPtr<FAvaPlaybackInstance>* FoundInstance = ActivePlaybackInstances.Find(InInstanceId);
		return FoundInstance ? *FoundInstance : TSharedPtr<FAvaPlaybackInstance>();
	}

	bool RemoveActivePlaybackInstance(const FGuid& InInstanceId)
	{
		return ActivePlaybackInstances.Remove(InInstanceId) > 0;	
	}

	bool RemovePlaybackInstanceTransition(const FGuid& InTransitionId);
	
	void SendPlayableTransitionEvent(
		const FGuid& InTransitionId, const FAvaPlaybackTransitionEventEntry& InEntry,
		const FName& InChannelName, const FString& InClientName);
	
	void SendPlayableTransitionEventMulti(
		const FGuid& InTransitionId, TArray<FAvaPlaybackTransitionEventEntry>&& InEntries,
		const FName& InChannelName, const FString& InClientName);
	
public:
	// Message handlers
	void HandlePlaybackPing(const FAvaPlaybackPing& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleUpdateClientUserData(const FAvaPlaybackUpdateClientUserData& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleStatCommand(const FAvaPlaybackStatCommand& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleDeviceProviderDataRequest(const FAvaPlaybackDeviceProviderDataRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleUpdateClientInfo(const FAvaPlaybackUpdateClientInfo& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleAvaInstanceSettingsUpdate(const FAvaPlaybackInstanceSettingsUpdate& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandlePlayableSettingsUpdate(const FAvaPlaybackPlayableSettingsUpdate& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandlePackageEvent(const FAvaPlaybackPackageEvent& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandlePlaybackAssetStatusRequest(const FAvaPlaybackAssetStatusRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandlePlaybackRequest(const FAvaPlaybackRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleAnimPlaybackRequest(const FAvaPlaybackAnimPlaybackRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleRemoteControlUpdateRequest(const FAvaPlaybackRemoteControlUpdateRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandlePlayableTransitionStartRequest(const FAvaPlaybackTransitionStartRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandlePlayableTransitionStopRequest(const FAvaPlaybackTransitionStopRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleBroadcastSettingsUpdate(const FAvaBroadcastSettingsUpdate& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleBroadcastRequest(const FAvaBroadcastRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleBroadcastChannelSettingsUpdate(const FAvaBroadcastChannelSettingsUpdate& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleBroadcastStatusRequest(const FAvaBroadcastStatusRequest& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);

	/** Returns underlying message bus endpoint address id  */
	FString GetMessageEndpointAddressId() const;

protected:
	void Tick();

	void RegisterCommands();
	
	// Command handlers
	void StartPlaybackCommand(const TArray<FString>& InArgs);
	void StopPlaybackCommand(const TArray<FString>& InArgs);
	void StartBroadcastCommand(const TArray<FString>& InArgs);
	void StopBroadcastCommand(const TArray<FString>& InArgs);
	void SetUserDataCommand(const TArray<FString>& InArgs);
	void ShowStatusCommand(const TArray<FString>& InArgs);
	
	// Event handlers
	void OnAvaMediaSettingsChanged(UObject*, struct FPropertyChangedEvent&);
	void OnChannelChanged(const FAvaBroadcastOutputChannel& InChannel, EAvaBroadcastChannelChange InChange);
	void OnMediaOutputStateChanged(const FAvaBroadcastOutputChannel& InChannel, const UMediaOutput* InMediaOutput);
	void OnAvaAssetSyncStatusReceived(const FAvaPlaybackAssetSyncStatusReceivedParams& InParams);
	void OnPlaybackInstanceInvalidated(const FAvaPlaybackInstance& InPlaybackInstance);
	void OnPlaybackInstanceStatusChanged(const FAvaPlaybackInstance& InPlaybackInstance);
	void OnPlaybackAssetRemoved(const FSoftObjectPath& InAssetPath);
	void OnPlayableSequenceEvent(UAvaPlayable* InPlayable, FName InSequenceLabel, EAvaPlayableSequenceEventType InEventType);
	
	void ApplyAvaMediaSettings();

	template<typename MessageType>
	void FillServerInfo(MessageType* InMessage)
	{
		InMessage->ServerName = ServerName;
	}
	
	template<typename MessageType>
	void SendResponse(MessageType* InMessage, const FMessageAddress& InRecipient, EMessageFlags flags = EMessageFlags::None)
	{
		FillServerInfo(InMessage);
		MessageEndpoint->Send(InMessage, MessageType::StaticStruct(), flags, nullptr,
			TArrayBuilder<FMessageAddress>().Add(InRecipient), FTimespan::Zero(), FDateTime::MaxValue());
	}

	template<typename MessageType>
	void SendResponse(MessageType* InMessage, const TArray<FMessageAddress>& InRecipients, EMessageFlags flags = EMessageFlags::None)
	{
		FillServerInfo(InMessage);
		MessageEndpoint->Send(InMessage, MessageType::StaticStruct(), flags, nullptr,
			InRecipients, FTimespan::Zero(), FDateTime::MaxValue());
	}

	void SendUserDataUpdate(const TArray<FMessageAddress>& InRecipients);

	void SendChannelStatusUpdate(const FString& InChannelName, const FAvaBroadcastOutputChannel& InChannel, const FMessageAddress& InSender, bool bInIncludeOutputData = false);
	void SendAllChannelStatusUpdate(const FMessageAddress& InSender, bool bInIncludeOutputData = false);
	void SendLogMessage(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category, double Time);

	// Playback Commands
	void ExecutePendingPlaybackCommands(const FDateTime& InUtcNow);

	TSharedPtr<FAvaPlaybackInstance> GetOrLoadPlaybackInstance(const FGuid& InInstanceId, const FString& InChannelName, const FSoftObjectPath& InAssetPath, const FString& InLoadOptions);
	void LoadPlayback(const FMessageAddress& InReplyToAddress, const FGuid& InInstanceId, const FString& InChannelName, const FSoftObjectPath& InAssetPath, const FString& InLoadOptions);
	void StartPlayback(const FMessageAddress& InReplyToAddress, const FGuid& InInstanceId, const FString& InChannelName, const FSoftObjectPath& InAssetPath, const FString& InLoadOptions);
	void StopPlayback(const FMessageAddress& InReplyToAddress, const FGuid& InInstanceId, const FString& InChannelName, const FSoftObjectPath& InAssetPath);
	void UnloadPlayback(const FMessageAddress& InReplyToAddress, const FGuid& InInstanceId, const FString& InChannelName, const FSoftObjectPath& InAssetPath);
	bool SetPlaybackUserData(const FMessageAddress& InReplyToAddress, const FGuid& InInstanceId, const FString& InUserData);
	bool SendPlaybackUserData(const FMessageAddress& InReplyToAddress, const FGuid& InInstanceId);
	void SendPlaybackStatus(const FMessageAddress& InReplyToAddress, const FGuid& InInstanceId, const FString& InChannelName, const FSoftObjectPath& InAssetPath);
	
	void SendPlaybackStatus(const FMessageAddress& InSendTo, const FGuid& InInstanceId, const FString& InChannelName, const FSoftObjectPath& InAssetPath, EAvaPlaybackStatus InStatus);
	void SendPlaybackStatus(const TArray<FMessageAddress>& InRecipients, const FGuid& InInstanceId, const FString& InChannelName, const FSoftObjectPath& InAssetPath, EAvaPlaybackStatus InStatus);
	void SendPlaybackStatuses(const FMessageAddress& InSendTo, const FString& InChannelName, const TArray<FPlaybackInstanceReference>& InInstances, EAvaPlaybackStatus InStatus);
	void SendAllPlaybackStatusesForChannelAndAssetPath(const FMessageAddress& InSendTo, const FString& InChannelName, const FSoftObjectPath& InAssetPath);
	void SendPlaybackAssetStatus(const FMessageAddress& InSendTo, const FSoftObjectPath& InAssetPath, EAvaPlaybackAssetStatus InStatus);

	EAvaPlaybackStatus GetUnloadedPlaybackStatus(const FSoftObjectPath& InAssetPath)
	{
		return Manager->GetUnloadedPlaybackStatus(InAssetPath);
	}

	bool UpdateChannelOutputConfig(FAvaBroadcastOutputChannel& InChannel, const TArray<FAvaBroadcastOutputData>& InMediaOutputs, bool bInRefreshState);

private:
	FString ComputerName;
	FString ServerName;
	FString ProjectContentPath;
	uint32 ProcessId = 0;
	TMap<FString, FString> UserDataEntries;
	
	/** Holds the messaging endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	TArray<IConsoleObject*> ConsoleCommands;

	/** The playback server has its own playback manager to not interfere with the local one. */
	TSharedPtr<FAvaPlaybackManager> Manager;

	// This is used to block sending status update from the channel event while the channels
	// are refreshing state on media output state changes. This avoid sending spurious channel states
	// while the update is not completed for all outputs.
	bool bBlockChannelStatusUpdate = false;

	struct FPendingPlaybackCommand
	{
		FDateTime ReceivedUtc;
		uint32 ReceivedFrameNumber;
		int32 Priority;
		FMessageAddress ReplyTo;
		FAvaPlaybackCommand Command;

		FPendingPlaybackCommand(const FDateTime& InReceivedUtc, uint32 InReceivedFrameNumber, int32 InPriority, const FMessageAddress& InReplyTo, const FAvaPlaybackCommand& InCommand)
			: ReceivedUtc(InReceivedUtc), ReceivedFrameNumber(InReceivedFrameNumber), Priority(InPriority), ReplyTo(InReplyTo), Command(InCommand)
		{}
	};

	/** Accumulate all the playback commands and execute them all in one batch on the next tick. */
	TArray<TSharedPtr<FPendingPlaybackCommand>> PendingPlaybackCommands;

	/** Keep an map of active instances per id for fast lookup. */
	TMap<FGuid, TSharedPtr<FAvaPlaybackInstance>> ActivePlaybackInstances;
	
	class FServerPlaybackInstanceTransitionCollection;
	TUniquePtr<FServerPlaybackInstanceTransitionCollection> PlaybackInstanceTransitions;
	
	/**
	 * Routes the log messages to the server for replication over the message bus.
	 */
	class FReplicationOutputDevice : public FOutputDevice
	{
	public:
		FReplicationOutputDevice(FAvaPlaybackServer* InServer);
		virtual ~FReplicationOutputDevice() override;

		/** Set the minimum verbosity that will be replicated to the client. */
		void SetVerbosityThreshold(ELogVerbosity::Type InVerbosityThreshold);

	protected:
		//~ FOutputDevice interface.
		virtual void Serialize(const TCHAR* InText, ELogVerbosity::Type InVerbosity, const FName& InCategory) override;
		virtual void Serialize(const TCHAR* InText, ELogVerbosity::Type InVerbosity, const FName& InCategory, double InTime) override;

	private:
		FAvaPlaybackServer* Server;
		ELogVerbosity::Type VerbosityThreshold = ELogVerbosity::Type::NoLogging; 
	};
	
	TUniquePtr<FReplicationOutputDevice> ReplicationOutputDevice;

	TOptional<ELogVerbosity::Type> LogReplicationVerbosityFromCommandLine;

	class FClientBroadcastSettings : public IAvaBroadcastSettings
	{
	public:
		FAvaBroadcastSettings Settings;
		virtual ~FClientBroadcastSettings() override = default;

		// IAvaBroadcastSettings
		virtual const FLinearColor& GetChannelClearColor() const override { return Settings.ChannelClearColor; }
		virtual EPixelFormat GetDefaultPixelFormat() const override { return Settings.ChannelDefaultPixelFormat; }
		virtual const FIntPoint& GetDefaultResolution() const override { return Settings.ChannelDefaultResolution; }
		virtual bool IsDrawPlaceholderWidget() const override { return Settings.bDrawPlaceholderWidget; }
		virtual const FSoftObjectPath& GetPlaceholderWidgetClass() const override { return Settings.PlaceholderWidgetClass; }
		// ~IAvaBroadcastSettings
	};
	
	/** Remote Client Info */
	class FClientInfo
	{
	public:
		FMessageAddress Address;
		FString ClientName;
		FString ComputerName;
		FString ProjectContentPath;
		uint32 ProcessId = 0;
		TMap<FString, FString> UserDataEntries;
		FClientBroadcastSettings BroadcastSettings;
		FAvaInstanceSettings AvaInstanceSettings;
		FAvaPlayableSettings PlayableSettings;

		bool bClientInfoReceived = false;
	
		/**
		 * The status of the server's asset compared to the remote is tracked
		 * on the server side. For now, the remote client is considered to have the
		 * reference asset. But in the future, a hub may hold the reference asset instead.
		 */
		TSharedPtr<FAvaPlaybackSyncManager> MediaSyncManager;

		TOptional<FDateTime> PingTimeout;
		
		FClientInfo(const FMessageAddress& InClientAddress, const FString& InClientName);
		~FClientInfo();

		
		/**
		* Add a time by which a new ping should arrive
		*/
		void AddTimeout(const FDateTime& InNewTimeout)
		{
			if (!PingTimeout || InNewTimeout < PingTimeout.GetValue())
			{
				PingTimeout = InNewTimeout;
			}
		}

		/**
		 * Check if new ping arrived on time
		*/
		bool HasTimedOut(const FDateTime& InNow) const
		{
			return PingTimeout && InNow > PingTimeout.GetValue();
		}

		/**
		 * Resets the ping timeout.
		 * This is done when the ping response is received.
		 */
		void ResetPingTimeout()
		{
			PingTimeout.Reset();
		}
	};
	TMap<FString, TSharedPtr<FClientInfo>> Clients;	// Key is ClientName

	FClientInfo& GetOrCreateClientInfo(const FString& InClientName, const FMessageAddress& InClientAddress);
	FClientInfo* GetClientInfo(const FMessageAddress& InClientAddress) const;
	FClientInfo* GetClientInfo(const FString& InClientName) const
	{
		const TSharedPtr<FClientInfo>* ClientInfoPtr = Clients.Find(InClientName);
		return ClientInfoPtr ? ClientInfoPtr->Get() : nullptr;
	}
	const FString& GetClientNameSafe(const FMessageAddress& InClientAddress) const;
	const FMessageAddress& GetClientAddressSafe(const FString& InClientName) const;
	TArray<FMessageAddress> GetAllClientAddresses(bool bInExcludeClientOnLocalProcess = false) const;
	void RemoveDeadClients(const FDateTime& InCurrentTime);
	void OnClientAdded(const FClientInfo& InClientInfo);
	void OnClientRemoved(const FClientInfo& InRemovedClient);
	/** A local client will run from the same computer and project folder as the current server. */
	bool IsLocalClient(const FClientInfo& ClientInfo) const;
	bool IsLocalClient(const FMessageAddress& InClientAddress) const
	{
		const FClientInfo* ClientInfo = GetClientInfo(InClientAddress);
		return ClientInfo && IsLocalClient(*ClientInfo);
	}
	bool IsClientOnLocalProcess(const FClientInfo& ClientInfo) const
	{
		return ClientInfo.ComputerName == ComputerName && ClientInfo.ProcessId == ProcessId;
	}
};
