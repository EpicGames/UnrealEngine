// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPlaybackServerProcess.h"

#include "AvaMediaSettings.h"
#include "IAvaMediaModule.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Playback/AvaPlaybackClient.h"

namespace UE::AvaPlaybackServerProcess::Private
{
	FString GetLocalServerName()
	{
		return UAvaMediaSettings::Get().LocalPlaybackServerSettings.ServerName;
	}

	const TCHAR* GetPlaybackServerLogReplicationVerbosityString()
	{
		const UAvaMediaSettings& Settings = UAvaMediaSettings::Get(); 
		return ToString(UAvaMediaSettings::ToLogVerbosity(Settings.PlaybackServerLogReplicationVerbosity));
	}
}

FAvaPlaybackServerProcess::FAvaPlaybackServerProcess(FProcHandle&& InProcessHandle)
	: ProcessHandle(MoveTemp(InProcessHandle))
{}

bool FAvaPlaybackServerProcess::IsLaunched()
{
	return ProcessHandle.IsValid() && FPlatformProcess::IsProcRunning(ProcessHandle);
}

bool FAvaPlaybackServerProcess::Launch()
{
	if (IsLaunched())
	{
		UE_LOGF(LogAvaMedia, Warning, "Local playback server is already running.");
		return true;
	}

	using namespace UE::AvaPlaybackServerProcess::Private;

	FString GameNameOrProjectFile;
	if (FPaths::IsProjectFilePathSet())
	{
		GameNameOrProjectFile = FString::Printf(TEXT("\"%s\""), *FPaths::GetProjectFilePath());
	}
	else
	{
		GameNameOrProjectFile = FApp::GetProjectName();
	}

	// Todo: It will likely need a more complete set of parameters. Reference: UDisplayClusterLaunchEditorProjectSettings.
	const FString CommandLine = FString::Printf(TEXT("%s %s")
		, *GameNameOrProjectFile
		, *UAvaMediaSettings::Get().GenerateLocalPlaybackServerCommandLine());

	const FString ExecutablePath = FPlatformProcess::ExecutablePath();

	uint32 ProcessID = 0;
	constexpr bool bLaunchDetached = true;
	constexpr bool bLaunchMinimized = false;
	constexpr bool bLaunchWindowHidden = false;
	constexpr uint32 PriorityModifier = 0;

	UE_LOGF(LogAvaMedia, Log, "Launching a playback server in game mode in a new process with the following command line:");
	UE_LOGF(LogAvaMedia, Log, "%ls %ls", *ExecutablePath, *CommandLine);

	ProcessHandle = FPlatformProcess::CreateProc(
		*ExecutablePath, *CommandLine, bLaunchDetached,
		bLaunchMinimized, bLaunchWindowHidden, &ProcessID,
		PriorityModifier, nullptr, nullptr, nullptr);
	
	return ProcessHandle.IsValid();
}

void FAvaPlaybackServerProcess::Stop()
{
	FPlatformProcess::TerminateProc(ProcessHandle);
}

TSharedPtr<FAvaPlaybackServerProcess> FAvaPlaybackServerProcess::Find(const FAvaPlaybackClient& InPlaybackClient)
{
	using namespace UE::AvaPlaybackServerProcess::Private;

	// We use the playback client to find connected server processes that are considered local.
	// Local means that it is in the same project content path and the same machine. We figure out
	// if it is the same machine if the process Id can be opened.

	const FString ProjectContentPath = InPlaybackClient.GetProjectContentPath();
	const uint32 ClientProcessId = FPlatformProcess::GetCurrentProcessId();
	const FString LocalServerName = GetLocalServerName();
	
	// First try with the configured local server name.
	if (InPlaybackClient.GetServerProjectContentPath(LocalServerName) == ProjectContentPath)
	{
		const uint32 ServerProcessId = InPlaybackClient.GetServerProcessId(LocalServerName);
		if (ServerProcessId != 0 && ClientProcessId != ServerProcessId)
		{
			FProcHandle ProcessHandle = FPlatformProcess::OpenProcess(ServerProcessId);
			if (ProcessHandle.IsValid())
			{
				return MakeShared<FAvaPlaybackServerProcess>(MoveTemp(ProcessHandle));
			}

			// Warn about the server name collision issue.
			UE_LOGF(LogAvaMedia, Warning, "Found Connected Server \"%ls\" but it is not a local process.", *LocalServerName);
			UE_LOGF(LogAvaMedia, Warning, "This indicates a name collision between the servers. The local server should be renamed.");
		}
	}

	// Fallback
	// Try to find any connected server that is "local", i.e. on the same machine and
	// in the same project content folder, but on a different process id.
	
	TArray<FString> ServerNames = InPlaybackClient.GetServerNames();
	for (const FString& ServerName : ServerNames)
	{
		if (InPlaybackClient.GetServerProjectContentPath(ServerName) == ProjectContentPath)
		{
			const uint32 ServerProcessId = InPlaybackClient.GetServerProcessId(ServerName);
			if (ClientProcessId != ServerProcessId)
			{
				FProcHandle ProcessHandle = FPlatformProcess::OpenProcess(ServerProcessId);
				if (ProcessHandle.IsValid())
				{
					return MakeShared<FAvaPlaybackServerProcess>(MoveTemp(ProcessHandle));
				}
			}
		}
	}
	
	return nullptr;	
}

TSharedPtr<FAvaPlaybackServerProcess> FAvaPlaybackServerProcess::FindOrCreate(const FAvaPlaybackClient& InPlaybackClient)
{
	TSharedPtr<FAvaPlaybackServerProcess> ServerProcess = Find(InPlaybackClient);
	if (!ServerProcess.IsValid())
	{
		ServerProcess = MakeShared<FAvaPlaybackServerProcess>();
	}
	return ServerProcess;
}