// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformProcess.h"
#include "Templates/SharedPointer.h"

class FAvaPlaybackClient;

/**
 * Helper class wrapping and managing a process handle to the game mode local playback server process. 
 */
class FAvaPlaybackServerProcess
{
public:
	FAvaPlaybackServerProcess() = default;

	/**
	 * @brief Construct from an already running process handle.
	 * @param InProcessHandle Existing process handle.
	 */
	explicit FAvaPlaybackServerProcess(FProcHandle&& InProcessHandle);
	
	bool IsLaunched();
	bool Launch();
	void Stop();

	/**
	 * @brief Returns a playback server process object wrapping the process handle of an existing server process.
	 * @param InPlaybackClient Playback client instance to use for finding the server.
	 * @return A new object wrapping the process handle of the existing server process. It will return null if no existing server is found.
	 */
	static TSharedPtr<FAvaPlaybackServerProcess> Find(const FAvaPlaybackClient& InPlaybackClient);
	
	/**
	 * @brief Returns a playback server process object that wraps the process handle of either the existing server process or a newly created one.
	 * 
	 * The playback server may be running in a separate process already. If so, it will be detected by the playback client.
	 * The playback client will also know it's process id so a handle to the process can be opened.
	 *
	 * @param InPlaybackClient Playback client instance to use for finding the server.
	 * @return A new object wrapping the process handle of the server process.
	 */
	static TSharedPtr<FAvaPlaybackServerProcess> FindOrCreate(const FAvaPlaybackClient& InPlaybackClient);

private:
	FProcHandle ProcessHandle;
};