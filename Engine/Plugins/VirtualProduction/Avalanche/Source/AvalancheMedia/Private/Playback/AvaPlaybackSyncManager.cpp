// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPlaybackSyncManager.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AvaMediaSettings.h"
#include "IAvaMediaModule.h"
#include "ModularFeature/IAvaMediaSyncProvider.h"
#include "Playback/AvaPlaybackServer.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"

FAvaPlaybackSyncManager::FAvaPlaybackSyncManager(const FString& InRemoteName)
	: RemoteName(InRemoteName)
{
	if (bEnabled)
	{
		RegisterEventHandlers();
	}
}

FAvaPlaybackSyncManager::~FAvaPlaybackSyncManager()
{
	UnregisterEventHandlers();
}

void FAvaPlaybackSyncManager::SetEnable(bool bInEnabled)
{
	if (bInEnabled == bEnabled)
	{
		return;
	}

	bEnabled = bInEnabled;

	if (bEnabled)
	{
		RegisterEventHandlers();
	}
	else
	{
		UnregisterEventHandlers();
		PackageStatuses.Reset();
		CompareRequests.Reset();
	}
	UE_LOGF(LogAvaPlaybackServer, Verbose, "Server's Media Sync Manager is %ls.",
		   bEnabled ? TEXT("enabled") : TEXT("disabled"));
}

void FAvaPlaybackSyncManager::Tick()
{
	const FDateTime CurrentTime = FDateTime::UtcNow();
	for (TMap<FName, FDateTime>::TIterator PendingRequestIterator(CompareRequests); PendingRequestIterator; ++PendingRequestIterator)
	{
		if (CurrentTime > PendingRequestIterator.Value())
		{
			HandleExpiredRequest(PendingRequestIterator.Key());
			PendingRequestIterator.RemoveCurrent();
		}
	}
}

TOptional<bool> FAvaPlaybackSyncManager::GetAssetSyncStatus(const FSoftObjectPath& InAssetPath, bool bInForceRefresh)
{
	// AvaMediaSyncProvider is a modular feature, it may not be available.
	if (!bEnabled || !IAvaMediaModule::Get().GetAvaMediaSyncProvider())
	{
		// Assumes the asset is locally present. Returning "false" means it doesn't need sync.
		return TOptional<bool>(false);
	}
	
	if (bInForceRefresh)
	{
		PackageStatuses.Remove(InAssetPath.GetLongPackageFName());
	}
	else if (const bool *bNeedSynchronization = PackageStatuses.Find(InAssetPath.GetLongPackageFName()))
	{
		return TOptional<bool>(*bNeedSynchronization);
	}
	
	RequestPackageSyncStatus(InAssetPath.GetLongPackageFName());
	return TOptional<bool>();
}

void FAvaPlaybackSyncManager::EnumerateAllTrackedPackages(TFunctionRef<void(const FName& /*InPackageName*/, bool /*bInNeedsSync*/)> InFunction) const
{
	for (const TPair<FName, bool>& PackageStatusEntry : PackageStatuses)
	{
		InFunction(PackageStatusEntry.Key, PackageStatusEntry.Value);
	}
}

TArray<FName> FAvaPlaybackSyncManager::GetPendingRequests() const
{
	TArray<FName> Requests;
	Requests.Reserve(CompareRequests.Num());
	for (const TPair<FName, FDateTime>& Entry : CompareRequests)
	{
		Requests.Add(Entry.Key);
	}
	return Requests;
}

void FAvaPlaybackSyncManager::RegisterEventHandlers()
{
	UPackage::PackageSavedWithContextEvent.AddRaw(this, &FAvaPlaybackSyncManager::HandlePackageSaved);
	IAvaMediaModule::Get().GetOnAvaMediaSyncPackageModified().AddRaw(this, &FAvaPlaybackSyncManager::HandleAvaSyncPackageModified);
	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		AssetRegistry->OnAssetRemoved().AddRaw(this, &FAvaPlaybackSyncManager::HandleAssetRemoved);
	}
}

void FAvaPlaybackSyncManager::UnregisterEventHandlers()
{
	UPackage::PackageSavedWithContextEvent.RemoveAll(this);
	IAvaMediaModule::Get().GetOnAvaMediaSyncPackageModified().RemoveAll(this);
	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		AssetRegistry->OnAssetRemoved().RemoveAll(this);
	}
}

void FAvaPlaybackSyncManager::RequestPackageSyncStatus(const FName& InPackageName)
{
	// Check if we already have a pending request and that it hasn't expired.
	const FDateTime CurrentTime = FDateTime::UtcNow();
	const FDateTime* PendingRequest = CompareRequests.Find(InPackageName);
	if (PendingRequest && CurrentTime < *PendingRequest)
	{
		UE_LOGF(LogAvaPlaybackServer, Verbose,
			"Sync Compare of package \"%ls\" with remote \"%ls\" is already pending.",
			*InPackageName.ToString(), *RemoteName);
		return;
	}

	if (PendingRequest)
	{
		HandleExpiredRequest(InPackageName);
		CompareRequests.Remove(InPackageName);
		return;
	}
	
	IAvaMediaSyncProvider* AvaMediaSyncProvider = IAvaMediaModule::Get().GetAvaMediaSyncProvider();
	if (!AvaMediaSyncProvider)
	{
		return;
	}
	
	const FDateTime ExpirationTime = CurrentTime + FTimespan::FromSeconds(UAvaMediaSettings::Get().ServerPendingStatusRequestTimeout);
	CompareRequests.Add(InPackageName, ExpirationTime);

	TWeakPtr<FAvaPlaybackSyncManager> LocalWeakThis = SharedThis(this);
	AvaMediaSyncProvider->CompareWithRemote(RemoteName,{InPackageName},
		FOnAvaMediaSyncCompareResponse::CreateLambda([LocalWeakThis, InPackageName](const TSharedPtr<FAvaMediaSyncCompareResponse>& Response)
		{				
			if (const TSharedPtr<FAvaPlaybackSyncManager> LocalThis = LocalWeakThis.Pin())
			{
				LocalThis->CompareRequests.Remove(InPackageName);
				if (Response->IsValid())
				{
					LocalThis->HandleSyncStatusReceived(InPackageName, Response->bNeedsSynchronization);
				}
				else
				{
					LocalThis->HandleFailedRequest(InPackageName, Response->HasError() ? Response->ToString() : FString(TEXT("Unknown error")));
				}
			}
		}));
}

void FAvaPlaybackSyncManager::HandleExpiredRequest(const FName& InPackageName)
{
	// An expired request means the storm sync layer didn't respond, and is unlikely to respond even
	// if the request is made again. The best course of action is to consider it non operational and
	// work as if the assets are up to date.
	UE_LOGF(LogAvaPlaybackServer, Warning,
		"Sync Compare of package \"%ls\" with remote \"%ls\" timed out. Asset will be considered up to date until sync layer can resume operation.",
		*InPackageName.ToString(), *RemoteName);

	// We want the client to receive a response.
	HandleSyncStatusReceived(InPackageName, false);
}

void FAvaPlaybackSyncManager::HandleFailedRequest(const FName& InPackageName, const FString& InErrorMessage)
{
	// For a failed request, we could try again, but since it is not likely to work, the same logic
	// will be applied as an expired request. The most likely event when an error occurs is for the
	// request to time out anyway. Haven't encountered an actual error.
	UE_LOGF(LogAvaPlaybackServer, Error,
		"Sync Compare of package \"%ls\" with remote \"%ls\" failed: %ls. Asset will be considered up to date until sync layer can resume operation.",
		*InPackageName.ToString(), *RemoteName, *InErrorMessage);

	// We want the client to receive a response.
	HandleSyncStatusReceived(InPackageName, false);
}

void FAvaPlaybackSyncManager::HandleSyncStatusReceived(const FName& InPackageName, bool bInNeedsSynchronization)
{
	PackageStatuses.Add(InPackageName, bInNeedsSynchronization);
	
	// Broadcast an event for each asset in the package.
	if (const IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		IAssetRegistry::FLoadPackageRegistryData RegistryData;
		if (AssetRegistry->GetAssetsByPackageName(InPackageName, RegistryData.Data))
		{
			// Asset Data is not yet in registry's cache, need to load it directly.
			if (RegistryData.Data.Num() == 0)
			{
				FString PackageFilename;
				if (FPackageName::DoesPackageExist(InPackageName.ToString(), &PackageFilename))
				{
					AssetRegistry->LoadPackageRegistryData(PackageFilename, RegistryData);
				}
			}
			
			for (const FAssetData& Asset : RegistryData.Data)
			{
				if (FAvaPlaybackManager::IsPlaybackAsset(Asset))
				{
					OnAvaAssetSyncStatusReceived.Broadcast({RemoteName, Asset.GetSoftObjectPath(), bInNeedsSynchronization});
				}
			}
		}
	}
}

void FAvaPlaybackSyncManager::HandleAvaSyncPackageModified(IAvaMediaSyncProvider* InAvaMediaSyncProvider, const FName& InPackageName)
{
	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		// If this package just got received from sync provider, it may not be loaded yet.
		AssetRegistry->WaitForPackage(InPackageName.ToString());
	}
	
	// We had a sync operation for the given package. We could assume it is now up to date.
	// It would be better if we knew the sync operation came from the remote but this is not available
	// from the IAvaMediaSyncProvider (should be added).
	HandleSyncStatusReceived(InPackageName, false);
}

void FAvaPlaybackSyncManager::HandlePackageSaved(const FString& InPackageFileName, UPackage* InPackage, FObjectPostSaveContext InObjectSaveContext)
{
	const FName PackageName = InPackage->GetFName();
	if (PackageStatuses.Contains(PackageName))
	{
		PackageStatuses.Remove(PackageName);

		// We don't know what the status is anymore, so we request it.
		RequestPackageSyncStatus(PackageName);
	}
}

void FAvaPlaybackSyncManager::HandleAssetRemoved(const FAssetData& InAssetData)
{
	if (PackageStatuses.Contains(InAssetData.PackageName))
	{
		// Invalidate status.
		PackageStatuses.Remove(InAssetData.PackageName);

		// If there are remaining assets in the package, we will ask for a status update.
		if (const IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
		{
			if (AssetRegistry->HasAssets(InAssetData.PackageName))
			{
				RequestPackageSyncStatus(InAssetData.PackageName);
			}
		}
	}
}