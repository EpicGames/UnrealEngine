// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Misc/DateTime.h"

class FObjectPostSaveContext;
class IAvaMediaSyncProvider;
struct FAssetData;

struct FAvaPlaybackAssetSyncStatusReceivedParams
{
	const FString& RemoteName;
	const FSoftObjectPath& AssetPath;
	bool bNeedsSync;
};

/**
 * Implements tracking and caching the status of local assets compared to the corresponding assets
 * on a given remote. The remote assets are considered to be the reference. For each tracked
 * assets, a status of "need sync" (true) or "up to date" (false) is kept.
 */
class FAvaPlaybackSyncManager : public TSharedFromThis<FAvaPlaybackSyncManager>
{
public:
	FAvaPlaybackSyncManager(const FString& InRemoteName);
	~FAvaPlaybackSyncManager();

	bool IsEnabled() const { return bEnabled; }
	void SetEnable(bool bInEnabled);
	
	void Tick();
	
	/**
	 * Returns the last cached asset sync status for the given asset if available.
	 * The sync status can be true (i.e. need sync) or false (i.e. up to date).
	 * If it is not available, a sync compare request is made with the specified
	 * remote and the result is posted through OnAvaAssetSyncStatusReceived event
	 * when it is received.
	 * If bInForceRefresh is true, a request will be made even if the status was already cached.
	 */
	TOptional<bool> GetAssetSyncStatus(const FSoftObjectPath& InAssetPath, bool bInForceRefresh);

	/**
	 * Enumerate all tracked packages in the manager.
	 */
	void EnumerateAllTrackedPackages(TFunctionRef<void(const FName& /*InPackageName*/, bool /*bInNeedsSync*/)> InFunction) const;

	TArray<FName> GetPendingRequests() const;
	
	/**
	 *	Delegate called when an asset sync status changed.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAvaAssetSyncStatusReceived, const FAvaPlaybackAssetSyncStatusReceivedParams&);
	FOnAvaAssetSyncStatusReceived OnAvaAssetSyncStatusReceived;

private:
	void RegisterEventHandlers();
	void UnregisterEventHandlers();
	
	void RequestPackageSyncStatus(const FName& InPackageName);
	void HandleExpiredRequest(const FName& InPackageName);
	void HandleFailedRequest(const FName& InPackageName, const FString& InErrorMessage);
	void HandleSyncStatusReceived(const FName& InPackageName, bool bInNeedsSynchronization);
	
	// Event handlers
	void HandleAvaSyncPackageModified(IAvaMediaSyncProvider* InAvaMediaSyncProvider, const FName& InPackageName);
	void HandlePackageSaved(const FString& InPackageFileName, UPackage* InPackage, FObjectPostSaveContext InObjectSaveContext);
	void HandleAssetRemoved(const FAssetData& InAssetData);

	/** Corresponding remote to be comparing with. */
	FString RemoteName;

	/** Determines if the sync manager is going to be used for the corresponding remote. */
	bool bEnabled = true;
	
	/** Maps package names and sync status. */
	TMap<FName, bool> PackageStatuses;

	/** Currently pending sync compare requests. Key is long package names. */
	TMap<FName, FDateTime> CompareRequests;
};