// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/AvaPlaybackUtils.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AvaAssetTags.h"
#include "ContentStreaming.h"
#include "Engine/StreamableRenderAsset.h"
#include "Framework/AvaSoftAssetPtr.h"
#include "HAL/PlatformFileManager.h"
#include "IAvaMediaModule.h"
#include "MaterialCache/AvaMaterialCacheHelper.h"
#include "MaterialCache/AvaMaterialCacheSettings.h"
#include "Misc/Paths.h"
#include "RenderingThread.h"
#include "ShaderCompiler.h"
#include "UObject/Linker.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "Editor.h"
#include "PackageTools.h"
#include "Selection.h"
#include "Subsystems/AssetEditorSubsystem.h"
#endif

namespace UE::AvaPlayback::Utils::Private
{
	static TAutoConsoleVariable<bool> CVarForceResetLoadersInPackageFlush(
		TEXT("MotionDesignPlayback.ForceResetLoadersInPackageFlush"),
		false,
		TEXT("Forces to ResetLoaders instead of DetachLoader during FlushPackageLoading")
	);

	static TAutoConsoleVariable<bool> CVarRuntimePrestreamRCReferencedAssets(
		TEXT("MotionDesignPlayback.RuntimePrestreamRemoteControlObjects"),
		true,
		TEXT("Manually streams and waits for all Remote Control referenced objects to be ready while preparing to start play.")
	);

	static TAutoConsoleVariable<bool> CVarRuntimePrecacheRCReferencedAssetsMaterials(
		TEXT("MotionDesignPlayback.RuntimePrecacheRemoteControlObjectMaterials"),
		false, // False by default to avoid the extra cost of having to try cache materials at runtime/playtime.
		TEXT("Manually caches and waits for materials used by Remote Control referenced objects to be complete while preparing to start play.")
	);
}

void FAvaPlaybackUtils::FlushPackageLoading(UPackage* InPackage, bool bInResetLoaders)
{
	if (!InPackage->IsFullyLoaded())
	{
		FlushAsyncLoading();
		InPackage->FullyLoad();
	}

	if (bInResetLoaders || UE::AvaPlayback::Utils::Private::CVarForceResetLoadersInPackageFlush.GetValueOnGameThread())
	{
		ResetLoaders(InPackage);
	}
	else if (FLinkerLoad* LinkerLoad = FLinkerLoad::FindExistingLinkerForPackage(InPackage))
	{
		if (!LinkerLoad->IsDestroyingLoader())
		{
			// Ideally there should be an option for the Linker to not lock the package files preventing saves.
			// Resetting the loader will cause the level to be reloaded via ConditionalFlushAsyncLoadingForLinkers, hitching the rundown playback.
			// DetachLoader allows releasing the lock on the file handle so the package can be saved.
			LinkerLoad->DetachLoader();
		}
	}
}

bool FAvaPlaybackUtils::IsPackageDeleted(const UPackage* InExistingPackage)
{
	if (!InExistingPackage)
	{
		return false;
	}
	
	const FString PackageExtension = InExistingPackage->ContainsMap() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension();
	const FString PackageFilename = FPackageName::LongPackageNameToFilename(InExistingPackage->GetName(), PackageExtension);
			
	return !FPaths::FileExists(PackageFilename);
}

namespace UE::AvaPlaybackUtils::Private
{
#if WITH_EDITOR
	void CollectObjectToPurge(UObject* InObject, TArray<UObject*>& OutObjectsToPurge)
	{
		if (InObject->IsAsset() && GIsEditor)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(InObject);
			GEditor->GetSelectedObjects()->Deselect(InObject);
		}
		OutObjectsToPurge.Add(InObject);
	}
#endif
}

// Adapted from: PurgePackages in ConcertSyncClientUtil.cpp
// Notes:
// - The method used in USourceControlHelpers::ApplyOperationAndReloadPackages with the
//   asset registry and ObjectTools::DeleteObjectsUnchecked can't be used because we want
//   this to work in game mode.
// - We assume the assets purged will not be the edited world (current level editor world)
//   so we can skip the special case from the original code.
void FAvaPlaybackUtils::PurgePackages(const TArray<UPackage*>& InExistingPackages)
{
#if WITH_EDITOR
	using namespace UE::AvaPlaybackUtils::Private;

	TArray<UObject*> ObjectsToPurge;

	for (UPackage* ExistingPackage : InExistingPackages)
	{
		if (!IsValid(ExistingPackage))
		{
			continue;
		}
		
		// Prevent any message from the editor saying a package is not saved or doesn't exist on disk.
		ExistingPackage->SetDirtyFlag(false);

		CollectObjectToPurge(ExistingPackage, ObjectsToPurge);
		ForEachObjectWithPackage(ExistingPackage, [&ObjectsToPurge](UObject* InObject)
		{
			CollectObjectToPurge(InObject, ObjectsToPurge);
			return true;
		});
	}
	
	// Broadcast the eminent objects destruction (ex. tell BlueprintActionDatabase to release its reference(s) on Blueprint(s) right now)
	FEditorDelegates::OnAssetsPreDelete.Broadcast(ObjectsToPurge);

	// Mark objects as purgeable.
	for (UObject* Object : ObjectsToPurge)
	{
		if (Object->IsRooted())
		{
			Object->RemoveFromRoot();
		}
		Object->ClearFlags(RF_Public | RF_Standalone);
	}

	if (ObjectsToPurge.Num() > 0)
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}
#endif
}

// Adapted from HotReloadPackages in ConcertSyncClientUtil.cpp
bool FAvaPlaybackUtils::ReloadPackages(const TArray<UPackage*>& InExistingPackages)
{
	FlushAsyncLoading();
	{
		bool bRunGC = false;
		for (UPackage* Package : InExistingPackages)
		{
			if (FLinkerLoad::RemoveKnownMissingPackage(Package->GetFName()))
			{
				UE_LOGF(LogAvaMedia, Verbose, "Package \"%ls\" was removed from known missing.", *Package->GetName());
				bRunGC = true;
			}
			if (Package->HasAnyPackageFlags(PKG_NewlyCreated))
			{
				UE_LOGF(LogAvaMedia, Verbose, "Clearing Newly Created flag on Package \"%ls\".", *Package->GetName());
				Package->ClearPackageFlags(PKG_NewlyCreated);
			}
		}
		if (bRunGC)
		{
			UE_LOGF(LogAvaMedia, Log, "Some packages where removed from known missing, garbage collecting ...");
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		}
	}
	FlushRenderingCommands();

	FText ErrorMessage;
#if WITH_EDITOR
	UPackageTools::ReloadPackages(InExistingPackages, ErrorMessage, UPackageTools::EReloadPackagesInteractionMode::AssumePositive);
#endif
	
	if (!ErrorMessage.IsEmpty())
	{
		UE_LOGF(LogAvaMedia, Error, "%ls", *ErrorMessage.ToString());
		return false;
	}
	return true;
}

bool FAvaPlaybackUtils::IsMapAsset(const FString& InPackageName)
{
	if (const IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		IAssetRegistry::FLoadPackageRegistryData RegistryData;
		if (AssetRegistry->GetAssetsByPackageName(FName(*InPackageName), RegistryData.Data))
		{
			// In case Asset Data is not yet in registry's cache, need to load it directly.
			if (RegistryData.Data.Num() == 0)
			{
				FString PackageFilename;
				if (FPackageName::DoesPackageExist(InPackageName, &PackageFilename))
				{
					AssetRegistry->LoadPackageRegistryData(PackageFilename, RegistryData);
				}
			}

			for (const FAssetData& AssetData : RegistryData.Data)
			{
				if (AssetData.IsValid() && AssetData.HasAnyPackageFlags(PKG_ContainsMap))
				{
					return true;
				}
			}
		}
	}
	return false;
}

bool FAvaPlaybackUtils::IsPlayableAsset(const FAssetData& InAssetData)
{
	const EMotionDesignAssetType AssetType = FAvaSoftAssetPath::GetAssetTypeFromClass(InAssetData.GetClass(), true);
	if (AssetType == EMotionDesignAssetType::Unknown)
	{
		return false;
	}
	// For world type, we need to check the tags.
	if (AssetType == EMotionDesignAssetType::World)
	{
		const FAssetTagValueRef SceneTag = InAssetData.TagsAndValues.FindTag(UE::Ava::AssetTags::MotionDesignScene);
		if (!SceneTag.IsSet() || !SceneTag.Equals(UE::Ava::AssetTags::Values::Enabled))
		{
			return false;
		}
	}
	return true;
}


namespace UE::AvaPlayback::Utils
{
	FString GetBriefFrameInfo()
	{
		return FString::Printf(TEXT("[%d]"), GFrameNumber);
	}

	UStreamableRenderAsset* StreamRenderAsset(UObject* InObject)
	{
		UStreamableRenderAsset* const StreamableAsset = Cast<UStreamableRenderAsset>(InObject);

		if (StreamableAsset && StreamableAsset->IsStreamable())
		{
			constexpr bool bHighPriority = false;
			StreamableAsset->StreamIn(FStreamableRenderResourceState::MAX_LOD_COUNT, bHighPriority);
			return StreamableAsset;
		}

		return nullptr;
	}

	bool CacheObjectMaterials(UObject* InObject, FName InShaderProfile)
	{
		return UE::Ava::FMaterialCacheHelper::Get().RequestCacheMaterials(InObject, InShaderProfile);
	}

	void FAsyncAssetLoader::ProcessLoadedObject(UObject* InObject, FName InShaderProfile)
	{
		if (Private::CVarRuntimePrestreamRCReferencedAssets.GetValueOnGameThread())
		{
			if (UStreamableRenderAsset* RenderAsset = StreamRenderAsset(InObject))
			{
				PendingStreamingAssets.Add(RenderAsset);
			}
		}

		if (Private::CVarRuntimePrecacheRCReferencedAssetsMaterials.GetValueOnGameThread())
		{
			if (CacheObjectMaterials(InObject, InShaderProfile))
			{
				PendingMaterialCompletionObjects.Add(InObject);
			}
		}
	}

	void FAsyncAssetLoader::BeginLoadingAssets(const TArray<FSoftObjectPath>& InAssetsToLoad)
	{
		const FName ShaderProfile = GetDefault<UAvaMaterialCacheSettings>()->GetRealtimeProfile();

		for (const FSoftObjectPath& AssetToLoad : InAssetsToLoad)
		{
			if (PendingAssets.Contains(AssetToLoad))
			{
				// Already processed asset
				continue;
			}

			UObject* ResolvedObject = AssetToLoad.ResolveObject();
			if (ResolvedObject)
			{
				// Already loaded... but streamable objects could still require streaming in, or it could have materials to cache.
				ProcessLoadedObject(ResolvedObject, ShaderProfile);
				continue;
			}

			PendingAssets.Add(AssetToLoad);

			AssetToLoad.LoadAsync(FLoadSoftObjectPathAsyncDelegate::CreateSPLambda(this, [this, ShaderProfile](const FSoftObjectPath& InObjectPath, UObject* InLoadedObject)
			{
				PendingAssets.Remove(InObjectPath);
				if (PendingAssets.IsEmpty())
				{
					OnLoadingCompleted.Broadcast();
				}
				ProcessLoadedObject(InLoadedObject, ShaderProfile);
			}));
		}
	}

	bool FAsyncAssetLoader::IsLoadingCompleted() const
	{
		if (!PendingAssets.IsEmpty())
		{
			return false;
		}

		CleanupStreamingAssets();
		if (!PendingStreamingAssets.IsEmpty())
		{
			return false;
		}

		CleanupObjectsWithCompleteMaterials();
		if (!PendingMaterialCompletionObjects.IsEmpty())
		{
			return false;
		}

		return true;
	}

	void FAsyncAssetLoader::CleanupStreamingAssets() const
	{
		PendingStreamingAssets.RemoveAll(
			[](UStreamableRenderAsset* InObject)
			{
				return !InObject || !InObject->HasPendingInitOrStreaming();
			});
	}

	void FAsyncAssetLoader::CleanupObjectsWithCompleteMaterials() const
	{
		UE::Ava::FMaterialCacheHelper& MaterialCacheHelper = UE::Ava::FMaterialCacheHelper::Get();

		PendingMaterialCompletionObjects.RemoveAll(
			[&MaterialCacheHelper](UObject* InObject)
			{
				return !InObject || !MaterialCacheHelper.IsCaching(InObject);
			});
	}

	void FAsyncAssetLoader::AddReferencedObjects(FReferenceCollector& InCollector)
	{
		InCollector.AddReferencedObjects(PendingMaterialCompletionObjects);
		InCollector.AddReferencedObjects(PendingStreamingAssets);
	}

	FString FAsyncAssetLoader::GetReferencerName() const
	{
		return TEXT("UE::AvaPlayback::Utils::FAsyncAssetLoader");
	}
}
