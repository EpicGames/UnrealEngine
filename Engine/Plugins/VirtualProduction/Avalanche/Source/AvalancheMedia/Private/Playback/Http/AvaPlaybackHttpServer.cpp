// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Http/AvaPlaybackHttpServer.h"

#include "HttpServerModule.h"
#include "HttpServerResponse.h"
#include "Playback/AvaPlaybackServer.h"
#include "Serialization/JsonWriter.h"

DEFINE_LOG_CATEGORY(LogAvaPlaybackHttpServer)

static const FString& GetHttpRouteVerbString(EHttpServerRequestVerbs InVerbs)
{
	static TMap<EHttpServerRequestVerbs, FString> VerbStrings =
	{
		{ EHttpServerRequestVerbs::VERB_POST, TEXT("POST") },
		{ EHttpServerRequestVerbs::VERB_PUT, TEXT("PUT") },
		{ EHttpServerRequestVerbs::VERB_GET, TEXT("GET") },
		{ EHttpServerRequestVerbs::VERB_PATCH, TEXT("PATCH") },
		{ EHttpServerRequestVerbs::VERB_DELETE, TEXT("DELETE") },
		{ EHttpServerRequestVerbs::VERB_NONE, TEXT("NONE") },
	};

	if (const FString* FoundEntry = VerbStrings.Find(InVerbs))
	{
		return *FoundEntry;		
	}

	static const FString Unknown = TEXT("UNKNOWN");
	return Unknown;
}

FAvaPlaybackHttpRouteBuilder::FAvaPlaybackHttpRouteBuilder(
	const TSharedPtr<FAvaPlaybackHttpServer>& InHttpServer,
	const TSharedPtr<FAvaPlaybackServer>& InPlaybackServer)
	: HttpServer(InHttpServer)
	, PlaybackServer(InPlaybackServer)
{
}

void FAvaPlaybackHttpServer::Start(const TSharedPtr<FAvaPlaybackServer>& InPlaybackServer, int32 InPortToUse)
{
	PortToUse = InPortToUse;
	HttpRouter = FHttpServerModule::Get().GetHttpRouter(PortToUse);

	RegisterRoutes(InPlaybackServer);

	FHttpServerModule::Get().StartAllListeners();
}

void FAvaPlaybackHttpServer::RegisterRoutes(const TSharedPtr<FAvaPlaybackServer>& InPlaybackServer)
{
	// Map as per FAvaPlaybackServer::Init(const FString& AssignedServerName)
	FAvaPlaybackHttpServer::FBuilder(AsShared(), InPlaybackServer)
	.Route<FAvaPlaybackPing>(TEXT("/playback/ping"), EHttpServerRequestVerbs::VERB_POST, &FAvaPlaybackServer::HandlePlaybackPing)
	.Route<FAvaPlaybackDeviceProviderDataRequest>(TEXT("/playback/devices"), EHttpServerRequestVerbs::VERB_POST, &FAvaPlaybackServer::HandleDeviceProviderDataRequest)
	.Route<FAvaPlaybackRequest>(TEXT("/playback"), EHttpServerRequestVerbs::VERB_POST, &FAvaPlaybackServer::HandlePlaybackRequest)
	.Route<FAvaPlaybackAnimPlaybackRequest>(TEXT("/playback/animation"), EHttpServerRequestVerbs::VERB_POST, &FAvaPlaybackServer::HandleAnimPlaybackRequest)
	.Route<FAvaBroadcastRequest>(TEXT("/broadcast"), EHttpServerRequestVerbs::VERB_POST, &FAvaPlaybackServer::HandleBroadcastRequest)
	.Route<FAvaBroadcastStatusRequest>(TEXT("/broadcast/status"), EHttpServerRequestVerbs::VERB_POST, &FAvaPlaybackServer::HandleBroadcastStatusRequest);

	// We always want the ListRegisteredRoutes route bound, no matter what.
	RegisterNewRoute(TEXT("ListRegisteredRoutes"), FHttpPath("/listroutes"), EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateSP(this, &FAvaPlaybackHttpServer::HttpListOpenRoutes), true);
}

bool FAvaPlaybackHttpServer::GetRegisteredRoute(FName RouteName, FAvaPlaybackHttpRouteInfo& OutRouteInfo)
{
	if (RegisteredRoutes.Find(RouteName))
	{
		OutRouteInfo.RouteName = RouteName;
		OutRouteInfo.RoutePath = RegisteredRoutes[RouteName].Handle->Path;
		OutRouteInfo.RequestVerbs = RegisteredRoutes[RouteName].Handle->Verbs;
		OutRouteInfo.InputContentType = RegisteredRoutes[RouteName].InputContentType;
		OutRouteInfo.InputExpectedFormat = RegisteredRoutes[RouteName].InputExpectedFormat;
		return true;
	}
	return false;
}

void FAvaPlaybackHttpServer::RegisterNewRoute(FName RouteName, const FHttpPath& HttpPath, const EHttpServerRequestVerbs& RequestVerbs, const FHttpRequestHandler& Handler, bool bOverrideIfBound /* = false */, FString OptionalContentType /* = TEXT("")*/, FString OptionalExpectedFormat /*= TEXT("")*/)
{
	FAvaPlaybackHttpRouteInfo InRouteInfo;
	InRouteInfo.RouteName = RouteName;
	InRouteInfo.RoutePath = HttpPath;
	InRouteInfo.RequestVerbs = RequestVerbs;
	InRouteInfo.InputContentType = OptionalContentType;
	InRouteInfo.InputExpectedFormat = OptionalExpectedFormat;
	RegisterNewRoute(InRouteInfo, Handler, bOverrideIfBound);
}

void FAvaPlaybackHttpServer::RegisterNewRoute(FAvaPlaybackHttpRouteInfo InRouteInfo, const FHttpRequestHandler& Handler, bool bOverrideIfBound /* = false */)
{
	if (HttpRouter.IsValid())
	{
		if (RegisteredRoutes.Find(InRouteInfo.RouteName))
		{
			if (!bOverrideIfBound)
			{
				UE_LOGF(LogAvaPlaybackHttpServer, Error, "Failed to bind route with friendly key %ls - a route at location %ls already exists.", *InRouteInfo.RouteName.ToString(), *InRouteInfo.RoutePath.GetPath());
				return;
			}
			UE_LOGF(LogAvaPlaybackHttpServer, Log, "Overwriting route at friendly key %ls - from %ls to %ls ", *InRouteInfo.RouteName.ToString(), *RegisteredRoutes[InRouteInfo.RouteName].Handle->Path, *InRouteInfo.RoutePath.GetPath());
			HttpRouter->UnbindRoute(RegisteredRoutes[InRouteInfo.RouteName].Handle);
		}
		FAvaPlaybackHttpRouteDesc RouteDesc;
		RouteDesc.Handle = HttpRouter->BindRoute(InRouteInfo.RoutePath, InRouteInfo.RequestVerbs, Handler);
		RouteDesc.InputContentType = InRouteInfo.InputContentType;
		RouteDesc.InputExpectedFormat = InRouteInfo.InputExpectedFormat;
		RegisteredRoutes.Add(InRouteInfo.RouteName, RouteDesc);
	}
}

void FAvaPlaybackHttpServer::CleanUpRoute(FName RouteName, bool bFailIfUnbound /* = false */)
{
	if (HttpRouter.IsValid())
	{
		if (RegisteredRoutes.Find(RouteName))
		{
			HttpRouter->UnbindRoute(RegisteredRoutes[RouteName].Handle);
			RegisteredRoutes.Remove(RouteName);
			UE_LOGF(LogAvaPlaybackHttpServer, Log, "Route name %ls was unbound!", *RouteName.ToString());
		}
		else
		{
			UE_LOGF(LogAvaPlaybackHttpServer, Warning, "Route name %ls does not exist, could not unbind.", *RouteName.ToString());
			check(!bFailIfUnbound);
		}
	}
}

bool FAvaPlaybackHttpServer::HttpListOpenRoutes(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	FString ResponseStr;
	TArray<FName> OutRouteKeys;
	RegisteredRoutes.GetKeys(OutRouteKeys);
	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&ResponseStr);
	JsonWriter->WriteArrayStart();
	for (const FName& RouteKey : OutRouteKeys)
	{
		JsonWriter->WriteObjectStart();
		JsonWriter->WriteValue(TEXT("name"), RouteKey.ToString());
		JsonWriter->WriteValue(TEXT("route"), RegisteredRoutes[RouteKey].Handle->Path);
		JsonWriter->WriteValue(TEXT("verb"), GetHttpRouteVerbString(RegisteredRoutes[RouteKey].Handle->Verbs));
		if (!RegisteredRoutes[RouteKey].InputContentType.IsEmpty())
		{
			JsonWriter->WriteValue(TEXT("inputContentType"), RegisteredRoutes[RouteKey].InputContentType);
		}
		if (!RegisteredRoutes[RouteKey].InputExpectedFormat.IsEmpty())
		{
			JsonWriter->WriteValue(TEXT("inputExpectedFormat"), RegisteredRoutes[RouteKey].InputExpectedFormat);
		}
		JsonWriter->WriteObjectEnd();
	}
	JsonWriter->WriteArrayEnd();
	JsonWriter->Close();
	TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(ResponseStr, TEXT("application/json"));
	OnComplete(MoveTemp(Response));
	return true;
}