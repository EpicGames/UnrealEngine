// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Backends/JsonStructDeserializerBackend.h"
#include "HttpResultCallback.h"
#include "HttpRouteHandle.h"
#include "HttpServerRequest.h"
#include "IHttpRouter.h"
#include "IMessageAttachment.h"
#include "IMessageContext.h"
#include "Serialization/MemoryReader.h"
#include "StructDeserializer.h"
#include "AvaPlaybackHttpServer.generated.h"

class FAvaPlaybackServer;

namespace UE::Ava::Web::Private
{
	template <typename MessageType>
	[[nodiscard]] bool DeserializeRequest(const FHttpServerRequest& InRequest, const FHttpResultCallback* InCompleteCallback, MessageType& OutDeserializedRequest);
}

// Required by Messaging API which this wraps, so a dummy context is required
class FAvaPlaybackEmptyMessageContext
	: public IMessageContext
{
public:
	FAvaPlaybackEmptyMessageContext() = default;
	virtual ~FAvaPlaybackEmptyMessageContext() override = default;

public:
	//~ IMessageContext interface
	virtual const TMap<FName, FString>& GetAnnotations() const override { return EmptyMap; }
	virtual TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe> GetAttachment() const override { return nullptr; }
	virtual const FDateTime& GetExpiration() const override { return EmptyDate; }
	virtual const void* GetMessage() const override { return nullptr; }
	virtual const TWeakObjectPtr<UScriptStruct>& GetMessageTypeInfo() const override { return EmptyTypeInfo; }
	virtual TSharedPtr<IMessageContext, ESPMode::ThreadSafe> GetOriginalContext() const override { return nullptr; }
	virtual const TArray<FMessageAddress>& GetRecipients() const override { return EmptyRecipients; }
	virtual EMessageScope GetScope() const override { return {}; }
	virtual EMessageFlags GetFlags() const override { return {}; }
	virtual const FMessageAddress& GetSender() const override { return EmptyAddress; }
	virtual const FMessageAddress& GetForwarder() const override { return EmptyAddress; }
	virtual ENamedThreads::Type GetSenderThread() const override  { return {}; }
	virtual const FDateTime& GetTimeForwarded() const override { return EmptyDate; }
	virtual const FDateTime& GetTimeSent() const override { return EmptyDate; }

private:
	TMap<FName, FString> EmptyMap;
	FDateTime EmptyDate;
	TWeakObjectPtr<UScriptStruct> EmptyTypeInfo;
	TArray<FMessageAddress> EmptyRecipients;
	FMessageAddress EmptyAddress;
};

USTRUCT()
struct FAvaPlaybackHttpRouteInfo
{
	GENERATED_BODY()
public:
	FName RouteName;
	FHttpPath RoutePath;
	EHttpServerRequestVerbs RequestVerbs;
	FString InputContentType;
	FString InputExpectedFormat;
	FAvaPlaybackHttpRouteInfo()
	{
		RouteName = FName(TEXT(""));
		RoutePath = FHttpPath();
		RequestVerbs = EHttpServerRequestVerbs::VERB_NONE;
		InputContentType = TEXT("");
		InputExpectedFormat = TEXT("");
	}
	FAvaPlaybackHttpRouteInfo(FName InRouteName, FHttpPath InRoutePath, EHttpServerRequestVerbs InRequestVerbs, FString InContentType = TEXT(""), FString InExpectedFormat = TEXT(""))
	{
		RouteName = InRouteName;
		RoutePath = InRoutePath;
		RequestVerbs = InRequestVerbs;
		InputContentType = InContentType;
		InputExpectedFormat = InExpectedFormat;
	}
};

USTRUCT()
struct FAvaPlaybackHttpRouteDesc
{
	GENERATED_BODY()
public:
	FHttpRouteHandle Handle;
	FString InputContentType;
	FString InputExpectedFormat;
	FAvaPlaybackHttpRouteDesc()
	{
	
	}
	FAvaPlaybackHttpRouteDesc(FHttpRouteHandle InHandle, FString InContentType, FString InExpectedFormat)
	{
		Handle = InHandle;
		InputContentType = InContentType;
		InputExpectedFormat = InExpectedFormat;
	}
};

DECLARE_LOG_CATEGORY_EXTERN(LogAvaPlaybackHttpServer, Log, All);

class FAvaPlaybackHttpServer;

struct FAvaPlaybackHttpRouteBuilder
{
public:
	template <typename MessageType, typename HandlerType>
	struct TMessageHandler
	{
		typedef void (HandlerType::* FuncType)(const MessageType&, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>&);
	};

public:
	FAvaPlaybackHttpRouteBuilder(const TSharedPtr<FAvaPlaybackHttpServer>& InHttpServer, const TSharedPtr<FAvaPlaybackServer>& InPlaybackServer);
	
public:	
	template <typename MessageType>
	FAvaPlaybackHttpRouteBuilder& Route(const FString& InPath, const EHttpServerRequestVerbs& InVerb, typename TMessageHandler<MessageType, FAvaPlaybackServer>::FuncType InHandlerFunc);
	
private:
	TSharedPtr<FAvaPlaybackHttpServer> HttpServer;
	TSharedPtr<FAvaPlaybackServer> PlaybackServer;
};

class FAvaPlaybackHttpServer : public TSharedFromThis<FAvaPlaybackHttpServer>
{
public:
	using FBuilder = FAvaPlaybackHttpRouteBuilder;
	
	FAvaPlaybackHttpServer() = default;
	virtual ~FAvaPlaybackHttpServer() = default;

	void Start(const TSharedPtr<FAvaPlaybackServer>& InPlaybackServer, int32 InPortToUse = 10123);

	void RegisterRoutes(const TSharedPtr<FAvaPlaybackServer>& InPlaybackServer);

	/**
	 * Try to get a route registered under given friendly name. Returns false if could not be found.
	 */
	bool GetRegisteredRoute(FName RouteName, FAvaPlaybackHttpRouteInfo& OutRouteInfo);

	void RegisterNewRoute(FAvaPlaybackHttpRouteInfo InRouteInfo, const FHttpRequestHandler& Handler, bool bOverrideIfBound = false);

	/**
	 * Register a new route.
	 * Will override existing routes if option is set, otherwise will error and fail to bind.
	 */
	void RegisterNewRoute(FName RouteName, const FHttpPath& HttpPath, const EHttpServerRequestVerbs& RequestVerbs, const FHttpRequestHandler& Handler, bool bOverrideIfBound = false, FString OptionalContentType = TEXT(""), FString OptionalExpectedFormat = TEXT(""));

	/**
	 * Clean up a route.
	 * Can be set to fail if trying to unbind an unbound route.
	 */
	void CleanUpRoute(FName RouteName, bool bFailIfUnbound = false);

	/**
	 * Default Route Listing http call. Spits out all registered routes and describes them via a REST API call.
	 * Always registered at /listroutes GET by default
	 */
	bool HttpListOpenRoutes(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	
	bool IsRunning() const { return HttpRouter.IsValid(); }

protected:
	int PortToUse = 10123;
	TSharedPtr<IHttpRouter> HttpRouter;
	TMap<FName, FAvaPlaybackHttpRouteDesc> RegisteredRoutes;
};

template <typename MessageType>
bool UE::Ava::Web::Private::DeserializeRequest(
	const FHttpServerRequest& InRequest,
	const FHttpResultCallback* InCompleteCallback,
	MessageType& OutDeserializedRequest)
{
	TArray<uint8> TCHARPayload;

	const int32 StartIndex = TCHARPayload.Num();
	TCHARPayload.AddUninitialized(FUTF8ToTCHAR_Convert::ConvertedLength((ANSICHAR*)InRequest.Body.GetData(), InRequest.Body.Num() / sizeof(ANSICHAR)) * sizeof(TCHAR));
	FUTF8ToTCHAR_Convert::Convert((TCHAR*)(TCHARPayload.GetData() + StartIndex), (TCHARPayload.Num() - StartIndex) / sizeof(TCHAR), (ANSICHAR*)InRequest.Body.GetData(), InRequest.Body.Num() / sizeof(ANSICHAR));

	FMemoryReaderView Reader(TCHARPayload);
	FJsonStructDeserializerBackend DeserializerBackend(Reader);
	if (!FStructDeserializer::Deserialize(&OutDeserializedRequest, *MessageType::StaticStruct(), DeserializerBackend, FStructDeserializerPolicies()))
	{
		if (InCompleteCallback)
		{
			TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
			Response->Code = EHttpServerResponseCodes::BadRequest;
			(*InCompleteCallback)(MoveTemp(Response));
		}
		return false;
	}

	return true;
}

template <typename MessageType>
FAvaPlaybackHttpRouteBuilder& FAvaPlaybackHttpRouteBuilder::Route(
	const FString& InPath,
	const EHttpServerRequestVerbs& InVerb,
	typename TMessageHandler<MessageType, FAvaPlaybackServer>::FuncType InHandlerFunc)
{
	static_assert(TModels<CStaticStructProvider, MessageType>::Value, "MessageType must be a UStruct");

	TWeakPtr<FAvaPlaybackServer> WeakPlaybackServerPtr = TWeakPtr<FAvaPlaybackServer>(PlaybackServer);
	
	HttpServer->RegisterNewRoute(MessageType::StaticStruct()->GetFName(), FHttpPath(InPath), InVerb,
	FHttpRequestHandler::CreateLambda([WeakPlaybackServerPtr, HandlerFunc = MoveTemp(InHandlerFunc)](const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
	{
		if (WeakPlaybackServerPtr.IsValid())
		{
			MessageType DeserializedMessage;
			if (!UE::Ava::Web::Private::DeserializeRequest(Request, &OnComplete, DeserializedMessage))
			{
				return false;
			}

			static TSharedRef<IMessageContext> EmptyContext = MakeShared<FAvaPlaybackEmptyMessageContext>();
			(WeakPlaybackServerPtr.Pin().Get()->*HandlerFunc)(DeserializedMessage, EmptyContext);

			if (OnComplete)
			{
				TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
				Response->Headers.Add(TEXT("content-type"), { TEXT("application/json") });
				Response->Code = EHttpServerResponseCodes::Ok;

				OnComplete(MoveTemp(Response));
			}
			
			return true;
		}
		return false;
	}), true);

	return *this;
}
