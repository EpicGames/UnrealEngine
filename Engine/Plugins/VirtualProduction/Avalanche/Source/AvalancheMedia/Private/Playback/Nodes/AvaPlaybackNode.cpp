// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Nodes/AvaPlaybackNode.h"
#include "EngineUtils.h"
#include "Internationalization/Text.h"
#include "Misc/App.h"
#include "Playback/AvaPlaybackGraph.h"
#include "Playback/IAvaPlaybackGraphEditor.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"

#if WITH_EDITOR
#include "EdGraph/EdGraph.h"
#endif

#define LOCTEXT_NAMESPACE "AvaPlaybackNode"

const FText UAvaPlaybackNode::NodeCategory::Default          = LOCTEXT("Category_Default", "Default");
const FText UAvaPlaybackNode::NodeCategory::EventTrigger     = LOCTEXT("Category_EventTrigger", "Event Triggers");
const FText UAvaPlaybackNode::NodeCategory::EventAction      = LOCTEXT("Category_EventAction", "Event Actions");
const FText UAvaPlaybackNode::NodeCategory::EventFlowControl = LOCTEXT("Category_EventFlowControl", "Event Flow Control");

void UAvaPlaybackNode::PostAllocateNode()
{
	if (UAvaPlaybackGraph* const Playback = GetPlayback())
	{
		PlaybackStateChangedHandle = Playback->OnPlaybackStateChanged.AddUObject(this, &UAvaPlaybackNode::NotifyPlaybackStateChanged);
	}
}

void UAvaPlaybackNode::BeginDestroy()
{
	if (PlaybackStateChangedHandle.IsValid() && GetPlayback())
	{
		GetPlayback()->OnPlaybackStateChanged.Remove(PlaybackStateChangedHandle);
	}
	UObject::BeginDestroy();
}

UAvaPlaybackGraph* UAvaPlaybackNode::GetPlayback() const
{
	return Cast<UAvaPlaybackGraph>(GetOuter());
}

FText UAvaPlaybackNode::GetNodeDisplayNameText() const
{
	return FText::FromName(GetClass()->GetFName());
}

FText UAvaPlaybackNode::GetNodeCategoryText() const
{
	return NodeCategory::Default;
}

FText UAvaPlaybackNode::GetNodeTooltipText() const
{
	return FText::GetEmpty();
}

double UAvaPlaybackNode::GetLastTimeTicked() const
{
	return LastValidTick;
}

double UAvaPlaybackNode::GetChildLastTimeTicked(int32 ChildIndex) const
{
	if (const double* const FoundTime = ChildLastValidTicks.Find(ChildIndex))
	{
		return *FoundTime;
	}
	return -1.f;
}

void UAvaPlaybackNode::DryRunNode(TArray<UAvaPlaybackNode*>& InAncestors)
{
	DryRun(InAncestors);
	
	TSet<TObjectPtr<UAvaPlaybackNode>> TraversedChildNodes;
	TraversedChildNodes.Reserve(ChildNodes.Num());
	
	for (const TObjectPtr<UAvaPlaybackNode>& ChildNode : ChildNodes)
	{
		if (ChildNode && !TraversedChildNodes.Contains(ChildNode))
		{
			TraversedChildNodes.Add(ChildNode);
			
			InAncestors.Push(this);
			ChildNode->DryRunNode(InAncestors);
			InAncestors.Pop(EAllowShrinking::No);
		}
	}
}

void UAvaPlaybackNode::NotifyChildNodeSucceeded(int32 ChildIndex)
{
	const double Time = FApp::GetCurrentTime();
	ChildNodes[ChildIndex]->LastValidTick = Time;
	ChildLastValidTicks.Add(ChildIndex, Time);
}

void UAvaPlaybackNode::TickChild(float DeltaTime, int32 ChildIndex, FAvaPlaybackChannelParameters& ChannelParameters)
{
	if (ChildNodes.IsValidIndex(ChildIndex) && ChildNodes[ChildIndex])
	{
		ChildNodes[ChildIndex]->Tick(DeltaTime, ChannelParameters);

		//Record Last Valid Tick times, if the given Child Node managed to obtain valid Channel Parameters.
		if (ChannelParameters.HasAssets())
		{
			NotifyChildNodeSucceeded(ChildIndex);
		}
	}
}

void UAvaPlaybackNode::Tick(float DeltaTime, FAvaPlaybackChannelParameters& ChannelParameters)
{
	for (int32 Index = 0; Index < ChildNodes.Num(); ++Index)
	{
		TickChild(DeltaTime, Index, ChannelParameters);
	}
}

void UAvaPlaybackNode::GetAllNodes(TArray<UAvaPlaybackNode*>& OutPlaybackNodes)
{
	OutPlaybackNodes.Add(this);
	for (UAvaPlaybackNode* const ChildNode : ChildNodes)
	{
		if (ChildNode)
		{
			ChildNode->GetAllNodes(OutPlaybackNodes);
		}
	}
}

void UAvaPlaybackNode::PostLoad()
{
	UObject::PostLoad();
	
	PostAllocateNode();

#if WITH_EDITOR
	// Make sure nodes are transactional (so they work with undo system)
	SetFlags(RF_Transactional);
#endif
}

#if WITH_EDITOR
void UAvaPlaybackNode::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UAvaPlaybackNode* const This = CastChecked<UAvaPlaybackNode>(InThis);
	Collector.AddReferencedObject(This->GraphNode, This);
	Super::AddReferencedObjects(InThis, Collector);
}

void UAvaPlaybackNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const bool bDryRunGraph = EditorDryRunGraphOnNodeRefresh(PropertyChangedEvent);
	RefreshNode(bDryRunGraph);
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void UAvaPlaybackNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	FStripDataFlags StripFlags(Ar);
#if WITH_EDITORONLY_DATA
	if (!StripFlags.IsEditorDataStripped())
	{
		Ar << GraphNode;
	}
#endif
}

#if WITH_EDITORONLY_DATA
void UAvaPlaybackNode::SetGraphNode(UEdGraphNode* InGraphNode)
{
	GraphNode = InGraphNode;
}

UEdGraphNode* UAvaPlaybackNode::GetGraphNode() const
{
	return GraphNode;
}
#endif

void UAvaPlaybackNode::SetChildNodes(TArray<UAvaPlaybackNode*>&& InChildNodes)
{
	const int32 MaxChildNodes = GetMaxChildNodes();
	const int32 MinChildNodes = GetMinChildNodes();
	
	if (MaxChildNodes >= InChildNodes.Num() && InChildNodes.Num() >= MinChildNodes)
	{
		ChildNodes = MoveTemp(InChildNodes);
	}
}

void UAvaPlaybackNode::ReconstructNode()
{
	CreateStartingConnectors();

	const int32 MaxChildNodes = GetMaxChildNodes();
	while (ChildNodes.Num() > MaxChildNodes)
	{
		RemoveChildNode(ChildNodes.Num() - 1);
	}

#if WITH_EDITOR
	if (GraphNode)
	{
		GraphNode->ReconstructNode();
		GraphNode->GetGraph()->NotifyGraphChanged();
	}
#endif
}

void UAvaPlaybackNode::RefreshNode(bool bDryRunGraph)
{
	if (UAvaPlaybackGraph* const Playback = GetPlayback())
	{
#if WITH_EDITOR
		Playback->RefreshPlaybackNode(this);
#endif
		if (bDryRunGraph)
		{
			//Deferred Dry Run Graph in case there are Multiple Nodes refreshing
			Playback->DryRunGraph(true);
		}
	}
}

const TArray<TObjectPtr<UAvaPlaybackNode>>& UAvaPlaybackNode::GetChildNodes() const
{
	return ChildNodes;
}

void UAvaPlaybackNode::CreateStartingConnectors()
{
	const int32 MinChildNodes = GetMinChildNodes();
	while (ChildNodes.Num() < MinChildNodes)
	{
		InsertChildNode(ChildNodes.Num());
	}
}

void UAvaPlaybackNode::InsertChildNode(int32 Index)
{
	check(Index >= 0 && Index <= ChildNodes.Num());
	int32 MaxChildNodes = GetMaxChildNodes();
	if (MaxChildNodes > ChildNodes.Num())
	{
		ChildNodes.InsertZeroed(Index);
#if WITH_EDITOR
		UAvaPlaybackGraph* const Playback = GetPlayback();
		if (Playback && GraphNode && Playback->GetGraphEditor().IsValid())
		{
			Playback->GetGraphEditor()->CreateInputPin(GraphNode);
		}
#endif
	}
}

void UAvaPlaybackNode::RemoveChildNode(int32 Index)
{
	check(Index >= 0 && Index < ChildNodes.Num());
	const int32 MinChildNodes = GetMinChildNodes();
	if (ChildNodes.Num() > MinChildNodes)
	{
		ChildNodes.RemoveAt(Index);
	}
}

#if WITH_EDITOR
FName UAvaPlaybackNode::GetInputPinName(int32 InputPinIndex) const
{
	return NAME_None;
}
#endif

#undef LOCTEXT_NAMESPACE
