// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Nodes/AvaPlaybackNodeCombiner.h"
#include "AvaMediaDefines.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"

#if WITH_EDITOR
#include "UObject/NameTypes.h"
#endif

#define LOCTEXT_NAMESPACE "AvaPlaybackNodeCombiner"

FText UAvaPlaybackNodeCombiner::GetNodeDisplayNameText() const
{
	return LOCTEXT("CombinerNode_Title", "Combiner");
}

void UAvaPlaybackNodeCombiner::CreateStartingConnectors()
{
	InsertChildNode(ChildNodes.Num());
	InsertChildNode(ChildNodes.Num());
}

void UAvaPlaybackNodeCombiner::Tick(float DeltaTime, FAvaPlaybackChannelParameters& ChannelParameters)
{
	for (int32 ChildIndex = 0; ChildIndex < ChildNodes.Num(); ++ChildIndex)
	{
		if (!ChildNodes.IsValidIndex(ChildIndex) || !ChildNodes[ChildIndex])
		{
			continue;
		}

		if (EnabledIndices.IsValidIndex(ChildIndex) && EnabledIndices[ChildIndex])
		{
			TickChild(DeltaTime, ChildIndex, ChannelParameters);
		}
		else
		{
			// Special case - tick the other nodes to set their channel indices.
			FAvaPlaybackChannelParameters TmpParameters;
			TmpParameters.ChannelIndex = ChannelParameters.ChannelIndex;
			ChildNodes[ChildIndex]->Tick(DeltaTime, TmpParameters);
		}
	}
}

#if WITH_EDITOR
FName UAvaPlaybackNodeCombiner::GetInputPinName(int32 InputPinIndex) const
{
	return *FString::FromInt(InputPinIndex);
}
#endif

#undef LOCTEXT_NAMESPACE
