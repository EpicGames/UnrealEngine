// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playback/Nodes/AvaPlaybackNode.h"
#include "AvaPlaybackNodeCombiner.generated.h"

class FText;
struct FAvaPlaybackChannelParameters;

#if WITH_EDITOR
class FName;
#endif

UCLASS()
class UAvaPlaybackNodeCombiner : public UAvaPlaybackNode
{
	GENERATED_BODY()
	
	virtual FText GetNodeDisplayNameText() const override;
	
	virtual int32 GetMinChildNodes() const override { return 1; }
	virtual int32 GetMaxChildNodes() const override { return MaxAllowedChildNodes; }
	
	virtual void CreateStartingConnectors() override;
	
	virtual void Tick(float DeltaTime, FAvaPlaybackChannelParameters& ChannelParameters) override;

#if WITH_EDITOR
	virtual FName GetInputPinName(int32 InputPinIndex) const override;
#endif
	
protected:
	UPROPERTY(EditAnywhere, Category = "Motion Design")
	TArray<bool> EnabledIndices;	
};
