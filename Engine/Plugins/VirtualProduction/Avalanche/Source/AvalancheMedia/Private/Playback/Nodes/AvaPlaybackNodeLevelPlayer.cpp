// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Nodes/AvaPlaybackNodeLevelPlayer.h"
#include "Engine/World.h"
#include "Framework/AvaSoftAssetPtr.h"
#include "Internationalization/Text.h"
#include "Playback/AvaPlaybackGraph.h"

#define LOCTEXT_NAMESPACE "AvaPlaybackNodeLevelPlayer"

UAvaPlaybackNodeLevelPlayer::UAvaPlaybackNodeLevelPlayer()
{
	//Update to Default Text
	UpdateDisplayNameText();
}

void UAvaPlaybackNodeLevelPlayer::RefreshNode(bool bDryRunGraph)
{
	UpdateDisplayNameText();
	Super::RefreshNode(bDryRunGraph);
}

void UAvaPlaybackNodeLevelPlayer::PostLoad()
{
	Super::PostLoad();
	UpdateDisplayNameText();
}

void UAvaPlaybackNodeLevelPlayer::SetAsset(const TSoftObjectPtr<UWorld>& InAsset)
{
	LevelAsset = InAsset;
}

void UAvaPlaybackNodeLevelPlayer::UpdateDisplayNameText()
{
	const FString AssetName = LevelAsset.GetAssetName();
	
	if (AssetName.IsEmpty())
	{
		DisplayNameText = LOCTEXT("AvaPlaybackNode_LevelPlayerNoName", "Motion Design Level Player");
	}
	else
	{
		DisplayNameText = FText::Format(LOCTEXT("AvaPlaybackNode_LevelPlayerName", "Motion Design Level Player\n{0}")
			, FText::FromString(AssetName));
	}
}

FAvaSoftAssetPtr UAvaPlaybackNodeLevelPlayer::GetAssetPtr() const
{
	FAvaSoftAssetPtr OutAsset;
	OutAsset.AssetClassPath = FSoftClassPath(UWorld::StaticClass());
	OutAsset.AssetPtr = LevelAsset;
	return OutAsset;
}

#undef LOCTEXT_NAMESPACE
