// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Nodes/AvaPlaybackNodePlayer.h"
#include "AvaMediaDefines.h"
#include "Broadcast/AvaBroadcast.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Internationalization/Text.h"
#include "Playable/AvaPlayable.h"
#include "Playback/AvaPlaybackGraph.h"
#include "Playback/Nodes/Events/AvaPlaybackNodeEvent.h"

#define LOCTEXT_NAMESPACE "AvaPlaybackNodePlayer"

UAvaPlaybackNodePlayer::UAvaPlaybackNodePlayer() = default;

void UAvaPlaybackNodePlayer::PostAllocateNode()
{
	Super::PostAllocateNode();
	if (UAvaPlaybackGraph* const Playback = GetPlayback())
	{
		Playback->AddPlayerNode(this);
	}
}

void UAvaPlaybackNodePlayer::Tick(float DeltaTime, FAvaPlaybackChannelParameters& ChannelParameters)
{
	FAvaSoftAssetPtr Asset = GetAssetPtr();
	
	// We still don't support multi-instance within a playback graph. The asset remains the unique key.
	auto IsSameAsset = [&Asset](const FAvaPlaybackPlayerParameters& InPlayer)
	{
		return InPlayer.Asset == Asset;
	};

	if (!ChannelParameters.Players.FindByPredicate(IsSameAsset))
	{
		ChannelParameters.Players.Add({this, Asset});	
	}
	
	ChannelIndices.Add(ChannelParameters.ChannelIndex);
	//No Children Ticks. The Player Node is the Dead End for Ticking.
	//Events are handled separately
}

void UAvaPlaybackNodePlayer::TickEventFeed(float DeltaTime)
{
	for (int32 Index = 0; Index < ChildNodes.Num(); ++Index)
	{
		//TODO: Should we cache these Event Nodes? 
		if (UAvaPlaybackNodeEvent* const EventNode = Cast<UAvaPlaybackNodeEvent>(ChildNodes[Index]))
		{
			FAvaPlaybackEventParameters EventParameters;
			EventParameters.ChannelIndices = ChannelIndices;
			EventParameters.Asset = GetAssetPtr();
			
			EventNode->TickEvent(DeltaTime, EventParameters);
			
			if (EventParameters.ShouldTriggerEventAction())
			{
				NotifyChildNodeSucceeded(Index);
			}
		}
	}
}

void UAvaPlaybackNodePlayer::ResetEvents()
{
	// Note: Keep the channel indices from last tick around so the preview knows which
	// channel to fetch. This happens outside of the playback tick so the ChannelIndices
	// would be reset at that point (see GetPreviewRenderTarget, called from slate UI update).
	LastTickChannelIndices = ChannelIndices;

	ChannelIndices.Reset();
	
	TArray<TObjectPtr<UAvaPlaybackNode>> RemainingNodes = ChildNodes;
	while (!RemainingNodes.IsEmpty())
	{
		//Only iterate Event Nodes (all of Player's child nodes should be node events.)
		if (UAvaPlaybackNodeEvent* const EventNode = Cast<UAvaPlaybackNodeEvent>(RemainingNodes.Pop()))
		{
			EventNode->Reset();
			RemainingNodes.Append(EventNode->GetChildNodes());
		}
	}
}

FText UAvaPlaybackNodePlayer::GetNodeDisplayNameText() const
{
	return DisplayNameText;
}

FText UAvaPlaybackNodePlayer::GetNodeTooltipText() const
{
	return LOCTEXT("PlayerNode_ToolTip", "Plays the given Motion Design Asset");
}

#if WITH_EDITOR

UTextureRenderTarget2D* UAvaPlaybackNodePlayer::GetPreviewRenderTarget() const
{
	// Returns the first valid channel index we have.
	for (const int32 ChannelIndex : LastTickChannelIndices)
	{
		const FName ChannelName = UAvaBroadcast::Get().GetChannelName(ChannelIndex);
		if (ChannelName.IsNone())
		{
			continue;
		}

		if (const UAvaPlaybackGraph* const Playback = GetPlayback())
		{
			if (const UAvaPlayable* const Playable = Playback->FindPlayable(GetAssetPath(), ChannelName))
			{
				if (const UAvaPlayableGroup* PlayableGroup = Playable->GetPlayableGroup())
				{
					if (UTextureRenderTarget2D* RenderTarget = PlayableGroup->GetRenderTarget())
					{
						return RenderTarget;
					}
				}
			}
		}

		// Fallback to channel RT which is likely to be (actually should be) the same as the game instance.
		const FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetChannel(ChannelName);
		if (Channel.IsValidChannel())
		{
			return Channel.GetCurrentRenderTarget(true);
		}
	}
	return nullptr;
}
#endif

const FSoftObjectPath& UAvaPlaybackNodePlayer::GetAssetPath() const
{
	static FSoftObjectPath EmptyPath;
	return EmptyPath;
}

void UAvaPlaybackNodePlayer::SetLoadOptions(const FString& InLoadOptions)
{
	LoadOptions = InLoadOptions;
}

#undef LOCTEXT_NAMESPACE
