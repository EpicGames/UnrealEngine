// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Nodes/AvaPlaybackNodeRoot.h"

#include "Broadcast/AvaBroadcast.h"
#include "Playback/AvaPlaybackGraph.h"

#define LOCTEXT_NAMESPACE "AvaPlaybackNodeRoot"

void UAvaPlaybackNodeRoot::PostAllocateNode()
{
	Super::PostAllocateNode();
	
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	
	BroadcastChangedHandle = Broadcast.AddChangeListener(FOnAvaBroadcastChanged::FDelegate::CreateUObject(this
		, &UAvaPlaybackNodeRoot::OnBroadcastChanged));
		
	ChannelChangedHandle = FAvaBroadcastOutputChannel::GetOnChannelChanged().AddUObject(this
		, &UAvaPlaybackNodeRoot::OnChannelChanged);

	if (UAvaPlaybackGraph* const Playback = GetPlayback())
	{
		Playback->SetRootNode(this);
	}
}

void UAvaPlaybackNodeRoot::BeginDestroy()
{
	if (BroadcastChangedHandle.IsValid())
	{
		UAvaBroadcast::Get().RemoveChangeListener(BroadcastChangedHandle);
	}
	if (ChannelChangedHandle.IsValid())
	{
		FAvaBroadcastOutputChannel::GetOnChannelChanged().Remove(ChannelChangedHandle);
	}
	Super::BeginDestroy();
}

void UAvaPlaybackNodeRoot::TickRoot(float InDeltaTime, TMap<FName, FAvaPlaybackChannelParameters>& OutPlaybackSettings)
{
	OutPlaybackSettings.Empty(ChildNodes.Num());
	LastValidTick = FApp::GetCurrentTime();
	
	for (int32 Index = 0; Index < ChildNodes.Num(); ++Index)
	{
		FName ChannelName = GetChannelName(Index);
		if (!ChannelName.IsNone())
		{
			FAvaPlaybackChannelParameters& PlaybackParameters = OutPlaybackSettings.Add(ChannelName);
			PlaybackParameters.ChannelIndex = Index;
			TickChild(InDeltaTime, Index, PlaybackParameters);
		}
	}
}

FText UAvaPlaybackNodeRoot::GetNodeDisplayNameText() const
{
	return LOCTEXT("RootNode_Title", "Channels");
}

FText UAvaPlaybackNodeRoot::GetNodeTooltipText() const
{
	return LOCTEXT("RootNode_Tooltip", "Wire the Playback Nodes into this Final Node for Broadcast");
}

void UAvaPlaybackNodeRoot::OnBroadcastChanged(EAvaBroadcastChange InChange)
{
	//TODO: Changing Profiles should not trigger reconstruct, rather the Pins should react to whether the Channel is Available or not
	ReconstructNode();
}

void UAvaPlaybackNodeRoot::OnChannelChanged(const FAvaBroadcastOutputChannel& InChannel, EAvaBroadcastChannelChange InChange)
{
	if (EnumHasAnyFlags(InChange, EAvaBroadcastChannelChange::State))
	{
		//TODO: Instead of Reconstructing, have the Pins just react to each Individual Channel State Change
		ReconstructNode();
	}
}

int32 UAvaPlaybackNodeRoot::GetMinChildNodes() const
{
	return UAvaBroadcast::Get().GetChannelNameCount();
}

int32 UAvaPlaybackNodeRoot::GetMaxChildNodes() const
{
	return UAvaBroadcast::Get().GetChannelNameCount();
}

#if WITH_EDITOR
FName UAvaPlaybackNodeRoot::GetInputPinName(int32 InputPinIndex) const
{
	return UAvaBroadcast::Get().GetChannelName(InputPinIndex);
}
#endif

FName UAvaPlaybackNodeRoot::GetChannelName(int32 InChannelNameIndex) const
{
	const UAvaBroadcast& AvaBroadcast = UAvaBroadcast::Get();
	return AvaBroadcast.GetChannelName(InChannelNameIndex);
}

#undef LOCTEXT_NAMESPACE
