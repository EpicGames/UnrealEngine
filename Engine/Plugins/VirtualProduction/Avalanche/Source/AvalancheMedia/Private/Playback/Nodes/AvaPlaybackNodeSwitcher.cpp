// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Nodes/AvaPlaybackNodeSwitcher.h"
#include "AvaMediaDefines.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"

#if WITH_EDITOR
#include "UObject/NameTypes.h"
#endif

#define LOCTEXT_NAMESPACE "AvaPlaybackNodeSwitcher"

FText UAvaPlaybackNodeSwitcher::GetNodeDisplayNameText() const
{
	return LOCTEXT("SwitcherNode_Title", "Switcher");
}

void UAvaPlaybackNodeSwitcher::CreateStartingConnectors()
{
	InsertChildNode(ChildNodes.Num());
	InsertChildNode(ChildNodes.Num());
}

void UAvaPlaybackNodeSwitcher::Tick(float DeltaTime, FAvaPlaybackChannelParameters& ChannelParameters)
{
	// Special case - tick the other nodes to set their channel indices.
	for (int32 ChildIndex = 0; ChildIndex < ChildNodes.Num(); ++ChildIndex)
	{
		if (ChildIndex != SelectedIndex && ChildNodes.IsValidIndex(ChildIndex) && ChildNodes[ChildIndex])
		{
			FAvaPlaybackChannelParameters TmpParameters;
			TmpParameters.ChannelIndex = ChannelParameters.ChannelIndex;
			ChildNodes[ChildIndex]->Tick(DeltaTime, TmpParameters);
		}
	}
	
	if (bEnabled && ChildNodes.IsValidIndex(SelectedIndex) && IsValid(ChildNodes[SelectedIndex]))
	{
		TickChild(DeltaTime, SelectedIndex, ChannelParameters);
	}
}

#if WITH_EDITOR
FName UAvaPlaybackNodeSwitcher::GetInputPinName(int32 InputPinIndex) const
{
	return *FString::FromInt(InputPinIndex);
}
#endif

#undef LOCTEXT_NAMESPACE
