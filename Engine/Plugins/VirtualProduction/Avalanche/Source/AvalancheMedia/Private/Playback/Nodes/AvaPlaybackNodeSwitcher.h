// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playback/Nodes/AvaPlaybackNode.h"
#include "AvaPlaybackNodeSwitcher.generated.h"

class FName;
class FText;
struct FAvaPlaybackChannelParameters;

UCLASS()
class UAvaPlaybackNodeSwitcher : public UAvaPlaybackNode
{
	GENERATED_BODY()
	
	virtual FText GetNodeDisplayNameText() const override;
	
	virtual int32 GetMinChildNodes() const override { return 1; }
	virtual int32 GetMaxChildNodes() const override { return MaxAllowedChildNodes; }
	
	virtual void CreateStartingConnectors() override;
	
	virtual void Tick(float DeltaTime, FAvaPlaybackChannelParameters& ChannelParameters) override;

#if WITH_EDITOR
	virtual FName GetInputPinName(int32 InputPinIndex) const override;
#endif
	
protected:
	UPROPERTY(EditAnywhere, Category = "Motion Design")
	bool bEnabled = true;
	
	UPROPERTY(EditAnywhere, Category = "Motion Design", meta=(ClampMin = "0"))
	int32 SelectedIndex = 0;	
};
