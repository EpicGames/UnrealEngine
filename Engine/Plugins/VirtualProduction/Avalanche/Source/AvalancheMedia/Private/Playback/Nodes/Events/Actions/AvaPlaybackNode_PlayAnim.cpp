// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPlaybackNode_PlayAnim.h"
#include "Async/Async.h"
#include "AvaMediaDefines.h"
#include "AvaScene.h"
#include "AvaSequence.h"
#include "Broadcast/AvaBroadcast.h"
#include "Engine/Level.h"
#include "Internationalization/Text.h"
#include "Playback/AvaPlaybackGraph.h"
#include "Playback/Nodes/AvaPlaybackNode.h"
#include "Playback/Nodes/AvaPlaybackNodeLevelPlayer.h"
#include "Playback/Nodes/Events/Actions/AvaPlaybackAnimations.h"
#include "UObject/NoExportTypes.h"

#define LOCTEXT_NAMESPACE "AvaPlaybackNode_PlayAnim"

namespace UE::AvaMedia::NodePlayAnim::Private
{
	FName GetSequenceName(const TObjectPtr<UAvaSequence>& InSequence)
	{
		return InSequence->GetFName();
	}
}

FText UAvaPlaybackNode_PlayAnim::GetNodeDisplayNameText() const
{
	FFormatNamedArguments Args;
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		Args.Add("AnimationTree", FText::GetEmpty());
	}
	else
	{
		FString AnimationTree;
		for (TMap<FSoftObjectPath, FAvaPlaybackAnimations>::TConstIterator Iter(AnimationMap); Iter; ++Iter)
		{
			bool bAction = false;
			for (const FAvaPlaybackAnimPlaySettings& PlaySettings : Iter->Value.AvailableAnimations)
			{
				if (PlaySettings.Action != EAvaPlaybackAnimAction::None)
				{
					bAction = true;
					break;
				}
			}
			
			if (bAction)
			{
				AnimationTree.Append(TEXT("\n") + Iter->Key.GetAssetName());
			}
		}		
		Args.Add("AnimationTree", FText::FromString(AnimationTree));
	}
	return FText::Format(LOCTEXT("PlayAnimNode_Title", "Play Animation{AnimationTree}"), Args);
}

FText UAvaPlaybackNode_PlayAnim::GetNodeTooltipText() const
{
	return LOCTEXT("PlayAnimNode_Tooltip", "Plays an Animation from the Motion Design Asset");
}

void UAvaPlaybackNode_PlayAnim::OnEventTriggered(const FAvaPlaybackEventParameters& InEventParameters)
{
	UAvaPlaybackGraph* const Playback = GetPlayback();
	if (!Playback)
	{
		return;
	}

	const FSoftObjectPath& AssetPath = InEventParameters.Asset.ToSoftObjectPath();
	TArray<FName> ChannelNames = Playback->GetChannelNamesForIndices(InEventParameters.ChannelIndices);
	
	// Gather the Animations to Play
	if (FAvaPlaybackAnimations* const FoundAnimations = AnimationMap.Find(AssetPath))
	{
		for (const FAvaPlaybackAnimPlaySettings& PlaySettings : FoundAnimations->AvailableAnimations)
		{
			if (PlaySettings.Action != EAvaPlaybackAnimAction::None)
			{
				for (const FName& ChannelName : ChannelNames)
				{
					Playback->PushAnimationCommand(AssetPath, ChannelName.ToString(), PlaySettings.Action, PlaySettings);
				}
			}
		}
	}
}

void UAvaPlaybackNode_PlayAnim::PreDryRun()
{
	SeenAssetsInDryRun.Empty();
}

void UAvaPlaybackNode_PlayAnim::DryRun(const TArray<UAvaPlaybackNode*>& InAncestors)
{
	for (UAvaPlaybackNode* const PlaybackNode : InAncestors)
	{
		if (UAvaPlaybackNodeLevelPlayer* const PlayerNode = Cast<UAvaPlaybackNodeLevelPlayer>(PlaybackNode))
		{
			TSoftObjectPtr<UWorld> Asset = PlayerNode->GetAsset();
			if (UWorld* const World = Asset.LoadSynchronous())
			{
				SeenAssetsInDryRun.Add(PlayerNode->GetAssetPath());
				FAvaPlaybackAnimations& Animations = AnimationMap.FindOrAdd(PlayerNode->GetAssetPath());

				AAvaScene* Scene = nullptr;
				World->PersistentLevel->Actors.FindItemByClass(&Scene);
				if (IsValid(Scene))
				{
					for (const TObjectPtr<UAvaSequence>& Animation : Scene->GetSequences())
					{
						if (Animation)
						{
							Animations.AvailableAnimations.FindOrAdd(UE::AvaMedia::NodePlayAnim::Private::GetSequenceName(Animation));
						}
					}
				}
			}
		}
	}
}

void UAvaPlaybackNode_PlayAnim::PostDryRun()
{
	bool bRefreshNode = false;
	
	//Remove all the Assets that were not Seen.
	for (TMap<FSoftObjectPath, FAvaPlaybackAnimations>::TIterator Iter(AnimationMap); Iter; ++Iter)
	{
		if (!SeenAssetsInDryRun.Contains(Iter->Key))
		{
			Iter.RemoveCurrent();
			bRefreshNode = true;
		}
	}

	if (bRefreshNode)
	{
		RefreshNode(false);
	}

	SeenAssetsInDryRun.Empty();
}

#undef LOCTEXT_NAMESPACE
