// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Playback/Nodes/Events/Actions/AvaPlaybackAnimations.h"
#include "Playback/Nodes/Events/AvaPlaybackNodeAction.h"
#include "AvaPlaybackNode_PlayAnim.generated.h"

class FText;
class UAvaPlaybackNode;
struct FAvaPlaybackAnimations;
struct FAvaPlaybackEventParameters;
struct FSoftObjectPath;

UCLASS()
class UAvaPlaybackNode_PlayAnim : public UAvaPlaybackNodeAction
{
	GENERATED_BODY()

public:
	virtual FText GetNodeDisplayNameText() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual void OnEventTriggered(const FAvaPlaybackEventParameters& InEventParameters) override;

	virtual void PreDryRun() override;
	virtual void DryRun(const TArray<UAvaPlaybackNode*>& InAncestors) override;
	virtual void PostDryRun() override;

protected:
	UPROPERTY(VisibleAnywhere, Category = "Motion Design")
	TMap<FSoftObjectPath, FAvaPlaybackAnimations> AnimationMap;

	//All the assets connected to this Node from the Dry Run (only populated while Dry Running!)
	TSet<FSoftObjectPath> SeenAssetsInDryRun;
};
