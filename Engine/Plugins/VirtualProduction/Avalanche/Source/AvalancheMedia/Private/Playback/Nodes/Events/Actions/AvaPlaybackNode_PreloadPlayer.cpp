// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPlaybackNode_PreloadPlayer.h"
#include "AvaMediaDefines.h"
#include "Broadcast/AvaBroadcast.h"
#include "Internationalization/Text.h"
#include "Playback/AvaPlaybackGraph.h"

#define LOCTEXT_NAMESPACE "AvaPlaybackNode_PreloadPlayer"

FText UAvaPlaybackNode_PreloadPlayer::GetNodeDisplayNameText() const
{
	return LOCTEXT("PreloadPlayerNode_Title", "Preload Player");
}

FText UAvaPlaybackNode_PreloadPlayer::GetNodeTooltipText() const
{
	return LOCTEXT("PreloadPlayerNode_Tooltip", "Loads the Motion Design World Preemptively and Begins Play on that World");
}

void UAvaPlaybackNode_PreloadPlayer::OnEventTriggered(const FAvaPlaybackEventParameters& InEventParameters)
{
	UAvaPlaybackGraph* const Playback = GetPlayback();
	
	//Make sure we have a valid Motion Design Asset
	if (Playback && InEventParameters.IsAssetValid())
	{
		TArray<FName> ChannelNames = Playback->GetChannelNamesForIndices(InEventParameters.ChannelIndices);
		for (const FName& ChannelName : ChannelNames)
		{
			Playback->LoadAsset(InEventParameters.Asset, ChannelName, FString());
		}
	}
}

#undef LOCTEXT_NAMESPACE 