// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playback/Nodes/Events/AvaPlaybackNodeAction.h"
#include "AvaPlaybackNode_PreloadPlayer.generated.h"

class FText;
struct FAvaPlaybackEventParameters;

UCLASS()
class UAvaPlaybackNode_PreloadPlayer : public UAvaPlaybackNodeAction
{
	GENERATED_BODY()

public:
	virtual FText GetNodeDisplayNameText() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual void OnEventTriggered(const FAvaPlaybackEventParameters& InEventParameters) override;
};
