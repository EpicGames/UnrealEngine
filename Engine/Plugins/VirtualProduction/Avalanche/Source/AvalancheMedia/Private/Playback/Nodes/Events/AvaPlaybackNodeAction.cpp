// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Nodes/Events/AvaPlaybackNodeAction.h"
#include "Internationalization/Text.h"
#include "Playback/Nodes/AvaPlaybackNode.h"

FText UAvaPlaybackNodeAction::GetNodeCategoryText() const
{
	return NodeCategory::EventAction;
}
