// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Nodes/Events/AvaPlaybackNodeEvent.h"
#include "AvaMediaDefines.h"

void UAvaPlaybackNodeEvent::TickEvent(float DeltaTime, FAvaPlaybackEventParameters& OutEventParameters)
{
	bool bShouldTriggerEventAction = false;
	
	for (int32 Index = 0; Index < ChildNodes.Num(); ++Index)
	{
		if (ChildNodes.IsValidIndex(Index) && ChildNodes[Index])
		{
			Cast<UAvaPlaybackNodeEvent>(ChildNodes[Index])->TickEvent(DeltaTime, OutEventParameters);
			if (OutEventParameters.ShouldTriggerEventAction())
			{
				bShouldTriggerEventAction = true;
				NotifyChildNodeSucceeded(Index);
			}
		}
	}

	//Note: Currently outside of the Iteration loop to avoid having to call it multiple times from different sources
	if (bShouldTriggerEventAction)
	{
		OnEventTriggered(OutEventParameters);
	}
}
