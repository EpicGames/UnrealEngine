// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Nodes/Events/AvaPlaybackNodeFlow.h"
#include "Internationalization/Text.h"
#include "Playback/Nodes/AvaPlaybackNode.h"

FText UAvaPlaybackNodeFlow::GetNodeCategoryText() const
{
	return NodeCategory::EventFlowControl;
}
