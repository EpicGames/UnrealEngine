// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Nodes/Events/AvaPlaybackNodeTrigger.h"
#include "AvaMediaDefines.h"
#include "Internationalization/Text.h"
#include "Playback/Nodes/AvaPlaybackNode.h"

FText UAvaPlaybackNodeTrigger::GetNodeCategoryText() const
{
	return NodeCategory::EventTrigger;
}

void UAvaPlaybackNodeTrigger::TriggerEvent()
{
	bEventTriggered = true;
}

void UAvaPlaybackNodeTrigger::Reset()
{
	bEventTriggered = false;
}

void UAvaPlaybackNodeTrigger::TickEvent(float DeltaTime, FAvaPlaybackEventParameters& OutEventParameters)
{
	if (bEventTriggered)
	{
		OutEventParameters.RequestTriggerEventAction();
	}
}
