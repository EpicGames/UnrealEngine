// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPlaybackNode_Hub.h"
#include "AvaMediaDefines.h"
#include "Internationalization/Text.h"

#define LOCTEXT_NAMESPACE "AvaPlaybackNode_Hub"

FText UAvaPlaybackNode_Hub::GetNodeDisplayNameText() const
{
	return LOCTEXT("HubNode_Title", "Hub");
}

FText UAvaPlaybackNode_Hub::GetNodeTooltipText() const
{
	return LOCTEXT("HubNode_Tooltip", "Iterates each Input in order, from start to finish.");
}

void UAvaPlaybackNode_Hub::TickEvent(float DeltaTime, FAvaPlaybackEventParameters& OutEventParameters)
{
	for (int32 Index = 0; Index < ChildNodes.Num(); ++Index)
	{
		if (ChildNodes.IsValidIndex(Index) && ChildNodes[Index])
		{
			FAvaPlaybackEventParameters EventParameters;
			EventParameters.Asset = OutEventParameters.Asset;
			
			Cast<UAvaPlaybackNodeEvent>(ChildNodes[Index])->TickEvent(DeltaTime, EventParameters);
			
			if (EventParameters.ShouldTriggerEventAction())
			{
				OutEventParameters.RequestTriggerEventAction();
				NotifyChildNodeSucceeded(Index);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE