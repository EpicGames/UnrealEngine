// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playback/Nodes/Events/AvaPlaybackNodeFlow.h"
#include "AvaPlaybackNode_Hub.generated.h"

class FText;
struct FAvaPlaybackEventParameters;

UCLASS()
class UAvaPlaybackNode_Hub : public UAvaPlaybackNodeFlow
{
	GENERATED_BODY()

public:
	virtual FText GetNodeDisplayNameText() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual int32 GetMaxChildNodes() const override { return MaxAllowedChildNodes; }

	virtual void TickEvent(float DeltaTime, FAvaPlaybackEventParameters& OutEventParameters) override;
};
