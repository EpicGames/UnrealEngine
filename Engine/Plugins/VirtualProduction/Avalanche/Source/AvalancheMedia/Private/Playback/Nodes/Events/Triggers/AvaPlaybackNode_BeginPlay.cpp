// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPlaybackNode_BeginPlay.h"
#include "Internationalization/Text.h"

#define LOCTEXT_NAMESPACE "AvaPlaybackNode_BeginPlay"

FText UAvaPlaybackNode_BeginPlay::GetNodeDisplayNameText() const
{
	return LOCTEXT("BeginPlayNode_Title", "Event Begin Play");
}

FText UAvaPlaybackNode_BeginPlay::GetNodeTooltipText() const
{
	return LOCTEXT("BeginPlayNode_Tooltip", "Triggers when the Playback Asset Starts Play");
}

void UAvaPlaybackNode_BeginPlay::NotifyPlaybackStateChanged(bool bPlaying)
{
	if (bPlaying)
	{
		TriggerEvent();
	}
}

#undef LOCTEXT_NAMESPACE