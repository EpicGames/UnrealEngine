// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playback/Nodes/Events/AvaPlaybackNodeTrigger.h"
#include "AvaPlaybackNode_BeginPlay.generated.h"

class FText;

UCLASS()
class UAvaPlaybackNode_BeginPlay : public UAvaPlaybackNodeTrigger
{
	GENERATED_BODY()

public:
	virtual FText GetNodeDisplayNameText() const override;
	virtual FText GetNodeTooltipText() const override;
	virtual void NotifyPlaybackStateChanged(bool bPlaying) override;
};
