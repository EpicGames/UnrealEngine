// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPlaybackNode_KeyInput.h"

#include "Framework/Application/SlateApplication.h"
#include "Input/Events.h"
#include "Internationalization/Text.h"
#include "Playback/AvaPlaybackGraph.h"

#define LOCTEXT_NAMESPACE "AvaPlaybackNode_KeyInput"

bool UAvaPlaybackNode_KeyInput::FEventInputProcessor::HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent)
{
	return Node ? Node->HandleKeyDownEvent(SlateApp, InKeyEvent) : false;
}

void UAvaPlaybackNode_KeyInput::PostAllocateNode()
{
	Super::PostAllocateNode();
	
	InputProcessor = MakeShared<FEventInputProcessor>(this);
	
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().RegisterInputPreProcessor(InputProcessor);
	}
}

void UAvaPlaybackNode_KeyInput::BeginDestroy()
{
	if (InputProcessor.IsValid() && FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().UnregisterInputPreProcessor(InputProcessor);
	}
	Super::BeginDestroy();
}

FText UAvaPlaybackNode_KeyInput::GetNodeDisplayNameText() const
{
	FFormatNamedArguments Args;
	
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		static const FText LineBreak = FText::FromString(TEXT("\n"));
		Args.Add(TEXT("LineBreak"), LineBreak);

		if (InputChord.IsValidChord())
		{
			Args.Add(TEXT("InputChord"), InputChord.GetInputText());
		}
		else
		{
			static const FText None = LOCTEXT("KeyInput_NoInputTitle", "None");
			Args.Add(TEXT("InputChord"), None);
		}
	}
	else
	{
		//For Graph Schema (i.e. using CDO), a generic name should display rather than with any keys or extra formatting
		Args.Add(TEXT("LineBreak"), FText::GetEmpty());
		Args.Add(TEXT("InputChord"), FText::GetEmpty());
	}
	
	return FText::Format(LOCTEXT("KeyInput_Title", "Event Key Input{LineBreak}{InputChord}"), Args);
}

FText UAvaPlaybackNode_KeyInput::GetNodeTooltipText() const
{
	return LOCTEXT("KeyInputNode_Tooltip", "Triggers the Event when the set Key is pressed");
}

bool UAvaPlaybackNode_KeyInput::HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent)
{
	// Don't process input while not Playing
	UAvaPlaybackGraph* const Playback = GetPlayback();
	if (!Playback || !Playback->IsPlaying())
	{
		return false;
	}
		
	const FInputChord KeyDownChord(InKeyEvent.GetKey()
		, InKeyEvent.IsShiftDown()
		, InKeyEvent.IsControlDown()
		, InKeyEvent.IsAltDown()
		, InKeyEvent.IsCommandDown());
	
	if (InputChord.IsValidChord() && KeyDownChord == InputChord)
	{
		TriggerEvent();
		return true; 
	}
	
	return false;
}

#undef LOCTEXT_NAMESPACE
