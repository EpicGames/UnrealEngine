// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Application/IInputProcessor.h"
#include "Framework/Commands/InputChord.h"
#include "Playback/Nodes/Events/AvaPlaybackNodeTrigger.h"
#include "Templates/SharedPointer.h"
#include "AvaPlaybackNode_KeyInput.generated.h"

class ICursor;
class FSlateApplication;
class FText;
struct FKeyEvent;

UCLASS()
class UAvaPlaybackNode_KeyInput : public UAvaPlaybackNodeTrigger
{
	GENERATED_BODY()
	
	class FEventInputProcessor: public IInputProcessor
	{
	public:
		FEventInputProcessor(UAvaPlaybackNode_KeyInput* InNode) : Node(InNode) {}
		virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor) override {}
		virtual bool HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent) override;
		
	protected:
		UAvaPlaybackNode_KeyInput* Node = nullptr;
	};

public:
	virtual void PostAllocateNode() override;
	virtual void BeginDestroy() override;

	virtual FText GetNodeDisplayNameText() const override;
	virtual FText GetNodeTooltipText() const override;
	
	bool HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent);

protected:
	TSharedPtr<UAvaPlaybackNode_KeyInput::FEventInputProcessor> InputProcessor;
	
	UPROPERTY(EditAnywhere, Category = "Motion Design", meta = (ShowOnlyInnerProperties))
	FInputChord InputChord;
};
