// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Transition/AvaPlaybackServerTransition.h"

#include "Playable/AvaPlayableGroupManager.h"
#include "Playable/Transition/AvaPlayableTransition.h"
#include "Playback/AvaPlaybackManager.h"
#include "Playback/AvaPlaybackServer.h"
#include "Playback/AvaPlaybackUtils.h"

namespace UE::AvaPlaybackServerTransition::Private
{
	FString GetPrettyPlaybackInstanceInfo(const FAvaPlaybackInstance* InPlaybackInstance)
	{
		if (InPlaybackInstance)
		{
			return FString::Printf(TEXT("Id:%s, Asset:%s, Channel:%s, UserData:\"%s\""),
				*InPlaybackInstance->GetInstanceId().ToString(),
				*InPlaybackInstance->GetSourcePath().GetAssetName(),
				*InPlaybackInstance->GetChannelName(),
				*InPlaybackInstance->GetInstanceUserData());
		}
		return TEXT("");
	}

	UAvaPlayable* GetPlayable(const FAvaPlaybackInstance* InPlaybackInstance)
	{
		if (InPlaybackInstance && InPlaybackInstance->GetPlayback())
		{
			return InPlaybackInstance->GetPlayback()->GetFirstPlayable();
		}
		return nullptr;
	}

	TSharedPtr<FAvaPlaybackInstance> FindInstanceForPlayable(const TArray<TWeakPtr<FAvaPlaybackInstance>>& InPlaybackInstancesWeak, const UAvaPlayable* InPlayable)
	{
		for (const TWeakPtr<FAvaPlaybackInstance>& InstanceWeak : InPlaybackInstancesWeak)
		{
			TSharedPtr<FAvaPlaybackInstance> Instance = InstanceWeak.Pin();
			if (GetPlayable(Instance.Get()) == InPlayable)
			{
				return Instance;
			}
		}
		return nullptr;
	}

	TSharedPtr<FAvaPlaybackInstance> FindInstance(const TArray<TWeakPtr<FAvaPlaybackInstance>>& InPlaybackInstancesWeak, const FGuid& InInstanceId)
	{
		for (const TWeakPtr<FAvaPlaybackInstance>& InstanceWeak : InPlaybackInstancesWeak)
		{
			TSharedPtr<FAvaPlaybackInstance> Instance = InstanceWeak.Pin();
			if (Instance && Instance->GetInstanceId() == InInstanceId)
			{
				return Instance;
			}
		}
		return nullptr;
	}

	// Check if the given set of playback instances are preventing the start of the transition.
	bool CanStartTransition(const UAvaPlaybackServerTransition* InTransition,
		const TArray<TWeakPtr<FAvaPlaybackInstance>>& InPlaybackInstancesWeak, EAvaPlayableTransitionEntryRole InEntryRole, bool& bOutShouldDiscard)
	{
		using namespace UE::AvaPlayback::Utils;
		for (const TWeakPtr<FAvaPlaybackInstance>& InstanceWeak : InPlaybackInstancesWeak)
		{
			const TSharedPtr<FAvaPlaybackInstance> Instance = InstanceWeak.Pin();
			if (!Instance)
			{
				// For now, we discard transitions with invalid instance.
				UE_LOGF(LogAvaPlaybackServer, Warning,
					"%ls Discarding Playback Transition {%ls}. Reason: Invalid Instance. ",
					*GetBriefFrameInfo(), *InTransition->GetPrettyTransitionInfo());

				bOutShouldDiscard = true;
				return false;
			}
			
			const UAvaPlayable* Playable = GetPlayable(Instance.Get());
			if (!Playable)
			{
				bOutShouldDiscard = false;
				return false;	// Playable not yet created.
			}

			const EAvaPlayableStatus PlayableStatus = Playable->GetPlayableStatus();

			if (PlayableStatus == EAvaPlayableStatus::Unknown
				|| PlayableStatus == EAvaPlayableStatus::Error)
			{
				UE_LOGF(LogAvaPlaybackServer, Warning,
            		"%ls Discarding Playback Transition {%ls}. Reason: Playable status: \"%ls\". ",
            		*GetBriefFrameInfo(), *InTransition->GetPrettyTransitionInfo(), *StaticEnumToString(PlayableStatus));

				// Discard the command
				bOutShouldDiscard = true;
				return false;
			}

			// todo: this might cause commands to become stale and fill the pending command list
			if (PlayableStatus == EAvaPlayableStatus::Unloaded)
			{
				UE_LOGF(LogAvaPlaybackServer, Warning,
					"%ls Playback Transition {%ls}: Playable \"%ls\" (Id:%ls) is unloaded.",
					*GetBriefFrameInfo(), *InTransition->GetPrettyTransitionInfo(), *Playable->GetSourceAssetPath().GetAssetName(), *Playable->GetInstanceId().ToString());

				return false;
			}

			// Asset status must be visible to run the command.
			// If not visible, the components are not yet added to the world.
			if (PlayableStatus != EAvaPlayableStatus::Visible)
			{
				// Keep the command in the queue for next tick.
				bOutShouldDiscard = false;
				return false;
			}

			// Only check ready to play for enter playables,
			// as these are the ones that could be introducing a new asset to prepare
			if (InEntryRole == EAvaPlayableTransitionEntryRole::Enter && !Playable->IsReadyToPlay())
			{
				// Keep the command in the queue for next tick.
				bOutShouldDiscard = false;
				return false;
			}
		}

		bOutShouldDiscard = true;
		return true;
	}
}

UAvaPlaybackServerTransition* UAvaPlaybackServerTransition::MakeNew(const TSharedPtr<FAvaPlaybackServer>& InPlaybackServer)
{
	UAvaPlaybackServerTransition* NewTransition = NewObject<UAvaPlaybackServerTransition>();
	NewTransition->PlaybackServerWeak = InPlaybackServer.ToWeakPtr();
	return NewTransition;
}

void UAvaPlaybackServerTransition::AddPendingEnterInstanceIds(const TArray<FGuid>& InInstanceIds)
{
	PendingEnterInstanceIds.Reserve(InInstanceIds.Num());
	
	for (const FGuid& InstanceId : InInstanceIds)
	{
		PendingEnterInstanceIds.AddUnique(InstanceId);		
	}
}

void UAvaPlaybackServerTransition::AddPendingPlayingInstanceId(const FGuid& InInstanceId)
{
	PendingPlayingInstanceIds.AddUnique(InInstanceId);
}
	
void UAvaPlaybackServerTransition::AddPendingExitInstanceId(const FGuid& InInstanceId)
{
	PendingExitInstanceIds.AddUnique(InInstanceId);
}

void UAvaPlaybackServerTransition::SetEnterValues(const TArray<FAvaPlayableRemoteControlValues>& InEnterValues)
{
	EnterValues.Reserve(InEnterValues.Num());
	for (const FAvaPlayableRemoteControlValues& Values : InEnterValues)
	{
		EnterValues.Add(MakeShared<FAvaPlayableRemoteControlValues>(Values));
	}
}

bool UAvaPlaybackServerTransition::AddEnterInstance(const TSharedPtr<FAvaPlaybackInstance>& InPlaybackInstance)
{
	if (!InPlaybackInstance)
	{
		return false;
	}
	
	// Register this transition as a visibility constraint.
	using namespace UE::AvaPlaybackServerTransition::Private;
	if (const UAvaPlayable* Playable = GetPlayable(InPlaybackInstance.Get()))
	{
		if (UAvaPlayableGroup* PlayableGroup = Playable->GetPlayableGroup())
		{
			PlayableGroup->RegisterVisibilityConstraint(this);
		}
	}
	else if (UAvaPlaybackGraph* Playback = InPlaybackInstance->GetPlayback())
	{
		// If the playable is not created yet, register to the creation event.
		Playback->OnPlayableCreated.AddUObject(this, &UAvaPlaybackServerTransition::OnPlayableCreated);
	}
	
	return AddPlaybackInstance(InPlaybackInstance, EnterPlaybackInstancesWeak);
}

bool UAvaPlaybackServerTransition::AddPlayingInstance(const TSharedPtr<FAvaPlaybackInstance>& InPlaybackInstance)
{
	return AddPlaybackInstance(InPlaybackInstance, PlayingPlaybackInstancesWeak);
}

bool UAvaPlaybackServerTransition::AddExitInstance(const TSharedPtr<FAvaPlaybackInstance>& InPlaybackInstance)
{
	return AddPlaybackInstance(InPlaybackInstance, ExitPlaybackInstancesWeak);
}

void UAvaPlaybackServerTransition::TryResolveInstances(const FAvaPlaybackServer& InPlaybackServer)
{
	for (TArray<FGuid>::TIterator InstanceIdIt(PendingEnterInstanceIds); InstanceIdIt; ++InstanceIdIt)
	{
		if (TSharedPtr<FAvaPlaybackInstance> Instance = InPlaybackServer.FindActivePlaybackInstance(*InstanceIdIt))
		{
			AddEnterInstance(Instance);
			InstanceIdIt.RemoveCurrent();
		}
	}
	
	for (TArray<FGuid>::TIterator InstanceIdIt(PendingPlayingInstanceIds); InstanceIdIt; ++InstanceIdIt)
	{
		if (TSharedPtr<FAvaPlaybackInstance> Instance = InPlaybackServer.FindActivePlaybackInstance(*InstanceIdIt))
		{
			AddPlayingInstance(Instance);
			InstanceIdIt.RemoveCurrent();
		}
	}

	for (TArray<FGuid>::TIterator InstanceIdIt(PendingExitInstanceIds); InstanceIdIt; ++InstanceIdIt)
	{
		if (TSharedPtr<FAvaPlaybackInstance> Instance = InPlaybackServer.FindActivePlaybackInstance(*InstanceIdIt))
		{
			AddExitInstance(Instance);
			InstanceIdIt.RemoveCurrent();
		}
	}
}

bool UAvaPlaybackServerTransition::ContainsInstance(const FGuid& InInstanceId) const
{
	if (PendingEnterInstanceIds.Contains(InInstanceId)
		|| PendingPlayingInstanceIds.Contains(InInstanceId)
		|| PendingExitInstanceIds.Contains(InInstanceId))
	{
		return true;
	}

	using namespace UE::AvaPlaybackServerTransition::Private;
	if (FindInstance(EnterPlaybackInstancesWeak, InInstanceId).IsValid()
		|| FindInstance(PlayingPlaybackInstancesWeak, InInstanceId).IsValid()
		|| FindInstance(ExitPlaybackInstancesWeak, InInstanceId).IsValid())
	{
		return true;
	}
	return false;
}

bool UAvaPlaybackServerTransition::IsVisibilityConstrained(const UAvaPlayable* InPlayable) const
{
	using namespace UE::AvaPlaybackServerTransition::Private;
	bool bAllPlayablesLoaded = true;
	bool bIsPlayableInThisTransition = false;
	
	for (const TWeakPtr<FAvaPlaybackInstance>& InstanceWeak : EnterPlaybackInstancesWeak)
	{
		if (const TSharedPtr<FAvaPlaybackInstance> Instance = InstanceWeak.Pin())
		{
			if (const UAvaPlayable* Playable = GetPlayable(Instance.Get()))
			{
				if (Playable == InPlayable)
				{
					bIsPlayableInThisTransition = true;
				}
				const EAvaPlayableStatus PlayableStatus = Playable->GetPlayableStatus();
				if (PlayableStatus != EAvaPlayableStatus::Loaded && PlayableStatus != EAvaPlayableStatus::Visible)
				{
					bAllPlayablesLoaded = false;
				}
			}
		}
	}
	return bIsPlayableInThisTransition && !bAllPlayablesLoaded;
}


bool UAvaPlaybackServerTransition::CanStart(bool& bOutShouldDiscard)
{
	using namespace UE::AvaPlaybackServerTransition::Private;

	// Wait for any unresolved instances to be loaded.
	if (!PendingEnterInstanceIds.IsEmpty()
		|| !PendingPlayingInstanceIds.IsEmpty()
		|| !PendingExitInstanceIds.IsEmpty())
	{
		bOutShouldDiscard = false;
		return false;
	}
	
	if (!EnterValues.IsEmpty() && !AsyncAssetLoader)
	{
		// Start loading the RC referenced assets.
		AsyncAssetLoader = MakeShared<UE::AvaPlayback::Utils::FAsyncAssetLoader>();
		TSet<FSoftObjectPath> Assets;
		for (const TSharedPtr<FAvaPlayableRemoteControlValues>& Values : EnterValues)
		{
			FAvaPlayableRemoteControlValues::CollectReferencedAssetPaths(Values->ControllerValues, Assets);
			FAvaPlayableRemoteControlValues::CollectReferencedAssetPaths(Values->EntityValues, Assets);
		}
		AsyncAssetLoader->BeginLoadingAssets(Assets.Array());
	}

	if (!CanStartTransition(this, EnterPlaybackInstancesWeak, EAvaPlayableTransitionEntryRole::Enter, bOutShouldDiscard))
	{
		return false;
	}

	// Note: need to check the "non-entering" instances too in case the playback commands got delayed
	// causing playables to also need loading/recovering.
	
	if (!CanStartTransition(this, PlayingPlaybackInstancesWeak, EAvaPlayableTransitionEntryRole::Playing, bOutShouldDiscard))
	{
		return false;
	}

	if (!CanStartTransition(this, ExitPlaybackInstancesWeak, EAvaPlayableTransitionEntryRole::Exit, bOutShouldDiscard))
	{
		return false;
	}

	// Wait for RC referenced assets to be loaded.
	if (AsyncAssetLoader && !AsyncAssetLoader->IsLoadingCompleted())
	{
		bOutShouldDiscard = false;
		return false;
	}

	bOutShouldDiscard = true;
	return true;
}

void UAvaPlaybackServerTransition::Start()
{
	TWeakObjectPtr<UAvaPlaybackServerTransition> ThisWeak(this);
	auto StartEventHandler = [ThisWeak]()
	{
		if (UAvaPlaybackServerTransition* This = ThisWeak.Get())
		{
			This->Start_Synchronized();
		}
	};

	FString StartEventSignature = FString(TEXT("PlaybackServerTransitionStart_")) + TransitionId.ToString();
	
	if (TSharedPtr<FAvaPlaybackServer> PlaybackServer = PlaybackServerWeak.Pin())
	{
		if (UAvaPlayableGroupManager* GroupManager = PlaybackServer->GetPlaybackManager().GetPlayableGroupManager())
		{
			if (!GroupManager->IsSynchronizedEventPushed(StartEventSignature))
			{
				GroupManager->PushSynchronizedEvent(MoveTemp(StartEventSignature), MoveTemp(StartEventHandler));
			}
		}
	}
}

void UAvaPlaybackServerTransition::Start_Synchronized()
{
	RegisterToPlayableTransitionEvent();

	// Playables should be loaded at this point since the synchronized part is to wait on loading assets.
	MakePlayableTransition();

	bool bTransitionStarted = false;
	
	if (PlayableTransition)
	{
		LogDetailedTransitionInfo();
		bTransitionStarted = PlayableTransition->Start();
	}

	if (!bTransitionStarted)
	{
		Stop();
	}
}

void UAvaPlaybackServerTransition::Stop()
{
	// Guard against reentrancy from different event propagation paths,
	// i.e. direct stop call from server command or from playable transition event.
	if (bIsStopping)
	{
		return;
	}

	TGuardValue ReentrancyGuard(bIsStopping, true);

	if (PlayableTransition)
	{
		PlayableTransition->Stop();
		PlayableTransition = nullptr;
	}

	for (const TWeakPtr<FAvaPlaybackInstance>& InstanceWeak : EnterPlaybackInstancesWeak)
	{
		if (const TSharedPtr<FAvaPlaybackInstance> Instance = InstanceWeak.Pin())
		{
			if (UAvaPlaybackGraph* Playback = Instance->GetPlayback())
			{
				Playback->OnPlayableCreated.RemoveAll(this);
			}
		}
	}
	
	UnregisterFromPlayableTransitionEvent();
	
	// Remove transition from server.
	if (const TSharedPtr<FAvaPlaybackServer> PlaybackServer = PlaybackServerWeak.Pin())
	{
		if (!PlaybackServer->RemovePlaybackInstanceTransition(TransitionId))
		{
			UE_LOGF(LogAvaPlaybackServer, Error,
				"%ls Failed to remove Playback Transition {%ls}. Reason: not found in server's active transitions.",
				*UE::AvaPlayback::Utils::GetBriefFrameInfo(), *GetPrettyTransitionInfo());
		}

		// Make sure we send the finish transition event to the client, in case it hasn't been sent through the
		// playable transition event path already.
		if (!bIsFinishedSent)
		{
			SendFinishTransitionEvent(*PlaybackServer, EAvaPlayableTransitionEventFlags::Finished);
		}
	}
}

bool UAvaPlaybackServerTransition::IsRunning() const
{
	return PlayableTransition ? PlayableTransition->IsRunning() : false;
}

FString UAvaPlaybackServerTransition::GetPrettyTransitionInfo() const
{
	return FString::Printf(TEXT("Id:%s, Channel:%s, Client:%s"),
		*TransitionId.ToString(), *ChannelName.ToString(), *ClientName);
}

FString UAvaPlaybackServerTransition::GetBriefTransitionDescription() const
{
	auto MakeInstanceIdList = [](const TArray<TWeakPtr<FAvaPlaybackInstance>>& InPlaybackInstancesWeak) -> FString
	{
		FString InstanceIdList;
		for (const TWeakPtr<FAvaPlaybackInstance>& InstanceWeak : InPlaybackInstancesWeak)
		{
			if (const TSharedPtr<FAvaPlaybackInstance> Instance = InstanceWeak.Pin())
			{
				InstanceIdList += FString::Printf(TEXT("%s%s"), InstanceIdList.IsEmpty() ? TEXT("") : TEXT(", "), *Instance->GetInstanceId().ToString());
			}
		}
		return InstanceIdList.IsEmpty() ? TEXT("None") : InstanceIdList;
	};
	
	const FString EnterInstanceList = MakeInstanceIdList(EnterPlaybackInstancesWeak);
	const FString PlayingInstanceList = MakeInstanceIdList(PlayingPlaybackInstancesWeak);
	const FString ExitInstanceList = MakeInstanceIdList(ExitPlaybackInstancesWeak);

	return FString::Printf(TEXT("Enter Instance(s): [%s], Playing Instance(s): [%s], Exit Instance(s): [%s]."), *EnterInstanceList, *PlayingInstanceList, *ExitInstanceList);
}

TSharedPtr<FAvaPlaybackInstance> UAvaPlaybackServerTransition::FindInstanceForPlayable(const UAvaPlayable* InPlayable)
{
	using namespace UE::AvaPlaybackServerTransition;
	if (!InPlayable)
	{
		return nullptr;
	}

	TSharedPtr<FAvaPlaybackInstance> Instance = Private::FindInstanceForPlayable(EnterPlaybackInstancesWeak, InPlayable);
	if (Instance)
	{
		return Instance;
	}
	
	Instance = Private::FindInstanceForPlayable(PlayingPlaybackInstancesWeak, InPlayable);
	if (Instance)
	{
		return Instance;
	}

	Instance = Private::FindInstanceForPlayable(ExitPlaybackInstancesWeak, InPlayable);
	if (Instance)
	{
		return Instance;
	}
	return nullptr;
}

void UAvaPlaybackServerTransition::OnTransitionEvent(UAvaPlayable* InPlayable, UAvaPlayableTransition* InTransition, EAvaPlayableTransitionEventFlags InTransitionFlags)
{
	using namespace UE::AvaPlaybackServerTransition::Private;
	using namespace UE::AvaPlayback::Utils;
	
	// not this transition.
	if (InTransition != PlayableTransition || PlayableTransition == nullptr)
	{
		return;
	}

	const TSharedPtr<FAvaPlaybackServer> PlaybackServer = PlaybackServerWeak.Pin();

	// Find the page player for this playable
	if (const TSharedPtr<FAvaPlaybackInstance> Instance = FindInstanceForPlayable(InPlayable))
	{
		// Relay the transition event back to the client
		if (PlaybackServer)
		{
			// Keeping a record of the instance event. It will be packed in the final "finish" message to guarantee the order of events.			
			FAvaPlaybackTransitionEventEntry& EventEntry = InstanceEventEntries.Emplace_GetRef(InPlayable->GetInstanceId(), InTransitionFlags);

			// We also send it immediately in its own message in case the transition is long.
			// The client will process only once, which ever comes first between those to transmission paths.
			PlaybackServer->SendPlayableTransitionEvent(TransitionId, EventEntry, ChannelName, ClientName);
		}
		
		if (EnumHasAnyFlags(InTransitionFlags, EAvaPlayableTransitionEventFlags::StopPlayable))
		{
			// Validating that we are not removing an "enter" playable.
			if (PlayableTransition->IsEnterPlayable(InPlayable))
			{
				UE_LOGF(LogAvaPlaybackServer, Error,
					"%ls Playback Transition {%ls} Error: An \"enter\" playable is being discarded for instance {%ls}.",
					*GetBriefFrameInfo(), *GetPrettyTransitionInfo(), *GetPrettyPlaybackInstanceInfo(Instance.Get()));
			}

			// See UAvaRundownPagePlayer::Stop()
			const EAvaPlaybackStopOptions PlaybackStopOptions = bUnloadDiscardedInstances ?
				EAvaPlaybackStopOptions::Default | EAvaPlaybackStopOptions::Unload : EAvaPlaybackStopOptions::Default;	
			Instance->GetPlayback()->Stop(PlaybackStopOptions);
			
			if (bUnloadDiscardedInstances)
			{
				Instance->Unload();
				// Remove instance from the server.
				if (PlaybackServer)
				{
					if (!PlaybackServer->RemoveActivePlaybackInstance(Instance->GetInstanceId()))
					{
						UE_LOGF(LogAvaPlaybackServer, Error,
							"%ls Playback Transition {%ls} Error: \"exit\" instance {%ls} was not found in server's active instances. ",
							*GetBriefFrameInfo(), *GetPrettyTransitionInfo(), *GetPrettyPlaybackInstanceInfo(Instance.Get()));
					}
				}
			}
			else
			{
				Instance->Recycle();
			}
		}
	}

	if (EnumHasAnyFlags(InTransitionFlags, EAvaPlayableTransitionEventFlags::Finished))
	{
		if (PlaybackServer)
		{
			SendFinishTransitionEvent(*PlaybackServer, InTransitionFlags);
		}
		Stop();
	}
}

void UAvaPlaybackServerTransition::SendFinishTransitionEvent(FAvaPlaybackServer& InPlaybackServer, EAvaPlayableTransitionEventFlags InTransitionFlags)
{
	// Transition Finish event must be last, i.e. after all playable instance events.
	InstanceEventEntries.Add({FGuid(), InTransitionFlags});
	
	// This will resend all the playable instance events to ensure they are processed prior to the "finish" event.
	InPlaybackServer.SendPlayableTransitionEventMulti(TransitionId, MoveTemp(InstanceEventEntries), ChannelName, ClientName);
	bIsFinishedSent = true;
}

void UAvaPlaybackServerTransition::OnPlayableCreated(UAvaPlaybackGraph* InPlayback, UAvaPlayable* InPlayable)
{
	if (UAvaPlayableGroup* PlayableGroup = InPlayable->GetPlayableGroup())
	{
		PlayableGroup->RegisterVisibilityConstraint(this);
	}
}

void UAvaPlaybackServerTransition::MakePlayableTransition()
{
	using namespace UE::AvaPlaybackServerTransition::Private;
	using namespace UE::AvaPlayback::Utils;
	
	FAvaPlayableTransitionBuilder TransitionBuilder;

	auto AddInstancesToBuilder = [&TransitionBuilder, this](const TArray<TWeakPtr<FAvaPlaybackInstance>>& InPlaybackInstancesWeak,
		const TCHAR* InCategory, EAvaPlayableTransitionEntryRole InEntryRole, bool bInAllowMultipleAdd)
	{
		using namespace UE::AvaPlaybackServerTransition::Private;
		int32 ArrayIndex = 0;
		for (const TWeakPtr<FAvaPlaybackInstance>& InstanceWeak : InPlaybackInstancesWeak)
		{
			if (const TSharedPtr<FAvaPlaybackInstance> Instance = InstanceWeak.Pin())
			{
				if (UAvaPlayable* Playable = GetPlayable(Instance.Get()))
				{
					const bool bPlayableAdded = TransitionBuilder.AddPlayable(Playable, InEntryRole, bInAllowMultipleAdd);
					if (InEntryRole == EAvaPlayableTransitionEntryRole::Enter && bPlayableAdded)
					{
						TransitionBuilder.AddEnterPlayableValues(EnterValues.IsValidIndex(ArrayIndex) ? EnterValues[ArrayIndex] : nullptr);	
					}
				}
				else
				{
					// If this happens, likely the playable is not yet loaded.
					UE_LOGF(LogAvaPlaybackServer, Error,
						"%ls Playback Transition {%ls} Error: Failed to retrieve \"%ls\" playable for instance {%ls}.",
						*GetBriefFrameInfo(), *GetPrettyTransitionInfo(), InCategory, *GetPrettyPlaybackInstanceInfo(Instance.Get()));
				}
			}
			++ArrayIndex;
		}
	};

	constexpr bool bAllowMultipleAddEnter = false;
	const bool bAllowMultipleAddPlaying = EnumHasAnyFlags(TransitionFlags, EAvaPlayableTransitionFlags::HasReusedPlayables);
	constexpr bool bAllowMultipleAddExit = false;
 
	AddInstancesToBuilder(EnterPlaybackInstancesWeak, TEXT("Enter"), EAvaPlayableTransitionEntryRole::Enter, bAllowMultipleAddEnter);
	AddInstancesToBuilder(PlayingPlaybackInstancesWeak, TEXT("Playing"), EAvaPlayableTransitionEntryRole::Playing,  bAllowMultipleAddPlaying);
	AddInstancesToBuilder(ExitPlaybackInstancesWeak, TEXT("Exit"), EAvaPlayableTransitionEntryRole::Exit,  bAllowMultipleAddExit);
	PlayableTransition = TransitionBuilder.MakeTransition(this, TransitionId);

	if (PlayableTransition)
	{
		PlayableTransition->SetTransitionFlags(TransitionFlags);
	}
}

void UAvaPlaybackServerTransition::LogDetailedTransitionInfo() const
{
	using namespace UE::AvaPlayback::Utils;
	UE_LOGF(LogAvaPlaybackServer, Verbose, "%ls Playback Transition {%ls}:", *GetBriefFrameInfo(), *GetPrettyTransitionInfo());

	auto LogInstances = [](const TArray<TWeakPtr<FAvaPlaybackInstance>>& InPlaybackInstancesWeak, const TCHAR* InCategory)
	{
		using namespace UE::AvaPlaybackServerTransition::Private;

		for (const TWeakPtr<FAvaPlaybackInstance>& InstanceWeak : InPlaybackInstancesWeak)
		{
			if (const TSharedPtr<FAvaPlaybackInstance> Instance = InstanceWeak.Pin())
			{
				UE_LOGF(LogAvaPlaybackServer, Verbose, "- %ls Instance: {%ls}.", InCategory, *GetPrettyPlaybackInstanceInfo(Instance.Get()));
			}
		}
	};

	LogInstances(EnterPlaybackInstancesWeak, TEXT("Enter"));
	LogInstances(PlayingPlaybackInstancesWeak, TEXT("Playing"));
	LogInstances(ExitPlaybackInstancesWeak, TEXT("Exit"));
}

void UAvaPlaybackServerTransition::RegisterToPlayableTransitionEvent()
{
	UAvaPlayable::OnTransitionEvent().RemoveAll(this);
	UAvaPlayable::OnTransitionEvent().AddUObject(this, &UAvaPlaybackServerTransition::OnTransitionEvent);
}

void UAvaPlaybackServerTransition::UnregisterFromPlayableTransitionEvent() const
{
	UAvaPlayable::OnTransitionEvent().RemoveAll(this);
}

bool UAvaPlaybackServerTransition::AddPlaybackInstance(const TSharedPtr<FAvaPlaybackInstance>& InPlaybackInstance, TArray<TWeakPtr<FAvaPlaybackInstance>>& OutPlaybackInstancesWeak)
{
	if (!InPlaybackInstance)
	{
		return false;	
	}

	OutPlaybackInstancesWeak.Add(InPlaybackInstance);
	UpdateChannelName(InPlaybackInstance.Get());
	return true;
}

void UAvaPlaybackServerTransition::UpdateChannelName(const FAvaPlaybackInstance* InPlaybackInstance)
{
	if (ChannelName.IsNone())
	{
		ChannelName = InPlaybackInstance->GetChannelFName();
	}
	else
	{
		using namespace UE::AvaPlaybackServerTransition::Private;
		using namespace UE::AvaPlayback::Utils;

		// Validate the channel is the same.
		if (ChannelName != InPlaybackInstance->GetChannelFName())
		{
			UE_LOGF(LogAvaPlaybackServer, Error,
				"%ls Playback Transition {%ls}: Adding Playback Instance {%ls} in a different channel than previous playback instance (\"%ls\").",
				*GetBriefFrameInfo(), *GetPrettyTransitionInfo(), *GetPrettyPlaybackInstanceInfo(InPlaybackInstance), *ChannelName.ToString());
		}
	}
}