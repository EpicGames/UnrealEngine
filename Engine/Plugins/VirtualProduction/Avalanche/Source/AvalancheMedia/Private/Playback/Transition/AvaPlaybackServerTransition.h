// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playable/AvaPlayable.h"
#include "Playback/AvaPlaybackMessages.h"
#include "Playback/Transition/AvaPlaybackTransition.h"
#include "AvaPlaybackServerTransition.generated.h"

class FAvaPlaybackInstance;
class FAvaPlaybackServer;
class UAvaPlayableTransition;
class UAvaPlaybackGraph;

namespace UE::AvaPlayback::Utils
{
	class FAsyncAssetLoader;
}

/**
 * Class for creating and tracking playback graph instance transitions on the server.
 * It is responsible for creating the playable transition object when requested from
 * the playback graphs.
 *
 * This class handles each playback graph instance as a single playable. It is
 * meant to be used by the playback server primarily.
 */
UCLASS()
class UAvaPlaybackServerTransition : public UAvaPlaybackTransition
{
	GENERATED_BODY()
	
public:
	static UAvaPlaybackServerTransition* MakeNew(const TSharedPtr<FAvaPlaybackServer>& InPlaybackServer);
	
	void SetChannelName(const FName& InChannelName) { ChannelName = InChannelName; }
	void SetTransitionId(const FGuid& InTransitionId) { TransitionId = InTransitionId; }
	void SetClientName(const FString& InClientName) { ClientName = InClientName; }
	void SetUnloadDiscardedInstances(bool bInUnloadDiscardedInstances) { bUnloadDiscardedInstances = bInUnloadDiscardedInstances; }
	void SetTransitionFlags(EAvaPlayableTransitionFlags InTransitionFlags) { TransitionFlags = InTransitionFlags; }
	void AddPendingEnterInstanceIds(const TArray<FGuid>& InInstanceIds);
	void AddPendingPlayingInstanceId(const FGuid& InInstanceId);
	void AddPendingExitInstanceId(const FGuid& InInstanceId);
	void SetEnterValues(const TArray<FAvaPlayableRemoteControlValues>& InEnterValues);
	bool AddEnterInstance(const TSharedPtr<FAvaPlaybackInstance>& InPlaybackInstance);
	bool AddPlayingInstance(const TSharedPtr<FAvaPlaybackInstance>& InPlaybackInstance);
	bool AddExitInstance(const TSharedPtr<FAvaPlaybackInstance>& InPlaybackInstance);
	
	void TryResolveInstances(const FAvaPlaybackServer& InPlaybackServer);

	bool ContainsInstance(const FGuid& InInstanceId) const;
	
	//~ Begin IAvaPlayableVisibilityConstraint
	virtual bool IsVisibilityConstrained(const UAvaPlayable* InPlayable) const override;
	//~ End IAvaPlayableVisibilityConstraint
	
	//~ Begin UAvaPlaybackTransition
	virtual bool CanStart(bool& bOutShouldDiscard) override;
	virtual void Start() override;
	virtual void Stop() override;
	virtual bool IsRunning() const override;
	//~ End UAvaPlaybackTransition

	/** Returns the channel this transition is happening in. A transition can only have instances within the same channel. */
	FName GetChannelName() const { return ChannelName; }

	FString GetPrettyTransitionInfo() const;
	FString GetBriefTransitionDescription() const;
	
protected:
	/** Implementation of the Start function intended to be synchronized on cluster. */
	void Start_Synchronized();

	TSharedPtr<FAvaPlaybackInstance> FindInstanceForPlayable(const UAvaPlayable* InPlayable);

	void OnTransitionEvent(UAvaPlayable* InPlayable, UAvaPlayableTransition* InTransition, EAvaPlayableTransitionEventFlags InTransitionFlags);
	void OnPlayableCreated(UAvaPlaybackGraph* InPlayback, UAvaPlayable* InPlayable);
	
	void MakePlayableTransition();

	void LogDetailedTransitionInfo() const;

	void RegisterToPlayableTransitionEvent();
	void UnregisterFromPlayableTransitionEvent() const;

	bool AddPlaybackInstance(const TSharedPtr<FAvaPlaybackInstance>& InPlaybackInstance, TArray<TWeakPtr<FAvaPlaybackInstance>>& OutPlaybackInstancesWeak);	
	void UpdateChannelName(const FAvaPlaybackInstance* InPlaybackInstance);

	void SendFinishTransitionEvent(FAvaPlaybackServer& InPlaybackServer, EAvaPlayableTransitionEventFlags InTransitionFlags);

protected:
	TWeakPtr<FAvaPlaybackServer> PlaybackServerWeak;
	
	FString ClientName;
	FName ChannelName;
	bool bUnloadDiscardedInstances = false;
	EAvaPlayableTransitionFlags TransitionFlags = EAvaPlayableTransitionFlags::None;

	/** Instance Ids pending resolve. */
	TArray<FGuid> PendingEnterInstanceIds;
	TArray<FGuid> PendingPlayingInstanceIds;
	TArray<FGuid> PendingExitInstanceIds;

	TArray<TWeakPtr<FAvaPlaybackInstance>> EnterPlaybackInstancesWeak;
	TArray<TWeakPtr<FAvaPlaybackInstance>> PlayingPlaybackInstancesWeak;
	TArray<TWeakPtr<FAvaPlaybackInstance>> ExitPlaybackInstancesWeak;
	TArray<TSharedPtr<FAvaPlayableRemoteControlValues>> EnterValues;

	TSharedPtr<UE::AvaPlayback::Utils::FAsyncAssetLoader> AsyncAssetLoader;

	UPROPERTY(Transient)
	TObjectPtr<UAvaPlayableTransition> PlayableTransition;

	/** Keep track of all instance event entries so we can all send them in a single message to ensure order. */
	TArray<FAvaPlaybackTransitionEventEntry> InstanceEventEntries;

	/** Re-entry guard for stop() function. */
	bool bIsStopping = false;

	/** Indicate if the "finish" event has been sent. There is 2 possible paths to send the finish event, but we only want to send once. */
	bool bIsFinishedSent = false;
};