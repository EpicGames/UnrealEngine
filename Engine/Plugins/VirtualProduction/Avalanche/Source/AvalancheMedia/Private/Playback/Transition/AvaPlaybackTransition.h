// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playable/IAvaPlayableVisibilityConstraint.h"
#include "UObject/Object.h"
#include "AvaPlaybackTransition.generated.h"

/**
 * Abstract base class for playback transitions that can be queued in
 * the playback manager's commands.
 */
UCLASS()
class UAvaPlaybackTransition : public UObject, public IAvaPlayableVisibilityConstraint
{
	GENERATED_BODY()
	
public:
	/**
	 * @brief Evaluate the status of loading playables to determine if the transition can start.
	 * @param bOutShouldDiscard indicate if the pending start transition command should be discarded.
	 * @return true if the transition can start, false otherwise.
	 */
	virtual bool CanStart(bool& bOutShouldDiscard) { bOutShouldDiscard = true; return false; }

	/** 
	* Start the transition.
	* @remark Transition start is synchronized on clusters, but it is implemented by the derived classes.
	 */
	virtual void Start() {}

	/** Stop transition */
	virtual void Stop() {}

	/** Returns true if the transition is running, false otherwise. */
	virtual bool IsRunning() const { return false; }
	
	//~ Begin IAvaPlayableVisibilityConstraint
	virtual bool IsVisibilityConstrained(const UAvaPlayable* InPlayable) const override { return false; }
	//~ End IAvaPlayableVisibilityConstraint

	/** Returns the TransitionId, a unique identifier (replicated on server) for this transition. */
	const FGuid& GetTransitionId() const { return TransitionId; }
	
protected:
	FGuid TransitionId;
};