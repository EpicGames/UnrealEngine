// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundown.h"

#include "AvaMediaSettings.h"
#include "Broadcast/AvaBroadcast.h"
#include "IAvaMediaModule.h"
#include "MaterialCache/AvaMaterialCacheHelper.h"
#include "MaterialCache/AvaMaterialCacheSettings.h"
#include "MediaOutput.h"
#include "Playable/AvaPlayable.h"
#include "Playback/AvaPlaybackGraph.h"
#include "Playback/AvaPlaybackManager.h"
#include "Playback/AvaPlaybackUtils.h"
#include "Rundown/AvaRundownManagedInstanceCache.h"
#include "Rundown/AvaRundownPageCommand.h"
#include "Rundown/AvaRundownPageLoadingManager.h"
#include "Rundown/AvaRundownPagePlayer.h"
#include "Rundown/AvaRundownPlaybackClientWatcher.h"
#include "Rundown/Transition/AvaRundownPageTransition.h"
#include "Rundown/Transition/AvaRundownPageTransitionBuilder.h"

#if WITH_EDITOR
#include "Cooker/CookEvents.h"
#include "Editor.h"
#include "EngineAnalytics.h"
#endif

DEFINE_LOG_CATEGORY(LogAvaRundown);

#define LOCTEXT_NAMESPACE "AvaRundown"

void FAvaRundownPageCollection::Empty(UAvaRundown* InRundown, const FAvaRundownPageListReference& InPageListReference)
{
	TArray<int32> PageIds;
	if (InRundown->GetOnPageListChanged().IsBound() && Pages.IsEmpty() == false)
	{
		PageIds.Reserve(Pages.Num());
		for (const FAvaRundownPage& Page : Pages)
		{
			PageIds.Add(Page.GetPageId());
		}
	}
	
	Pages.Empty();
	PageIndices.Empty();
	
	if (PageIds.IsEmpty() == false)
	{
		InRundown->GetOnPageListChanged().Broadcast({InRundown, InPageListReference, EAvaRundownPageListChange::RemovedPages, PageIds});
	}
}

namespace UE::AvaMedia::Rundown::Private
{
	FString ToString(const FAvaRundownPageListReference& InPageListReference)
	{
		return FString::Printf(TEXT("{ Type: %s, Id: %s"),
			*StaticEnum<EAvaRundownPageListType>()->GetNameStringByValue((int64)InPageListReference.Type),
			*InPageListReference.SubListId.ToString());
	}
}

const FAvaRundownPageListReference UAvaRundown::TemplatePageList = {EAvaRundownPageListType::Template, FGuid()};
const FAvaRundownPageListReference UAvaRundown::InstancePageList = {EAvaRundownPageListType::Instance, FGuid()};
const FAvaRundownSubList UAvaRundown::InvalidSubList;
FAvaRundownSubList UAvaRundown::InvalidSubListMutable;

UAvaRundown::UAvaRundown()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
#if WITH_EDITOR
		FEditorDelegates::PrePIEEnded.AddUObject(this, &UAvaRundown::NotifyPIEEnded);
#endif
	}
}

UAvaRundown::~UAvaRundown() = default;

int32 UAvaRundown::GenerateUniquePageId(int32 InReferencePageId, int32 InIncrement) const
{
	if (InIncrement == 0)
	{
		InIncrement = 1;
	}

	// Search space must be zero-positive.
	int32 UniquePageId = FMath::Max(0, InReferencePageId);

	// Search a unique id in the given direction.
	while (!IsPageIdUnique(UniquePageId))
	{
		UniquePageId += InIncrement;
		
		// End of the search space is reached, start in the other direction from initial value.
		if (UniquePageId < 0 && InIncrement < 0)
		{
			return GenerateUniquePageId(InReferencePageId, -InIncrement);
		}
	}

	return UniquePageId;
}

int32 UAvaRundown::GenerateUniquePageId(const FAvaRundownPageIdGeneratorParams& InParams) const
{
	return GenerateUniquePageId(InParams.ReferenceId, InParams.Increment);
}

void UAvaRundown::RefreshPageIndices()
{	
	TemplatePages.RefreshPageIndices();
	InstancedPages.RefreshPageIndices();
}

void UAvaRundown::RefreshSubListIndices()
{
	SubListIndices.Empty(SubLists.Num());
	for (int32 Index = 0; Index < SubLists.Num(); ++Index)
	{
		SubListIndices.Add(SubLists[Index].Id, Index);
	}	
}

namespace UE::AvaMedia::Rundown::Private
{
	void OnPostLoadPages(TArray<FAvaRundownPage>& InPages)
	{
		for (FAvaRundownPage& Page : InPages)
		{
			Page.PostLoad();
		}
	}

	/**
	 * Collect the referenced asset paths from page RC values.
	 */
	void CollectReferencedAssetPaths(const TArray<FAvaRundownPage>& InPages, TSet<FSoftObjectPath>& OutReferencedPaths)
	{
		for (const FAvaRundownPage& Page : InPages)
		{
			// Collect asset package references from values
			const FAvaPlayableRemoteControlValues& Values = Page.GetRemoteControlValues();
			FAvaPlayableRemoteControlValues::CollectReferencedAssetPaths(Values.EntityValues, OutReferencedPaths);
			FAvaPlayableRemoteControlValues::CollectReferencedAssetPaths(Values.ControllerValues, OutReferencedPaths);
		}
	}
}

void UAvaRundown::BeginDestroy()
{
	Super::BeginDestroy();

#if WITH_EDITOR
	FEditorDelegates::PrePIEEnded.RemoveAll(this);
#endif
}

void UAvaRundown::PostLoad()
{
	UObject::PostLoad();

	if (!Pages_DEPRECATED.IsEmpty())
	{
		AddPagesFromTemplates(AddTemplates(Pages_DEPRECATED));
		Pages_DEPRECATED.Empty();
	}

	UE::AvaMedia::Rundown::Private::OnPostLoadPages(TemplatePages.Pages);
	UE::AvaMedia::Rundown::Private::OnPostLoadPages(InstancedPages.Pages);
	
	for (FAvaRundownSubList& SubList : SubLists)
	{
		if (!SubList.Id.IsValid())
		{
			SubList.Id = FGuid::NewGuid();
		}
	}

	RefreshSubListIndices();
	RefreshPageIndices();
}

#if WITH_EDITOR

// Undo backup helper.
class UAvaRundown::FPreUndoBackup
{
public:
	FAvaRundownPageCollection TemplatePages;
	FAvaRundownPageCollection InstancedPages;

	static const FAvaRundownPage& GetPage(int32 InPageId, const FAvaRundownPageCollection& InCollection)
	{
		const int32 PageIndex = InCollection.GetPageIndex(InPageId);
		return InCollection.Pages.IsValidIndex(PageIndex) ? InCollection.Pages[PageIndex] : FAvaRundownPage::NullPage;
	}

	static void NofifyPageValueChanges(UAvaRundown* InRundown, const FAvaRundownPageCollection& InCollection, const FAvaRundownPageCollection& InOtherCollection)
	{
		for (const FAvaRundownPage& Page : InCollection.Pages)
		{
			const FAvaRundownPage& BackupPage = GetPage(Page.GetPageId(), InOtherCollection);
			if (BackupPage.IsValidPage())
			{
				EAvaPlayableRemoteControlChanges ValueChanges = EAvaPlayableRemoteControlChanges::None;
				if (!BackupPage.GetRemoteControlValues().HasSameControllerValues(Page.GetRemoteControlValues()))
				{
					ValueChanges |= EAvaPlayableRemoteControlChanges::ControllerValues;
				}
				if (!BackupPage.GetRemoteControlValues().HasSameEntityValues(Page.GetRemoteControlValues()))
				{
					ValueChanges |= EAvaPlayableRemoteControlChanges::EntityValues;
				}
				if (ValueChanges != EAvaPlayableRemoteControlChanges::None)
				{
					InRundown->NotifyPageRemoteControlValueChanged(Page.GetPageId(), ValueChanges);
				}
			}
		}
	}
};

void UAvaRundown::PreEditUndo()
{
	PreUndoBackup = MakePimpl<FPreUndoBackup>();
	PreUndoBackup->TemplatePages = TemplatePages;
	PreUndoBackup->InstancedPages = InstancedPages;
}

void UAvaRundown::PostEditUndo()
{
	UObject::PostEditUndo();
	
	//Force Refresh for any Undo
	RefreshPageIndices();
	RefreshSubListIndices();
	
	GetOnPageListChanged().Broadcast({this, TemplatePageList, EAvaRundownPageListChange::All, {}});
	GetOnPageListChanged().Broadcast({this, InstancePageList, EAvaRundownPageListChange::All, {}});

	for (const FAvaRundownSubList& SubList : SubLists)
	{
		GetOnPageListChanged().Broadcast({this, CreateSubListReference(SubList), EAvaRundownPageListChange::All, {}});
	}

	GetOnActiveListChanged().Broadcast();

	if (PreUndoBackup)
	{
		FPreUndoBackup::NofifyPageValueChanges(this, TemplatePages, PreUndoBackup->TemplatePages);
		FPreUndoBackup::NofifyPageValueChanges(this, InstancedPages, PreUndoBackup->InstancedPages);
		PreUndoBackup.Reset();
	}
}

void UAvaRundown::OnCookEvent(UE::Cook::ECookEvent InCookEvent, UE::Cook::FCookEventContext& InCookContext)
{
	Super::OnCookEvent(InCookEvent, InCookContext);

	if (InCookEvent == UE::Cook::ECookEvent::PlatformCookDependencies)
	{
		TSet<FSoftObjectPath> ReferencedPaths;
		UE::AvaMedia::Rundown::Private::CollectReferencedAssetPaths(TemplatePages.Pages, ReferencedPaths);
		UE::AvaMedia::Rundown::Private::CollectReferencedAssetPaths(InstancedPages.Pages, ReferencedPaths);

		TSet<FName> ReferencedPackages;
		Algo::Transform(ReferencedPaths, ReferencedPackages, [](const FSoftObjectPath& InPath){ return InPath.GetLongPackageFName(); });
		
		// Let the referenced packages be picked up by StormSync.
		for (const FName PackageName : ReferencedPackages)
		{
			InCookContext.AddLoadBuildDependency(UE::Cook::FCookDependency::Package(PackageName));
		}

		if (InCookContext.IsCooking())
		{
			// Seems to be necessary for references to be picked up when cooking.
			for (const FSoftObjectPath& ObjectPath : ReferencedPaths)
			{
				InCookContext.AddRuntimeDependency(ObjectPath);
			}
		}
	}
}

#endif

bool UAvaRundown::IsEmpty() const
{
	return TemplatePages.Pages.IsEmpty() && InstancedPages.Pages.IsEmpty();
}

bool UAvaRundown::Empty()
{
	if (IsPlaying())
	{
		return false;
	}

	// Since we are about to delete the sublists,
	// make sure we return the active page list to something not deleted.
	if (HasActiveSubList())
	{
		SetActivePageList(InstancePageList);
	}
	
	SubLists.Empty();
	InstancedPages.Empty(this, InstancePageList);
	TemplatePages.Empty(this, TemplatePageList);
	
	return true;
}

int32 UAvaRundown::AddTemplateInternal(const FAvaRundownPageIdGeneratorParams& InIdGeneratorParams, const TFunctionRef<bool(FAvaRundownPage&)> InSetupTemplateFunction)
{
	if (!CanAddPage())
	{
		return FAvaRundownPage::InvalidPageId;
	}
	const int32 TemplateId = GenerateUniquePageId(InIdGeneratorParams);
	FAvaRundownPage NewTemplate(TemplateId);
	
	if (!InSetupTemplateFunction(NewTemplate))
	{
		return FAvaRundownPage::InvalidPageId;
	}
	
	TemplatePages.Pages.Emplace(MoveTemp(NewTemplate));
	
	RefreshPageIndices();

	GetOnPageListChanged().Broadcast({this, TemplatePageList, EAvaRundownPageListChange::AddedPages, {TemplateId}});

	return TemplateId;
}

int32 UAvaRundown::AddTemplate(const FAvaRundownPageIdGeneratorParams& InIdGeneratorParams)
{
	return AddTemplateInternal(InIdGeneratorParams, [](FAvaRundownPage&){return true;});
}

int32 UAvaRundown::AddComboTemplate(const TArray<int32>& InTemplateIds, const FAvaRundownPageIdGeneratorParams& InIdGeneratorParams)
{
	return AddTemplateInternal(InIdGeneratorParams, [&InTemplateIds](FAvaRundownPage& InNewTemplate)
	{
		InNewTemplate.CombinedTemplateIds = InTemplateIds;
		return true;
	});
}

TArray<int32> UAvaRundown::ValidateTemplateIdsForComboTemplate(const TArray<int32>& InTemplateIds, FOutputDevice& InErrorLog)
{
	TSet<FAvaTagId> LayerIds;
	TArray<int32> TemplateIds;
	TemplateIds.Reserve(InTemplateIds.Num());
	FAvaPlayableRemoteControlValues MergedValues;
	
	for (const int32 SelectedPageId : InTemplateIds)
	{
		const FAvaRundownPage& Page = GetPage(SelectedPageId);
		if (!Page.IsValidPage())
		{
			InErrorLog.Logf(TEXT("Template %d is not valid."), SelectedPageId);
			continue;
		}
		if (!Page.IsTemplate())
		{
			InErrorLog.Logf(TEXT("Page %d is not a template."), SelectedPageId);
			continue;
		}
		if (Page.IsComboTemplate())
		{
			InErrorLog.Logf(TEXT("Template %d is already a combo template."), SelectedPageId);
			continue;
		}
		if (!Page.HasTransitionLogic(this))
		{
			InErrorLog.Logf(TEXT("Template %d doesn't have transition logic."), SelectedPageId);
			continue;
		}
		if (!Page.GetTransitionLayer(this).IsValid())
		{
			InErrorLog.Logf(TEXT("Template %d doesn't have a valid transition logic layer."), SelectedPageId);
			continue;
		}
		if (LayerIds.Contains(Page.GetTransitionLayer(this).TagId))
		{
			InErrorLog.Logf(TEXT("Template %d's layer %s is already in the selection."), SelectedPageId, *Page.GetTransitionLayer(this).ToString());
			continue;
		}

		// Make sure the RC values can merge correctly. If not, original template needs fixing.
		if (MergedValues.HasIdCollisions(Page.GetRemoteControlValues()))
		{
			InErrorLog.Logf(TEXT("Template %d's RemoteControl values have Id collisions with other templates in the selection."), SelectedPageId);
			continue;
		}
		
		MergedValues.Merge(Page.GetRemoteControlValues());
		LayerIds.Add(Page.GetTransitionLayer(this).TagId);
		TemplateIds.Add(Page.GetPageId());
	}
	return TemplateIds;
}

TArray<int32> UAvaRundown::AddTemplates(const TArray<FAvaRundownPage>& InSourceTemplates)
{
	if (!CanAddPage() || InSourceTemplates.IsEmpty())
	{
		return {};
	}

	TArray<int32> OutTemplateIds;
	OutTemplateIds.Reserve(InSourceTemplates.Num());

	for (const FAvaRundownPage& SourceTemplate : InSourceTemplates)
	{
		// Try to preserve the template id from the source.
		int32 NewTemplateId = GenerateUniquePageId(SourceTemplate.GetPageId());

		// Add to template list.
		const int32 Index = TemplatePages.Pages.Add(SourceTemplate);
		TemplatePages.Pages[Index].PageId = NewTemplateId;
		TemplatePages.Pages[Index].TemplateId = FAvaRundownPage::InvalidPageId;
		TemplatePages.PageIndices.Emplace(NewTemplateId, Index);
		
		OutTemplateIds.Add(NewTemplateId);
	}

	if (!OutTemplateIds.IsEmpty())
	{
		GetOnPageListChanged().Broadcast({this, TemplatePageList, EAvaRundownPageListChange::AddedPages, OutTemplateIds});
	}

	return OutTemplateIds;
}

TArray<int32> UAvaRundown::AddPagesFromTemplates(const TArray<int32>& InTemplateIds)
{
	TArray<int32> OutPageIds;
	OutPageIds.Reserve(InTemplateIds.Num());
	
	FAvaRundownPageIdGeneratorParams IdGeneratorParams;

	// Special id generation case: start from the last page id.
	{
		int32 LastInstancedPageId = 0;
		for (const FAvaRundownPage& Page : InstancedPages.Pages)
		{
			LastInstancedPageId = FMath::Max(LastInstancedPageId, Page.GetPageId());
		}
		IdGeneratorParams.ReferenceId = LastInstancedPageId;
	}

	for (const int32& TemplateId : InTemplateIds)
	{
		const int32 NewId = AddPageFromTemplateInternal(TemplateId, IdGeneratorParams);
		if (NewId != FAvaRundownPage::InvalidPageId)
		{
			OutPageIds.Add(NewId);
			IdGeneratorParams.ReferenceId = NewId;
		}
	}

	RefreshPageIndices();
	GetOnPageListChanged().Broadcast({this, InstancePageList, EAvaRundownPageListChange::AddedPages, OutPageIds});

	return OutPageIds;
}

int32 UAvaRundown::AddPageFromTemplate(int32 InTemplateId, const FAvaRundownPageIdGeneratorParams& InIdGeneratorParams, const FAvaRundownPageInsertPosition& InInsertAt)
{
	const int32 NewId = AddPageFromTemplateInternal(InTemplateId, InIdGeneratorParams, InInsertAt);
	
	if (NewId != FAvaRundownPage::InvalidPageId)
	{
		RefreshPageIndices();
		GetOnPageListChanged().Broadcast({this, InstancePageList, EAvaRundownPageListChange::AddedPages, {NewId}});
	}

	return NewId;
}

bool UAvaRundown::CanAddPage() const
{
	// Pages can always be added. This is needed for live editing rundowns.
	// Because of this, having pointers to pages is risky.
	// Pages should always be referred to by page id and de-referenced only when needed.
	return true;
}

bool UAvaRundown::CanChangePageOrder() const
{
	return !IsPlaying();
}

bool UAvaRundown::ChangePageOrder(const FAvaRundownPageListReference& InPageListReference, const TArray<int32>& InPageIndices)
{
	TSet<int32> MovedIndices;

	// Templates & Instances
	if (InPageListReference.Type != EAvaRundownPageListType::View)
	{
		FAvaRundownPageCollection& Collection = InPageListReference.Type == EAvaRundownPageListType::Template ? TemplatePages : InstancedPages;
		TArray<FAvaRundownPage> NewPages;
		NewPages.Reserve(Collection.Pages.Num());

		for (int32 PageIndex : InPageIndices)
		{
			NewPages.Add(MoveTemp(Collection.Pages[PageIndex]));
			MovedIndices.Add(PageIndex);
		}

		// Make sure all pages were moved.
		for (int32 PageIndex = 0; PageIndex < Collection.Pages.Num(); ++PageIndex)
		{
			if (!MovedIndices.Contains(PageIndex))
			{
				NewPages.Add(MoveTemp(Collection.Pages[PageIndex]));
			}
		}

		Collection.Pages = NewPages;
		Collection.PageIndices.Empty();
		RefreshPageIndices();

		GetOnPageListChanged().Broadcast({this, InPageListReference, EAvaRundownPageListChange::ReorderedPageView, {}});

		return true;
	}

	if (IsValidSubList(InPageListReference))
	{
		FAvaRundownSubList& SubList = GetSubList(InPageListReference);
		TArray<int32> NewIndices;
		NewIndices.Reserve(SubList.PageIds.Num());

		for (int32 PageIndex : InPageIndices)
		{
			NewIndices.Add(SubList.PageIds[PageIndex]);
			MovedIndices.Add(PageIndex);
		}

		// Make sure all pages were moved.
		for (int32 PageIndex = 0; PageIndex < SubList.PageIds.Num(); ++PageIndex)
		{
			if (!MovedIndices.Contains(PageIndex))
			{
				NewIndices.Add(SubList.PageIds[PageIndex]);
			}
		}

		SubList.PageIds = NewIndices;
		GetOnPageListChanged().Broadcast({this, InPageListReference, EAvaRundownPageListChange::ReorderedPageView, {}});

		return true;
	}

	return false;
}

bool UAvaRundown::RemovePage(int32 InPageId)
{
	return RemovePages({InPageId}) > 0;
}

bool UAvaRundown::CanRemovePage(int32 InPageId) const
{
	return CanRemovePages({InPageId});
}

int32 UAvaRundown::RemovePages(const TArray<int32>& InPageIds)
{
	if (!CanRemovePages(InPageIds))
	{
		return 0;
	}

	// Find the instanced page ids to remove
	TArray<int32> TemplatesIndicesToRemove;
	TSet<int32> InstancesToRemove;

	for (int32 PageId : InPageIds)
	{
		const int32* TemplateIdx = TemplatePages.PageIndices.Find(PageId);
		const int32* InstanceIdx = InstancedPages.PageIndices.Find(PageId);

		if (TemplateIdx)
		{
			if (TemplatePages.Pages[*TemplateIdx].Instances.IsEmpty() == false)
			{
				InstancesToRemove.Append(TemplatePages.Pages[*TemplateIdx].Instances);
			}

			// Double check, just in case.
			for (const FAvaRundownPage& Page : InstancedPages.Pages)
			{
				if (Page.TemplateId == PageId)
				{
					InstancesToRemove.Add(Page.GetPageId());
				}
			}

			TemplatesIndicesToRemove.Add(*TemplateIdx);
			TemplatePages.PageIndices.Remove(PageId);
		}
		else if (InstanceIdx)
		{
			InstancesToRemove.Add(PageId);

			const FAvaRundownPage& Instance = InstancedPages.Pages[*InstanceIdx];

			if (const int32* InstanceTemplateIdx = TemplatePages.PageIndices.Find(Instance.TemplateId))
			{
				TemplatePages.Pages[*InstanceTemplateIdx].Instances.Remove(PageId);
			}
		}
	}

	TArray<int32> RemovedTemplateIds;

	if (TemplatesIndicesToRemove.IsEmpty() == false)
	{
		TemplatesIndicesToRemove.Sort();
		RemovedTemplateIds.Reserve(TemplatesIndicesToRemove.Num());

		for (int32 TemplateIdx = TemplatesIndicesToRemove.Num() - 1; TemplateIdx >= 0; --TemplateIdx)
		{
			RemovedTemplateIds.Add(TemplatePages.Pages[TemplatesIndicesToRemove[TemplateIdx]].PageId);
			TemplatePages.Pages.RemoveAt(TemplatesIndicesToRemove[TemplateIdx]);
		}
	}

	if (InstancesToRemove.IsEmpty() == false)
	{
		// Process list from highest to lowest so we don't change future indices while removing.
		TArray<int32> InstancesToRemoveIndices;
		InstancesToRemoveIndices.Reserve(InstancesToRemove.Num());

		for (int32 InstanceToRemove : InstancesToRemove)
		{
			if (int32* InstanceIndexPtr = InstancedPages.PageIndices.Find(InstanceToRemove))
			{
				InstancesToRemoveIndices.Add(*InstanceIndexPtr);
				InstancedPages.PageIndices.Remove(InstanceToRemove);
			}
		}

		InstancesToRemoveIndices.Sort();

		for (int32 RemoveIdx = InstancesToRemoveIndices.Num() - 1; RemoveIdx >= 0; --RemoveIdx)
		{
			const int32 InstanceToRemoveIdx = InstancesToRemoveIndices[RemoveIdx];
			InstancedPages.Pages.RemoveAt(InstanceToRemoveIdx);
		}
	}
		
	if (RemovedTemplateIds.IsEmpty() && InstancesToRemove.IsEmpty())
	{
		return 0;
	}

	RefreshPageIndices();

	if (!RemovedTemplateIds.IsEmpty())
	{
		GetOnPageListChanged().Broadcast({this, TemplatePageList, EAvaRundownPageListChange::RemovedPages, RemovedTemplateIds});
	}

	if (!InstancesToRemove.IsEmpty())
	{
		GetOnPageListChanged().Broadcast({this, InstancePageList, EAvaRundownPageListChange::RemovedPages, InstancesToRemove.Array()});
	}

	for (FAvaRundownSubList& SubList : SubLists)
	{
		TArray<int32> RemovedInstanceIds;
		RemovedInstanceIds.Reserve(InstancesToRemove.Num());

		for (TArray<int32>::TIterator Iter(SubList.PageIds); Iter; ++Iter)
		{
			if (InstancesToRemove.Contains(*Iter))
			{
				RemovedInstanceIds.Add(*Iter);
				Iter.RemoveCurrent();
			}
		}

		if (!RemovedInstanceIds.IsEmpty())
		{
			GetOnPageListChanged().Broadcast({this, CreateSubListReference(SubList), EAvaRundownPageListChange::RemovedPages, RemovedInstanceIds});
		}
	}

	return RemovedTemplateIds.Num() + InstancesToRemove.Num();	// Total Pages removed.
}

bool UAvaRundown::CanRemovePages(const TArray<int32>& InPageIds) const
{
	for (int32 PageId : InPageIds)
	{
		if (IsPagePlayingOrPreviewing(PageId))
		{
			return false;
		}

		// Prevent deletion of templates that have playing page instances.
		const FAvaRundownPage& Page = GetPage(PageId);
		if (Page.IsValidPage() && Page.IsTemplate())
		{
			for (const int32 InstancePageId : Page.GetInstancedIds())
			{
				if (IsPagePlayingOrPreviewing(InstancePageId))
				{
					return false;
				}
			}
		}
	}
	return InPageIds.Num() > 0;
}

bool UAvaRundown::RenumberPageIds(const TArray<int32>& InPageIds, const FAvaRundownPageIdGeneratorParams& InIdParams)
{
	int32 CurrentId = InIdParams.ReferenceId;

	for (const int32 PageId : InPageIds)
	{
		const int32 NewId = GenerateUniquePageId(CurrentId, InIdParams.Increment);

		// If a page re-number fails, ignore it and continue on
		RenumberPageId(PageId, NewId);
		
		CurrentId += InIdParams.Increment;
	}

	return true;
}

bool UAvaRundown::RenumberPageId(int32 InPageId, int32 InNewPageId)
{
	if (!CanRenumberPageId(InPageId, InNewPageId))
	{
		return false;
	}

	FAvaRundownPage& Page = GetPage(InPageId);
	check(Page.IsValidPage());

	Page.PageId = InNewPageId;
	
	const int32* TemplateIdx = TemplatePages.PageIndices.Find(InPageId);
	const int32* InstanceIdx = InstancedPages.PageIndices.Find(InPageId);

	if (TemplateIdx)
	{
		bool bFoundInstanceOfTemplate = false;

		for (const int32& InstancePageId : TemplatePages.Pages[*TemplateIdx].Instances)
		{
			if (const int32* InstancePageIdx = InstancedPages.PageIndices.Find(InstancePageId))
			{
				InstancedPages.Pages[*InstancePageIdx].TemplateId = InNewPageId;
				bFoundInstanceOfTemplate = true;
			}
		}

		// Double check, just in case.
		for (FAvaRundownPage& InstancedPage : InstancedPages.Pages)
		{
			if (InstancedPage.TemplateId == InPageId)
			{
				InstancedPage.TemplateId = InNewPageId;
				bFoundInstanceOfTemplate = true;

				// Has become desynced somehow so add it to the template page instance set.
				TemplatePages.Pages[*TemplateIdx].Instances.Add(InstancedPage.GetPageId());
			}
		}

		GetOnPageListChanged().Broadcast({this, TemplatePageList, EAvaRundownPageListChange::RenumberedPageId, {InNewPageId}});

		if (bFoundInstanceOfTemplate)
		{
			GetOnPageListChanged().Broadcast({this, InstancePageList, EAvaRundownPageListChange::RenumberedPageId, {InNewPageId}});
		}
	}
	else if (InstanceIdx)
	{
		GetOnPageListChanged().Broadcast({this, InstancePageList, EAvaRundownPageListChange::RenumberedPageId, {InNewPageId}});

		for (FAvaRundownSubList& SubList : SubLists)
		{
			const int32 Index = SubList.PageIds.Find(InPageId);

			if (Index != INDEX_NONE)
			{
				SubList.PageIds[Index] = InNewPageId;
				GetOnPageListChanged().Broadcast({this, CreateSubListReference(SubList), EAvaRundownPageListChange::RenumberedPageId, {InNewPageId}});
			}
		}
	}
	else
	{
		return false;
	}

	RefreshPageIndices();

	return true;
}

bool UAvaRundown::CanRenumberPageId(int32 InPageId) const
{
	//There must be a valid Page that we will be renumbering
	const bool bPageIdValid = GetPage(InPageId).IsValidPage();

	return !IsPagePlayingOrPreviewing(InPageId)
		&& bPageIdValid;
}

bool UAvaRundown::CanRenumberPageId(int32 InPageId, int32 InNewPageId) const
{
	//There must be a valid Page that we will be renumbering
	const bool bPageIdValid = GetPage(InPageId).IsValidPage();
	
	//Make sure that if we get a Page with New Page Id, it returns a Null Page
	const bool bNewPageIdAvailable = !GetPage(InNewPageId).IsValidPage();

	return !IsPagePlayingOrPreviewing(InPageId)
		&& InPageId != InNewPageId
		&& bPageIdValid
		&& bNewPageIdAvailable;
}

bool UAvaRundown::SetRemoteControlEntityValue(int32 InPageId, const FGuid& InId, const FAvaPlayableRemoteControlValue& InValue)
{
	FAvaRundownPage& Page = GetPage(InPageId);
	if (Page.IsValidPage())
	{
		Page.SetRemoteControlEntityValue(InId, InValue);
		NotifyPageRemoteControlValueChanged(InPageId, EAvaPlayableRemoteControlChanges::EntityValues);
		return true;
	}
	return false;
}

bool UAvaRundown::SetRemoteControlControllerValue(int32 InPageId, const FGuid& InId, const FAvaPlayableRemoteControlValue& InValue)
{
	FAvaRundownPage& Page = GetPage(InPageId);
	if (Page.IsValidPage())
	{
		Page.SetRemoteControlControllerValue(InId, InValue);
		NotifyPageRemoteControlValueChanged(InPageId, EAvaPlayableRemoteControlChanges::ControllerValues);
		return true;
	}
	return false;
}

EAvaPlayableRemoteControlChanges UAvaRundown::UpdateRemoteControlValues(int32 InPageId, const FAvaPlayableRemoteControlValues& InRemoteControlValues, bool bInUpdateDefaults)
{
	FAvaRundownPage& Page = GetPage(InPageId);
	if (Page.IsValidPage())
	{
		const EAvaPlayableRemoteControlChanges Changes = Page.UpdateRemoteControlValues(InRemoteControlValues, bInUpdateDefaults);
		if (Changes != EAvaPlayableRemoteControlChanges::None)
		{
			NotifyPageRemoteControlValueChanged(InPageId, Changes);
		}
		return Changes;
	}
	return EAvaPlayableRemoteControlChanges::None;
}

EAvaPlayableRemoteControlChanges UAvaRundown::ResetRemoteControlValues(int32 InPageId, bool bInUseTemplateValues, bool bInIsDefault)
{
	FAvaRundownPage& Page = GetPage(InPageId);
	if (Page.IsValidPage())
	{
		EAvaPlayableRemoteControlChanges Changes = Page.ResetRemoteControlValues(this, bInUseTemplateValues, bInIsDefault);
		if (Changes != EAvaPlayableRemoteControlChanges::None)
		{
			NotifyPageRemoteControlValueChanged(InPageId, Changes);
		}
		return Changes;
	}
	return EAvaPlayableRemoteControlChanges::None;
}


EAvaPlayableRemoteControlChanges UAvaRundown::ResetRemoteControlControllerValue(int32 InPageId, const FGuid& InControllerId, bool bInUseTemplateValues, bool bInIsDefault)
{
	FAvaRundownPage& Page = GetPage(InPageId);
	if (Page.IsValidPage())
	{
		EAvaPlayableRemoteControlChanges Changes = Page.ResetRemoteControlControllerValue(this, InControllerId, bInUseTemplateValues, bInIsDefault);
		if (Changes != EAvaPlayableRemoteControlChanges::None)
		{
			NotifyPageRemoteControlValueChanged(InPageId, Changes);
		}
		return Changes;
	}
	return EAvaPlayableRemoteControlChanges::None;
}

EAvaPlayableRemoteControlChanges UAvaRundown::ResetRemoteControlEntityValue(int32 InPageId, const FGuid& InEntityId, bool bInUseTemplateValues, bool bInIsDefault)
{
	FAvaRundownPage& Page = GetPage(InPageId);
	if (Page.IsValidPage())
	{
		EAvaPlayableRemoteControlChanges Changes = Page.ResetRemoteControlEntityValue(this, InEntityId, bInUseTemplateValues, bInIsDefault);
		if (Changes != EAvaPlayableRemoteControlChanges::None)
		{
			NotifyPageRemoteControlValueChanged(InPageId, Changes);
		}
		return Changes;
	}
	return EAvaPlayableRemoteControlChanges::None;
}

void UAvaRundown::InvalidateManagedInstanceCacheForPages(const TArray<int32>& InPageIds) const
{
	if (!IAvaMediaModule::IsModuleLoaded())
	{
		return;
	}
	
	FAvaRundownManagedInstanceCache& ManagedInstanceCache = IAvaMediaModule::Get().GetManagedInstanceCache();
	
	for (const int32 PageId : InPageIds)
	{
		const FAvaRundownPage& Page = GetPage(PageId);

		if (Page.IsValidPage())
		{
			ManagedInstanceCache.InvalidateNoDelete(Page.GetAssetPath(this));
		}
	}

	// Delete all invalidated entries immediately.
	ManagedInstanceCache.FinishPendingActions();
}

void UAvaRundown::UpdateAssetForPages(const TArray<int32>& InPageIds, bool bInReimportPage)
{
	for (const int32 SelectedPageId : InPageIds)
	{
		FAvaRundownPage& Page = GetPage(SelectedPageId);
		if (!Page.IsValidPage())
		{
			UE_LOGF(LogAvaRundown, Error, "Reimport asset failed: page id %d is not valid.", SelectedPageId);
			continue;
		}
		if (!Page.IsTemplate())
		{
			UE_LOGF(LogAvaRundown, Error, "Reimport asset failed: page id %d is not a template.", SelectedPageId);
			continue;
		}				
		Page.UpdateAsset(Page.GetAssetPath(this), bInReimportPage);
	}
}

const FAvaRundownPage& UAvaRundown::GetPage(int32 InPageId) const
{
	if (const int32* TemplateIdx = TemplatePages.PageIndices.Find(InPageId))
	{
		return TemplatePages.Pages[*TemplateIdx];
	}

	if (const int32* InstancedIdx = InstancedPages.PageIndices.Find(InPageId))
	{
		return InstancedPages.Pages[*InstancedIdx];
	}

	return FAvaRundownPage::NullPage;
}

FAvaRundownPage& UAvaRundown::GetPage(int32 InPageId)
{
	if (const int32* TemplateIdx = TemplatePages.PageIndices.Find(InPageId))
	{
		return TemplatePages.Pages[*TemplateIdx];
	}

	if (const int32* InstancedIdx = InstancedPages.PageIndices.Find(InPageId))
	{
		return InstancedPages.Pages[*InstancedIdx];
	}

	return FAvaRundownPage::NullPage;
}

const FAvaRundownPage& UAvaRundown::GetNextPage(int32 InPageId, const FAvaRundownPageListReference& InPageListReference) const
{
	if (const int32* TemplateIdx = TemplatePages.PageIndices.Find(InPageId))
	{
		if (TemplatePages.Pages.IsValidIndex((*TemplateIdx) + 1))
		{
			return TemplatePages.Pages[(*TemplateIdx) + 1];
		}
		else if (TemplatePages.Pages.IsEmpty() == false)
		{
			return TemplatePages.Pages[0];
		}
	}

	if (const int32* InstancedIdx = InstancedPages.PageIndices.Find(InPageId))
	{
		if (IsValidSubList(InPageListReference))
		{
			const FAvaRundownSubList& SubList = GetSubList(InPageListReference);
			const int32 Index = SubList.PageIds.Find(InPageId);

			if (Index != INDEX_NONE)
			{
				return GetNextFromSubList(SubList.PageIds, Index);
			}

			if (SubList.PageIds.IsEmpty())
			{
				return GetNextFromPages(InstancedPages.Pages, (*InstancedIdx));
			}
		}

		if (InPageListReference.Type == EAvaRundownPageListType::Instance)
		{
			return GetNextFromPages(InstancedPages.Pages, (*InstancedIdx));
		}
	}

	return FAvaRundownPage::NullPage;
}

FAvaRundownPage& UAvaRundown::GetNextPage(int32 InPageId, const FAvaRundownPageListReference& InPageListReference)
{
	if (const int32* TemplateIdx = TemplatePages.PageIndices.Find(InPageId))
	{
		if (TemplatePages.Pages.IsValidIndex((*TemplateIdx) + 1))
		{
			return TemplatePages.Pages[(*TemplateIdx) + 1];
		}
		else if (TemplatePages.Pages.IsEmpty() == false)
		{
			return TemplatePages.Pages[0];
		}
	}

	if (const int32* InstancedIdx = InstancedPages.PageIndices.Find(InPageId))
	{
		if (IsValidSubList(InPageListReference))
		{
			const FAvaRundownSubList& SubList = GetSubList(InPageListReference);
			const int32 Index = SubList.PageIds.Find(InPageId);

			if (Index != INDEX_NONE)
			{
				return GetNextFromSubList(SubList.PageIds, Index);
			}

			if (SubList.PageIds.IsEmpty())
			{
				return GetNextFromPages(InstancedPages.Pages, (*InstancedIdx));
			}
		}

		if (InPageListReference.Type == EAvaRundownPageListType::Instance)
		{
			return GetNextFromPages(InstancedPages.Pages, (*InstancedIdx));
		}
	}

	return FAvaRundownPage::NullPage;
}

void UAvaRundown::InitializePlaybackContext()
{
	if (!PlaybackClientWatcher)
	{
		PlaybackClientWatcher = MakePimpl<FAvaRundownPlaybackClientWatcher>(this);
	}
}

bool UAvaRundown::CanClosePlaybackContext() const
{
	bool bResult = true;
	OnCanClosePlaybackContext.Broadcast(this, bResult);
	return bResult;
}

void UAvaRundown::ClosePlaybackContext(bool bInStopAllPages)
{
	if (bInStopAllPages)
	{
		for (UAvaRundownPagePlayer* PagePlayer : PagePlayers)
		{
			if (PagePlayer)
			{
				PagePlayer->Stop();
			}
		}
		RemoveStoppedPagePlayers();
	}
	
	PlaybackClientWatcher.Reset();
}

bool UAvaRundown::IsPlaying() const
{
	return PagePlayers.Num() > 0;
}

bool UAvaRundown::IsPagePreviewing(int32 InPageId) const
{
	return PagePlayers.ContainsByPredicate([InPageId](const UAvaRundownPagePlayer* InPagePlayer)
	{
		return InPagePlayer->bIsPreview && InPagePlayer->PageId == InPageId && InPagePlayer->IsPlaying();
	});
}

bool UAvaRundown::IsPagePlaying(int32 InPageId) const
{
	return PagePlayers.ContainsByPredicate([InPageId](const UAvaRundownPagePlayer* InPagePlayer)
	{
		return !InPagePlayer->bIsPreview && InPagePlayer->PageId == InPageId && InPagePlayer->IsPlaying();
	});
}

bool UAvaRundown::IsPagePlayingOrPreviewing(int32 InPageId) const
{
	return PagePlayers.ContainsByPredicate([InPageId](const UAvaRundownPagePlayer* InPagePlayer)
	{
		return InPagePlayer->PageId == InPageId && InPagePlayer->IsPlaying();
	});
}

bool UAvaRundown::UnloadPage(int32 InPageId, const FString& InChannelName)
{
	FAvaPlaybackManager& Manager = GetPlaybackManager();
	
	if (const FAvaRundownPage& SelectedPage = GetPage(InPageId); SelectedPage.IsValidPage())
	{
		// Ensure all players for this page have stopped.
		for (UAvaRundownPagePlayer* PagePlayer : PagePlayers)
		{
			if (PagePlayer->PageId == InPageId)
			{
				PagePlayer->Stop();
			}
		}
		RemoveStoppedPagePlayers();

		bool bSuccess = false;
		for (const FSoftObjectPath& AssetPath : SelectedPage.GetAssetPaths(this))
		{
			// This will unload all the "available" (i.e. not used) instances of that asset on that channel.
			bSuccess |= Manager.UnloadPlaybackInstances(AssetPath, InChannelName);
		}
		return bSuccess;
	}
	return false;
}

TArray<UAvaRundown::FLoadedInstanceInfo> UAvaRundown::LoadPage(int32 InPageId,  bool bInPreview, const FName& InPreviewChannelName)
{
	const FAvaRundownPage& Page = GetPage(InPageId);
	if (!Page.IsValidPage())
	{
		return {};
	}

	const FName ChannelName = bInPreview ? (InPreviewChannelName.IsNone() ? GetDefaultPreviewChannelName() : InPreviewChannelName) : Page.GetChannelName();

	// Get the Load Options from page command, if any.
	FString LoadOptions;
	FAvaRundownPageCommandContext PageCommandContext = {*this, Page, ChannelName};

	// Built-in Load Options:
	LoadOptions += TEXT(" -CacheShaders");

	Page.ForEachInstancedCommands(
		[&PageCommandContext, &LoadOptions](const FAvaRundownPageCommand& InCommand, const FAvaRundownPage& InPage)
		{
			InCommand.ExecuteOnLoad(PageCommandContext, LoadOptions);
		},
		this, /*bInDirectOnly*/ false); // Traverse templates.

	const TArray<FSoftObjectPath> AssetPaths = Page.GetAssetPaths(this);

	TArray<FLoadedInstanceInfo> LoadedInstances;
	LoadedInstances.Reserve(AssetPaths.Num());

	for (const FSoftObjectPath& AssetPath : AssetPaths)
	{
		const TSharedPtr<FAvaPlaybackInstance> PlaybackInstance = GetPlaybackManager().AcquireOrLoadPlaybackInstance(AssetPath, ChannelName.ToString(), LoadOptions);
		if (!PlaybackInstance || !PlaybackInstance->GetPlayback())
		{
			continue;
		}

		UAvaRundownPagePlayer::SetInstanceUserDataFromPage(*PlaybackInstance, Page);
		if (bInPreview)
		{
			PlaybackInstance->GetPlayback()->SetPreviewChannelName(ChannelName);
		}
		PlaybackInstance->GetPlayback()->LoadInstances();
		PlaybackInstance->UpdateStatus();
		PlaybackInstance->Recycle();
		LoadedInstances.Add({PlaybackInstance->GetInstanceId(), AssetPath});
	}

	// Load RC Assets Async, stream them and request cache its materials
	{
		TSet<FSoftObjectPath> RemoteControlAssets;

		const FAvaPlayableRemoteControlValues& RemoteControlValues = Page.GetRemoteControlValues();
		FAvaPlayableRemoteControlValues::CollectReferencedAssetPaths(RemoteControlValues.ControllerValues, RemoteControlAssets);
		FAvaPlayableRemoteControlValues::CollectReferencedAssetPaths(RemoteControlValues.EntityValues, RemoteControlAssets);

		const FName ShaderProfile = GetDefault<UAvaMaterialCacheSettings>()->GetOfflineProfile();

		for (const FSoftObjectPath& Asset : RemoteControlAssets)
		{
			if (UObject* Object = Asset.ResolveObject())
			{
				UE::AvaPlayback::Utils::StreamRenderAsset(Object);
				UE::AvaPlayback::Utils::CacheObjectMaterials(Object, ShaderProfile);
			}
			else
			{
				Asset.LoadAsync(FLoadSoftObjectPathAsyncDelegate::CreateLambda(
					[ShaderProfile](const FSoftObjectPath&, UObject* InObject)
					{
						if (InObject)
						{
							UE::AvaPlayback::Utils::StreamRenderAsset(InObject);
							UE::AvaPlayback::Utils::CacheObjectMaterials(InObject, ShaderProfile);
						}
					}));
			}
		}
	}

	return LoadedInstances;
}

TArray<int32> UAvaRundown::PlayPages(const TArray<int32>& InPageIds, EAvaRundownPagePlayType InPlayType, EAvaRundownPagePlaySource InSource)
{
	return PlayPages(InPageIds, InPlayType, UE::AvaRundown::IsPreviewPlayType(InPlayType) ? GetDefaultPreviewChannelName() : NAME_None, InSource);
}

TArray<int32> UAvaRundown::PlayPages(const TArray<int32>& InPageIds, EAvaRundownPagePlayType InPlayType, const FName& InPreviewChannelName, EAvaRundownPagePlaySource InSource)
{
	const bool bIsPreview = UE::AvaRundown::IsPreviewPlayType(InPlayType);

	TArray<int32> PlayedPageIds;
	PlayedPageIds.Reserve(InPageIds.Num());

	FAvaRundownPageTransitionBuilder TransitionBuilder(this);
	
	for (const int32 PageId : InPageIds)
	{
		const FAvaRundownPage& SelectedPage = GetPage(PageId);
		if (SelectedPage.IsValidPage() && SelectedPage.IsEnabled())
		{
			FString FailureReason;
			if (!IsChannelTypeCompatibleForRequest(SelectedPage, bIsPreview, InPreviewChannelName, &FailureReason) || !CanPlayPage(PageId, bIsPreview))
			{
				UE_LOGF(LogAvaRundown, Error, "Page Id:%d failed to play: %ls.", PageId, *FailureReason);
				continue;
			}

			if (PlayPageWithTransition(TransitionBuilder, SelectedPage, InPlayType, bIsPreview, InPreviewChannelName))
			{
				GetOrCreatePageListPlaybackContextCollection().GetOrCreateContext(bIsPreview, InPreviewChannelName).PlayHeadPageId = PageId;
				PlayedPageIds.Add(PageId);
			}
		}
	}

	OnPlayPages(PlayedPageIds, InPlayType, InPreviewChannelName, InSource);
	return PlayedPageIds;
}

bool UAvaRundown::PlayPage(int32 InPageId, EAvaRundownPagePlayType InPlayType, EAvaRundownPagePlaySource InSource)
{
	return !PlayPages({InPageId}, InPlayType, InSource).IsEmpty();
}

bool UAvaRundown::PlayPage(int32 InPageId, EAvaRundownPagePlayType InPlayType, const FName& InPreviewChannelName, EAvaRundownPagePlaySource InSource)
{
	return !PlayPages({InPageId}, InPlayType, InPreviewChannelName, InSource).IsEmpty();
}

bool UAvaRundown::RestorePlaySubPage(int32 InPageId, int32 InSubPageIndex, const FGuid& InExistingInstanceId, bool bInIsPreview, const FName& InPreviewChannelName)
{
	using namespace UE::AvaPlayback::Utils;
	
	auto LogError = [InPageId, InPreviewChannelName](const FString& InReason)
	{
		UE_LOGF(LogAvaRundown, Error,
			"%ls Couldn't restore playback state of page %d on channel \"%ls\": %ls.",
			*GetBriefFrameInfo(), InPageId, *InPreviewChannelName.ToString(), *InReason);
	};

	const FAvaRundownPage& Page = GetPage(InPageId);
	if (!Page.IsValidPage() || !Page.IsEnabled())
	{
		LogError(TEXT("Page is either not valid or disabled"));
		return false;
	}

	FString FailureReason;
	if (!IsChannelTypeCompatibleForRequest(Page, bInIsPreview, InPreviewChannelName, &FailureReason))
	{
		LogError(FString::Printf(TEXT("Channel Type is not compatible: %s"), *FailureReason));
		return false;
	}

	if (!InExistingInstanceId.IsValid())
	{
		LogError(TEXT("Specified instance id is invalid"));
		return false;
	}
	
	bool bPagePlayerCreated = false;
	UAvaRundownPagePlayer* PagePlayer = FindPlayerForPage(InPageId, bInIsPreview, InPreviewChannelName);

	if (!PagePlayer)
	{
		bPagePlayerCreated = true;
		PagePlayer = NewObject<UAvaRundownPagePlayer>(this);
		if (!PagePlayer->Initialize(this, Page, bInIsPreview, InPreviewChannelName))
		{
			return false;
		}
		UE_LOGF(LogAvaRundown, Verbose, "%ls Restored page player for page %d.", *GetBriefFrameInfo(), InPageId);
	}
	
	if (const UAvaRundownPlaybackInstancePlayer* LoadedInstancePlayer = PagePlayer->LoadInstancePlayer(InSubPageIndex, InExistingInstanceId))
	{
		if (bPagePlayerCreated)
		{
			AddPagePlayer(PagePlayer);
			GetOrCreatePageListPlaybackContextCollection().GetOrCreateContext(bInIsPreview, InPreviewChannelName).PlayHeadPageId = InPageId;
		}
		
		UAvaPlaybackGraph* Playback = LoadedInstancePlayer->Playback;
		if (Playback && !Playback->IsPlaying())
		{
			Playback->Play();
		}
		return true;
	}
	
	LogError(TEXT("Unable to acquire or load playback object"));
	return false;
}

bool UAvaRundown::CanPlayPage(int32 InPageId, bool bInPreview) const
{
	return CanPlayPage(InPageId, bInPreview, bInPreview ? GetDefaultPreviewChannelName() : NAME_None, nullptr);
}

bool UAvaRundown::CanPlayPage(int32 InPageId, bool bInPreview, const FName& InPreviewChannelName, FString* OutFailureReason) const
{
	using namespace UE::AvaPlayback::Utils;
	
	// Check if the page is valid and enabled.
	const FAvaRundownPage& SelectedPage = GetPage(InPageId);
	if (!SelectedPage.IsValidPage())
	{
		if (OutFailureReason)
		{
			*OutFailureReason = TEXT("Invalid Page Id");
		}
		return false;
	}

	if (!SelectedPage.IsEnabled())
	{
		if (OutFailureReason)
		{
			*OutFailureReason = TEXT("Page is disabled");
		}
		return false;
	}

	// Check channel validity and type compatibility.
	if (!IsChannelTypeCompatibleForRequest(SelectedPage, bInPreview, InPreviewChannelName, OutFailureReason))
	{
		return false;
	}

	// Check that if it is a template page it is meant to preview
	if (SelectedPage.IsTemplate() && !bInPreview)
	{
		if (OutFailureReason)
		{
			*OutFailureReason = TEXT("Template page can't be taken to program");
		}
		return false;
	}

	// Check if the asset path is valid
	if (!SelectedPage.HasAssets(*this))
	{
		if (!SelectedPage.HasCommands(this))
		{
			if (OutFailureReason)
			{
				*OutFailureReason = TEXT("Page has no assets nor commands");
			}
			return false; // no asset and no commands, can't play.
		}

		// Verify if the commands can be executed.
		bool bCanRunCommand = false;
		FName ChannelName = bInPreview ? InPreviewChannelName : SelectedPage.GetChannelName();
		FAvaRundownPageCommandContext PageCommandContext = { *this, SelectedPage, ChannelName };

		SelectedPage.ForEachInstancedCommands([&bCanRunCommand, &PageCommandContext, OutFailureReason](const FAvaRundownPageCommand& InCommand, const FAvaRundownPage& InPage)
		{
			FString FailureReason;
			const bool bCanExec = InCommand.CanExecuteOnPlay(PageCommandContext, OutFailureReason ? &FailureReason : nullptr);
			bCanRunCommand |= bCanExec;
			if (!bCanExec && OutFailureReason)
			{
				*OutFailureReason += FString::Printf(TEXT("%s%s"), OutFailureReason->IsEmpty() ? TEXT("") : TEXT("; "), *FailureReason);
			}
		}, this, /*bDirectOnly*/ false);

		if (!bCanRunCommand)
		{
			return false;
		}
	}

	// For page with TL, need to see if a transition can be started for that page.
	if (SelectedPage.HasTransitionLogic(this) && !CanStartTransitionForPage(SelectedPage, bInPreview, InPreviewChannelName, OutFailureReason))
	{
		return false;
	}

	// Remark:
	// No longer checks if the playback object is already playing because
	// a "playing" page can be played again, it means the animation will be restarted.
	return true;
}

TArray<int32> UAvaRundown::StopPages(const TArray<int32>& InPageIds, EAvaRundownPageStopOptions InOptions, bool bInPreview)
{
	return StopPages(InPageIds, InOptions, bInPreview, bInPreview ? GetDefaultPreviewChannelName() : NAME_None);
}

TArray<int32> UAvaRundown::StopPages(const TArray<int32>& InPageIds, EAvaRundownPageStopOptions InOptions, bool bInPreview, const FName& InPreviewChannelName)
{
	const bool bForceNoTransition = EnumHasAnyFlags(InOptions, EAvaRundownPageStopOptions::ForceNoTransition);
	TArray<int32> StoppedPageIds;
	StoppedPageIds.Reserve(InPageIds.Num());

	FAvaRundownPageTransitionBuilder TransitionBuilder(this);

	for (const int32 PageId : InPageIds)
	{
		const FAvaRundownPage& SelectedPage = GetPage(PageId);

		if (!SelectedPage.IsValidPage())
		{
			continue;
		}

		// Force stop all transitions for the selected page (if any).
		if (bForceNoTransition)
		{
			StopPageTransitionsForPage(SelectedPage, bInPreview, InPreviewChannelName);
		}

		if (SelectedPage.HasTransitionLogic(this) && !bForceNoTransition)
		{
			if (StopPageWithTransition(TransitionBuilder, SelectedPage, bInPreview, InPreviewChannelName))
			{
				StoppedPageIds.Add(PageId);
			}
		}
		else
		{
			if (StopPageNoTransition(SelectedPage, bInPreview, InPreviewChannelName))
			{
				StoppedPageIds.Add(PageId);
			}
		}
	}
	return StoppedPageIds;
}

bool UAvaRundown::CanStopPage(int32 InPageId, EAvaRundownPageStopOptions InOptions, bool bInPreview) const
{
	return CanStopPage(InPageId, InOptions, bInPreview, bInPreview ? GetDefaultPreviewChannelName() : NAME_None, nullptr);
}

bool UAvaRundown::CanStopPage(int32 InPageId, EAvaRundownPageStopOptions InOptions, bool bInPreview, const FName& InPreviewChannelName, FString* OutFailureReason) const
{
	const FAvaRundownPage& SelectedPage = GetPage(InPageId);

	if (!SelectedPage.IsValidPage())
	{
		if (OutFailureReason)
		{
			*OutFailureReason = TEXT("Invalid Page Id");
		}
		return false;
	}
	
	// For page with TL, need to see if a transition can be started for that page.
	if (!EnumHasAnyFlags(InOptions, EAvaRundownPageStopOptions::ForceNoTransition)
		&& SelectedPage.HasTransitionLogic(this)
		&& !CanStartTransitionForPage(SelectedPage, bInPreview, InPreviewChannelName, OutFailureReason))
	{
		return false;
	}
	
	const UAvaRundownPagePlayer* Player = FindPlayerForPage(InPageId, bInPreview, InPreviewChannelName);
	
	if (!Player)
	{
		if (OutFailureReason)
		{
			*OutFailureReason = FString::Printf(TEXT("No page player found on channel \"%s\""), bInPreview ? *InPreviewChannelName.ToString() : *SelectedPage.GetChannelName().ToString());
		}
		return false;
	}

	// Note: for the failure reason, we want to know about missing player vs player not playing.
	if (OutFailureReason && !Player->IsPlaying())
	{
		*OutFailureReason = TEXT("Page Player is not playing.");
	}
	
	return Player->IsPlaying();
}

TArray<int32> UAvaRundown::StopLayers(FName InChannelName, const TArray<FAvaTagHandle>& InLayers, EAvaRundownPageStopOptions InOptions)
{
	TArray<int32> StoppedPageIds;
	StoppedPageIds.Reserve(PagePlayers.Num());

	// We want to build a transition that is going to kick out the specified layers.
	FAvaRundownPageTransitionBuilder TransitionBuilder(this);
	
	const bool bUseTransition = !EnumHasAnyFlags(InOptions, EAvaRundownPageStopOptions::ForceNoTransition);

	if (bUseTransition)
	{
		// We need to make a special transition that kicks out layers
		if (UAvaRundownPageTransition* PageTransition = TransitionBuilder.FindOrAddTransition(InChannelName))
		{
			// Add the layers to kick out.
			PageTransition->ExitLayers.Append(InLayers);
		}
	}

	for (UAvaRundownPagePlayer* PagePlayer : PagePlayers)
	{
		if (PagePlayer->ChannelFName != InChannelName)
		{
			continue;
		}

		int32 LayerOverlapCount = 0;
		PagePlayer->ForEachInstancePlayer([&InLayers, &LayerOverlapCount, bUseTransition](UAvaRundownPlaybackInstancePlayer* InInstancePlayer)
		{
			for (const FAvaTagHandle& Layer : InLayers)
			{
				if (InInstancePlayer->TransitionLayer.Overlaps(Layer))
				{
					if (!bUseTransition)
					{
						InInstancePlayer->Stop();
					}
					++LayerOverlapCount;
					break;
				}
			}
		});	

		// Note: if the overlap count is smaller than the number of instances, only part of the page will be taken down.
		if (LayerOverlapCount)
		{
			StoppedPageIds.Add(PagePlayer->PageId);
		}
	}

	if (!StoppedPageIds.IsEmpty() && !bUseTransition)
	{
		RemoveStoppedPagePlayers();
	}
	return StoppedPageIds;
}

bool UAvaRundown::CanStopLayer(FName InChannelName, const FAvaTagHandle& InLayer) const
{
	for (const UAvaRundownPagePlayer* PagePlayer : PagePlayers)
	{
		if (PagePlayer->ChannelFName != InChannelName)
		{
			continue;
		}

		int32 LayerOverlapCount = 0;
		PagePlayer->ForEachInstancePlayer([&InLayer, &LayerOverlapCount](const UAvaRundownPlaybackInstancePlayer* InInstancePlayer)
		{
			if (InInstancePlayer->TransitionLayer.Overlaps(InLayer))
			{
				++LayerOverlapCount;
			}
		});	

		if (LayerOverlapCount)
		{
			return true;
		}
	}
	return false;
}

bool UAvaRundown::StopChannel(const FString& InChannelName)
{
	const FName ChannelName(InChannelName);
	int32 NumStoppedPages = 0;
	for (UAvaRundownPagePlayer* PagePlayer : PagePlayers)
	{
		// Don't let something else play on this channel.
		if (PagePlayer->ChannelName == ChannelName)
		{
			if (PagePlayer->Stop())
			{
				++NumStoppedPages;
			}
		}
	}
	RemoveStoppedPagePlayers();
	return NumStoppedPages > 0 ? true : false;
}

bool UAvaRundown::CanStopChannel(const FString& InChannelName) const
{
	const FName ChannelName(InChannelName);
	for (const UAvaRundownPagePlayer* PagePlayer : PagePlayers)
	{
		if (PagePlayer->ChannelName == ChannelName && PagePlayer->IsPlaying())
		{
			return true;
		}
	}
	return false;
}

bool UAvaRundown::ContinuePage(int32 InPageId, bool bInPreview)
{
	return ContinuePage(InPageId, bInPreview, bInPreview ? GetDefaultPreviewChannelName() : NAME_None);	
}

bool UAvaRundown::ContinuePage(int32 InPageId, bool bInPreview, const FName& InPreviewChannelName)
{
	const FAvaRundownPage& SelectedPage = GetPage(InPageId);

	if (SelectedPage.IsValidPage() && SelectedPage.IsEnabled())
	{
		if (UAvaRundownPagePlayer* Player = FindPlayerForPage(InPageId, bInPreview, InPreviewChannelName))
		{
			return Player->Continue();
		}
	}
	return false;
}

bool UAvaRundown::CanContinuePage(int32 InPageId, bool bInPreview) const
{
	return CanContinuePage(InPageId, bInPreview, bInPreview ? GetDefaultPreviewChannelName() : NAME_None);	
}

bool UAvaRundown::CanContinuePage(int32 InPageId, bool bInPreview, const FName& InPreviewChannelName) const
{
	const FAvaRundownPage& SelectedPage = GetPage(InPageId);

	if (SelectedPage.IsValidPage() && SelectedPage.IsEnabled())
	{
		const UAvaRundownPagePlayer* Player = FindPlayerForPage(InPageId, bInPreview, InPreviewChannelName);
		return Player && Player->IsPlaying();
	}

	return false;
}

FAvaRundownPageListReference UAvaRundown::AddSubList()
{
	const int32 SubListIdx = SubLists.Add(FAvaRundownSubList());
	SubLists[SubListIdx].Id = FGuid::NewGuid();
	SubListIndices.Add(SubLists[SubListIdx].Id, SubListIdx);

	const FAvaRundownPageListReference SubListReference = CreateSubListReference(SubLists[SubListIdx].Id);
	GetOnPageListChanged().Broadcast({this, SubListReference, EAvaRundownPageListChange::SubListAddedOrRemoved, {}});
	
	return SubListReference;
}

bool UAvaRundown::RemoveSubList(const FAvaRundownPageListReference& InPageListReference)
{
	if (IsValidSubList(InPageListReference))
	{
		// Update active list
		if (ActivePageList == InPageListReference)
		{
			SetActivePageList(InstancePageList);
		}

		if (const int32 *SubListIndex = SubListIndices.Find(InPageListReference.SubListId))
		{
			if (SubLists.IsValidIndex(*SubListIndex))
			{
				SubLists.RemoveAt(*SubListIndex);
				RefreshSubListIndices();
				GetOnPageListChanged().Broadcast({this, InPageListReference, EAvaRundownPageListChange::SubListAddedOrRemoved, {}});
				return true;
			}
		}
	}

	using namespace UE::AvaMedia::Rundown::Private;
	UE_LOGF(LogAvaRundown, Error, "Remove SubList failed: Invalid SubList Reference: %ls.", *ToString(InPageListReference));
	return false;
}

bool UAvaRundown::RenameSubList(const FAvaRundownPageListReference& InPageListReference, const FText& InNewName)
{
	FAvaRundownSubList& SubList = GetSubList(InPageListReference); 
	if (SubList.IsValid())
	{
		SubList.Name = InNewName;
		GetOnPageListChanged().Broadcast({this, InPageListReference, EAvaRundownPageListChange::SubListRenamed, {}});
		return true;
	}

	using namespace UE::AvaMedia::Rundown::Private;
	UE_LOGF(LogAvaRundown, Error, "Rename SubList failed: Invalid SubList Reference: %ls.", *ToString(InPageListReference));
	return false;
}

TArray<int32> UAvaRundown::GetPlayingPageIds(const FName InProgramChannelName) const
{
	TArray<int32> OutPlayedIds;
	OutPlayedIds.Reserve(PagePlayers.Num());
	for (const UAvaRundownPagePlayer* PagePlayer : PagePlayers)
	{
		if (PagePlayer->bIsPreview || !PagePlayer->IsPlaying())
		{
			continue;
		}

		if (!InProgramChannelName.IsNone() && PagePlayer->ChannelFName != InProgramChannelName)
		{
			continue;
		}

		OutPlayedIds.AddUnique(PagePlayer->PageId);
	}
	return OutPlayedIds;
}

TArray<int32> UAvaRundown::GetPreviewingPageIds(const FName InPreviewChannelName) const
{
	TArray<int32> OutPreviewingIds;
	OutPreviewingIds.Reserve(PagePlayers.Num());
	for (const UAvaRundownPagePlayer* PagePlayer : PagePlayers)
	{
		if (!PagePlayer->bIsPreview || !PagePlayer->IsPlaying())
		{
			continue;
		}

		if (!InPreviewChannelName.IsNone() && PagePlayer->ChannelFName != InPreviewChannelName)
		{
			continue;
		}
		
		OutPreviewingIds.AddUnique(PagePlayer->PageId);
	}
	return OutPreviewingIds;
}

bool UAvaRundown::SetActivePageList(const FAvaRundownPageListReference& InPageListReference)
{
	if (InPageListReference.Type == EAvaRundownPageListType::Instance)
	{
		ActivePageList = InstancePageList;
		OnActiveListChanged.Broadcast();
		return true;
	}

	if (IsValidSubList(InPageListReference))
	{
		ActivePageList = InPageListReference;
		OnActiveListChanged.Broadcast();
		return true;
	}

	return false;
}

bool UAvaRundown::HasActiveSubList() const
{
	return IsValidSubList(ActivePageList);
}

const FAvaRundownSubList& UAvaRundown::GetSubList(int32 InSubListIndex) const
{
	return SubLists.IsValidIndex(InSubListIndex) ? SubLists[InSubListIndex] : InvalidSubList; 
}

FAvaRundownSubList& UAvaRundown::GetSubList(int32 InSubListIndex)
{
	return SubLists.IsValidIndex(InSubListIndex) ? SubLists[InSubListIndex] : InvalidSubListMutable; 
}

const FAvaRundownSubList& UAvaRundown::GetSubList(const FGuid& InSubListId) const
{
	const int32 *Index = SubListIndices.Find(InSubListId);
	return Index ? GetSubList(*Index) : InvalidSubList;
}

FAvaRundownSubList& UAvaRundown::GetSubList(const FGuid& InSubListId)
{
	const int32 *Index = SubListIndices.Find(InSubListId);
	return Index ? GetSubList(*Index) : InvalidSubListMutable;
}

int32 UAvaRundown::GetSubListIndex(const FAvaRundownSubList& InSubList) const
{
	const int32 *Index = SubListIndices.Find(InSubList.Id);
	return Index ? *Index : INDEX_NONE;
}

bool UAvaRundown::IsValidSubList(const FAvaRundownPageListReference& InPageListReference) const
{
	return InPageListReference.Type == EAvaRundownPageListType::View && SubListIndices.Contains(InPageListReference.SubListId);
}

bool UAvaRundown::AddPageToSubList(const FAvaRundownPageListReference& InPageListReference, int32 InPageId, const FAvaRundownPageInsertPosition& InInsertPosition)
{
	if (!IsValidSubList(InPageListReference))
	{
		return false;
	}
	
	FAvaRundownSubList& SubList = GetSubList(InPageListReference);
	
	if (InstancedPages.PageIndices.Contains(InPageId) && !SubList.PageIds.Contains(InPageId))
	{
		int32 ExistingPageIndex = INDEX_NONE;

		if (InInsertPosition.IsValid())
		{
			ExistingPageIndex = SubList.PageIds.IndexOfByKey(InInsertPosition.AdjacentId);
		}

		if (InInsertPosition.IsAddBelow() && SubList.PageIds.IsValidIndex(ExistingPageIndex))
		{
			++ExistingPageIndex;
		}

		if (SubList.PageIds.IsValidIndex(ExistingPageIndex))
		{
			SubList.PageIds.Insert(InPageId, ExistingPageIndex);
		}
		else
		{
			SubList.PageIds.Add(InPageId);
		}
		
		GetOnPageListChanged().Broadcast({this, InPageListReference, EAvaRundownPageListChange::AddedPages, {InPageId}});
		return true;
	}

	return false;
}

bool UAvaRundown::AddPagesToSubList(const FAvaRundownPageListReference& InPageListReference, const TArray<int32>& InPages)
{
	if (IsValidSubList(InPageListReference))
	{
		FAvaRundownSubList& SubList = GetSubList(InPageListReference);
		
		bool bAddedPage = false;

		// Super inefficient for now.
		for (int32 PageId : InPages)
		{
			if (InstancedPages.PageIndices.Contains(PageId) && !SubList.PageIds.Contains(PageId))
			{
				SubList.PageIds.Add(PageId);
				bAddedPage = true;
			}
		}

		if (bAddedPage)
		{
			GetOnPageListChanged().Broadcast({this, InPageListReference, EAvaRundownPageListChange::AddedPages, InPages});
			return true;
		}
	}

	return false;
}

int32 UAvaRundown::RemovePagesFromSubList(const FAvaRundownPageListReference& InPageListReference, const TArray<int32>& InPages)
{
	if (IsValidSubList(InPageListReference))
	{
		const int32 Removed = GetSubList(InPageListReference).PageIds.RemoveAll([InPages](const int32& PageId)
			{
				return InPages.Contains(PageId);
			});

		if (Removed > 0)
		{
			GetOnPageListChanged().Broadcast({this, InPageListReference, EAvaRundownPageListChange::RemovedPages, InPages});
		}

		return Removed;
	}

	return 0;
}

namespace UE::AvaMedia::Rundown::Private
{
	const UAvaPlayableGroup* FindPlayableGroup(const UAvaRundownPagePlayer* InPagePlayer)
	{
		if (!InPagePlayer)
		{
			return nullptr;
		}

		for (const TObjectPtr<UAvaRundownPlaybackInstancePlayer>& InstancePlayer : InPagePlayer->InstancePlayers)
		{
			if (InstancePlayer->IsPlaying())
			{
				if (const UAvaPlayable* const Playable = InstancePlayer->Playback->GetFirstPlayable())
				{
					if (const UAvaPlayableGroup* const PlayableGroup = Playable->GetPlayableGroup())
					{
						return PlayableGroup;
					}
				}
			}
		}
		return nullptr;
	}
}

UTextureRenderTarget2D* UAvaRundown::GetPreviewRenderTarget(const FName& InPreviewChannel) const
{
	// For preview, there can be an output channel or not.
	// If there is one, we will prefer getting the render target directly from the channel.
	
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	const FAvaBroadcastOutputChannel& OutputChannel = Broadcast.GetCurrentProfile().GetChannel(InPreviewChannel); 
	if (OutputChannel.IsValidChannel())
	{
		return OutputChannel.GetCurrentRenderTarget(true);
	}

	// If there is no channel, we can get the render target from the playable group of a previewing page's playable
	// in the given channel. When playable group composition is implemented, this may have to change. 

	using namespace UE::AvaMedia::Rundown::Private;
	for (const TObjectPtr<UAvaRundownPagePlayer>& PagePlayer : PagePlayers)
	{
		if (PagePlayer->bIsPreview && PagePlayer->ChannelName == InPreviewChannel)
		{
			if (const UAvaPlayableGroup* PlayableGroup = FindPlayableGroup(PagePlayer))
			{
				return PlayableGroup->IsRenderTargetReady() ? PlayableGroup->GetRenderTarget() : nullptr;	
			}
		}
	}
	
	return nullptr;
}

FName UAvaRundown::GetDefaultPreviewChannelName()
{
	// Even if the user selected preview channel is empty, we need a default name as a key for the playback manager.
	const UAvaMediaSettings& Settings = UAvaMediaSettings::Get();
	return !Settings.PreviewChannelName.IsEmpty() ? FName(Settings.PreviewChannelName) : UE::AvaMedia::Playback::DefaultPreviewChannelName;
}

void UAvaRundown::OnParentWordBeginTearDown()
{
	PagePlayers.Reset();
}

namespace UE::AvaMedia::Rundown::Private
{
	/**
	 * Build an RC Values object with the given controller event ids.
	 * This will be used to push an "controller events" RC update with only those controllers.
	 * 
	 * @param InRcItemIds List of controllers to be part of this update.
	 * @param InPageValues Page Values, in case the controllers have values "payload" for this event.
	 * @return Shared object with the controller values that can be pushed in the RC update command.
	 */
	TSharedRef<FAvaPlayableRemoteControlValues> MakeControllerEvents(TConstArrayView<FGuid> InRcItemIds, const FAvaPlayableRemoteControlValues& InPageValues)
	{
		// Build the RC values with the events
		TSharedRef<FAvaPlayableRemoteControlValues> ControllerEvents = MakeShared<FAvaPlayableRemoteControlValues>();

		for (const FGuid& ItemId : InRcItemIds)
		{
			// If the controller has payload in the page, we can push it.
			if (const FAvaPlayableRemoteControlValue* ControllerValue = InPageValues.GetControllerValue(ItemId))
			{
				ControllerEvents->SetControllerValue(ItemId, *ControllerValue);
			}
			else
			{
				// Push empty value, in case of stateless controller.
				ControllerEvents->SetControllerValue(ItemId, FAvaPlayableRemoteControlValue()); 
			}
		}

		return ControllerEvents;
	}
}

bool UAvaRundown::PushRuntimeRemoteControlEvents(TConstArrayView<FGuid> InControllerIds, int32 InPageId, bool bInPushToProgram, bool bInPushToPreview, FName InPreviewChannelName)
{
	const FAvaRundownPage& Page = GetPage(InPageId);
	if (!Page.IsValidPage())
	{
		return false;
	}

	bool bValuesPushed = false;
	TSharedPtr<FAvaPlayableRemoteControlValues> ControllerEvents;

	for (const UAvaRundownPagePlayer* PagePlayer : PagePlayers)
	{
		if (PagePlayer->PageId != InPageId)
		{
			continue;
		}

		if (PagePlayer->bIsPreview)
		{
			// Filter on preview channel if provided.
			if (!bInPushToPreview || (!InPreviewChannelName.IsNone() && PagePlayer->ChannelFName != InPreviewChannelName))
			{
				continue;
			}
		}
		else
		{
			if (!bInPushToProgram)
			{
				continue;
			}
		}

		// Build the controller events update for the page (only once for all players).
		if (!ControllerEvents.IsValid())
		{
			using namespace UE::AvaMedia::Rundown::Private;
			ControllerEvents = MakeControllerEvents(InControllerIds, Page.GetRemoteControlValues());
		}

		for (int32 InstanceIndex = 0; InstanceIndex < PagePlayer->GetNumInstancePlayers(); ++InstanceIndex)
		{
			if (UAvaPlaybackGraph* Playback = PagePlayer->GetPlayback(InstanceIndex))
			{
				constexpr EAvaPlayableRCUpdateFlags UpdateFlags = EAvaPlayableRCUpdateFlags::ExecuteControllerBehaviors | EAvaPlayableRCUpdateFlags::ControllerValuesAsEvents;
				Playback->PushRemoteControlValues(PagePlayer->GetSourceAssetPath(InstanceIndex), PagePlayer->ChannelName, ControllerEvents.ToSharedRef(), UpdateFlags);
				bValuesPushed = true;
			}
		}
	}
	return bValuesPushed;
}


bool UAvaRundown::PushRuntimeRemoteControlValues(int32 InPageId, bool bInIsPreview, const FName& InPreviewChannelName) const
{
	const FAvaRundownPage& Page = GetPage(InPageId);
	if (!Page.IsValidPage())
	{
		return false;
	}

	bool bValuesPushed = false;
	for (const UAvaRundownPagePlayer* PagePlayer : PagePlayers)
	{
		if (PagePlayer->PageId != InPageId || PagePlayer->bIsPreview != bInIsPreview)
		{
			continue;
		}
		
		// Filter on preview channel if provided. 
		if (bInIsPreview && !InPreviewChannelName.IsNone() && PagePlayer->ChannelFName != InPreviewChannelName)
		{
			continue;
		}

		TSharedRef<FAvaPlayableRemoteControlValues> SharedRCValues = MakeShared<FAvaPlayableRemoteControlValues>(Page.GetRemoteControlValues());
		for (int32 InstanceIndex = 0; InstanceIndex < PagePlayer->GetNumInstancePlayers(); ++InstanceIndex)
		{
			if (UAvaPlaybackGraph* Playback = PagePlayer->GetPlayback(InstanceIndex))
			{
				// Remote Control Value (live) updates, when not part of a transition, will execute the controller behaviors on the runtime instances.
				constexpr EAvaPlayableRCUpdateFlags UpdateFlags = EAvaPlayableRCUpdateFlags::ExecuteControllerBehaviors;
				Playback->PushRemoteControlValues(PagePlayer->GetSourceAssetPath(InstanceIndex), PagePlayer->ChannelName, SharedRCValues, UpdateFlags);
				bValuesPushed = true;
			}
		}
	}
	return bValuesPushed;
}

// Note: This can be called if
// - RC entity values are either added or modified.
// - RC controller values are either added or modified.
void UAvaRundown::NotifyPageRemoteControlValueChanged(int32 InPageId, EAvaPlayableRemoteControlChanges InRemoteControlChanges)
{
	// For the previewed page, we automatically update the playback object's RC values live.
	// Note: Only the entity values are updated in the runtime (playback) RCP. No need to push controller values to runtime.
	if (EnumHasAnyFlags(InRemoteControlChanges, EAvaPlayableRemoteControlChanges::EntityValues))
	{
		// For now, potentially pushing all values multiple time (per frame) is mitigated by the
		// optimization in FAvaRemoteControlUtils::SetValueOfEntity that
		// will only set the value of the entity if it changed.
		PushRuntimeRemoteControlValues(InPageId, /*bInIsPreview*/ true);
	}
	OnPagesChanged.Broadcast(this, GetPage(InPageId), EAvaRundownPageChanges::RemoteControlValues);
}

#if WITH_EDITOR
void UAvaRundown::NotifyPIEEnded(const bool)
{
	// When PIE Ends, all worlds should be forcibly destroyed
	OnParentWordBeginTearDown();
}
#endif

FAvaPlaybackManager& UAvaRundown::GetPlaybackManager() const
{
	return IAvaMediaModule::Get().GetLocalPlaybackManager();
}

int32 UAvaRundown::AddPageFromTemplateInternal(int32 InTemplateId, const FAvaRundownPageIdGeneratorParams& InIdGeneratorParams, const FAvaRundownPageInsertPosition& InInsertAt)
{
	const int32* TemplateIndex = TemplatePages.PageIndices.Find(InTemplateId);

	if (!TemplateIndex)
	{
		return FAvaRundownPage::InvalidPageId;
	}

	const int32 NewId = GenerateUniquePageId(InIdGeneratorParams);

	int32 ExistingPageIndex = INDEX_NONE;

	if (InInsertAt.IsValid())
	{
		if (const int32* ExistingPageIndexPtr = InstancedPages.PageIndices.Find(InInsertAt.AdjacentId))
		{
			ExistingPageIndex = *ExistingPageIndexPtr;
		}
	}

	if (InInsertAt.IsAddBelow() && InstancedPages.Pages.IsValidIndex(ExistingPageIndex))
	{
		++ExistingPageIndex;
	}

	int32 NewIndex = INDEX_NONE;

	if (InstancedPages.Pages.IsValidIndex(ExistingPageIndex))
	{
		InstancedPages.Pages.Insert(TemplatePages.Pages[*TemplateIndex], ExistingPageIndex);
		NewIndex = ExistingPageIndex;
		
		// Need to update page indices after insertion.
		InstancedPages.PostInsertRefreshPageIndices(ExistingPageIndex + 1);
	}
	else
	{
		InstancedPages.Pages.Emplace(TemplatePages.Pages[*TemplateIndex]);
		NewIndex = InstancedPages.Pages.Num() - 1;
	}

	InstancedPages.PageIndices.Emplace(NewId, NewIndex);
	TemplatePages.Pages[*TemplateIndex].Instances.Add(NewId);
	InitializePage(InstancedPages.Pages[NewIndex], NewId, InTemplateId);

	return NewId;
}

void UAvaRundown::InitializePage(FAvaRundownPage& InOutPage, int32 InPageId, int32 InTemplateId) const
{
	InOutPage.PageId = InPageId;
	InOutPage.TemplateId = InTemplateId;
	InOutPage.CombinedTemplateIds.Empty();
	InOutPage.InstancedCommands.Empty();
	InOutPage.SetPageFriendlyName(FText::GetEmpty());
	InOutPage.UpdatePageSummary(this);
}

bool UAvaRundown::IsChannelTypeCompatibleForRequest(const FAvaRundownPage& InSelectedPage, bool bInIsPreview, const FName& InPreviewChannelName, FString* OutFailureReason) const
{
	// Check channel validity and type compatibility.
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	if (bInIsPreview)
	{
		// The incoming preview channel name may not exist, that is allowed.
		if (Broadcast.GetCurrentProfile().GetChannel(InPreviewChannelName).IsValidChannel()
			&& Broadcast.GetChannelType(InPreviewChannelName) != EAvaBroadcastChannelType::Preview)
		{
			if (OutFailureReason)
			{
				*OutFailureReason = FString::Printf(TEXT("Channel \"%s\" is not a \"preview\" channel in profile \"%s\"."),
					*InPreviewChannelName.ToString(), *Broadcast.GetCurrentProfileName().ToString());
			}
			return false;
		}
	}
	else
	{
		const FAvaBroadcastOutputChannel& Channel = Broadcast.GetCurrentProfile().GetChannel(InSelectedPage.GetChannelName()); 
		if (!Channel.IsValidChannel())
		{
			if (OutFailureReason)
			{
				*OutFailureReason = FString::Printf(TEXT("Channel \"%s\" is not a valid channel in \"%s\" profile."),
					*InSelectedPage.GetChannelName().ToString(), *Broadcast.GetCurrentProfileName().ToString());
			}
			return false;
		}
		if (Broadcast.GetChannelType(InSelectedPage.GetChannelName()) != EAvaBroadcastChannelType::Program)
		{
			if (OutFailureReason)
			{
				*OutFailureReason = FString::Printf(TEXT("Channel \"%s\" is not a \"program\" channel in profile \"%s\"."),
					*InSelectedPage.GetChannelName().ToString(), *Broadcast.GetCurrentProfileName().ToString());
			}
			return false;
		}

		// Check if the channel is offline.
		bool bHasOfflineOutput = false;
		bool bHasLocalOutput = false;
		const TArray<UMediaOutput*>& Outputs = Channel.GetMediaOutputs();
		for (const UMediaOutput* Output : Outputs)
		{
			if (Channel.IsMediaOutputRemote(Output) && Channel.GetMediaOutputState(Output) == EAvaBroadcastOutputState::Offline)
			{
				bHasOfflineOutput = true;
			}
			else
			{
				bHasLocalOutput = true;
				break;	// If a local output is detected all is good.
			}
		}

		// A channel is considered offline only if it doesn't have any local outputs since
		// the local outputs take priority (for now at least).
		if (bHasOfflineOutput && !bHasLocalOutput)
		{
			if (OutFailureReason)
			{
				*OutFailureReason = FString::Printf(TEXT("Channel \"%s\" is offline."),
					*InSelectedPage.GetChannelName().ToString());
			}
			return false;
		}
	}
	return true;
}

void UAvaRundown::OnPlayPages(const TArray<int32>& InPlayedPageIds, EAvaRundownPagePlayType InPlayType, FName InPreviewChannelName, EAvaRundownPagePlaySource InSource)
{
	if (InPlayedPageIds.IsEmpty())
	{
		return;
	}

#if WITH_EDITOR
	if (FEngineAnalytics::IsAvailable())
	{
		const bool bIsPreview = UE::AvaRundown::IsPreviewPlayType(InPlayType);

		const FAvaBroadcastProfile& Profile = UAvaBroadcast::Get().GetCurrentProfile();

		int32 ChannelCount = 0;
		FString ChannelStatuses;
		FString MediaOutputs;

		// Called before appending to the string
		auto OnPreAppendString = [](FString& OutString, const TCHAR* InDelimiter)
			{
				if (!OutString.IsEmpty())
				{
					OutString += InDelimiter;
				}
			};

		auto GatherChannelMediaOutputs = [&OnPreAppendString](TConstArrayView<UMediaOutput*> InMediaOutputs)
			{
				FString ChannelMediaOutputs;
				for (UMediaOutput* MediaOutput : InMediaOutputs)
				{
					if (MediaOutput)
					{
						OnPreAppendString(ChannelMediaOutputs, TEXT("|"));
						ChannelMediaOutputs += MediaOutput->GetClass()->GetName();
					}
				}
				return ChannelMediaOutputs;
			};

		if (bIsPreview)
		{
			const FAvaBroadcastOutputChannel& PreviewChannel = Profile.GetChannel(InPreviewChannelName);
			ChannelStatuses += StaticEnum<EAvaBroadcastChannelState>()->GetNameStringByValue(static_cast<int64>(PreviewChannel.GetState()));
			MediaOutputs += GatherChannelMediaOutputs(PreviewChannel.GetMediaOutputs());
			ChannelCount = 1;
		}
		else
		{
			TArray<const FAvaBroadcastOutputChannel*, TInlineAllocator<1>> Channels;
			for (int32 PageId : InPlayedPageIds)
			{
				const FAvaRundownPage& SelectedPage = GetPage(PageId);
				Channels.AddUnique(&Profile.GetChannel(SelectedPage.GetChannelName()));
			}
			ChannelCount = Channels.Num();

			// Gather channel statuses, and the media outputs per channel.
			// Media outputs groups are per channel and separated with ",". A media output group contain media outputs separated by "|".
			// E.g. "MediaOutput1,MediaOutput2|MediaOutput3"
			// Would be a setup for where there's 2 channels: 1st channel has 1 media output, 2nd channel has 2 media outputs.
			for (const FAvaBroadcastOutputChannel* Channel : Channels)
			{
				OnPreAppendString(ChannelStatuses, TEXT(","));
				ChannelStatuses += StaticEnum<EAvaBroadcastChannelState>()->GetNameStringByValue(static_cast<int64>(Channel->GetState()));

				OnPreAppendString(MediaOutputs, TEXT(","));
				MediaOutputs += GatherChannelMediaOutputs(Channel->GetMediaOutputs());
			}
		}

		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Reserve(6);
		Attributes.Emplace(TEXT("PlayType"), StaticEnum<EAvaRundownPagePlayType>()->GetNameStringByValue(static_cast<int64>(InPlayType)));
		Attributes.Emplace(TEXT("Source"), StaticEnum<EAvaRundownPagePlaySource>()->GetNameStringByValue(static_cast<int64>(InSource)));
		Attributes.Emplace(TEXT("PageCount"), FString::FromInt(InPlayedPageIds.Num()));
		Attributes.Emplace(TEXT("ChannelCount"), FString::FromInt(ChannelCount));
		Attributes.Emplace(TEXT("ChannelStatuses"), ChannelStatuses);
		Attributes.Emplace(TEXT("MediaOutputs"), MediaOutputs);
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MotionDesign.Rundown.Play"), Attributes);
	}
#endif
}

void UAvaRundown::AddPagePlayer(UAvaRundownPagePlayer* InPagePlayer)
{
	PagePlayers.Add(InPagePlayer);
	OnPagePlayerAdded.Broadcast(this, InPagePlayer);
}

IAvaRundownPageLoadingManager& UAvaRundown::MakePageLoadingManager()
{
	PageLoadingManager = MakeUnique<FAvaRundownPageLoadingManager>(this);
	return *PageLoadingManager;
}

namespace UE::AvaMedia::Rundown::Private
{
	bool ArePageRCValuesEqualForSubTemplate(const FAvaRundownPage& InSubTemplate, const FAvaRundownPage& InPage, const FAvaRundownPage& InOtherPage)
	{
		// Comparing only the entity values for now. For playback, this is what determines if the values are the
		// same or not. The controllers are for editing only.
		for (const TPair<FGuid, FAvaPlayableRemoteControlValue>& EntityEntry : InSubTemplate.GetRemoteControlValues().EntityValues)
		{
			const FAvaPlayableRemoteControlValue* Value = InPage.GetRemoteControlEntityValue(EntityEntry.Key);
			const FAvaPlayableRemoteControlValue* OtherValue = InOtherPage.GetRemoteControlEntityValue(EntityEntry.Key);
			if (!Value || !OtherValue || !Value->IsSameValueAs(*OtherValue))
			{
				return false;
			}
		}
		return true;
	}

	bool ArePageRCValuesEqualForSubTemplate(const FAvaRundownPage& InSubTemplate, const FAvaRundownPage& InPage, const UAvaRundownPlaybackInstancePlayer* InInstancePlayer)
	{
		if (InInstancePlayer && InInstancePlayer->GetPagePlayer())
		{
			if (UAvaRundown* Rundown = InInstancePlayer->GetPagePlayer()->GetRundown())
			{
				const FAvaRundownPage PlayingPage = Rundown->GetPage(InInstancePlayer->GetPagePlayer()->PageId);
				if (PlayingPage.IsValidPage())
				{
					return ArePageRCValuesEqualForSubTemplate(InSubTemplate, InPage, PlayingPage);
				}
			}
		}
		return false;
	}

	/**
	 * @brief Search for an existing instance player for the given template and sub-template.
	 * @param InRundown Rundown
	 * @param InPageToPlay New Page to be played.
	 * @param InTemplate Template to be played. Should be direct template of the page.
	 * @param InSubPageIndex Index of the sub-template.
	 * @param bInIsPreview True if the channel is a preview channel.
	 * @param InPreviewChannelName Name of the preview channel (if it is a preview channel).
	 * @return Pointer to found instance player.
	 */
	UAvaRundownPlaybackInstancePlayer* FindExistingInstancePlayer(
		const UAvaRundown* InRundown,
		const FAvaRundownPage& InPageToPlay,
		const FAvaRundownPage& InTemplate,
		int32 InSubPageIndex,
		bool bInIsPreview,
		const FName& InPreviewChannelName)
	{
		const FAvaRundownPage& SubTemplate = InTemplate.GetTemplate(InRundown, InSubPageIndex);

		if (!SubTemplate.IsValidPage())
		{
			return nullptr;
		}

		for (const TObjectPtr<UAvaRundownPagePlayer>& PagePlayer : InRundown->GetPagePlayers())
		{
			// Early filter on preview/channel.
			if (!PagePlayer || PagePlayer->bIsPreview != bInIsPreview)
			{
				continue;
			}

			// Preview: Skip if player channel name mismatches the passed in preview channel
			if (bInIsPreview && PagePlayer->ChannelName != InPreviewChannelName)
			{
				continue;
			}

			// Non-Preview: Skip if player channel mismatch the page's channel name
			if (!bInIsPreview && PagePlayer->ChannelFName != InPageToPlay.GetChannelName())
			{
				continue;
			}

			const FAvaRundownPage& PlayingPage = InRundown->GetPage(PagePlayer->PageId);

			if (!PlayingPage.IsValidPage())
			{
				continue;
			}

			const FAvaRundownPage& PlayingTemplate = PlayingPage.ResolveTemplate(InRundown);

			if (!PlayingTemplate.IsValidPage())
			{
				continue;
			}
			
			// Check if we have a corresponding template.
			if (PlayingTemplate.IsComboTemplate())
			{
				if (!PlayingTemplate.GetCombinedTemplateIds().Contains(SubTemplate.GetPageId()))
				{
					continue;
				}
			}
			else if (PlayingTemplate.GetPageId() != SubTemplate.GetPageId())
			{
				continue;
			}
			
			// Find Instance Player for the given sub-template.			
			// Remark: if not found, keep looking. With "reuse" instancing mode, instance players can bounce from combo to single and back to combo.
			UAvaRundownPlaybackInstancePlayer* InstancePlayer = PagePlayer->FindInstancePlayerByAssetPath(SubTemplate.GetAssetPath(InRundown));
			if (InstancePlayer && InstancePlayer->PlaybackInstance)
			{
				return InstancePlayer;
			}
		}

		return nullptr;
	}
}

bool UAvaRundown::PlayPageWithTransition(FAvaRundownPageTransitionBuilder& InBuilder, const FAvaRundownPage& InPage, EAvaRundownPagePlayType InPlayType, bool bInIsPreview, const FName& InPreviewChannelName)
{
	using namespace UE::AvaMedia::Rundown::Private;

	// Execute page's commands
	if (InPage.HasCommands(this))
	{
		const FName ChannelName = bInIsPreview ? InPreviewChannelName : InPage.GetChannelName();
		FAvaRundownPageCommandContext PageCommandContext = {*this, InPage, ChannelName};

		InPage.ForEachInstancedCommands(
			[&InBuilder, &PageCommandContext](const FAvaRundownPageCommand& InCommand, const FAvaRundownPage& InPage)
			{
				InCommand.ExecuteOnPlay(InBuilder, PageCommandContext);
			},
			this, /*bInDirectOnly*/ false); // Traverse templates

		// If this is a page with no assets, stop here to avoid making a page player.
		if (!InPage.HasAssets(*this))
		{
			return true;
		}
	}

	// For now, we always start a new page player, loading a new instance.
	UAvaRundownPagePlayer* NewPagePlayer = NewObject<UAvaRundownPagePlayer>(this);

	if (!NewPagePlayer || !NewPagePlayer->Initialize(this, InPage, bInIsPreview, InPreviewChannelName))
	{
		return false;
	}
	
	// -- TL vs No-TL pages mutual exclusion rule.
	// The way it is resolved for now, it is first come first serve. The pages that are entered first in
	// the transition will win. We may want to have say, the first no-TL page win. Will have to see what
	// is the best rule after some testing.
	if (const UAvaRundownPageTransition* ExistingPageTransition = InBuilder.FindTransition(NewPagePlayer))
	{
		// Don't add a page with TL in a transition that has a non-TL page. Can't co-exist.
		if (InPage.HasTransitionLogic(this))
		{
			if (ExistingPageTransition->HasEnterPagesWithNoTransitionLogic())
			{
				return false;	
			}

			// This is the layer exclusion rule. Rejects the page if any of the layers
			// are already in the transition. This is to prevent combo pages to start
			// "on top" of another page with same layer (in same transition only).
			for (const FAvaTagHandle& TagHandle : InPage.GetTransitionLayers(this))
			{
				if (ExistingPageTransition->ContainsTransitionLayer(TagHandle.TagId))
				{
					return false;
				}
			}
		}
		// Don't add a page with no-TL in a transition that has enter pages already (any page).
		else if (ExistingPageTransition->HasEnterPages())
		{
			return false;
		}
	}
	
	// Load or Recycle Instance Players.
	const int32 NumTemplates = InPage.GetNumTemplates(this);
	NewPagePlayer->InstancePlayers.Reserve(NumTemplates);

	const FAvaRundownPage& Template = InPage.ResolveTemplate(this);

	const UAvaMediaSettings& AvaMediaSettings = UAvaMediaSettings::Get();
	const bool bBypassTransitionOnSameValues = Template.IsComboTemplate()
		? AvaMediaSettings.bEnableComboTemplateSpecialLogic
		: AvaMediaSettings.bEnableSingleTemplateSpecialLogic;

	// Only allow reuse of instances if not doing a preview frame (or overridden in settings)
	// See UAvaPlayableLocalTransition::Start. It kicks out all the pages at the given layer when doing PreviewFrameTransition.
	// If the instance is re-used, it will kick that out as well.
	const bool bAllowReuse = InPlayType != EAvaRundownPagePlayType::PreviewFromFrame || AvaMediaSettings.bAllowReuseInPreviewFrame;

	TSet<FGuid> InstancesBypassingTransition;
	TSet<FGuid> ReusedInstances;
	
	for (int32 SubPageIndex = 0; SubPageIndex < NumTemplates; ++SubPageIndex)
	{
		bool bUsingExistingInstancePlayer = false;

		const FAvaRundownPage& SubTemplate = Template.GetTemplate(this, SubPageIndex);

		// -- Logic for Instance Player Reuse --
		if (bBypassTransitionOnSameValues || SubTemplate.GetTransitionMode(this) == EAvaTransitionInstancingMode::Reuse)
		{
			// Try to find an existing instance player for this template.
			UAvaRundownPlaybackInstancePlayer* InstancePlayer =
				FindExistingInstancePlayer(this, InPage, Template, SubPageIndex, bInIsPreview, InPreviewChannelName);
			
			if (InstancePlayer && InstancePlayer->PlaybackInstance)
			{
				if (bBypassTransitionOnSameValues && ArePageRCValuesEqualForSubTemplate(SubTemplate, InPage, InstancePlayer))
				{
					// Mark this instance as "bypassing" the next playable transition.
					InstancesBypassingTransition.Add(InstancePlayer->GetPlaybackInstanceId());
					bUsingExistingInstancePlayer = true;
				}
				else if (bAllowReuse && SubTemplate.GetTransitionMode(this) == EAvaTransitionInstancingMode::Reuse)
				{
					ReusedInstances.Add(InstancePlayer->GetPlaybackInstanceId());
					bUsingExistingInstancePlayer = true;
				}

				if (bUsingExistingInstancePlayer)
				{
					NewPagePlayer->AddInstancePlayer(InstancePlayer);

					// Setup user instance data to be able to track this page.
					UAvaRundownPagePlayer::SetInstanceUserDataFromPage(*InstancePlayer->PlaybackInstance, InPage);
				}
			}
		}

		if (!bUsingExistingInstancePlayer)
		{
			NewPagePlayer->LoadInstancePlayer(SubPageIndex, FGuid());
		}
	}

	if (NewPagePlayer->IsLoaded())
	{
		if (UAvaRundownPageTransition* PageTransition = InBuilder.FindOrAddTransition(NewPagePlayer))
		{
			if (PageTransition->AddEnterPage(NewPagePlayer))
			{
				PageTransition->InstancesBypassingTransition.Append(InstancesBypassingTransition);
				PageTransition->ReusedInstances.Append(ReusedInstances);
				PageTransition->bIsPreviewFrameTransition = (InPlayType == EAvaRundownPagePlayType::PreviewFromFrame);

				AddPagePlayer(NewPagePlayer);

				// Start the playback, will only actually start on next tick.
				// Animation command will not be pushed, relying on TL to start the appropriate animations.
				NewPagePlayer->Play(InPlayType);
				return true;
			}
		}
	}
	return false;
}

bool UAvaRundown::StopPageNoTransition(const FAvaRundownPage& InPage, bool bInPreview, const FName& InPreviewChannelName)
{
	if (UAvaRundownPagePlayer* PagePlayer = FindPlayerForPage(InPage.GetPageId(), bInPreview, InPreviewChannelName))
	{
		const bool bPlayerStopped = PagePlayer->Stop();
		RemoveStoppedPagePlayers();
		return bPlayerStopped;
	}
	return false;
}

bool UAvaRundown::StopPageWithTransition(FAvaRundownPageTransitionBuilder& InBuilder, const FAvaRundownPage& InPage, bool bInPreview, const FName& InPreviewChannelName)
{
	if (UAvaRundownPagePlayer* PagePlayer = FindPlayerForPage(InPage.GetPageId(), bInPreview, InPreviewChannelName))
	{
		if (UAvaRundownPageTransition* PageTransition = InBuilder.FindOrAddTransition(PagePlayer))
		{
			PageTransition->AddExitPage(PagePlayer);
			return true;
		}
	}
	return false;
}

const FAvaRundownPage& UAvaRundown::GetNextFromPages(const TArray<FAvaRundownPage>& InPages, int32 InStartingIndex) const
{
	if (InPages.IsEmpty())
	{
		return FAvaRundownPage::NullPage;
	}

	int32 NextIndex = InStartingIndex;
	const FAvaRundownPage& CurrentPage = InPages[InStartingIndex];
	do
	{
		if (InPages.IsValidIndex(++NextIndex))
		{
			const FAvaRundownPage& NextPage = InPages[NextIndex];
			if (NextPage.GetChannelName() == CurrentPage.GetChannelName())
			{
				return NextPage;
			}
		}
		else
		{
			NextIndex = FAvaRundownPage::InvalidPageId;
		}
	}
	while (NextIndex != InStartingIndex);

	return FAvaRundownPage::NullPage;
}

FAvaRundownPage& UAvaRundown::GetNextFromPages(TArray<FAvaRundownPage>& InPages, int32 InStartingIndex) const
{
	if (InPages.IsEmpty())
	{
		return FAvaRundownPage::NullPage;
	}

	int32 NextIndex = InStartingIndex;
	const FAvaRundownPage& CurrentPage = InPages[InStartingIndex];
	do
	{
		if (InPages.IsValidIndex(++NextIndex))
		{
			FAvaRundownPage& NextPage = InPages[NextIndex];
			if (NextPage.GetChannelName() == CurrentPage.GetChannelName())
			{
				return NextPage;
			}
		}
		else
		{
			NextIndex = FAvaRundownPage::InvalidPageId;
		}
	}
	while (NextIndex != InStartingIndex);

	return FAvaRundownPage::NullPage;
}

const FAvaRundownPage& UAvaRundown::GetNextFromSubList(const TArray<int32>& InSubListIds, int32 InStartingIndex) const
{
	if (InSubListIds.IsEmpty())
	{
		return FAvaRundownPage::NullPage;
	}

	int32 NextIndex = InStartingIndex;
	const FAvaRundownPage& CurrentPage = GetPage(InSubListIds[InStartingIndex]);
	do
	{
		if (InSubListIds.IsValidIndex(++NextIndex))
		{
			const FAvaRundownPage& NextPage = GetPage(InSubListIds[NextIndex]);
			if (NextPage.GetChannelName() == CurrentPage.GetChannelName())
			{
				return NextPage;
			}
		}
		else
		{
			NextIndex = FAvaRundownPage::InvalidPageId;
		}
	}
	while (NextIndex != InStartingIndex);

	return FAvaRundownPage::NullPage;
}

FAvaRundownPage& UAvaRundown::GetNextFromSubList(const TArray<int32>& InSubListIds, int32 InStartingIndex)
{
	if (InSubListIds.IsEmpty())
	{
		return FAvaRundownPage::NullPage;
	}

	int32 NextIndex = InStartingIndex;
	const FAvaRundownPage& CurrentPage = GetPage(InSubListIds[InStartingIndex]);
	do
	{
		if (InSubListIds.IsValidIndex(++NextIndex))
		{
			FAvaRundownPage& NextPage = GetPage(InSubListIds[NextIndex]);
			if (NextPage.GetChannelName() == CurrentPage.GetChannelName())
			{
				return NextPage;
			}
		}
		else
		{
			NextIndex = FAvaRundownPage::InvalidPageId;
		}
	}
	while (NextIndex != InStartingIndex);

	return FAvaRundownPage::NullPage;
}

UAvaRundownPageTransition* UAvaRundown::GetPageTransition(const FGuid& InTransitionId) const
{
	for (UAvaRundownPageTransition* PageTransition : PageTransitions)
	{
		if (PageTransition && PageTransition->GetTransitionId() == InTransitionId)
		{
			return PageTransition;
		}
	}
	return nullptr;
}

bool UAvaRundown::CanStartTransitionForPage(const FAvaRundownPage& InPage, bool bInIsPreview, const FName& InPreviewChannelName, FString* OutFailureReason) const
{
	// Current constraint: There can only be one transition (running properly) at a time in a world.
	// Given that, for now (and the foreseeable future until we support more playable groups per channels),
	// we can equate a "channel" to a "world", this is hardcoded for the level streaming playables.
	// So, we can just check the channels for now.
	const FName ChannelName = bInIsPreview ? InPreviewChannelName : InPage.GetChannelName();
	for (const UAvaRundownPageTransition* PageTransition : PageTransitions)
	{
		if (PageTransition && PageTransition->GetChannelName() == ChannelName)
		{
			if (OutFailureReason)
			{
				*OutFailureReason = FString::Printf(TEXT("Channel %s already has transition: %s"), *ChannelName.ToString(), *PageTransition->GetBriefTransitionDescription());
			}
			return false;
		}
	}
	return true;
}

int32 UAvaRundown::StopPageTransitionsForPage(const FAvaRundownPage& InPage, bool bInIsPreview, const FName& InPreviewChannelName)
{
	// Implementation note: there can only be one transition at the moment and all playing pages are part of it.
	// This is why this function is equivalent to stopping the transitions in a given channel.
	// Keeping the function around for future evolution of the system.
	const FName ChannelName = bInIsPreview ? InPreviewChannelName : InPage.GetChannelName();
	return StopPageTransitionsForChannel(ChannelName);
}

int32 UAvaRundown::StopPageTransitionsForChannel(const FName InChannelName)
{
	return StopPageTransitionsByPredicate([InChannelName](UAvaRundownPageTransition* InTransition)
	{
		return InTransition->GetChannelName() == InChannelName;
	});
}

int32 UAvaRundown::StopPageTransitionsByPredicate(TFunctionRef<bool(UAvaRundownPageTransition*)> InPredicate)
{
	TArray<UAvaRundownPageTransition*> TransitionsToStop;
	TransitionsToStop.Reserve(PageTransitions.Num());

	// Note: we build a separate list because stopping the transitions should
	// lead to the transitions being removed from PageTransitions (through the events).
	for (UAvaRundownPageTransition* PageTransition : PageTransitions)
	{
		if (PageTransition && InPredicate(PageTransition))
		{
			TransitionsToStop.Add(PageTransition);
		}
	}

	for (UAvaRundownPageTransition* Transition : TransitionsToStop)
	{
		Transition->Stop();

		// Normal course of events should have removed the transition, but
		// if something is wrong with the events, we double check it is indeed removed.
		if (PageTransitions.Contains(Transition))
		{
			UE_LOGF(LogAvaRundown, Warning, "A page transition was not properly cleaned up.");
			PageTransitions.Remove(Transition);
		}
	}

	return TransitionsToStop.Num();
}

UAvaRundownPagePlayer* UAvaRundown::FindPlayerForProgramPage(int32 InPageId) const
{
	const TObjectPtr<UAvaRundownPagePlayer>* FoundPlayer = PagePlayers.FindByPredicate([InPageId](const UAvaRundownPagePlayer* InPagePlayer)
	{
		return InPagePlayer->PageId == InPageId && !InPagePlayer->bIsPreview;
	});
	return FoundPlayer ? *FoundPlayer : nullptr;
}

UAvaRundownPagePlayer* UAvaRundown::FindPlayerForPreviewPage(int32 InPageId, const FName& InPreviewChannelFName) const
{
	const TObjectPtr<UAvaRundownPagePlayer>* FoundPlayer = PagePlayers.FindByPredicate([InPageId, InPreviewChannelFName](const UAvaRundownPagePlayer* InPagePlayer)
	{
		return InPagePlayer->PageId == InPageId && InPagePlayer->bIsPreview && InPagePlayer->ChannelFName == InPreviewChannelFName;
	});
	return FoundPlayer ? *FoundPlayer : nullptr;
}

UAvaRundownPagePlayer* UAvaRundown::FindPagePlayer(int32 InPageId, FName InChannelName) const
{
	const TObjectPtr<UAvaRundownPagePlayer>* FoundPlayer = PagePlayers.FindByPredicate([InPageId, InChannelName](const UAvaRundownPagePlayer* InPagePlayer)
	{
		return InPagePlayer->PageId == InPageId && InPagePlayer->ChannelFName == InChannelName;
	});
	return FoundPlayer ? *FoundPlayer : nullptr;
}

void UAvaRundown::RemoveStoppedPagePlayers()
{
	for (UAvaRundownPagePlayer* PagePlayer : PagePlayers)
	{
		if (PagePlayer && !PagePlayer->IsPlaying())
		{
			OnPagePlayerRemoving.Broadcast(this, PagePlayer);
		}
	}
	
	PagePlayers.RemoveAll([](const UAvaRundownPagePlayer* InPagePlayer) { return !InPagePlayer || InPagePlayer->IsPlaying() == false;});
}

#undef LOCTEXT_NAMESPACE
