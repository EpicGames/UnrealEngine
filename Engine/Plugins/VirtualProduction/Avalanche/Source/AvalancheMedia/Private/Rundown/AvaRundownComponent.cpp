// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownComponent.h"

#include "Engine/World.h"
#include "IAvaMediaModule.h"
#include "Playback/AvaPlaybackManager.h"
#include "Rundown/AvaRundown.h"

UAvaRundownComponent::UAvaRundownComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bWantsInitializeComponent = true;
}

bool UAvaRundownComponent::PlayPage(int32 InPageId)
{
	if (Rundown && !Rundown->IsPagePlaying(InPageId))
	{
		return Rundown->PlayPage(InPageId, EAvaRundownPagePlayType::PlayFromStart, EAvaRundownPagePlaySource::Component);
	}
	return false;
}

bool UAvaRundownComponent::StopPage(int32 InPageId)
{
	if (Rundown && Rundown->IsPagePlaying(InPageId))
	{
		return Rundown->StopPage(InPageId, EAvaRundownPageStopOptions::Default, false);
	}
	return false;
}

int32 UAvaRundownComponent::GetNumberOfPages() const
{
	return Rundown ? Rundown->GetInstancedPages().Pages.Num() : 0;
}

int32 UAvaRundownComponent::GetPageIdForIndex(int32 InPageIndex) const
{
	if (Rundown && Rundown->GetInstancedPages().Pages.IsValidIndex(InPageIndex))
	{
		return Rundown->GetInstancedPages().Pages[InPageIndex].GetPageId();
	}
	return 0;
}

void UAvaRundownComponent::InitializeComponent()
{
	Super::InitializeComponent();
	FWorldDelegates::OnWorldBeginTearDown.AddUObject(this, &UAvaRundownComponent::OnWorldBeginTearDown);
}

void UAvaRundownComponent::UninitializeComponent()
{
	FWorldDelegates::OnWorldBeginTearDown.RemoveAll(this);
	Super::UninitializeComponent();
}

void UAvaRundownComponent::OnWorldBeginTearDown(UWorld* InWorld)
{
	if (InWorld == GetWorld())
	{
		IAvaMediaModule::Get().GetLocalPlaybackManager().OnParentWorldBeginTearDown();
		if (Rundown)
		{
			// Since we have forcibly teared down the manager, update the
			// status of the rundown to reflect the appropriate state.
			Rundown->OnParentWordBeginTearDown();
		}
	}
}

