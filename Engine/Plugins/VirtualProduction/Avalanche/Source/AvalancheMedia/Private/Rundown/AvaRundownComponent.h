// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "AvaRundownComponent.generated.h"

class UAvaRundown;

/**
 * Add this actor component to blueprint actor to expose the API to control an
 * Motion Design Rundown in game.
 */
UCLASS(BlueprintType, Blueprintable, ClassGroup = "Motion Design Rundown", 
	meta = (DisplayName = "Motion Design Rundown Component", BlueprintSpawnableComponent))
class UAvaRundownComponent : public UActorComponent
{
	GENERATED_BODY()

public:
	UAvaRundownComponent(const FObjectInitializer& ObjectInitializer);
	
	UPROPERTY(EditAnywhere, Category = "Motion Design Media", meta = (DisplayName = "Motion Design Rundown"))
	TObjectPtr<UAvaRundown> Rundown = nullptr;
	
	UFUNCTION(BlueprintCallable, Category = "Motion Design Media")
	bool PlayPage(int32 InPageId);

	UFUNCTION(BlueprintCallable, Category = "Motion Design Media")
	bool StopPage(int32 InPageId);

	UFUNCTION(BlueprintCallable, Category = "Motion Design Media")
	int32 GetNumberOfPages() const;
	
	UFUNCTION(BlueprintCallable, Category = "Motion Design Media")
	int32 GetPageIdForIndex(int32 InPageIndex) const;

	virtual void InitializeComponent() override;
	virtual void UninitializeComponent() override;

protected:
	void OnWorldBeginTearDown(UWorld* InWorld);
};
