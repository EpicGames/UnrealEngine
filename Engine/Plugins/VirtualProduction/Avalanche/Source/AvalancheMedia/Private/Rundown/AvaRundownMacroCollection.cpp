// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownMacroCollection.h"

bool UAvaRundownMacroCollection::HasBindingFor(const FInputChord& InInputChord) const
{
	for (const FAvaRundownMacroKeyBinding& KeyBinding : KeyBindings)
	{
		if (KeyBinding.InputChord == InInputChord)
		{
			return true;
		}
	}
	return false;
}

int32 UAvaRundownMacroCollection::ForEachBinding(const FInputChord& InInputChord, TFunctionRef<bool(const FAvaRundownMacroKeyBinding&)> InCallback) const
{
	int32 NumBindings = 0;
	for (const FAvaRundownMacroKeyBinding& KeyBinding : KeyBindings)
	{
		if (KeyBinding.InputChord == InInputChord)
		{
			if (InCallback(KeyBinding))
			{
				++NumBindings;
			}
		}
	}
	return NumBindings;
}

int32 UAvaRundownMacroCollection::ForEachCommand(const FInputChord& InInputChord, TFunctionRef<bool(const FAvaRundownMacroCommand&)> InCallback) const
{
	int32 NumCommands = 0;
	for (const FAvaRundownMacroKeyBinding& KeyBinding : KeyBindings)
	{
		if (KeyBinding.InputChord == InInputChord)
		{
			for (const FAvaRundownMacroCommand& Command : KeyBinding.Commands)
			{
				if (InCallback(Command))
				{
					++NumCommands;
				}
			}
		}
	}
	return NumCommands;
}
