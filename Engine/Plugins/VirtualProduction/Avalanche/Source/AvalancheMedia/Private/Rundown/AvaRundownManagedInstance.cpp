// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownManagedInstance.h"

#include "AvaMediaSettings.h"
#include "Playable/AvaPlayableRemoteControlPresetInfoCache.h"
#include "RemoteControlPreset.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownManagedInstanceCache.h"

namespace UE::AvaMedia::RundownManagedInstance::Private
{
	bool IsEditorTransacting()
	{
#if WITH_EDITOR
		return GIsTransacting;
#else
		return false;
#endif
	}
}

FAvaRundownManagedInstance::FAvaRundownManagedInstance(FAvaRundownManagedInstanceCache* InParentCache, const FSoftObjectPath& InAssetPath)
	: ParentCache(InParentCache)
	, SourceAssetPath(InAssetPath)
	, ActivePageId(FAvaRundownPage::InvalidPageId)
{}

FAvaRundownManagedInstance::~FAvaRundownManagedInstance()
{}

void FAvaRundownManagedInstance::SetCurrentRundownPageForEvents(UAvaRundown* InRundown, int32 InPageId)
{
	RundownForEvents = InRundown;
	ActivePageId = InPageId;
}

void FAvaRundownManagedInstance::InvalidateFromCache()
{
	if (ParentCache)
	{
		// The managed asset is not deleted immediately to avoid issues
		// with destroying delegates while they are being called.
		ParentCache->InvalidateNoDelete(SourceAssetPath);
		ParentCache = nullptr;
	}
}

FAvaPlayableRemoteControlPresetInfo& FAvaRundownManagedInstance::GetRemoteControlPresetInfoFromCache(URemoteControlPreset* InRemoteControlPreset) const
{
	RemoteControlPresetInfo = IAvaPlayableRemoteControlPresetInfoCache::Get().GetRemoteControlPresetInfo(GetSourceAssetPath(), InRemoteControlPreset);
	if (RemoteControlPresetInfo.IsValid())
	{
		return *RemoteControlPresetInfo;
	}

	static FAvaPlayableRemoteControlPresetInfo EmptyPresetInfo;
	return EmptyPresetInfo;
}

void FAvaRundownManagedInstance::RegisterSourceRemoteControlPresetDelegates(URemoteControlPreset* InSourceRemoteControlPreset)
{
	if (!::IsValid(InSourceRemoteControlPreset))
	{
		return;
	}
	
	InSourceRemoteControlPreset->OnEntityExposed().AddRaw(
		this, &FAvaRundownManagedInstance::OnSourceRemoteControlEntitiesExposed);
	InSourceRemoteControlPreset->OnEntityUnexposed().AddRaw(
		this, &FAvaRundownManagedInstance::OnSourceRemoteControlEntitiesUnexposed);
	InSourceRemoteControlPreset->OnEntitiesUpdated().AddRaw(
		this, &FAvaRundownManagedInstance::OnSourceRemoteControlEntitiesUpdated);
	InSourceRemoteControlPreset->OnExposedPropertiesModified().AddRaw(
		this, &FAvaRundownManagedInstance::OnSourceRemoteControlExposedPropertiesModified);
	InSourceRemoteControlPreset->OnControllerAdded().AddRaw(
		this, &FAvaRundownManagedInstance::OnSourceRemoteControlControllerAdded);
	InSourceRemoteControlPreset->OnControllerRemoved().AddRaw(
		this, &FAvaRundownManagedInstance::OnSourceRemoteControlControllerRemoved);
	InSourceRemoteControlPreset->OnControllerRenamed().AddRaw(
		this, &FAvaRundownManagedInstance::OnSourceRemoteControlControllerRenamed);
	InSourceRemoteControlPreset->OnControllerModified().AddRaw(
		this, &FAvaRundownManagedInstance::OnSourceRemoteControlControllerModified);
}

void FAvaRundownManagedInstance::UnregisterSourceRemoteControlPresetDelegates(URemoteControlPreset* InSourceRemoteControlPreset) const
{
	if (!::IsValid(InSourceRemoteControlPreset))
	{
		return;
	}
	
	InSourceRemoteControlPreset->OnEntityExposed().RemoveAll(this);
	InSourceRemoteControlPreset->OnEntityUnexposed().RemoveAll(this);
	InSourceRemoteControlPreset->OnEntitiesUpdated().RemoveAll(this);
	InSourceRemoteControlPreset->OnExposedPropertiesModified().RemoveAll(this);
	InSourceRemoteControlPreset->OnControllerAdded().RemoveAll(this);
	InSourceRemoteControlPreset->OnControllerRemoved().RemoveAll(this);
	InSourceRemoteControlPreset->OnControllerRenamed().RemoveAll(this);
	InSourceRemoteControlPreset->OnControllerModified().RemoveAll(this);
}

void FAvaRundownManagedInstance::RegisterManagedRemoteControlPresetDelegates(URemoteControlPreset* InManagedRemoteControlPreset)
{
	if (::IsValid(InManagedRemoteControlPreset))
	{
		InManagedRemoteControlPreset->OnControllerModified().AddRaw(this, &FAvaRundownManagedInstance::OnManagedRemoteControlControllerModified);
	}
	
}

void FAvaRundownManagedInstance::UnregisterManagedRemoteControlPresetDelegates(URemoteControlPreset* InManagedRemoteControlPreset) const
{
	if (::IsValid(InManagedRemoteControlPreset))
	{
		InManagedRemoteControlPreset->OnControllerModified().RemoveAll(this);
	}
}

void FAvaRundownManagedInstance::OnSourceRemoteControlEntitiesExposed(URemoteControlPreset* InSourcePreset, const FGuid& InEntityId)
{
	InvalidateFromCache();
}

void FAvaRundownManagedInstance::OnSourceRemoteControlEntitiesUnexposed(URemoteControlPreset* InSourcePreset, const FGuid& InEntityId)
{
	InvalidateFromCache();
}

void FAvaRundownManagedInstance::OnSourceRemoteControlEntitiesUpdated(URemoteControlPreset* InSourcePreset, const TSet<FGuid>& InModifiedEntities)
{
	// Info: Delegate called when the exposed entity wrapper itself is updated (ie. binding change, rename).
	// Not sure if we should invalidate the cache for this one.
}

void FAvaRundownManagedInstance::OnSourceRemoteControlExposedPropertiesModified(URemoteControlPreset* InSourcePreset, const TSet<FGuid>& InModifiedProperties)
{
	// This will be called if values are modified from the Remote Control tab in the Ava BP editor.
	// Experimental: Propagate the default value immediately if it exists.
	// We don't want to apply those to the managed remote control because we don't want to overwrite the page's values.
	// We probably need an extra event to propagate a change in default value so pages with no values for that property
	// can be updated.

	// There is no event called when entities are unbound to actions, so rescan all actions just in case.
	if (RemoteControlPresetInfo.IsValid())
	{
		RemoteControlPresetInfo->Refresh(InSourcePreset);
	}
	else
	{
		GetRemoteControlPresetInfoFromCache(InSourcePreset);
	}

	for (const FGuid& PropertyId : InModifiedProperties)
	{
		// Update existing values only.
		// Remark: added values will go through OnSourceRemoteControlControllerAdded instead.
		if (DefaultRemoteControlValues.HasEntityValue(PropertyId))
		{
			DefaultRemoteControlValues.SetEntityValue(PropertyId, InSourcePreset, true);
		}
	}
}

void FAvaRundownManagedInstance::OnSourceRemoteControlControllerAdded(URemoteControlPreset* InSourcePreset, const FName NewControllerName, const FGuid& InControllerId)
{
	InvalidateFromCache();
}

void FAvaRundownManagedInstance::OnSourceRemoteControlControllerRemoved(URemoteControlPreset* InSourcePreset, const FGuid& InControllerId)
{
	InvalidateFromCache();
}

void FAvaRundownManagedInstance::OnSourceRemoteControlControllerRenamed(URemoteControlPreset* InSourcePreset, const FName InOldLabel, const FName InNewLabel)
{
	InvalidateFromCache();
}

void FAvaRundownManagedInstance::OnSourceRemoteControlControllerModified(URemoteControlPreset* InSourcePreset, const TSet<FGuid>& InModifiedControllerIds)
{
	// This will be called if values are modified from the Remote Control tab in the Ava BP editor.
	// Experimental: Propagate the default value immediately if it exists.
	// We don't want to apply those to the managed remote control because we don't want to overwrite the page's values.
	// We probably need an extra event to propagate a change in default value so pages with no values for that property
	// can be updated.
	// Todo: Need to hookup some reimport notification for all templates/pages using this asset.
	for (const FGuid& ControllerId : InModifiedControllerIds)
	{
		// Update existing values only.
		if (DefaultRemoteControlValues.HasControllerValue(ControllerId))
		{
			DefaultRemoteControlValues.SetControllerValue(ControllerId, InSourcePreset, true);
		}
	}
}

void FAvaRundownManagedInstance::OnManagedRemoteControlControllerModified(URemoteControlPreset* InPreset, const TSet<FGuid>& InModifiedControllerIds)
{
	using namespace UE::AvaMedia::RundownManagedInstance::Private;
	// Note: Ignore changes from the RCP Transaction listener.
	if (!::IsValid(InPreset) || GetRemoteControlPreset() != InPreset || IsEditorTransacting())
	{
		return;
	}

	UAvaRundown* Rundown = RundownForEvents.Get();

	if (!Rundown || !Rundown->GetPage(ActivePageId).IsValidPage())
	{
		return;
	}
	
	TArray<FGuid, TInlineAllocator<1>> EventControllerIds;

	for (const FGuid& Id : InModifiedControllerIds)
	{
		if (URCVirtualPropertyBase* Controller = InPreset->GetController(Id))
		{
			if (FAvaPlayableRemoteControlValues::IsRuntimeEventController(Controller))
			{
				EventControllerIds.Add(Id);
			}
		}
	}

	if (!EventControllerIds.IsEmpty())
	{
		const bool bPushToProgram = UAvaMediaSettings::Get().bRundownPushControllerEventsToProgram;
		const bool bPushToPreview = UAvaMediaSettings::Get().bRundownPushControllerEventsToPreview;
		Rundown->PushRuntimeRemoteControlEvents(EventControllerIds, ActivePageId, bPushToProgram, bPushToPreview);
	}
}
