// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownManagedInstanceCache.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AvaMediaSettings.h"
#include "IAvaMediaModule.h"
#include "Playback/AvaPlaybackUtils.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownManagedInstanceLevel.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"

FAvaRundownManagedInstanceCache::FAvaRundownManagedInstanceCache()
{
	UPackage::PackageSavedWithContextEvent.AddRaw(this, &FAvaRundownManagedInstanceCache::OnPackageSaved);
	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		AssetRegistry->OnAssetRemoved().AddRaw(this, &FAvaRundownManagedInstanceCache::OnAssetRemoved);
	}
	IAvaMediaModule::Get().GetOnAvaMediaSyncPackageModified().AddRaw(this, &FAvaRundownManagedInstanceCache::OnAvaSyncPackageModified);
	
#if WITH_EDITOR
	UAvaMediaSettings::GetMutable().OnSettingChanged().AddRaw(this, &FAvaRundownManagedInstanceCache::OnSettingChanged);	
#endif
}

FAvaRundownManagedInstanceCache::~FAvaRundownManagedInstanceCache()
{
	UPackage::PackageSavedWithContextEvent.RemoveAll(this);
	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		AssetRegistry->OnAssetRemoved().RemoveAll(this);
	}
	IAvaMediaModule::Get().GetOnAvaMediaSyncPackageModified().RemoveAll(this);
	
#if WITH_EDITOR
	UAvaMediaSettings::GetMutable().OnSettingChanged().RemoveAll(this);
#endif
}

TSharedPtr<FAvaRundownManagedInstance> FAvaRundownManagedInstanceCache::GetOrLoadInstance(const FSoftObjectPath& InAssetPath)
{
	FinishPendingActions();

	if (InAssetPath.IsNull())
	{
		return nullptr;
	}
	
	OrderQueue.Remove(InAssetPath); // Removes preserves the order. O(n)
	OrderQueue.Add(InAssetPath); // Most recent is at the end of the array.

	if (const TSharedPtr<FAvaRundownManagedInstance>* ExistingEntry = Instances.Find(InAssetPath))
	{
		if (ExistingEntry->IsValid())
		{
			return *ExistingEntry;
		}
	}

	TSharedPtr<FAvaRundownManagedInstance> NewEntry;

	const FString PackageName = InAssetPath.GetLongPackageName();
	if (FAvaPlaybackUtils::IsMapAsset(PackageName))
	{
		NewEntry = MakeShared<FAvaRundownManagedInstanceLevel>(this, InAssetPath);
	}

	if (NewEntry.IsValid() && NewEntry->IsValid())
	{
		Instances.Add(InAssetPath, NewEntry);
		TrimCache();
	}

	return NewEntry;
}

void FAvaRundownManagedInstanceCache::InvalidateNoDelete(const FSoftObjectPath& InAssetPath)
{
	if (Instances.Contains(InAssetPath) && !PendingInvalidatedPaths.Contains(InAssetPath))
	{
		PendingInvalidatedPaths.Add(InAssetPath);
		OnEntryInvalidated.Broadcast(InAssetPath);
	}
}

void FAvaRundownManagedInstanceCache::Invalidate(const FSoftObjectPath& InAssetPath)
{
	// Ensure no pending actions. For instance, current path could already be pending
	// and we don't want the events to be fired multiple time.
	FinishPendingActions();

	if (Instances.Contains(InAssetPath))
	{
		RemoveEntry(InAssetPath);
		OnEntryInvalidated.Broadcast(InAssetPath);
	}
}

int32 FAvaRundownManagedInstanceCache::GetMaximumCacheSize() const
{
	return UAvaMediaSettings::Get().ManagedInstanceCacheMaximumSize;
}

void FAvaRundownManagedInstanceCache::Flush(const FSoftObjectPath& InAssetPath)
{
	if (const TSharedPtr<FAvaRundownManagedInstance>* ExistingEntry = Instances.Find(InAssetPath))
	{
		if (ExistingEntry->GetSharedReferenceCount() <= 1)
		{
			RemoveEntry(InAssetPath);
		}
	}
}

void FAvaRundownManagedInstanceCache::Flush()
{
	RemoveEntries([](const FSoftObjectPath& InAssetPath, const TSharedPtr<FAvaRundownManagedInstance>& InManagedInstance)
	{
		return (InManagedInstance.GetSharedReferenceCount() <= 1) ? true : false;
	}, false);
	
	// Dump post-flush remaining assets to help diagnose which editor keeps the asset alive.
	// Since the cache is shared with all editors (currently), assets can still remain if used by other playlist editors/servers.
	if (UE_LOG_ACTIVE(LogAvaRundown, Verbose))
	{
		if (Instances.Num() > 0)
		{
			UE_LOGF(LogAvaRundown, Verbose, "Managed Instance Cache: Assets remaining in cache after flush:");
			for (TPair<FSoftObjectPath, TSharedPtr<FAvaRundownManagedInstance>>& Entry : Instances)
			{
				UE_LOGF(LogAvaRundown, Verbose, "- Asset: \"%ls\" RefCount: %d", *Entry.Key.ToString(), Entry.Value.GetSharedReferenceCount());
			}
		}
		else
		{
			UE_LOGF(LogAvaRundown, Verbose, "Managed Instance Cache: No Assets remaining in cache after flush.");
		}
	}
}

void FAvaRundownManagedInstanceCache::TrimCache()
{
	const int32 MaximumCacheSize = GetMaximumCacheSize(); 
	if (MaximumCacheSize > 0)
	{
		while (OrderQueue.Num() > MaximumCacheSize)
		{
			// LRU: oldest is at the start of the array.
			Instances.Remove(OrderQueue[0]);
			OrderQueue.RemoveAt(0);
		}
	}
}

void FAvaRundownManagedInstanceCache::FinishPendingActions()
{
	RemovePendingInvalidatedPaths();
}

TArray<TSharedPtr<FAvaRundownManagedInstance>> FAvaRundownManagedInstanceCache::GetManagedInstancesForPage(const UAvaRundown* InRundown, const FAvaRundownPage& InPage)
{
	const TArray<FSoftObjectPath> AssetPaths = InPage.GetAssetPaths(InRundown);
	
	TArray<TSharedPtr<FAvaRundownManagedInstance>> OutInstances;
	OutInstances.Reserve(AssetPaths.Num());
	
	for (const FSoftObjectPath& AssetPath : AssetPaths)
	{		
		if (TSharedPtr<FAvaRundownManagedInstance> Instance = GetOrLoadInstance(AssetPath))
		{
			OutInstances.Add(Instance);
		}
	}

	return OutInstances;
}

FAvaRundownManagedInstanceHandles FAvaRundownManagedInstanceCache::GetManagedHandlesForPage(UAvaRundown* InRundown, const FAvaRundownPage& InPage)
{
	return FAvaRundownManagedInstanceHandles(GetManagedInstancesForPage(InRundown, InPage), InRundown, InPage.GetPageId());
}

void FAvaRundownManagedInstanceCache::RemovePendingInvalidatedPaths()
{
	for (const FSoftObjectPath& InvalidatedPath : PendingInvalidatedPaths)
	{
		RemoveEntry(InvalidatedPath);
	}
	PendingInvalidatedPaths.Reset();
}

void FAvaRundownManagedInstanceCache::RemoveEntry(const FSoftObjectPath& InAssetPath)
{
	OrderQueue.Remove(InAssetPath);
	Instances.Remove(InAssetPath);
}

void FAvaRundownManagedInstanceCache::RemoveEntries(TFunctionRef<bool(const FSoftObjectPath&, const TSharedPtr<FAvaRundownManagedInstance>&)> InRemovePredicate, bool bInNotify)
{
	TArray<FSoftObjectPath> AssetsToNotify;
	
	for (TMap<FSoftObjectPath, TSharedPtr<FAvaRundownManagedInstance>>::TIterator EntryIt = Instances.CreateIterator(); EntryIt; ++EntryIt)
	{
		if (InRemovePredicate(EntryIt.Key(), EntryIt.Value()))
		{
			if (bInNotify)
			{
				AssetsToNotify.Add(EntryIt.Key());
			}
			OrderQueue.Remove(EntryIt.Key());
			EntryIt.RemoveCurrent();
		}
	}

	// Notify outside of the clean up loop since it is very likely the result of this
	// will be a reload of the asset that got invalidated.
	for (const FSoftObjectPath& AssetPath : AssetsToNotify)
	{
		OnEntryInvalidated.Broadcast(AssetPath);
	}
}

void FAvaRundownManagedInstanceCache::OnPackageSaved(const FString& InPackageFileName, UPackage* InPackage, FObjectPostSaveContext InObjectSaveContext)
{
	if (InObjectSaveContext.IsProceduralSave())
	{
		return;
	}

	OnPackageModified(InPackage->GetFName());
}

void FAvaRundownManagedInstanceCache::OnAvaSyncPackageModified(IAvaMediaSyncProvider* InAvaMediaSyncProvider, const FName& InPackageName)
{
	UE_LOGF(LogAvaMedia, Verbose,
		"A sync operation has touched the package \"%ls\" on disk. Managed Motion Design Instance Cache notified.",
		*InPackageName.ToString());

	OnPackageModified(InPackageName);
}

void FAvaRundownManagedInstanceCache::OnAssetRemoved(const FAssetData& InAssetData)
{
	// Invalidate the internal cache for the given package.
	OnPackageModified(InAssetData.PackageName);
}

void FAvaRundownManagedInstanceCache::OnPackageModified(const FName& InPackageName)
{
	// Invalidate corresponding assets from that package.
	RemoveEntries([InPackageName](const FSoftObjectPath& InAssetPath, const TSharedPtr<FAvaRundownManagedInstance>&)
		{
			if (InAssetPath.GetLongPackageFName() == InPackageName)
			{
				UE_LOGF(LogAvaMedia, Log,
					"Managed Motion Design Instance Cache: Package \"%ls\" being touched caused asset \"%ls\" to be invalidated.",
					*InPackageName.ToString(), *InAssetPath.ToString());
				return true;
			}
			return false;
		}, true);	// Notify of asset being invalidated to trigger a UI refresh.
}

void FAvaRundownManagedInstanceCache::OnSettingChanged(UObject* , struct FPropertyChangedEvent&)
{
	TrimCache();
}