// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownManagedInstanceHandle.h"

#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownManagedInstance.h"

namespace UE::AvaMedia::Rundown::Private
{
	/** Utility function to unregister the current rundown page from controller events routing. */
	void UnregisterRundownForEvents(const TSharedPtr<FAvaRundownManagedInstance>& InInstance, const UAvaRundown* InRundown, int32 InPageId)
	{
		if (InInstance && InInstance->GetRundownForEvents() == InRundown && InInstance->GetActivePageIdForEvents() == InPageId)
		{
			UE_LOGF(LogAvaRundown, Verbose, "Unregister Managed Instance %ls page id %d from RC Events routing.",
				*InInstance->GetSourceAssetPath().ToString(), InPageId);

			InInstance->SetCurrentRundownPageForEvents(nullptr, InvalidPageId);
		}
	}
	
	/** Utility function to unregister the current rundown page from controller events routing. */
	void UnregisterRundownForEvents(TConstArrayView<TSharedPtr<FAvaRundownManagedInstance>> InInstances, const UAvaRundown* InRundown, int32 InPageId)
	{
		for (const TSharedPtr<FAvaRundownManagedInstance>& Instance : InInstances)
		{
			UnregisterRundownForEvents(Instance, InRundown, InPageId);
		}
	}
}

FAvaRundownManagedInstanceHandles::FAvaRundownManagedInstanceHandles(const TArray<TSharedPtr<FAvaRundownManagedInstance>>& InInstances, UAvaRundown* InRundown, int32 InPageId)
	: Instances(InInstances)
	, RegisteredRundown(InRundown)
	, RegisteredPageId(InPageId)
{
	for (const TSharedPtr<FAvaRundownManagedInstance>& Instance : Instances)
	{
		// RC Controller events will be routed to the given rundown.
		Instance->SetCurrentRundownPageForEvents(InRundown, InPageId);
	}
}

void FAvaRundownManagedInstanceHandles::Reset()
{
	UE::AvaMedia::Rundown::Private::UnregisterRundownForEvents(Instances, RegisteredRundown.Get(), RegisteredPageId);
	Instances.Reset();
}

FAvaRundownManagedInstanceHandles& FAvaRundownManagedInstanceHandles::operator=(FAvaRundownManagedInstanceHandles&& InOther)
{
	if (&InOther == this)
	{
		return *this;
	}

	// Remove any instances that are also in the incoming set of instances to make sure
	// they don't get unregistered by the moved-from object destructor.
	for (TSharedPtr<FAvaRundownManagedInstance>& Instance : Instances)
	{
		if (Instance)
		{
			if (InOther.Instances.Contains(Instance))
			{
				Instance.Reset();
			}
			else
			{
				// Unregister instances from event routing immediately.
				UE::AvaMedia::Rundown::Private::UnregisterRundownForEvents(Instance, RegisteredRundown.Get(), RegisteredPageId);
			}
		}
	}

	// Swap the registered rundown and page id so the moved-from object destructor unregisters
	// the remaining unused instances from the previous page id.
	Swap(RegisteredRundown,InOther.RegisteredRundown);
	Swap(RegisteredPageId, InOther.RegisteredPageId);

	Instances = MoveTemp(InOther.Instances);
	return *this;
}
