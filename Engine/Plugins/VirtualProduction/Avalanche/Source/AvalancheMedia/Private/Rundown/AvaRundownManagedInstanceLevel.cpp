// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownManagedInstanceLevel.h"

#include "AssetRegistry/AssetData.h"
#include "AvaRemoteControlRebind.h"
#include "AvaRemoteControlUtils.h"
#include "AvaScene.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "IAvaMediaModule.h"
#include "Playback/AvaPlaybackUtils.h"
#include "RemoteControlPreset.h"
#include "Rundown/AvaRundownManagedInstanceUtils.h"
#include "UObject/Package.h"

namespace UE::AvaRundownManagedInstanceLevel::Private
{
	UWorld* LoadLevel(const FSoftObjectPath& InAssetPath)
	{
		return Cast<UWorld>(InAssetPath.TryLoad());
	}

	/** Loads the given source level in the given package. */
	UWorld* LoadLevelInstanceInPackage(const FSoftObjectPath& InSourceAssetPath, UPackage* InDestinationPackage)
	{
		if (!InDestinationPackage)
		{
			return nullptr;
		}
		
		const FPackagePath SourcePackagePath = FPackagePath::FromPackageNameUnchecked(InSourceAssetPath.GetLongPackageFName());
		constexpr EPackageFlags PackageFlags = PKG_ContainsMap;
		const FLinkerInstancingContext* InstancingContextPtr = nullptr;
		const FName ManagedPackageName = InDestinationPackage->GetFName();
#if WITH_EDITOR
		FLinkerInstancingContext InstancingContext;

		// When loading an instanced package we need to invoke an instancing context function in case non external actors
		// part of the level are pulling on external actors.
		const FString ExternalActorsPathStr = ULevel::GetExternalActorsPath(SourcePackagePath.GetPackageName());
		const FString DesiredPackageNameStr = ManagedPackageName.ToString();

		InstancingContext.AddPackageMappingFunc([ExternalActorsPathStr, DesiredPackageNameStr](FName Original)
		{
			const FString OriginalStr = Original.ToString();
			if (OriginalStr.StartsWith(ExternalActorsPathStr))
			{
				return FName(*ULevel::GetExternalActorPackageInstanceName(DesiredPackageNameStr, OriginalStr));
			}
			return Original;
		});

		InstancingContextPtr = &InstancingContext;
#endif

		// Since we are going to block on it, make sure it is high priority.
		constexpr int32 LoadPriority = MAX_int32;

		const int32 LocalRequestId = LoadPackageAsync(SourcePackagePath, ManagedPackageName,
			FLoadPackageAsyncDelegate(), PackageFlags, INDEX_NONE, LoadPriority, InstancingContextPtr);

		FlushAsyncLoading(LocalRequestId);

		// Workaround to destroy the Linker Load so that it does not keep the underlying File Opened
		FAvaPlaybackUtils::FlushPackageLoading(InDestinationPackage);

		return UWorld::FindWorldInPackage(InDestinationPackage);
	}

	URemoteControlPreset* FindRemoteControlPreset(ULevel* InLevel)
	{
		const AAvaScene* AvaScene = AAvaScene::GetScene(InLevel, false);
		return AvaScene ? AvaScene->GetRemoteControlPreset() : nullptr;
	}
}

FAvaRundownManagedInstanceLevel::FAvaRundownManagedInstanceLevel(FAvaRundownManagedInstanceCache* InParentCache, const FSoftObjectPath& InAssetPath)
	: FAvaRundownManagedInstance(InParentCache, InAssetPath)
{
	using namespace UE::AvaRundownManagedInstanceLevel::Private;
	
	// Register the delegates on the source RCP (if loaded) in case the level is currently being edited.
	if (UWorld* SourceLevel = Cast<UWorld>(InAssetPath.ResolveObject()))
	{
		// Keep a weak pointer
		SourceLevelWeak = SourceLevel;

		RegisterSourceRemoteControlPresetDelegates(FindRemoteControlPreset(SourceLevel->PersistentLevel));
	}

	ManagedLevelPackage = FAvaRundownManagedInstanceUtils::MakeManagedInstancePackage(InAssetPath);
	if (!ManagedLevelPackage)
	{
		UE_LOGF(LogAvaMedia, Error, "Unable to create a Managed Motion Design Level Package for %ls", *InAssetPath.ToString());
		return;
	}

	// Load a copy of the source package.
	ManagedLevel = LoadLevelInstanceInPackage(InAssetPath, ManagedLevelPackage);
	
	if (!ManagedLevel)
	{
		UE_LOGF(LogAvaMedia, Error, "Unable to duplicate Source Motion Design Level: %ls", *InAssetPath.ToString());
		return;
	}

	ManagedLevel->SetFlags(RF_Public | RF_Transient);
	
	ManagedRemoteControlPreset = FindRemoteControlPreset(ManagedLevel->PersistentLevel);
	FAvaRemoteControlRebind::RebindUnboundEntities(ManagedRemoteControlPreset, ManagedLevel->PersistentLevel);
	RegisterManagedRemoteControlPresetDelegates(ManagedRemoteControlPreset);
	
	// Backup the remote control values from the source asset.
	constexpr bool bIsDefault = true;	// Flag the values as "default".
	DefaultRemoteControlValues.CopyFrom(ManagedRemoteControlPreset, bIsDefault);

	FAvaRundownManagedInstanceUtils::PreventWorldFromBeingSeenAsLeakingByLevelEditor(ManagedLevel.Get());

	IAvaMediaModule::Get().GetOnMapChangedEvent().AddRaw(this, &FAvaRundownManagedInstanceLevel::OnMapChangedEvent);
}

FAvaRundownManagedInstanceLevel::~FAvaRundownManagedInstanceLevel()
{
	IAvaMediaModule::Get().GetOnMapChangedEvent().RemoveAll(this);

	if (ManagedRemoteControlPreset)
	{
		FAvaRemoteControlUtils::UnregisterRemoteControlPreset(ManagedRemoteControlPreset);
		UnregisterManagedRemoteControlPresetDelegates(ManagedRemoteControlPreset);
	}

	if (ManagedLevelPackage)
	{
		ManagedLevelPackage->ClearDirtyFlag();
	}
	
	ManagedLevel = nullptr;
	ManagedLevelPackage = nullptr;
	ManagedRemoteControlPreset = nullptr;
	DiscardSourceLevel();
}

void FAvaRundownManagedInstanceLevel::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject( ManagedLevel );
	Collector.AddReferencedObject( ManagedLevelPackage );
	Collector.AddReferencedObject( ManagedRemoteControlPreset );
}

FString FAvaRundownManagedInstanceLevel::GetReferencerName() const
{
	return TEXT("FAvaRundownManagedInstanceLevel");
}

IAvaSceneInterface* FAvaRundownManagedInstanceLevel::GetSceneInterface() const
{
	return ManagedLevel ? static_cast<IAvaSceneInterface*>(AAvaScene::GetScene(ManagedLevel->PersistentLevel, false)) : nullptr;
}

void FAvaRundownManagedInstanceLevel::DiscardSourceLevel()
{
	if (const UWorld* SourceLevel = SourceLevelWeak.Get())
	{
		using namespace UE::AvaRundownManagedInstanceLevel::Private;
		UnregisterSourceRemoteControlPresetDelegates(FindRemoteControlPreset(SourceLevel->PersistentLevel));
	}
	SourceLevelWeak.Reset();
}

void FAvaRundownManagedInstanceLevel::OnMapChangedEvent(UWorld* InWorld, EAvaMediaMapChangeType InEventType)
{
	if (!InWorld->GetPackage())
	{
		return;
	}

	if (SourceAssetPath.GetLongPackageFName() != InWorld->GetPackage()->GetFName())
	{
		return;
	}
	
	if (InEventType == EAvaMediaMapChangeType::LoadMap)
	{
		using namespace UE::AvaRundownManagedInstanceLevel::Private;
		// This should be fast given the level has been loaded in the editor.
		if (UWorld* SourceLevel = LoadLevel(SourceAssetPath))
		{
			SourceLevelWeak = SourceLevel;
			RegisterSourceRemoteControlPresetDelegates(FindRemoteControlPreset(SourceLevel->PersistentLevel));
		}
	}
	else if (InEventType == EAvaMediaMapChangeType::TearDownWorld)
	{
		DiscardSourceLevel();
	}
}