// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/World.h"
#include "Rundown/AvaRundownManagedInstance.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"

enum class EAvaMediaMapChangeType : uint8;

class FAvaRundownManagedInstanceLevel : public FAvaRundownManagedInstance
{
public:
	UE_AVA_INHERITS(FAvaRundownManagedInstanceLevel, FAvaRundownManagedInstance);

	FAvaRundownManagedInstanceLevel(FAvaRundownManagedInstanceCache* InParentCache, const FSoftObjectPath& InAssetPath);
	virtual ~FAvaRundownManagedInstanceLevel() override;

	//~ Begin FGCObject
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override;
	//~ End FGCObject
	
	//~ Begin FAvaRundownManagedInstance
	virtual bool IsValid() const override { return ManagedLevel != nullptr;}
	virtual URemoteControlPreset* GetRemoteControlPreset() const override { return ManagedRemoteControlPreset;}
	virtual IAvaSceneInterface* GetSceneInterface() const override;
	//~ End FAvaRundownManagedInstance

private:
	void DiscardSourceLevel();
	void OnMapChangedEvent(UWorld* InWorld, EAvaMediaMapChangeType InEventType);

private:
	/** Keep a weak pointer to the loaded source level. This is used to unregister delegates if needed. */
	TWeakObjectPtr<UWorld> SourceLevelWeak;
	
	TObjectPtr<UWorld> ManagedLevel;
	TObjectPtr<UPackage> ManagedLevelPackage;
	TObjectPtr<URemoteControlPreset> ManagedRemoteControlPreset;
};