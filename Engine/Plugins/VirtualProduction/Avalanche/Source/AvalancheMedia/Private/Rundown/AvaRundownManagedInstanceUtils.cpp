// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownManagedInstanceUtils.h"

#include "Engine/Engine.h"
#include "Engine/World.h"
#include "IAvaMediaModule.h"
#include "UObject/Package.h"

UPackage* FAvaRundownManagedInstanceUtils::MakeManagedInstancePackage(const FSoftObjectPath& InAssetPath)
{
	// Remote Control Preset will be registered with the package name.
	// We want a package name unique to the game instance and that will be
	// human readable since it will show up in the Web Remote Control page.

	FString InstancePackageName = TEXT("/Temp/Managed");	// Keep it short for web page.

	// In order to keep things short, we remove "/Game".
	FString InstanceSubPath;
	if (!InAssetPath.GetLongPackageName().Split(TEXT("/Game"), nullptr, &InstanceSubPath))
	{
		InstanceSubPath = InAssetPath.GetLongPackageName();
	}
	InstancePackageName += InstanceSubPath;

	UPackage* ManagedInstancePackage = nullptr;

	// Make sure the package name is unique. We don't want stomping of existing instances.
	if (FindObject<UPackage>( nullptr, *InstancePackageName))
	{
		FString UniqueInstancePackageName = MakeUniqueObjectName( nullptr, UPackage::StaticClass(), FName(*InstancePackageName)).ToString();
		
		UE_LOGF(LogAvaMedia, Warning, 
			"Package \"%ls\" for Motion Design Managed Instance already exists. Will use \"%ls\" instead.",
			*InstancePackageName, *UniqueInstancePackageName);

		InstancePackageName = UniqueInstancePackageName;
	}
	
	ManagedInstancePackage = CreatePackage(*InstancePackageName);
	if (ManagedInstancePackage)
	{
		ManagedInstancePackage->SetFlags(RF_Transient | RF_Public);
	}
	else
	{
		// Note: The outer will fallback to GEngine in that case.
		UE_LOGF(LogAvaMedia, Error, 
			"Unable to create package \"%ls\" for Motion Design Managed Instance.",
			*InstancePackageName);
	}
	return ManagedInstancePackage;
}

void FAvaRundownManagedInstanceUtils::PreventWorldFromBeingSeenAsLeakingByLevelEditor(UWorld* InWorld)
{
	// Appease the level editor clean up when it sees worlds, it must
	// not consider it as a leaking world.
	InWorld->WorldType = EWorldType::Inactive;

	if (UPackage* Package = InWorld->GetPackage())
	{
		// Mark package as transient. Prevents SaveDirtyPackages from trying to save this package.
		Package->SetFlags(RF_Transient);

#if WITH_EDITOR
		// Prevents "unloading" of map package by marking it as new.
		Package->MarkAsNewlyCreated();
#endif
	
		// Prevents "unload" to try to save it (just in case).
		Package->ClearDirtyFlag();
	}
}
