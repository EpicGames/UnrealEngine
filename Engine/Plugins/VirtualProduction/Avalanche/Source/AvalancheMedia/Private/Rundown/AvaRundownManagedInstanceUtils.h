// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/StrongObjectPtr.h"

class UPackage;
class UWorld;
struct FSoftObjectPath;

/** Common utility functions for implementing Managed Instance. */ 
class FAvaRundownManagedInstanceUtils
{
public:
	static UPackage* MakeManagedInstancePackage(const FSoftObjectPath& InAssetPath);
	static void PreventWorldFromBeingSeenAsLeakingByLevelEditor(UWorld* InWorld);
};