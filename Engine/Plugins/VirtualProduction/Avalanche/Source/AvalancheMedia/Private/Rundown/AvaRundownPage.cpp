// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownPage.h"

#include "AssetRegistry/AssetData.h"
#include "AvaScene.h"
#include "AvaTransitionTree.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "Broadcast/AvaBroadcast.h"
#include "IAvaMediaModule.h"
#include "Playable/AvaPlayableRemoteControl.h"
#include "Playback/AvaPlaybackClient.h"
#include "Playback/AvaPlaybackManager.h"
#include "RCVirtualProperty.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownManagedInstance.h"
#include "Rundown/AvaRundownManagedInstanceCache.h"
#include "Rundown/AvaRundownPageAssetUtils.h"
#include "Rundown/AvaRundownPageCommand.h"
#include "Rundown/AvaRundownPagePlayer.h"
#include "Rundown/AvaRundownSerializationUtils.h"

#define LOCTEXT_NAMESPACE "AvaRundownPage"

FAvaRundownPage FAvaRundownPage::NullPage;
const int32 FAvaRundownPage::InvalidPageId = -1;
static_assert(FAvaRundownPage::InvalidPageId == static_cast<int32>(UE::AvaMedia::Rundown::InvalidPageId), "Mismatching InvalidPageId definitions.");

FAvaRundownPage::FAvaRundownPage(int32 InPageId, int32 InTemplateId)
	: PageId(InPageId)
	, TemplateId(InTemplateId)
	, PageName(TEXT("New Page"))
{
}

bool FAvaRundownPage::IsValidPage() const
{
	return this != &FAvaRundownPage::NullPage
		&& PageId != InvalidPageId;
}

void FAvaRundownPage::Rename(const FString& InNewName)
{
	PageName = InNewName;
}

void FAvaRundownPage::RenameFriendlyName(const FString& InNewName)
{
	FriendlyName = FText::FromString(InNewName);
}

namespace UE::AvaRundownPage::Private
{
	// Determines the page status for a given playback status and secondary states.
	FAvaRundownChannelPageStatus GetPageStatus(EAvaBroadcastChannelType InType, EAvaPlaybackStatus InPlaybackStatus, bool bInPagePlaying, bool bInAssetNeedSync)
	{
		switch (InPlaybackStatus)
		{
		case EAvaPlaybackStatus::Unknown:
			return {InType,!bInAssetNeedSync ? EAvaRundownPageStatus::Unknown : EAvaRundownPageStatus::NeedsSync, bInAssetNeedSync};
		case EAvaPlaybackStatus::Missing:			
			return {InType, EAvaRundownPageStatus::Missing, false};
		case EAvaPlaybackStatus::Syncing:
			return {InType, EAvaRundownPageStatus::Syncing, false};
		case EAvaPlaybackStatus::Available:
			// There is an explicit "needs sync" page status, along with the flag. This is just to make it more explicit.
			return {InType, !bInAssetNeedSync ? EAvaRundownPageStatus::Available : EAvaRundownPageStatus::NeedsSync, bInAssetNeedSync};
		case EAvaPlaybackStatus::Loading:
			return {InType, EAvaRundownPageStatus::Loading, bInAssetNeedSync};
		case EAvaPlaybackStatus::Loaded:
			return {InType, EAvaRundownPageStatus::Loaded, bInAssetNeedSync};
		case EAvaPlaybackStatus::Starting:
			return {InType, EAvaRundownPageStatus::Loading, bInAssetNeedSync};
		case EAvaPlaybackStatus::Started:
			return {InType, bInPagePlaying ? EAvaRundownPageStatus::Playing : EAvaRundownPageStatus::Loaded, bInAssetNeedSync};
		case EAvaPlaybackStatus::Stopping:
		case EAvaPlaybackStatus::Unloading:
			return {InType, EAvaRundownPageStatus::Available, bInAssetNeedSync};
		case EAvaPlaybackStatus::Error:
		default:
			return {InType, EAvaRundownPageStatus::Error, bInAssetNeedSync};
		}
	}

	FAvaRundownChannelPageStatus GetProgramPageStatus(EAvaPlaybackStatus InPlaybackStatus, bool bInPagePlaying, bool bInAssetNeedSync)
	{
		return GetPageStatus(EAvaBroadcastChannelType::Program, InPlaybackStatus, bInPagePlaying, bInAssetNeedSync);
	}
	
	EAvaPlaybackStatus GetPlaybackStatus(EAvaPlaybackAssetStatus InAssetStatus)
	{
		switch (InAssetStatus)
		{
		case EAvaPlaybackAssetStatus::Unknown:
			return EAvaPlaybackStatus::Unknown;
		case EAvaPlaybackAssetStatus::Missing:
			return EAvaPlaybackStatus::Missing;
		case EAvaPlaybackAssetStatus::MissingDependencies:
		case EAvaPlaybackAssetStatus::NeedsSync:
		case EAvaPlaybackAssetStatus::Available:
			return EAvaPlaybackStatus::Available;
		default:
			return EAvaPlaybackStatus::Unknown;
		}
	}
}


FText FAvaRundownPage::GetPageDescription() const
{
	if (HasPageFriendlyName())
	{
		return FriendlyName;
	}

	if (HasPageSummary())
	{
		return PageSummary;
	}

	return FText::FromString(PageName);
}

bool FAvaRundownPage::UpdatePageSummary(const UAvaRundown* InRundown)
{
	const bool bShouldUpdateDescription = !HasPageSummary();
	if (!IsValidPage() || IsTemplate() || !bShouldUpdateDescription)
	{
		return false;
	}

	const TArray<FSoftObjectPath> AvaAssetPaths = GetAssetPaths(InRundown);
	TArray<const URemoteControlPreset*> Presets;
	Presets.Reserve(AvaAssetPaths.Num());

	FAvaRundownManagedInstanceCache& ManagedInstanceCache = IAvaMediaModule::Get().GetManagedInstanceCache();
	
	for (const FSoftObjectPath& AvaAssetPath : AvaAssetPaths)
	{
		if (const TSharedPtr<FAvaRundownManagedInstance> ManagedInstance = ManagedInstanceCache.GetOrLoadInstance(AvaAssetPath))
		{
			if (const URemoteControlPreset* Preset = ManagedInstance->GetRemoteControlPreset())
			{
				Presets.Add(Preset);
			}
		}
	}
	
	return UpdatePageSummary(InRundown, Presets, /*bInForceUpdate*/ false);
}

bool FAvaRundownPage::UpdatePageSummary(const TArray<const URemoteControlPreset*>& InPresets, bool bInIsPresetChanged)
{
	return UpdatePageSummary(nullptr, InPresets, bInIsPresetChanged);
}

bool FAvaRundownPage::UpdatePageSummary(const UAvaRundown* InRundown, const TArray<const URemoteControlPreset*>& InPresets, bool bInForceUpdate)
{
	const bool bShouldUpdateDescription = bInForceUpdate ? true : !HasPageSummary();
	if (!IsValidPage() || IsTemplate() || (InPresets.IsEmpty() && !HasCommands(InRundown)) || !bShouldUpdateDescription)
	{
		return false;
	}
	
	TSortedMap<int32, FText> OrderedDescription;

	int32 DisplayIndexOffset = 0;

	for (const URemoteControlPreset* Preset : InPresets)
	{
		for (const URCVirtualPropertyBase* VirtualProperty : Preset->GetControllers())
		{
			if (!FAvaPlayableRemoteControlValues::ShouldIgnoreController(VirtualProperty))
			{
				FString StringValue = VirtualProperty->GetDisplayValueAsString();
				const FText TextToAdd = FText::FromString(StringValue);
				if (!TextToAdd.IsEmptyOrWhitespace())
				{
					OrderedDescription.FindOrAdd(VirtualProperty->DisplayIndex + DisplayIndexOffset) = TextToAdd;
				}
			}
		}
		DisplayIndexOffset += Preset->GetNumControllers();
	}

	// Also include commands in the page summary.
	if (InRundown)
	{
		ForEachInstancedCommands([&OrderedDescription, &DisplayIndexOffset](const FAvaRundownPageCommand& InCommand, const FAvaRundownPage&)
		{
			OrderedDescription.FindOrAdd(DisplayIndexOffset) = InCommand.GetDescription(); 
			++DisplayIndexOffset;
		}, InRundown, /*bInDirectOnly*/ false); // Traverse templates
	}
	
	if (!OrderedDescription.IsEmpty())
	{
		FFormatOrderedArguments InitialDescription;
		InitialDescription.Reserve(OrderedDescription.Num());

		constexpr int32 FinishDescriptionWhenExceedLimit = 50;
		int32 CurrentLength = 0;
		for (const TPair<int32, FText>& OrderedValue : OrderedDescription)
		{
			InitialDescription.Add(OrderedValue.Value);
			CurrentLength += OrderedValue.Value.ToString().Len();
			if (CurrentLength >= FinishDescriptionWhenExceedLimit)
			{
				InitialDescription.Add(LOCTEXT("EndOfDescription", "..."));
				break;
			}
		}

		PageSummary = FText::Join(LOCTEXT("DescriptionDelimiter", " / "), InitialDescription);
		return true;
	}

	return false;
}

bool FAvaRundownPage::UpdateTransitionLogic()
{
	check(IsTemplate());
	if (const UObject* LoadedSourceAsset = AssetPath.TryLoad())
	{
		if (const IAvaSceneInterface* SceneInterface = FAvaRundownPageAssetUtils::GetSceneInterface(LoadedSourceAsset))
		{
			const IAvaTransitionBehavior* TransitionBehavior = FAvaRundownPageAssetUtils::FindTransitionBehavior(SceneInterface);
			bHasTransitionLogic = TransitionBehavior ? TransitionBehavior->IsEnabled() : false;
			TransitionLayerTag = TransitionBehavior ? TransitionBehavior->GetTransitionLayer() : FAvaTagHandle();
			TransitionMode = TransitionBehavior ? TransitionBehavior->GetInstancingMode() : EAvaTransitionInstancingMode::New;
		}
	}
	return false;
}

bool FAvaRundownPage::HasTransitionLogic(const UAvaRundown* InRundown) const
{	
	const FAvaRundownPage& Template = ResolveTemplate(InRundown);
	if (Template.IsValidPage())
	{
		// Combo templates have Transition Logic.
		return Template.IsComboTemplate() || Template.bHasTransitionLogic;
	}

	return bHasTransitionLogic;
}

FAvaTagHandle FAvaRundownPage::GetTransitionLayer(const UAvaRundown* InRundown, int32 InTemplateIndex) const
{
	const FAvaRundownPage& Template = GetTemplate(InRundown, InTemplateIndex);
	return Template.IsValidPage() ?  Template.TransitionLayerTag : TransitionLayerTag;
}

TArray<FAvaTagHandle> FAvaRundownPage::GetTransitionLayers(const UAvaRundown* InRundown) const
{
	TArray<FAvaTagHandle> TransitionLayers;

	const int32 NumTemplates = GetNumTemplates(InRundown);
	TransitionLayers.Reserve(NumTemplates);

	for(int32 TemplateIndex = 0; TemplateIndex < NumTemplates; ++TemplateIndex)
	{
		const FAvaRundownPage& Template = GetTemplate(InRundown, TemplateIndex);
		if (Template.IsValidPage())
		{
			TransitionLayers.Add(Template.TransitionLayerTag);
		}
	}

	return TransitionLayers;
}

EAvaTransitionInstancingMode FAvaRundownPage::GetTransitionMode( const UAvaRundown* InRundown, int32 InTemplateIndex) const
{
	const FAvaRundownPage& Template = GetTemplate(InRundown, InTemplateIndex);
	return Template.IsValidPage() ?  Template.TransitionMode : TransitionMode;
}

TArray<EAvaTransitionInstancingMode> FAvaRundownPage::GetTransitionModes(const UAvaRundown* InRundown) const
{
	TArray<EAvaTransitionInstancingMode> TransitionModes;

	const int32 NumTemplates = GetNumTemplates(InRundown);
	TransitionModes.Reserve(NumTemplates);

	for(int32 TemplateIndex = 0; TemplateIndex < NumTemplates; ++TemplateIndex)
	{
		const FAvaRundownPage& Template = GetTemplate(InRundown, TemplateIndex);
		if (Template.IsValidPage())
		{
			TransitionModes.Add(Template.TransitionMode);
		}
	}

	return TransitionModes;
}

int32 FAvaRundownPage::AppendPageProgramStatuses(const UAvaRundown* InParentRundown, TArray<FAvaRundownChannelPageStatus>& OutPageStatuses) const
{
	if (!InParentRundown || IsTemplate())
	{
		return 0;
	}
	
	const int32 PreviousNumStatuses = OutPageStatuses.Num();

	// Special bypass for command-only page
	if (!HasAssets(*InParentRundown) && HasCommands(InParentRundown))
	{
		// todo: remote status?
		OutPageStatuses.Add({EAvaBroadcastChannelType::Program, EAvaRundownPageStatus::Available, /*bNeedsSync*/ false});
		return OutPageStatuses.Num() - PreviousNumStatuses;
	}
	
	IAvaMediaModule& AvaMediaModule = IAvaMediaModule::Get();
	FAvaPlaybackManager& PlaybackManager = InParentRundown->GetPlaybackManager();
	const FSoftObjectPath ResolvedAssetPath = GetAssetPath(InParentRundown);
	const FString ChannelName = GetChannelName().ToString();
	const UAvaRundownPagePlayer* ProgramPagePlayer = InParentRundown->FindPlayerForProgramPage(GetPageId());
	const bool bIsPagePlaying = ProgramPagePlayer ? ProgramPagePlayer->IsPlaying() : false;
	const FAvaPlaybackInstance* LocalPlaybackInstance = ProgramPagePlayer ? ProgramPagePlayer->GetPlaybackInstance() : nullptr;
	const FGuid LocalPlaybackInstanceId = LocalPlaybackInstance ? LocalPlaybackInstance->GetInstanceId() : FGuid();
	
	bool bLocalStatusAdded = false;

	auto AddLocalProgramStatusOnce = [LocalPlaybackInstance, &PlaybackManager, &bLocalStatusAdded, &OutPageStatuses, bIsPagePlaying, &ResolvedAssetPath, &ChannelName]()
	{
		using namespace UE::AvaRundownPage::Private;

		// Local status is added only once.
		if (bLocalStatusAdded)
		{
			return;
		}
		
		// if page is playing, access the status from the page player's playback instance.
		if (LocalPlaybackInstance)
		{
			OutPageStatuses.Add(GetProgramPageStatus(LocalPlaybackInstance->GetStatus(), bIsPagePlaying, false));
		}
		else
		{
			// If the page is not playing, try to find a cached playback instance of the asset for the given channel.
			constexpr FGuid InvalidId;	// Providing an invalid instance id will fallback to searching by channel. 
			if (const TSharedPtr<FAvaPlaybackInstance> PlaybackInstance = PlaybackManager.FindPlaybackInstance(InvalidId, ResolvedAssetPath, ChannelName))
			{
				OutPageStatuses.Add(GetProgramPageStatus(PlaybackInstance->GetStatus(), bIsPagePlaying, false));
			}
			else
			{
				OutPageStatuses.Add(GetProgramPageStatus(PlaybackManager.GetUnloadedPlaybackStatus(ResolvedAssetPath), bIsPagePlaying, false));
			}
		}
		bLocalStatusAdded = true;
	};
	
	const FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetChannel(GetChannelName());
	const TArray<UMediaOutput*>& Outputs = Channel.GetMediaOutputs();
	IAvaPlaybackClient& PlaybackClient = AvaMediaModule.GetPlaybackClient();

	TSet<FString> AddedServers;

	for (const UMediaOutput* Output : Outputs)
	{
		if (Channel.IsMediaOutputRemote(Output))
		{
			if (Channel.GetMediaOutputState(Output) != EAvaBroadcastOutputState::Offline)
			{
				const FString& ServerForOutput = Channel.GetMediaOutputServerName(Output);

				if (AddedServers.Contains(ServerForOutput))
				{
					continue; // We have already added that server.
				}

				AddedServers.Add(ServerForOutput);	// Keep track of our servers so we add it only once.

				// There is only one playback/asset status per server, even if it has many outputs.
				TOptional<EAvaPlaybackStatus> PlaybackStatus = PlaybackClient.GetRemotePlaybackStatus(LocalPlaybackInstanceId, ResolvedAssetPath, ChannelName, ServerForOutput);
				const TOptional<EAvaPlaybackAssetStatus> PlaybackAssetStatus = PlaybackClient.GetRemotePlaybackAssetStatus(ResolvedAssetPath, ServerForOutput);

				if (!PlaybackAssetStatus.IsSet())
				{
					PlaybackClient.RequestPlaybackAssetStatus(ResolvedAssetPath, ServerForOutput, false);
				}

				using namespace UE::AvaRundownPage::Private;
				if (!PlaybackStatus.IsSet())
				{
					PlaybackClient.RequestPlayback(LocalPlaybackInstanceId, ResolvedAssetPath, ChannelName, EAvaPlaybackAction::Status);
					// Derive playback status from the asset status.
					PlaybackStatus = PlaybackAssetStatus.IsSet() ? GetPlaybackStatus(PlaybackAssetStatus.GetValue()) : EAvaPlaybackStatus::Unknown;
				}

				const bool bAssetNeedsSync = PlaybackAssetStatus.IsSet() && PlaybackAssetStatus.GetValue() == EAvaPlaybackAssetStatus::NeedsSync;
				OutPageStatuses.Add(GetProgramPageStatus(PlaybackStatus.GetValue(), bIsPagePlaying, bAssetNeedsSync));
			}
			else
			{
				OutPageStatuses.Add({EAvaBroadcastChannelType::Program, EAvaRundownPageStatus::Offline, false});
			}
		}
		else
		{
			// All local outputs lead to only one status, the local one.
			AddLocalProgramStatusOnce();
		}
	}

	// If there are no outputs defined, use the local status.
	if (Outputs.IsEmpty())
	{
		AddLocalProgramStatusOnce();
	}
	
	return OutPageStatuses.Num() - PreviousNumStatuses;
}

int32 FAvaRundownPage::AppendPagePreviewStatuses(const UAvaRundown* InParentRundown, const FName& InPreviewChannelName, TArray<FAvaRundownChannelPageStatus>& OutPageStatuses) const
{
	if (!InParentRundown)
	{
		return 0;
	}
	
	const int32 PreviousNumStatuses = OutPageStatuses.Num();

	// Special bypass for command-only page
	if (!HasAssets(*InParentRundown) && HasCommands(InParentRundown))
	{
		OutPageStatuses.Add({EAvaBroadcastChannelType::Program, EAvaRundownPageStatus::Available, /*bNeedsSync*/ false});
		return OutPageStatuses.Num() - PreviousNumStatuses;
	}

	FAvaPlaybackManager& PlaybackManager = InParentRundown->GetPlaybackManager();
	const FSoftObjectPath ResolvedAssetPath = GetAssetPath(InParentRundown);

	// For the preview, we only add the status if it is playing.
	// We are not really interested if the preview is loaded.
	if (InParentRundown->IsPagePreviewing(PageId))
	{
		OutPageStatuses.Add({EAvaBroadcastChannelType::Preview, EAvaRundownPageStatus::Previewing, false});
	}
	else
	{
		using namespace UE::AvaRundownPage::Private;
		const FString PreviewChannelName = InPreviewChannelName.IsNone() ? InParentRundown->GetDefaultPreviewChannelName().ToString() : InPreviewChannelName.ToString();

		// If the page is not previewing, try to find a cached playback instance of the asset for the given preview channel.
		constexpr FGuid InvalidId;	// Providing an invalid instance id will fallback to searching by channel. 
		if (const TSharedPtr<FAvaPlaybackInstance> PlaybackInstance = PlaybackManager.FindPlaybackInstance(InvalidId, ResolvedAssetPath, PreviewChannelName))
		{
			OutPageStatuses.Add(GetPageStatus(EAvaBroadcastChannelType::Preview, PlaybackInstance->GetStatus(), false, false));
		}
		else
		{
			OutPageStatuses.Add(GetPageStatus(EAvaBroadcastChannelType::Preview, PlaybackManager.GetUnloadedPlaybackStatus(ResolvedAssetPath), false, false));
		}
	}

	return OutPageStatuses.Num() - PreviousNumStatuses;
}

TArray<FAvaRundownChannelPageStatus> FAvaRundownPage::GetPageStatuses(const UAvaRundown* InParentRundown) const
{
	TArray<FAvaRundownChannelPageStatus> Statuses;
	if (InParentRundown)
	{
		// Typical instanced page: 1 program channel + 1 preview channel.
		// Template page only has a preview channel.
		Statuses.Reserve(IsTemplate() ? 1 : 2);
		
		if (!IsTemplate())
		{
			AppendPageProgramStatuses(InParentRundown, Statuses);
		}
		AppendPagePreviewStatuses(InParentRundown, InParentRundown->GetDefaultPreviewChannelName(), Statuses);
	}
	return Statuses;
}

TArray<FAvaRundownChannelPageStatus> FAvaRundownPage::GetPageContextualStatuses(const UAvaRundown* InParentRundown) const
{
	TArray<FAvaRundownChannelPageStatus> Statuses;
	if (InParentRundown)
	{
		Statuses.Reserve(1);
		
		if (!IsTemplate())
		{
			AppendPageProgramStatuses(InParentRundown, Statuses);
		}
		else
		{
			AppendPagePreviewStatuses(InParentRundown, InParentRundown->GetDefaultPreviewChannelName(), Statuses);
		}
	}
	return Statuses;
}

TArray<FAvaRundownChannelPageStatus> FAvaRundownPage::GetPageProgramStatuses(const UAvaRundown* InParentRundown) const
{
	TArray<FAvaRundownChannelPageStatus> Statuses;
	if (InParentRundown && !IsTemplate())
	{
		Statuses.Reserve(1);
		AppendPageProgramStatuses(InParentRundown, Statuses);
	}
	return Statuses;
}

TArray<FAvaRundownChannelPageStatus> FAvaRundownPage::GetPagePreviewStatuses(const UAvaRundown* InParentRundown, const FName& InPreviewChannelName) const
{
	TArray<FAvaRundownChannelPageStatus> Statuses;
	if (InParentRundown)
	{
		Statuses.Reserve(1);
		AppendPagePreviewStatuses(InParentRundown, InPreviewChannelName, Statuses);
	}
	return Statuses;
}

FSoftObjectPath FAvaRundownPage::GetAssetPath(const UAvaRundown* InRundown, int32 InTemplateIndex) const
{
	const FAvaRundownPage& Template = GetTemplate(InRundown, InTemplateIndex);
	return Template.IsValidPage() ? Template.AssetPath : AssetPath;
}

TArray<FSoftObjectPath> FAvaRundownPage::GetAssetPaths(const UAvaRundown* InRundown) const
{
	TArray<FSoftObjectPath> AssetPaths;

	const int32 NumTemplates = GetNumTemplates(InRundown);
	AssetPaths.Reserve(NumTemplates);

	for(int32 TemplateIndex = 0; TemplateIndex < NumTemplates; ++TemplateIndex)
	{
		const FAvaRundownPage& Template = GetTemplate(InRundown, TemplateIndex);
		if (Template.IsValidPage())
		{
			AssetPaths.Add(Template.AssetPath);
		}
	}

	return AssetPaths;
}

bool FAvaRundownPage::HasAssets(const UAvaRundown& InRundown) const
{
	const int32 NumTemplates = GetNumTemplates(&InRundown);
	for(int32 TemplateIndex = 0; TemplateIndex < NumTemplates; ++TemplateIndex)
	{
		const FAvaRundownPage& Template = GetTemplate(&InRundown, TemplateIndex);
		if (Template.IsValidPage() && !Template.AssetPath.IsNull())
		{
			return true;
		}
	}
	return false;
}

bool FAvaRundownPage::UpdateAsset(const FSoftObjectPath& InAssetPath, bool bInReimportPage)
{
	if (IsComboTemplate())
	{
		UE_LOGF(LogAvaRundown, Error, "Can't update asset on a combo page directly.");
		return false;
	}
	
	if (InAssetPath != AssetPath || bInReimportPage)
	{
		AssetPath = InAssetPath;

		if (!HasPageFriendlyName())
		{
			// Is GetName() the same as the asset name from soft object path?
			PageName = InAssetPath.GetAssetName();
		}

		UpdateTransitionLogic();
		return true;
	}
	return false;
}

bool FAvaRundownPage::HasCommands(const UAvaRundown* InRundown) const
{
	// Don't count empty commands.
	auto HasValidCommands = [](const TArray<FInstancedStruct>& InCommands) -> bool
	{
		for (const FInstancedStruct& Command : InCommands)
		{
			if (Command.GetPtr<FAvaRundownPageCommand>())
			{
				return true;
			}
		}
		return false;
	};

	// Direct test (either instanced page or direct template)
	if (HasValidCommands(InstancedCommands))
	{
		return true;
	}
	
	const FAvaRundownPage& DirectTemplate = ResolveTemplate(InRundown);
	if (DirectTemplate.IsValidPage())
	{
		if (!IsTemplate()	// Already checked if this is a template.
			&& HasValidCommands(DirectTemplate.InstancedCommands))
		{
			return true;
		}

		// Sub-template commands - Remark: for now, currently only do 1 level of recursion.
		for (const int32 SubTemplateId : DirectTemplate.GetCombinedTemplateIds())
		{
			const FAvaRundownPage& SubTemplate = InRundown->GetPage(SubTemplateId);
			if (SubTemplate.IsValidPage())
			{
				if (HasValidCommands(SubTemplate.InstancedCommands))
				{
					return true;
				}
			}
		}
	}
	return false;
}

FName FAvaRundownPage::GetChannelName() const
{
	return UAvaBroadcast::Get().GetChannelName(OutputChannel);
}

void FAvaRundownPage::SetChannelName(FName InChannelName)
{
	OutputChannel = UAvaBroadcast::Get().GetChannelIndex(InChannelName);
	if (OutputChannel == INDEX_NONE)
	{
		UE_LOGF(LogAvaBroadcast, Error, "Channel %ls was not found in broadcast channels, using %ls instead.",
			*InChannelName.ToString(), *UAvaBroadcast::Get().GetChannelName(0).ToString());
		OutputChannel = 0;
	}
}

EAvaPlayableRemoteControlChanges FAvaRundownPage::PruneRemoteControlValues(const FAvaPlayableRemoteControlValues& InRemoteControlValues)
{
	return RemoteControlValues.PruneRemoteControlValues(InRemoteControlValues);
}

EAvaPlayableRemoteControlChanges FAvaRundownPage::UpdateRemoteControlValues(const FAvaPlayableRemoteControlValues& InRemoteControlValues, bool bInUpdateDefaults)
{
	return RemoteControlValues.UpdateRemoteControlValues(InRemoteControlValues, bInUpdateDefaults);
}

void FAvaRundownPage::SetRemoteControlEntityValue(const FGuid& InId, const FAvaPlayableRemoteControlValue& InValue)
{
	RemoteControlValues.SetEntityValue(InId, InValue);
}

void FAvaRundownPage::SetRemoteControlControllerValue(const FGuid& InId, const FAvaPlayableRemoteControlValue& InValue)
{
	RemoteControlValues.SetControllerValue(InId, InValue);
}

bool FAvaRundownPage::GetDefaultRemoteControlValues(const UAvaRundown* InRundown, bool bInUseTemplateValues, FAvaPlayableRemoteControlValues& OutValues) const
{
	if (bInUseTemplateValues)
	{
		if (!IsTemplate())
		{
			const FAvaRundownPage& PageTemplate = ResolveTemplate(InRundown);
			if (PageTemplate.IsValidPage())
			{
				OutValues = PageTemplate.GetRemoteControlValues();
				return true;
			}
		}

		return false;
	}

	FAvaRundownManagedInstanceCache& ManagedInstanceCache = IAvaMediaModule::Get().GetManagedInstanceCache();

	for (const FSoftObjectPath& PageAssetPath : GetAssetPaths(InRundown))
	{
		const TSharedPtr<FAvaRundownManagedInstance> ManagedInstance = ManagedInstanceCache.GetOrLoadInstance(PageAssetPath);
		if (!ManagedInstance)
		{
			return false;
		}

		OutValues.Merge(ManagedInstance->GetDefaultRemoteControlValues());
	}

	return true;
}

bool FAvaRundownPage::GetDefaultEntityValue(const UAvaRundown* InRundown, const FGuid& InId, bool bInUseTemplateValues, FAvaPlayableRemoteControlValue& OutValue) const
{
	if (bInUseTemplateValues)
	{
		if (!IsTemplate())
		{
			const FAvaRundownPage& PageTemplate = ResolveTemplate(InRundown);
			if (PageTemplate.IsValidPage())
			{
				const FAvaPlayableRemoteControlValues& PageValues = PageTemplate.GetRemoteControlValues();
				if (PageValues.HasEntityValue(InId))
				{
					OutValue = *PageValues.GetEntityValue(InId);
					return true;
				}
			}
		}

		return false;
	}

	FAvaRundownManagedInstanceCache& ManagedInstanceCache = IAvaMediaModule::Get().GetManagedInstanceCache();

	for (const FSoftObjectPath& PageAssetPath : GetAssetPaths(InRundown))
	{
		const TSharedPtr<FAvaRundownManagedInstance> ManagedInstance = ManagedInstanceCache.GetOrLoadInstance(PageAssetPath);
		if (!ManagedInstance)
		{
			return false;
		}

		const FAvaPlayableRemoteControlValues& DefaultValues = ManagedInstance->GetDefaultRemoteControlValues();
		if (DefaultValues.HasEntityValue(InId))
		{
			const FAvaPlayableRemoteControlValue* const EntityValue = DefaultValues.GetEntityValue(InId);
			if (EntityValue)
			{
				OutValue = *EntityValue;
				return true;
			}
		}
	}

	return false;
}

bool FAvaRundownPage::GetDefaultControllerValue(const UAvaRundown* InRundown, const FGuid& InId, bool bInUseTemplateValues, FAvaPlayableRemoteControlValue& OutValue) const
{
	if (bInUseTemplateValues)
	{
		if (!IsTemplate())
		{
			const FAvaRundownPage& PageTemplate = ResolveTemplate(InRundown);
			if (PageTemplate.IsValidPage())
			{
				const FAvaPlayableRemoteControlValues& PageValues = PageTemplate.GetRemoteControlValues();
				if (PageValues.HasControllerValue(InId))
				{
					OutValue = *PageValues.GetControllerValue(InId);
					return true;
				}
			}
		}

		return false;
	}

	FAvaRundownManagedInstanceCache& ManagedInstanceCache = IAvaMediaModule::Get().GetManagedInstanceCache();

	for (const FSoftObjectPath& PageAssetPath : GetAssetPaths(InRundown))
	{
		const TSharedPtr<FAvaRundownManagedInstance> ManagedInstance = ManagedInstanceCache.GetOrLoadInstance(PageAssetPath);
		if (!ManagedInstance)
		{
			return false;
		}

		const FAvaPlayableRemoteControlValues& DefaultValues = ManagedInstance->GetDefaultRemoteControlValues();
		if (DefaultValues.HasControllerValue(InId))
		{
			const FAvaPlayableRemoteControlValue* const ControllerValue = DefaultValues.GetControllerValue(InId);
			if (ControllerValue)
			{
				OutValue = *ControllerValue;
				return true;
			}
		}
	}

	return false;
}

bool FAvaRundownPage::IsDefaultEntityValue(const UAvaRundown* InRundown, const FGuid& InId, bool bInUseTemplateValues) const
{
	FAvaPlayableRemoteControlValue DefaultValue;
	if (GetDefaultEntityValue(InRundown, InId, bInUseTemplateValues, DefaultValue))
	{
		if (RemoteControlValues.HasEntityValue(InId))
		{
			if (const FAvaPlayableRemoteControlValue* Value = RemoteControlValues.GetEntityValue(InId))
			{
				if (Value->IsSameValueAs(DefaultValue))
				{
					return true;
				}
			}
		}
	}
	return false;
}

bool FAvaRundownPage::IsDefaultControllerValue(const UAvaRundown* InRundown, const FGuid& InId, bool bInUseTemplateValues) const
{
	FAvaPlayableRemoteControlValue DefaultValue;
	if (GetDefaultControllerValue(InRundown, InId, bInUseTemplateValues, DefaultValue))
	{
		if (RemoteControlValues.HasControllerValue(InId))
		{
			if (const FAvaPlayableRemoteControlValue* Value = RemoteControlValues.GetControllerValue(InId))
			{
				if (Value->IsSameValueAs(DefaultValue))
				{
					return true;
				}
			}
		}
	}
	return false;
}

EAvaPlayableRemoteControlChanges FAvaRundownPage::ResetRemoteControlValues(const UAvaRundown* InRundown, bool bInUseTemplateValues, bool bInIsDefault)
{
	FAvaPlayableRemoteControlValues DefaultValues;
	if (GetDefaultRemoteControlValues(InRundown, bInUseTemplateValues, DefaultValues))
	{
		return RemoteControlValues.ResetRemoteControlValues(DefaultValues, bInIsDefault);
	}
	return EAvaPlayableRemoteControlChanges::None;
}

EAvaPlayableRemoteControlChanges FAvaRundownPage::ResetRemoteControlEntityValue(const UAvaRundown* InRundown, const FGuid& InId, bool bInUseTemplateValues, bool bInIsDefault)
{
	FAvaPlayableRemoteControlValue DefaultValue;
	if (GetDefaultEntityValue(InRundown, InId, bInUseTemplateValues, DefaultValue))
	{
		return RemoteControlValues.ResetRemoteControlEntityValue(InId, DefaultValue, bInIsDefault);
	}
	return EAvaPlayableRemoteControlChanges::None;
}

EAvaPlayableRemoteControlChanges FAvaRundownPage::ResetRemoteControlControllerValue(const UAvaRundown* InRundown, const FGuid& InId, bool bInUseTemplateValues, bool bInIsDefault)
{
	FAvaPlayableRemoteControlValue DefaultValue;
	if (GetDefaultControllerValue(InRundown, InId, bInUseTemplateValues, DefaultValue))
	{
		return RemoteControlValues.ResetRemoteControlControllerValue(InId, DefaultValue, bInIsDefault);
	}
	return EAvaPlayableRemoteControlChanges::None;
}

void FAvaRundownPage::PostLoad()
{
}

int32 FAvaRundownPage::GetNumTemplates(const UAvaRundown* InRundown) const
{
	const FAvaRundownPage& Template = ResolveTemplate(InRundown);
	if (Template.IsValidPage())
	{
		const int32 NumCombinedTemplates = Template.CombinedTemplateIds.Num();
		return NumCombinedTemplates > 0 ? NumCombinedTemplates : 1;
	}
	return 0;
}

const FAvaRundownPage& FAvaRundownPage::GetTemplate(const UAvaRundown* InRundown, int32 InIndex) const
{
	const FAvaRundownPage& Template = ResolveTemplate(InRundown);
	
	if (Template.IsValidPage() && Template.CombinedTemplateIds.Num())
	{
		if (Template.CombinedTemplateIds.IsValidIndex(InIndex))
		{
			// Remark: not supporting recursive template combos.
			const FAvaRundownPage& OtherTemplate  = InRundown->GetPage(Template.CombinedTemplateIds[InIndex]);
			if (OtherTemplate.IsValidPage())
			{
				return OtherTemplate;
			}
			
			UE_LOGF(LogAvaRundown, Error,
				"Internal error while accessing page %d's template (%d index %d): reference to template Id %d is not valid.",
				GetPageId(), Template.GetPageId(), InIndex, Template.CombinedTemplateIds[InIndex]);
		}
		
		UE_LOGF(LogAvaRundown, Error,
			"Internal error while accessing page %d's template (%d): specified index %d is not valid.",
			GetPageId(), Template.GetPageId(), InIndex);
	}
	
	return Template;
}

const FAvaRundownPage& FAvaRundownPage::ResolveTemplate(const UAvaRundown* InRundown) const
{
	if (IsTemplate())
	{
		return *this;
	}

	if (IsValid(InRundown))
	{
		if (InRundown->GetTemplatePages().PageIndices.Contains(GetPageId()))
		{
			UE_LOGF(LogAvaRundown, Warning,
				"PageId %d is in the template list but has \"IsTemplate\" flag to false.", GetPageId());

			// We're obviously a template, but there's been a mix-up...
			return *this;
		}

		if (InRundown->GetInstancedPages().PageIndices.Contains(GetPageId()))
		{
			if (const int32* TemplateIdxPtr = InRundown->GetTemplatePages().PageIndices.Find(GetTemplateId()))
			{
				return InRundown->GetTemplatePages().Pages[*TemplateIdxPtr];
			}
			
			UE_LOGF(LogAvaRundown, Error,
				"PageId [%d] is an instanced page, has template id [%d], but that template doesn't exist.",
				GetPageId(), GetTemplateId());
		}
	}

	return NullPage;
}

bool FAvaRundownPage::IsTemplateMatchingByValue(const FAvaRundownPage& InTemplatePage) const
{
	if (!IsTemplate() || !InTemplatePage.IsTemplate())
	{
		return false;
	}

	if (AssetPath != InTemplatePage.AssetPath)
	{
		return false;
	}

	if (!RemoteControlValues.HasSameEntityValues(InTemplatePage.RemoteControlValues)
		|| !RemoteControlValues.HasSameControllerValues(InTemplatePage.RemoteControlValues))
	{
		return false;
	}
	
	return true;
}

void FAvaRundownPage::LoadInstancedCommands(const TArray<FAvaRundownPageCommandData>& InCommands)
{
	InstancedCommands.Reset(InCommands.Num());
	
	for (int32 CommandIndex = 0; CommandIndex < InCommands.Num(); ++CommandIndex)
	{
		const FAvaRundownPageCommandData& CommandData = InCommands[CommandIndex];
		FInstancedStruct CommandInstance;		
		if (UScriptStruct* CommandStruct = UClass::TryFindTypeSlow<UScriptStruct>(CommandData.Name))
		{
			CommandInstance.InitializeAs(CommandStruct);
			using namespace UE::AvaMedia::RundownSerializationUtils;
			DeserializeRundownPageCommandFromJson(CommandInstance.GetMutablePtr<FAvaRundownPageCommand>(), *CommandStruct, CommandData.Payload);
		}
		InstancedCommands.Add(MoveTemp(CommandInstance));
	}
}

TArray<FAvaRundownPageCommandData> FAvaRundownPage::SaveInstancedCommands() const
{
	TArray<FAvaRundownPageCommandData> Commands;
	Commands.SetNum(InstancedCommands.Num());
	for (int32 CommandIndex = 0; CommandIndex < InstancedCommands.Num(); ++CommandIndex)
	{
		if (const FAvaRundownPageCommand* Command = InstancedCommands[CommandIndex].GetPtr<FAvaRundownPageCommand>())
		{
			if (UScriptStruct* CommandStruct = const_cast<UScriptStruct*>(InstancedCommands[CommandIndex].GetScriptStruct()))
			{
				Commands[CommandIndex].Name = FSoftObjectPath(CommandStruct).ToString();
				UE::AvaMedia::RundownSerializationUtils::SerializeRundownPageCommandToJsonString(Command, *CommandStruct, Commands[CommandIndex].Payload);
				continue;
			}
		}

		// fallback
		Commands[CommandIndex].Name.Empty();
		Commands[CommandIndex].Payload.Empty();
	}
	return Commands;
}

void FAvaRundownPage::SetInstancedCommands(const TArray<FInstancedStruct>& InInstancedCommands)
{	
	InstancedCommands = InInstancedCommands;
}

void FAvaRundownPage::ForEachInstancedCommands(TFunctionRef<void(const FAvaRundownPageCommand&, const FAvaRundownPage&)> InFunction, const UAvaRundown* InRundown, bool bInDirectOnly) const
{
	auto CallOnCommands = [&InFunction, this](const TArray<FInstancedStruct>& InCommands)
	{
		for (const FInstancedStruct& Command : InCommands)
		{
			if (const FAvaRundownPageCommand* CommandPtr = Command.GetPtr<FAvaRundownPageCommand>())
			{
				InFunction(*CommandPtr, *this);
			}
		}
	};
	
	// Direct page commands
	CallOnCommands(InstancedCommands);

	// Note: if no rundown is provided to resolve templates, we have to stop here.
	if (bInDirectOnly || !InRundown)
	{
		return;
	}

	// Direct Template commands
	const FAvaRundownPage& DirectTemplate = ResolveTemplate(InRundown);
	if (DirectTemplate.IsValidPage())
	{
		if (!IsTemplate()) // Already done if this is a template.
		{
			CallOnCommands(DirectTemplate.InstancedCommands);
		}

		// Sub-template commands (if any) - Remark: for now, currently only do 1 level of recursion.
		for (const int32 SubTemplateId : DirectTemplate.GetCombinedTemplateIds())
		{
			const FAvaRundownPage& SubTemplate = InRundown->GetPage(SubTemplateId);
			if (SubTemplate.IsValidPage())
			{
				CallOnCommands(SubTemplate.InstancedCommands);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
