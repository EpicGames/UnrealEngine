// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownPageAssetUtils.h"

#include "AvaScene.h"
#include "AvaTagHandle.h"
#include "AvaTransitionSubsystem.h"
#include "AvaTransitionTree.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "IAvaSceneInterface.h"

namespace UE::AvaRundownPageAssetUtils::Private
{
	template<typename InInterfaceType>
	const InInterfaceType* GetInterfaceFromAsset(const UObject* InLoadedAsset)
	{
		if (const InInterfaceType* const Interface = Cast<InInterfaceType>(InLoadedAsset))
		{
			return Interface;
		}
		if (const UWorld* const World = Cast<UWorld>(InLoadedAsset))
		{
			if (const AAvaScene* AvaScene = AAvaScene::GetScene(World->PersistentLevel, false))
			{
				return static_cast<const InInterfaceType*>(AvaScene);
			}
		}
		return nullptr;
	}
}

const IAvaSceneInterface* FAvaRundownPageAssetUtils::GetSceneInterface(const UObject* InLoadedAsset)
{
	using namespace UE::AvaRundownPageAssetUtils::Private;
	return GetInterfaceFromAsset<IAvaSceneInterface>(InLoadedAsset);
}

const IAvaTransitionBehavior* FAvaRundownPageAssetUtils::FindTransitionBehavior(const IAvaSceneInterface* InSceneInterface)
{
	if (!InSceneInterface)
	{
		return nullptr;
	}
	
	ULevel* SceneLevel = InSceneInterface->GetSceneLevel();
	if (!SceneLevel)
	{
		return nullptr;
	}

	// Method 1 - Using the UAvaTransitionSubsystem
	// This method may not work if the world is not initialized (inactive, as in the managed instance).
	if (const UWorld* SceneWorld = SceneLevel->GetWorld())
	{
		if (const UAvaTransitionSubsystem* TransitionSubsystem = SceneWorld->GetSubsystem<UAvaTransitionSubsystem>())
		{
			return TransitionSubsystem->GetTransitionBehavior(SceneLevel);
		}
	}

	// Method 2 - Fallback using direct lookup of the private actor class.
	// This is needed to lookup in managed instances and source asset.
	return UAvaTransitionSubsystem::FindTransitionBehavior(SceneLevel);
}
