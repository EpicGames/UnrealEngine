// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/StrongObjectPtr.h"

class IAvaSceneInterface;
class IAvaSequenceProvider;
class IAvaTransitionBehavior;
class UAvaTransitionTree;
class UObject;
struct FAvaTagHandle;

/**
 * Common utility functions to query information about loaded Motion Design assets.
 * This is intended to work on "inactive" assets, i.e. either source assets
 * directly or managed ones that don't have an active world.
 * For "runtime" (fully initialized) instances, UAvaPlayable should be used instead.
 */ 
class FAvaRundownPageAssetUtils
{
public:
	static const IAvaSceneInterface* GetSceneInterface(const UObject* InLoadedAsset);
	static const IAvaTransitionBehavior* FindTransitionBehavior(const IAvaSceneInterface* InSceneInterface);
};