// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownPageLoadingManager.h"

#include "Broadcast/AvaBroadcast.h"
#include "Playback/AvaPlaybackManager.h"

namespace UE::AvaRundownPageLoadingManager::Private
{
	bool IsLoaded(EAvaPlaybackStatus InStatus)
	{
		switch (InStatus)
		{
		case EAvaPlaybackStatus::Unknown:
		case EAvaPlaybackStatus::Missing:
		case EAvaPlaybackStatus::Syncing:
		case EAvaPlaybackStatus::Available:
		case EAvaPlaybackStatus::Loading:
			return false;
		case EAvaPlaybackStatus::Loaded:
		case EAvaPlaybackStatus::Starting:
		case EAvaPlaybackStatus::Stopping:
		case EAvaPlaybackStatus::Unloading:
			return true;
		case EAvaPlaybackStatus::Error:
		default:
			return false;
		}
	}

	bool IsError(EAvaPlaybackStatus InStatus)
	{
		return InStatus == EAvaPlaybackStatus::Error;
	}

	bool IsLoading(EAvaPlaybackStatus InStatus)
	{
		return InStatus == EAvaPlaybackStatus::Loading;
	}
}

FAvaRundownPageLoadingManager::FAvaRundownPageLoadingManager(UAvaRundown* InRundown)
	: Rundown(InRundown)
{
	PendingRequestSet.Reserve(32);
	LoadingInstances.Reserve(4);

	if (Rundown)
	{
		Rundown->GetPlaybackManager().OnPlaybackInstanceStatusChanged.AddRaw(this, &FAvaRundownPageLoadingManager::HandlePlaybackInstanceStatusChanged);
		Rundown->GetPlaybackManager().OnBeginTick.AddRaw(this, &FAvaRundownPageLoadingManager::HandlePlaybackManagerBeginTick);
		PlaybackManagerWeak = Rundown->GetPlaybackManager().AsShared();
	}
}

FAvaRundownPageLoadingManager::~FAvaRundownPageLoadingManager()
{
	if (const TSharedPtr<FAvaPlaybackManager> PlaybackManager = PlaybackManagerWeak.Pin())
	{
		PlaybackManager->OnPlaybackInstanceStatusChanged.RemoveAll(this);
		PlaybackManager->OnBeginTick.RemoveAll(this);
	}
}

bool FAvaRundownPageLoadingManager::RequestLoadPage(int32 InPageId, bool bInIsPreview, const FName& InPreviewChannelName)
{
	using namespace UE::AvaRundownPageLoadingManager::Private;
	
	if (!Rundown)
	{
		return false;
	}

	const FAvaRundownPage& Page = Rundown->GetPage(InPageId);
	if (!Page.IsValidPage())
	{
		return false;
	}
	
	if (LoadingInstances.Num() < MaxLoadingInstances)
	{
		return RequestLoadPageInternal(Page, bInIsPreview, InPreviewChannelName);
	}

	const FPageLoadRequest Request = {InPageId, bInIsPreview, InPreviewChannelName};

	// Prevent the same request from being added more than once.
	// It may be necessary to do that to prevent request spamming.
	if (!PendingRequestSet.Contains(Request))
	{
		PendingRequestQueue.Enqueue(Request);
		PendingRequestSet.Add(Request);
	}
	else
	{
		UE_LOGF(LogAvaRundown, Warning,
			"Page Loading Manager: Request for page id \"%d\" type: \"%ls\" Channel \"%ls\" is already in the queue.",
			InPageId, bInIsPreview ? TEXT("Preview") : TEXT("Program"), *InPreviewChannelName.ToString());
	}
	return true;
}

bool FAvaRundownPageLoadingManager::RequestLoadPageInternal(const FAvaRundownPage& InPage, bool bInIsPreview, const FName& InPreviewChannelName)
{
	using namespace UE::AvaRundownPageLoadingManager::Private;
	
	const TArray<UAvaRundown::FLoadedInstanceInfo> LoadedInstances = Rundown->LoadPage(InPage.GetPageId(), bInIsPreview, InPreviewChannelName);
	
	if (LoadedInstances.IsEmpty())
	{
		return false;
	}
	
	for (const UAvaRundown::FLoadedInstanceInfo& LoadedInstance : LoadedInstances)
	{
		FInstanceInfo InstanceInfo;
		InstanceInfo.InstanceId = LoadedInstance.InstanceId;
		InstanceInfo.ChannelName = bInIsPreview ? InPreviewChannelName.ToString() : InPage.GetChannelName().ToString();
		InstanceInfo.AssetPath = LoadedInstance.AssetPath;
		
		// Check if the instance is already loaded
		const TSharedPtr<FAvaPlaybackInstance> PlaybackInstance = FindInstance(InstanceInfo);

		if (!PlaybackInstance || IsError(PlaybackInstance->GetStatus()))
		{
			continue;
		}

		// If it is not loaded yet, wait for it.
		if (!IsLoaded(PlaybackInstance->GetStatus()))
		{
			// Keep track of the current status so we can better track what is going on.
			InstanceInfo.Status = PlaybackInstance->GetStatus();
			// Keep track of submit time so we can time it out if it takes too long.
			InstanceInfo.SubmitTime = FDateTime::UtcNow();
			LoadingInstances.Add(InstanceInfo);
		}
	}
	
	return !LoadedInstances.IsEmpty();
}

void FAvaRundownPageLoadingManager::HandlePlaybackInstanceStatusChanged(const FAvaPlaybackInstance& InPlaybackInstance)
{
	using namespace UE::AvaRundownPageLoadingManager::Private;
	for (TArray<FInstanceInfo>::TIterator InstanceIterator = LoadingInstances.CreateIterator(); InstanceIterator; ++InstanceIterator  )
	{
		FInstanceInfo& InstanceInfo = *InstanceIterator;
		if (InstanceInfo.InstanceId == InPlaybackInstance.GetInstanceId())
		{
			if (IsError(InPlaybackInstance.GetStatus()) || IsLoaded(InPlaybackInstance.GetStatus()))
			{
				InstanceIterator.RemoveCurrent();
			}
		}
	}
}

void FAvaRundownPageLoadingManager::HandlePlaybackManagerBeginTick(float InDeltaSeconds)
{
	using namespace UE::AvaRundownPageLoadingManager::Private;

	const FDateTime CurrentTime = FDateTime::UtcNow();
	const FTimespan TimeoutSpan = FTimespan::FromSeconds(2);

	// Attempt to prune the waiting list.
	for (TArray<FInstanceInfo>::TIterator InstanceIterator = LoadingInstances.CreateIterator(); InstanceIterator; ++InstanceIterator  )
	{
		FInstanceInfo& InstanceInfo = *InstanceIterator;
		const TSharedPtr<FAvaPlaybackInstance> PlaybackInstance = FindInstance(InstanceInfo);

		if (!PlaybackInstance)
		{
			UE_LOGF(LogAvaRundown, Error,
				"Page Loading Manager: Failed to find playback instance Id \"%ls\" Path \"%ls\" Channel \"%ls\".",
				*InstanceInfo.InstanceId.ToString(), *InstanceInfo.AssetPath.ToString(), *InstanceInfo.ChannelName);
			
			InstanceIterator.RemoveCurrent();
			continue;
		}

		if (IsError(PlaybackInstance->GetStatus()))
		{
			UE_LOGF(LogAvaRundown, Error,
				"Page Loading Manager: Failed to load playback instance Id \"%ls\" Path \"%ls\" Channel \"%ls\".",
				*InstanceInfo.InstanceId.ToString(), *InstanceInfo.AssetPath.ToString(), *InstanceInfo.ChannelName);
			InstanceIterator.RemoveCurrent();
			continue;
		}

		// Success condition
		if (IsLoaded(PlaybackInstance->GetStatus()))
		{
			InstanceIterator.RemoveCurrent();
			continue;
		}

		// Handle an edge case:
		// for the remote server, the load command might have been lost. If the playback instance returns to a status
		// of "Available", something went wrong and the asset loading has been interrupted. Server went offline or
		// was externally reset.
		if (IsLoading(InstanceInfo.Status) && !IsLoading(PlaybackInstance->GetStatus()))
		{
			UE_LOGF(LogAvaRundown, Warning,
				"Page Loading Manager: Playback instance Id \"%ls\" Path \"%ls\" Channel \"%ls\" stopped loading.",
					*InstanceInfo.InstanceId.ToString(), *InstanceInfo.AssetPath.ToString(), *InstanceInfo.ChannelName);
			// TODO: special recovery? Might push the request again back in the queue and try again.
			InstanceIterator.RemoveCurrent();
			continue;
		}

		// Finally check for timeout.
		// Remark: load time may be super long for things that need to import animated skeletal mesh (alembic) those will timeout for sure.
		// If a load request times out, we just keep going and request the next page, it will finish eventually.
		if (CurrentTime - InstanceInfo.SubmitTime > TimeoutSpan)
		{
			UE_LOGF(LogAvaRundown, Warning,
				"Page Loading Manager: Playback instance Id \"%ls\" Path \"%ls\" Channel \"%ls\" loading request timed out.",
					*InstanceInfo.InstanceId.ToString(), *InstanceInfo.AssetPath.ToString(), *InstanceInfo.ChannelName);
			InstanceIterator.RemoveCurrent();
		}

		// Update status so we can detect deltas.
		// This would normally be detected and propagated as status change event in playback manager.
		InstanceInfo.Status = PlaybackInstance->GetStatus();
	}

	if (Rundown)
	{
		while (LoadingInstances.Num() < MaxLoadingInstances && !PendingRequestQueue.IsEmpty())
		{
			FPageLoadRequest Request;
			if (PendingRequestQueue.Dequeue(Request))
			{
				PendingRequestSet.Remove(Request);
				const FAvaRundownPage& Page = Rundown->GetPage(Request.PageId);
				if (Page.IsValidPage())
				{
					RequestLoadPageInternal(Page, Request.bIsPreview, Request.PreviewChannelName);
				}
			}
		}
	}
}

TSharedPtr<FAvaPlaybackInstance> FAvaRundownPageLoadingManager::FindInstance(const FInstanceInfo& InInstanceInfo) const
{
	if (Rundown)
	{
		return Rundown->GetPlaybackManager().FindPlaybackInstance(InInstanceInfo.InstanceId, InInstanceInfo.AssetPath, InInstanceInfo.ChannelName);
	}
	return nullptr;
}