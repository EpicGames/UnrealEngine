// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Queue.h"
#include "Rundown/AvaRundown.h"

class FAvaPlaybackInstance;

/**
 * Manager for page loading requests.
 *
 * The main problem this class is solving right now is the throttling of page loading.
 * We don't want to start loading all the pages at the same time because it leads to main thread
 * bottleneck both when the requests are made and when the levels are loaded. To alleviate that
 * we make the operations more granular by loading a subset of levels at any given time.
 */
class FAvaRundownPageLoadingManager final : public IAvaRundownPageLoadingManager
{
public:
	explicit FAvaRundownPageLoadingManager(UAvaRundown* InRundown);
	virtual ~FAvaRundownPageLoadingManager() override;

	//~ Begin IAvaRundownPageLoadingManager
	virtual bool RequestLoadPage(int32 InPageId, bool bInIsPreview, const FName& InPreviewChannelName) override;
	//~ End IAvaRundownPageLoadingManager

private:
	bool RequestLoadPageInternal(const FAvaRundownPage& InPage, bool bInIsPreview, const FName& InPreviewChannelName);
	
	void HandlePlaybackInstanceStatusChanged(const FAvaPlaybackInstance& InPlaybackInstance);
	void HandlePlaybackManagerBeginTick(float InDeltaSeconds);

	struct FInstanceInfo;
	TSharedPtr<FAvaPlaybackInstance> FindInstance(const FInstanceInfo& InInstanceInfo) const;
	
private:
	UAvaRundown* Rundown = nullptr;
	TWeakPtr<FAvaPlaybackManager> PlaybackManagerWeak;

	struct FPageLoadRequest
	{
		int32 PageId;
		bool bIsPreview;
		FName PreviewChannelName;

		bool operator==(FPageLoadRequest const& Other) const
		{
			return PageId == Other.PageId && bIsPreview == Other.bIsPreview && PreviewChannelName == Other.PreviewChannelName;
		}

		FORCEINLINE friend uint32 GetTypeHash(FPageLoadRequest const& This)
		{
			uint32 Hash = 0;
			Hash = HashCombine(Hash, GetTypeHash(This.PageId));
			Hash = HashCombine(Hash, GetTypeHash(This.PreviewChannelName));
			Hash = HashCombine(Hash, GetTypeHash(This.bIsPreview));
			return Hash;
		}
	};
	/** Ordered queue. Requests are executed in order. */
	TQueue<FPageLoadRequest> PendingRequestQueue;
	/** Unordered set to ensure uniqueness of requests. */
	TSet<FPageLoadRequest> PendingRequestSet;

	/** Instances currently loading. */
	struct FInstanceInfo
	{
		FGuid InstanceId;
		FSoftObjectPath AssetPath;
		FString ChannelName;
		EAvaPlaybackStatus Status = EAvaPlaybackStatus::Unknown;
		FDateTime SubmitTime;	// Time stamp when the load request was submitted.
	};
	TArray<FInstanceInfo> LoadingInstances;

	/** Maximum concurrent instance to load. */
	int32 MaxLoadingInstances = 1;
};
