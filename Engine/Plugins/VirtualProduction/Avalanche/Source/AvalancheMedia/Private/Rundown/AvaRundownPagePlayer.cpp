// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownPagePlayer.h"

#include "AvaMediaSettings.h"
#include "Playable/AvaPlayable.h"
#include "Playback/AvaPlaybackManager.h"
#include "Playback/AvaPlaybackUtils.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownPageCommand.h"

namespace UE::AvaMedia::Rundown::PagePlayer::Private
{
	static void PushCameraCut(UAvaPlaybackGraph* InPlaybackObject, const UAvaRundown* InRundown, const FAvaRundownPage& InPage
		, const FString& InChannelName)
	{
		InPlaybackObject->PushAnimationCommand(InPage.GetAssetPath(InRundown), InChannelName,
			EAvaPlaybackAnimAction::CameraCut, FAvaPlaybackAnimPlaySettings());
	}
}

UAvaRundownPlaybackInstancePlayer::UAvaRundownPlaybackInstancePlayer() = default;
UAvaRundownPlaybackInstancePlayer::~UAvaRundownPlaybackInstancePlayer() = default;

bool UAvaRundownPlaybackInstancePlayer::Load(const UAvaRundownPagePlayer& InPagePlayer, const UAvaRundown* InRundown, const FAvaRundownPage& InPage, int32 InSubPageIndex, const FGuid& InInstanceId)
{
	SourceAssetPath = InPage.GetAssetPath(InRundown, InSubPageIndex);
	TransitionLayer = InPage.GetTransitionLayer(InRundown, InSubPageIndex);

	// Get the Load Options from page command, if any.
	FString LoadOptions;
	FAvaRundownPageCommandContext PageCommandContext = {*InRundown, InPage, InPagePlayer.ChannelFName};
	
	InPage.ForEachInstancedCommands(
		[&PageCommandContext, &LoadOptions](const FAvaRundownPageCommand& InCommand, const FAvaRundownPage& InPage)
		{
			InCommand.ExecuteOnLoad(PageCommandContext, LoadOptions);
		},
		InRundown, /*bInDirectOnly*/ false); // Traverse templates.

	FAvaPlaybackManager& Manager = InRundown->GetPlaybackManager();
	PlaybackInstance = Manager.AcquireOrLoadPlaybackInstance(SourceAssetPath, InPagePlayer.ChannelName, LoadOptions);
	Playback = PlaybackInstance ? PlaybackInstance->GetPlayback() : nullptr;

	// If restoring from a remote instance.
	if (InInstanceId.IsValid())
	{
		PlaybackInstance->SetInstanceId(InInstanceId);
	}

	// Setup user instance data to be able to track this page.
	if (PlaybackInstance)
	{
		UAvaRundownPagePlayer::SetInstanceUserDataFromPage(*PlaybackInstance, InPage);
	}

	if (Playback && InPagePlayer.bIsPreview)
	{
		Playback->SetPreviewChannelName(InPagePlayer.ChannelFName);
	}

	return IsLoaded();
}

bool UAvaRundownPlaybackInstancePlayer::IsLoaded() const
{
	return Playback != nullptr;
}

void UAvaRundownPlaybackInstancePlayer::Play(const UAvaRundownPagePlayer& InPagePlayer, const UAvaRundown* InRundown, EAvaRundownPagePlayType InPlayType)
{
	using namespace UE::AvaMedia::Rundown::PagePlayer::Private;
	
	if (!Playback || !InRundown)
	{
		return;
	}

	const bool bPlaybackObjectWasPlaying = Playback->IsPlaying();
	
	if (!Playback->IsPlaying())
	{
		Playback->Play();
	}

	if (bPlaybackObjectWasPlaying)
	{
		const FAvaRundownPage& Page = InRundown->GetPage(InPagePlayer.PageId);
		if (Page.IsValidPage())
		{
			PushCameraCut(Playback, InRundown, Page, InPagePlayer.ChannelName);
		}
	}
}

bool UAvaRundownPlaybackInstancePlayer::IsPlaying() const
{
	return Playback != nullptr && Playback->IsPlaying();
}

bool UAvaRundownPlaybackInstancePlayer::Continue(const FString& InChannelName)
{
	if (Playback && Playback->IsPlaying())
	{
		// Animation command, within this playback, needs channel for now.
		const FAvaPlaybackAnimPlaySettings AnimSettings;	// Note: Leaving the name to None means the action apply to all animations.
		Playback->PushAnimationCommand(SourceAssetPath, InChannelName, EAvaPlaybackAnimAction::Continue, AnimSettings);
		return true;
	}
	return false;
}

bool UAvaRundownPlaybackInstancePlayer::Stop()
{
	const bool bUnload = !UAvaMediaSettings::Get().bKeepPagesLoaded;
	bool bWasStopped = false; 
	
	if (Playback && Playback->IsPlaying())
	{
		// Propagate the unload options in case this object is playing remote.
		const EAvaPlaybackStopOptions PlaybackStopOptions = bUnload ?
			EAvaPlaybackStopOptions::Default | EAvaPlaybackStopOptions::Unload : EAvaPlaybackStopOptions::Default;
		Playback->Stop(PlaybackStopOptions);
		bWasStopped = true;
	}

	if (PlaybackInstance)
	{
		// Unload the local object as well.
		if (bUnload)
		{
			PlaybackInstance->Unload();
		}
		else
		{
			PlaybackInstance->Recycle();
		}
	}

	Playback = nullptr;
	PlaybackInstance.Reset();
	return bWasStopped;
}

bool UAvaRundownPlaybackInstancePlayer::HasPlayable(const UAvaPlayable* InPlayable) const
{
	return Playback && Playback->HasPlayable(InPlayable);
}

UAvaPlayable* UAvaRundownPlaybackInstancePlayer::GetFirstPlayable() const
{
	return Playback ? Playback->GetFirstPlayable() : nullptr;
}

UAvaRundownPagePlayer* UAvaRundownPlaybackInstancePlayer::GetPagePlayer() const
{
	return ParentPagePlayer.Get();
}

void UAvaRundownPlaybackInstancePlayer::SetPagePlayer(UAvaRundownPagePlayer* InPagePlayer)
{
	ParentPagePlayer = InPagePlayer;
}

UAvaRundownPagePlayer::UAvaRundownPagePlayer()
{
	UAvaPlayable::OnSequenceEvent().AddUObject(this, &UAvaRundownPagePlayer::HandleOnPlayableSequenceEvent);
}

UAvaRundownPagePlayer::~UAvaRundownPagePlayer()
{
	UAvaPlayable::OnSequenceEvent().RemoveAll(this);
}

bool UAvaRundownPagePlayer::Initialize(UAvaRundown* InRundown, const FAvaRundownPage& InPage, bool bInIsPreview, const FName& InPreviewChannel)
{
	if (!InRundown)
	{
		UE_LOGF(LogAvaRundown, Error, "UAvaRundownPagePlayer::Initialize: Invalid rundown.");
		return false;
	}

	if (!InPage.IsValidPage())
	{
		UE_LOGF(LogAvaRundown, Error, "UAvaRundownPagePlayer::Initialize: Invalid page.");
		return false;
	}
	
	checkf(InstancePlayers.IsEmpty(), TEXT("Can't initialize a page player if already loaded or playing."));

	RundownWeak = InRundown;
	PageId = InPage.GetPageId();
	bIsPreview = bInIsPreview;
	ChannelFName = bIsPreview ? InPreviewChannel : InPage.GetChannelName();
	ChannelName = ChannelFName.ToString();
	return true;
}

bool UAvaRundownPagePlayer::InitializeAndLoad(UAvaRundown* InRundown, const FAvaRundownPage& InPage, bool bInIsPreview, const FName& InPreviewChannel)
{
	if (!Initialize(InRundown, InPage, bInIsPreview, InPreviewChannel))
	{
		return false;
	}
	
	const int32 NumTemplates = InPage.GetNumTemplates(InRundown);
	for (int32 SubPageIndex = 0; SubPageIndex < NumTemplates; ++SubPageIndex)
	{
		CreateAndLoadInstancePlayer(InRundown, InPage, SubPageIndex, FGuid());
	}
	return InstancePlayers.Num() > 0;
}

UAvaRundownPlaybackInstancePlayer* UAvaRundownPagePlayer::LoadInstancePlayer(int32 InSubPageIndex, const FGuid& InInstanceId)
{
	UAvaRundown* Rundown = RundownWeak.Get();
	if (!Rundown)
	{
		UE_LOGF(LogAvaRundown, Error, "UAvaRundownPagePlayer::LoadSubPage: Rundown is no longuer valid.");
		return nullptr;
	}

	const FAvaRundownPage& Page = Rundown->GetPage(PageId);
	if (!Page.IsValidPage())
	{
		UE_LOGF(LogAvaRundown, Error, "UAvaRundownPagePlayer::LoadSubPage: Invalid pageId %d.", PageId);
		return nullptr;
	}

	return CreateAndLoadInstancePlayer(Rundown, Page, InSubPageIndex, InInstanceId);
}

void UAvaRundownPagePlayer::AddInstancePlayer(UAvaRundownPlaybackInstancePlayer* InExistingInstancePlayer)
{
	// Remove from previous player.
	if (UAvaRundownPagePlayer* PreviousPagePlayer = InExistingInstancePlayer->GetPagePlayer())
	{
		PreviousPagePlayer->RemoveInstancePlayer(InExistingInstancePlayer);
	}

	InstancePlayers.Add(InExistingInstancePlayer);
	InExistingInstancePlayer->SetPagePlayer(this);
}

bool UAvaRundownPagePlayer::IsLoaded() const
{
	for (const TObjectPtr<UAvaRundownPlaybackInstancePlayer>& InstancePlayer : InstancePlayers)
	{
		if (InstancePlayer && InstancePlayer->IsLoaded())
		{
			return true;
		}
	}
	return false;
}

bool UAvaRundownPagePlayer::Play(EAvaRundownPagePlayType InPlayType)
{
	const UAvaRundown* Rundown = RundownWeak.Get();
	if (!Rundown)
	{
		return false;
	}

	bool bIsPlaying = false;
	for (const TObjectPtr<UAvaRundownPlaybackInstancePlayer>& InstancePlayer : InstancePlayers)
	{
		InstancePlayer->Play(*this, Rundown, InPlayType);
		bIsPlaying |= InstancePlayer->IsPlaying();
	}

	return bIsPlaying;
}

bool UAvaRundownPagePlayer::IsPlaying() const
{
	for (const TObjectPtr<UAvaRundownPlaybackInstancePlayer>& InstancePlayer : InstancePlayers)
	{
		if (InstancePlayer && InstancePlayer->IsPlaying())
		{
			return true;
		}
	}
	return false;
}

bool UAvaRundownPagePlayer::Continue()
{
	bool bSuccess = false;
	for (const TObjectPtr<UAvaRundownPlaybackInstancePlayer>& InstancePlayer : InstancePlayers)
	{
		bSuccess |= InstancePlayer->Continue(ChannelName);
	}
	return bSuccess;
}

bool UAvaRundownPagePlayer::Stop()
{
	bool bSuccess = false;
	for (const TObjectPtr<UAvaRundownPlaybackInstancePlayer>& InstancePlayer : InstancePlayers)
	{
		bSuccess |= InstancePlayer->Stop();
	}
	if (RundownWeak.IsValid())
	{
		RundownWeak->NotifyPageStopped(PageId);
	}
	return bSuccess;
}

int32 UAvaRundownPagePlayer::GetPageIdFromInstanceUserData(const FString& InUserData)
{
	FString PageIdString;
	if (FParse::Value(*InUserData, TEXT("PageId="), PageIdString))
	{
		return FCString::Atoi(*PageIdString);
	}
	return FAvaRundownPage::InvalidPageId;
}

void UAvaRundownPagePlayer::SetInstanceUserDataFromPage(FAvaPlaybackInstance& InPlaybackInstance, const FAvaRundownPage& InPage)
{
	InPlaybackInstance.SetInstanceUserData(FString::Printf(TEXT("PageId=%d"), InPage.GetPageId()));
}

bool UAvaRundownPagePlayer::HasPlayable(const UAvaPlayable* InPlayable) const
{
	for (const TObjectPtr<UAvaRundownPlaybackInstancePlayer>& InstancePlayer : InstancePlayers)
	{
		if (InstancePlayer->HasPlayable(InPlayable))
		{
			return true;
		}
	}
	return false;
}

UAvaRundownPlaybackInstancePlayer* UAvaRundownPagePlayer::FindInstancePlayerForPlayable(const UAvaPlayable* InPlayable) const
{
	for (const TObjectPtr<UAvaRundownPlaybackInstancePlayer>& InstancePlayer : InstancePlayers)
	{
		if (InstancePlayer && InstancePlayer->HasPlayable(InPlayable))
		{
			return InstancePlayer;
		}
	}
	return nullptr;
}

UAvaRundownPlaybackInstancePlayer* UAvaRundownPagePlayer::FindInstancePlayerByInstanceId(const FGuid& InInstanceId) const
{
	for (const TObjectPtr<UAvaRundownPlaybackInstancePlayer>& InstancePlayer : InstancePlayers)
	{
		if (InstancePlayer && InstancePlayer->GetPlaybackInstanceId() == InInstanceId)
		{
			return InstancePlayer;
		}
	}
	return nullptr;
}

UAvaRundownPlaybackInstancePlayer* UAvaRundownPagePlayer::FindInstancePlayerByAssetPath(const FSoftObjectPath& InAssetPath) const
{
	for (const TObjectPtr<UAvaRundownPlaybackInstancePlayer>& InstancePlayer : InstancePlayers)
	{
		if (InstancePlayer && InstancePlayer->SourceAssetPath == InAssetPath)
		{
			return InstancePlayer;
		}
	}
	return nullptr;
}

UAvaRundownPlaybackInstancePlayer* UAvaRundownPagePlayer::CreateAndLoadInstancePlayer(UAvaRundown* InRundown, const FAvaRundownPage& InPage, int32 InSubPageIndex, const FGuid& InInstanceId)
{
	UAvaRundownPlaybackInstancePlayer* InstancePlayer = NewObject<UAvaRundownPlaybackInstancePlayer>(InRundown);
	if (InstancePlayer->Load(*this, InRundown, InPage, InSubPageIndex, InInstanceId))
	{
		InstancePlayer->SetPagePlayer(this);
		InstancePlayers.Add(InstancePlayer);
		return InstancePlayer;
	}

	return nullptr;
}

void UAvaRundownPagePlayer::RemoveInstancePlayer(UAvaRundownPlaybackInstancePlayer* InInstancePlayer)
{
	InstancePlayers.Remove(InInstancePlayer);
}

void UAvaRundownPagePlayer::HandleOnPlayableSequenceEvent(UAvaPlayable* InPlayable, FName InSequenceLabel, EAvaPlayableSequenceEventType InEventType)
{
	// Check that this is the playable for this page player.
	if (!HasPlayable(InPlayable))
	{
		return;
	}
	
	// Notify the rundown.
	if (RundownWeak.IsValid())
	{
		using namespace UE::AvaPlayback::Utils;
		if (InEventType == EAvaPlayableSequenceEventType::Started)
		{
			UE_LOGF(LogAvaRundown, Verbose,
				"%ls Rundown Page %d (playable %ls): Sequence Started \"%ls\".",
				*GetBriefFrameInfo(), PageId, *InPlayable->GetInstanceId().ToString(), *InSequenceLabel.ToString());
		}

		if (InEventType == EAvaPlayableSequenceEventType::Finished)
		{
			UE_LOGF(LogAvaRundown, Verbose,
				"%ls Rundown Page %d (playable %ls): Sequence Finished \"%ls\".",
				*GetBriefFrameInfo(), PageId, *InPlayable->GetInstanceId().ToString(), *InSequenceLabel.ToString());

			RundownWeak->NotifyPageSequenceFinished(PageId);
		}
	}
}
