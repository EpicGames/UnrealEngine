// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownPlaybackClientWatcher.h"

#include "Broadcast/AvaBroadcast.h"
#include "IAvaMediaModule.h"
#include "Playback/AvaPlaybackUtils.h"
#include "Playback/IAvaPlaybackClient.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownPagePlayer.h"

FAvaRundownPlaybackClientWatcher::FAvaRundownPlaybackClientWatcher(UAvaRundown* InRundown)
	: Rundown(InRundown)
{
	using namespace UE::AvaPlaybackClient::Delegates;
	GetOnPlaybackStatusChanged().AddRaw(this, &FAvaRundownPlaybackClientWatcher::HandlePlaybackStatusChanged);
	
}
FAvaRundownPlaybackClientWatcher::~FAvaRundownPlaybackClientWatcher()
{
	using namespace UE::AvaPlaybackClient::Delegates;
	GetOnPlaybackStatusChanged().RemoveAll(this);
}

void FAvaRundownPlaybackClientWatcher::TryRestorePlaySubPage(int InPageId, const UE::AvaPlaybackClient::Delegates::FPlaybackStatusChangedArgs& InEventArgs) const
{
	using namespace UE::AvaPlayback::Utils;
	
	// Restore page player and local playback proxies.
	// (Need to specify the InstanceId from the server for everything to match.)
	const FName ChannelFName(InEventArgs.ChannelName);

	// Ensure the specified channel exists.
	if (UAvaBroadcast::Get().GetChannelIndex(ChannelFName) == INDEX_NONE)
	{
		UE_LOGF(LogAvaRundown, Error,
			"%ls Received a playback object on channel \"%ls\" which doesn't exist locally. Playback Server should be reset.",
			*GetBriefFrameInfo(), *InEventArgs.ChannelName);
		return;
	}
	
	const bool bIsPreview = UAvaBroadcast::Get().GetChannelType(ChannelFName) == EAvaBroadcastChannelType::Preview ? true : false;

	const FAvaRundownPage& PageToRestore = Rundown->GetPage(InPageId);

	const int32 SubPageIndex = PageToRestore.GetAssetPaths(Rundown).Find(InEventArgs.AssetPath);

	if (SubPageIndex == INDEX_NONE)
	{
		UE_LOGF(LogAvaRundown, Error,
			"%ls Asset mismatch (expected (any of): \"%ls\", received: \"%ls\") for restoring page %d. Playback Server should be reset.",
			*GetBriefFrameInfo(),
			*FString::JoinBy(PageToRestore.GetAssetPaths(Rundown), TEXT(","), [](const FSoftObjectPath& Path){ return Path.ToString();}),
			*InEventArgs.AssetPath.ToString(), InPageId);
		return;
	}

	if (!Rundown->RestorePlaySubPage(InPageId, SubPageIndex, InEventArgs.InstanceId, bIsPreview, ChannelFName))
	{
		UE_LOGF(LogAvaRundown, Error, "%ls Failed to restore page %d. Playback Server should be reset.", *GetBriefFrameInfo(), InPageId);
	}
}

void FAvaRundownPlaybackClientWatcher::HandlePlaybackStatusChanged(IAvaPlaybackClient& InPlaybackClient,
	const UE::AvaPlaybackClient::Delegates::FPlaybackStatusChangedArgs& InEventArgs)
{
	if (!Rundown)
	{
		return;
	}

	static const TArray<EAvaPlaybackStatus> RunningStates =
	{
		//EAvaPlaybackStatus::Starting, // Starting is not reliable, it may also mean "loading". FIXME.
		EAvaPlaybackStatus::Started
	};

	// Try to determine if a playback has started or stopped.
	const bool bWasRunning = IsAnyOf(InEventArgs.PrevStatus, RunningStates);
	const bool bIsRunning = IsAnyOf(InEventArgs.NewStatus, RunningStates);

	// TODO: Reconcile forked channels. Need to keep track of status per server. (Seems to work well enough for now, but may need to revisit)
	
	// If a playback instance is stopping, stop corresponding page (if any).
	if (bWasRunning && !bIsRunning)
	{
		UE_LOGF(LogAvaRundown, Verbose, 
			"%ls Playback Client Watcher: Detected asset stopping Id:%ls from Server \"%ls\".",
			*UE::AvaPlayback::Utils::GetBriefFrameInfo(), *InEventArgs.InstanceId.ToString(), *InEventArgs.ServerName);

		for (UAvaRundownPagePlayer* PagePlayer : Rundown->PagePlayers)
		{
			// Search for a match with the event:				
			if (PagePlayer && PagePlayer->ChannelName == InEventArgs.ChannelName)
			{
				if (UAvaRundownPlaybackInstancePlayer* InstancePlayer = PagePlayer->FindInstancePlayerByInstanceId(InEventArgs.InstanceId))
				{
					if (InstancePlayer->SourceAssetPath != InEventArgs.AssetPath)
					{
						UE_LOGF(LogAvaRundown, Error, "%ls Playback Client Watcher: Instance Id:%ls asset path mismatch in page player %d.",
							*UE::AvaPlayback::Utils::GetBriefFrameInfo(), *InEventArgs.InstanceId.ToString(), PagePlayer->PageId);
						continue;
					}

					UE_LOGF(LogAvaRundown, Verbose, "%ls Playback Client Watcher: Stopping Instance Id:%ls in page player %d.",
						*UE::AvaPlayback::Utils::GetBriefFrameInfo(), *InEventArgs.InstanceId.ToString(), PagePlayer->PageId);
					
					InstancePlayer->Stop();
				}

				// If we stopped all the instance players, stop the page (to broadcast events).
				if (!PagePlayer->IsPlaying())
				{
					UE_LOGF(LogAvaRundown, Verbose, "%ls Playback Client Watcher: Stopping Page player %d, no more instances playing.",
						*UE::AvaPlayback::Utils::GetBriefFrameInfo(), PagePlayer->PageId);

					PagePlayer->Stop();	
				}
			}
		}
		
		Rundown->RemoveStoppedPagePlayers();
	}

	// Note: execute this even if not on rising transition because it may be a user data update following the "GetUserData" request.
	if (bIsRunning)
	{
		// We need to figure out which page it is.
		const FString* RemoteUserData = InPlaybackClient.GetRemotePlaybackUserData(InEventArgs.InstanceId, InEventArgs.AssetPath, InEventArgs.ChannelName, InEventArgs.ServerName);

		// We haven't received the user data for this playback. So we request it.
		// This event will be received again with user data next time.
		if (!RemoteUserData)
		{
			InPlaybackClient.RequestPlayback(InEventArgs.InstanceId, InEventArgs.AssetPath, InEventArgs.ChannelName, EAvaPlaybackAction::GetUserData);
		}
		else
		{
			const TSharedPtr<FAvaPlaybackInstance> LocalPlaybackInstance = IAvaMediaModule::Get().GetLocalPlaybackManager().FindPlaybackInstance(InEventArgs.InstanceId, InEventArgs.AssetPath, InEventArgs.ChannelName);
			if (LocalPlaybackInstance && LocalPlaybackInstance->GetInstanceUserData() != *RemoteUserData)
			{
				UE_LOGF(LogAvaRundown, Error, 
					"%ls Playback Client Watcher: Playback Instance Id:%ls on server \"%ls\": user data mismatch \"%ls\", local user data: \"%ls\".",
					*UE::AvaPlayback::Utils::GetBriefFrameInfo(),
					*InEventArgs.InstanceId.ToString(), *InEventArgs.ServerName, *(*RemoteUserData), *LocalPlaybackInstance->GetInstanceUserData());
			}
			
			const int32 PageId = UAvaRundownPagePlayer::GetPageIdFromInstanceUserData(*RemoteUserData);
			if (PageId != FAvaRundownPage::InvalidPageId)
			{
				const UAvaRundownPagePlayer* PagePlayer = Rundown->FindPlayerForProgramPage(PageId);
				if (!PagePlayer || !PagePlayer->FindInstancePlayerByInstanceId(InEventArgs.InstanceId))
				{
					TryRestorePlaySubPage(PageId, InEventArgs);
				}
			}
		}
	}
}