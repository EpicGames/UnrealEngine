// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "Playback/AvaPlaybackClientDelegates.h"

class IAvaPlaybackClient;
class UAvaRundown;

/**
 * Playback Client Watcher ensures external playback events are reconciled.
 * Cases covered:
 * 1 - if a remote playable is stopped (server reset), the corresponding page in the rundown will be stopped
 * 2 - if a remote playable is playing (server side), the corresponding page's state will be restored, i.e. rebuilding the proxies.
 * to be continued...
 */
class FAvaRundownPlaybackClientWatcher
{
public:
	FAvaRundownPlaybackClientWatcher(UAvaRundown* InRundown);
	~FAvaRundownPlaybackClientWatcher();

	void TryRestorePlaySubPage(int InPageId, const UE::AvaPlaybackClient::Delegates::FPlaybackStatusChangedArgs& InEventArgs) const;

	static bool IsAnyOf(EAvaPlaybackStatus InStatus, const TArray<EAvaPlaybackStatus>& InStatuses)
	{
		for (const EAvaPlaybackStatus Status : InStatuses)
		{
			if (Status == InStatus)
			{
				return true;
			}
		}
		return false;
	}
	
	void HandlePlaybackStatusChanged(IAvaPlaybackClient& InPlaybackClient,
		const UE::AvaPlaybackClient::Delegates::FPlaybackStatusChangedArgs& InEventArgs);
	
private:
	UAvaRundown* Rundown;
};
