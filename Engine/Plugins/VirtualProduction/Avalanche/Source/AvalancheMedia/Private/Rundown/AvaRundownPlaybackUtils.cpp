// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownPlaybackUtils.h"

#include "Rundown/AvaRundown.h"

namespace UE::AvaRundownPlaybackUtils::Private
{
	template <typename Predicate>
	TArray<int32> FilterSelectedOrPreviewingPages(const UAvaRundown* InRundown, const TArray<int32>& InSelectedPageIds, FName InPreviewChannelName, Predicate InPredicate)
	{
		if (InRundown)
		{
			if (InSelectedPageIds.Num())
			{
				return ::Invoke(InPredicate, InRundown, InSelectedPageIds);
			}
			return ::Invoke(InPredicate, InRundown, InRundown->GetPreviewingPageIds(InPreviewChannelName));
		}
		return {};
	}
}

// Selecting the "next page" implies we know what the last page played was and we go to the next.
// Normally, the last page played was the one currently selected, but the user might decide to select
// another page for editing purpose, but still want to hit play next from the last played page.
// This is why the "play head" is now added in the playback context for the given list.
int32 FAvaRundownPlaybackUtils::GetPageIdToPlayNext(const UAvaRundown* InRundown, const FAvaRundownPageListReference& InPageListReference, bool bInPreview, FName InPreviewChannelName)
{
	// If there is no playback context, it means no pages are playing and can't have a next page if nothing is playing.
	if (InRundown && InRundown->GetPageListPlaybackContextCollection())
	{
		if (const TSharedPtr<FAvaRundownPageListPlaybackContext> PlaybackContext = InRundown->GetPageListPlaybackContextCollection()->GetContext(bInPreview, InPreviewChannelName))
		{
			// Check if the current play head page is playing.
			const bool bIsHeadPlaying = bInPreview ? InRundown->IsPagePreviewing(PlaybackContext->PlayHeadPageId) : InRundown->IsPagePlaying(PlaybackContext->PlayHeadPageId); 

			// Exception for Command only pages. They don't have a page player.
			bool bIsCommandOnly = false;
			if (!bIsHeadPlaying)
			{
				const FAvaRundownPage& Page = InRundown->GetPage(PlaybackContext->PlayHeadPageId);
				bIsCommandOnly = Page.HasCommands(InRundown) && !Page.HasAssets(*InRundown); 
			}

			if (bIsHeadPlaying || bIsCommandOnly)
			{
				const int32 NextPageId = InRundown->GetNextPage(PlaybackContext->PlayHeadPageId, InPageListReference).GetPageId();
				if (IsPageIdValid(NextPageId) && InRundown->CanPlayPage(NextPageId, bInPreview))
				{
					return NextPageId;
				}
			}
		}
	}
	return FAvaRundownPage::InvalidPageId;
}

TArray<int32> FAvaRundownPlaybackUtils::GetPagesToTakeToProgram(const UAvaRundown* InRundown, const TArray<int32>& InSelectedPageIds, FName InPreviewChannel)
{
	using namespace UE::AvaRundownPlaybackUtils::Private;
	auto KeepPagesToTakeToProgram = [](const UAvaRundown* InRundown, const TArray<int32>& InPageIds)
	{
		TArray<int32> OutPageIds;
		OutPageIds.Reserve(InPageIds.Num());
		for (int32 PageId : InPageIds)
		{
			// We can't take templates to program. Only instanced pages.
			// Note: CanPlayPage returns true for playing pages, so need to make sure it is not already playing.
			if (InRundown->CanPlayPage(PageId, false) && !InRundown->IsPagePlaying(PageId))
			{
				OutPageIds.Add(PageId);
			}
		}
		return OutPageIds;
	};
	return FilterSelectedOrPreviewingPages(InRundown, InSelectedPageIds, InPreviewChannel, KeepPagesToTakeToProgram);
}
