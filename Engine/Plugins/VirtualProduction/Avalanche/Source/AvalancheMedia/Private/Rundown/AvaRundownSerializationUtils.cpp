// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownSerializationUtils.h"

#include "AvaMediaSerializationUtils.h"
#include "Backends/JsonStructDeserializerBackend.h"
#include "Backends/JsonStructSerializerBackend.h"
#include "HAL/FileManager.h"
#include "JsonObjectConverter.h"
#include "Rundown/AvaRundown.h"
#include "StructDeserializer.h"
#include "StructSerializer.h"
#include "StructUtils/InstancedStruct.h"

#define LOCTEXT_NAMESPACE "AvaRundownSerializationUtils"

// Keep the legacy serializer in case there are issues with the new one.
static TAutoConsoleVariable<bool> CVarRundownNewJsonSerializer(
	TEXT("MotionDesignRundown.UseNewJsonSerializer"),
	true,
	TEXT("Enable/Disable New Rundown Serializer. If disabled, will use the legacy serializer."),
	ECVF_Default);

namespace UE::AvaMedia::RundownSerializationUtils::Private
{
	// Don't serialize the transient properties.
	static auto TransientPropertyFilter = [](const FProperty* InCurrentProp, const FProperty* InParentProp)
	{
		const bool bIsTransient = InCurrentProp && InCurrentProp->HasAnyPropertyFlags(CPF_Transient); 
		return !bIsTransient; 
	};

	struct FRundownSerializerPolicies : public FStructSerializerPolicies
	{
		FRundownSerializerPolicies()
		{
			PropertyFilter = TransientPropertyFilter;
		}
	};

	struct FRundownDeserializerPolicies : public FStructDeserializerPolicies
	{
		FRundownDeserializerPolicies()
		{
			PropertyFilter = TransientPropertyFilter;
		}
	};

	const TCHAR* GetJsonTypeName(EJson InType)
	{
		switch (InType)
		{
		case EJson::None: return TEXT("None");
		case EJson::Null: return TEXT("Null");
		case EJson::String: return TEXT("String");
		case EJson::Number: return TEXT("Number");
		case EJson::Boolean: return TEXT("Boolean");
		case EJson::Array: return TEXT("Array");
		case EJson::Object: return TEXT("Object");
		default: return TEXT("Invalid");
		}
	}
	
	TSharedPtr<FJsonObject> SerializeToJson(const UStruct* InStruct, const void* InObject);

	static TSharedPtr<FJsonValue> ExportPropertyCallback(FProperty* InProperty, const void* InValue)
	{
		TSharedPtr<FJsonValue> Result = nullptr;

		if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
		{
			if (StructProperty->Struct == FInstancedStruct::StaticStruct())
			{
				const FInstancedStruct& InstancedStruct = *static_cast<const FInstancedStruct*>(InValue);
				if (const TSharedPtr<FJsonObject> LoadedStruct = SerializeToJson(InstancedStruct.GetScriptStruct(), InstancedStruct.GetMemory()))
				{
					const TSharedPtr<FJsonObject> InstancedStructObj = MakeShared<FJsonObject>();
					const FSoftObjectPath StructPath(InstancedStruct.GetScriptStruct());
					InstancedStructObj->SetField(StructPath.ToString(), MakeShared<FJsonValueObject>(LoadedStruct));
					
					Result = MakeShared<FJsonValueObject>(InstancedStructObj);
				}
			}
		}
		return Result;
	}

	TSharedPtr<FJsonObject> SerializeToJson(const UStruct* InStruct, const void* InObject)
	{
		FJsonObjectConverter::CustomExportCallback CustomCB;
		CustomCB.BindStatic(&ExportPropertyCallback);

		const TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		constexpr int64 CheckFlags = 0;
		constexpr int64 SkipFlags = 0;
		const bool bLoaded = FJsonObjectConverter::UStructToJsonObject(InStruct, InObject, JsonObject.ToSharedRef(), CheckFlags, SkipFlags, &CustomCB, EJsonObjectConversionFlags::SkipStandardizeCase);
		return bLoaded ? JsonObject : nullptr;
	}

	TSharedPtr<FJsonObject> SerializeToJson(const UObject* InObject)
	{
		return SerializeToJson(InObject->GetClass(), InObject);
	}

	bool DeserializeFromJson(const TSharedRef<FJsonObject>& InJsonObject, const UStruct* InStruct, void* InNativeObject, FText& OutErrorMessage);

	// The deserialization error will get lost, so we log instead.
	//
	// Note: 
	//	Returning false means the callback doesn't handle the value, so fallback code does.
	//	However, if this is our case we handle, even if there is an error, we return true to indicate it is handled.
	bool ImportPropertyCallback(const TSharedPtr<FJsonValue>& InJsonValue, FProperty* InProperty, void* InValue)
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
		{
			if (StructProperty->Struct == FInstancedStruct::StaticStruct())
			{
				constexpr bool bHandledButError = true;	// There is no way to indicate an error or return the error message.
				
				FInstancedStruct& InstancedStruct = *static_cast<FInstancedStruct*>(InValue);

				if (InJsonValue->Type !=  EJson::Object)
				{
					UE_LOGF(LogAvaRundown, Error, "Rundown Json Parsing FInstancedStruct Property: Json value should be of type Object, is of type: %ls", GetJsonTypeName(InJsonValue->Type));
					return bHandledButError;
				}
				
				// json value should be an object with 1 field
				const TSharedPtr<FJsonObject>* InstancedStructObject;
				if (InJsonValue->TryGetObject(InstancedStructObject))
				{
					if ((*InstancedStructObject)->Values.IsEmpty())
					{
						UE_LOGF(LogAvaRundown, Error, "Rundown Json Parsing FInstancedStruct Property: Json Object should have at least 1 field (has %d)", (*InstancedStructObject)->Values.Num());
						return bHandledButError;
					}

					if ((*InstancedStructObject)->Values.Num() > 1)
					{
						UE_LOGF(LogAvaRundown, Warning, "Rundown Json Parsing FInstancedStruct Property: Json Object should have only 1 field (has %d)", (*InstancedStructObject)->Values.Num());
					}
						
					for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Value : (*InstancedStructObject)->Values)
					{
						if (!Value.Value.IsValid())
						{
							UE_LOGF(LogAvaRundown, Error, "Rundown Json Parsing FInstancedStruct Property: Json value is invalid");
							continue;
						}

						FSoftObjectPath ScriptPath(*Value.Key);
						if (const UScriptStruct* Struct = Cast<UScriptStruct>(ScriptPath.ResolveObject()))	// so we got a valid struct
						{
							if (Value.Value->Type !=  EJson::Object)
							{
								UE_LOGF(LogAvaRundown, Error, "Rundown Json Parsing FInstancedStruct Property: Json value should be of type Object, is of type: %ls", GetJsonTypeName(Value.Value->Type));
							}
							
							const TSharedPtr<FJsonObject>* StructObject;
							if (Value.Value->TryGetObject(StructObject))
							{
								InstancedStruct.InitializeAs(Struct);
								FText DeserializationError;
								if (DeserializeFromJson((*StructObject).ToSharedRef(), Struct, InstancedStruct.GetMutableMemory(), DeserializationError))
								{
									constexpr bool bHandledSuccess = true;
									return bHandledSuccess;
								}
								
								UE_LOGF(LogAvaRundown, Error, "Rundown Json Parsing FInstancedStruct Property: Failed to deserialize: \"%ls\"", *DeserializationError.ToString());
							}
						}
						else
						{
							UE_LOGF(LogAvaRundown, Error, "Rundown Json Parsing FInstancedStruct Property: Unknown Script Type: \"%ls\"", *Value.Key);
						}
					}

					return bHandledButError;
				}
			}
		}
		
		return false;
	}

	bool DeserializeFromJson(const TSharedRef<FJsonObject>& InJsonObject, const UStruct* InStruct, void* InNativeObject, FText& OutErrorMessage)
	{
		FJsonObjectConverter::CustomImportCallback CustomCB;
		CustomCB.BindStatic(&ImportPropertyCallback);

		constexpr int64 CheckFlags = 0;
		constexpr int64 SkipFlags = 0;
		constexpr bool bStrictMode = false;
		return FJsonObjectConverter::JsonObjectToUStruct(InJsonObject, InStruct, InNativeObject, CheckFlags, SkipFlags, bStrictMode, &OutErrorMessage, &CustomCB);
	}

	
}

namespace UE::AvaMedia::RundownSerializationUtils
{
	bool SaveRundownToJson(const UAvaRundown* InRundown, FArchive& InArchive, FText& OutErrorMessage)
	{
		if (!IsValid(InRundown))
		{
			OutErrorMessage = LOCTEXT("SaveRundownJson_InvalidRundown", "Invalid rundown.");
			return false;
		}

		if(CVarRundownNewJsonSerializer.GetValueOnGameThread())
		{
			const TSharedPtr<FJsonObject> JsonRundown = Private::SerializeToJson(InRundown);

			const TSharedPtr<FJsonObject> JsonRoot = MakeShared<FJsonObject>();
			JsonRoot->SetStringField(TEXT("RundownFileVersion"), TEXT("1.0"));
			JsonRoot->SetObjectField(TEXT("Rundown"), JsonRundown);

			const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&InArchive);
			if (!FJsonSerializer::Serialize(JsonRoot.ToSharedRef(), Writer))
			{
				return false;
			}
		}
		else
		{
			// Legacy serializer
			// Remark: this is hardcoded to encode in utf16-le.
			FJsonStructSerializerBackend Backend(InArchive, EStructSerializerBackendFlags::Default);
			FStructSerializer::Serialize(InRundown, *InRundown->GetClass(), Backend, Private::FRundownSerializerPolicies());
		}
		return true;
	}

	bool SaveRundownToJson(const UAvaRundown* InRundown, const TCHAR* InFilepath, FText& OutErrorMessage)
	{
		const TUniquePtr<FArchive> FileWriter(IFileManager::Get().CreateFileWriter(InFilepath));
		if (FileWriter)
		{
			const bool bSerialized = SaveRundownToJson(InRundown, *FileWriter, OutErrorMessage);
			FileWriter->Close();
			return bSerialized;
		}

		OutErrorMessage = LOCTEXT("SaveRundownJson_FailedFileWriting", "Failed to open file for writing.");
		return false;
	}
	
	bool LoadRundownFromJson(UAvaRundown* InRundown, FArchive& InArchive, FText& OutErrorMessage)
	{
		if (!IsValid(InRundown))
		{
			OutErrorMessage = LOCTEXT("LoadRundownJson_InvalidRundown", "Invalid rundown.");
			return false;
		}
			
		// Serializing doesn't reset content, it will add to it.
		// so we need to explicitly make the rundown empty first.
		if (!InRundown->Empty())
		{
			// One reason this could fail is if the rundown is currently playing.
			if (InRundown->IsPlaying())
			{
				OutErrorMessage = LOCTEXT("LoadRundownJson_RundownIsPlaying", "Cannot import on a playing rundown. Stop rundown playback first.");
			}
			else
			{
				OutErrorMessage = LOCTEXT("LoadRundownJson_FailedClearRundown", "Failed to clear rundown content.");
			}
			return false;
		}

		const int64 ArchivePosition = InArchive.Tell();
		
		TSharedPtr<FJsonObject> JsonRoot = MakeShared<FJsonObject>();
		const TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(&InArchive);
		if (!FJsonSerializer::Deserialize(JsonReader, JsonRoot))
		{
			OutErrorMessage = LOCTEXT("LoadRundownJson_FailedParseJson", "Couldn't parse json data.");
			return false;
		}

		bool bTryLoadLegacy = true;
		bool bLoaded = false;
		FText DeserializationError;
		
		// If we can find the rundown field, we got the new version.
		if (const TSharedPtr<FJsonObject> RundownJson = JsonRoot->GetObjectField(TEXT("Rundown")))
		{
			bTryLoadLegacy = false;
			bLoaded = Private::DeserializeFromJson(RundownJson.ToSharedRef(), InRundown->GetClass(), InRundown, DeserializationError);
		}

		if (bTryLoadLegacy)
		{
			InArchive.Seek(ArchivePosition); // Return to start.
			FJsonStructDeserializerBackend Backend(InArchive);
			bLoaded = FStructDeserializer::Deserialize(InRundown, *InRundown->GetClass(), Backend, Private::FRundownDeserializerPolicies());
			if (!bLoaded)
			{
				DeserializationError = FText::FromString(Backend.GetLastErrorMessage()); 
			}
		}
			
		if (bLoaded)
		{
			InRundown->PostLoad();
			InRundown->MarkPackageDirty();
		}
		else
		{
			OutErrorMessage =
				FText::Format(LOCTEXT("LoadRundownJson_DeserializerError", "Json Deserializer error: {0}"), DeserializationError);
		}
		return bLoaded;
	}

	bool LoadRundownFromJson(UAvaRundown* InRundown, const TCHAR* InFilepath, FText& OutErrorMessage)
	{
		const TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(InFilepath));
		if (FileReader)
		{
			const bool bLoaded = LoadRundownFromJson(InRundown, *FileReader, OutErrorMessage);
			FileReader->Close();
			return bLoaded;
		}

		OutErrorMessage = LOCTEXT("FileNotFound", "File not found");
		return false;
	}

	bool SerializeRundownPageCommandToJsonString(const FAvaRundownPageCommand* InRundownPageCommand, UStruct& InStruct, FString& OutString)
	{
		TArray<uint8> ValueAsBytes;
		FMemoryWriter Writer = FMemoryWriter(ValueAsBytes);
		FJsonStructSerializerBackend Backend(Writer, EStructSerializerBackendFlags::Default);
		FStructSerializer::Serialize(InRundownPageCommand, InStruct, Backend, Private::FRundownSerializerPolicies());
		Writer.Close();
		AvaMediaSerializationUtils::JsonValueConversion::BytesToString(ValueAsBytes, OutString);
		return true;
	}
	
	bool DeserializeRundownPageCommandFromJson(FAvaRundownPageCommand* InRundownPageCommand, UStruct& InStruct, const FString& InString)
	{
		FMemoryReaderView Reader(AvaMediaSerializationUtils::JsonValueConversion::ValueToConstBytesView(InString));
		FJsonStructDeserializerBackend Backend(Reader);
		return FStructDeserializer::Deserialize(InRundownPageCommand, InStruct, Backend, Private::FRundownDeserializerPolicies());
	}
}

#undef LOCTEXT_NAMESPACE
