// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownServer.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Async/Async.h"
#include "AvaMediaMessageUtils.h"
#include "AvaMediaRenderTargetUtils.h"
#include "AvaMediaSerializationUtils.h"
#include "AvaRemoteControlUtils.h"
#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputClassItem.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputDeviceItem.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputRootItem.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputServerItem.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputTreeItem.h"
#include "Broadcast/OutputDevices/AvaBroadcastRenderTargetMediaUtils.h"
#include "Engine/Engine.h"
#include "IAvaMediaModule.h"
#include "IRemoteControlModule.h"
#include "ImageUtils.h"
#include "MediaOutput.h"
#include "MessageEndpointBuilder.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Playable/Transition/AvaPlayableTransition.h"
#include "Playback/AvaPlaybackManager.h"
#include "Playback/AvaPlaybackUtils.h"
#include "RemoteControlSettings.h"
#include "RenderingThread.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownManagedInstanceCache.h"
#include "Rundown/AvaRundownPagePlayer.h"
#include "Rundown/AvaRundownPlaybackUtils.h"
#include "Rundown/AvaRundownSerializationUtils.h"
#include "Rundown/AvaRundownServerMediaOutputUtils.h"
#include "TextureResource.h"
#include "Transition/AvaRundownPageTransition.h"

#if WITH_EDITOR
#include "Editor.h"
#include "ObjectTools.h"
#include "ScopedTransaction.h"
#include "Subsystems/EditorAssetSubsystem.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogAvaRundownServer, Log, All);

#define LOCTEXT_NAMESPACE "AvaRundownServer"

namespace UE::AvaRundownServer::Private
{
	// We still support the initial version.
	constexpr int32 CurrentMinimumApiVersion = EAvaRundownApiVersion::Initial;

	EAvaRundownServerBuildTargetType GetRundownEngineBuild()
	{
		switch (FApp::GetBuildTargetType())
		{
		case EBuildTargetType::Unknown:
			return EAvaRundownServerBuildTargetType::Unknown;
		case EBuildTargetType::Game:
			return EAvaRundownServerBuildTargetType::Game;
		case EBuildTargetType::Server:
			return EAvaRundownServerBuildTargetType::Server;
		case EBuildTargetType::Client:
			return EAvaRundownServerBuildTargetType::Client;
		case EBuildTargetType::Editor:
			return EAvaRundownServerBuildTargetType::Editor;
		case EBuildTargetType::Program:
			return EAvaRundownServerBuildTargetType::Program;
		default:
			return EAvaRundownServerBuildTargetType::Unknown;
		}
	}

	EAvaRundownServerEngineMode GetRundownEngineMode()
	{
		if (!GEngine)
		{
			return EAvaRundownServerEngineMode::Unknown;
		}
		
		if (IsRunningDedicatedServer())
		{
			return EAvaRundownServerEngineMode::Server;
		}
		
		if (IsRunningCommandlet())
		{
			return EAvaRundownServerEngineMode::Commandlet;
		}

		// This is checking GIsEditor
		if (GEngine->IsEditor())
		{
			return EAvaRundownServerEngineMode::Editor;
		}
		
		if (IsRunningGame())
		{
			return EAvaRundownServerEngineMode::Game;
		}
		
		return EAvaRundownServerEngineMode::Other;
	}

	void SanitizeInvalidCharsInline(FString& InOutText, const TCHAR* InvalidChars)
	{
		const TCHAR* InvalidChar = InvalidChars ? InvalidChars : TEXT("");
		while (*InvalidChar)
		{
			InOutText.ReplaceCharInline(*InvalidChar, TCHAR('_'), ESearchCase::CaseSensitive);
			++InvalidChar;
		}
	}
	
	FString SanitizePackageName(const FString& InPackageName)
	{
		// Ensure no backslashes.
		FString SanitizedName = InPackageName.Replace(TEXT("\\"), TEXT("/"));

		// Replace any other invalid characters with '_'.
		SanitizeInvalidCharsInline(SanitizedName, INVALID_LONGPACKAGE_CHARACTERS);

		// Coalesce multiple contiguous slashes into a single slash
		int32 CharIndex = 0;
		while (CharIndex < SanitizedName.Len())
		{
			if (SanitizedName[CharIndex] == TEXT('/'))
			{
				int32 SlashCount = 1;
				while (CharIndex + SlashCount < SanitizedName.Len() &&
					   SanitizedName[CharIndex + SlashCount] == TEXT('/'))
				{
					SlashCount++;
				}

				if (SlashCount > 1)
				{
					SanitizedName.RemoveAt(CharIndex + 1, SlashCount - 1, EAllowShrinking::No);
				}
			}

			CharIndex++;
		}

		// Finally, ensure it begins with "/" since this is an absolute package name.
		if (!SanitizedName.StartsWith(TEXT("/")))
		{
			SanitizedName = FString(TEXT("/")) + SanitizedName;
		}

		return SanitizedName;
	}
	
	inline FAvaRundownPageInfo GetPageInfo(const UAvaRundown* InRundown, const FAvaRundownPage& InPage)
	{
		FAvaRundownPageInfo PageInfo;
		PageInfo.PageId = InPage.GetPageId();
		PageInfo.PageName = InPage.GetPageName();
		PageInfo.PageSummary = InPage.GetPageSummary().ToString();
		PageInfo.FriendlyName = InPage.GetPageFriendlyName().ToString();
		PageInfo.IsTemplate = InPage.IsTemplate();
		PageInfo.TemplateId = InPage.GetTemplateId();
		PageInfo.CombinedTemplateIds = InPage.GetCombinedTemplateIds();
		PageInfo.AssetPath = InPage.GetAssetPath(InRundown);	// Todo: combo templates
		PageInfo.Statuses = InPage.GetPageStatuses(InRundown);
		PageInfo.TransitionLayerName = InPage.GetTransitionLayer(InRundown).ToString(); 	// Todo: combo templates
		PageInfo.bTransitionLogicEnabled = InPage.HasTransitionLogic(InRundown);
		PageInfo.Commands = InPage.SaveInstancedCommands();
		PageInfo.OutputChannel = InPage.GetChannelName().ToString();
		PageInfo.bIsEnabled = InPage.IsEnabled();
		PageInfo.bIsPlaying = InRundown->IsPagePlaying(InPage);
		return PageInfo;
	}

	/**
	 * Utility function load a rundown asset in memory.
	 */
	static TStrongObjectPtr<UAvaRundown> LoadRundown(const FSoftObjectPath& InRundownPath)
	{
		UObject* Object = InRundownPath.ResolveObject();
		if (!Object)
		{
			Object = InRundownPath.TryLoad();
		}
		return TStrongObjectPtr<UAvaRundown>(Cast<UAvaRundown>(Object));
	}

	static FAvaRundownChannel SerializeChannel(const FAvaBroadcastOutputChannel& InChannel)
	{
		FAvaRundownChannel Channel;
		Channel.Name = InChannel.GetChannelName().ToString();
		Channel.State = InChannel.GetState();
		Channel.Type = InChannel.GetChannelType();
		Channel.IssueSeverity = InChannel.GetIssueSeverity();
		const TArray<UMediaOutput*>& MediaOutputs = InChannel.GetMediaOutputs();
		for (const UMediaOutput* MediaOutput : MediaOutputs)
		{
			FAvaRundownOutputDeviceItem DeviceItem;
			DeviceItem.Name = MediaOutput->GetFName().ToString();
			DeviceItem.OutputInfo = InChannel.GetMediaOutputInfo(MediaOutput);
			DeviceItem.OutputState = InChannel.GetMediaOutputState(MediaOutput);
			DeviceItem.IssueSeverity = InChannel.GetMediaOutputIssueSeverity(DeviceItem.OutputState, MediaOutput);
			DeviceItem.IssueMessages = InChannel.GetMediaOutputIssueMessages(MediaOutput);
			DeviceItem.Data = FAvaRundownServerMediaOutputUtils::SerializeMediaOutput(MediaOutput);
			Channel.Devices.Push(MoveTemp(DeviceItem));
		}

		return Channel;
	}
	
	// recursively search device and children
	static FAvaOutputTreeItemPtr RecursiveFindOutputTreeItem(const FAvaOutputTreeItemPtr& InOutputTreeItem, const FString& InDeviceName)
	{
		if (!InOutputTreeItem->IsA<FAvaBroadcastOutputRootItem>() && InDeviceName == InOutputTreeItem->GetDisplayName().ToString())
		{
			return InOutputTreeItem;
		}
		
		for (const TSharedPtr<IAvaBroadcastOutputTreeItem>& Child : InOutputTreeItem->GetChildren())
		{
			if (FAvaOutputTreeItemPtr TreeItem = RecursiveFindOutputTreeItem(Child, InDeviceName))
			{
				return TreeItem;
			}
		}
		
		return nullptr;
	}

	static UMediaOutput* FindChannelMediaOutput(const FAvaBroadcastOutputChannel& InOutputChannel, const FString& InOutputMediaName)
	{
		const TArray<UMediaOutput*>& MediaOutputs = InOutputChannel.GetMediaOutputs();
		for (UMediaOutput* MediaOutput : MediaOutputs)
		{
			if (MediaOutput->GetName() == InOutputMediaName)
			{
				return MediaOutput;
			}
		}
		
		return nullptr;
	}
	
	inline TArray<int32> GetPlayingPages(const UAvaRundown* InRundown, bool bInIsPreview, FName InChannelName)
	{
		return bInIsPreview ? InRundown->GetPreviewingPageIds(InChannelName) : InRundown->GetPlayingPageIds(InChannelName);
	}

	static bool ContinuePages(UAvaRundown* InRundown, const TArray<int32>& InPageIds, bool bInIsPreview, FName InPreviewChannelName, FString& OutFailureReason)
	{
		bool bSuccess = false;
		for (const int32 PageId : InPageIds)
		{
			if (InRundown->CanContinuePage(PageId, bInIsPreview, InPreviewChannelName))
			{
				bSuccess |= InRundown->ContinuePage(PageId, bInIsPreview, InPreviewChannelName);
			}
			else if (bInIsPreview)
			{
				OutFailureReason.Appendf(TEXT("PageId %d was not previewing on channel \"%s\". "), PageId, *InPreviewChannelName.ToString());
			}
			else
			{
				OutFailureReason.Appendf(TEXT("PageId %d was not playing. "), PageId);
			}
		}
		return bSuccess;
	}

	static bool UpdatePagesValues(const UAvaRundown* InRundown, const TArray<int32>& InPageIds, bool bInIsPreview, FName InPreviewChannelName)
	{
		bool bSuccess = false;
		for (const int32 PageId : InPageIds)
		{
			bSuccess |= InRundown->PushRuntimeRemoteControlValues(PageId, bInIsPreview, InPreviewChannelName);
		}
		return bSuccess;
	}

	TArray<int32> GetPageIds(TConstArrayView<TWeakObjectPtr<UAvaRundownPagePlayer>> InPagePlayers)
	{
		TArray<int32> PageIds;
		PageIds.Reserve(InPagePlayers.Num());
		for (const TWeakObjectPtr<UAvaRundownPagePlayer>& PagePlayerWeak : InPagePlayers)
		{
			if (const UAvaRundownPagePlayer* PagePlayer = PagePlayerWeak.Get())
			{
				PageIds.Add(PagePlayer->PageId);
			}
		}
		return PageIds;
	}

	void FillPageTransitionInfo(const UAvaRundownPageTransition& InPageTransition, FAvaRundownPageTransitionEvent& OutMessage)
	{
		OutMessage.Channel = InPageTransition.GetChannelName().ToString();
		OutMessage.TransitionId = InPageTransition.GetTransitionId();
		OutMessage.EnteringPageIds = GetPageIds(InPageTransition.GetEnterPlayers());
		OutMessage.PlayingPageIds = GetPageIds(InPageTransition.GetPlayingPlayers());
		OutMessage.ExitingPageIds = GetPageIds(InPageTransition.GetExitPlayers());
	}
}

/**
 * Holds the render target for copying the channel image.
 * The render target needs to be held for many frames until it is done.
 */
struct FAvaRundownServer::FChannelImage
{
	// Optional temporary render target for converting pixel format.
	TStrongObjectPtr<UTextureRenderTarget2D> RenderTarget;

	// Pixels readback from the render target. Format is PF_B8G8R8A8 (for now).
	TArray<FColor> RawPixels;
	int32 SizeX = 0;
	int32 SizeY = 0;

	void UpdateRenderTarget(int32 InSizeX, int32 InSizeY, EPixelFormat InFormat, const FLinearColor& InClearColor)
	{
		if (!RenderTarget.IsValid())
		{
			static const FName ChannelImageRenderTargetBaseName = TEXT("AvaRundownServer_ChannelImageRenderTarget");
			RenderTarget.Reset(UE::AvaMediaRenderTargetUtils::CreateDefaultRenderTarget(ChannelImageRenderTargetBaseName));
		}
	
		UE::AvaMediaRenderTargetUtils::UpdateRenderTarget(RenderTarget.Get(), FIntPoint(InSizeX, InSizeY), InFormat, InClearColor);
	}

	void UpdateRawPixels(int32 InSizeX, int32 InSizeY)
	{
		SizeX = InSizeX;
		SizeY = InSizeY;
		RawPixels.SetNum(InSizeX * InSizeY);
	}
};

FAvaRundownServer::FAvaRundownServer()
{
	
}

FAvaRundownServer::~FAvaRundownServer()
{
	RemovePlaybackDelegates();
	RemoveBroadcastDelegates(&UAvaBroadcast::Get());
	RemoveEditorDelegates();
	
	FMessageEndpoint::SafeRelease(MessageEndpoint);
	
	for (IConsoleObject* ConsoleCommand : ConsoleCommands)
	{
		IConsoleManager::Get().UnregisterConsoleObject(ConsoleCommand);
	}
	ConsoleCommands.Empty();
}

const FMessageAddress& FAvaRundownServer::GetMessageAddress() const
{
	if (MessageEndpoint.IsValid())
	{
		return MessageEndpoint->GetAddress();
	}
	static FMessageAddress InvalidMessageAddress;
	return InvalidMessageAddress;
}

void FAvaRundownServer::AddReferencedObjects(FReferenceCollector& InCollector)
{
	for (const TPair<FSoftObjectPath, TWeakPtr<FRundownEntry>>& RundownEntryWeak : LoadedRundownCache)
	{
		if (const TSharedPtr<FRundownEntry> RundownEntry = RundownEntryWeak.Value.Pin())
		{
			RundownEntry->AddReferencedObjects(InCollector);
		}
	}

	for (TPair<FSoftObjectPath, TObjectPtr<UAvaRundown>>& ManagedRundown : ManagedRundowns)
	{
		if (IsValid(ManagedRundown.Value.Get()))
		{
			InCollector.AddReferencedObject(ManagedRundown.Value);
		}
	}
}

FString FAvaRundownServer::GetReferencerName() const
{
	return TEXT("FAvaRundownServer");
}

void FAvaRundownServer::Init(const FString& InAssignedHostName)
{
	HostName = InAssignedHostName.IsEmpty() ? FPlatformProcess::ComputerName() : InAssignedHostName;
	
	MessageEndpoint = FMessageEndpoint::Builder("MotionDesignRundownServer")
	.Handling<FAvaRundownPing>(this, &FAvaRundownServer::HandleRundownPing)
	.Handling<FAvaRundownGetServerInfo>(this, &FAvaRundownServer::HandleGetRundownServerInfo)
	.Handling<FAvaRundownGetPlayableAssets>(this, &FAvaRundownServer::HandleGetPlayableAssets)
	.Handling<FAvaRundownGetRundowns>(this, &FAvaRundownServer::HandleGetRundowns)
	.Handling<FAvaRundownLoadRundown>(this, &FAvaRundownServer::HandleLoadRundown)
	.Handling<FAvaRundownCreateRundown>(this, &FAvaRundownServer::HandleCreateRundown)
	.Handling<FAvaRundownDeleteRundown>(this, &FAvaRundownServer::HandleDeleteRundown)
	.Handling<FAvaRundownImportRundown>(this, &FAvaRundownServer::HandleImportRundown)
	.Handling<FAvaRundownExportRundown>(this, &FAvaRundownServer::HandleExportRundown)
	.Handling<FAvaRundownSaveRundown>(this, &FAvaRundownServer::HandleSaveRundown)
	.Handling<FAvaRundownCreatePage>(this, &FAvaRundownServer::HandleCreatePage)
	.Handling<FAvaRundownDeletePage>(this, &FAvaRundownServer::HandleDeletePage)
	.Handling<FAvaRundownCreateTemplate>(this, &FAvaRundownServer::HandleCreateTemplate)
	.Handling<FAvaRundownCreateComboTemplate>(this, &FAvaRundownServer::HandleCreateComboTemplate)
	.Handling<FAvaRundownDeleteTemplate>(this, &FAvaRundownServer::HandleDeleteTemplate)
	.Handling<FAvaRundownChangeTemplateBP>(this, &FAvaRundownServer::HandleChangeTemplateBP)
	.Handling<FAvaRundownGetPages>(this, &FAvaRundownServer::HandleGetPages)
	.Handling<FAvaRundownGetPageDetails>(this, &FAvaRundownServer::HandleGetPageDetails)
	.Handling<FAvaRundownPageChangeChannel>(this, &FAvaRundownServer::HandleChangePageChannel)
	.Handling<FAvaRundownChangePageName>(this, &FAvaRundownServer::HandleChangePageName)
	.Handling<FAvaRundownUpdatePageFromRCP>(this, &FAvaRundownServer::HandleUpdatePageFromRCP)
	.Handling<FAvaRundownPageAction>(this, &FAvaRundownServer::HandlePageAction)
	.Handling<FAvaRundownPagePreviewAction>(this, &FAvaRundownServer::HandlePagePreviewAction)
	.Handling<FAvaRundownPageActions>(this, &FAvaRundownServer::HandlePageActions)
	.Handling<FAvaRundownPagePreviewActions>(this, &FAvaRundownServer::HandlePagePreviewActions)
	.Handling<FAvaRundownTransitionAction>(this, &FAvaRundownServer::HandleTransitionAction)
	.Handling<FAvaRundownTransitionLayerAction>(this, &FAvaRundownServer::HandleTransitionLayerAction)
	.Handling<FAvaRundownGetProfiles>(this, &FAvaRundownServer::HandleGetProfiles)
	.Handling<FAvaRundownDuplicateProfile>(this, &FAvaRundownServer::HandleDuplicateProfile)
	.Handling<FAvaRundownCreateProfile>(this, &FAvaRundownServer::HandleCreateProfile)
	.Handling<FAvaRundownRenameProfile>(this, &FAvaRundownServer::HandleRenameProfile)	
	.Handling<FAvaRundownDeleteProfile>(this, &FAvaRundownServer::HandleDeleteProfile)
	.Handling<FAvaRundownSetCurrentProfile>(this, &FAvaRundownServer::HandleSetCurrentProfile)
	.Handling<FAvaRundownGetChannel>(this, &FAvaRundownServer::HandleGetChannel)
	.Handling<FAvaRundownGetChannels>(this, &FAvaRundownServer::HandleGetChannels)
	.Handling<FAvaRundownChannelAction>(this, &FAvaRundownServer::HandleChannelAction)
	.Handling<FAvaRundownChannelEditAction>(this, &FAvaRundownServer::HandleChannelEditAction)
	.Handling<FAvaRundownRenameChannel>(this, &FAvaRundownServer::HandleRenameChannel)
	.Handling<FAvaRundownGetDevices>(this, &FAvaRundownServer::HandleGetDevices)
	.Handling<FAvaRundownAddChannelDevice>(this, &FAvaRundownServer::HandleAddChannelDevice)
	.Handling<FAvaRundownEditChannelDevice>(this, &FAvaRundownServer::HandleEditChannelDevice)
	.Handling<FAvaRundownRemoveChannelDevice>(this, &FAvaRundownServer::HandleRemoveChannelDevice)
	.Handling<FAvaRundownGetChannelImage>(this, &FAvaRundownServer::HandleGetChannelImage)
	.Handling<FAvaRundownGetChannelQualitySettings>(this, &FAvaRundownServer::HandleGetChannelQualitySettings)
	.Handling<FAvaRundownSetChannelQualitySettings>(this, &FAvaRundownServer::HandleSetChannelQualitySettings)
	.Handling<FAvaRundownSaveBroadcast>(this, &FAvaRundownServer::HandleSaveBroadcast)
	.NotificationHandling(FOnBusNotification::CreateSP(this, &FAvaRundownServer::OnMessageBusNotification))
	.ReceivingOnThread(ENamedThreads::GameThread);

	if (MessageEndpoint.IsValid())
	{
		// Subscribe to the server listing requests
		MessageEndpoint->Subscribe<FAvaRundownPing>();

		SetupPlaybackDelegates();
		SetupBroadcastDelegates(&UAvaBroadcast::Get());
		SetupEditorDelegates();

		UE_LOGF(LogAvaRundownServer, Log, "Motion Design Rundown Server \"%ls\" Started.", *HostName);
	}
}

void FAvaRundownServer::SetupPlaybackDelegates()
{
	FAvaPlaybackManager& Manager = IAvaMediaModule::Get().GetLocalPlaybackManager();
	Manager.OnPlaybackInstanceStatusChanged.AddSP(this, &FAvaRundownServer::OnPlaybackInstanceStatusChanged);

	UAvaPlayable::OnSequenceEvent().AddSP(this, &FAvaRundownServer::OnPlayableSequenceEvent);
	UAvaPlayable::OnTransitionEvent().AddSP(this, &FAvaRundownServer::OnPlayableTransitionEvent);
}

void FAvaRundownServer::SetupBroadcastDelegates(UAvaBroadcast* InBroadcast)
{
	RemoveBroadcastDelegates(InBroadcast);
	InBroadcast->GetOnChannelsListChanged().AddSP(this, &FAvaRundownServer::OnBroadcastChannelListChanged);
	FAvaBroadcastOutputChannel::GetOnChannelChanged().AddSP(this, &FAvaRundownServer::OnBroadcastChannelChanged);
}

void FAvaRundownServer::SetupEditorDelegates()
{
	RemoveEditorDelegates();
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	AssetRegistryModule.Get().OnAssetAdded().AddSP(this, &FAvaRundownServer::OnAssetAdded);
	AssetRegistryModule.Get().OnAssetRemoved().AddSP(this, &FAvaRundownServer::OnAssetRemoved);
#if WITH_EDITOR
	FEditorDelegates::OnAssetsPreDelete.AddSP(this, &FAvaRundownServer::OnAssetsPreDelete);
#endif
}

void FAvaRundownServer::RemovePlaybackDelegates() const
{
	const IAvaMediaModule& AvaMediaModule = IAvaMediaModule::Get();
	if (AvaMediaModule.IsLocalPlaybackManagerAvailable())
	{
		FAvaPlaybackManager& Manager = AvaMediaModule.GetLocalPlaybackManager();
		Manager.OnPlaybackInstanceStatusChanged.RemoveAll(this);
	}

	UAvaPlayable::OnSequenceEvent().RemoveAll(this);
	UAvaPlayable::OnTransitionEvent().RemoveAll(this);
}

void FAvaRundownServer::RemoveBroadcastDelegates(UAvaBroadcast* InBroadcast) const
{
	InBroadcast->GetOnChannelsListChanged().RemoveAll(this);
	FAvaBroadcastOutputChannel::GetOnChannelChanged().RemoveAll(this);
}

void FAvaRundownServer::RemoveEditorDelegates() const
{
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	AssetRegistryModule.Get().OnAssetAdded().RemoveAll(this);
	AssetRegistryModule.Get().OnAssetRemoved().RemoveAll(this);
#if WITH_EDITOR
	FEditorDelegates::OnAssetsPreDelete.RemoveAll(this);
#endif
}

void FAvaRundownServer::OnPageListChanged(const FAvaRundownPageListChangeParams& InParams) const
{
	FAvaRundownPageListChanged* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownPageListChanged>();
	ReplyMessage->Rundown = FSoftObjectPath(InParams.Rundown).ToString();
	ReplyMessage->ListType = InParams.PageListReference.Type;
	ReplyMessage->SubListId = InParams.PageListReference.SubListId;
	ReplyMessage->ChangeType = static_cast<uint8>(InParams.ChangeType);
	ReplyMessage->AffectedPages = InParams.AffectedPages;
	
	SendResponse(ReplyMessage, ClientAddresses);
}

void FAvaRundownServer::OnPagesChanged(const UAvaRundown* InRundown, const FAvaRundownPage& InPage, EAvaRundownPageChanges InChange) const
{
	if (EnumHasAnyFlags(InChange, EAvaRundownPageChanges::AnimationSettings))
	{
		PageAnimSettingsChanged(InRundown, InPage);
	}
	if (EnumHasAnyFlags(InChange, EAvaRundownPageChanges::Blueprint))
	{
		PageBlueprintChanged(InRundown, InPage, InPage.GetAssetPath(InRundown).ToString());
	}
	if (EnumHasAnyFlags(InChange, EAvaRundownPageChanges::Status))
	{
		PageStatusChanged(InRundown, InPage);
	}
	if (EnumHasAnyFlags(InChange, EAvaRundownPageChanges::Channel))
	{
		PageChannelChanged(InRundown, InPage, InPage.GetChannelName().ToString());
	}
	if (EnumHasAnyFlags(InChange, EAvaRundownPageChanges::Name))
	{
		PageNameChanged(InRundown, InPage, /*bInFriendlyName*/ false);
	}
	if (EnumHasAnyFlags(InChange, EAvaRundownPageChanges::FriendlyName))
	{
		PageNameChanged(InRundown, InPage, /*bInFriendlyName*/ true);
	}

	// todo: EAvaRundownPageChanges::RemoteControlValues
	// -> tbd: rundown server api doesn't expose RC value directly.
}
void FAvaRundownServer::PageStatusChanged(const UAvaRundown* InRundown, const FAvaRundownPage& InPage) const
{
	FAvaRundownPagesStatuses* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownPagesStatuses>();
	ReplyMessage->Rundown = FSoftObjectPath(InRundown).ToString();
	ReplyMessage->PageInfo = UE::AvaRundownServer::Private::GetPageInfo(InRundown, InPage);
	SendResponse(ReplyMessage, ClientAddresses);
}

void FAvaRundownServer::PageBlueprintChanged(const UAvaRundown* InRundown, const FAvaRundownPage& InPage, const FString& InBlueprintPath) const
{
	FAvaRundownPageBlueprintChanged* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownPageBlueprintChanged>();
	ReplyMessage->Rundown = FSoftObjectPath(InRundown).ToString();
	ReplyMessage->PageId = InPage.GetPageId();
	ReplyMessage->BlueprintPath = InBlueprintPath;
	SendResponse(ReplyMessage, ClientAddresses);
}

void FAvaRundownServer::PageChannelChanged(const UAvaRundown* InRundown, const FAvaRundownPage& InPage, const FString& InChannelName) const
{
	FAvaRundownPageChannelChanged* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownPageChannelChanged>();
	ReplyMessage->Rundown = FSoftObjectPath(InRundown).ToString();
	ReplyMessage->PageId = InPage.GetPageId();
	ReplyMessage->ChannelName = InChannelName;
	SendResponse(ReplyMessage, ClientAddresses);
}

void FAvaRundownServer::PageNameChanged(const UAvaRundown* InRundown, const FAvaRundownPage& InPage, bool bInFriendlyName) const
{
	FAvaRundownPageNameChanged* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownPageNameChanged>();
	ReplyMessage->Rundown = FSoftObjectPath(InRundown).ToString();
	ReplyMessage->PageId = InPage.GetPageId();
	ReplyMessage->PageName = bInFriendlyName ? InPage.GetPageFriendlyName().ToString() : InPage.GetPageName();
	ReplyMessage->bFriendlyName = bInFriendlyName;
	SendResponse(ReplyMessage, ClientAddresses);
}

void FAvaRundownServer::PageAnimSettingsChanged(const UAvaRundown* InRundown, const FAvaRundownPage& InPage) const
{
	FAvaRundownPageAnimSettingsChanged* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownPageAnimSettingsChanged>();
	ReplyMessage->Rundown = FSoftObjectPath(InRundown).ToString();
	ReplyMessage->PageId = InPage.GetPageId();
	SendResponse(ReplyMessage, ClientAddresses);
}

void FAvaRundownServer::OnBroadcastChannelListChanged(const FAvaBroadcastProfile& InProfile) const
{
	if (ClientAddresses.IsEmpty())
	{
		return;
	}

	FAvaRundownChannelListChanged* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownChannelListChanged>();

	const TArray<FAvaBroadcastOutputChannel*>& OutputChannels = InProfile.GetChannels();
	
	ReplyMessage->Channels.Reserve(OutputChannels.Num());

	for (const FAvaBroadcastOutputChannel* OutputChannel : OutputChannels)
	{
		FAvaRundownChannel Channel = UE::AvaRundownServer::Private::SerializeChannel(*OutputChannel);
		ReplyMessage->Channels.Push(MoveTemp(Channel));
	}
	SendResponse(ReplyMessage, ClientAddresses);
}

void FAvaRundownServer::OnBroadcastChannelChanged(const FAvaBroadcastOutputChannel& InChannel, EAvaBroadcastChannelChange InChange) const
{
	if (ClientAddresses.IsEmpty())
	{
		return;
	}

	FAvaRundownChannelResponse* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownChannelResponse>();
	
	ReplyMessage->Channel = UE::AvaRundownServer::Private::SerializeChannel(InChannel);
	SendResponse(ReplyMessage, ClientAddresses);
}

void FAvaRundownServer::OnAssetAdded(const FAssetData& InAssetData) const
{
	NotifyAssetEvent(InAssetData, EAvaRundownAssetEvent::Added);
}

void FAvaRundownServer::OnAssetRemoved(const FAssetData& InAssetData) const
{
	NotifyAssetEvent(InAssetData, EAvaRundownAssetEvent::Removed);
}

void FAvaRundownServer::OnAssetsPreDelete(const TArray<UObject*>& InObjects)
{
	for (UObject* Object : InObjects)
	{
		if (const UAvaRundown* Rundown = Cast<UAvaRundown>(Object))
		{
			// Allow the edited rundown to be deleted.
			EditCommandContext.ConditionalFlush(SharedThis(this), Rundown);

			// Allow the playback rundown to be deleted, unless it is playing.
			if (!Rundown->IsPlaying())
			{
				PlaybackCommandContext.ConditionalFlush(SharedThis(this), Rundown);
			}
		}
	}
}

void FAvaRundownServer::HandleRundownPing(const FAvaRundownPing& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	if (!InMessage.bAuto)
	{
		UE_LOGF(LogAvaRundownServer, Log, "Received Ping request from %ls", *InContext->GetSender().ToString());
	}

	FAvaRundownPong* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownPong>();
	ReplyMessage->RequestId = InMessage.RequestId;
	ReplyMessage->bAuto = InMessage.bAuto;

	using namespace UE::AvaRundownServer::Private;

	// Consider clients that didn't request a version to be the latest version.
	const int32 RequestedApiVersion = InMessage.RequestedApiVersion != -1 ? InMessage.RequestedApiVersion : EAvaRundownApiVersion::LatestVersion;

	// Determine the version we will communicate with this client.
	int32 HonoredApiVersion = EAvaRundownApiVersion::LatestVersion;
	
	if (RequestedApiVersion >= CurrentMinimumApiVersion && RequestedApiVersion <= EAvaRundownApiVersion::LatestVersion)
	{
		HonoredApiVersion = RequestedApiVersion;
	}
	
	ReplyMessage->ApiVersion = HonoredApiVersion;
	ReplyMessage->MinimumApiVersion = CurrentMinimumApiVersion;
	ReplyMessage->LatestApiVersion = EAvaRundownApiVersion::LatestVersion;
	ReplyMessage->HostName = HostName;

	FClientInfo& ClientInfo = GetOrAddClientInfo(InContext->GetSender());
	ClientInfo.ApiVersion = HonoredApiVersion;

	SendResponse(ReplyMessage, InContext->GetSender());
}

void FAvaRundownServer::HandleGetRundownServerInfo(const FAvaRundownGetServerInfo& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	FAvaRundownServerInfo* ServerInfo = FMessageEndpoint::MakeMessage<FAvaRundownServerInfo>();
	ServerInfo->RequestId = InMessage.RequestId;
	
	const FClientInfo* ClientInfo = GetClientInfo(InContext->GetSender());
	ServerInfo->ApiVersion = ClientInfo ? ClientInfo->ApiVersion : EAvaRundownApiVersion::Unspecified;
	ServerInfo->MinimumApiVersion = UE::AvaRundownServer::Private::CurrentMinimumApiVersion;
	ServerInfo->LatestApiVersion = EAvaRundownApiVersion::LatestVersion;
	ServerInfo->HostName = HostName;
	ServerInfo->EngineVersion = FNetworkVersion::GetLocalNetworkVersion();
	ServerInfo->InstanceId = FApp::GetInstanceId();
	ServerInfo->InstanceBuild = UE::AvaRundownServer::Private::GetRundownEngineBuild();
	ServerInfo->InstanceMode = UE::AvaRundownServer::Private::GetRundownEngineMode();
	ServerInfo->SessionId = FApp::GetSessionId();
	ServerInfo->ProjectName = FApp::GetProjectName();
	ServerInfo->ProjectDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());

	const URemoteControlSettings* RCSettings = GetDefault<URemoteControlSettings>();
	ServerInfo->RemoteControlHttpServerPort = RCSettings->RemoteControlHttpServerPort;
	ServerInfo->RemoteControlWebSocketServerPort = RCSettings->RemoteControlWebSocketServerPort;

	SendResponse(ServerInfo, InContext->GetSender());
}

void FAvaRundownServer::HandleGetPlayableAssets(const FAvaRundownGetPlayableAssets& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	FAvaRundownPlayableAssets* Response = FMessageEndpoint::MakeMessage<FAvaRundownPlayableAssets>();
	Response->RequestId = InMessage.RequestId;
	
	if (const IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		FARFilter Filter;
		
		// todo: Add all supported playable asset types. Hardcoded for now, need a extensible factory system.
		Filter.ClassPaths.Add(UWorld::StaticClass()->GetClassPathName());
		
		TArray<FAssetData> Assets;
		AssetRegistry->GetAssets(Filter, Assets);

		for (const FAssetData& AssetData : Assets)
		{
			if (FAvaPlaybackUtils::IsPlayableAsset(AssetData))
			{
				if (!InMessage.Query.IsEmpty())
				{
					if (AssetData.AssetName.ToString().Contains(*InMessage.Query))
					{
						Response->Assets.Add(AssetData.ToSoftObjectPath());
					}
				}
				else
				{
					Response->Assets.Add(AssetData.ToSoftObjectPath());
				}

				if (InMessage.Limit > 0 && Response->Assets.Num() >= InMessage.Limit)
				{
					break;
				}
			}
		}
	}

	SendResponse(Response, InContext->GetSender());
}

void FAvaRundownServer::HandleGetRundowns(const FAvaRundownGetRundowns& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	FAvaRundownRundowns* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownRundowns>();

	ReplyMessage->RequestId = InMessage.RequestId;
	
	// List all the rundown assets.
	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		TArray<FAssetData> Assets;
		if (AssetRegistry->GetAssetsByClass(UAvaRundown::StaticClass()->GetClassPathName(), Assets))
		{
			ReplyMessage->Rundowns.Reserve(Assets.Num());		
			for (const FAssetData& Data : Assets)
			{
				ReplyMessage->Rundowns.Add(Data.ToSoftObjectPath().ToString());
			}
		}
	}

	// Adding the managed rundowns as well, in case they are not listed in the asset registry.
	for (const TPair<FSoftObjectPath, TObjectPtr<UAvaRundown>>& ManagedRundown : ManagedRundowns)
	{
		if (IsValid(ManagedRundown.Value))
		{
			ReplyMessage->Rundowns.AddUnique(ManagedRundown.Key.ToString());
		}
	}

	SendResponse(ReplyMessage, InContext->GetSender());
}

void FAvaRundownServer::HandleLoadRundown(const FAvaRundownLoadRundown& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	// If the requested path is empty, we assume this is a request for information only.
	if (!InMessage.Rundown.IsEmpty())
	{
		const FSoftObjectPath NewRundownPath(InMessage.Rundown);
		const UAvaRundown* Rundown = GetOrLoadRundownForContext(NewRundownPath, PlaybackCommandContext);
		
		if (!Rundown)
		{
			SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
				TEXT("Rundown \"%s\" not loaded."), *InMessage.Rundown);
			return;
		}
	}

	SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log,
		TEXT("Rundown \"%s\" loaded."), *PlaybackCommandContext.GetCurrentRundownPath().ToString());
}

void FAvaRundownServer::HandleCreateRundown(const FAvaRundownCreateRundown& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	if (InMessage.PackagePath.IsEmpty() || InMessage.AssetName.IsEmpty())
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"CreateRundown\" Failed: No rundown path/name specified."));
		return;
	}

	using namespace UE::AvaRundownServer;
	
	FString PackageName = Private::SanitizePackageName(InMessage.PackagePath + TEXT("/") + InMessage.AssetName);

#if WITH_EDITOR
	const bool bTransient = InMessage.bTransient;
#else
	const bool bTransient = true;
#endif

	if (bTransient)
	{
		static const FString GamePath(TEXT("/Game"));
		if (PackageName.StartsWith(GamePath))	// ignore case
		{
			PackageName = PackageName.Mid(GamePath.Len());
		}

		static const FString TempPath(TEXT("/Temp"));
		
		// Ensure the path begins with /Temp
		if (!PackageName.StartsWith(TempPath))	// ignore case
		{
			PackageName = Private::SanitizePackageName(TempPath + TEXT("/") + PackageName);
		}
	}

	if (FindPackage(nullptr, *PackageName))
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"CreateRundown\" Failed: Requested package \"%s\" already exists."), *PackageName);
		return;
	}

	UPackage* const RundownPackage = CreatePackage(*PackageName);

	if (!RundownPackage)
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"CreateRundown\" Failed: Requested package \"%s\" could not be created."), *PackageName);
		return;
	}
		
	if (bTransient)
	{
		RundownPackage->SetFlags(RF_Transient);
	}

	constexpr EObjectFlags AssetFlags = RF_Public|RF_Standalone|RF_Transactional;
	constexpr EObjectFlags TransientFlags = RF_Public|RF_Transactional;
	UAvaRundown* Rundown = NewObject<UAvaRundown>(RundownPackage, FName(*InMessage.AssetName), bTransient ? TransientFlags : AssetFlags);

	if (!bTransient)
	{
		FAssetRegistryModule::AssetCreated(Rundown);
		RundownPackage->MarkPackageDirty();
	}

	// The created rundown is added to a managed list to be kept alive by the server as long as it is running.
	const FSoftObjectPath RundownPath(Rundown);

	if (bTransient)
	{
		ManagedRundowns.Add(RundownPath, Rundown);
	}

	SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log,
	TEXT("Rundown \"%s\" Created."), *RundownPath.ToString());
}

void FAvaRundownServer::HandleDeleteRundown(const FAvaRundownDeleteRundown& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	// Deleting requires explicit specification of the rundown.
	if (InMessage.Rundown.IsEmpty())
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"DeleteRundown\": Rundown asset not specified."));
		return;
	}
	
	FSoftObjectPath RundownPath(InMessage.Rundown);

	// Only allow rundowns to be deleted if not playing.
	// We will require an explicit stop command for security reasons.
	UAvaRundown* Rundown = Cast<UAvaRundown>(RundownPath.ResolveObject());
	if (Rundown && Rundown->IsPlaying())
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"DeleteRundown\": Rundown is currently playing. It must be stopped first."));
		return;
	}

	if (ManagedRundowns.Remove(RundownPath) > 0)
	{
		// Also, flush command contexts if associated to this rundown.
		EditCommandContext.ConditionalFlush(SharedThis(this), RundownPath);
		PlaybackCommandContext.ConditionalFlush(SharedThis(this), RundownPath);
		CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );
		
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log,
			TEXT("\"DeleteRundown\": Rundown \"%s\" removed."), *InMessage.Rundown);
		return;
	}
	
#if WITH_EDITOR
	FAssetData RundownAsset;
	if (FAssetRegistryModule::GetRegistry().TryGetAssetByObjectPath(RundownPath, RundownAsset) == UE::AssetRegistry::EExists::Exists)
	{
		TArray<FAssetData> AssetData;
		AssetData.Add(RundownAsset);
		int32 NumDeleted = ObjectTools::DeleteAssets(AssetData, /*bShowConfirmation*/ false);

		if (NumDeleted == AssetData.Num())
		{
			SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log,
				TEXT("\"DeleteRundown\": Rundown \"%s\" deleted."), *InMessage.Rundown);
		}
		else
		{
			SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
				TEXT("\"DeleteRundown\": Rundown \"%s\" could not be deleted."), *InMessage.Rundown);
		}
		return;
	}
#endif
	
	SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
		TEXT("\"DeleteRundown\": Rundown \"%s\" not found."), *InMessage.Rundown);
}

void FAvaRundownServer::HandleImportRundown(const FAvaRundownImportRundown& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaRundown* Rundown = GetOrLoadRundownForEdit(InContext->GetSender(), InMessage.RequestId, InMessage.Rundown);
	if (!Rundown)
	{
		return;	// Response sent by GetOrLoadRundownForEdit.
	}

	using namespace UE::AvaMedia::RundownSerializationUtils;

	// Load from file
	if (!InMessage.RundownFile.IsEmpty())
	{
		FText ErrorMessage;
		if (LoadRundownFromJson(Rundown, *InMessage.RundownFile, ErrorMessage))
		{
			SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log,
				TEXT("\"ImportRundown\": Loaded from file \"%s\"."), *InMessage.RundownFile);
		}
		else
		{
			SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
				TEXT("\"ImportRundown\": Failed to load from file \"%s\". Reason: %s"), *InMessage.RundownFile, *ErrorMessage.ToString());
		}
		return;
	}

	// Load from data
	if (!InMessage.RundownData.IsEmpty())
	{
		FText ErrorMessage;
		FMemoryReaderView Reader(UE::AvaMediaSerializationUtils::JsonValueConversion::ValueToConstBytesView(InMessage.RundownData));
		if (LoadRundownFromJson(Rundown, Reader, ErrorMessage))
		{
			SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log,
				TEXT("\"ImportRundown\": Loaded from data."));
		}
		else
		{
			SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
				TEXT("\"ImportRundown\": Failed to load from data. Reason: %s"), *ErrorMessage.ToString());
		}
		return;
	}

	SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
		TEXT("\"ImportRundown\": No data was provided to import from. Either a filename or json data must be provided."));
}

void FAvaRundownServer::HandleExportRundown(const FAvaRundownExportRundown& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	const UAvaRundown* Rundown = GetOrLoadRundownForEdit(InContext->GetSender(), InMessage.RequestId, InMessage.Rundown);
	if (!Rundown)
	{
		return; // Response sent by GetOrLoadRundownForEdit.
	}
	
	using namespace UE::AvaMedia::RundownSerializationUtils;

	FText ErrorMessage;
	
	// Export to file on the server.
	if (!InMessage.RundownFile.IsEmpty())
	{
		if (SaveRundownToJson(Rundown, *InMessage.RundownFile, ErrorMessage))
		{
			SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log,
				TEXT("\"ExportRundown\": Rundown exported to \"%s\"."), *InMessage.RundownFile);
		}
		else
		{
			SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
				TEXT("\"ExportRundown\": Failed to export rundown to \"%s\". Reason: %s"), *InMessage.RundownFile, *ErrorMessage.ToString());
		}
		return;
	}
	
	TArray<uint8> RundownDataAsBytes;
	FMemoryWriter Writer(RundownDataAsBytes);
	if (SaveRundownToJson(Rundown, Writer, ErrorMessage))
	{
		FAvaRundownExportedRundown* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownExportedRundown>();
		ReplyMessage->RequestId = InMessage.RequestId;
		ReplyMessage->Rundown = InMessage.Rundown;
		UE::AvaMediaSerializationUtils::JsonValueConversion::BytesToString(RundownDataAsBytes, ReplyMessage->RundownData);
		SendResponse(ReplyMessage, InContext->GetSender());
	}
	else
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"ExportRundown\": Failed to export rundown. Reason: %s"), *ErrorMessage.ToString());
	}
}

void FAvaRundownServer::HandleSaveRundown(const FAvaRundownSaveRundown& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
#if WITH_EDITOR
	UEditorAssetSubsystem* EditorAssetSubsystem = GEditor ? GEditor->GetEditorSubsystem<UEditorAssetSubsystem>() : nullptr;
	
	if (!EditorAssetSubsystem)
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("Saving assets is only available in editor mode."));
		return;
	}
	
	if (InMessage.Rundown.IsEmpty())
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("Rundown asset not specified."));
		return;
	}

	const FSoftObjectPath RundownAssetPath(InMessage.Rundown);

	if (!RundownAssetPath.IsValid())
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("Rundown asset path \"%s\" is not valid."), *InMessage.Rundown);
		return;
	}

	UObject* FoundObject = RundownAssetPath.ResolveObject();

	if (!FoundObject)
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("Rundown asset \"%s\" is not loaded."), *InMessage.Rundown);
		return;
	}

	UAvaRundown* FoundRundown = Cast<UAvaRundown>(FoundObject);
	
	if (!FoundRundown)
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("Asset path \"%s\" is loaded but is not a Rundown asset."), *InMessage.Rundown);
		return;
	}
	
	if (!EditorAssetSubsystem->SaveLoadedAsset(FoundRundown, InMessage.bOnlyIfIsDirty))
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("Unable to save asset \"%s\" to location \"%s\"."), *FoundRundown->GetName(), *RundownAssetPath.ToString());
		return;
	}

	SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log,
		TEXT("Asset \"%s\" save to location \"%s\"."), *FoundRundown->GetName(), *RundownAssetPath.ToString());
#else
	SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("Save rundown is not available in game build."));
#endif
}

void FAvaRundownServer::HandleGetPages(const FAvaRundownGetPages& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	const UAvaRundown* Rundown = GetOrLoadRundownForEdit(InContext->GetSender(), InMessage.RequestId, InMessage.Rundown);
	if (!Rundown)
	{
		return; // Response sent by GetOrLoadRundownForEdit.
	}
	
	FAvaRundownPages* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownPages>();
	ReplyMessage->RequestId = InMessage.RequestId;
	ReplyMessage->Pages.Reserve(Rundown->GetInstancedPages().Pages.Num() + Rundown->GetTemplatePages().Pages.Num());
	for (const FAvaRundownPage& Page : Rundown->GetInstancedPages().Pages)
	{
		FAvaRundownPageInfo PageInfo = UE::AvaRundownServer::Private::GetPageInfo(Rundown, Page);
		ReplyMessage->Pages.Add(MoveTemp(PageInfo));
	}
	
	for (const FAvaRundownPage& Page : Rundown->GetTemplatePages().Pages)
	{
		FAvaRundownPageInfo PageInfo = UE::AvaRundownServer::Private::GetPageInfo(Rundown, Page);
		ReplyMessage->Pages.Add(MoveTemp(PageInfo));
	}
	
	SendResponse(ReplyMessage, InContext->GetSender());
}

void FAvaRundownServer::HandleCreatePage(const FAvaRundownCreatePage& InMessage,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaRundown* Rundown = GetOrLoadRundownForEdit(InContext->GetSender(), InMessage.RequestId, InMessage.Rundown);
	if (!Rundown)
	{
		return; // Response sent by GetOrLoadRundownForEdit.
	}

	const FAvaRundownPage& Template = Rundown->GetPage(InMessage.TemplateId);
    if (!Template.IsValidPage() || !Template.IsTemplate())
    {
    	LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("Template %d is not valid or is not a template"), InMessage.TemplateId);
    	return;
    }
	
	const int32 PageId = Rundown->AddPageFromTemplate(InMessage.TemplateId);
	if (PageId != FAvaRundownPage::InvalidPageId)
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log, TEXT("Page %d Created from Template %d"), PageId, InMessage.TemplateId);
	}
	else
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("Failed to create a page from Template %d"), InMessage.TemplateId);
	}
}

void FAvaRundownServer::HandleDeletePage(const FAvaRundownDeletePage& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaRundown* Rundown = GetOrLoadRundownForEdit(InContext->GetSender(), InMessage.RequestId, InMessage.Rundown);
	if (!Rundown)
	{
		return; // Response sent by GetOrLoadRundownForEdit.
	}

	const FAvaRundownPage& Page = Rundown->GetPage(InMessage.PageId);
	if (!Page.IsValidPage())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("page %d is not valid"), InMessage.PageId);
		return;
	}

	if (Rundown->RemovePage(InMessage.PageId))
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log, TEXT("Page %d deleted"), InMessage.PageId);
	}
	else
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("page %d can't be deleted"), InMessage.PageId);
	}
}

void FAvaRundownServer::HandleDeleteTemplate(const FAvaRundownDeleteTemplate& InMessage,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaRundown* Rundown = GetOrLoadRundownForEdit(InContext->GetSender(), InMessage.RequestId, InMessage.Rundown);
	if (!Rundown)
	{
		return; // Response sent by GetOrLoadRundownForEdit.
	}

	const FAvaRundownPage& Page = Rundown->GetPage(InMessage.PageId);
	if (!Page.IsValidPage())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("page %d is not valid"), InMessage.PageId);
		return;
	}

	if (!Page.IsTemplate())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("page %d is not a template"), InMessage.PageId);
		return;
	}

	if (!Page.GetInstancedIds().IsEmpty())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("Template has instanced pages"), InMessage.PageId);
		return;
	}

	if (Rundown->RemovePage(InMessage.PageId))
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log, TEXT("Page template %d deleted"), InMessage.PageId);
	}
	else
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("Page template %d can't be deleted"), InMessage.PageId);
	}
}

void FAvaRundownServer::HandleCreateTemplate(const FAvaRundownCreateTemplate& InMessage,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaRundown* Rundown = GetOrLoadRundownForEdit(InContext->GetSender(), InMessage.RequestId, InMessage.Rundown);
	if (!Rundown)
	{
		return; // Response sent by GetOrLoadRundownForEdit.
	}

	const FAvaRundownPageIdGeneratorParams IdGeneratorParams(InMessage.IdGeneratorParams.ReferenceId, InMessage.IdGeneratorParams.Increment);
	const FString& AssetPath = InMessage.AssetPath;
	FString ErrorString;

	// Note: using AddTemplateInternal because it doesn't add the template if there is an error and we can capture the error in the lambda.
	const int32 TemplateId = Rundown->AddTemplateInternal(IdGeneratorParams, [&AssetPath, &ErrorString](FAvaRundownPage& InNewTemplate)
	{
		if (!AssetPath.IsEmpty())
		{
			constexpr bool bInReimportPage = true; // Ensures the asset is updated in the template.
			if (!InNewTemplate.UpdateAsset(AssetPath, bInReimportPage))
			{
				ErrorString = FString::Printf(TEXT("asset \"%s\" is invalid"), *AssetPath);
				return false;
			}
		}
		return true;
	});
	
	if (TemplateId != FAvaRundownPage::InvalidPageId)
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log, TEXT("Template %d Created"), TemplateId);
	}
	else if (!ErrorString.IsEmpty())
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("Failed to create a new template: %s"), *ErrorString);
	}
	else
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("Failed to create a new template"));
	}
}

class FAvaRundownServerErrorContext : public FOutputDevice
{
public:
	TArray<FString> Errors;
	
	virtual void Serialize(const TCHAR* InText, ELogVerbosity::Type InVerbosity, const class FName& InCategory) override
	{
		Errors.Add(InText);
	}
};

void FAvaRundownServer::HandleCreateComboTemplate(const FAvaRundownCreateComboTemplate& InMessage,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaRundown* Rundown = GetOrLoadRundownForEdit(InContext->GetSender(), InMessage.RequestId, InMessage.Rundown);
	if (!Rundown)
	{
		return; // Response sent by GetOrLoadRundownForEdit.
	}

	FAvaRundownServerErrorContext ErrorContext;
	const TArray<int32> TemplateIds = Rundown->ValidateTemplateIdsForComboTemplate(InMessage.CombinedTemplateIds, ErrorContext);

	if (TemplateIds.Num() > 1)
	{
		const FAvaRundownPageIdGeneratorParams IdGeneratorParams(InMessage.IdGeneratorParams.ReferenceId, InMessage.IdGeneratorParams.Increment);
		const int32 TemplateId = Rundown->AddComboTemplate(TemplateIds, IdGeneratorParams);
	
		if (TemplateId != FAvaRundownPage::InvalidPageId)
		{
			SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log, TEXT("Template %d Created"), TemplateId);
		}
		else
		{
			SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("Failed to create a new combo template"));
		}
	}
	else
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("Need at least 2 suitable templates to create a combo template: %s"), *FString::Join(ErrorContext.Errors, TEXT("; ")));
	}
}

void FAvaRundownServer::HandleChangeTemplateBP(const FAvaRundownChangeTemplateBP& InMessage,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaRundown* Rundown = GetOrLoadRundownForEdit(InContext->GetSender(), InMessage.RequestId, InMessage.Rundown);
	if (!Rundown)
	{
		return; // Response sent by GetOrLoadRundownForEdit.
	}

	FAvaRundownPage& Page = Rundown->GetPage(InMessage.TemplateId);

	if (!Page.IsValidPage())
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("Asset change of template failed: PageId %d is not a valid page."), InMessage.TemplateId);
		return;
	}

	if (!Page.IsTemplate())
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("Asset change of template failed: PageId %d is not a template."), InMessage.TemplateId);
		return;
	}
	
	if (Page.UpdateAsset(InMessage.AssetPath, InMessage.bReimport))
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log, TEXT("Asset change of template: %d to %s"), InMessage.TemplateId, *InMessage.AssetPath);
		Rundown->GetOnPagesChanged().Broadcast(Rundown, Page, EAvaRundownPageChanges::Blueprint);
	}
	else
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("Asset change of template: %d to %s failed."), InMessage.TemplateId, *InMessage.AssetPath);	
	}
}

void FAvaRundownServer::HandleGetPageDetails(const FAvaRundownGetPageDetails& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{	
	UAvaRundown* Rundown = GetOrLoadRundownForEdit(InContext->GetSender(), InMessage.RequestId, InMessage.Rundown);
	if (!Rundown)
	{
		return; // Response sent by GetOrLoadRundownForEdit.
	}

	const FAvaRundownPage& Page = Rundown->GetPage(InMessage.PageId);
	if (!Page.IsValidPage())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"PageDetails\" not available: PageId %d is invalid."), InMessage.PageId);
		return;
	}

	if (Page.GetAssetPath(Rundown).IsNull())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("Page has no asset selected"), InMessage.PageId);
		return;
	}
	
	if (InMessage.bLoadRemoteControlPreset)
	{
		const TArray<FSoftObjectPath> AssetPaths = Page.GetAssetPaths(Rundown);
		
		FAvaRundownManagedInstanceCache& ManagedInstanceCache = IAvaMediaModule::Get().GetManagedInstanceCache();

		FAvaRundownManagedInstanceHandles ManagedHandles = ManagedInstanceCache.GetManagedHandlesForPage(Rundown, Page);

		if (!ManagedHandles.IsEmpty())
		{
			EditCommandContext.SaveCurrentRemoteControlPresetToPage(true);

			for (const TSharedPtr<FAvaRundownManagedInstance>& ManagedInstance : ManagedHandles.Instances)
			{
				// Applying the controller values can break the WYSIWYG of the editor,
				// in case multiple controllers set the same exposed entity with different values.
				// There is no guaranty that the controller actions are self consistent.
				// To avoid this issue, we apply the controllers first, and then
				// restore the entity values in a second pass.
			
				Page.GetRemoteControlValues().ApplyControllerValuesToRemoteControlPreset(ManagedInstance->GetRemoteControlPreset(), true);
				Page.GetRemoteControlValues().ApplyEntityValuesToRemoteControlPreset(ManagedInstance->GetRemoteControlPreset());

				// Register the RC Preset to Remote Control module to make it available through WebRC.
				FAvaRemoteControlUtils::RegisterRemoteControlPreset(ManagedInstance->GetRemoteControlPreset(), /*bInEnsureUniqueId*/ true);
			}

			// Keep track of what is currently registered.
			EditCommandContext.ManagedPageId = InMessage.PageId;
			EditCommandContext.ManagedHandles = MoveTemp(ManagedHandles);
		}
	}
	
	FAvaRundownPageDetails* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownPageDetails>();
	ReplyMessage->RequestId = InMessage.RequestId;
	ReplyMessage->Rundown = InMessage.Rundown;
	ReplyMessage->PageInfo = UE::AvaRundownServer::Private::GetPageInfo(Rundown, Page);
	ReplyMessage->RemoteControlValues = Page.GetRemoteControlValues();
	
	if (InMessage.bLoadRemoteControlPreset)
	{
		ReplyMessage->RemoteControlPresetNames.Reserve(EditCommandContext.ManagedHandles.Num());
		ReplyMessage->RemoteControlPresetIds.Reserve(EditCommandContext.ManagedHandles.Num());

		for (const TSharedPtr<FAvaRundownManagedInstance>& ManagedInstance : EditCommandContext.ManagedHandles.Instances)
		{
			if (ManagedInstance->GetRemoteControlPreset())
			{
				ReplyMessage->RemoteControlPresetNames.Add(ManagedInstance->GetRemoteControlPreset()->GetPresetName().ToString());
				ReplyMessage->RemoteControlPresetIds.Add(ManagedInstance->GetRemoteControlPreset()->GetPresetId().ToString());
			}
		}

		ReplyMessage->RemoteControlPresetName = !ReplyMessage->RemoteControlPresetNames.IsEmpty() ? ReplyMessage->RemoteControlPresetNames[0] : FString();
		ReplyMessage->RemoteControlPresetId = !ReplyMessage->RemoteControlPresetIds.IsEmpty() ? ReplyMessage->RemoteControlPresetIds[0] : FString();
	}
	SendResponse(ReplyMessage, InContext->GetSender());
}

void FAvaRundownServer::HandleChangePageChannel(const FAvaRundownPageChangeChannel& InMessage,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	if (InMessage.ChannelName.IsEmpty())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("No Channel Name Provided"));
		return;
	}

	const FName ChannelName(InMessage.ChannelName);
	const FAvaBroadcastOutputChannel& Channel = Broadcast.GetCurrentProfile().GetChannel(ChannelName);
	if (!Channel.IsValidChannel())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("%s is not a valid channel"), *ChannelName.ToString());
		return;
	}

	UAvaRundown* Rundown = GetOrLoadRundownForEdit(InContext->GetSender(), InMessage.RequestId, InMessage.Rundown);
	if (!Rundown)
	{
		return; // Response sent by GetOrLoadRundownForEdit.
	}

	FAvaRundownPage& Page = Rundown->GetPage(InMessage.PageId);
	if (!Page.IsValidPage())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("PageId %d is invalid."), InMessage.PageId);
		return;
	}

	if (Page.GetChannelName() == ChannelName)
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log, TEXT("Same Channel Selected"), InMessage.PageId);
		return;
	}

	Page.SetChannelName(ChannelName);
	Rundown->GetOnPagesChanged().Broadcast(Rundown, Page, EAvaRundownPageChanges::Channel);
	SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log, TEXT("Channel changed"));
}

void FAvaRundownServer::HandleChangePageName(const FAvaRundownChangePageName& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaRundown* Rundown = GetOrLoadRundownForEdit(InContext->GetSender(), InMessage.RequestId, InMessage.Rundown);
	if (!Rundown)
	{
		return; // Response sent by GetOrLoadRundownForEdit.
	}

	FAvaRundownPage& Page = Rundown->GetPage(InMessage.PageId);
	if (!Page.IsValidPage())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("PageId %d is invalid."), InMessage.PageId);
		return;
	}

	if (InMessage.bSetFriendlyName)
	{
		Page.RenameFriendlyName(InMessage.PageName);
		Rundown->GetOnPagesChanged().Broadcast(Rundown, Page, EAvaRundownPageChanges::FriendlyName);
	}
	else
	{
		Page.Rename(InMessage.PageName);
		Rundown->GetOnPagesChanged().Broadcast(Rundown, Page, EAvaRundownPageChanges::Name);
	}

	SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log,
		TEXT("Page %d Name Changed to \"%s\""), InMessage.PageId, *InMessage.PageName);
}

void FAvaRundownServer::HandleUpdatePageFromRCP(const FAvaRundownUpdatePageFromRCP& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	// Note that this doesn't save the rundown.
	EditCommandContext.SaveCurrentRemoteControlPresetToPage(InMessage.bUnregister);
	if (InMessage.bUnregister)
	{
		EditCommandContext.ManagedPageId = FAvaRundownPage::InvalidPageId;
		EditCommandContext.ManagedHandles.Reset();
	}
	SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log, TEXT("\"UpdatePageFromRCP\" Ok."));
}

void FAvaRundownServer::HandlePageAction(const FAvaRundownPageAction& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	const FRequestInfo RequestInfo = {InMessage.RequestId, InContext->GetSender()};
	HandlePageActions(RequestInfo, {InMessage.PageId}, false, FName(), InMessage.Action);
}

void FAvaRundownServer::HandlePagePreviewAction(const FAvaRundownPagePreviewAction& InMessage,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	const FRequestInfo RequestInfo = {InMessage.RequestId, InContext->GetSender()};
	HandlePageActions(RequestInfo, {InMessage.PageId}, true, FName(InMessage.PreviewChannelName), InMessage.Action);
}

void FAvaRundownServer::HandlePageActions(const FAvaRundownPageActions& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	const FRequestInfo RequestInfo = {InMessage.RequestId, InContext->GetSender()};
	HandlePageActions(RequestInfo, InMessage.PageIds, false, FName(), InMessage.Action);
}

void FAvaRundownServer::HandlePagePreviewActions(const FAvaRundownPagePreviewActions& InMessage,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	const FRequestInfo RequestInfo = {InMessage.RequestId, InContext->GetSender()};
	HandlePageActions(RequestInfo, InMessage.PageIds, true, FName(InMessage.PreviewChannelName), InMessage.Action);
}

void FAvaRundownServer::HandleTransitionAction(const FAvaRundownTransitionAction& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	const FRequestInfo RequestInfo = {InMessage.RequestId, InContext->GetSender()};
	const FName ActionName = StaticEnum<EAvaRundownTransitionActions>()->GetNameByValue((int64)InMessage.Action);

	bool bSuccess = false;
	FString FailureReason;

	// This is the response handler for this command.
	auto HandleCommandResponse = [this, &RequestInfo, &bSuccess, &FailureReason, ActionName]()
	{
		if (bSuccess)
		{
			SendMessage(RequestInfo.Sender, RequestInfo.RequestId, ELogVerbosity::Log,
				TEXT("Transition Action \"%s\": Ok."), *ActionName.ToString());
		}
		else if (!FailureReason.IsEmpty())
		{
			LogAndSendMessage(RequestInfo.Sender, RequestInfo.RequestId, ELogVerbosity::Error,
				TEXT("Transition Action \"%s\" Failed. Reason: %s"), *ActionName.ToString(), *FailureReason);
		}
		else
		{
			LogAndSendMessage(RequestInfo.Sender, RequestInfo.RequestId, ELogVerbosity::Error,
				TEXT("Transition Action \"%s\" Failed."), *ActionName.ToString());
		}
	};
	
	UAvaRundown* Rundown = PlaybackCommandContext.GetCurrentRundown();
	
	if (!Rundown)
	{
		FailureReason = TEXT("no rundown currently loaded for playback.");
		HandleCommandResponse();
		return;
	}

	// Validate that the channel is specified and exists.
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	if (InMessage.ChannelName.IsEmpty())
	{
		FailureReason = TEXT("No Channel Name Provided");
		HandleCommandResponse();
		return;
	}

	const FName ChannelName(InMessage.ChannelName);
	const FAvaBroadcastOutputChannel& Channel = Broadcast.GetCurrentProfile().GetChannel(ChannelName);
	if (!Channel.IsValidChannel())
	{
		FailureReason = FString::Printf(TEXT("\"%s\" is not a valid channel"), *ChannelName.ToString());
		HandleCommandResponse();
		return;
	}

	if (InMessage.Action == EAvaRundownTransitionActions::ForceStop)
	{
		const int32 NumTransitions = Rundown->StopPageTransitionsForChannel(ChannelName);
		if (NumTransitions > 0)
		{
			bSuccess = true;
		}
		else
		{
			FailureReason = TEXT("No Transitions were stopped");
		}
	}
	else
	{
		FailureReason = TEXT("Invalid action");
	}

	HandleCommandResponse();
}

void FAvaRundownServer::HandleTransitionLayerAction(const FAvaRundownTransitionLayerAction& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	const FRequestInfo RequestInfo = {InMessage.RequestId, InContext->GetSender()};

	const FName ActionName = StaticEnum<EAvaRundownTransitionLayerActions>()->GetNameByValue((int64)InMessage.Action);

	bool bSuccess = false;
	FString FailureReason;

	// This is the response handler for this command.
	auto HandleCommandResponse = [this, &RequestInfo, &bSuccess, &FailureReason, ActionName]()
	{
		if (bSuccess)
		{
			SendMessage(RequestInfo.Sender, RequestInfo.RequestId, ELogVerbosity::Log,
				TEXT("Transition Layer Action \"%s\": Ok."), *ActionName.ToString());
		}
		else if (!FailureReason.IsEmpty())
		{
			LogAndSendMessage(RequestInfo.Sender, RequestInfo.RequestId, ELogVerbosity::Error,
				TEXT("Transition Layer Action \"%s\" Failed. Reason: %s"), *ActionName.ToString(), *FailureReason);
		}
		else
		{
			LogAndSendMessage(RequestInfo.Sender, RequestInfo.RequestId, ELogVerbosity::Error,
				TEXT("Transition Layer Action \"%s\" Failed."), *ActionName.ToString());
		}
	};

	UAvaRundown* Rundown = PlaybackCommandContext.GetCurrentRundown();
	
	if (!Rundown)
	{
		FailureReason = TEXT("no rundown currently loaded for playback.");
		HandleCommandResponse();
		return;
	}
	
	// Validate that the channel is specified and exists.
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	if (InMessage.ChannelName.IsEmpty())
	{
		FailureReason = TEXT("No Channel Name Provided");
		HandleCommandResponse();
		return;
	}

	const FName ChannelName(InMessage.ChannelName);
	const FAvaBroadcastOutputChannel& Channel = Broadcast.GetCurrentProfile().GetChannel(ChannelName);
	if (!Channel.IsValidChannel())
	{
		FailureReason = FString::Printf(TEXT("\"%s\" is not a valid channel"), *ChannelName.ToString());
		HandleCommandResponse();
		return;
	}

	if (InMessage.Action == EAvaRundownTransitionLayerActions::Stop
		|| InMessage.Action == EAvaRundownTransitionLayerActions::ForceStop)
	{
		// We need to gather playing layer handles that correspond to the layer names from the command.
		TArray<FAvaTagHandle> Layers;
		Layers.Reserve(InMessage.LayerNames.Num());
	
		for (UAvaRundownPagePlayer* PagePlayer : Rundown->GetPagePlayers())
		{
			if (PagePlayer->ChannelFName != ChannelName)
			{
				continue;
			}

			PagePlayer->ForEachInstancePlayer([&Layers, &LayerNames = InMessage.LayerNames](UAvaRundownPlaybackInstancePlayer* InInstancePlayer)
			{
				const FString LayerName = InInstancePlayer->TransitionLayer.ToString();
				if (LayerNames.Contains(LayerName))
				{
					const bool bAlreadyAdded = Layers.ContainsByPredicate([&OtherTagHandle = InInstancePlayer->TransitionLayer](const FAvaTagHandle& InTagHandle)
					{
						return InTagHandle.MatchesExact(OtherTagHandle);
					});
					if (!bAlreadyAdded)
					{
						Layers.Add(InInstancePlayer->TransitionLayer);
					}
				}
			});
		}

		if (Layers.IsEmpty())
		{
			FailureReason = TEXT("No playing layers corresponding to given layer names were found.");
		}
		else
		{
			EAvaRundownPageStopOptions StopOptions = InMessage.Action == EAvaRundownTransitionLayerActions::ForceStop ?
				EAvaRundownPageStopOptions::ForceNoTransition : EAvaRundownPageStopOptions::Default;
		
			const TArray<int32> StoppedPages = Rundown->StopLayers(ChannelName, Layers, StopOptions);
			if (StoppedPages.Num() > 0)
			{
				bSuccess = true;
			}
			else
			{
				FailureReason = TEXT("No Pages were stopped");
			}
		}
	}
	else
	{
		FailureReason = TEXT("Invalid action");
	}

	HandleCommandResponse();
}

void FAvaRundownServer::HandleGetProfiles(const FAvaRundownGetProfiles& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	const UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	FAvaRundownProfiles* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownProfiles>();
	ReplyMessage->RequestId = InMessage.RequestId;
	ReplyMessage->Profiles.Reserve(Broadcast.GetProfiles().Num());
	for (const TPair<FName, FAvaBroadcastProfile>& Profile : Broadcast.GetProfiles())
	{
		ReplyMessage->Profiles.Add(Profile.Key.ToString());
	}
	ReplyMessage->CurrentProfile = Broadcast.GetCurrentProfileName().ToString();
	SendResponse(ReplyMessage, InContext->GetSender());
}

void FAvaRundownServer::HandleCreateProfile(const FAvaRundownCreateProfile& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	const FName ProfileName(InMessage.ProfileName);
	
	if (Broadcast.GetProfile(ProfileName).IsValidProfile())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"CreateProfile\" Failed. Reason: Profile \"%s\" already exist."), *InMessage.ProfileName);
		return;
	}

	Broadcast.CreateProfile(ProfileName, InMessage.bMakeCurrent);	// Always succeed apparently.
	LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log,
		TEXT("\"CreateProfile\" Profile \"%s\" created."), *InMessage.ProfileName);
}

void FAvaRundownServer::HandleDuplicateProfile(const FAvaRundownDuplicateProfile& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	const FName SourceProfileName(InMessage.SourceProfileName);
	const FName NewProfileName(InMessage.NewProfileName);
	
	if (!Broadcast.GetProfile(SourceProfileName).IsValidProfile())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"DuplicateProfile\" Failed. Reason: Source Profile \"%s\" does not exist."), *InMessage.SourceProfileName);
		return;
	}

	if (Broadcast.GetProfile(NewProfileName).IsValidProfile())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"DuplicateProfile\" Failed. Reason: Destination Profile \"%s\" already exist."), *InMessage.NewProfileName);
		return;
	}

	if (NewProfileName.IsNone())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"DuplicateProfile\" Failed. Reason: Destination Profile Name is empty."));
		return;
	}
	
	if (!Broadcast.DuplicateProfile(NewProfileName,  SourceProfileName, InMessage.bMakeCurrent))
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"DuplicateProfile\" Failed to duplicate \"%s\" from \"%s\" (Reason unknown)."), *InMessage.NewProfileName, *InMessage.SourceProfileName);
		return;
	}
	
	LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log,
		TEXT("\"DuplicateProfile\" Profile \"%s\" duplicated from \"%s\"."), *InMessage.NewProfileName, *InMessage.SourceProfileName);
}

void FAvaRundownServer::HandleRenameProfile(const FAvaRundownRenameProfile& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	const FName OldProfileName(InMessage.OldProfileName);
	const FName NewProfileName(InMessage.NewProfileName);
	FText FailReason;

	// CanRenameProfile doesn't check if the profile exists.
	if (!Broadcast.GetProfile(OldProfileName).IsValidProfile())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"RenameProfile\" Failed. Reason: Profile \"%s\" does not exist."), *InMessage.OldProfileName);
		return;
	}

	if (!Broadcast.CanRenameProfile(OldProfileName, NewProfileName, &FailReason))
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"RenameProfile\" Failed to rename profile \"%s\" to \"%s\". Reason: %s."),
			*InMessage.OldProfileName, *InMessage.NewProfileName, *FailReason.ToString());
		return;
	}

	Broadcast.RenameProfile(OldProfileName, NewProfileName);
	LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log,
		TEXT("\"RenameProfile\" Profile \"%s\" renamed to \"%s\"."), *InMessage.OldProfileName, *InMessage.NewProfileName);
}

void FAvaRundownServer::HandleDeleteProfile(const FAvaRundownDeleteProfile& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	const FName ProfileName(InMessage.ProfileName);
	
	if (!Broadcast.GetProfile(ProfileName).IsValidProfile())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"DeleteProfile\" Failed. Reason: Profile \"%s\" does not exist."), *InMessage.ProfileName);
		return;
	}

	if (Broadcast.GetCurrentProfileName() == ProfileName)
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"DeleteProfile\" Failed. Reason: Profile \"%s\" is the currently active profile and can't be deleted."), *InMessage.ProfileName);
		return;
	}
	
	
	if (!Broadcast.RemoveProfile(ProfileName))
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"DeleteProfile\" Failed to delete profile \"%s\" (Reason unknown)."), *InMessage.ProfileName);
		return;
	}
	
	LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log,
		TEXT("\"DeleteProfile\" Profile \"%s\" deleted."), *InMessage.ProfileName);
}

void FAvaRundownServer::HandleSetCurrentProfile(const FAvaRundownSetCurrentProfile& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	const FName ProfileName(InMessage.ProfileName);

	if (Broadcast.IsBroadcastingAnyChannel())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"SetCurrentProfile\" Failed. Reason: Channels are currently broadcasting."));
		return;
	}
	
	if (!Broadcast.GetProfile(ProfileName).IsValidProfile())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"SetCurrentProfile\" Failed. Reason: Profile \"%s\" does not exist."), *InMessage.ProfileName);
		return;
	}

	if (!Broadcast.SetCurrentProfile(ProfileName))
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"SetCurrentProfile\" Failed to set current profile \"%s\" (Reason unknown)."), *InMessage.ProfileName);
		return;
	}

	LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log,
		TEXT("\"SetCurrentProfile\" Profile \"%s\" is current."), *InMessage.ProfileName);
}

void FAvaRundownServer::HandleGetChannel(const FAvaRundownGetChannel& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	const FName ChannelName(InMessage.ChannelName);
	
	const UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	const FAvaBroadcastOutputChannel& Channel = Broadcast.GetCurrentProfile().GetChannel(ChannelName);

	if (!Channel.IsValidChannel())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"GetChannel\" Channel \"%s\" not found."), *InMessage.ChannelName);
		return;
	}
	
	FAvaRundownChannelResponse* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownChannelResponse>();
	ReplyMessage->RequestId = InMessage.RequestId;
	ReplyMessage->Channel = UE::AvaRundownServer::Private::SerializeChannel(Channel);
	SendResponse(ReplyMessage, InContext->GetSender());
}

void FAvaRundownServer::HandleGetChannels(const FAvaRundownGetChannels& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	FAvaRundownChannels* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownChannels>();
	ReplyMessage->RequestId = InMessage.RequestId;
	
	const TArray<FAvaBroadcastOutputChannel*>& Channels = Broadcast.GetCurrentProfile().GetChannels();
	ReplyMessage->Channels.Reserve(Channels.Num());

	for (const FAvaBroadcastOutputChannel* OutputChannel : Channels)
	{
		FAvaRundownChannel Channel = UE::AvaRundownServer::Private::SerializeChannel(*OutputChannel);
		ReplyMessage->Channels.Push(MoveTemp(Channel));
	}
	
	SendResponse(ReplyMessage, InContext->GetSender());
}

void FAvaRundownServer::HandleChannelAction(const FAvaRundownChannelAction& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	if (InMessage.Action == EAvaRundownChannelActions::Start)
	{
		if (InMessage.ChannelName.IsEmpty())
		{
			Broadcast.StartBroadcast();
			SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log, TEXT("\"ChannelAction\" Ok."));
		}
		else
		{
			const FName ChannelName(InMessage.ChannelName);
			FAvaBroadcastOutputChannel& Channel = Broadcast.GetCurrentProfile().GetChannelMutable(ChannelName);
			if (Channel.IsValidChannel())
			{
				Channel.StartChannelBroadcast();
				SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log, TEXT("\"ChannelAction\" Ok."));
			}
			else
			{
				LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"ChannelAction\" Failed. Reason: Invalid Channel \"%s\"."), *InMessage.ChannelName);
			}
		}
	}
	else if (InMessage.Action == EAvaRundownChannelActions::Stop)
	{
		if (InMessage.ChannelName.IsEmpty())
		{
			Broadcast.StopBroadcast();
			SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log, TEXT("\"ChannelAction\" Ok."));
		}
		else
		{
			const FName ChannelName(InMessage.ChannelName);
			FAvaBroadcastOutputChannel& Channel = Broadcast.GetCurrentProfile().GetChannelMutable(ChannelName);
			if (Channel.IsValidChannel())
			{
				Channel.StopChannelBroadcast();
				SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log, TEXT("\"ChannelAction\" Ok."));
			}
			else
			{
				LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"ChannelAction\" Failed. Reason: Invalid Channel \"%s\"."), *InMessage.ChannelName);
			}
		}
	}
	else
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"ChannelAction\" Failed. Reason: Invalid Action (must be \"Start\" or \"Stop\"."));
	}
}

void FAvaRundownServer::HandleGetChannelImage(const FAvaRundownGetChannelImage& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	if (InMessage.ChannelName.IsEmpty())
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"GetChannelImage\" Failed. Reason: Invalid ChannelName."));
		return;
	}
	
	const FName ChannelName(InMessage.ChannelName);
	const FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetChannel(ChannelName);
	
	if (!Channel.IsValidChannel())
	{
		SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"GetChannelImage\" Failed. Reason: Invalid Channel \"%s\"."), *InMessage.ChannelName);
		return;
	}

	const FRequestInfo RequestInfo = {InMessage.RequestId, InContext->GetSender()};
	
	UTextureRenderTarget2D* ChannelRenderTarget = Channel.GetCurrentRenderTarget(true);
	
	// If the channel's render target is not the desired format, we will need to convert it.
	TSharedPtr<FChannelImage> ChannelImage;
	if (AvailableChannelImages.Num() > 0)
	{
		ChannelImage = AvailableChannelImages.Pop();
	}
	else
	{
		ChannelImage = MakeShared<FChannelImage>();
	}

	if (ChannelRenderTarget->GetFormat() != PF_B8G8R8A8)
	{
		ChannelImage->UpdateRenderTarget(ChannelRenderTarget->SizeX, ChannelRenderTarget->SizeY, PF_B8G8R8A8, ChannelRenderTarget->ClearColor);
	}
	else
	{
		ChannelImage->RenderTarget.Reset(); // No need for conversion.
	}
	
	TWeakPtr<FAvaRundownServer> WeakRundownServer(SharedThis(this));

	// The conversion is done by the GPU in the render thread.
	ENQUEUE_RENDER_COMMAND(FAvaConvertChannelImage)(
		[ChannelRenderTarget, ChannelImage, RequestInfo, WeakRundownServer](FRHICommandListImmediate& RHICmdList)
		{
			FRHITexture* SourceRHI = ChannelRenderTarget->GetResource()->GetTexture2DRHI();
			FRHITexture* ReadbackRHI = SourceRHI;

			// Convert if needed.
			if (ChannelImage->RenderTarget.IsValid())
			{
				FRHITexture* DestinationRHI = ChannelImage->RenderTarget->GetResource()->GetTexture2DRHI();
				UE::AvaBroadcastRenderTargetMediaUtils::CopyTexture(RHICmdList, SourceRHI, DestinationRHI);
				ReadbackRHI = DestinationRHI;
			}

			// Reading Render Target pixels in the render thread to avoid a flush render commands.
			const FReadSurfaceDataFlags ReadDataFlags(RCM_UNorm, CubeFace_MAX);
			const FIntRect SourceRect = FIntRect(0, 0, ChannelRenderTarget->SizeX, ChannelRenderTarget->SizeY);

			ChannelImage->UpdateRawPixels(SourceRect.Width(), SourceRect.Height());
			
			RHICmdList.ReadSurfaceData(ReadbackRHI, SourceRect, ChannelImage->RawPixels, ReadDataFlags);

			// When the converted render target is ready, we resume the work in the game thread.
			AsyncTask(ENamedThreads::GameThread, [WeakRundownServer, RequestInfo, ChannelImage]()
			{
				if (const TSharedPtr<FAvaRundownServer> RundownServer = WeakRundownServer.Pin())
				{
					RundownServer->FinishGetChannelImage(RequestInfo, ChannelImage);
				}
			});
		});
}

void FAvaRundownServer::HandleChannelEditAction(const FAvaRundownChannelEditAction& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();

	if (InMessage.ChannelName.IsEmpty())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"ChannelEditAction\" Failed. Reason: Empty Channel Name."));
		return;
	}

	const FName ChannelName(InMessage.ChannelName);

	if (InMessage.Action == EAvaRundownChannelEditActions::Add)
	{
		if (Broadcast.GetCurrentProfile().GetChannel(ChannelName).IsValidChannel())
		{
			LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
				TEXT("\"ChannelEditAction\" Add Failed. Reason: Channel \"%s\" already exist."), *ChannelName.ToString());
			return;
		}
		
		Broadcast.GetCurrentProfile().AddChannel(ChannelName);	// This function doesn't fail apparently.
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log,
			TEXT("\"ChannelEditAction\" Add Channel %s succeeded."), *ChannelName.ToString());
		return;
	}

	if (InMessage.Action == EAvaRundownChannelEditActions::Remove)
	{
		if (!Broadcast.GetCurrentProfile().RemoveChannel(ChannelName))
		{
			LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
				TEXT("\"ChannelEditAction\" Remove Failed. Reason: Channel \"%s\" didn't exist in profile."), *ChannelName.ToString());
			return;
		}

		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log,
			TEXT("\"ChannelEditAction\" Remove Channel %s succeeded."), *ChannelName.ToString());
		return;
	}
	
	LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
		TEXT("\"ChannelEditAction\" Failed. Reason: Unknown action."));
}

void FAvaRundownServer::HandleRenameChannel(const FAvaRundownRenameChannel& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	const FName OldChannelName(InMessage.OldChannelName);
	const FName NewChannelName(InMessage.NewChannelName);
	
	if (InMessage.NewChannelName.IsEmpty())
    {
    	LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
    		TEXT("\"ChannelEditAction\" Failed. Reason: Empty New Channel Name."));
    	return;
    }
	
	if (!Broadcast.GetCurrentProfile().GetChannel(OldChannelName).IsValidChannel())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"RenameChannel\" Failed. Reason: Channel \"%s\" does not exist."), *OldChannelName.ToString());
		return;
	}

	if (Broadcast.GetCurrentProfile().GetChannel(NewChannelName).IsValidChannel())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"RenameChannel\" Failed. Reason: Channel \"%s\" already exist."), *NewChannelName.ToString());
		return;
	}

	FText ErrorMessage;
	if (!Broadcast.RenameChannel(OldChannelName, NewChannelName, ErrorMessage))
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error,
			TEXT("\"RenameChannel\" Failed to rename channel \"%s\" to \"%s\". Reason: %s"), *OldChannelName.ToString(), *NewChannelName.ToString(), *ErrorMessage.ToString());
		return;
	}

	LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log,
		TEXT("\"RenameChannel\" Channel \"%s\" rename to \"%s\"."), *OldChannelName.ToString(), *NewChannelName.ToString());
}

void FAvaRundownServer::HandleAddChannelDevice(const FAvaRundownAddChannelDevice& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	if (InMessage.ChannelName.IsEmpty() || InMessage.MediaOutputName.IsEmpty())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"AddChannelDevice\" Failed. Reason: One or more Empty Parameters."));
		return;
	}

	const FName ChannelName(InMessage.ChannelName);
	const FAvaBroadcastOutputChannel& OutputChannel = Broadcast.GetCurrentProfile().GetChannel(ChannelName);
	if (!OutputChannel.IsValidChannel())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"AddChannelDevice\" Failed. Reason: Invalid Channel \"%s\"."), *InMessage.ChannelName);
		return;    
	}

	if (OutputChannel.GetState() == EAvaBroadcastChannelState::Live)
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"AddChannelDevice\" Failed. Reason: Channel is Live."), *InMessage.ChannelName);
		return;    
	}

	/*
		We're essentially replicating the UI editor here. The editor:
		1. Builds an output tree
		2. Allows drag-and-drop of output/devices to a channel
		3. AddMediaOutputToChannel() is called

		We don't have immediate drag-and-drop information here, since this is called externally, so we'll rebuild a tree,
		and recursively search for a match, and then issue the same AddMediaOutputToChannel call the editor UI would've called.

		This won't be called frequently, so it's equivalent to an end-user opening up and adding a device to a channel via
		the broadcast window (tree rebuild -> drag and drop item)
	*/
	const FAvaOutputTreeItemPtr OutputDevices = MakeShared<FAvaBroadcastOutputRootItem>();
	IAvaBroadcastOutputTreeItem::FRefreshChildrenParams RefreshDevicesParams;
	RefreshDevicesParams.bShowAllMediaOutputClasses = true; // Listing all classes so the specified device is present.
	FAvaBroadcastOutputTreeItem::RefreshTree(OutputDevices, RefreshDevicesParams);
	FAvaOutputTreeItemPtr TreeItem = UE::AvaRundownServer::Private::RecursiveFindOutputTreeItem(OutputDevices, InMessage.MediaOutputName);
	
	if (!TreeItem.IsValid())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"AddChannelDevice\" Failed. Reason: Invalid Device \"%s\"."), *InMessage.MediaOutputName);
		return;
	}

	const FAvaBroadcastMediaOutputInfo OutputInfo;
	const UMediaOutput* OutputDevice = TreeItem->AddMediaOutputToChannel(OutputChannel.GetChannelName(), OutputInfo);

	if (InMessage.bSaveBroadcast)
	{
		Broadcast.SaveBroadcast();
	}

	LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log, TEXT("\"AddChannelDevice\" successfully added device \"%s\""), *OutputDevice->GetFName().ToString());
}

void FAvaRundownServer::HandleEditChannelDevice(const FAvaRundownEditChannelDevice& InMessage,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	if (InMessage.ChannelName.IsEmpty() || InMessage.MediaOutputName.IsEmpty())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"EditChannelDevice\" Failed. Reason: One or more Empty Parameters."));
		return;
	}

	const FName ChannelName(InMessage.ChannelName);
	const FAvaBroadcastOutputChannel& OutputChannel = Broadcast.GetCurrentProfile().GetChannel(ChannelName);
	if (!OutputChannel.IsValidChannel())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"EditChannelDevice\" Failed. Reason: Invalid Channel \"%s\"."), *InMessage.ChannelName);
		return;    
	}

	if (OutputChannel.GetState() == EAvaBroadcastChannelState::Live)
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"EditChannelDevice\" Failed. Reason: Channel is Live."), *InMessage.ChannelName);
		return;    
	}

	UMediaOutput* MediaOutput = UE::AvaRundownServer::Private::FindChannelMediaOutput(OutputChannel, InMessage.MediaOutputName);
	
	if (!MediaOutput)
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"EditChannelDevice\" Failed. Reason: Invalid Device \"%s\"."), *InMessage.MediaOutputName);
		return;
	}

	FAvaRundownServerMediaOutputUtils::EditMediaOutput(MediaOutput, InMessage.Data);
	
	if (InMessage.bSaveBroadcast)
	{
		Broadcast.SaveBroadcast();
	}
	
	SendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log, TEXT("\"EditChannelDevice\". Successfully edited device \"%s\" on \"%s\""), *InMessage.MediaOutputName, *InMessage.ChannelName); 
}

void FAvaRundownServer::HandleRemoveChannelDevice(const FAvaRundownRemoveChannelDevice& InMessage,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	if (InMessage.ChannelName.IsEmpty() || InMessage.MediaOutputName.IsEmpty())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"RemoveChannelDevice\" Failed. Reason: One or more Empty Parameters."));
		return;
	}
	const FName ChannelName(InMessage.ChannelName);
	const FAvaBroadcastOutputChannel& OutputChannel = Broadcast.GetCurrentProfile().GetChannel(ChannelName);
	if (!OutputChannel.IsValidChannel())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"RemoveChannelDevice\" Failed. Reason: Invalid Channel \"%s\"."), *InMessage.ChannelName);
		return;    
	}

	if (OutputChannel.GetState() == EAvaBroadcastChannelState::Live)
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"RemoveChannelDevice\" Failed. Reason: Channel is Live."), *InMessage.ChannelName);
		return;    
	}

	UMediaOutput* MediaOutput = UE::AvaRundownServer::Private::FindChannelMediaOutput(OutputChannel, InMessage.MediaOutputName);
	
	if (!MediaOutput)
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"RemoveChannelDevice\" Failed. Reason: Invalid Device \"%s\"."), *InMessage.MediaOutputName);
		return;
	}
	
#if WITH_EDITOR
	FScopedTransaction Transaction(LOCTEXT("RemoveMediaOutput", "Remove Media Output"));
	Broadcast.Modify();
#endif

	const int32 RemovedCount = Broadcast.GetCurrentProfile().RemoveChannelMediaOutputs(ChannelName, TArray{MediaOutput});

	if (RemovedCount == 0)
	{
#if WITH_EDITOR
		Transaction.Cancel();
#endif
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"RemoveChannelDevice\" Didn't remove device."));
		return;
	}
	
	if (InMessage.bSaveBroadcast)
	{
		Broadcast.SaveBroadcast();
	}
	
	LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log, TEXT("\"RemoveChannelDevice\" Removed Device \"%s\""), *InMessage.MediaOutputName);
}

void FAvaRundownServer::HandleGetChannelQualitySettings(const FAvaRundownGetChannelQualitySettings& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	const FAvaBroadcastOutputChannel& Channel = Broadcast.GetCurrentProfile().GetChannel(FName(InMessage.ChannelName));
	if (!Channel.IsValidChannel())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"GetChannelQualitySettings\" Failed. Reason: Invalid Channel \"%s\"."), *InMessage.ChannelName);
		return;
	}

	FAvaRundownChannelQualitySettings* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownChannelQualitySettings>();
	ReplyMessage->RequestId = InMessage.RequestId;
	ReplyMessage->ChannelName = InMessage.ChannelName;
	ReplyMessage->Features = Channel.GetViewportQualitySettings().Features;
	SendResponse(ReplyMessage, InContext->GetSender());
}

void FAvaRundownServer::HandleSetChannelQualitySettings(const FAvaRundownSetChannelQualitySettings& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	FAvaBroadcastOutputChannel& Channel = Broadcast.GetCurrentProfile().GetChannelMutable(FName(InMessage.ChannelName));
	if (!Channel.IsValidChannel())
	{
		LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Error, TEXT("\"SetChannelQualitySettings\" Failed. Reason: Invalid Channel \"%s\"."), *InMessage.ChannelName);
		return;
	}

	Channel.SetViewportQualitySettings(FAvaViewportQualitySettings(InMessage.Features));
	LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log, TEXT("\"SetChannelQualitySettings\" Channel \"%s\" success."), *InMessage.ChannelName);
}

void FAvaRundownServer::HandleSaveBroadcast(const FAvaRundownSaveBroadcast& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	Broadcast.SaveBroadcast();
	LogAndSendMessage(InContext->GetSender(), InMessage.RequestId, ELogVerbosity::Log, TEXT("\"SaveBroadcast\" success."));
}

void FAvaRundownServer::HandleGetDevices(const FAvaRundownGetDevices& InMessage,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	FAvaRundownDevicesList* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownDevicesList>();
	ReplyMessage->RequestId = InMessage.RequestId;
	const FAvaOutputTreeItemPtr OutputDevices = MakeShared<FAvaBroadcastOutputRootItem>();
	IAvaBroadcastOutputTreeItem::FRefreshChildrenParams RefreshDevicesParams;
	RefreshDevicesParams.bShowAllMediaOutputClasses = InMessage.bShowAllMediaOutputClasses;
	FAvaBroadcastOutputTreeItem::RefreshTree(OutputDevices, RefreshDevicesParams);
	// OutputDevices here aren't literally a physical device, just a construct representing
	// output. This convention was pulled from the SAvaBroadcastOutputDevices->RefreshDevices() call
	for (const TSharedPtr<IAvaBroadcastOutputTreeItem>& ServerItem : OutputDevices->GetChildren())
	{
		const FAvaBroadcastOutputServerItem* OutputServerItem = ServerItem->CastTo<FAvaBroadcastOutputServerItem>();
		if (!OutputServerItem)
		{
			continue;
		}
		
		for (const TSharedPtr<IAvaBroadcastOutputTreeItem>& ClassItem : ServerItem->GetChildren())
		{
			const FAvaBroadcastOutputClassItem* AvaOutputClassItem = ClassItem->CastTo<FAvaBroadcastOutputClassItem>();
			if (!AvaOutputClassItem)
			{
				continue;
			}
			
			FAvaRundownOutputClassItem OutputClassItem;
			OutputClassItem.Name = ClassItem->GetDisplayName().ToString();
			OutputClassItem.Server = OutputServerItem->GetServerName();
			
			for (const TSharedPtr<IAvaBroadcastOutputTreeItem>& OutputDeviceItem : AvaOutputClassItem->GetChildren())
			{
				if (!OutputDeviceItem->IsA<FAvaBroadcastOutputDeviceItem>())
				{
					continue;
				}
				
				FAvaRundownOutputDeviceItem DeviceItem;
				DeviceItem.Name = OutputDeviceItem->GetDisplayName().ToString();
				// Intentionally leaving DeviceItem.Data blank, as it's not usable data by itself
				// .Data will be filled out on a GetChannels call, where it becomes usable

				OutputClassItem.Devices.Push(MoveTemp(DeviceItem));
			}

			if (OutputClassItem.Devices.IsEmpty())
			{
				FAvaRundownOutputDeviceItem DeviceItem;
				DeviceItem.Name = OutputClassItem.Name;
				OutputClassItem.Devices.Push(MoveTemp(DeviceItem));
			}

			ReplyMessage->DeviceClasses.Push(MoveTemp(OutputClassItem));
		}
	}
	SendResponse(ReplyMessage, InContext->GetSender());
}

void FAvaRundownServer::LogAndSendMessage(const FMessageAddress& InSender, int32 InRequestId, ELogVerbosity::Type InVerbosity, const TCHAR* InFormat, ...) const
{
	TCHAR TempString[1024];
	va_list Args;
	va_start(Args, InFormat);
	FCString::GetVarArgs(TempString, UE_ARRAY_COUNT(TempString), InFormat, Args);
	va_end(Args);

	// The UE_LOG macro adds ELogVerbosity:: to the verbosity, which prevents
	// us from using it with a variable.
	switch (InVerbosity)
	{
	case ELogVerbosity::Type::Log:
		UE_LOGF(LogAvaRundownServer, Log, "%ls", TempString);
		break;
	case ELogVerbosity::Type::Display:
		UE_LOGF(LogAvaRundownServer, Display, "%ls", TempString);
		break;
	case ELogVerbosity::Type::Warning:
		UE_LOGF(LogAvaRundownServer, Warning, "%ls", TempString);
		break;
	case ELogVerbosity::Type::Error:
		UE_LOGF(LogAvaRundownServer, Error, "%ls", TempString);
		break;
	default:
		UE_LOGF(LogAvaRundownServer, Log, "%ls", TempString);
		break;
	}
	
	// Send the error message to the client.
	SendMessageImpl(InSender, InRequestId, InVerbosity, TempString);
}

void FAvaRundownServer::SendMessage(const FMessageAddress& InSender, int32 InRequestId, ELogVerbosity::Type InVerbosity, const TCHAR* InFormat, ...) const
{
	TCHAR TempString[1024];
	va_list Args;
	va_start(Args, InFormat);
	FCString::GetVarArgs(TempString, UE_ARRAY_COUNT(TempString), InFormat, Args);
	va_end(Args);

	// Send the error message to the client.
	SendMessageImpl(InSender, InRequestId, InVerbosity, TempString);
}

void FAvaRundownServer::SendMessageImpl(const FMessageAddress& InSender, int32 InRequestId, ELogVerbosity::Type InVerbosity, const TCHAR* InMsg) const
{
	FAvaRundownServerMsg* ErrorMessage = FMessageEndpoint::MakeMessage<FAvaRundownServerMsg>();
	ErrorMessage->RequestId = InRequestId;
	ErrorMessage->Verbosity = ToString(InVerbosity);
	ErrorMessage->Text = InMsg;
	SendResponse(ErrorMessage, InSender);
}

void FAvaRundownServer::RegisterConsoleCommands()
{
	if (ConsoleCommands.Num() != 0)
	{
		return;
	}
		
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("MotionDesignRundownServer.Status"),
			TEXT("Display current status of all server info."),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAvaRundownServer::ShowStatusCommand),
			ECVF_Default
			));
}

void FAvaRundownServer::ShowStatusCommand(const TArray<FString>& InArgs)
{
	UE_LOGF(LogAvaRundownServer, Display, "Rundown Server: \"%ls\"", *HostName);
	UE_LOGF(LogAvaRundownServer, Display, "- Endpoint Bus Address: \"%ls\"", MessageEndpoint.IsValid() ? *MessageEndpoint->GetAddress().ToString() : TEXT("Invalid"));
	UE_LOGF(LogAvaRundownServer, Display, "- Computer: \"%ls\"", *HostName);

	for (const TPair<FMessageAddress, TSharedPtr<FClientInfo>>& Client : Clients)
	{
		const FClientInfo& ClientInfo = *Client.Value;
		UE_LOGF(LogAvaRundownServer, Display, "Connected Client: \"%ls\"", *ClientInfo.Address.ToString());
		UE_LOGF(LogAvaRundownServer, Display, "   - Api Version: %d", ClientInfo.ApiVersion);
	}
	
	UE_LOGF(LogAvaRundownServer, Display, "Rundown Caches:");
	UE_LOGF(LogAvaRundownServer, Display, "- Editing Rundown: \"%ls\"", *EditCommandContext.GetCurrentRundownPath().ToString());
	UE_LOGF(LogAvaRundownServer, Display, "- Editing PageId: \"%d\"", EditCommandContext.ManagedPageId);
	UE_LOGF(LogAvaRundownServer, Display, "- Playing Rundown: \"%ls\"", *PlaybackCommandContext.GetCurrentRundownPath().ToString());

	if (const UAvaRundown* CurrentPlaybackRundown = PlaybackCommandContext.GetCurrentRundown())
	{
		TArray<int32> PlayingPages = CurrentPlaybackRundown->GetPlayingPageIds();
		for (const int32 PlayingPageId : PlayingPages)
		{
			UE_LOGF(LogAvaRundownServer, Display, "- Playing PageId: \"%d\"", PlayingPageId);
		}
		TArray<int32> PreviewingPages = CurrentPlaybackRundown->GetPreviewingPageIds();
		for (const int32 PreviewingPageId : PreviewingPages)
		{
			UE_LOGF(LogAvaRundownServer, Display, "- Previewing PageId: \"%d\"", PreviewingPageId);
		}
	}
}

void FAvaRundownServer::NotifyPlaybackContextSwitch(const FSoftObjectPath& InPreviousRundownPath, const FSoftObjectPath& InNewRundownPath) const
{
	if (ClientAddresses.IsEmpty())
	{
		return;
	}

	FAvaRundownPlaybackContextChanged* Notification = FMessageEndpoint::MakeMessage<FAvaRundownPlaybackContextChanged>();
	Notification->PreviousRundown = InPreviousRundownPath.ToString();
	Notification->NewRundown = InNewRundownPath.ToString();
	SendResponse(Notification, ClientAddresses);
}

void FAvaRundownServer::NotifyAssetEvent(const FAssetData& InAssetData, const EAvaRundownAssetEvent InEventType) const
{
	if (ClientAddresses.IsEmpty())
	{
		return;
	}

	// todo: probably need some event filtering (playable, by class, etc).
	FAvaRundownAssetsChanged* Message = FMessageEndpoint::MakeMessage<FAvaRundownAssetsChanged>();
	Message->AssetName = InAssetData.AssetName.ToString();
	Message->AssetPath = InAssetData.GetSoftObjectPath().ToString();
	Message->AssetClass = InAssetData.AssetClassPath.ToString();
	Message->bIsPlayable = FAvaPlaybackUtils::IsPlayableAsset(InAssetData);
	Message->EventType = InEventType;
	
	SendResponse(Message, ClientAddresses);
}

void FAvaRundownServer::FinishGetChannelImage(const FRequestInfo& InRequestInfo, const TSharedPtr<FChannelImage>& InChannelImage)
{
	FAvaRundownChannelImage* ReplyMessage = FMessageEndpoint::MakeMessage<FAvaRundownChannelImage>();
	ReplyMessage->RequestId = InRequestInfo.RequestId;
	FImage Image;
	bool bSuccess = false;

	// Note: replacing FImageUtils::GetRenderTargetImage since we already have the raw pixels.
	{
		constexpr EPixelFormat Format = PF_B8G8R8A8;
		const int32 ImageBytes = CalculateImageBytes(InChannelImage->SizeX, InChannelImage->SizeY, 0, Format);
		Image.RawData.AddUninitialized(ImageBytes);
		FMemory::Memcpy( Image.RawData.GetData(), InChannelImage->RawPixels.GetData(), InChannelImage->RawPixels.Num() * sizeof(FColor) );
		Image.SizeX = InChannelImage->SizeX;
		Image.SizeY = InChannelImage->SizeY;
		Image.NumSlices = 1;
		Image.Format = ERawImageFormat::BGRA8;
		Image.GammaSpace = EGammaSpace::sRGB;
	}

	// TODO: profile this.
	// Options: resize the render target on the gpu prior to reading pixels.
	{
		FImage ResizedImage;
		Image.ResizeTo(ResizedImage, Image.GetWidth() * .25f, Image.GetHeight() * .25f, Image.Format, EGammaSpace::Linear);

		TArray64<uint8> CompressedData;
		if (FImageUtils::CompressImage(CompressedData,TEXT("JPEG"), ResizedImage, 95))
		{
			const uint32 SafeMessageSizeLimit = UE::AvaMediaMessageUtils::GetSafeMessageSizeLimit();
			if (CompressedData.Num() > SafeMessageSizeLimit)
			{
				LogAndSendMessage(InRequestInfo.Sender, InRequestInfo.RequestId, ELogVerbosity::Error,
					TEXT("\"GetChannelImage\" Failed. Reason: (DataSize: %d) is larger that the safe size limit for udp segmenter (%d)."),
					CompressedData.Num(), SafeMessageSizeLimit);
				return;
			}

			ReplyMessage->ImageData.Append(CompressedData.GetData(), CompressedData.GetAllocatedSize());
			bSuccess = true;
		}
	}
	
	if (!bSuccess)
	{
		LogAndSendMessage(InRequestInfo.Sender, InRequestInfo.RequestId, ELogVerbosity::Error,
			TEXT("\"GetChannelImage\" Failed. Reason: Unable to retrieve Channel Image."));
		return;
	}

	SendResponse(ReplyMessage, InRequestInfo.Sender);
	
	// Put the image back in the pool of available images for next request. (or we could abandon it)
	AvailableChannelImages.Add(InChannelImage);
}

void FAvaRundownServer::HandlePageActions(const FRequestInfo& InRequestInfo, const TArray<int32>& InPageIds,
	bool bInIsPreview, FName InPreviewChannelName, EAvaRundownPageActions InAction) const
{
	using namespace UE::AvaRundownServer::Private;

	UAvaRundown* Rundown = PlaybackCommandContext.GetCurrentRundown();
	
	if (!Rundown)
	{
		LogAndSendMessage(InRequestInfo.Sender, InRequestInfo.RequestId, ELogVerbosity::Error,
			TEXT("\"PageAction\" Failed. Reason: no rundown currently loaded for playback."));
		return;
	}

	{
		// Validate the pages - the command will be considered a failure (as a whole) if it contains invalid pages.
		FString InvalidPages;
		for (const int32 PageId : InPageIds)
		{
			const FAvaRundownPage& Page = Rundown->GetPage(PageId);
			if (!Page.IsValidPage())
			{
				InvalidPages.Appendf(TEXT("%s%d"), InvalidPages.IsEmpty() ? TEXT("") : TEXT(", "), PageId);
			}
		}

		if (!InvalidPages.IsEmpty())
		{
			LogAndSendMessage(InRequestInfo.Sender, InRequestInfo.RequestId, ELogVerbosity::Error,
				TEXT("\"PageAction\" Failed. Reason: PageIds {%s} are invalid."), *InvalidPages);
			return;
		}
	}

	const FName PreviewChannelName = !InPreviewChannelName.IsNone() ? InPreviewChannelName : UAvaRundown::GetDefaultPreviewChannelName();
	// Todo: support program channel name in command.
	const FName CommandChannelName = bInIsPreview ? InPreviewChannelName : NAME_None; 
	
	bool bSuccess = false;
	FString FailureReason;
	switch (InAction)
	{
		case EAvaRundownPageActions::Load:
			for (const int32 PageId : InPageIds)
			{
				bSuccess |= Rundown->GetPageLoadingManager().RequestLoadPage(PageId, bInIsPreview, PreviewChannelName);
			}
			break;
		case EAvaRundownPageActions::Unload:
			for (const int32 PageId : InPageIds)
			{
				const FAvaRundownPage& Page =  Rundown->GetPage(PageId);
				if (Page.IsValidPage())
				{
					bSuccess |= Rundown->UnloadPage(PageId, (bInIsPreview ? PreviewChannelName : Page.GetChannelName()).ToString());
				}
			}
			break;
		case EAvaRundownPageActions::Play:
			bSuccess = !Rundown->PlayPages(InPageIds, bInIsPreview ? EAvaRundownPagePlayType::PreviewFromStart : EAvaRundownPagePlayType::PlayFromStart, PreviewChannelName, EAvaRundownPagePlaySource::Server).IsEmpty();
			break;
		case EAvaRundownPageActions::PlayNext:
			{
				const int32 NextPageId = FAvaRundownPlaybackUtils::GetPageIdToPlayNext(Rundown, UAvaRundown::InstancePageList, bInIsPreview, PreviewChannelName);
				if (FAvaRundownPlaybackUtils::IsPageIdValid(NextPageId))
				{
					bSuccess = Rundown->PlayPage(NextPageId, bInIsPreview ? EAvaRundownPagePlayType::PreviewFromFrame : EAvaRundownPagePlayType::PlayFromStart, EAvaRundownPagePlaySource::Server);
				}
				break;
			}
		case EAvaRundownPageActions::Stop:
			if (InPageIds.IsEmpty())
			{
				// If the list of pages is empty, we will stop all the playing pages.
				const TArray<int32> PageIds = GetPlayingPages(Rundown, bInIsPreview, CommandChannelName);
				bSuccess = !Rundown->StopPages(PageIds, EAvaRundownPageStopOptions::Default, bInIsPreview).IsEmpty();
			}
			else
			{
				bSuccess = !Rundown->StopPages(InPageIds, EAvaRundownPageStopOptions::Default, bInIsPreview).IsEmpty();
			}
			break;
		case EAvaRundownPageActions::ForceStop:
			if (InPageIds.IsEmpty())
			{
				// If the list of pages is empty, we will stop all the playing pages.
				const TArray<int32> PageIds = GetPlayingPages(Rundown, bInIsPreview, CommandChannelName);
				bSuccess = !Rundown->StopPages(PageIds, EAvaRundownPageStopOptions::ForceNoTransition, bInIsPreview).IsEmpty();
			}
			else
			{
				bSuccess = !Rundown->StopPages(InPageIds, EAvaRundownPageStopOptions::ForceNoTransition, bInIsPreview).IsEmpty();
			}
			break;
		case EAvaRundownPageActions::Continue:
			if (InPageIds.IsEmpty())
			{
				// If the list of pages is empty, we will continue all the playing pages.
				const TArray<int32> PageIds = GetPlayingPages(Rundown, bInIsPreview, CommandChannelName);
				bSuccess = ContinuePages(Rundown, PageIds, bInIsPreview, PreviewChannelName, FailureReason);
			}
			else
			{
				bSuccess = ContinuePages(Rundown, InPageIds, bInIsPreview, PreviewChannelName, FailureReason);
			}
			break;
		case EAvaRundownPageActions::UpdateValues:
			if (InPageIds.IsEmpty())
			{
				// If the list of pages is empty, we will continue all the playing pages.
				const TArray<int32> PageIds = GetPlayingPages(Rundown, bInIsPreview, CommandChannelName);
				bSuccess = UpdatePagesValues(Rundown, PageIds, bInIsPreview, PreviewChannelName);
			}
			else
			{
				bSuccess = UpdatePagesValues(Rundown, InPageIds, bInIsPreview, PreviewChannelName);
			}
			break;
		case EAvaRundownPageActions::TakeToProgram:
			{
				const TArray<int32> PageIds = FAvaRundownPlaybackUtils::GetPagesToTakeToProgram(Rundown, InPageIds, PreviewChannelName);
				Rundown->PlayPages(PageIds, EAvaRundownPagePlayType::PlayFromStart, EAvaRundownPagePlaySource::Server);
			}
			break;
		default:
			FailureReason.Appendf(TEXT("Invalid action. "));
			break;
	}

	const TCHAR* CommandName = bInIsPreview ? TEXT("PagePreviewAction") : TEXT("PageAction");

	// For multi-page commands, we consider a partial success as success.
	// Remote applications are notified of the page status with FAvaRundownPagesStatuses.
	//
	// Todo:
	// For pages that failed to execute the command, the failure reason is not sent
	// to remote applications. Given the more complex status information, we would
	// probably need a response message for this command with additional error information.
	
	if (bSuccess)
	{
		SendMessage(InRequestInfo.Sender, InRequestInfo.RequestId, ELogVerbosity::Log,
			TEXT("\"%s\" Ok."), CommandName);
	}
	else if (!FailureReason.IsEmpty())
	{
		LogAndSendMessage(InRequestInfo.Sender, InRequestInfo.RequestId, ELogVerbosity::Error,
			TEXT("\"%s\" Failed. Reason: %s"), CommandName, *FailureReason);
	}
	else
	{
		LogAndSendMessage(InRequestInfo.Sender, InRequestInfo.RequestId, ELogVerbosity::Error,
			TEXT("\"%s\" Failed."), CommandName);
	}
}

UAvaRundown* FAvaRundownServer::GetOrLoadRundownForEdit(const FMessageAddress& InSender, int32 InRequestId, const FString& InRundownPath)
{
	UAvaRundown* Rundown;
	
	if (!InRundownPath.IsEmpty())
	{
		// If a path is specified, the rundown gets reloaded
		// unless it was already loaded from a previous editing command.
		// This will not affect the currently loaded rundown for playback.
		const FSoftObjectPath NewRundownPath(InRundownPath);
		
		Rundown = GetOrLoadRundownForContext(NewRundownPath, EditCommandContext);

		if (!Rundown)
		{
			LogAndSendMessage(InSender, InRequestId, ELogVerbosity::Error, TEXT("Failed to load Rundown \"%s\"."), *InRundownPath);
		}
	}
	else
	{
		// If the path is not specified, we assume it is using the previously loaded rundown.
		Rundown = EditCommandContext.GetCurrentRundown();

		// Note: for backward compatibility with QA python script, we allow this command to use the current "playback" rundown as fallback.
		if (!Rundown)
		{
			Rundown = GetOrLoadRundownForContext(PlaybackCommandContext.GetCurrentRundownPath(), EditCommandContext);
		}

		if (!Rundown)
		{
			LogAndSendMessage(InSender, InRequestId, ELogVerbosity::Error, TEXT("No rundown path specified and no rundown currently loaded."));
		}
	}
	return Rundown;
}

void FAvaRundownServer::OnMessageBusNotification(const FMessageBusNotification& InNotification)
{
	// This is called when the websocket client disconnects.
	if (InNotification.NotificationType == EMessageBusNotification::Unregistered)
	{
		TWeakPtr<FAvaRundownServer> ServerWeak = SharedThis(this);
		auto RemoveClient = [ServerWeak, RegistrationAddress = InNotification.RegistrationAddress]()
		{
			if (const TSharedPtr<FAvaRundownServer> Server = ServerWeak.Pin())
			{
				if (Server->Clients.Contains(RegistrationAddress))
				{
					UE_LOGF(LogAvaRundownServer, Log, "Client \"%ls\" disconnected.", *RegistrationAddress.ToString());
					Server->Clients.Remove(RegistrationAddress);
					Server->RefreshClientAddresses();
				}
			}
		};

		if (IsInGameThread())
		{
			RemoveClient();
		}
		else
		{
			Async(EAsyncExecution::TaskGraphMainThread, MoveTemp(RemoveClient));
		}
	}
}

void FAvaRundownServer::RefreshClientAddresses()
{
	ClientAddresses.Reset(Clients.Num());
	for (const TPair<FMessageAddress, TSharedPtr<FClientInfo>>& Client : Clients)
	{
		ClientAddresses.Add(Client.Key);
	}
}

FAvaRundownServer::FRundownEntry::FRundownEntry(const TSharedPtr<FAvaRundownServer>& InRundownServer, const FSoftObjectPath& InRundownPath)
	: RundownServerRaw(InRundownServer.Get())
{
	const TStrongObjectPtr<UAvaRundown> LoadedRundown = UE::AvaRundownServer::Private::LoadRundown(InRundownPath);
	Rundown = LoadedRundown.IsValid() ? LoadedRundown.Get() : nullptr;
	
	if (Rundown && InRundownServer)
	{
		const TSharedRef<FAvaRundownServer> RundownServerRef = InRundownServer.ToSharedRef();
		Rundown->GetOnPagesChanged().AddSP(RundownServerRef, &FAvaRundownServer::OnPagesChanged);
		Rundown->GetOnPageListChanged().AddSP(RundownServerRef, &FAvaRundownServer::OnPageListChanged);
		Rundown->GetOnCanClosePlaybackContext().AddSP(RundownServerRef, &FAvaRundownServer::OnCanClosePlaybackContext);
		Rundown->GetOnPageTransitionRemoving().AddSP(RundownServerRef, &FAvaRundownServer::OnPageTransitionRemoved);
	}
}

FAvaRundownServer::FRundownEntry::~FRundownEntry()
{
	if (Rundown)
	{
		Rundown->GetOnPagesChanged().RemoveAll(RundownServerRaw);
		Rundown->GetOnPageListChanged().RemoveAll(RundownServerRaw);
		Rundown->GetOnCanClosePlaybackContext().RemoveAll(RundownServerRaw);
		Rundown->GetOnPageTransitionRemoving().RemoveAll(RundownServerRaw);
	}
}

void FAvaRundownServer::CompactLoadedRundownCache()
{
	for (TMap<FSoftObjectPath, TWeakPtr<FRundownEntry>>::TIterator ContextIterator = LoadedRundownCache.CreateIterator();
		ContextIterator; ++ContextIterator  )
	{
		if (!ContextIterator->Value.IsValid())
		{
			ContextIterator.RemoveCurrent();
		}
	}
}

TSharedPtr<FAvaRundownServer::FRundownEntry> FAvaRundownServer::GetOrLoadRundown(const FSoftObjectPath& InRundownPath)
{
	if (const TWeakPtr<FRundownEntry>* ExistingEntryWeak = LoadedRundownCache.Find(InRundownPath))
	{
		if (TSharedPtr<FRundownEntry> ExistingEntry = (*ExistingEntryWeak).Pin())
		{
			return ExistingEntry;
		}
	}

	TSharedPtr<FRundownEntry> NewEntry = MakeShared<FRundownEntry>(SharedThis(this), InRundownPath);
	if (NewEntry->IsValid())
	{
		CompactLoadedRundownCache();
		LoadedRundownCache.Add(InRundownPath, NewEntry);
		return NewEntry;
	}

	// Failed to load asset.
	return nullptr;
}

UAvaRundown* FAvaRundownServer::GetOrLoadRundownForContext(const FSoftObjectPath& InRundownPath, FCommandContext& InContext)
{
	if (InRundownPath != InContext.GetCurrentRundownPath())
	{
		if (const TSharedPtr<FRundownEntry> NewRundownEntry = GetOrLoadRundown(InRundownPath))
		{
			InContext.SetCurrentRundown(SharedThis(this), InRundownPath, NewRundownEntry);
		}
		else
		{
			// Indicates failure of loading new rundown asset.
			// Context is not modified.
			return nullptr;
		}
	}
	return InContext.GetCurrentRundown();
}

UAvaRundownPagePlayer* FAvaRundownServer::FindPagePlayerForInstance(const FAvaPlaybackInstance& InPlaybackInstance) const
{
	// We don't know the channel, nor the rundown, but can at least get the page Id from the instance user data.
	const int32 PageId = UAvaRundownPagePlayer::GetPageIdFromInstanceUserData(InPlaybackInstance.GetInstanceUserData());

	// Search in any of the currently loaded rundowns in the server. 
	for (const TPair<FSoftObjectPath, TWeakPtr<FRundownEntry>>& RundownEntryWeak : LoadedRundownCache)
	{
		const TSharedPtr<FRundownEntry> RundownEntry = RundownEntryWeak.Value.Pin();
		if (!RundownEntry)
		{
			continue;
		}
		
		const UAvaRundown* Rundown = RundownEntry->GetRundown();
		if (!Rundown)
		{
			continue;
		}
		
		// if we have a pageId, we can skip any rundown that doesn't have that page.
		if (PageId != FAvaRundownPage::InvalidPageId && !Rundown->GetPage(PageId).IsValidPage())
		{
			continue;
		}
		
		// We don't know the channel (could be preview), so we need to check all page players.
		for (const TObjectPtr<UAvaRundownPagePlayer>& PagePlayer : Rundown->GetPagePlayers())
		{
			// If we have a pageId, we can skip any players for other pages.
			if (PageId != FAvaRundownPage::InvalidPageId && PagePlayer->PageId != PageId)
			{
				continue;
			}

			// Using the instanceId to identify the correct instance.
			if (PagePlayer->FindInstancePlayerByInstanceId(InPlaybackInstance.GetInstanceId()))
			{
				return PagePlayer.Get();
			}
		}
	}
	return nullptr;
}

void FAvaRundownServer::OnPlaybackInstanceStatusChanged(const FAvaPlaybackInstance& InPlaybackInstance)
{
	const UAvaRundownPagePlayer* PagePlayer = FindPagePlayerForInstance(InPlaybackInstance);
	if (!PagePlayer)
	{
		return;
	}
	UAvaRundown* Rundown = PagePlayer->GetRundown();
	if (!Rundown)
	{
		return;
	}
	
	const FAvaRundownPage& Page = Rundown->GetPage(PagePlayer->PageId);
	if (Page.IsValidPage())
	{
		PageStatusChanged(Rundown, Page);
	}
}

void FAvaRundownServer::OnPlayableSequenceEvent(UAvaPlayable* InPlayable, FName InSequenceLabel, EAvaPlayableSequenceEventType InSequenceEvent)
{
	if (!InPlayable || !InPlayable->GetPlayableGroup())
	{
		return;
	}
	
	const UAvaRundown* CurrentPlaybackRundown = PlaybackCommandContext.GetCurrentRundown();
	if (!CurrentPlaybackRundown)
	{
		return; // Not an event from current playback rundown.
	}

	const FName ChannelName = InPlayable->GetPlayableGroup()->GetChannelName();
	const int32 PageId = UAvaRundownPagePlayer::GetPageIdFromInstanceUserData(InPlayable->GetUserData()); 

	if (!CurrentPlaybackRundown->FindPagePlayer(PageId, ChannelName))
	{
		return; // Not a playable from current playback rundown.
	}
	
	FAvaRundownPageSequenceEvent* Message = FMessageEndpoint::MakeMessage<FAvaRundownPageSequenceEvent>();
	Message->Channel = ChannelName.ToString();
	Message->PageId = PageId;
	Message->InstanceId = InPlayable->GetInstanceId();
	Message->AssetPath = InPlayable->GetSourceAssetPath().ToString();
	Message->SequenceLabel = InSequenceLabel.ToString();
	Message->Event = InSequenceEvent;

	SendResponse(Message, ClientAddresses);
}

void FAvaRundownServer::OnPlayableTransitionEvent(UAvaPlayable* InPlayable, UAvaPlayableTransition* InPlayableTransition, EAvaPlayableTransitionEventFlags InTransitionFlags)
{
	const UAvaRundown* CurrentPlaybackRundown = PlaybackCommandContext.GetCurrentRundown();
	if (!CurrentPlaybackRundown)
	{
		return; // Not an event from current playback rundown.
	}
	
	if (!EnumHasAnyFlags(InTransitionFlags, EAvaPlayableTransitionEventFlags::Finished | EAvaPlayableTransitionEventFlags::Starting))
	{
		return; // Not interested;
	}

	// Note: Page Transition can already be removed from Rundown, in that case event is propagated from the OnPageTransitionRemoved callback.
	const UAvaRundownPageTransition* PageTransition = CurrentPlaybackRundown->GetPageTransition(InPlayableTransition->GetTransitionId()); 
	if (!PageTransition)
	{
		return; // Not transition from current rundown.
	}
	
	FAvaRundownPageTransitionEvent* Message = FMessageEndpoint::MakeMessage<FAvaRundownPageTransitionEvent>();

	UE::AvaRundownServer::Private::FillPageTransitionInfo(*PageTransition, *Message);

	if (EnumHasAnyFlags(InTransitionFlags, EAvaPlayableTransitionEventFlags::Finished))
	{
		Message->Event = EAvaRundownPageTransitionEvents::Finished;
	}
	else if (EnumHasAnyFlags(InTransitionFlags, EAvaPlayableTransitionEventFlags::Starting))
	{
		Message->Event = EAvaRundownPageTransitionEvents::Started;
	}

	SendResponse(Message, ClientAddresses);
}

void FAvaRundownServer::OnPageTransitionRemoved(UAvaRundown* InRundown, UAvaRundownPageTransition* InPageTransition)
{
	const UAvaRundown* CurrentPlaybackRundown = PlaybackCommandContext.GetCurrentRundown();
	
	if (!CurrentPlaybackRundown || InRundown != CurrentPlaybackRundown || !InPageTransition)
	{
		return; // Not an event from current playback rundown.
	}

	// Note: if the page transition is removed from the rundown, it indicates the transition is finished.
	// It can be received before the "playable" transition event because the order of event handlers is not guaranteed.
	FAvaRundownPageTransitionEvent* Message = FMessageEndpoint::MakeMessage<FAvaRundownPageTransitionEvent>();
	UE::AvaRundownServer::Private::FillPageTransitionInfo(*InPageTransition, *Message);
	Message->Event = EAvaRundownPageTransitionEvents::Finished;
	SendResponse(Message, ClientAddresses);
}

void FAvaRundownServer::OnCanClosePlaybackContext(const UAvaRundown* InRundown, bool& bOutResult) const
{
	const UAvaRundown* CurrentPlaybackRundown = PlaybackCommandContext.GetCurrentRundown();
	if (CurrentPlaybackRundown != nullptr && CurrentPlaybackRundown == InRundown)
	{
		bOutResult = false;
	}
}

FAvaRundownServer::FEditCommandContext::~FEditCommandContext()
{
	SaveCurrentRemoteControlPresetToPage(true);
}

void FAvaRundownServer::FEditCommandContext::SetCurrentRundown(const TSharedPtr<FAvaRundownServer>& InRundownServer, const FSoftObjectPath& InRundownPath, const TSharedPtr<FRundownEntry>& InRundownEntry)
{
	SaveCurrentRemoteControlPresetToPage(true);
	FCommandContext::SetCurrentRundown(InRundownServer, InRundownPath, InRundownEntry);
}

void FAvaRundownServer::FEditCommandContext::SaveCurrentRemoteControlPresetToPage(bool bInUnregister)
{
	if (ManagedHandles.IsEmpty())
	{
		return;
	}

	if (bInUnregister)
	{
		for (const TSharedPtr<FAvaRundownManagedInstance>& ManagedInstance : ManagedHandles.Instances)
		{
			FAvaRemoteControlUtils::UnregisterRemoteControlPreset(ManagedInstance->GetRemoteControlPreset());
		}
	}

	UAvaRundown* CurrentRundown = GetCurrentRundown();

	if (!CurrentRundown)
	{
		return;
	}

	// Save the modified values to the page.
	FAvaRundownPage& ManagedPage = CurrentRundown->GetPage(ManagedPageId);
	if (!ManagedPage.IsValidPage())
	{
		return;
	}

	constexpr bool bIsDefault = false;
	FAvaPlayableRemoteControlValues NewValues;
	
	for (const TSharedPtr<FAvaRundownManagedInstance>& ManagedInstance : ManagedHandles.Instances)
	{
		if (ManagedInstance->GetRemoteControlPreset())
		{
			FAvaPlayableRemoteControlValues Values;
			Values.CopyFrom(ManagedInstance->GetRemoteControlPreset(), bIsDefault);
			NewValues.Merge(Values);
		}
	}

	// UpdateRemoteControlValues does half the job by ensuring that missing values are added and
	// extra values are removed. But it doesn't change existing values.
	EAvaPlayableRemoteControlChanges RemoteControlChanges = ManagedPage.UpdateRemoteControlValues(NewValues, bIsDefault);

	// Modify existing values if different.
	for (const TPair<FGuid, FAvaPlayableRemoteControlValue>& NewValue : NewValues.EntityValues)
	{
		const FAvaPlayableRemoteControlValue* ExistingValue = ManagedPage.GetRemoteControlEntityValue(NewValue.Key);
		
		if (ExistingValue && !NewValue.Value.IsSameValueAs(*ExistingValue))
		{
			ManagedPage.SetRemoteControlEntityValue(NewValue.Key, NewValue.Value);
			RemoteControlChanges |= EAvaPlayableRemoteControlChanges::EntityValues;
		}
	}
	for (const TPair<FGuid, FAvaPlayableRemoteControlValue>& NewValue : NewValues.ControllerValues)
	{
		const FAvaPlayableRemoteControlValue* ExistingValue = ManagedPage.GetRemoteControlControllerValue(NewValue.Key);

		if (ExistingValue && !NewValue.Value.IsSameValueAs(*ExistingValue))
		{
			ManagedPage.SetRemoteControlControllerValue(NewValue.Key, NewValue.Value);
			RemoteControlChanges |= EAvaPlayableRemoteControlChanges::ControllerValues;
		}
	}

	if (RemoteControlChanges != EAvaPlayableRemoteControlChanges::None)
	{
		CurrentRundown->NotifyPageRemoteControlValueChanged(ManagedPageId, RemoteControlChanges);
	}
}

FAvaRundownServer::FPlaybackCommandContext::~FPlaybackCommandContext()
{
	TSharedPtr<FRundownEntry> PreviousRundownEntry = CurrentRundownEntry; // Prevent GC for current scope.
	UAvaRundown* PreviousRundown = GetCurrentRundown();

	// Reset current rundown so it doesn't prevent closing playback context.
	CurrentRundownEntry.Reset();
	
	ClosePlaybackContext(PreviousRundown);
}

void FAvaRundownServer::FPlaybackCommandContext::SetCurrentRundown(const TSharedPtr<FAvaRundownServer>& InRundownServer, const FSoftObjectPath& InRundownPath, const TSharedPtr<FRundownEntry>& InRundownEntry)
{
	TSharedPtr<FRundownEntry> PreviousRundownEntry = CurrentRundownEntry;	// Prevent GC for current scope.
	UAvaRundown* PreviousRundown = GetCurrentRundown();

	// Notify clients that the current playback context is switching.
	if (InRundownServer)
	{
		InRundownServer->NotifyPlaybackContextSwitch(GetCurrentRundownPath(), InRundownPath);
	}

	FCommandContext::SetCurrentRundown(InRundownServer, InRundownPath, InRundownEntry);
	
	ClosePlaybackContext(PreviousRundown);
	
	// Initialize new playback context
	InitializePlaybackContext();
}

void FAvaRundownServer::FPlaybackCommandContext::InitializePlaybackContext()
{
	if (UAvaRundown* CurrentRundown = GetCurrentRundown())
	{
		CurrentRundown->InitializePlaybackContext();
	}
}

void FAvaRundownServer::FPlaybackCommandContext::ClosePlaybackContext(UAvaRundown* InRundownToClose)
{
	if (InRundownToClose && InRundownToClose->CanClosePlaybackContext())
	{
		InRundownToClose->ClosePlaybackContext(/*bInStopAllPages*/ true);
	}
}

#undef LOCTEXT_NAMESPACE