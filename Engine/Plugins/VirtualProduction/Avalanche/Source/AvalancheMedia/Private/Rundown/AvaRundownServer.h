// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Broadcast/AvaBroadcastProfile.h"
#include "MessageEndpoint.h"
#include "Playback/AvaPlaybackManager.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownManagedInstanceHandle.h"
#include "Rundown/AvaRundownMessages.h"
#include "Rundown/IAvaRundownServer.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/StrongObjectPtr.h"

class FAvaPlaybackInstance;
class FAvaRundownManagedInstance;
class UMediaOutput;

/**
 * Implements a rundown server that listens to commands on message bus.
 * The intention is to run a web socket transport bridge so the messages can
 * come from external applications.
 */
class FAvaRundownServer : public TSharedFromThis<FAvaRundownServer>, public FGCObject, public IAvaRundownServer
{
public:
	FAvaRundownServer();
	virtual ~FAvaRundownServer() override;

	//~ Begin IAvaRundownServer
	virtual const FString& GetName() const override { return HostName; }
	virtual const FMessageAddress& GetMessageAddress() const override;
	virtual TArray<FMessageAddress> GetClientAddresses() const override { return ClientAddresses; }
	//~ End IAvaRundownServer

	//~ Begin FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	virtual FString GetReferencerName() const override;
	//~ End FGCObject

	void Init(const FString& InAssignedHostName);

	void SetupPlaybackDelegates();
	void SetupBroadcastDelegates(UAvaBroadcast* InBroadcast);
	void SetupEditorDelegates();
	void RemovePlaybackDelegates() const;
	void RemoveBroadcastDelegates(UAvaBroadcast* InBroadcast) const;
	void RemoveEditorDelegates() const;
	
	void OnPagesChanged(const UAvaRundown* InRundown, const FAvaRundownPage& InPage, EAvaRundownPageChanges InChange) const;
	void OnPageListChanged(const FAvaRundownPageListChangeParams& InParams) const;
	void PageBlueprintChanged(const UAvaRundown* InRundown, const FAvaRundownPage& InPage, const FString& InBlueprintPath) const;
	void PageStatusChanged(const UAvaRundown* InRundown, const FAvaRundownPage& InPage) const;
	void PageChannelChanged(const UAvaRundown* InRundown, const FAvaRundownPage& InPage, const FString& InChannelName) const;
	void PageNameChanged(const UAvaRundown* InRundown, const FAvaRundownPage& InPage, bool bInFriendlyName) const;
	void PageAnimSettingsChanged(const UAvaRundown* InRundown, const FAvaRundownPage& InPage) const;
	void OnBroadcastChannelListChanged(const FAvaBroadcastProfile& InProfile) const;
	void OnBroadcastChannelChanged(const FAvaBroadcastOutputChannel& InChannel, EAvaBroadcastChannelChange InChange) const;
	void OnAssetAdded(const FAssetData& InAssetData) const;
	void OnAssetRemoved(const FAssetData& InAssetData) const;
	void OnAssetsPreDelete(const TArray<UObject*>& InObjects);
	void OnPlaybackInstanceStatusChanged(const FAvaPlaybackInstance& InPlaybackInstance);
	void OnPlayableSequenceEvent(UAvaPlayable* InPlayable, FName InSequenceLabel, EAvaPlayableSequenceEventType InSequenceEvent);
	void OnPlayableTransitionEvent(UAvaPlayable* InPlayable, UAvaPlayableTransition* InPlayableTransition, EAvaPlayableTransitionEventFlags InTransitionFlags);
	void OnPageTransitionRemoved(UAvaRundown* InRundown, UAvaRundownPageTransition* InPageTransition);
	void OnCanClosePlaybackContext(const UAvaRundown* InRundown, bool& bOutResult) const;
	
	// Rundown message handlers
	void HandleRundownPing(const FAvaRundownPing& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleGetRundownServerInfo(const FAvaRundownGetServerInfo& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleGetPlayableAssets(const FAvaRundownGetPlayableAssets& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleGetRundowns(const FAvaRundownGetRundowns& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleLoadRundown(const FAvaRundownLoadRundown& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleCreateRundown(const FAvaRundownCreateRundown& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleDeleteRundown(const FAvaRundownDeleteRundown& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleImportRundown(const FAvaRundownImportRundown& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleExportRundown(const FAvaRundownExportRundown& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleSaveRundown(const FAvaRundownSaveRundown& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleGetPages(const FAvaRundownGetPages& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleCreatePage(const FAvaRundownCreatePage& InMessage,const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleCreateTemplate(const FAvaRundownCreateTemplate& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleCreateComboTemplate(const FAvaRundownCreateComboTemplate& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleDeletePage(const FAvaRundownDeletePage& InMessage,const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleDeleteTemplate(const FAvaRundownDeleteTemplate& InMessage,const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleChangeTemplateBP(const FAvaRundownChangeTemplateBP& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleGetPageDetails(const FAvaRundownGetPageDetails& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleChangePageChannel(const FAvaRundownPageChangeChannel& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleChangePageName(const FAvaRundownChangePageName& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleUpdatePageFromRCP(const FAvaRundownUpdatePageFromRCP& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandlePageAction(const FAvaRundownPageAction& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandlePagePreviewAction(const FAvaRundownPagePreviewAction& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandlePageActions(const FAvaRundownPageActions& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandlePagePreviewActions(const FAvaRundownPagePreviewActions& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleTransitionAction(const FAvaRundownTransitionAction& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleTransitionLayerAction(const FAvaRundownTransitionLayerAction& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);

	// Broadcast message handlers
	void HandleGetProfiles(const FAvaRundownGetProfiles& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleCreateProfile(const FAvaRundownCreateProfile& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleDuplicateProfile(const FAvaRundownDuplicateProfile& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleRenameProfile(const FAvaRundownRenameProfile& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleDeleteProfile(const FAvaRundownDeleteProfile& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleSetCurrentProfile(const FAvaRundownSetCurrentProfile& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleGetChannel(const FAvaRundownGetChannel& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleGetChannels(const FAvaRundownGetChannels& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleChannelAction(const FAvaRundownChannelAction& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleChannelEditAction(const FAvaRundownChannelEditAction& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleRenameChannel(const FAvaRundownRenameChannel& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleAddChannelDevice(const FAvaRundownAddChannelDevice& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleEditChannelDevice(const FAvaRundownEditChannelDevice& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleRemoveChannelDevice(const FAvaRundownRemoveChannelDevice& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleGetChannelImage(const FAvaRundownGetChannelImage& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleGetChannelQualitySettings(const FAvaRundownGetChannelQualitySettings& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleSetChannelQualitySettings(const FAvaRundownSetChannelQualitySettings& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleSaveBroadcast(const FAvaRundownSaveBroadcast& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	
	void HandleGetDevices(const FAvaRundownGetDevices& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	
	void LogAndSendMessage(const FMessageAddress& InSender, int32 InRequestId, ELogVerbosity::Type InVerbosity, const TCHAR* InFormat, ...) const;
	void SendMessage(const FMessageAddress& InSender, int32 InRequestId, ELogVerbosity::Type InVerbosity, const TCHAR* InFormat, ...) const;
	void SendMessageImpl(const FMessageAddress& InSender, int32 InRequestId, ELogVerbosity::Type InVerbosity, const TCHAR* InMsg) const;

	void RegisterConsoleCommands();
	
	void ShowStatusCommand(const TArray<FString>& InArgs);

	/** Broadcast a rundown playback context switch to all connected clients. */
	void NotifyPlaybackContextSwitch(const FSoftObjectPath& InPreviousRundownPath, const FSoftObjectPath& InNewRundownPath) const;
	
	void NotifyAssetEvent(const FAssetData& InAssetData, const EAvaRundownAssetEvent InEventType) const;

protected:
	struct FRequestInfo
	{
		int32 RequestId;
		FMessageAddress Sender;
	};
	
	struct FChannelImage;
	void FinishGetChannelImage(const FRequestInfo& InRequestInfo, const TSharedPtr<FChannelImage>& InChannelImage);

	void HandlePageActions(const FRequestInfo& InRequestInfo, const TArray<int32>& InPageIds,
		bool bInIsPreview, FName InPreviewChannelName, EAvaRundownPageActions InAction) const;
	
	UAvaRundownPagePlayer* FindPagePlayerForInstance(const FAvaPlaybackInstance& InPlaybackInstance) const;
	
	/**
	 * Helper function to retrieve the appropriate rundown for editing commands.
	 */
	UAvaRundown* GetOrLoadRundownForEdit(const FMessageAddress& InSender, int32 InRequestId, const FString& InRundownPath);

	template<typename MessageType>
	void SendResponse(MessageType* InMessage, const FMessageAddress& InRecipient, EMessageFlags InFlags = EMessageFlags::None) const
	{
		if (MessageEndpoint)
		{
			MessageEndpoint->Send(InMessage, InRecipient);
		}
	}
	
	template<typename MessageType>
	void SendResponse(MessageType* InMessage, const TArray<FMessageAddress>& InRecipients, EMessageFlags InFlags = EMessageFlags::None) const
	{
		if (MessageEndpoint)
		{
			MessageEndpoint->Send(InMessage, MessageType::StaticStruct(), InFlags, nullptr,
				InRecipients, FTimespan::Zero(), FDateTime::MaxValue());
		}
	}

	void OnMessageBusNotification(const FMessageBusNotification& InNotification);

private:
	FString HostName;	
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;
	TArray<IConsoleObject*> ConsoleCommands;

	/** Keep information on connected clients. */
	struct FClientInfo
	{
		FMessageAddress Address;
		/** Api version for communication with this client. */
		int32 ApiVersion = -1;

		explicit FClientInfo(const FMessageAddress& InAddress) : Address(InAddress) {}
	};

	/** Keep track of remote clients context information. */
	TMap<FMessageAddress, TSharedPtr<FClientInfo>> Clients;

	/** Array of just the client addresses for sending responses. */
	TArray<FMessageAddress> ClientAddresses;
	
	FClientInfo* GetClientInfo(const FMessageAddress& InAddress) const
	{
		const TSharedPtr<FClientInfo>* ClientInfoPtr = Clients.Find(InAddress);
		return ClientInfoPtr ? ClientInfoPtr->Get() : nullptr;
	}
	
	FClientInfo& GetOrAddClientInfo(const FMessageAddress& InAddress)
	{
		if (const TSharedPtr<FClientInfo>* ExistingClientInfo = Clients.Find(InAddress))
		{
			return *ExistingClientInfo->Get();
		}

		const TSharedPtr<FClientInfo> NewClientInfo = MakeShared<FClientInfo>(InAddress);
		Clients.Add(InAddress, NewClientInfo);
		RefreshClientAddresses();
		return *NewClientInfo;
	}

	void RefreshClientAddresses();
	
	/** Pool of images that can be recycled. */
	TArray<TSharedPtr<FChannelImage>> AvailableChannelImages;
	
	/**
	 * Manages rundown delegates binding with the rundown server handlers.
	 */
	class FRundownEntry
	{
	public:
		FRundownEntry(const TSharedPtr<FAvaRundownServer>& InRundownServer, const FSoftObjectPath& InRundownPath);
		~FRundownEntry();
		
		bool IsValid() const { return Rundown != nullptr; }
		UAvaRundown* GetRundown() const { return Rundown;}
		
		void AddReferencedObjects(FReferenceCollector& InCollector)
		{
			InCollector.AddReferencedObject(Rundown);
		}
		
	private:
		TObjectPtr<UAvaRundown> Rundown;
		
		// Keep a raw ptr for unregistering delegates only.
		FAvaRundownServer* RundownServerRaw = nullptr;
	};

	/**
	 * Cache of loaded rundowns currently referenced by the command contexts.
	 * There is one entry per loaded asset (shared by command contexts).
	 * Key is the rundown's asset path.
	 */
	TMap<FSoftObjectPath, TWeakPtr<FRundownEntry>> LoadedRundownCache;

	/** Remove stale rundown entries. */
	void CompactLoadedRundownCache();
	
	/**
	 * Returns requested rundown specified by InRundownPath.
	 * Will load it if necessary or return the cached one.
	 * If the new rundown fails to load, the returned value is nullptr.
	 */
	TSharedPtr<FRundownEntry> GetOrLoadRundown(const FSoftObjectPath& InRundownPath);
	
	/**	 
	 * Associates a rundown entry with it's contextual resources needed to execute commands.
	 * There is only one "current" rundown per context. Changing the rundown will flush previous
	 * resources and allocate new ones of the new rundown.
	 */
	struct FCommandContext
	{
		virtual ~FCommandContext() = default;

		const FSoftObjectPath& GetCurrentRundownPath() const
		{
			return CurrentRundownPath;
		}
		
		UAvaRundown* GetCurrentRundown() const
		{
			return CurrentRundownEntry.IsValid() ? CurrentRundownEntry->GetRundown() : nullptr;
		}

		void Flush(const TSharedPtr<FAvaRundownServer>& InRundownServer)
		{
			SetCurrentRundown(InRundownServer, FSoftObjectPath(), nullptr);
		}
 
		void ConditionalFlush(const TSharedPtr<FAvaRundownServer>& InRundownServer, const FSoftObjectPath& InRundownPath)
		{
			if (GetCurrentRundownPath() == InRundownPath)
			{
				Flush(InRundownServer);
			}
		}
		
		void ConditionalFlush(const TSharedPtr<FAvaRundownServer>& InRundownServer, const UAvaRundown* InRundown)
		{
			if (GetCurrentRundown() == InRundown)
			{
				Flush(InRundownServer);
			}
		}
		
		/** Set a new current rundown for the context. Derived classes will handle context switching implementation. */
		virtual void SetCurrentRundown(const TSharedPtr<FAvaRundownServer>& InRundownServer, const FSoftObjectPath& InRundownPath, const TSharedPtr<FRundownEntry>& InRundownEntry)
		{
			CurrentRundownPath = InRundownPath;
			CurrentRundownEntry = InRundownEntry;
		}
		
	protected:
		/** Currently loaded/cached rundown's path. */
		FSoftObjectPath CurrentRundownPath;

		/** Currently loaded/cached rundown object. */
		TSharedPtr<FRundownEntry> CurrentRundownEntry;
	};

	/**
	 * Returns requested rundown specified by InRundownPath. Will load it if necessary or returned the cached one if it is the same.
	 * If the new rundown fails to load, the return value is nullptr, and the previous rundown will remain loaded.
	 * Only one rundown entry can be loaded per context (for now).
	 * 
	 * @param InRundownPath	Requested rundown path.
	 * @param InContext Command context (either edit or playback).
	 */
	UAvaRundown* GetOrLoadRundownForContext(const FSoftObjectPath& InRundownPath, FCommandContext& InContext);
	
	/**
	 * Context for page editing commands (i.e. GetPages, GetPageDetails, etc). 
	 */
	struct FEditCommandContext : public FCommandContext
	{
		/** PageId of the current managed ava asset. */
		int32 ManagedPageId = FAvaRundownPage::InvalidPageId;
		FAvaRundownManagedInstanceHandles ManagedHandles;

		virtual ~FEditCommandContext() override;

		//~ Begin FRundownContext
		virtual void SetCurrentRundown(const TSharedPtr<FAvaRundownServer>& InRundownServer, const FSoftObjectPath& InRundownPath, const TSharedPtr<FRundownEntry>& InRundownEntry) override;
		//~ End FRundownContext
		
		/**
		 * Checks if previous RCP was registered.
		 * If so, save modified values to corresponding page.
		 * This may result in the Rundown to be modified.
		 * Will also unregister RCP from RC Module if requested.
		 */
		void SaveCurrentRemoteControlPresetToPage(bool bInUnregister);
	};

	// TODO: it is likely we will need an edit command context per client connection (i.e. move to FClientInfo).
	FEditCommandContext EditCommandContext;

	/**
	 * Context for playback commands (i.e. LoadRundown, PageAction, etc). 
	 */
	struct FPlaybackCommandContext : public FCommandContext
	{
		virtual ~FPlaybackCommandContext() override;

		//~ Begin FRundownContext
		virtual void SetCurrentRundown(const TSharedPtr<FAvaRundownServer>& InRundownServer, const FSoftObjectPath& InRundownPath, const TSharedPtr<FRundownEntry>& InRundownEntry) override;
		//~ End FRundownContext

		void InitializePlaybackContext();
		static void ClosePlaybackContext(UAvaRundown* InRundownToClose);
	};

	// TODO: Will likely need to split playback context between preview (per client) and program (per rundown).
	FPlaybackCommandContext PlaybackCommandContext;

	/** Keep a map of created transient rundowns. */
	TMap<FSoftObjectPath, TObjectPtr<UAvaRundown>> ManagedRundowns;
};
