// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownServerMediaOutputUtils.h"

#include "Dom/JsonObject.h"
#include "MediaOutput.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"

DEFINE_LOG_CATEGORY_STATIC(LogAvaRundownServerMediaOutputUtils, Warning, All);

namespace UE::AvaMedia::RundownServerMediaOutputUtils::Private
{
	// Serialization helper functions
	static TSharedPtr<FJsonObject> ParsePropertyInfo(TFieldIterator<FProperty> InProperty, const void* InOwnerObject);
	static TSharedPtr<FJsonObject> ParseElementaryPropertyInfo(TFieldIterator<FProperty> InProperty, const void* InOwnerObject);
	static TSharedPtr<FJsonObject> ParseEnumPropertyInfo(TFieldIterator<FProperty> InProperty, const void* InOwnerObject);
	static TSharedPtr<FJsonObject> ParseStructPropertyInfo(TFieldIterator<FProperty> InProperty, const void* InOwnerObject);

	// Helper functions for editing Media Output Devices
	static void SetProperty(void* InOwnerObject, const TSharedPtr<FJsonObject>& InPropertyObject, const TFieldIterator<FProperty>& InProperty);
	static void SetElementaryProperty(void* InOwnerObject, const TSharedPtr<FJsonObject>& InPropertyObject, TFieldIterator<FProperty> InProperty);
	static void SetEnumProperty(void* InOwnerObject, const TSharedPtr<FJsonObject>& InPropertyObject, TFieldIterator<FProperty> InProperty);
	static void SetStructProperties(void* InOwnerObject, const TSharedPtr<FJsonObject>& InPropertyObject, TFieldIterator<FProperty> InProperty);

	FString GetPropertyDisplayNameString(TFieldIterator<FProperty> InProperty)
	{
#if WITH_EDITORONLY_DATA
		return InProperty->GetDisplayNameText().ToString();
#else
		return TEXT("NotAvailable"); // Not available in game build.
	#endif
	}
}

FString FAvaRundownServerMediaOutputUtils::SerializeMediaOutput(const UMediaOutput* InMediaOutput)
{
	using namespace UE::AvaMedia::RundownServerMediaOutputUtils::Private;
	
	const TSharedPtr<FJsonObject> OutputObject = MakeShareable(new FJsonObject);

	OutputObject->SetStringField(TEXT("Class"), InMediaOutput->GetClass()->GetPathName());
	OutputObject->SetStringField(TEXT("Name"), InMediaOutput->GetName());

	TArray<TSharedPtr<FJsonValue>> PropertyArray;
	for (TFieldIterator<FProperty> Iterator(InMediaOutput->GetClass()); Iterator; ++Iterator)
	{
		PropertyArray.Add(MakeShareable(new FJsonValueObject(ParsePropertyInfo(Iterator, InMediaOutput))));
	}

	const TSharedPtr<FJsonObject> PropertiesObject = MakeShareable(new FJsonObject);
	OutputObject->SetArrayField(TEXT("Properties"), PropertyArray);

	FString OutputString;
	const TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&OutputString);
	FJsonSerializer::Serialize(OutputObject.ToSharedRef(), JsonWriter);
	
	return OutputString;
}

void FAvaRundownServerMediaOutputUtils::EditMediaOutput(UMediaOutput* InMediaOutput, const FString& InDeviceData)
{
	using namespace UE::AvaMedia::RundownServerMediaOutputUtils::Private;
	
	const TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(InDeviceData);
	if (TSharedPtr<FJsonObject> NewDeviceData; FJsonSerializer::Deserialize(JsonReader, NewDeviceData))
	{
		for (TArray<TSharedPtr<FJsonValue>> PropertyValues = NewDeviceData->GetArrayField(TEXT("Properties"));
		     TSharedPtr<FJsonValue> PropertyValue : PropertyValues)
		{
			const TSharedPtr<FJsonObject> PropertyObject = PropertyValue.Get()->AsObject();

			for (TFieldIterator<FProperty> PropertyIterator(InMediaOutput->GetClass());
			     PropertyIterator; ++PropertyIterator)
			{
				if (PropertyObject->GetStringField(TEXT("Name")) == PropertyIterator->GetName())
				{
					SetProperty(InMediaOutput, PropertyObject, PropertyIterator);
				}
			}
		}
	}
}

TSharedPtr<FJsonObject> UE::AvaMedia::RundownServerMediaOutputUtils::Private::ParsePropertyInfo(TFieldIterator<FProperty> InProperty,
                                                                       const void* InOwnerObject)
{
	const FFieldClass* FieldClass = InProperty->GetClass();
	if (FieldClass == FIntProperty::StaticClass() || FieldClass == FNameProperty::StaticClass() ||
		FieldClass == FBoolProperty::StaticClass() || FieldClass == FStrProperty::StaticClass())
	{
		return ParseElementaryPropertyInfo(InProperty, InOwnerObject);
	}
	if (FieldClass == FEnumProperty::StaticClass())
	{
		return ParseEnumPropertyInfo(InProperty, InOwnerObject);
	}
	if (FieldClass == FStructProperty::StaticClass())
	{
		return ParseStructPropertyInfo(InProperty, InOwnerObject);
	}
	return MakeShareable(new FJsonObject);
}

TSharedPtr<FJsonObject> UE::AvaMedia::RundownServerMediaOutputUtils::Private::ParseElementaryPropertyInfo(
	TFieldIterator<FProperty> InProperty, const void* InOwnerObject)
{
	const TSharedPtr<FJsonObject> PropertyObject = MakeShareable(new FJsonObject);
	if (InProperty->GetClass() == FIntProperty::StaticClass())
	{
		const FIntProperty* IntProperty = CastField<FIntProperty>(InProperty->GetOwnerProperty());
		const int32 Value = IntProperty->GetPropertyValue_InContainer(InOwnerObject);
		PropertyObject->SetStringField(TEXT("Name"), InProperty->GetName());
		PropertyObject->SetStringField(TEXT("DisplayName"), GetPropertyDisplayNameString(InProperty));
		PropertyObject->SetStringField(TEXT("Type"), "number");
		PropertyObject->SetNumberField(TEXT("Value"), Value);
	}
	else if (InProperty->GetClass() == FBoolProperty::StaticClass())
	{
		const FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty->GetOwnerProperty());
		const bool Value = BoolProperty->GetPropertyValue_InContainer(InOwnerObject);
		PropertyObject->SetStringField(TEXT("Name"), InProperty->GetName());
		PropertyObject->SetStringField(TEXT("DisplayName"), GetPropertyDisplayNameString(InProperty));
		PropertyObject->SetStringField(TEXT("Type"), "boolean");
		PropertyObject->SetBoolField(TEXT("Value"), Value);
	}
	else if (InProperty->GetClass() == FStrProperty::StaticClass())
	{
		const FStrProperty* TextProperty = CastField<FStrProperty>(InProperty->GetOwnerProperty());
		const FString Value = TextProperty->GetPropertyValue_InContainer(InOwnerObject);
		PropertyObject->SetStringField(TEXT("Name"), InProperty->GetName());
		PropertyObject->SetStringField(TEXT("DisplayName"), GetPropertyDisplayNameString(InProperty));
		PropertyObject->SetStringField(TEXT("Type"), "string");
		PropertyObject->SetStringField(TEXT("Value"), Value);
	}
	else if (InProperty->GetClass() == FNameProperty::StaticClass())
	{
		const FNameProperty* NameProperty = CastField<FNameProperty>(InProperty->GetOwnerProperty());
		const FName Value = NameProperty->GetPropertyValue_InContainer(InOwnerObject);
		PropertyObject->SetStringField(TEXT("Name"), InProperty->GetName());
		PropertyObject->SetStringField(TEXT("DisplayName"), GetPropertyDisplayNameString(InProperty));
		PropertyObject->SetStringField(TEXT("Type"), "name");
		PropertyObject->SetStringField(TEXT("Value"), Value.ToString());
	}
	else
	{
		UE_LOGF(LogAvaRundownServerMediaOutputUtils, Warning, "Property type: %ls is not supported", *InProperty->GetClass()->GetName());
	}

	return PropertyObject;
}

TSharedPtr<FJsonObject> UE::AvaMedia::RundownServerMediaOutputUtils::Private::ParseEnumPropertyInfo(
	TFieldIterator<FProperty> InProperty, const void* InOwnerObject)
{
	using namespace UE::AvaMedia::RundownServerMediaOutputUtils::Private;
	
	const TSharedPtr<FJsonObject> PropertyObject = MakeShareable(new FJsonObject);
	const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty->GetOwnerProperty());

	PropertyObject->SetStringField(TEXT("Name"), InProperty->GetName());
	PropertyObject->SetStringField(TEXT("DisplayName"), GetPropertyDisplayNameString(InProperty));
	PropertyObject->SetStringField(TEXT("Type"), "enum");
	PropertyObject->SetStringField(TEXT("Class"), EnumProperty->GetEnum()->GetName());

	const int32 MaxIndex = EnumProperty->GetEnum()->NumEnums();
	TArray<TSharedPtr<FJsonValue>> EnumArray;
	for (int32 Index = 0; Index < MaxIndex; ++Index)
	{
		const TSharedPtr<FJsonObject> EnumValueObject = MakeShareable(new FJsonObject);
		EnumValueObject->SetStringField(FString::FromInt(Index),
		                                *EnumProperty->GetEnum()->GetAuthoredNameStringByIndex(Index));
		EnumArray.Add(MakeShareable(new FJsonValueObject(EnumValueObject)));
		PropertyObject->SetArrayField(TEXT("EnumValues"), EnumArray);
	}
	const uint8* Value = EnumProperty->ContainerPtrToValuePtr<uint8>(InOwnerObject);

	PropertyObject->SetStringField(TEXT("Value"), *EnumProperty->GetEnum()->GetAuthoredNameStringByValue(*Value));
	return PropertyObject;
}

TSharedPtr<FJsonObject> UE::AvaMedia::RundownServerMediaOutputUtils::Private::ParseStructPropertyInfo(
	TFieldIterator<FProperty> InProperty, const void* InOwnerObject)
{
	using namespace UE::AvaMedia::RundownServerMediaOutputUtils::Private;
	
	const TSharedPtr<FJsonObject> PropertyObject = MakeShareable(new FJsonObject);
	const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty->GetOwnerProperty());

	PropertyObject->SetStringField(TEXT("Name"), InProperty->GetName());
	PropertyObject->SetStringField(TEXT("DisplayName"), GetPropertyDisplayNameString(InProperty));
	PropertyObject->SetStringField(TEXT("Type"), "struct");

	const UScriptStruct* Struct = StructProperty->Struct;
	PropertyObject->SetStringField(TEXT("Class"), Struct->GetName());

	const void* StructPtr = StructProperty->ContainerPtrToValuePtr<void>(InOwnerObject);

	TArray<TSharedPtr<FJsonValue>> PropertyArray;
	for (TFieldIterator<FProperty> StructIterator(Struct); StructIterator; ++StructIterator)
	{
		PropertyArray.Add(MakeShareable(new FJsonValueObject(ParsePropertyInfo(StructIterator, StructPtr))));
	}
	PropertyObject->SetArrayField(TEXT("StructProperties"), PropertyArray);

	return PropertyObject;
}

void UE::AvaMedia::RundownServerMediaOutputUtils::Private::SetProperty(void* InOwnerObject, const TSharedPtr<FJsonObject>& InPropertyObject,
                                              const TFieldIterator<FProperty>& InProperty)
{
	if (InPropertyObject->GetStringField(TEXT("Type")) == "number" || InPropertyObject->GetStringField(TEXT("Type")) == "string"
		|| InPropertyObject->GetStringField(TEXT("Type")) == "boolean" || InPropertyObject->GetStringField(TEXT("Type")) == "name")
	{
		SetElementaryProperty(InOwnerObject, InPropertyObject, InProperty);
	}
	else if (InPropertyObject->GetStringField(TEXT("Type")) == "enum")
	{
		SetEnumProperty(InOwnerObject, InPropertyObject, InProperty);
	}
	else if (InPropertyObject->GetStringField(TEXT("Type")) == "struct")
	{
		SetStructProperties(InOwnerObject, InPropertyObject, InProperty);
	}
}

void UE::AvaMedia::RundownServerMediaOutputUtils::Private::SetElementaryProperty(void* InOwnerObject,
	const TSharedPtr<FJsonObject>& InPropertyObject, TFieldIterator<FProperty> InProperty)
{
	if (InPropertyObject->GetStringField(TEXT("Type")) == "number")
	{
		const int32 NumberValue = InPropertyObject->GetNumberField(TEXT("Value"));
		InProperty->GetOwnerProperty()->SetValue_InContainer(InOwnerObject, &NumberValue);
	}
	else if (InPropertyObject->GetStringField(TEXT("Type")) == "string")
	{
		const FString StringValue = InPropertyObject->GetStringField(TEXT("Value"));
		InProperty->GetOwnerProperty()->SetValue_InContainer(InOwnerObject, &StringValue);
	}
	else if (InPropertyObject->GetStringField(TEXT("Type")) == "boolean")
	{
		const bool BoolValue = InPropertyObject->GetBoolField(TEXT("Value"));
		InProperty->GetOwnerProperty()->SetValue_InContainer(InOwnerObject, &BoolValue);
	}
	else if (InPropertyObject->GetStringField(TEXT("Type")) == "name")
	{
		const FName NameValue = FName(InPropertyObject->GetStringField(TEXT("Value")));
		InProperty->GetOwnerProperty()->SetValue_InContainer(InOwnerObject, &NameValue);
	}
}

void UE::AvaMedia::RundownServerMediaOutputUtils::Private::SetEnumProperty(void* InOwnerObject,
	const TSharedPtr<FJsonObject>& InPropertyObject, TFieldIterator<FProperty> InProperty)
{
	const FString ValueName = InPropertyObject->GetStringField(TEXT("Value"));

	const FEnumProperty* EnumProperty = CastField<FEnumProperty>(
		InProperty->GetOwnerProperty());
	const int32 EnumValue = EnumProperty->GetEnum()->GetValueByName(FName(ValueName));
	EnumProperty->GetOwnerProperty()->SetValue_InContainer(InOwnerObject, &EnumValue);
}

void UE::AvaMedia::RundownServerMediaOutputUtils::Private::SetStructProperties(void* InOwnerObject,
	const TSharedPtr<FJsonObject>& InPropertyObject, TFieldIterator<FProperty> InProperty)
{
	const FStructProperty* StructProperty = CastField<FStructProperty>(
		InProperty->GetOwnerProperty());

	const UScriptStruct* Struct = StructProperty->Struct;
	void* StructPtr = StructProperty->ContainerPtrToValuePtr<void>(InOwnerObject);

	for (TArray<TSharedPtr<FJsonValue>> StructPropertyValues = InPropertyObject->GetArrayField(TEXT("StructProperties"));
			 TSharedPtr<FJsonValue> StructPropertyValue : StructPropertyValues)
	{
		const TSharedPtr<FJsonObject> StructPropertyObject = StructPropertyValue.Get()->AsObject();

		for (TFieldIterator<FProperty> StructIterator(Struct); StructIterator; ++StructIterator)
		{
			if (StructPropertyObject->GetStringField(TEXT("Name")) == StructIterator->GetName())
			{
				SetProperty(StructPtr, StructPropertyObject, StructIterator);
			}
		}
	}
}
