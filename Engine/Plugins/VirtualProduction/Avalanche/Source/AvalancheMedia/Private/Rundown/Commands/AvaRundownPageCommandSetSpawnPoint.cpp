// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageCommandSetSpawnPoint.h"

#define LOCTEXT_NAMESPACE "AvaRundownPageCommandSetSpawnPoint"

FText FAvaRundownPageCommandSetSpawnPoint::GetDescription() const
{
	return LOCTEXT("Command_Description", "Set Spawn Point");
}

bool FAvaRundownPageCommandSetSpawnPoint::HasTransitionLogic() const
{
	return false;
}

FString FAvaRundownPageCommandSetSpawnPoint::GetTransitionLayerString(const FString& InSeparator) const
{
	return TEXT("");
}

bool FAvaRundownPageCommandSetSpawnPoint::CanExecuteOnLoad(FAvaRundownPageCommandContext& InContext, FString* OutFailureReason) const
{
	return true;
}

bool FAvaRundownPageCommandSetSpawnPoint::ExecuteOnLoad(FAvaRundownPageCommandContext& InContext, FString& OutLoadOptions) const
{
	OutLoadOptions += FString::Printf(TEXT(" -SpawnPointTag=\"%s\""), *SpawnPointTag.ToString());
	return true;
}

#undef LOCTEXT_NAMESPACE