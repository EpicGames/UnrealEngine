// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/AvaRundownPageCommand.h"

#include "AvaRundownPageCommandSetSpawnPoint.generated.h"

/**
 * Specify a reference null actor to use the position to load the graphic at.
 * This can be used to position a null actor with the given tag in the base level and have the graphics
 * load at those positions.
 */
USTRUCT(BlueprintType, meta = (DisplayName = "Set Spawn Point"))
struct FAvaRundownPageCommandSetSpawnPoint : public FAvaRundownPageCommand
{
	GENERATED_BODY()

	//~ Begin FAvaRundownPageCommand
	virtual FText GetDescription() const override;
	virtual bool HasTransitionLogic() const override;
	virtual FString GetTransitionLayerString(const FString& InSeparator) const override;
	virtual bool CanExecuteOnLoad(FAvaRundownPageCommandContext& InContext, FString* OutFailureReason) const override;
	virtual bool ExecuteOnLoad(FAvaRundownPageCommandContext& InContext, FString& OutLoadOptions) const override;
	//~ End FAvaRundownPageCommand
	
	/** Indicate which spawn point, by tag, to search for. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Page Data")
	FName SpawnPointTag;
};
