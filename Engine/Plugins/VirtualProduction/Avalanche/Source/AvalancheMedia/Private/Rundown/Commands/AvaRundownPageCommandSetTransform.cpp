// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageCommandSetTransform.h"

#define LOCTEXT_NAMESPACE "AvaRundownPageCommandSetTransform"

FText FAvaRundownPageCommandSetTransform::GetDescription() const
{
	return LOCTEXT("Command_Description", "Set Transform");
}

bool FAvaRundownPageCommandSetTransform::HasTransitionLogic() const
{
	return false;
}

FString FAvaRundownPageCommandSetTransform::GetTransitionLayerString(const FString& InSeparator) const
{
	return TEXT("");
}

bool FAvaRundownPageCommandSetTransform::CanExecuteOnLoad(FAvaRundownPageCommandContext& InContext, FString* OutFailureReason) const
{
	return true;
}

bool FAvaRundownPageCommandSetTransform::ExecuteOnLoad(FAvaRundownPageCommandContext& InContext, FString& OutLoadOptions) const
{
	OutLoadOptions += FString::Printf(TEXT(" -Transform=\"%s\""), *Transform.ToString());
	return true;
}

#undef LOCTEXT_NAMESPACE