// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/AvaRundownPageCommand.h"

#include "AvaRundownPageCommandSetTransform.generated.h"


/**
 * Page Command to specify the transform when loading the graphic.
 * For streaming levels, this is applied when the level is loaded.
 */
USTRUCT(BlueprintType, meta = (DisplayName = "Set Transform"))
struct FAvaRundownPageCommandSetTransform : public FAvaRundownPageCommand
{
	GENERATED_BODY()

	//~ Begin FAvaRundownPageCommand
	virtual FText GetDescription() const override;
	virtual bool HasTransitionLogic() const override;
	virtual FString GetTransitionLayerString(const FString& InSeparator) const override;
	virtual bool CanExecuteOnLoad(FAvaRundownPageCommandContext& InContext, FString* OutFailureReason) const override;
	virtual bool ExecuteOnLoad(FAvaRundownPageCommandContext& InContext, FString& OutLoadOptions) const override;
	//~ End FAvaRundownPageCommand
	
	/** Specify the root transform to apply at load time. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Page Data")
	FTransform Transform;
};
