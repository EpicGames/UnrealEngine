// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageCommandStopLayers.h"

#include "AvaTagCollection.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownPagePlayer.h"
#include "Rundown/AvaRundownPlaybackUtils.h"
#include "Rundown/Transition/AvaRundownPageTransition.h"
#include "Rundown/Transition/AvaRundownPageTransitionBuilder.h"

#define LOCTEXT_NAMESPACE "AvaRundownPageCommandStopLayers"

namespace UE::AvaMedia::RundownPageCommandStopLayers::Private
{
	// todo: might be worth having this in FAvaTagHandleContainer. 
	bool IsEmpty(const FAvaTagHandleContainer& InLayers)
	{
		return InLayers.GetTagIds().IsEmpty() || InLayers.Source == nullptr;	
	}

	FString GetLayersString(const FAvaTagHandleContainer& InLayers, const FString& InLayerPrefix, const FString& InSeparator)
	{
		FString LayersString;
		LayersString.Reserve(64);
		for (const FAvaTagId& TagId : InLayers.GetTagIds())
		{
			if (!LayersString.IsEmpty())
			{
				LayersString += InSeparator;
			}
			LayersString += InLayerPrefix;
			LayersString += InLayers.Source->GetTagName(TagId).ToString();		
		}
		return LayersString;
	}

	bool Overlaps(const FAvaTagHandleContainer& InLayers, const FAvaTagHandle& InOther)
	{
		for (const FAvaTagId& TagId : InLayers.GetTagIds())
		{
			if (FAvaTagHandle(InLayers.Source, TagId).Overlaps(InOther))
			{
				return true;
			}
		}
		return false;
	}
}

FText FAvaRundownPageCommandStopLayers::GetDescription() const
{
	using namespace UE::AvaMedia::RundownPageCommandStopLayers::Private;
	return FText::Format(LOCTEXT("Command_Description", "Stop Layers: {0}"), FText::FromString(GetLayersString(Layers, TEXT(""), TEXT(", "))));
}

bool FAvaRundownPageCommandStopLayers::HasTransitionLogic() const
{
	using namespace UE::AvaMedia::RundownPageCommandStopLayers::Private;
	return !IsEmpty(Layers);
}

FString FAvaRundownPageCommandStopLayers::GetTransitionLayerString(const FString& InSeparator) const
{
	// For the layers text of a stop layer command, we'll add a '-' sign in front of the layer name.
	using namespace UE::AvaMedia::RundownPageCommandStopLayers::Private;
	return GetLayersString(Layers, TEXT("-"), InSeparator);
}

bool FAvaRundownPageCommandStopLayers::CanExecuteOnPlay(FAvaRundownPageCommandContext& InContext, FString* OutFailureReason) const
{
	using namespace UE::AvaMedia::RundownPageCommandStopLayers::Private;
	if (IsEmpty(Layers))
	{
		if (OutFailureReason)
		{
			*OutFailureReason = TEXT("Stop Layers: no layers defined in command.");
		}
		return false;
	}

	// Check if there are currently playing pages that have layers than can be stopped.
	for (const UAvaRundownPagePlayer* PagePlayer : InContext.Rundown.GetPagePlayers())
	{
		if (!PagePlayer || PagePlayer->ChannelFName != InContext.ChannelName)	// Not this channel
		{
			continue;
		}

		bool bLayerOverlap = false;
		PagePlayer->ForEachInstancePlayer([this, &bLayerOverlap](const UAvaRundownPlaybackInstancePlayer* InInstancePlayer)
		{
			using namespace UE::AvaMedia::RundownPageCommandStopLayers::Private;
			bLayerOverlap |= Overlaps(Layers, InInstancePlayer->TransitionLayer);
		});

		// Found an instance of a layer that we can kick out.
		if (bLayerOverlap)
		{
			return true;
		}
	}

	if (OutFailureReason)
	{
		*OutFailureReason = TEXT("Stop Layers: no currently playing pages overlaps with defined layers.");
	}
	return false;
}

bool FAvaRundownPageCommandStopLayers::ExecuteOnPlay(FAvaRundownPageTransitionBuilder& InTransitionBuilder, FAvaRundownPageCommandContext& InContext) const
{
	using namespace UE::AvaMedia::RundownPageCommandStopLayers::Private;
	if (IsEmpty(Layers))
	{
		return false;
	}

	if (UAvaRundownPageTransition* Transition = InTransitionBuilder.FindOrAddTransition(InContext.ChannelName))
	{
		for (const FAvaTagId& TagId : Layers.GetTagIds())
		{
			FAvaRundownPlaybackUtils::AddTagHandleUnique(Transition->ExitLayers, FAvaTagHandle(Layers.Source, TagId));
		}
		return true;
	}
	return false;
}

#undef LOCTEXT_NAMESPACE