// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagHandleContainer.h"
#include "Rundown/AvaRundownPageCommand.h"

#include "AvaRundownPageCommandStopLayers.generated.h"

USTRUCT(BlueprintType, meta = (DisplayName = "Stop Layers"))
struct FAvaRundownPageCommandStopLayers : public FAvaRundownPageCommand
{
	GENERATED_BODY()

	//~ Begin FAvaRundownPageCommand
	virtual FText GetDescription() const override;
	virtual bool HasTransitionLogic() const override;
	virtual FString GetTransitionLayerString(const FString& InSeparator) const override;
	virtual bool CanExecuteOnPlay(FAvaRundownPageCommandContext& InContext, FString* OutFailureReason) const override;
	virtual bool ExecuteOnPlay(FAvaRundownPageTransitionBuilder& InTransitionBuilder, FAvaRundownPageCommandContext& InContext) const override;
	//~ End FAvaRundownPageCommand
	
	/** Layers to stop. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Page Data")
	FAvaTagHandleContainer Layers;
};
