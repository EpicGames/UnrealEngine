// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/Transition/AvaRundownPageTransition.h"

#include "Broadcast/AvaBroadcast.h"
#include "Playable/AvaPlayableGroupManager.h"
#include "Playable/AvaPlayableUtils.h"
#include "Playable/Transition/AvaPlayableTransition.h"
#include "Playback/AvaPlaybackManager.h"
#include "Playback/AvaPlaybackUtils.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownPagePlayer.h"

namespace UE::AvaMedia::Rundown::PageTransition::Private
{
	UAvaPlayable* GetPlayable(const UAvaRundownPlaybackInstancePlayer* InInstancePlayer)
	{
		return InInstancePlayer ? InInstancePlayer->GetFirstPlayable() : nullptr;
	}
	
	UAvaPlayable* GetPlayable(const UAvaRundownPagePlayer* InPagePlayer, int32 InIndex)
	{
		return InPagePlayer ? GetPlayable(InPagePlayer->GetInstancePlayer(InIndex)) : nullptr;
	}

	UAvaPlaybackGraph* GetPlayback(const UAvaRundownPlaybackInstancePlayer* InInstancePlayer)
	{
		return InInstancePlayer ? InInstancePlayer->Playback : nullptr; 
	}
	
	UAvaPlaybackGraph* GetPlayback(const UAvaRundownPagePlayer* InPagePlayer, int32 InIndex)
	{
		return InPagePlayer ? GetPlayback(InPagePlayer->GetInstancePlayer(InIndex)) : nullptr;
	}

	TSharedPtr<FAvaPlayableRemoteControlValues> GetRemoteControlValues(const UAvaRundownPagePlayer* InPagePlayer)
	{
		if (UAvaRundown* Rundown = InPagePlayer->GetRundown())
		{
			const FAvaRundownPage& Page = Rundown->GetPage(InPagePlayer->PageId);
			if (Page.IsValidPage())
			{
				return MakeShared<FAvaPlayableRemoteControlValues>(Page.GetRemoteControlValues());
			}
		}
		return nullptr;
	}

	FString GetPrettyInstancePlayerInfo(const UAvaRundownPlaybackInstancePlayer* InInstancePlayer)
	{
		if (InInstancePlayer)
		{
			return FString::Printf(TEXT("Id: %s, Asset: %s"),
				*InInstancePlayer->GetPlaybackInstanceId().ToString(),
				*InInstancePlayer->SourceAssetPath.ToString()); 
		}
		return FString(TEXT("Invalid"));
	}

	FString GetPrettyPagePlayerInfo(const UAvaRundownPagePlayer* InPagePlayer)
	{
		FString Info = FString::Printf(TEXT("Channel: %s"), *InPagePlayer->ChannelName);
		InPagePlayer->ForEachInstancePlayer([&Info](const UAvaRundownPlaybackInstancePlayer* InInstancePlayer)
		{
			Info += FString::Printf(TEXT(", Instance {%s}"), *GetPrettyInstancePlayerInfo(InInstancePlayer));
		});
		return Info;
	}
	
	UAvaRundownPlaybackInstancePlayer* FindInstancePlayerForPlayable(const UAvaPlayable* InPlayable, const TArray<TWeakObjectPtr<UAvaRundownPagePlayer>>& InPagePlayersWeak)
	{
		for (const TWeakObjectPtr<UAvaRundownPagePlayer>& PlayerWeak : InPagePlayersWeak)
		{
			if (const UAvaRundownPagePlayer* PagePlayer = PlayerWeak.Get())
			{
				if (UAvaRundownPlaybackInstancePlayer* InstancePlayer = PagePlayer->FindInstancePlayerForPlayable(InPlayable))
				{
					return InstancePlayer;
				}
			}
		}
		return nullptr;
	}

	UAvaRundownPagePlayer* FindPagePlayerForPlayable(const UAvaPlayable* InPlayable, const TArray<TWeakObjectPtr<UAvaRundownPagePlayer>>& InPagePlayersWeak)
	{
		for (const TWeakObjectPtr<UAvaRundownPagePlayer>& PlayerWeak : InPagePlayersWeak)
		{
			UAvaRundownPagePlayer* PagePlayer = PlayerWeak.Get();
			if (PagePlayer && PagePlayer->HasPlayable(InPlayable))
			{
				return PagePlayer;
			}
		}
		return nullptr;
	}
}

UAvaRundownPageTransition* UAvaRundownPageTransition::MakeNew(UAvaRundown* InRundown)
{
	UAvaRundownPageTransition* NewTransition = NewObject<UAvaRundownPageTransition>(InRundown);
	NewTransition->TransitionId = FGuid::NewGuid();
	return NewTransition;
}

bool UAvaRundownPageTransition::AddEnterPage(UAvaRundownPagePlayer* InPagePlayer)
{
	using namespace UE::AvaMedia::Rundown::PageTransition::Private;

	if (!InPagePlayer)
	{
		return false;	
	}

	// Multi-page constraint: Prevent pages on the same layer
	for (const TWeakObjectPtr<UAvaRundownPagePlayer>& PlayerWeak : EnterPlayersWeak)
	{
		if (const UAvaRundownPagePlayer* Player = PlayerWeak.Get())
		{
			bool bHasLayer = false;
			InPagePlayer->ForEachInstancePlayer([this, &bHasLayer](const UAvaRundownPlaybackInstancePlayer* InInstancePlayer)
			{
				if (CachedTransitionLayers.Contains(InInstancePlayer->TransitionLayer.TagId))
				{
					bHasLayer = true;
				}
			});
			
			if (bHasLayer)
			{
				UE_LOGF(LogAvaRundown, Error,
						"Page Transition \"%ls\" Error: page %d can't be played with page %d because they are on the same layer.",
						*GetInstanceName(), InPagePlayer->PageId, Player->PageId);
				return false;
			}
		}
	}

	RegisterEnterPagePlayerEvents(InPagePlayer);
	
	return AddPagePlayer(InPagePlayer, EnterPlayersWeak);
}

bool UAvaRundownPageTransition::AddPlayingPage(UAvaRundownPagePlayer* InPagePlayer)
{
	return AddPagePlayer(InPagePlayer, PlayingPlayersWeak);
}

bool UAvaRundownPageTransition::AddExitPage(UAvaRundownPagePlayer* InPagePlayer)
{
	return AddPagePlayer(InPagePlayer, ExitPlayersWeak);
}

bool UAvaRundownPageTransition::IsVisibilityConstrained(const UAvaPlayable* InPlayable) const
{
	using namespace UE::AvaMedia::Rundown::PageTransition::Private;
	bool bAllPlayablesLoaded = true;
	bool bIsPlayableInThisTransition = false;
	
	for (const TWeakObjectPtr<UAvaRundownPagePlayer>& PlayerWeak : EnterPlayersWeak)
	{
		if (const UAvaRundownPagePlayer* Player = PlayerWeak.Get())
		{
			Player->ForEachInstancePlayer([InPlayable, &bIsPlayableInThisTransition, &bAllPlayablesLoaded](const UAvaRundownPlaybackInstancePlayer* InInstancePlayer)
			{
				if (const UAvaPlayable* Playable = GetPlayable(InInstancePlayer))
				{
					if (Playable == InPlayable)
					{
						bIsPlayableInThisTransition = true;
					}
					const EAvaPlayableStatus PlayableStatus = Playable->GetPlayableStatus();
					if (PlayableStatus != EAvaPlayableStatus::Loaded && PlayableStatus != EAvaPlayableStatus::Visible)
					{
						bAllPlayablesLoaded = false;
					}
				}
			});
		}
	}
	
	return bIsPlayableInThisTransition && !bAllPlayablesLoaded;
}

bool UAvaRundownPageTransition::CanStart(bool& bOutShouldDiscard)
{
	using namespace UE::AvaMedia::Rundown::PageTransition::Private;
	if (!AsyncAssetLoader)
	{
		// Start loading the RC referenced assets.
		AsyncAssetLoader = MakeShared<UE::AvaPlayback::Utils::FAsyncAssetLoader>();
		TSet<FSoftObjectPath> Assets;

		const UAvaBroadcast& Broadcast = UAvaBroadcast::Get();

		for (const TWeakObjectPtr<UAvaRundownPagePlayer>& PlayerWeak : EnterPlayersWeak)
		{
			const UAvaRundownPagePlayer* Player = PlayerWeak.Get();
			if (!Player)
			{
				continue;
			}

			// Only collect references and assets to load for player with local output.
			if (Player->bIsPreview || UE::AvaMedia::PlayableUtils::HasLocalPlay(Broadcast, Player->ChannelFName))
			{
				if (const TSharedPtr<const FAvaPlayableRemoteControlValues> Values = GetRemoteControlValues(Player))
				{
					FAvaPlayableRemoteControlValues::CollectReferencedAssetPaths(Values->ControllerValues, Assets);
					FAvaPlayableRemoteControlValues::CollectReferencedAssetPaths(Values->EntityValues, Assets);
				}
			}
		}
		AsyncAssetLoader->BeginLoadingAssets(Assets.Array());
	}

	for (const TWeakObjectPtr<UAvaRundownPagePlayer>& PlayerWeak : EnterPlayersWeak)
	{
		if (const UAvaRundownPagePlayer* Player = PlayerWeak.Get())
		{
			const int32 NumInstancePlayers = Player->GetNumInstancePlayers();
			for (int32 InstancePlayerIndex = 0; InstancePlayerIndex < NumInstancePlayers; ++InstancePlayerIndex)
			{
				if (const UAvaPlayable* Playable = GetPlayable(Player, InstancePlayerIndex))
				{
					const EAvaPlayableStatus PlayableStatus = Playable->GetPlayableStatus();

					if (PlayableStatus == EAvaPlayableStatus::Unknown
						|| PlayableStatus == EAvaPlayableStatus::Error)
					{
						// Discard the command
						bOutShouldDiscard = true;
						return false;
					}

					// todo: this might cause commands to become stale and fill the pending command list
					if (PlayableStatus == EAvaPlayableStatus::Unloaded)
					{
						return false;
					}

					// Asset status must be visible to run the command locally.
					// If not visible, the components are not yet added to the world.
					// For remote proxies, we can run the command immediately, it will wait for the asset to be visible on the server instead. 
					if (!Playable->IsRemoteProxy() && PlayableStatus != EAvaPlayableStatus::Visible)
					{
						// Keep the command in the queue for next tick.
						bOutShouldDiscard = false;
						return false;
					}

					if (!Playable->IsRemoteProxy() && !Playable->IsReadyToPlay())
					{
						// Keep the command in the queue for next tick.
						bOutShouldDiscard = false;
						return false;
					}
				}
				else
				{
					// Playables not created - Keep the command in the queue for next tick.
					bOutShouldDiscard = false;
					return false;
				}
			}
		}
		else
		{
			bOutShouldDiscard = true;
			return false;
		}
	}

	// Wait for RC referenced assets to be loaded.
	if (AsyncAssetLoader && !AsyncAssetLoader->IsLoadingCompleted())
	{
		bOutShouldDiscard = false;
		return false;
	}
	
	bOutShouldDiscard = true;
	return true;
}

void UAvaRundownPageTransition::Start()
{
	TWeakObjectPtr<UAvaRundownPageTransition> ThisWeak(this);
	auto StartEventHandler = [ThisWeak]()
	{
		if (UAvaRundownPageTransition* This = ThisWeak.Get())
		{
			This->Start_Synchronized();
		}
	};

	// Build unique signature for this event.
	FString StartEventSignature = FString(TEXT("PageTransitionStart_")) + TransitionId.ToString();
	
	if (const UAvaRundown* Rundown = GetRundown())
	{
		if (UAvaPlayableGroupManager* GroupManager = Rundown->GetPlaybackManager().GetPlayableGroupManager())
		{
			if (!GroupManager->IsSynchronizedEventPushed(StartEventSignature))
			{
				GroupManager->PushSynchronizedEvent(MoveTemp(StartEventSignature), MoveTemp(StartEventHandler));
			}
		}
	}
}

void UAvaRundownPageTransition::Start_Synchronized()
{
	RegisterToPlayableTransitionEvent();

	// Playables should be loaded at this point since the synchronized part is to wait on loading assets.
	MakePlayableTransition();

	bool bTransitionStarted = false;

	if (PlayableTransition)
	{
		LogDetailedTransitionInfo();
		bTransitionStarted = PlayableTransition->Start();
	}
	
	if (!bTransitionStarted)
	{
		Stop();
	}
}

void UAvaRundownPageTransition::Stop()
{
	if (PlayableTransition)
	{
		PlayableTransition->Stop();
		PlayableTransition = nullptr;
	}

	for (TWeakObjectPtr<UAvaRundownPagePlayer>& PagePlayerWeak : EnterPlayersWeak)
	{
		if (UAvaRundownPagePlayer* PagePlayer = PagePlayerWeak.Get())
		{
			UnregisterEnterPagePlayerEvents(PagePlayer);
		}
	}

	UnregisterFromPlayableTransitionEvent();

	if (UAvaRundown* Rundown = GetRundown())
	{
		Rundown->RemovePageTransition(this);
		Rundown->RemoveStoppedPagePlayers();

		// Make sure that there are no instance players
		for (const TObjectPtr<UAvaRundownPagePlayer>& PagePlayer : Rundown->GetPagePlayers())
		{
			PagePlayer->ForEachInstancePlayer([this, PagePlayer](UAvaRundownPlaybackInstancePlayer* InInstancePlayer)
			{
				if (InstancesMarkedForDiscard.Contains(InInstancePlayer->GetPlaybackInstanceId()))
				{
					UE_LOGF(LogAvaRundown, Error, "%ls Page Transition \"%ls\" has marked instance \"%ls\" for discard but it is still playing in page %d",
						*UE::AvaPlayback::Utils::GetBriefFrameInfo(), *GetInstanceName(), *InInstancePlayer->GetPlaybackInstanceId().ToString(), PagePlayer->PageId);			
				}
			});
		}
	}
	else
	{
		UE_LOGF(LogAvaRundown, Error, "Page Transition \"%ls\" Failed to remove transition: No rundown specified.", *GetInstanceName());
	}
}

bool UAvaRundownPageTransition::IsRunning() const
{
	return 	PlayableTransition ? PlayableTransition->IsRunning() : false;
}

bool UAvaRundownPageTransition::HasEnterPagesWithNoTransitionLogic() const
{
	const UAvaRundown* Rundown = GetRundown();
	if (!Rundown)
	{
		return false;
	}
	
	for (const TWeakObjectPtr<UAvaRundownPagePlayer>& PlayerWeak : EnterPlayersWeak)
	{
		if (const UAvaRundownPagePlayer* Player = PlayerWeak.Get())
		{
			// Don't rely on instance players at this point. Assets may not be loaded.
			const FAvaRundownPage& Page = Rundown->GetPage(Player->PageId);
			if (Page.IsValidPage() && !Page.HasTransitionLogic(Rundown))
			{
				return true;
			}
		}
	}
	return false;
}

bool UAvaRundownPageTransition::HasPagePlayer(const UAvaRundownPagePlayer* InPagePlayer) const
{
	return EnterPlayersWeak.Contains(InPagePlayer) || PlayingPlayersWeak.Contains(InPagePlayer) || ExitPlayersWeak.Contains(InPagePlayer);
}

bool UAvaRundownPageTransition::ContainsTransitionLayer(const FAvaTagId& InTagId) const
{
	return CachedTransitionLayers.Contains(InTagId);
}

UAvaRundown* UAvaRundownPageTransition::GetRundown() const
{
	return Cast<UAvaRundown>(GetOuter());
}

UAvaRundownPlaybackInstancePlayer* UAvaRundownPageTransition::FindInstancePlayerForPlayable(const UAvaPlayable* InPlayable) const
{
	using namespace UE::AvaMedia::Rundown::PageTransition;
	if (!InPlayable)
	{
		return nullptr;
	}
	
	if (UAvaRundownPlaybackInstancePlayer* FoundInstancePlayer = Private::FindInstancePlayerForPlayable(InPlayable, EnterPlayersWeak))
	{
		return FoundInstancePlayer;
	}
	if (UAvaRundownPlaybackInstancePlayer* FoundInstancePlayer = Private::FindInstancePlayerForPlayable(InPlayable, PlayingPlayersWeak))
	{
		return FoundInstancePlayer;
	}
	if (UAvaRundownPlaybackInstancePlayer* FoundInstancePlayer = Private::FindInstancePlayerForPlayable(InPlayable, ExitPlayersWeak))
	{
		return FoundInstancePlayer;
	}
	return nullptr;
}

UAvaRundownPagePlayer* UAvaRundownPageTransition::FindPagePlayerForPlayable(const UAvaPlayable* InPlayable) const
{
	using namespace UE::AvaMedia::Rundown::PageTransition;
	if (UAvaRundownPagePlayer* FoundPagePlayer = Private::FindPagePlayerForPlayable(InPlayable, EnterPlayersWeak))
	{
		return FoundPagePlayer;
	}
	if (UAvaRundownPagePlayer* FoundPagePlayer = Private::FindPagePlayerForPlayable(InPlayable, PlayingPlayersWeak))
	{
		return FoundPagePlayer;
	}
	if (UAvaRundownPagePlayer* FoundPagePlayer = Private::FindPagePlayerForPlayable(InPlayable, ExitPlayersWeak))
	{
		return FoundPagePlayer;
	}
	return nullptr;
}

void UAvaRundownPageTransition::OnTransitionEvent(UAvaPlayable* InPlayable, UAvaPlayableTransition* InTransition, EAvaPlayableTransitionEventFlags InTransitionFlags)
{
	using namespace UE::AvaPlayback::Utils;

	// not this transition.
	if (InTransition != PlayableTransition || PlayableTransition == nullptr)
	{
		return;
	}

	// Find the page player for this playable
	UAvaRundownPlaybackInstancePlayer* InstancePlayer = FindInstancePlayerForPlayable(InPlayable);
	UAvaRundownPagePlayer* PagePlayer = InstancePlayer ? InstancePlayer->GetPagePlayer() : nullptr;

	if (InstancePlayer && EnumHasAnyFlags(InTransitionFlags, EAvaPlayableTransitionEventFlags::MarkPlayableDiscard))
	{
		InstancesMarkedForDiscard.Add(InPlayable->GetInstanceId());
		
		UE_LOGF(LogAvaRundown, Verbose, "%ls Instance Player Marked for Discard: Id:%ls, Asset:\"%ls\" -> Transition %ls",
			*GetBriefFrameInfo(), *InstancePlayer->GetPlaybackInstanceId().ToString(), *InstancePlayer->SourceAssetPath.ToString(), *TransitionId.ToString());
	}
	
	if (InstancePlayer && EnumHasAnyFlags(InTransitionFlags, EAvaPlayableTransitionEventFlags::StopPlayable))
	{
		// Validating that we are not removing an "enter" playable.
		if (PlayableTransition->IsEnterPlayable(InPlayable))
		{
			UE_LOGF(LogAvaRundown, Error,
				"%ls Page Transition \"%ls\" Error: An \"enter\" playable is being discarded for page %d.",
				*GetBriefFrameInfo(), *GetInstanceName(), (PagePlayer ? PagePlayer->PageId : -1));
		}

		UE_LOGF(LogAvaRundown, Verbose, "%ls Page Transition \"%ls\" Stopping Instance Player: Id:%ls, Asset:\"%ls\"",
			*GetBriefFrameInfo(), *GetInstanceName(), *InstancePlayer->GetPlaybackInstanceId().ToString(), *InstancePlayer->SourceAssetPath.ToString());

		// With combo-templates, page players can be partially stopped.
		InstancePlayer->Stop();

		// Check if the page player is still playing (i.e. if all instance players have been stopped)
		if (PagePlayer && !PagePlayer->IsPlaying())
		{
			UE_LOGF(LogAvaRundown, Verbose, "%ls Stopping Page Player: PageId:%d", *GetBriefFrameInfo(), PagePlayer->PageId);

			// Stop the whole page player and propagate page events.
			PagePlayer->Stop();

			if (UAvaRundown* Rundown = PagePlayer->GetRundown())
			{
				Rundown->RemoveStoppedPagePlayers();
			}
			else
			{
				UE_LOGF(LogAvaRundown, Error, "Page Transition \"%ls\" failed to remove stopped players: No rundown specified.", *GetInstanceName());
			}
		}
	}
	
	if (EnumHasAnyFlags(InTransitionFlags, EAvaPlayableTransitionEventFlags::Finished))
	{
		using namespace UE::AvaPlayback::Utils;
		UE_LOGF(LogAvaRundown, Verbose, "%ls Finishing Page Transition: %ls", *GetBriefFrameInfo(), *GetBriefTransitionDescription());

		Stop();
	}
}

void UAvaRundownPageTransition::OnPlayableCreated(UAvaPlaybackGraph* InPlayback, UAvaPlayable* InPlayable)
{
	if (UAvaPlayableGroup* PlayableGroup = InPlayable->GetPlayableGroup())
	{
		PlayableGroup->RegisterVisibilityConstraint(this);
	}
}

void UAvaRundownPageTransition::AddPlayersToBuilder(
	FAvaPlayableTransitionBuilder& InOutBuilder, const TArray<TWeakObjectPtr<UAvaRundownPagePlayer>>& InPlayersWeak, const TCHAR* InCategory, EAvaPlayableTransitionEntryRole InEntryRole) const
{
	for (const TWeakObjectPtr<UAvaRundownPagePlayer>& PlayerWeak : InPlayersWeak)
	{
		if (const UAvaRundownPagePlayer* Player = PlayerWeak.Get())
		{
			AddPlayablesToBuilder(InOutBuilder, Player, InCategory, InEntryRole);
		}
	}
}

void UAvaRundownPageTransition::AddPlayablesToBuilder(FAvaPlayableTransitionBuilder& InOutBuilder, const UAvaRundownPagePlayer* InPlayer, const TCHAR* InCategory, EAvaPlayableTransitionEntryRole InEntryRole) const
{
	using namespace UE::AvaMedia::Rundown::PageTransition::Private;

	for (const TObjectPtr<UAvaRundownPlaybackInstancePlayer>& InstancePlayer : InPlayer->InstancePlayers)
	{
		EAvaPlayableTransitionEntryRole EntryRole = InEntryRole;

		// -- Special Transition Logic --
		if (InstancesBypassingTransition.Contains(InstancePlayer->GetPlaybackInstanceId()))
		{
			if (EntryRole != EAvaPlayableTransitionEntryRole::Enter)
			{
				continue;	// Completely skipping if not an "enter" instance.
			}

			// Enter instances excluded should still be part as playing pages.
			EntryRole = EAvaPlayableTransitionEntryRole::Playing;
		}
	
		if (UAvaPlayable* Playable = GetPlayable(InstancePlayer))
		{
			// Begin -- Special Exit Layers Logic -- 
			if (EntryRole == EAvaPlayableTransitionEntryRole::Playing)
			{
				// Kick out playing instances that overlap with exit layers.
				for (const FAvaTagHandle& ExitLayer : ExitLayers)
				{
					// Note: relies on the instance player having the correct transition layer (from the page).
					if (ExitLayer.Overlaps(InstancePlayer->TransitionLayer))
					{
						EntryRole = EAvaPlayableTransitionEntryRole::Exit;
						break;
					}
				}
			}
			// End -- Special Exit Layers Logic --

			const bool bPlayableAdded = InOutBuilder.AddPlayable(Playable, EntryRole);
			if (EntryRole == EAvaPlayableTransitionEntryRole::Enter && bPlayableAdded)
			{
				InOutBuilder.AddEnterPlayableValues(GetRemoteControlValues(InPlayer));

				// For reused instances, we add them again in the playing role.
				if (ReusedInstances.Contains(InstancePlayer->GetPlaybackInstanceId()))
				{
					InOutBuilder.AddPlayable(Playable, EAvaPlayableTransitionEntryRole::Playing, /*bInAllowMultipleAdd*/true);
				}
			}
		}
		else
		{
			// If this happens, likely the playable is not yet loaded.
			UE_LOGF(LogAvaRundown, Error,
				"%ls Page Transition \"%ls\" Error: Failed to retrieve \"%ls\" playable for instance {%ls} of page %d.",
				*UE::AvaPlayback::Utils::GetBriefFrameInfo(), *GetInstanceName(), InCategory, *GetPrettyInstancePlayerInfo(InstancePlayer), InPlayer->PageId);
		}
	}
}

void UAvaRundownPageTransition::MakePlayableTransition()
{
	FAvaPlayableTransitionBuilder TransitionBuilder;
	AddPlayersToBuilder(TransitionBuilder, EnterPlayersWeak, TEXT("Enter"), EAvaPlayableTransitionEntryRole::Enter);
	AddPlayersToBuilder(TransitionBuilder, PlayingPlayersWeak, TEXT("Playing"), EAvaPlayableTransitionEntryRole::Playing);
	AddPlayersToBuilder(TransitionBuilder, ExitPlayersWeak, TEXT("Exit"), EAvaPlayableTransitionEntryRole::Exit);
	PlayableTransition = TransitionBuilder.MakeTransition(this, TransitionId);

	if (PlayableTransition)
	{
		EAvaPlayableTransitionFlags TransitionFlags = EAvaPlayableTransitionFlags::None;

		// For non-TL enter pages, we need to kick out the playing pages too. 
		if (HasEnterPagesWithNoTransitionLogic())
		{
			TransitionFlags |= EAvaPlayableTransitionFlags::TreatPlayingAsExiting;
		}

		// Server-side validation needs to know about transitions with in-place playables.
		if (!ReusedInstances.IsEmpty())
		{
			TransitionFlags |= EAvaPlayableTransitionFlags::HasReusedPlayables;
		}

		if (bIsPreviewFrameTransition)
		{
			TransitionFlags |= EAvaPlayableTransitionFlags::PlayEnterPlayablesAtPreviewFrame;
		}

		PlayableTransition->SetTransitionFlags(TransitionFlags);
	}
}

FString UAvaRundownPageTransition::GetInstanceName() const
{
	if (const UAvaRundown* Rundown = GetRundown())
	{
		return Rundown->GetName() + TEXT(":") + TransitionId.ToString();
	}
	return TransitionId.ToString();
}

void UAvaRundownPageTransition::LogDetailedTransitionInfo() const
{
	if (!UE_LOG_ACTIVE(LogAvaRundown, Verbose))
	{
		return;
	}
	
	using namespace UE::AvaPlayback::Utils;
	UE_LOGF(LogAvaRundown, Verbose, "%ls Starting Page Transition \"%ls\":", *GetBriefFrameInfo(), *GetInstanceName());
	
	auto LogPlayers = [this](const TArray<TWeakObjectPtr<UAvaRundownPagePlayer>>& InPlayersWeak, const TCHAR* InCategory)
	{
		using namespace UE::AvaMedia::Rundown::PageTransition::Private;
		for (const TWeakObjectPtr<UAvaRundownPagePlayer>& PlayerWeak : InPlayersWeak)
		{
			if (const UAvaRundownPagePlayer* Player = PlayerWeak.Get())
			{
				UE_LOGF(LogAvaRundown, Verbose, "- %ls Page: %d, %ls.", InCategory, Player->PageId, *GetPrettyPagePlayerInfo(Player));
			}
		}
	};

	LogPlayers(EnterPlayersWeak, TEXT("Enter"));
	LogPlayers(PlayingPlayersWeak, TEXT("Playing"));
	LogPlayers(ExitPlayersWeak, TEXT("Exit"));
}

FString UAvaRundownPageTransition::GetBriefTransitionDescription() const
{
	auto MakePageIdList = [](const TArray<TWeakObjectPtr<UAvaRundownPagePlayer>>& InPlayersWeak) -> FString
	{
		FString PageIdList;
		for (const TWeakObjectPtr<UAvaRundownPagePlayer>& PlayerWeak : InPlayersWeak)
		{
			if (const UAvaRundownPagePlayer* Player = PlayerWeak.Get())
			{
				PageIdList += FString::Printf(TEXT("%s%d"), PageIdList.IsEmpty() ? TEXT("") : TEXT(", "), Player->PageId);
			}
		}
		return PageIdList.IsEmpty() ? TEXT("None") : PageIdList;
	};

	const FString EnterPageList = MakePageIdList(EnterPlayersWeak);
	const FString PlayingPageList = MakePageIdList(PlayingPlayersWeak);
	const FString ExitPageList = MakePageIdList(ExitPlayersWeak);
	return FString::Printf(TEXT("Page Transition \"%s\": Enter Page(s): [%s], Playing Page(s): [%s], Exit Page(s): [%s]."),
		*GetInstanceName(), *EnterPageList, *PlayingPageList, *ExitPageList);
}

void UAvaRundownPageTransition::RegisterToPlayableTransitionEvent()
{
	UAvaPlayable::OnTransitionEvent().RemoveAll(this);
	UAvaPlayable::OnTransitionEvent().AddUObject(this, &UAvaRundownPageTransition::OnTransitionEvent);
}

void UAvaRundownPageTransition::UnregisterFromPlayableTransitionEvent() const
{
	UAvaPlayable::OnTransitionEvent().RemoveAll(this);
}

void UAvaRundownPageTransition::RegisterEnterPagePlayerEvents(UAvaRundownPagePlayer* InPagePlayer)
{
	using namespace UE::AvaMedia::Rundown::PageTransition::Private;

	if (!InPagePlayer)
	{
		return;
	}
	
	for (const UAvaRundownPlaybackInstancePlayer* InstancePlayer : InPagePlayer->InstancePlayers)
	{
		CachedTransitionLayers.Add(InstancePlayer->TransitionLayer.TagId);

		// Register this transition as a visibility constraint for the playable group.
		if (const UAvaPlayable* Playable = GetPlayable(InstancePlayer))
		{
			if (UAvaPlayableGroup* PlayableGroup = Playable->GetPlayableGroup())
			{
				PlayableGroup->RegisterVisibilityConstraint(this);
			}
		}
		else if (UAvaPlaybackGraph* Playback = GetPlayback(InstancePlayer))
		{
			// If the playable is not created yet, register to the creation event.
			Playback->OnPlayableCreated.AddUObject(this, &UAvaRundownPageTransition::OnPlayableCreated);
		}
	}
}

void UAvaRundownPageTransition::UnregisterEnterPagePlayerEvents(UAvaRundownPagePlayer* InPagePlayer)
{
	using namespace UE::AvaMedia::Rundown::PageTransition::Private;
	
	if (!InPagePlayer)
	{
		return;
	}
	
	InPagePlayer->ForEachInstancePlayer([this](const UAvaRundownPlaybackInstancePlayer* InInstancePlayer)
	{
		if (UAvaPlaybackGraph* Playback = GetPlayback(InInstancePlayer))
		{
			Playback->OnPlayableCreated.RemoveAll(this);
			Playback->ForEachPlayable([this](const UAvaPlayable* InPlayable)
			{
				if (UAvaPlayableGroup* PlayableGroup = InPlayable->GetPlayableGroup())
				{
					PlayableGroup->UnregisterVisibilityConstraint(this);
				}
			});
		}
	});
}

bool UAvaRundownPageTransition::AddPagePlayer(UAvaRundownPagePlayer* InPagePlayer, TArray<TWeakObjectPtr<UAvaRundownPagePlayer>>& OutPagePlayersWeak)
{
	if (!InPagePlayer)
	{
		return false;	
	}

	OutPagePlayersWeak.Add(InPagePlayer);

	UpdateChannelName(InPagePlayer);

	return true;
}

void UAvaRundownPageTransition::UpdateChannelName(const UAvaRundownPagePlayer* InPagePlayer)
{
	check(InPagePlayer);
	
	if (ChannelName.IsNone())
	{
		ChannelName = InPagePlayer->ChannelFName;
	}
	else
	{
		using namespace UE::AvaMedia::Rundown::PageTransition::Private;
		using namespace UE::AvaPlayback::Utils;

		// Validate the channel is the same.
		if (ChannelName != InPagePlayer->ChannelFName)
		{
			UE_LOGF(LogAvaRundown, Error, "%ls Page Transition \"%ls\": Adding Page: %d, {%ls} in a different channel than previous pages (\"%ls\").",
				*GetBriefFrameInfo(), *GetInstanceName(), InPagePlayer->PageId, *GetPrettyPagePlayerInfo(InPagePlayer), *ChannelName.ToString());
		}
	}
}

