// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagHandle.h"
#include "Playable/AvaPlayable.h"
#include "Playback/Transition/AvaPlaybackTransition.h"
#include "UObject/Object.h"
#include "AvaRundownPageTransition.generated.h"

class FAvaPlayableTransitionBuilder;
class UAvaPlayableTransition;
class UAvaPlaybackGraph;
class UAvaRundown;
class UAvaRundownPagePlayer;
class UAvaRundownPlaybackInstancePlayer;
enum class EAvaPlayableTransitionEntryRole : uint8;

namespace UE::AvaPlayback::Utils
{
	class FAsyncAssetLoader;
}

/**
 * Class for creating and tracking page transitions in the rundown.
 * It is responsible for creating the playable transition object when requested from
 * the playback graphs.
 */
UCLASS()
class UAvaRundownPageTransition : public UAvaPlaybackTransition
{
	GENERATED_BODY()
	
public:
	static UAvaRundownPageTransition* MakeNew(UAvaRundown* InRundown);
	
	bool AddEnterPage(UAvaRundownPagePlayer* InPagePlayer);
	bool AddPlayingPage(UAvaRundownPagePlayer* InPagePlayer);
	bool AddExitPage(UAvaRundownPagePlayer* InPagePlayer);

	TConstArrayView<TWeakObjectPtr<UAvaRundownPagePlayer>> GetEnterPlayers() const { return EnterPlayersWeak; }
	TConstArrayView<TWeakObjectPtr<UAvaRundownPagePlayer>> GetPlayingPlayers() const { return PlayingPlayersWeak; }
	TConstArrayView<TWeakObjectPtr<UAvaRundownPagePlayer>> GetExitPlayers() const { return ExitPlayersWeak; }
	
	//~ Begin IAvaPlayableVisibilityConstraint
	virtual bool IsVisibilityConstrained(const UAvaPlayable* InPlayable) const override;
	//~ End IAvaPlayableVisibilityConstraint
	
	//~ Begin UAvaPlaybackTransition
	virtual bool CanStart(bool& bOutShouldDiscard) override;
	virtual void Start() override;
	virtual void Stop() override;
	virtual bool IsRunning() const override;
	//~ End UAvaPlaybackTransition

	/** Returns the channel this transition is happening in. A transition can only have pages within the same channel. */
	FName GetChannelName() const { return ChannelName; }

	bool HasEnterPages() const { return !EnterPlayersWeak.IsEmpty(); }

	bool HasEnterPagesWithNoTransitionLogic() const;

	bool HasPagePlayer(const UAvaRundownPagePlayer* InPagePlayer) const;

	bool ContainsTransitionLayer(const FAvaTagId& InTagId) const;

	UAvaRundown* GetRundown() const;

	FString GetBriefTransitionDescription() const;

	/** Instances will not be added to the playable transition. */
	UPROPERTY(Transient)
	TSet<FGuid> InstancesBypassingTransition;

	/** Reused existing instance player. Will be added both as "entering" and "playing" in the playable transition. */
	UPROPERTY(Transient)
	TSet<FGuid> ReusedInstances;

	/** Layers to kick out as part of this transition command. */
	TArray<FAvaTagHandle> ExitLayers;

	/**
	 * Special mark if any of the enter playables where issued with a PreviewFrame play type.
	 * Current logic will mark the whole transition.
	 */
	bool  bIsPreviewFrameTransition = false;

protected:
	/** Implementation of the Start function intended to be synchronized on cluster. */
	void Start_Synchronized();
	
	UAvaRundownPlaybackInstancePlayer* FindInstancePlayerForPlayable(const UAvaPlayable* InPlayable) const;
	UAvaRundownPagePlayer* FindPagePlayerForPlayable(const UAvaPlayable* InPlayable) const;

	void OnTransitionEvent(UAvaPlayable* InPlayable, UAvaPlayableTransition* InTransition, EAvaPlayableTransitionEventFlags InTransitionFlags);
	void OnPlayableCreated(UAvaPlaybackGraph* InPlayback, UAvaPlayable* InPlayable);

	void AddPlayersToBuilder(FAvaPlayableTransitionBuilder& InOutBuilder, const TArray<TWeakObjectPtr<UAvaRundownPagePlayer>>& InPlayersWeak, const TCHAR* InCategory, EAvaPlayableTransitionEntryRole InEntryRole) const;
	void AddPlayablesToBuilder(FAvaPlayableTransitionBuilder& InOutBuilder, const UAvaRundownPagePlayer* InPlayer, const TCHAR* InCategory, EAvaPlayableTransitionEntryRole InEntryRole) const;

	void MakePlayableTransition();

	FString GetInstanceName() const;
	void LogDetailedTransitionInfo() const;

	void RegisterToPlayableTransitionEvent();
	void UnregisterFromPlayableTransitionEvent() const;

	void RegisterEnterPagePlayerEvents(UAvaRundownPagePlayer* InPagePlayer);
	void UnregisterEnterPagePlayerEvents(UAvaRundownPagePlayer* InPagePlayer);

	bool AddPagePlayer(UAvaRundownPagePlayer* InPagePlayer, TArray<TWeakObjectPtr<UAvaRundownPagePlayer>>& OutPagePlayersWeak);	
	void UpdateChannelName(const UAvaRundownPagePlayer* InPagePlayer);

protected:
	FName ChannelName;
	
	TArray<TWeakObjectPtr<UAvaRundownPagePlayer>> EnterPlayersWeak;
	TArray<TWeakObjectPtr<UAvaRundownPagePlayer>> PlayingPlayersWeak;
	TArray<TWeakObjectPtr<UAvaRundownPagePlayer>> ExitPlayersWeak;

	TSet<FAvaTagId> CachedTransitionLayers;
	
	TSet<FGuid> InstancesMarkedForDiscard;

	TSharedPtr<UE::AvaPlayback::Utils::FAsyncAssetLoader> AsyncAssetLoader;
	
	UPROPERTY(Transient)
	TObjectPtr<UAvaPlayableTransition> PlayableTransition;

	friend class FAvaRundownPageTransitionBuilder;
};