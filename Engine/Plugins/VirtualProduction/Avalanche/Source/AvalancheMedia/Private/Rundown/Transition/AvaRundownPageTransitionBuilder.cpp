// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageTransitionBuilder.h"

#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownPagePlayer.h"
#include "Rundown/Transition/AvaRundownPageTransition.h"

FAvaRundownPageTransitionBuilder::~FAvaRundownPageTransitionBuilder()
{
	if (Rundown)
	{
		for (UAvaRundownPageTransition* PageTransition : PageTransitions)
		{
			// Add any remaining playing pages in the channel that are not either exit or enter page already.
			for (UAvaRundownPagePlayer* PagePlayer : Rundown->GetPagePlayers())
			{
				if (PagePlayer->ChannelFName == PageTransition->GetChannelName())
				{
					if (!PageTransition->HasPagePlayer(PagePlayer))
					{
						PageTransition->AddPlayingPage(PagePlayer);
					}
				}
			}
			
			Rundown->AddPageTransition(PageTransition);

			bool bShouldDiscard = false;
			if (PageTransition->CanStart(bShouldDiscard))
			{
				PageTransition->Start();
			}
			else
			{
				// Some playables are still loading, push the command for later execution.
				Rundown->GetPlaybackManager().PushPlaybackTransitionStartCommand(PageTransition);
			}
		}
	}
}

UAvaRundownPageTransition* FAvaRundownPageTransitionBuilder::FindTransition(FName InChannelName) const
{
	for (UAvaRundownPageTransition* PageTransition : PageTransitions)
	{
		if (PageTransition->GetChannelName() == InChannelName)
		{
			return PageTransition;
		}
	}
	return nullptr;	
}

UAvaRundownPageTransition* FAvaRundownPageTransitionBuilder::FindTransition(const UAvaRundownPagePlayer* InPlayer) const
{
	// Currently, the only batching criteria is the channel.
	return InPlayer ? FindTransition(InPlayer->ChannelFName) : nullptr;
}

UAvaRundownPageTransition* FAvaRundownPageTransitionBuilder::FindOrAddTransition(FName InChannelName)
{
	UAvaRundownPageTransition* PageTransition = FindTransition(InChannelName);
	return PageTransition ? PageTransition : AddTransition(InChannelName);
}

UAvaRundownPageTransition* FAvaRundownPageTransitionBuilder::FindOrAddTransition(const UAvaRundownPagePlayer* InPlayer)
{
	UAvaRundownPageTransition* PageTransition = FindTransition(InPlayer);
	return PageTransition ? PageTransition : AddTransition(InPlayer->ChannelFName);
}

UAvaRundownPageTransition* FAvaRundownPageTransitionBuilder::AddTransition(FName InChannelName)
{
	UAvaRundownPageTransition* PageTransition = UAvaRundownPageTransition::MakeNew(Rundown);
	if (PageTransition)
	{
		PageTransition->ChannelName = InChannelName;
		PageTransitions.Add(PageTransition);
	}
	return PageTransition;
}