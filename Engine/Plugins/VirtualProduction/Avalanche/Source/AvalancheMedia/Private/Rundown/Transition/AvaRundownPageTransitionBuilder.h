// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

class UAvaRundown;
class UAvaRundownPagePlayer;
class UAvaRundownPageTransition;

/**
 * Page Transition Builder to batch pages appropriately in page transitions
 * to provide support for multiple pages being played at the same time.
 *
 * Summary of the page batching rules (as of yet):
 * - For enter pages, the batching requires all enter pages (of any TL Layer) of a given channel
 * to be in the same transition.
 * - (*) There can only be one enter page of a particular layer in the transition. This rule should be
 * implemented by TL itself, but it is not atm. It is thus a hard coded rule by the rundown
 * transition system for now. The rule for now is to keep only the first page being added.
 * - For exit pages, when they are part of a transition that has enter pages, we can add all
 * the currently playing pages (in the given channel) as "exit" pages.
 * - If the transition is a "take out" only transition, then we need to batch the exit pages
 * per layers (on top of per channel) since the whole transition has an override that allows only
 * one layer to be specified. This is not done at the moment. For take out, there is no batching
 * at all atm, all pages are added in a separate transition.
 *
 * Note: for now, FAvaRundownPageTransitionBuilder is not used for stopping pages. This may change
 * depending on refactor of the underlying TL and transition system. Some traces of this
 * case are left in this class for that reason, so it is easy to adapt it for the "take out"
 * case.
 */
class FAvaRundownPageTransitionBuilder
{
public:
	FAvaRundownPageTransitionBuilder(UAvaRundown* InRundown) : Rundown(InRundown) {}

	~FAvaRundownPageTransitionBuilder();

	/**
	 * @brief Finds a suitable page transition object for the channel.
	 * @param InChannel channel we are requesting the transition object for.
	 * @return the requested page transition object or null if not found.
	 */
	UAvaRundownPageTransition* FindTransition(FName InChannelName) const;

	/**
	 * @brief Finds a suitable page transition object for the given page player.
	 * @param InPlayer Page player we are requesting the transition object for.
	 * @return the requested page transition object or null if not found.
	 */
	UAvaRundownPageTransition* FindTransition(const UAvaRundownPagePlayer* InPlayer) const;

	/**
	 * @brief Retrieves a suitable page transition object for the given channel.
	 * @param InChannel Channel we are requesting the transition object for.
	 * @return the requested page transition object.
	 *
	 * This is where the batching occurs according to criteria.
	 */
	UAvaRundownPageTransition* FindOrAddTransition(FName InChannelName);
	
	/**
	 * @brief Retrieves a suitable page transition object for the given page player.
	 * @param InPlayer Page player we are requesting the transition object for.
	 * @return the requested page transition object.
	 *
	 * This is where the batching occurs according to criteria.
	 */
	UAvaRundownPageTransition* FindOrAddTransition(const UAvaRundownPagePlayer* InPlayer);

private:
	UAvaRundownPageTransition* AddTransition(FName InChannelName);
	
private:
	UAvaRundown* Rundown;

	/** There will be a transition for each channel. */
	TArray<UAvaRundownPageTransition*> PageTransitions;
};