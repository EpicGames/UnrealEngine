// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/AvaSoftAssetPtr.h"
#include "Misc/EnumClassFlags.h"
#include "AvaMediaDefines.generated.h"

/**
 * Channel state is a union summary of the output's states.
 */
UENUM()
enum class EAvaBroadcastChannelState : uint8
{
	/** Indicates that all channel outputs are offline. */
	Offline,
	/** Indicates that at least some of the channel outputs are idle (but none are live). */
	Idle,
	/** Indicates that at least some of the channel outputs are live.*/
	Live,

	Max UMETA(Hidden),
};

/**
 * The channel type defines what it is used for in the broadcast framework.
 *
 * Primarily, the channel type is intended to resolve channel collisions between
 * simultaneous "program" and "preview" playbacks on a given system. In other words:
 * - Channel selection for rundown pages is restricted to "program" channels.
 * - Channel selection for preview is restricted to "preview" channels.
 *
 * It is thus not possible for a user to mistakenly select the same channel for both preview and program.
 * 
 * Some additional restrictions are applied according to channel type:
 * - preview channels must only have outputs local to the process. "Remote" previews are not supported.
 * - [backend] playback request type (program or preview) must match with the channel type. This is a safety
 *	 net for any other extended code paths that are not in the Motion Design plugin.
 */
UENUM()
enum class EAvaBroadcastChannelType : uint8
{
	Program,
	Preview
};

/**
 *	Status of the media output.
 */
UENUM()
enum class EAvaBroadcastOutputState : uint8
{
	/** Invalid/Uninitialized state. */
	Invalid,
	/** For remote output that is not connected, output disabled. */
	Offline,
	/** Server Connected or Local (MediaCapture Status: Stopped) */
	Idle,
	/** MediaCapture Status: Preparing */
	Preparing,
	/** Broadcasting (MediaCapture Status: Capturing) */
	Live,
	/** MediaCapture Error (unrecoverable) */
	Error
};

/**
 * In case the broadcast device is Live (see EAvaBroadcastOutputState),
 * this extra status indicates if the device is operating normally.
 */
UENUM()
enum class EAvaBroadcastIssueSeverity : uint8
{
	None,
	Warnings,
	Errors,

	Max UMETA(Hidden),
};

//An enum indicating what changed in Broadcast
enum class EAvaBroadcastChange : uint8
{
	None            = 0,
	CurrentProfile  = 1 << 0, //When the Current Profile Changed
	ChannelGrid     = 1 << 1, //When a Channel has been Added/Removed
	ChannelRename   = 1 << 2, //When a Channel has been Renamed
	ChannelType     = 1 << 3, //When a Channel has changed type.
	OutputDevices   = 1 << 4, //When output devices have changed
	All             = 0xFF
};
ENUM_CLASS_FLAGS(EAvaBroadcastChange);

//An enum indicating what changed in Channel
enum class EAvaBroadcastChannelChange : uint8
{
	None         = 0,
	State        = 1 << 0,
	RenderTarget = 1 << 1, 
	MediaOutputs = 1 << 2,
	Settings     = 1 << 3,
};
ENUM_CLASS_FLAGS(EAvaBroadcastChannelChange);

/**
 * Action to perform when a media capture overrun (main vs render thread) occurs on a broadcast channel.
 */
UENUM()
enum class EAvaBroadcastOutputOverrunAction : uint8
{
	/** Flush rendering thread such that all scheduled commands are executed. */
	Flush,
	/** Skip capturing a frame if readback is trailing too much. */
	Skip
};

/**
 * The status of a playable object.
 *
 * This is related to the status of the transient playable object which is
 * duplicated from the source asset. The transient playable object is also
 * referred to as the "runtime" asset instance.
 */
UENUM()
enum class EAvaPlayableStatus : uint8
{
	Unknown,
	Error,
	Unloaded,
	Loading,
	Loaded,
	Visible
};

UENUM()
enum class EAvaPlayableSequenceEventType : uint8
{
	None UMETA(Hidden),
	Started,
	Paused,
	Finished
};

/**
 * Transition flags carry additional context to help resolve the behaviors.
 */
enum class EAvaPlayableTransitionFlags : uint8
{
	None = 0,
	/** Playing playables will be treated as exit playables. */
	TreatPlayingAsExiting = 1 << 0,
	/** Transition contains some reused playables (i.e. both enter and playing). */
	HasReusedPlayables = 1 << 1,
	/** Special logic to override the enter playables behaviors for the "PreviewFrame". */
	PlayEnterPlayablesAtPreviewFrame = 1 << 2,
};
ENUM_CLASS_FLAGS(EAvaPlayableTransitionFlags);

enum class EAvaPlayableTransitionEventFlags : uint8
{
	None = 0,
	/** The transition is starting. */
	Starting = 1 << 0,
	/** The enter playable can be shown. */
	ShowPlayable = 1 << 1,
	/** The playable needs to be stopped. */
	StopPlayable = 1 << 2,
	/** The playable needs to be discarded at the end of the transition. */
	MarkPlayableDiscard = 1 << 3,
	/** The transition is finished and can be cleaned up. */
	Finished = 1 << 4,
};
ENUM_CLASS_FLAGS(EAvaPlayableTransitionEventFlags);

/** Flags for the remote control values update command. */
UENUM(meta=(Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EAvaPlayableRCUpdateFlags : uint8
{
	None = 0,
	/** Execute Controller Behaviors */
	ExecuteControllerBehaviors = 1 << 0,
	/**
	 *  The controller values are to be considered as events and will only apply to the event controllers.
	 *  Non-event controllers and entity values are ignored during the execution of this update.
	 */
	ControllerValuesAsEvents = 1 << 1,
};
ENUM_CLASS_FLAGS(EAvaPlayableRCUpdateFlags);

/**
 * Parameters of a player entry for active player nodes in playback graph. 
 */
struct FAvaPlaybackPlayerParameters
{
	TWeakObjectPtr<class UAvaPlaybackNode> Node;
	FAvaSoftAssetPtr Asset;
};

/*
 * Parameters used to tell a Channel its Playback Settings
 */
struct FAvaPlaybackChannelParameters
{
	/** Channel index for the current playback setting. */
	int32 ChannelIndex = -1;

	/**
	 * If a playback node is reached during tick traversal,
	 * it is going to add an entry.
	 */
	TArray<FAvaPlaybackPlayerParameters> Players;

	bool HasAssets() const
	{
		return !Players.IsEmpty();
	}
};

/*
 * Parameters for Event/Action
 */
struct FAvaPlaybackEventParameters
{
	/** List of channel indices that this event originates from. */
	TArray<int32> ChannelIndices;

	FAvaSoftAssetPtr Asset;

	bool IsAssetValid() const
	{
		return !Asset.IsNull();
	}

	const FSoftObjectPath& GetAssetPath() const
	{
		return Asset.ToSoftObjectPath();
	}
	
	void RequestTriggerEventAction()
	{
		bTriggerEventAction = true;
	}

	bool ShouldTriggerEventAction() const
	{
		return bTriggerEventAction;
	}
	
protected:
	
	bool bTriggerEventAction = false;
};


enum class EAvaRundownPageListChange : uint8
{
	None = 0,

	AddedPages            = 1 << 0,
	RemovedPages          = 1 << 1,
	RenumberedPageId      = 1 << 2,
	SubListAddedOrRemoved = 1 << 3,
	SubListRenamed        = 1 << 4,
	ReorderedPageView     = 1 << 5,

	All                   = 0xFF,
};
ENUM_CLASS_FLAGS(EAvaRundownPageListChange);

enum class EAvaRundownPageChanges : uint8
{
	None = 0,

	AnimationSettings       = 1 << 0,
	RemoteControlValues     = 1 << 1,
	Blueprint				= 1 << 2,
	Channel					= 1 << 3,
	Status					= 1 << 4,
	Name					= 1 << 5,
	FriendlyName			= 1 << 6,
	Commands				= 1 << 7,

	All              = 0xFF,
};
ENUM_CLASS_FLAGS(EAvaRundownPageChanges);

/**
 *	The status of the playback asset on disk.
 *
 *	This is also sometimes referred to as the "source" asset, by contrast to the
 *	runtime (or managed) transient asset for runtime playback. It will also refer to
 *	the source Motion Design asset, rather than a playback object.
 */
UENUM()
enum class EAvaPlaybackAssetStatus
{
	/** Invalid status. */
	Unknown,
	/** Missing asset */
	Missing,
	/**
	 * Missing asset dependencies
	 * Note: An Motion Design asset can run even with missing dependencies.
	 */
	MissingDependencies,
	/** Asset is out of date from compare with remote. */
	NeedsSync,
	/** Asset is fully available and up to date. */
	Available
};

/**
 * The status of the playback object.
 *
 * This is related to the status of the transient playback object which is
 * duplicated from the source asset. The transient playback object is also
 * referred to as the "managed" or "runtime" asset.
 */
UENUM()
enum class EAvaPlaybackStatus
{
	/** Invalid status. */
	Unknown,
	/** Missing asset (or out of date?). */
	Missing,
	/** Asset is being downloaded. */
	Syncing,
	/** Asset is available, not loaded. */
	Available,
	/** Load has been requested. */
	Loading,
	/** Asset is loaded in memory and ready to play. */
	Loaded,
	/** Start has been requested. */
	Starting,
	/** Currently playing. */
	Started,
	/** Stop has been requested. After stopping, status goes back to Loaded. */
	Stopping,
	/** Unload has been requested. After unloaded, status goes back to Available. */
	Unloading,
	/** Something bad happened. */
	Error
};

/**
 * Rundown's page list type.
 */
UENUM(BlueprintType, DisplayName = "Motion Design Rundown Page List Type")
enum class EAvaRundownPageListType : uint8
{
	Template,
	Instance,
	View
};

USTRUCT(BlueprintType, DisplayName = "Motion Design Rundown Page List Reference")
struct FAvaRundownPageListReference
{
	GENERATED_BODY()

	UPROPERTY()
	EAvaRundownPageListType Type = EAvaRundownPageListType::Instance;

	UPROPERTY()
	FGuid SubListId;

	bool operator==(const FAvaRundownPageListReference& InOther) const
	{
		if (Type != InOther.Type)
		{
			return false;
		}

		if (Type == EAvaRundownPageListType::View && SubListId != InOther.SubListId)
		{
			return false;
		}

		return true;
	}

	bool operator!=(const FAvaRundownPageListReference& InOther) const { return !(*this == InOther); }
};
