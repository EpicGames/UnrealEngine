// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/StringFwd.h"
#include "Engine/DeveloperSettings.h"
#include "Framework/AvaInstanceSettings.h"
#include "Logging/LogVerbosity.h"
#include "Math/MathFwd.h"
#include "PixelFormat.h"
#include "Playable/AvaPlayableSettings.h"
#include "UObject/SoftObjectPtr.h"

#include "AvaMediaSettings.generated.h"

class UUserWidget;

#define UE_API AVALANCHEMEDIA_API

/**
 * Defines the verbosity level of the logging system.
 * This enum mirrors ELogVerbosity but can be used directly as a configuration property.
 */
UENUM()
enum class EAvaMediaLogVerbosity : uint8
{
	NoLogging = 0,
	Fatal,
	Error,
	Warning,
	Display,
	Log,
	Verbose,
	VeryVerbose
};

USTRUCT()
struct FAvaPlaybackServerLoggingEntry
{
	GENERATED_BODY()

	UPROPERTY(config, EditAnywhere, Category=Settings)
	FName Category;

	UPROPERTY(config, EditAnywhere, Category=Settings)
	EAvaMediaLogVerbosity VerbosityLevel = EAvaMediaLogVerbosity::VeryVerbose;
};

USTRUCT()
struct FAvaMediaLocalPlaybackServerSettings
{
	GENERATED_BODY()

	/** Name given to the game mode local playback server started as a separate process. */
	UPROPERTY(config, EditAnywhere, Category = Settings)
	FString ServerName = TEXT("LocalServer");

	/** Main window resolution of the local playback server. */
	UPROPERTY(config, EditAnywhere, Category = Settings)
	FIntPoint Resolution = FIntPoint(960, 540);

	/** Enable a log console for the local server process. */
	UPROPERTY(config, EditAnywhere, Category = Settings)
	bool bEnableLogConsole = false;

	/**
	 * Disables python with the -DisablePython command. 
	 * This is recommended for performance, as Python garbage collection call could impact the overall performance of a garbage collection pass.
	 */
	UPROPERTY(config, EditAnywhere, DisplayName="Disable Python (Performance)", Category = Settings)
	bool bDisablePython = true;

	/** Extra command line arguments. */
	UPROPERTY(config, EditAnywhere, Category = Settings)
	FString ExtraCommandLineArguments;

	/**
	 * Which logs to include and with which verbosity level
	*/
	UPROPERTY(Config, EditAnywhere, Category="Settings")
	TArray<FAvaPlaybackServerLoggingEntry> Logging;
};

UCLASS(MinimalAPI, config=Engine, meta=(DisplayName="Playback & Broadcast"))
class UAvaMediaSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UAvaMediaSettings();

	static const UAvaMediaSettings& Get() { return *GetSingletonInstance();}
	static UAvaMediaSettings& GetMutable() {return *GetSingletonInstance();}

	UE_API static ELogVerbosity::Type ToLogVerbosity(EAvaMediaLogVerbosity InAvaMediaLogVerbosity);

	/** Generates the command line arguments for the local playback server */
	UE_API FString GenerateLocalPlaybackServerCommandLine() const;

	//~ Begin UObject
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void PostReloadConfig(FProperty* InPropertyThatWasLoaded) override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	/** Specifies the background clear color for the channel. */
	UPROPERTY(config, EditAnywhere, Category = "Broadcast")
	FLinearColor ChannelClearColor = FLinearColor::Black;

	/** Pixel format used if no media output has specific format requirement. */
	UPROPERTY(config, EditAnywhere, Category = "Broadcast")
	TEnumAsByte<EPixelFormat> ChannelDefaultPixelFormat = EPixelFormat::PF_B8G8R8A8;

	/** Resolution used if no media output has specific resolution requirement. */
	UPROPERTY(config, EditAnywhere, Category = "Broadcast")
	FIntPoint ChannelDefaultResolution = FIntPoint(1920, 1080);

	/**
	 * Action to perform when game thread overruns render thread and all frames are in flights being captured / readback. 
	 */
	UPROPERTY(config, EditAnywhere, Category = "Broadcast")
	EAvaBroadcastOutputOverrunAction ChannelOutputOverrunAction = EAvaBroadcastOutputOverrunAction::Skip;
	
	/**
	 * Enables drawing the placeholder widget when there is no Motion Design asset playing.
	 * If false, the channel is cleared to the background color.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Broadcast")
	bool bDrawPlaceholderWidget = false;
	
	/** Specify a place holder widget to render when no Motion Design asset is playing. */
	UPROPERTY(config, EditAnywhere, Category = "Broadcast")
	TSoftClassPtr<UUserWidget> PlaceholderWidgetClass;

	/**
	 * Default resolution for rundown preview.
	 * This resolution may be lower than the broadcast resolution to improve gpu performance.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Rundown")
	FIntPoint PreviewDefaultResolution = FIntPoint(960, 540);

	UPROPERTY(config, EditAnywhere, Category = "Rundown")
	FString PreviewChannelName;

	/**
	 * Whether to allow re-using the instances of a page when doing preview frame.
	 * Default to false as preview frame kicks everything existing in the layer of the page to preview.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Rundown")
	bool bAllowReuseInPreviewFrame = false;

	/**
	 * Special logic will not play any transition if the RC values are the same.
	 * This applies to combo templates.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Rundown")
	bool bEnableComboTemplateSpecialLogic = true;

	/**
	 * Special logic will not play any transition if the RC values are the same.
	 * This applies to single (non-combo) templates.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Rundown")
	bool bEnableSingleTemplateSpecialLogic = false;

	/**
	 * Enables the propagation of controller events to pages playing in program channels. 
	 */
	UPROPERTY(config, EditAnywhere, Category = "Rundown")
	bool bRundownPushControllerEventsToProgram = true;

	/**
	 * Enables the propagation of controller events to pages playing in preview channels. 
	 */
	UPROPERTY(config, EditAnywhere, Category = "Rundown")
	bool bRundownPushControllerEventsToPreview = true;
	
	/** Whether playback client is started automatically. */
	UPROPERTY(config, EditAnywhere, Category = "Playback Client")
	bool bAutoStartPlaybackClient = false;

	/** Enable verbose logging for playback client. */
	UPROPERTY(config, EditAnywhere, Category = "Playback Client")
	bool bVerbosePlaybackClientLogging = false;

	/** Defines the interval in seconds in which servers are being pinged by the client. */
	UPROPERTY(config, EditAnywhere, Category = "Playback Client")
	float PingInterval = 2.5f;

	/** If servers do not respond after the ping timeout interval in seconds, they are disconnected. */
	UPROPERTY(config, EditAnywhere, Category = "Playback Client")
	float PingTimeoutInterval = 3*2.5f;

	/** Defines the timeout, in seconds, after which a pending status request is dropped and issued again. */
	UPROPERTY(config, EditAnywhere, Category = "Playback Client")
	float ClientPendingStatusRequestTimeout = 5.0f;
	
	/** Whether playback server is started automatically. */
	UPROPERTY(config, EditAnywhere, Category = "Playback Server")
	bool bAutoStartPlaybackServer = false;

	/** Name given to the playback server. If empty, the server name will be the computer name. */
	UPROPERTY(config, EditAnywhere, Category = "Playback Server")
	FString PlaybackServerName;

	/**
	 * Determines the verbosity level of the playback server's log replication.
	 * The server is not going to replicate any log event that is below this log level.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Playback Server")
	EAvaMediaLogVerbosity PlaybackServerLogReplicationVerbosity = EAvaMediaLogVerbosity::Error;

	/** Defines the timeout, in seconds, after which a pending status request is dropped and issued again. */
	UPROPERTY(config, EditAnywhere, Category = "Playback Server")
	float ServerPendingStatusRequestTimeout = 5.0f;

	/** Defines the timeout, in seconds, after which pending playback commands are discarded. */
	UPROPERTY(config, EditAnywhere, Category = "Playback Server")
	float ServerPendingPlaybackCommandTimeout = 5.0f;

	/** Settings for the local playback server process. See "Launch Local Server" in the broadcast editor toolbar. */
	UPROPERTY(config, EditAnywhere, Category = "Playback Server")
	FAvaMediaLocalPlaybackServerSettings LocalPlaybackServerSettings;
	
	/** If true, the playback objects are kept in memory after pages are stopped. They are unloaded otherwise.*/
	UPROPERTY(Config, EditAnywhere, Category = "Playback Manager")
	bool bKeepPagesLoaded = false;

	UPROPERTY(Config, EditAnywhere, Category = "Playback Manager")
	FAvaInstanceSettings AvaInstanceSettings;

	UPROPERTY(Config, EditAnywhere, Category = "Playback Manager")
	FAvaPlayableSettings PlayableSettings;

	/**
	 * Maximum cached Managed Motion Design assets used for rundown editor's page details.
	 * A value of 0 indicate the cache will grow without limit.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Managed Motion Design Instance Cache", meta = (DisplayName = "Maximum Cache Size"))
	int32 ManagedInstanceCacheMaximumSize = 20;
	
	/** Whether web server is started automatically. */
	UPROPERTY(config, EditAnywhere, Category = "Web Server")
	bool bAutoStartWebServer = true;

	/** The web remote control HTTP server's port. */
	UPROPERTY(config, EditAnywhere, Category = "Web Server")
	uint32 HttpServerPort = 10123;

	/** Default value of the synchronized events feature selection. */
	UE_API static const FName SynchronizedEventsFeatureSelection_Default;

private:
	UE_API static UAvaMediaSettings* GetSingletonInstance();

#if WITH_EDITOR
	/** Updates the command line preview to reflect the latest local playback server settings */
	void UpdateLocalPlaybackServerCommandLinePreview();
#endif

#if WITH_EDITORONLY_DATA
	/** Preview command line arguments from the playback server settings */
	UPROPERTY(Transient, VisibleAnywhere, Category = "Playback Server", meta=(MultiLine, DisplayAfter="LocalPlaybackServerSettings"))
	FString LocalPlaybackServerCommandLinePreview;
#endif
};

#undef UE_API
