// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaBroadcastProfile.h"
#include "AvaMediaDefines.h"
#include "Containers/Array.h"
#include "Containers/StringFwd.h"
#include "UObject/Object.h"
#include "AvaBroadcast.generated.h"

class FName;
class FDelegateHandle;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnAvaBroadcastChanged, EAvaBroadcastChange)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnAvaBroadcastChannelsListChanged, const FAvaBroadcastProfile& /*InProfile*/);

AVALANCHEMEDIA_API DECLARE_LOG_CATEGORY_EXTERN(LogAvaBroadcast, Log, All);

/**
 * Single Instance Class that manages all the Output Channels
 */
UCLASS(NotBlueprintable)
class AVALANCHEMEDIA_API UAvaBroadcast : public UObject
{
	GENERATED_BODY()
	
public:
	static UAvaBroadcast& Get();

	virtual void BeginDestroy() override;
	
	UFUNCTION(BlueprintCallable, Category = "Motion Design Broadcast")
	static UAvaBroadcast* GetBroadcast();
	
	UFUNCTION(BlueprintCallable, Category = "Motion Design Broadcast")
	void StartBroadcast();

	UFUNCTION(BlueprintCallable, Category = "Motion Design Broadcast")
	void StopBroadcast();
	
	UFUNCTION(BlueprintCallable, Category = "Motion Design Broadcast")
	bool IsBroadcastingAnyChannel() const;

	UFUNCTION(BlueprintCallable, Category = "Motion Design Broadcast")
	bool IsBroadcastingAllChannels() const;

	/**
	 * @brief Starts the channel if not started. Does nothing if already started.
	 * @param InChannelName Name of the channel to conditionally start.
	 * @return true if the channel is started (including if it was already started), false if the channel couldn't be started. 
	 */
	bool ConditionalStartBroadcastChannel(const FName& InChannelName);
	
	TArray<FName> GetProfileNames() const;
	
	const TMap<FName, FAvaBroadcastProfile>& GetProfiles() const;

	//Returns the actual name of the profile created. Defers in Number from InProfileName if it is an already existing Profile
	FName CreateProfile(FName InProfileName, bool bMakeCurrentProfile = true);

	bool DuplicateProfile(FName InNewProfile, FName InTemplateProfile, bool bMakeCurrentProfile = true);
	
	bool DuplicateCurrentProfile(FName InNewProfileName = NAME_None, bool bMakeCurrentProfile = true);
	
	bool RemoveProfile(FName InProfileName);

	bool CanRenameProfile(FName InProfileName, FName InNewProfileName, FText* OutErrorMessage = nullptr) const;
	
	bool RenameProfile(FName InProfileName, FName InNewProfileName);
	
	bool SetCurrentProfile(FName InProfileName);

	FName GetCurrentProfileName() const { return CurrentProfile; }

	FAvaBroadcastProfile& GetProfile(FName InProfileName);

	const FAvaBroadcastProfile& GetProfile(FName InProfileName) const;

	FAvaBroadcastProfile& GetCurrentProfile() { return GetProfile(CurrentProfile); }

	const FAvaBroadcastProfile& GetCurrentProfile() const { return GetProfile(CurrentProfile); }

	void LoadBroadcast();
	void SaveBroadcast();
	FString GetBroadcastSaveFilepath() const;

	//Rather than Immediately calling the Broadcast Event change, it will queue all the multiple calls and dispatch it once to avoid redundancy
	void QueueNotifyChange(EAvaBroadcastChange InChange);
	
	FDelegateHandle AddChangeListener(FOnAvaBroadcastChanged::FDelegate&& InDelegate);
	void RemoveChangeListener(FDelegateHandle InDelegateHandle);
	void RemoveChangeListener(FDelegateUserObjectConst InUserObject);

public:
	int32 GetChannelNameCount() const;

	/**
	 *	Returns the index of the channel in the channel name array.
	 *	Remark: don't use this index to lookup in a profile's channel array.
	 */
	int32 GetChannelIndex(FName InChannelName) const;
	
	FName GetChannelName(int32 ChannelIndex) const;
	FName GetOrAddChannelName(int32 ChannelIndex);

	/**
	 *	Add the given channel name to the array of channel names.
	 *	The channel name must be unique, if it already exists, the existing index is returned instead.
	 *	@return the index in the array where the name was added.
	 */
	int32 AddChannelName(FName InChannelName);
	
	void UpdateChannelNames();
	
	bool CanRenameChannel(FName InChannelName, FName InNewChannelName) const;
	bool CanRenameChannel(FName InChannelName, FName InNewChannelName, FText& OutErrorMessage) const;

	bool RenameChannel(FName InChannelName, FName InNewChannelName);
	bool RenameChannel(FName InChannelName, FName InNewChannelName, FText& OutErrorMessage);

	/**
	 * Sets the type (ex: Program or Preview) of the given channel across all profiles.
	 * Channel type for a given channel is the same in all profiles.
	 */
	void SetChannelType(FName InChannelName, EAvaBroadcastChannelType InChannelType);
	
	EAvaBroadcastChannelType GetChannelType(FName InChannelName) const;

	/**
	 * Pins the given channel from the given profile to persist across all the profiles.
	 */
	void PinChannel(FName InChannelName, FName InProfileName);

	/**
	 * Unpins the given channel (if it was pinned) so that is no longer persists across all profiles.
	 */
	void UnpinChannel(FName InChannelName);
	
	bool IsChannelPinned(FName InChannelName) const { return PinnedChannels.Contains(InChannelName); }

	/**
	 * If the given channel was pinned, returns the profile it was pinned in, returns Name_None otherwise.
	 */
	FName GetPinnedChannelProfileName(FName InChannelName) const;

	/**
	 * Rebuilds the channel lists that might include pinned channels from other profiles.
	 * This is necessary for functions such as GetChannels().
	 * @remark This relies on the channel names being updated (UpdateChannelNames) prior to this call.
	 */
	void RebuildProfiles();
	
	FOnAvaBroadcastChannelsListChanged& GetOnChannelsListChanged() { return OnChannelsListChanged; }

protected:
	// Todo: Proposal - make the events static and pass in the UAvaBroadcast that caused the event.
	// Reason: Attempt at gradually getting rid of the global UAvaBroadcast object. Channel events are also global.
	FOnAvaBroadcastChanged OnBroadcastChanged;
	FOnAvaBroadcastChannelsListChanged OnChannelsListChanged;
	
	EAvaBroadcastChange QueuedBroadcastChanges;
public:

#if WITH_EDITOR
	void SetCanShowPreview(bool bInCanShowPreview) { bCanShowPreview = bInCanShowPreview; }
	bool CanShowPreview() const { return bCanShowPreview; }
	virtual void PostEditUndo() override;
#endif
	
protected:

	FAvaBroadcastProfile& CreateProfileInternal(FName InProfileName);
	
	void EnsureValidCurrentProfile();
	void UpdateProfileNames();
	TArray<int32> BuildChannelIndices() const;

protected:
	UPROPERTY()
	FName CurrentProfile;

	UPROPERTY()
	TArray<FName> ChannelNames;
	
	UPROPERTY()
	TMap<FName, FAvaBroadcastProfile> Profiles;

	/**
	 * List of channels types.
	 * Backward compatibility: if not present in the map, defaults to "program" type.
	 */
	UPROPERTY()
	TMap<FName, EAvaBroadcastChannelType> ChannelTypes;

	/** Maps a channel name to a profile. Indicating which profile to use for a pinned channel. */
	UPROPERTY()
	TMap<FName, FName> PinnedChannels;

	/** Full path to the last loaded config file. */
	FString LoadedConfigFilepath;
	
#if WITH_EDITOR
	bool bCanShowPreview = true;
#endif
};
