// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaBroadcastDefines.generated.h"

UENUM()
enum class EAvaBroadcastAction
{
	None,
	Start,
	Stop,
	UpdateConfig,
	DeleteChannel
};
