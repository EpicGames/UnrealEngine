// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "AvaBroadcastLibrary.generated.h"

UCLASS(MinimalAPI, DisplayName = "Motion Design Broadcast Library", meta=(ScriptName = "MotionDesignBroadcastLibrary"))
class UAvaBroadcastLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Returns the current channel's viewport size. */
	UFUNCTION(BlueprintCallable, Category = "Motion Design|broadcast", meta=(WorldContext = "InWorldContextObject"))
	static AVALANCHEMEDIA_API FVector2D GetChannelViewportSize(const UObject* InWorldContextObject);

	/** Returns the current channel's name. Will return "None" if not running as part of the broadcast framework. */
	UFUNCTION(BlueprintCallable, Category = "Motion Design|broadcast", meta=(WorldContext = "InWorldContextObject"))
	static AVALANCHEMEDIA_API FName GetChannelName(const UObject* InWorldContextObject);

	/** Returns the channel status. */
	UFUNCTION(BlueprintCallable, Category = "Motion Design|broadcast")
	static AVALANCHEMEDIA_API EAvaBroadcastChannelState GetChannelStatus(const FName InChannelName);

	/** Returns the channel type. */
	UFUNCTION(BlueprintCallable, Category = "Motion Design|broadcast")
	static AVALANCHEMEDIA_API EAvaBroadcastChannelType GetChannelType(const FName InChannelName);
};
