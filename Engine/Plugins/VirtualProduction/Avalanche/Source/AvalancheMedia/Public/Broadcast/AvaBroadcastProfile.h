// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Broadcast/Channel/AvaBroadcastOutputChannel.h"
#include "AvaBroadcastProfile.generated.h"

class UAvaBroadcast;

USTRUCT()
struct AVALANCHEMEDIA_API FAvaBroadcastProfile
{
	GENERATED_BODY()

	friend UAvaBroadcast;
	
public:	
	static FAvaBroadcastProfile& GetNullProfile();
	FAvaBroadcastProfile() : FAvaBroadcastProfile(nullptr, NAME_None) {}
	FAvaBroadcastProfile(UAvaBroadcast* InBroadcast, FName InProfileName);

	void BeginDestroy();

	FName GetName() const { return ProfileName; }
	UAvaBroadcast& GetBroadcast() const;

	void SetProfileName(FName InProfileName);
	
	static void CopyProfiles(const FAvaBroadcastProfile& InSourceProfile, FAvaBroadcastProfile& OutTargetProfile);
	
	bool StartChannelBroadcast();	
	void StopChannelBroadcast();

	bool IsBroadcastingAnyChannel() const;
	bool IsBroadcastingAllChannels() const;
	
	bool IsValidProfile() const;
	
	void PostLoadProfile(bool bInIsProfileActive, UAvaBroadcast* InParentBroadcast);
	
	void UpdateChannels(bool bInIsProfileActive);

	/**
	 * Add a channel with the given name. If NAME_None, a default name is given to the channel.
	 */
	FAvaBroadcastOutputChannel& AddChannel(FName InChannelName = NAME_None);
	
	/**
	 * @brief Removes the channel with given name.
	 * @param InChannelName Name of the channel to remove.
	 * @return true if the channel was removed, false otherwise. 
	 */
	bool RemoveChannel(FName InChannelName);
	
	/**
	 * Finds the corresponding index of the channel with the given name in the current
	 * array of channels (including pinned channels) for this profile.
	 * @remark This index can be used to lookup directly in the profile's channel array.
	 * @return INDEX_NONE if not found.
	 */
	int32 GetChannelIndexInProfile(FName InChannelName) const;

	/**
	 * @brief Finds the channel (including pinned channels) with the given name.
	 * @remark	Pinned channels have priority, i.e. if the profile defines a local
	 *			channel of the given name, it will be overriden by the pinned channel. 
	 * @param InChannelName Name of the channel to find.
	 * @return reference to requested channel or invalid channel if not found.
	 */
	const FAvaBroadcastOutputChannel& GetChannel(FName InChannelName) const;

	/**
	 * @brief Finds the channel (including pinned channels) with the given name.
	 * @remark	Pinned channels have priority, i.e. if the profile defines a local
	 *			channel of the given name, it will be overriden by the pinned channel. 
	 * @param InChannelName Name of the channel to find.
	 * @return reference to requested channel or invalid channel if not found.
	 */
	FAvaBroadcastOutputChannel& GetChannelMutable(FName InChannelName);

	/**
	 * @brief Finds the existing channel with given name, or create it if missing.
	 * @param InChannelName Name of the channel to find or create.
	 * @return Reference to the requested or created channel.
	 */
	FAvaBroadcastOutputChannel& GetOrAddChannel(FName InChannelName);
	
	const TArray<FAvaBroadcastOutputChannel*>& GetChannels() const { return ResolvedChannels; }
	TArray<FAvaBroadcastOutputChannel*>& GetChannels() { return ResolvedChannels; }
	
	UMediaOutput* AddChannelMediaOutput(FName InChannelName, const UClass* InMediaOutputClass, const FAvaBroadcastMediaOutputInfo& InOutputInfo);
	int32 RemoveChannelMediaOutputs(FName InChannelName, const TArray<UMediaOutput*>& InMediaOutputs);

protected:
	/**
	 * Finds the corresponding index of the channel with the given name in the current
	 * array of local channels (not including pinned channels) for this profile.
	 * @remark This index can be used to lookup directly in the profile's channel array.
	 * @return INDEX_NONE if not found.
	 */
	int32 GetLocalChannelIndexInProfile(FName InChannelName) const;

	/**
	 * @brief Finds the local channel (not including pinned channels) with the given name.
	 * @param InChannelName Name of the channel to find.
	 * @return reference to requested channel or invalid channel if not found.
	 */
	const FAvaBroadcastOutputChannel& GetLocalChannel(FName InChannelName) const;

	/**
	 * @brief Finds the local channel (not including pinned channels) with the given name.
	 * @param InChannelName Name of the channel to find.
	 * @return reference to requested channel or invalid channel if not found.
	 */
	FAvaBroadcastOutputChannel& GetLocalChannelMutable(FName InChannelName);
	
	const TArray<FAvaBroadcastOutputChannel>& GetLocalChannels() const { return Channels; }
	TArray<FAvaBroadcastOutputChannel>& GetLocalChannels() { return Channels; }

	/** Resolve pinned channels. */
	void ResolveChannels();
	
protected:
	UPROPERTY(Transient)
	TWeakObjectPtr<UAvaBroadcast> ParentBroadcastWeak;

	UPROPERTY(Transient)
	FName ProfileName;
	
	UPROPERTY()
	TArray<FAvaBroadcastOutputChannel> Channels;

	/** This array contains all resolved channels, including pinned ones from other profiles. */
	TArray<FAvaBroadcastOutputChannel*> ResolvedChannels;
};

