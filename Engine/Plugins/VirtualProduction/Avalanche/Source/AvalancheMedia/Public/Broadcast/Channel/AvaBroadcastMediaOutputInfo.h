// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/Guid.h"
#include "UObject/NameTypes.h"
#include "AvaBroadcastMediaOutputInfo.generated.h"

/**
 *	Extra information about the Media Output object.
 *	This is used to determine the status of server hosting the device.
 */
USTRUCT()
struct AVALANCHEMEDIA_API FAvaBroadcastMediaOutputInfo
{
	GENERATED_BODY()
	
public:
	/**
	 * Unique identifier for this output.
	 * Allows easier management for client/server status and configuration replication.
	 */
	UPROPERTY()
	FGuid Guid;
	
	/**
	 * The server name if the media output was from a remote server.
	 * This will be empty if the device was local.
	 */
	UPROPERTY()
	FString ServerName;
	
	/**
	 * The device provider name, ex: BlackMagic, for this device (if any).
	 */
	UPROPERTY()
	FName DeviceProviderName;

	/**
	 *	Device name from the Device Provider.
	 *	For device that have no provider (like NDI for instance), this is
	 *	the name of the source or equivalent.
	 */
	UPROPERTY()
	FName DeviceName;
	
	bool IsValid() const
	{
		return !DeviceName.IsNone();
	}

	bool IsRemote() const { return IsRemote(ServerName);}

	void PostLoad();

	/** Determines if the given server name is considered "remote", i.e. from a different instance. */
	static bool IsRemote(const FString& InServerName);
};
