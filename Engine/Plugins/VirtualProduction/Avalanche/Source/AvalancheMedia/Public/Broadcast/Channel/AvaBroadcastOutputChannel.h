// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaBroadcastMediaOutputInfo.h"
#include "AvaMediaDefines.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Engine/TextureRenderTarget2D.h"
#include "UObject/ObjectPtr.h"
#include "Viewport/AvaViewportQualitySettings.h"
#include "AvaBroadcastOutputChannel.generated.h"

class FAudioDeviceHandle;
class FName;
class FWidgetRenderer;
class SAvaBroadcastPlaceholderWidget;
class SVirtualWindow;
class SWidget;
class UAvaBroadcast;
class UAvaPlayableGroup;
class UMediaCapture;
class UMediaOutput;
class UTextureRenderTarget2D;
class UUserWidget;
struct FAvaBroadcastProfile;
enum EPixelFormat : uint8;

/**
 *	Creates a bridge between the UMediaCapture's event
 *	and our events.
 */
USTRUCT()
struct AVALANCHEMEDIA_API FAvaBroadcastCapture
{
	GENERATED_BODY()
	
	UPROPERTY()
	TObjectPtr<UMediaCapture> MediaCapture = nullptr;

	UPROPERTY()
	TObjectPtr<UMediaOutput> MediaOutput = nullptr;

	// Keep track of the render target the MediaCapture is capturing.
	// We need to do this because the UMediaCapture doesn't have a public function to know
	// what RT it is capturing.
	TWeakObjectPtr<UTextureRenderTarget2D> RenderTarget;
	
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMediaCaptureStateChanged, const UMediaCapture* InMediaCapture, const UMediaOutput* InMediaOutput);
	FOnMediaCaptureStateChanged OnMediaCaptureStateChanged;

	FAvaBroadcastCapture() = default;
	~FAvaBroadcastCapture()
	{
		Reset();
	}
	
	void Reset(UMediaCapture* InMediaCapture = nullptr, UMediaOutput* InMediaOutput = nullptr, UTextureRenderTarget2D* InRenderTarget = nullptr);
	
private:
	/** Handler for MediaCapture's OnStateChangedNative event. */
	void OnStateChangedNative();
};

USTRUCT()
struct AVALANCHEMEDIA_API FAvaBroadcastOutputChannel
{
	GENERATED_BODY()

public:
	FAvaBroadcastOutputChannel() = default;
	FAvaBroadcastOutputChannel(ENoInit NoInit);

	UE_DEPRECATED(5.6, "Use constructor with profile name and broadcast.")
	explicit FAvaBroadcastOutputChannel(FAvaBroadcastProfile* InProfile);
	
	explicit FAvaBroadcastOutputChannel(FName InProfileName, UAvaBroadcast* InBroadcast);
	~FAvaBroadcastOutputChannel();

	void ReleasePlaceholderResources();
	void ReleasePlaceholderRenderTargets();
	void ReleaseOutputs();
	
	static FAvaBroadcastOutputChannel& GetNullChannel();
	
	static void DuplicateChannel(const FAvaBroadcastOutputChannel& InSourceChannel, FAvaBroadcastOutputChannel& OutTargetChannel);

	/**
	 * Set this channel's index in the broadcast's channel names array.
	 * Remark: don't use this index to lookup directly in the profile's channel array.
	 */
	void SetChannelIndex(int32 InIndex);

	/**
	 * Index of this channel in the broadcast's channel names array.
	 * Remark: don't use this index to lookup directly in the profile's channel array.
	 */
	int32 GetChannelIndex() const { return ChannelIndex; }
	
	FName GetChannelName() const;
	
	EAvaBroadcastChannelType GetChannelType() const;

	FName GetProfileName() const;

	FAvaBroadcastProfile& GetProfile() const;

	EAvaBroadcastChannelState RefreshState();
	
	EAvaBroadcastChannelState GetState() const { return ChannelState; }
	EAvaBroadcastIssueSeverity GetIssueSeverity() const { return ChannelIssueSeverity; }

	EAvaBroadcastIssueSeverity GetMediaOutputIssueSeverity(EAvaBroadcastOutputState InOutputState, const UMediaOutput* InMediaOutput) const;
	const TArray<FString>& GetMediaOutputIssueMessages(const UMediaOutput* InMediaOutput) const;
	EAvaBroadcastOutputState GetMediaOutputState(const UMediaOutput* InMediaOutput) const;
	const FAvaBroadcastMediaOutputInfo& GetMediaOutputInfo(const UMediaOutput* InMediaOutput) const;
	FAvaBroadcastMediaOutputInfo* GetMediaOutputInfoMutable(const UMediaOutput* InMediaOutput);
	
	bool IsEditingEnabled() const;
	
protected:
	void SetState(EAvaBroadcastChannelState InState, EAvaBroadcastIssueSeverity InIssueSeverity);

	void UpdatePlaceholder();
	void UpdatePlaceholderRenderTargets();
	void UpdateOverridePlaceholder();	
	void DrawPlaceholderWidget() const;
	
public:
	bool IsValidChannel() const;

	bool StartChannelBroadcast();
	void StopChannelBroadcast();
	
	UMediaOutput* AddMediaOutput(const UClass* InMediaOutputClass, const FAvaBroadcastMediaOutputInfo& InOutputInfo);
	void AddMediaOutput(UMediaOutput* InMediaOutput, const FAvaBroadcastMediaOutputInfo& InOutputInfo);
	int32 RemoveMediaOutput(UMediaOutput* InMediaOutput);

	/**
	 * Called when the output is modified.
	 * Propagates the configuration change accordingly.
	 */
	void OnMediaOutputModified(UMediaOutput* InMediaOutput);

	void SetProfileName(FName InProfileName);
	
	UE_DEPRECATED(5.6, "Use version with profile name and broadcast.")
	void PostLoadMediaOutputs(bool bInIsProfileActive, FAvaBroadcastProfile* InProfile);
	
	void PostLoadMediaOutputs(bool bInIsProfileActive, FName InProfileName, UAvaBroadcast* InBroadcast);
	
	void UpdateRenderTarget(UAvaPlayableGroup* InPlayableGroup, UTextureRenderTarget2D* InRenderTarget);
	void UpdateAudioDevice(const FAudioDeviceHandle& InAudioDeviceHandle);
	UAvaPlayableGroup* GetLastActivePlayableGroup() const;
	
	UTextureRenderTarget2D* GetCurrentRenderTarget(bool bInFallbackToPlaceholder) const;
	UTextureRenderTarget2D* GetPlaceholderRenderTarget() const;

	const TArray<UMediaOutput*>& GetMediaOutputs() const { return MediaOutputs; }

	/** Returns true if this channel has a least one local media output. */
	bool HasAnyLocalMediaOutputs() const;

	/** Returns true if this channel has a least one remote media output. */
	bool HasAnyRemoteMediaOutputs() const;
	
	/** Returns true if the Media Output object is associated to a remote server. */
	bool IsMediaOutputRemote(const UMediaOutput* InMediaOutput) const;

	/** Returns the server name associated with the Media Output object. */
	const FString& GetMediaOutputServerName(const UMediaOutput* InMediaOutput) const;
	
	/** Returns the Media Output objects that are associated to a remote server. */
	TArray<UMediaOutput*> GetRemoteMediaOutputs() const;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAvaChannelChanged, const FAvaBroadcastOutputChannel&, EAvaBroadcastChannelChange);
	static FOnAvaChannelChanged& GetOnChannelChanged() { return OnChannelChanged; }

	/**
	 * Called when the state of the capture changed.
	 * The callback is called on the game thread. The change may occur on the rendering thread.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FMediaOutputStateChanged, const FAvaBroadcastOutputChannel& InChannel, const UMediaOutput* InMediaOutput);
	static FMediaOutputStateChanged& GetOnMediaOutputStateChanged() { return OnMediaOutputStateChanged; }

protected:
	
	static FOnAvaChannelChanged OnChannelChanged;
	static FMediaOutputStateChanged OnMediaOutputStateChanged;

protected:

	bool IsMediaOutputCompatible(UMediaOutput* InMediaOutput, UTextureRenderTarget2D* InRenderTarget) const;
	void UpdateViewportTarget();

	UMediaCapture* GetMediaCaptureForOutput(const UMediaOutput* InMediaOutput) const;
	void StopCaptureForOutput(const UMediaOutput* InMediaOutput);

	void TickPlaceholder(float);
	void StartPlaceholderTick();
	void StopPlaceholderTick();

	void OnAvaMediaCaptureStateChanged(const UMediaCapture* InMediaCapture, const UMediaOutput* InMediaOutput);
	
public:
	
	/**
	* Updates Channel's Data and Resources.
	* The resources are going to be allocated if the profile is active or released if not.
	* This will also update the render targets according to the current configuration.
	* @param bInIsProfileActive Indicate if the parent profile is active.
	*/
	void UpdateChannelResources(bool bInIsProfileActive);

	FIntPoint DetermineRenderTargetSize() const;
	EPixelFormat DetermineRenderTargetFormat() const;

	static FIntPoint GetDefaultMediaOutputSize(EAvaBroadcastChannelType InChannelType);
	static EPixelFormat GetDefaultMediaOutputFormat();

	const FAvaViewportQualitySettings& GetViewportQualitySettings() const { return QualitySettings; }
	void SetViewportQualitySettings(const FAvaViewportQualitySettings& InQualitySettings);

protected:

	/** Index of this channel in the broadcast's channel names array. */
	UPROPERTY()
	int32 ChannelIndex = INDEX_NONE;
	
	UPROPERTY()
	TArray<TObjectPtr<UMediaOutput>> MediaOutputs;

	UPROPERTY()
	TArray<FAvaBroadcastMediaOutputInfo> MediaOutputInfos;

	UPROPERTY(EditAnywhere, Category = "Motion Design", meta = (HideHeader, ShowPresets))
	FAvaViewportQualitySettings QualitySettings;

	/*
	 * A Map of Media Outputs with their Media Captures when Broadcast Starts, and deleted when Broadcast Stops.
	 * Only succeeded Captures are added. 
	 */
	UPROPERTY(Transient)
	TMap<TObjectPtr<UMediaOutput>, FAvaBroadcastCapture> MediaCaptures;

	/*
	* Optional UMG Widget that is used to replace the Default Placeholder Widget
	*/
	UPROPERTY(Transient)
	TObjectPtr<UUserWidget> OverridePlaceholderUserWidget;

	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> PlaceholderRenderTarget;

	// Extra render target to perform alpha inversion needed to mimic scene rendering.
	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> PlaceholderRenderTargetTmp;

	/** Profile name owning this channel. */
	UPROPERTY(Transient)
	FName ProfileName;

	/** Broadcast object owning this channel. */
	UPROPERTY(Transient)
	TWeakObjectPtr<UAvaBroadcast> ParentBroadcastWeak;

	/**
	 * Last Active Playable group associated with the current render target.
	 * In the current design, there can be only one playable group associated
	 * to the currently active render target for a channel.
	 */
	TWeakObjectPtr<UAvaPlayableGroup> LastActivePlayableGroup;

	/**
	 * Keep track of the active render target associated with the current playable group.
	 * This is most likely the internal channel's "placeholder" render target, but
	 * it could also be one owned by the given playable group.
	 */
	TWeakObjectPtr<UTextureRenderTarget2D> CurrentRenderTarget;
	
	TSharedPtr<SAvaBroadcastPlaceholderWidget> PlaceholderWidget;

	TSharedPtr<SWidget> OverridePlaceholderWidget;
	
	FDelegateHandle PlaceholderTickHandle;
	
	TSharedPtr<FWidgetRenderer> WidgetRenderer;

	FText ChannelNameText;
	
	EAvaBroadcastChannelState ChannelState = EAvaBroadcastChannelState::Idle;	
	EAvaBroadcastIssueSeverity ChannelIssueSeverity = EAvaBroadcastIssueSeverity::None;

	// Keep track of the media capture/output status while
	// starting the broadcast.
	struct FLocalMediaOutputStatus
	{
		TArray<FString> Messages;
		EAvaBroadcastIssueSeverity Severity = EAvaBroadcastIssueSeverity::None;
	};
	mutable TMap<const UMediaOutput*, FLocalMediaOutputStatus> LocalMediaOutputStatuses;

	FString& EmplaceMessageForOutput(const UMediaOutput* InMediaOutput, EAvaBroadcastIssueSeverity InSeverity) const;
	EAvaBroadcastIssueSeverity GetLocalSeverityForOutput(const UMediaOutput* InMediaOutput) const;
	void ResetLocalMediaOutputStatus(const UMediaOutput* InMediaOutput);
};
