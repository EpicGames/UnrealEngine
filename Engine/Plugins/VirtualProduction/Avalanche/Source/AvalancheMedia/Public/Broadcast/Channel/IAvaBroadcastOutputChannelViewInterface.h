// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IAvaBroadcastOutputChannelViewInterface.generated.h"

UINTERFACE(MinimalAPI, Blueprintable)
class UAvaBroadcastOutputChannelViewInterface : public UInterface
{
	GENERATED_BODY()
};

class IAvaBroadcastOutputChannelViewInterface
{
	GENERATED_BODY()
	
public:
	
	UFUNCTION(BlueprintNativeEvent, Category = "Motion Design Output Channel")
	void SetChannelName(const FText& InChannelName);
};
