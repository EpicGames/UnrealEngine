// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"

enum EPixelFormat : uint8;
struct FSoftObjectPath;

/**
 * Interface for the broadcast settings.
 * Broadcast settings can either be local to the client, or if running
 * on a playback server, they may be replicated from a connected remote client.
 */
class IAvaBroadcastSettings
{
public:
	virtual const FLinearColor& GetChannelClearColor() const = 0;
	virtual EPixelFormat GetDefaultPixelFormat() const = 0;
	virtual const FIntPoint& GetDefaultResolution() const = 0;
	virtual bool IsDrawPlaceholderWidget() const = 0;
	virtual const FSoftObjectPath& GetPlaceholderWidgetClass() const = 0;

protected:
	virtual ~IAvaBroadcastSettings() = default;
};
