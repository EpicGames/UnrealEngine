// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaIOCoreDefinitions.h"
#include "MediaOutput.h"
#include "AvaBroadcastDisplayMediaOutput.generated.h"

/**
 * Output Media to a display adapter.
 */
UCLASS(MinimalAPI, ClassGroup = "Motion Design Broadcast",
	meta = (DisplayName = "Motion Design Broadcast Display Media Output", MediaIOCustomLayout = "AvaDisplay"))
class UAvaBroadcastDisplayMediaOutput : public UMediaOutput
{
	GENERATED_BODY()

public:
	UAvaBroadcastDisplayMediaOutput(const FObjectInitializer& ObjectInitializer);

	/** The device, port and video settings that correspond to the output. */
	UPROPERTY(EditAnywhere, Category = "AvaDisplay", meta = (DisplayName = "Configuration"))
	FMediaIOOutputConfiguration OutputConfiguration;

	//~ UMediaOutput interface
public:
	virtual bool Validate(FString& FailureReason) const override;
	virtual FIntPoint GetRequestedSize() const override;
	virtual EPixelFormat GetRequestedPixelFormat() const override;
	virtual EMediaCaptureConversionOperation GetConversionOperation(EMediaCaptureSourceType InSourceType) const override;

protected:
	virtual UMediaCapture* CreateMediaCaptureImpl() override;
	//~ End UMediaOutput interface
};
