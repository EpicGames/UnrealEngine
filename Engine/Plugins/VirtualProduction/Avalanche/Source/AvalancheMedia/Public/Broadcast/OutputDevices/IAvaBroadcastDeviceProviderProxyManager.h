// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FAvaBroadcastDeviceProviderWrapper;
class IMediaIOCoreDeviceProvider;
struct FAvaBroadcastDeviceProviderDataList;

class AVALANCHEMEDIA_API IAvaBroadcastDeviceProviderProxyManager
{
public:
	virtual ~IAvaBroadcastDeviceProviderProxyManager() = default;
	
	virtual const FAvaBroadcastDeviceProviderWrapper* GetDeviceProviderWrapper(FName InDeviceProviderName) const = 0;

	/** Collects all the device providers related to the given server. */
	virtual TArray<const IMediaIOCoreDeviceProvider*> GetDeviceProvidersForServer(const FString& InServerName) const = 0;

	/** Finds the server name of the given device. */
	virtual FString FindServerNameForDevice(const FName& InDeviceProviderName, const FName& InDeviceName) const = 0;

	/** Returns true if the device can be found in a local provider. */
	virtual bool IsLocalDevice(const FName& InDeviceProviderName, const FName& InDeviceName) const = 0;

	/** Install the given server's proxy data, installing wrappers as necessary. */
	virtual void Install(const FString& InServerName, const FAvaBroadcastDeviceProviderDataList& InDeviceProviderDataList) = 0;

	/** Remove the given server from all the wrappers. */
	virtual void Uninstall(const FString& InServerName) = 0;

	/** Returns the set of all server names. */
	virtual TSet<FString> GetServerNames() const = 0;

	/** Returns device provider data list (all device providers) for a specific server. */
	virtual const TSharedPtr<const FAvaBroadcastDeviceProviderDataList> GetDeviceProviderDataListForServer(const FString& InServerName) const = 0;

	/** Returns the default local server name. */
	virtual FString GetLocalServerName() const = 0;
};
