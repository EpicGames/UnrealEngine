// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAvaMedia, Log, All);

class FAvaPlaybackManager;
class FAvaRundownManagedInstanceCache;
class FCommonViewportClient;
class FName;
class IAvaBroadcastDeviceProviderProxyManager;
class IAvaBroadcastSettings;
class IAvaMediaSyncProvider;
class IAvaPlayableRemoteControlPresetInfoCache;
class IAvaPlaybackClient;
class IAvaPlaybackServer;
class IAvaRundownServer;
class IMediaIOCoreDeviceProvider;
class UWorld;
struct FAvaInstanceSettings;
struct FAvaPlayableSettings;
struct FMediaIOOutputConfiguration;

/** Maps one to one with the editor's map changed type (for now). */
enum class EAvaMediaMapChangeType : uint8
{
	None,
	LoadMap,
	SaveMap,
	NewMap,
	TearDownWorld
};

class IAvaMediaModule : public IModuleInterface
{
public:
	static bool IsModuleLoaded()
	{
		static const FName ModuleName = TEXT("AvalancheMedia");
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}
	
	static IAvaMediaModule& Get()
	{
		static const FName ModuleName = TEXT("AvalancheMedia");
		return FModuleManager::LoadModuleChecked<IAvaMediaModule>(ModuleName);
	};

	/**
	 * @brief Returns true if the playback client is started.
	 */
	virtual bool IsPlaybackClientStarted() const = 0;

	/**
	 * @brief Starts the playback client (if not already started).
	 * @remark In editor mode, this will stop the playback server. 
	 */
	virtual void StartPlaybackClient() = 0;

	/**
	 * @brief Stops the playback client.
	 */
	virtual void StopPlaybackClient() = 0;

	/**
	 * @brief Returns true if the playback server is started.
	 */
	virtual bool IsPlaybackServerStarted() const = 0;

	/**
	 * @brief Starts the playback server (if not already started).
	 * @param InPlaybackServerName Optional server name. If empty, the host (computer) name will be used.
	 */
	virtual void StartPlaybackServer(const FString& InPlaybackServerName) = 0;

	/**
	 * @brief Stops the playback server.
	 */
	virtual void StopPlaybackServer() = 0;

	virtual IAvaPlaybackClient& GetPlaybackClient() = 0;
	virtual IAvaPlaybackServer* GetPlaybackServer() const = 0;
	virtual const IMediaIOCoreDeviceProvider* GetDeviceProvider(FName InProviderName, const FMediaIOOutputConfiguration* InMediaIOOutputConfiguration) const = 0;
	virtual TArray<const IMediaIOCoreDeviceProvider*> GetDeviceProvidersForServer(const FString& InServerName) const = 0;
	virtual FString GetServerNameForDevice(const FName& InDeviceProviderName, const FName& InDeviceName) const = 0;
	virtual bool IsLocalDevice(const FName& InDeviceProviderName, const FName& InDeviceName) const = 0;

	/**
	 * @brief Launches a separate process in game mode to run a local playback server.
	 */
	virtual void LaunchGameModeLocalPlaybackServer() = 0;

	/**
	 * @brief Stops currently running game mode local playback server.
	 */
	virtual void StopGameModeLocalPlaybackServer() = 0;

	/**
	 * @brief Returns true if the game mode local playback server process is launched. 
	 */
	virtual bool IsGameModeLocalPlaybackServerLaunched() const = 0;

	/**
	 * Access the global broadcast settings.
	 */
	virtual const IAvaBroadcastSettings& GetBroadcastSettings() const = 0;

	/**
	 * Access the global Motion Design instance settings.
	 * 
	 * Remark: lifetime of the returned reference is not guaranteed beyond the current call context.
	 * If the settings are replicated from a client, it could get deleted if the client disconnects.
	 * If the use of the settings is deferred, the caller must make a local copy of the settings or
	 * call GetAvaInstanceSettings() in the deferred call instead.
	 */
	virtual const FAvaInstanceSettings& GetAvaInstanceSettings() const = 0;

	/**
	 * Access global Playable Settings.
	 * @remark These settings a replicated from connected the playback client (if connected).
	 */
	virtual const FAvaPlayableSettings& GetPlayableSettings() const = 0;
	
	/**
	 *	Returns true if the local playback manager is (still) available.
	 */
	virtual bool IsLocalPlaybackManagerAvailable() const = 0;

	/**
	 *	This is the backend for playing Motion Design assets locally.
	 */
	virtual FAvaPlaybackManager& GetLocalPlaybackManager() const = 0;

	/**
	 * Returns true if the managed instance cache is (still) available.
	 */
	virtual bool IsManagedInstanceCacheAvailable() const = 0;
	
	/**
	 *	Access the "managed" Motion Design Asset Instance cache.
	 */
	virtual FAvaRundownManagedInstanceCache& GetManagedInstanceCache() const = 0;

	/**
	 * Returns true if the AvaMediaSyncProvider modular feature is available.
	 */
	virtual bool IsAvaMediaSyncProviderFeatureAvailable() const = 0;
	
	/**
	 * Access the currently used Ava Media Sync Provider.
	 */
	virtual IAvaMediaSyncProvider* GetAvaMediaSyncProvider() const = 0;

	/**
	 * @brief Propagate a map changed event (from the level editor).
	 */
	virtual void NotifyMapChangedEvent(UWorld* InWorld, EAvaMediaMapChangeType InEventType) = 0;

	/**
	* Delegate called when a new IAvaMediaSyncProvider modular feature is used.
	*/
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAvaMediaSyncProviderChanged, IAvaMediaSyncProvider* /*InNewAvaSyncProvider*/);
	virtual FOnAvaMediaSyncProviderChanged& GetOnAvaMediaSyncProviderChanged() = 0;
	
	/**
	* Delegate called when a package has been touched by a sync operation from the given sync provider.
	*/
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAvaMediaSyncPackageModified, IAvaMediaSyncProvider* /*InAvaSyncProvider*/, const FName& /*InPackageName*/);
	virtual FOnAvaMediaSyncPackageModified& GetOnAvaMediaSyncPackageModified() = 0;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMapChangedEvent, UWorld* /*InWorld*/, EAvaMediaMapChangeType /*InEventType*/);
	virtual FOnMapChangedEvent& GetOnMapChangedEvent() = 0;
	
	DECLARE_MULTICAST_DELEGATE(FOnAvaPlaybackClientStarted);
	virtual FOnAvaPlaybackClientStarted& GetOnAvaPlaybackClientStarted() = 0;

	DECLARE_MULTICAST_DELEGATE(FOnAvaPlaybackClientStopped);
	virtual FOnAvaPlaybackClientStopped& GetAvaPlaybackClientStopped() = 0;

	DECLARE_MULTICAST_DELEGATE(FOnAvaPlaybackServerStarted);
	virtual FOnAvaPlaybackServerStarted& GetOnAvaPlaybackServerStarted() = 0;

	DECLARE_MULTICAST_DELEGATE(FOnAvaPlaybackServerStopped);
	virtual FOnAvaPlaybackServerStopped& GetAvaPlaybackServerStopped() = 0;
	
	/** Use to query the current editor viewport from the corresponding editor module. */
	DECLARE_DELEGATE_OneParam(FGetEditorViewportClient, FCommonViewportClient** );
	virtual FGetEditorViewportClient& GetEditorViewportClientDelegate() = 0;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRundownServerEvent, TSharedPtr<IAvaRundownServer>);
	virtual FOnRundownServerEvent& GetOnRundownServerStarted() = 0;
	virtual FOnRundownServerEvent& GetOnRundownServerStopping() = 0;

	/**
	 * @brief Returns true if the rundown server is started.
	 */
	virtual bool IsRundownServerStarted() const = 0;

	/**
	 * @brief Starts the rundown server (if not already started).
	 * @param InServerName Optional server name. If empty, the host (computer) name will be used.
	 */
	virtual void StartRundownServer(const FString& InServerName) = 0;

	/**
	 * @brief Stops the rundown server.
	 */
	virtual void StopRundownServer() = 0;
	
	/**
	 * @brief Returns currently running rundown server. 
	 */
	virtual TSharedPtr<IAvaRundownServer> GetRundownServer() const = 0;

	/**
	 * Creates a rundown server that is not managed by the module.
	 * @param InServerName Optional server name. if empty, the host name will be used.d
	 * @return Created server.
	 * @remark For internal use only (testing). Detached servers will interfere with the managed one. 
	 */
	virtual TSharedPtr<IAvaRundownServer> MakeDetachedRundownServer(const FString& InServerName) = 0;
	
	/**
	 * Access the device provider proxy manager.
	 */
	virtual IAvaBroadcastDeviceProviderProxyManager& GetDeviceProviderProxyManager() = 0;

	/** Access the global remote control preset info cache. */
	virtual IAvaPlayableRemoteControlPresetInfoCache& GetPlayableRemoteControlPresetInfoCache() const = 0;
};
