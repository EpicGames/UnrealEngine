// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaSyncProviderFeatureTypes.generated.h"

UENUM()
enum class EAvaMediaSyncResponseResult : uint8
{
	/** Indicates an Error happened */
	Error,

	/** Indicates successful completion */
	Success,

	/** Should not happen */
	Unknown
};

/** Engine type values for a sync connected device */
UENUM()
enum class EAvaMediaSyncEngineType : uint8
{
	Server,
	Commandlet,
	Editor,
	Game,
	Other,
	Unknown
};

/** Holds data about local project (such as Message Address Ids, Project Name, Hostname, InstanceId, etc.) */
USTRUCT()
struct AVALANCHEMEDIA_API FAvaMediaSyncConnectionInfo
{
	GENERATED_BODY()

	/** Holds the engine version checksum */
	int32 EngineVersion = 0;

	/** Holds the instance identifier */
	FGuid InstanceId;

	/** Holds the type of the engine instance */
	EAvaMediaSyncEngineType InstanceType = EAvaMediaSyncEngineType::Unknown;

	/** Holds the identifier of the session that the application belongs to */
	FGuid SessionId;
	
	/** The hostname this message was generated from */
	FString HostName;

	/** The unreal project name this message was generated from */
	FString ProjectName;
	
	/** The unreal project directory this message was generated from */
	FString ProjectDir;

	/** Default constructor */
	FAvaMediaSyncConnectionInfo();

	/** Returns debug string for this message */
	FString ToString() const;

	/** Internal helper to return the last portion of a path, similar to the Unix basename command. Trailing directory separators are ignored */
	static FString GetBasename(const FString& InPath);

	/** Returns user friendly instance type for display */
	FText GetHumanReadableInstanceType() const;
};

/** Base sync response with Status and Error information */
USTRUCT()
struct FAvaMediaSyncResponse
{
	GENERATED_BODY()

	/** Status of the response, error or success */
	EAvaMediaSyncResponseResult Status = EAvaMediaSyncResponseResult::Unknown;

	/** In case of error, a localized text describing what went wrong */
	FText ErrorText;

	/** In case of success, an optional localized text describing status of the operation */
	FText StatusText;
	
	/** Holds data about local project and send to remote for it to display as result status */
	FAvaMediaSyncConnectionInfo ConnectionInfo;

	/** Default constructor */
	FAvaMediaSyncResponse() = default;

	/** Returns whether Status is in Success state */
	bool IsValid() const
	{
		return Status == EAvaMediaSyncResponseResult::Success;
	}

	/** Returns whether Status is in Error State and ErrorText is filled with error information */
	bool HasError() const
	{
		return Status == EAvaMediaSyncResponseResult::Error && !ErrorText.IsEmpty();
	}

	/** Returns string representation */
	FString ToString() const
	{
		return FString::Printf(
			TEXT("Status: %s, ErrorText: %s, StatusText: %s (ConnectionInfo: %s)"),
			*UEnum::GetValueAsString(Status),
			*ErrorText.ToString(),
			*StatusText.ToString(),
			*ConnectionInfo.ToString()
		);
	}
};

/** Payload for a comparison request */
USTRUCT()
struct FAvaMediaSyncCompareResponse : public FAvaMediaSyncResponse
{
	GENERATED_BODY()

	/** Whether the two remotes have differing state */
	bool bNeedsSynchronization = false;

	/** Default constructor */
	FAvaMediaSyncCompareResponse() = default;

	/** Returns string representation */
	FString ToString() const
	{
		const FString SyncResponseText = Super::ToString();
		return FString::Printf(
			TEXT("%s, bNeedsSynchronization: %s"),
			*SyncResponseText,
			bNeedsSynchronization ? TEXT("true") : TEXT("false")
		);
	}
};
