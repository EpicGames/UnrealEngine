// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Templates/Function.h"

/**
 * @brief Base class for synchronized events
 */
struct FAvaMediaSynchronizedEvent
{
	FString Signature;
	TUniqueFunction<void()> Function;

	FAvaMediaSynchronizedEvent(const FString& InSignature, TUniqueFunction<void()> InFunction)
		: Signature(InSignature)
		, Function(MoveTemp(InFunction))
	{
	}
	
	FAvaMediaSynchronizedEvent(FString&& InSignature, TUniqueFunction<void()> InFunction)
		: Signature(MoveTemp(InSignature))
		, Function(MoveTemp(InFunction))
	{
	}

	virtual ~FAvaMediaSynchronizedEvent() = default;
};
