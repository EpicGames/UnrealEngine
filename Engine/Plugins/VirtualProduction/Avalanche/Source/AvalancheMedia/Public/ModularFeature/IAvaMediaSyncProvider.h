// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaSyncProviderFeatureTypes.h"
#include "Features/IModularFeature.h"

/** Generic delegate response handler */
DECLARE_DELEGATE_OneParam(FOnAvaMediaSyncResponse, const TSharedPtr<FAvaMediaSyncResponse>&);

/** Delegate with a comparison response. Payload is a FAvaMediaSyncCompareResponse shared ptr */
DECLARE_DELEGATE_OneParam(FOnAvaMediaSyncCompareResponse, const TSharedPtr<FAvaMediaSyncCompareResponse>&);

/** Delegate called when a package is modified by a sync operation, either added or modified. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnAvaMediaSyncPackageModified, const FName& /*PackageName*/);

/**
 * Base Sync feature interface
 *
 * TODO:
 * - Proper completion delegates handling for push and pull
 * - Handle errors and respond with error state (with error case either from local or remote)
 */
class AVALANCHEMEDIA_API IAvaMediaSyncProvider : public IModularFeature
{
public:
	virtual ~IAvaMediaSyncProvider() = default;

	static FName GetModularFeatureName()
	{
		static const FName FeatureName = TEXT("MotionDesign_Feature_Sync");
		return FeatureName;
	}

	/** Returns the first registered implementation based on GetModularFeatureName(), if there is one available */
    static IAvaMediaSyncProvider* Get();

	/** Returns underlying feature implementation name */
	virtual FName GetName() const = 0;

	/**
	 * Broadcasts a "sync" request to all connected remotes on the local network.
	 * 
	 * @param InPackageNames The list of "top level" package names (without inner dependencies / referencers)
	 *	to base the synchronization on
	 */
	virtual void SyncToAll(const TArray<FName>& InPackageNames) = 0;

	/**
	 * Sends a "push" request to a specific remote (either a playback server or client name)
	 * 
	 * The remote name can be either a playback server name or a playback client name, depending on where the request is being triggered.
	 *
	 * @param InRemoteName The playback client or server name to push to
	 * @param InPackageNames The list of "top level" package names (without inner dependencies / referencers) to send on remote
	 * @param DoneDelegate A completion delegate called when receiving a response with a `FOnAvaMediaSyncResponse` payload
	 */
	virtual void PushToRemote(const FString& InRemoteName, const TArray<FName>& InPackageNames, const FOnAvaMediaSyncResponse& DoneDelegate = FOnAvaMediaSyncResponse()) = 0;

	/**
	 * Sends a "pull" request to a specific remote (either a playback server or client name)
	 * 
	 * The remote name can be either a playback server name or a playback client name, depending on where the request is being triggered.
	 *
	 * @param InRemoteName The playback client or server name to push to
	 * @param InPackageNames The list of "top level" package names (without inner dependencies / referencers) to send on remote
	 * @param DoneDelegate A completion delegate called when receiving a response with a `FOnAvaMediaSyncResponse` payload
	 */
	virtual void PullFromRemote(const FString& InRemoteName, const TArray<FName>& InPackageNames, const FOnAvaMediaSyncResponse& DoneDelegate = FOnAvaMediaSyncResponse()) = 0;

	/**
	 * Issue a "compare" request to a given remote, and calls back the completion delegate with status payload.
	 *
	 * The remote name can be either a playback server name or a playback client name, depending on where the request is being triggered, eg.
	 *
	 * - When local editor instance is a playback client, remote name is considered to be a server name.
	 * - When local editor instance is a playback server, remote name is considered to be a client name.
	 * - If local editor instance if both a playback client and server, remote name is considered to be a server name.
	 *
	 * @param InRemoteName The playback client or server name to compare with
	 * @param InPackageNames The list of "top level" package names (without inner dependencies / referencers) to base the comparison on
	 * @param DoneDelegate A completion delegate called when receiving a response with a `FAvaMediaSyncCompareResponse` payload
	 */
	virtual void CompareWithRemote(const FString& InRemoteName, const TArray<FName>& InPackageNames, const FOnAvaMediaSyncCompareResponse& DoneDelegate = FOnAvaMediaSyncCompareResponse()) = 0;

	/** Delegate called when a package is modified by a sync write operation, either being added or modified. */
	virtual FOnAvaMediaSyncPackageModified& GetOnAvaSyncPackageModified() = 0;
};
