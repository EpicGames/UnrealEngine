// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Templates/Function.h"

enum class EAvaMediaSynchronizedEventState : uint8
{
	/**
	 * The requested event signature is not tracked by the dispatcher.
	 */
	NotFound,
	/**
	 * Has been seen on other nodes, but has not been pushed locally yet.
	 */
	Tracked,
	/**
	 * Has been pushed locally and is waiting on other nodes. 
	 */
	Pending,
	/**
	 * Has been marked from all node and will be invoke on next dispatch call.
	 * This state can only been seen on "late" dispatch implementation. Ready events
	 * should be dispatched asap and not linger in the ready queue.
	 */
	Ready
};

/**
 * Queue and Dispatcher for Synchronized events.
 */
class IAvaMediaSynchronizedEventDispatcher
{
public:
	/**
	 * Push a new punctual event in the queue.
	 * @param InEventSignature unique signature for the event.
	 * @param InFunction function to invoke when the event is signaled on all nodes.
	 */
	virtual bool PushEvent(FString&& InEventSignature, TUniqueFunction<void()> InFunction) = 0;

	/**
	 * Retrieve the given event state.
	 */
	virtual EAvaMediaSynchronizedEventState GetEventState(const FString& InEventSignature) const = 0;
	
	/**
	 * Let the implementation update and dispatch the queued events. 
	 */
	virtual void DispatchEvents() = 0;
	
	virtual ~IAvaMediaSynchronizedEventDispatcher() = default;
};
