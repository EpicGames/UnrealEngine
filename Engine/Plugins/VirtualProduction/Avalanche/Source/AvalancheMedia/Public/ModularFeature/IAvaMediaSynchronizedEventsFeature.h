// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class IAvaMediaSynchronizedEventDispatcher;

/**
 * Interface for synchronized events implementation in a distributed environment such as
 * the Display Cluster or forked channels.
 * 
 * For a given event signature across a distributed environment, i.e. the event itself
 * is distributed (one instance per node), the event will be fired when all nodes raise the
 * signal for that event. In other word, the event is invoked at the same time on all nodes only when
 * it has been raised on all nodes.
 */
class IAvaMediaSynchronizedEventsFeature : public IModularFeature
{
public:
	static FName GetModularFeatureName()
	{
		static const FName FeatureName = TEXT("AvaMediaSynchonizedEventsFeature");
		return FeatureName;
	}
	
	/** Returns underlying feature implementation name */
	virtual FName GetName() const = 0;

	/** Returns the localized feature implementation name for display. */ 
	virtual FText GetDisplayName() const = 0;

	/** Returns the feature implementation description. */ 
	virtual FText GetDisplayDescription() const = 0;

	/**
	 * Returns the feature implementation priority for automatic selection of most appropriate default.
	 * Implementations with higher priority will take precedence over lower ones.
	 */
	virtual int32 GetPriority() const = 0;

	/** Returns a default implementation priority. */
	static int32 GetDefaultPriority()
	{
		// This needs to be greater than 0 to allow factories to have both higher and lower priority than the default.
		constexpr int32 DefaultImportPriority = 100;
		return DefaultImportPriority;
	}
	
	/**
	 * Factory method to create an event dispatcher.
	 * 
	 * @param InSignature signature given to the dispatcher. This should match on all clustered/forked channel.
	 * @remark Current implementation does not use the signature to automatically add scope to all pushed events.
	 * @todo Investigate the convenience of using dispatcher signature to automatically add scope to all pushed events.
	 */
	virtual TSharedPtr<IAvaMediaSynchronizedEventDispatcher> CreateDispatcher(const FString& InSignature) = 0;

protected:
	virtual ~IAvaMediaSynchronizedEventsFeature() = default;
};