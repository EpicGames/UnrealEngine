// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "AvaPlayableGroup.h"
#include "AvaPlayableRemoteControlValues.h"
#include "Playback/Nodes/Events/Actions/AvaPlaybackAnimations.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"

#include "AvaPlayable.generated.h"

class FSceneView;
class FSceneViewFamily;
class IAvaSceneInterface;
class UAvaPlayableGroup;
class UAvaPlayableGroupManager;
class UAvaPlayableTransition;
class UAvaSequence;
class UAvaSequencePlayer;
class URemoteControlPreset;
struct FAvaInstancePlaySettings;
struct FAvaPlayableRemoteControlValues;
struct FAvaSoftAssetPtr;

AVALANCHEMEDIA_API DECLARE_LOG_CATEGORY_EXTERN(LogAvaPlayable, Log, All);

/** Option flags for the EndPlay function. */
UENUM()
enum class EAvaPlayableEndPlayOptions : uint8
{
	None = 0,
	/** End play world if no more assets are playing. */
	ConditionalEndPlayWorld = 1 << 0,
	/** Perform the request immediately instead of waiting on the next tick. */
	ForceImmediate = 1 << 1,
};
ENUM_CLASS_FLAGS(EAvaPlayableEndPlayOptions);

enum class EAvaPlayableCommandResult : uint8
{
	/** The command was executed successfully. */
	Executed = 0,
	/** The command failed and should be discarded. */
	ErrorDiscard = 1,
	/** The command couldn't be executed but should be kept and attempted again. */
	KeepPending = 2
};

/*
 * Base class for a Motion Design playable.
 *
 * A playable (a.k.a. graphic or page) is the basic element that can be rendered
 * and controlled through the animations and remote control.
 * 
 * Design goal:
 *
 * The design goal is to abstract the implementation of a playable.
 * So far we have one concrete implementation:
 * - Level Streaming that can be streamed with other levels in the same game instance.
 *
 * To support multiple playable in the same channel/output, there are 2 ways:
 * - rendering in the same world which can only be done with the level streaming playables.
 * - compositing different renders. This is not yet supported, but the playable abstraction should help.
 *
 * Distributed rendering is implemented using playable and playable group remote proxies.
 */
UCLASS(Abstract, NotBlueprintable, BlueprintType, ClassGroup = "Motion Design Playable",
	meta = (DisplayName = "Motion Design Playable"))	// Todo: review class specifier
class AVALANCHEMEDIA_API UAvaPlayable : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Playable creation information contains the necessary information to
	 * create an instance of a playable and it's playable group.
	 */
	struct FPlayableCreationInfo
	{
		/** Container for shared playable groups. */
		UAvaPlayableGroupManager* PlayableGroupManager = nullptr;
		/** Information about the asset type and path. */
		FAvaSoftAssetPtr SourceAsset;
		/** Channel name this playable will be instanced in. */
		FName ChannelName;
		/** Provided Playable Group, if not provided, the channel name will be used. */
		UAvaPlayableGroup* PlayableGroup = nullptr;
	};
	
	/**
	 * Factory method for the playables.
	 * This will create the appropriate playable object and initialize it (calling InitPlayable).
	 * This will setup the playable group appropriately, but will not load the asset.
	 */
	static UAvaPlayable* Create(UObject* InOuter, const FPlayableCreationInfo& InPlayableInfo);

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnSequenceEvent, UAvaPlayable*, FName /*InSequenceLabel*/, EAvaPlayableSequenceEventType);
	static FOnSequenceEvent& OnSequenceEvent() { return OnSequenceEventDelegate; }

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnTransitionEvent, UAvaPlayable*, UAvaPlayableTransition*, EAvaPlayableTransitionEventFlags);
	static FOnTransitionEvent& OnTransitionEvent() { return OnTransitionEventDelegate; }

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPlayableStatusChanged, UAvaPlayable*);
	FOnPlayableStatusChanged& OnPlayableStatusChanged() { return OnPlayableStatusChangedDelegate; }

	virtual bool LoadAsset(const FAvaSoftAssetPtr& InSourceAsset, bool bInInitiallyVisible, const FString& InLoadOptions = FString()) { return false;}
	virtual bool UnloadAsset() {return false;}
	virtual const FSoftObjectPath& GetSourceAssetPath() const;
	virtual EAvaPlayableStatus GetPlayableStatus() const { return EAvaPlayableStatus::Unknown;}
	virtual IAvaSceneInterface* GetSceneInterface() const {return nullptr;}
	virtual EAvaPlayableCommandResult ExecuteAnimationCommand(EAvaPlaybackAnimAction InAnimAction, const FAvaPlaybackAnimPlaySettings& InAnimPlaySettings);
	virtual EAvaPlayableCommandResult UpdateRemoteControlCommand(const TSharedRef<FAvaPlayableRemoteControlValues>& InRemoteControlValues, EAvaPlayableRCUpdateFlags InFlags = EAvaPlayableRCUpdateFlags::None);
	virtual bool IsRemoteProxy() const { return false; }
	virtual bool GetShouldBeVisible() const { return true; }
	virtual void SetShouldBeVisible(bool bInShouldBeVisible) {}
	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) {}

	/**
	 * Called when the playable is visible, to determine if the playable is ready to transition and be shown.
	 * Should return true when all the elements are loaded/ready after becoming visible.
	 */
	virtual bool IsReadyToPlay() const
	{
		return true;
	}

	/**
	 * @brief Ensures the given asset is playing (visible) with the given parameters.
	 * @param InWorldPlaySettings World settings (render target) to render this asset with.
	 * @remark This doesn't trigger the animations.
	 */
	void BeginPlay(const FAvaInstancePlaySettings& InWorldPlaySettings);

	/**
	 * @brief Make this asset not play anymore (will end up hidden)
	 */
	void EndPlay(EAvaPlayableEndPlayOptions InOptions);

	bool IsPlaying() const { return bIsPlaying; }
	
	UAvaPlayableGroup* GetPlayableGroup() const { return PlayableGroup; }

	UWorld* GetPlayWorld() const { return PlayableGroup ? PlayableGroup->GetPlayWorld() : nullptr; }

	bool HasSequence(const UAvaSequence* InSequence) const;

	virtual void SetInstanceId(const FGuid& InInstanceId ) { InstanceId = InInstanceId; }
	const FGuid& GetInstanceId() const { return InstanceId; }

	virtual void SetUserData(const FString& InUserData) { UserData = InUserData; }
	const FString& GetUserData() const { return UserData; }

	TSharedPtr<FAvaPlayableRemoteControlValues> GetLatestRemoteControlValues() const
	{
		return LatestRemoteControlValues;
	}

protected:
	/**
	 * @brief Performs derived initialization of the playable. Does not load the asset.
	 * @remark Playable Group is setup by the derived classes.
	 * @param InPlayableInfo Necessary information to setup the playable.
	 * @return true if the playable group and derived initialization was successful.
	 */
	virtual bool InitPlayable(const FPlayableCreationInfo& InPlayableInfo);

	/** Called by BeginPlay for derived classes implementation. */
	virtual void OnPlay() {}

	/** Called by EndPlay for derived classes implementation. */
	virtual void OnEndPlay() {}

	/** Called by UpdateRemoteControlCommand for derived classes implementation. */
	virtual void OnRemoteControlValuesApplied() {}
	
	void HandleOnSequenceStarted(UAvaSequencePlayer* InSequencePlayer, UAvaSequence* InSequence);
	void HandleOnSequencePaused(UAvaSequencePlayer* InSequencePlayer, UAvaSequence* InSequence);
	void HandleOnSequenceFinished(UAvaSequencePlayer* InSequencePlayer, UAvaSequence* InSequence);

	static UAvaPlayable* CreateLocalPlayable(UObject* InOuter, const FPlayableCreationInfo& InPlayableInfo);
	static UAvaPlayable* CreateRemoteProxyPlayable(UObject* InOuter, const FPlayableCreationInfo& InPlayableInfo);

protected:
	/**
	 * Playable "instancing container" group.
	 * Defines the interface for what a playable can do to it's container.
	 */
	UPROPERTY(Transient)
	TObjectPtr<UAvaPlayableGroup> PlayableGroup;
	
	bool bIsPlaying = false;

	/**
	 * Unique identifier for this playable instance.
	 * This is used for the playback client and server's playable replication. 
	 */
	FGuid InstanceId;

	/**
	 * User data that is replicated to and from the playback server.
	 * This can be used to transport additional information for this playable.
	 */
	FString UserData;

	TSharedPtr<FAvaPlayableRemoteControlValues> LatestRemoteControlValues;

	static FOnSequenceEvent OnSequenceEventDelegate;
	static FOnTransitionEvent OnTransitionEventDelegate;
	FOnPlayableStatusChanged OnPlayableStatusChangedDelegate;
};
