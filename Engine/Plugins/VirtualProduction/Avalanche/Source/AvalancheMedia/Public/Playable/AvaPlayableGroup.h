// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playable/IAvaPlayableVisibilityConstraint.h"
#include "UObject/Object.h"
#include "UObject/ObjectKey.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/WeakInterfacePtr.h"

#include "AvaPlayableGroup.generated.h"

class FSceneView;
class FSceneViewFamily;
class UAvaPlayable;
class UAvaPlayableGroupManager;
class UAvaPlayableTransition;
class UGameInstance;
class UTextureRenderTarget2D;
struct FAvaInstancePlaySettings;

/**
 * This class defines the interface and base of a playable group.
 *
 * A playable group is intended to group playables according to the
 * underlying rendering implementation. In most cases, it corresponds
 * to a game instance, either owned or not, local to the process or remote.
 *
 * It tracks and manage the playables state, transitions and
 * visibility constraints.
 *
 * The design goal of this class is to allow hooking the playable framework
 * to any game instance, including PIE so it can work with any work flow
 * (editor, PIE, game, nDisplay, etc).
 *
 * Ideally, the playable class itself should be "game instance" agnostic
 * and do all it's bidding on it's container through this class.
 */
UCLASS()
class AVALANCHEMEDIA_API UAvaPlayableGroup : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * PlayableGroup creation information contains the necessary information to
	 * create an instance of a playable group.
	 */
	struct FPlayableGroupCreationInfo
	{
		/** Container for shared playable groups. */
		UAvaPlayableGroupManager* PlayableGroupManager = nullptr;
		/** Source asset path. */
		FSoftObjectPath SourceAssetPath;
		/** Channel name this playable will be instanced in. */
		FName ChannelName;
		/** Indicate if the group is for remote proxy playables. */
		bool bIsRemoteProxy = false;
		/** Indicate if the group is shared for multiple playables. If so, it will be registered in the given playable group manager. */
		bool bIsSharedGroup = false;
		/** Existing Game Instance. In this case, the playable group will not own the game instance. */
		UGameInstance* GameInstance = nullptr;
	};
	
	static UAvaPlayableGroup* MakePlayableGroup(UObject* InOuter, const FPlayableGroupCreationInfo& InPlayableGroupInfo);

	/** Register a playable to this group when it is created. */
	void RegisterPlayable(UAvaPlayable* InPlayable);

	/** Unregister a playable when it is about to be deleted. */
	void UnregisterPlayable(UAvaPlayable* InPlayable);

	/** Returns true if there are any valid registered playables. */
	bool HasPlayables() const;

	/** Returns true if there are any valid registered playables that are currently playing. */
	bool HasPlayingPlayables() const;

	/** Finds all the playables that are instances of the given source asset. */
	void FindPlayablesBySourceAssetPath(const FSoftObjectPath& InSourceAssetPath, TArray<UAvaPlayable*>& OutFoundPlayables) const;

	void RegisterPlayableTransition(UAvaPlayableTransition* InPlayableTransition);
	
	void UnregisterPlayableTransition(UAvaPlayableTransition* InPlayableTransition);

	/** Tick transitions that have been registered. Returns the number of transitions that where ticked. */
	void TickTransitions(double InDeltaSeconds);

	bool HasTransitions() const;

	void PushSynchronizedEvent(FString&& InEventSignature, TUniqueFunction<void()> InFunction);

	bool IsSynchronizedEventPushed(const FString& InEventSignature) const;

	/**
	 * Creates the game instance's world if it wasn't already.
	 * @return true if the world was created. false if nothing was done.
	 */
	virtual bool ConditionalCreateWorld() { return true; }

	/**
	 * Begin playing the game instance's world if it wasn't already.
	 * @return true if the BeginPlay was done (i.e. on the state transition only), false otherwise.
	 */
	virtual bool ConditionalBeginPlay(const FAvaInstancePlaySettings& InWorldPlaySettings) { return false; }

	virtual void RequestEndPlayWorld(bool bInForceImmediate) {}

	/** Keep track of the last playable that applied it's camera in the viewport/controller. */
	void SetLastAppliedCameraPlayable(UAvaPlayable* InPlayable);

	virtual bool IsWorldPlaying() const { return true; }

	virtual bool IsRenderTargetReady() const { return true; }

	/**
	 * Current logic for the render target: use the game instance's if present, fallback to internal one if not.
	 */
	virtual UTextureRenderTarget2D* GetRenderTarget() const;

	/**
	 * Returns the currently managed render target.
	 */
	virtual UTextureRenderTarget2D* GetManagedRenderTarget() const;

	/**
	 * The playback graph determines if this playable group will render in a broadcast channel's
	 * render target or an offscreen one. In the later case, the playable group keeps
	 * track of that render target.
	 * 
	 * @remark The playable group does not automatically render in the current "managed" render target.
	 * The render target this group will render into is determined by the arguments of ConditionalBeginPlay.
	 */
	virtual void SetManagedRenderTarget(UTextureRenderTarget2D* InManageRenderTarget);

	/**
	 * Returns this group's game instance, if it has one.
	 */
	UGameInstance* GetGameInstance() const { return GameInstance; }

	/**
	* Returns this group's play world, if it has one.
	*/
	virtual UWorld* GetPlayWorld() const;

	/**
	 * Returns the broadcast channel name this playable group is part of.
	 */
	FName GetChannelName() const;
	
	/**
	 * Unloads the game instance's world if no more playables are loaded.
	 * @return true if the world was unloaded. false if nothing was done.
	 */ 
	virtual bool ConditionalRequestUnloadWorld(bool bForceImmediate) { return true; }

	/**
	 * @brief Notify the playable group that a playable is loading an asset.
	 * @param InPlayable Source of the event.
	 */
	void NotifyLevelStreaming(UAvaPlayable* InPlayable);

	UAvaPlayableGroupManager* GetPlayableGroupManager() const { return ParentPlayableGroupManagerWeak.Get(); }

	void RegisterVisibilityConstraint(const TWeakInterfacePtr<IAvaPlayableVisibilityConstraint>& InVisibilityConstraint);
	void UnregisterVisibilityConstraint(const IAvaPlayableVisibilityConstraint* InVisibilityConstraint);

	void RequestSetVisibility(UAvaPlayable* InPlayable, bool bInShouldBeVisible);

	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView);

	void ForEachPlayable(TFunctionRef<bool(UAvaPlayable*)> InFunction);
	
	void ForEachPlayableTransition(TFunctionRef<bool(UAvaPlayableTransition*)> InFunction);

	/**
	 * Search for the first playable group associated to the given world.
	 * @param InWorld	Play world to search the playable group of.
	 * @param bInFallbackToGlobalSearch If the direct link from world to the playable group is not found, fallback to a global search of the whole system.
	 */
	static UAvaPlayableGroup* FindPlayableGroupForWorld(const UWorld* InWorld, bool bInFallbackToGlobalSearch = true);
	
protected:
	bool IsVisibilityConstrained(const UAvaPlayable* InPlayable) const;

	void OnPlayableStatusChanged(UAvaPlayable* InPlayable);
	
	void ConditionalRegisterWorldDelegates(UWorld* InWorld);
	void UnregisterWorldDelegates(UWorld* InWorld);

	bool DisplayLoadedAssets(FText& OutText, FLinearColor& OutColor);
	bool DisplayPlayingAssets(FText& OutText, FLinearColor& OutColor);
	bool DisplayTransitions(FText& OutText, FLinearColor& OutColor);

	void HidePawnsForView(const UWorld* InPlayWorld, FSceneView& InView) const;
	
protected:
	/**
	 * Managed Render Target for this playable group.
	 */
	UPROPERTY(Transient)
	TObjectPtr<UTextureRenderTarget2D> ManagedRenderTarget;

	UPROPERTY(Transient)
	TObjectPtr<UGameInstance> GameInstance;

	/** Broadcast channel name this playable group is part of. */
	FName ChannelName;

	/** PlayableGroup Manager handling this playable group. */
	TWeakObjectPtr<UAvaPlayableGroupManager> ParentPlayableGroupManagerWeak;
	
	/** List of playables for this group. */
	TSet<TObjectKey<UAvaPlayable>> Playables;

	/** Last playable that applied a camera. */
	TWeakObjectPtr<UAvaPlayable> LastAppliedCameraPlayableWeak;

	/** Set of registered playable transitions for this group. Remark: used for ticking. */
	TSet<TObjectKey<UAvaPlayableTransition>> PlayableTransitions;

	/** If transitions are added or removed while ticking, we need to protect transition iterator. */
	bool bIsTickingTransitions = false;

	/** Set of transitions to remove accumulated during transition ticking. */
	TSet<TObjectKey<UAvaPlayableTransition>> PlayableTransitionsToRemove;

	/** Set of transitions to add accumulated during transition ticking. */
	TSet<TObjectKey<UAvaPlayableTransition>> PlayableTransitionsToAdd;

	/**
	 * Since the UViewportStatsSubsystem delegation mechanism does not allow us to verify
	 * if it is bound to this world or another one. We need this auxiliary binding
	 * tracking to compensate.
	 */
	TWeakObjectPtr<UWorld> LastWorldBoundToDisplayDelegates;

	/** UViewportStatsSubsystem display delegate indices. */
	TArray<int32> DisplayDelegateIndices;

	struct FVisibilityRequest
	{
		TWeakObjectPtr<UAvaPlayable> PlayableWeak;
		bool bShouldBeVisible;

		void Execute(const UAvaPlayableGroup* InPlayableGroup) const;
	};
	
	TArray<FVisibilityRequest> VisibilityRequests;

	TArray<TWeakInterfacePtr<IAvaPlayableVisibilityConstraint>> VisibilityConstraints;
};