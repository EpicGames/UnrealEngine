// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "AvaPlayableLibrary.generated.h"

class UAvaPlayable;
class UAvaPlayableTransition;

UCLASS(MinimalAPI, DisplayName = "Motion Design Playable Library", meta=(ScriptName = "MotionDesignPlayableLibrary"))
class UAvaPlayableLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Returns the playable corresponding to that world object. */
	static UAvaPlayable* GetPlayable(const UObject* InWorldContextObject);

	/** Returns the transition this playable is part of. */
	static UAvaPlayableTransition* GetPlayableTransition(const UAvaPlayable* InPlayable);

	/**
	 * Injects the remote control values from current transition for the current playable.
	 * @remark This does nothing if there is no current transition the current playable is part of or if
	 *		the current level is not managed by a playable.
	 * @return true if the values have been injected, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "Motion Design|Playable", meta=(WorldContext = "InWorldContextObject"))
	static AVALANCHEMEDIA_API bool UpdatePlayableRemoteControlValues(const UObject* InWorldContextObject);

	/**
	 * Returns the current hidden state of this playable.
	 * @remark This only works if current level is managed by a playable (i.e. in a rundown or playback graph).
	 * @return true if the hidden state is set, false otherwise or if not managed by a playable.
	 */
	UFUNCTION(BlueprintCallable, Category = "Motion Design|Playable", meta=(WorldContext = "InWorldContextObject"))
	static AVALANCHEMEDIA_API bool IsPlayableHidden(const UObject* InWorldContextObject);

	/**
	 * Sets the hidden state of all primitives under this playable. Hidden primitives will not be rendered.
	 * @remark This only works if current level is managed by a playable (i.e. in a rundown or playback graph).
	 * @return true if the value was set, false otherwise (if not managed by a playable).
	 */
	UFUNCTION(BlueprintCallable, Category = "Motion Design|Playable", meta=(WorldContext = "InWorldContextObject"))
	static AVALANCHEMEDIA_API bool SetPlayableHidden(const UObject* InWorldContextObject, bool bInShouldBeHidden);

	/**
	 * Push a playable sync event with the given signature on the cluster.
	 * The action will complete when the event has occured on all nodes of the cluster.
	 * The event signature includes the playable id as well. Therefore, the given signature need only be
	 * unique in terms of any sequence of synchronized events for the given playable.
	 * 
	 * @param InEventSignature Event signature, must be unique in terms of sequence of events for the given playable. 
	 * @param bInSuccess If true upon completion, the event has been synchronized successfully.
	 */
	UFUNCTION(BlueprintCallable, Category = "Motion Design|Playable", meta=(Latent, LatentInfo="InLatentInfo", WorldContext="InWorldContextObject"))
	static AVALANCHEMEDIA_API void PlayableSyncEventLatent(const UObject* InWorldContextObject, FLatentActionInfo InLatentInfo, const FString& InEventSignature, bool& bInSuccess);
};
