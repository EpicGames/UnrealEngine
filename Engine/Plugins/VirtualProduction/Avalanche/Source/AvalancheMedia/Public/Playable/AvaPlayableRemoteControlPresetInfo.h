// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "AvaPlayableRemoteControlPresetInfo.generated.h"

class URemoteControlPreset;
struct FSoftObjectPath;

/**
 * Container for the controlled entities extra information.
 */
USTRUCT()
struct FAvaPlayableRemoteControlControlledEntityInfo
{
	GENERATED_BODY()

	/** Controller ids that are controlling the entity. */
	UPROPERTY()
	TArray<FGuid> ControlledBy;
};

/**
 * Container for the remote control preset extra information used for playable.
 */
USTRUCT()
struct FAvaPlayableRemoteControlPresetInfo
{
	GENERATED_BODY()

	FAvaPlayableRemoteControlPresetInfo() = default;

	/**
	 * Refreshes from given remote control preset.
	 * @remark will reset bDirty flag.
	 */
	void Refresh(const URemoteControlPreset* InRemoteControlPreset);

	/**
	 * Call this whenever the source RCP is modified (or is likely to be) to
	 * invalidate the information and trigger a refresh on the next access.
	 */
	void MarkDirty()
	{
		bDirty = true;
	}

	/** Indicates if the data needs to be refreshed. */
	bool IsDirty() const
	{
		return bDirty;
	}

	/**
	 * Returns true if the given controller is considered "overlapping", i.e. it's set of controlled entities
	 * overlap with other controllers. An overlapping controller can't safely be updating its behaviors when the end
	 * result needs to be deterministic.
	 */
	bool IsControllerOverlapping(const FGuid& InControllerId) const
	{
		return OverlappingControllers.Contains(InControllerId);
	}

	/** Id of the RemoteControlPreset */
	UPROPERTY()
	FGuid PresetId;
	
	/** Contains a set of entity guids that are bound to a controller action. */
	UPROPERTY()
	TMap<FGuid, FAvaPlayableRemoteControlControlledEntityInfo> EntitiesControlledByController;

	/**
	 * Contains a set of controllers that are "overlapping". 
	 * The term "overlapping controllers" defines the scenario where multiple controllers influence
	 * a shared subset of controlled entities.
	 */ 
	UPROPERTY()
	TSet<FGuid> OverlappingControllers;

protected:
	/** When the source asset is modified (or likely to be), we mark as dirty to trigger a refresh on next access. */
	bool bDirty = true;
};

/**
 * Global Cache for remote control preset info 
 */
class IAvaPlayableRemoteControlPresetInfoCache
{
public:
	static AVALANCHEMEDIA_API IAvaPlayableRemoteControlPresetInfoCache& Get();

	/**
	 * Request the RCP info cached for the given asset path.
	 * If not available, will be created from the given RCP.
	 */
	virtual TSharedPtr<FAvaPlayableRemoteControlPresetInfo> GetRemoteControlPresetInfo(const FSoftObjectPath& InAssetPath, const URemoteControlPreset* InRemoteControlPreset) = 0;

	/**
	 *	Flush specified entry from the cache.
	 */
	virtual void Flush(const FSoftObjectPath& InAssetPath) = 0;

	/**
	 * Flush all unused entries from the cache.
	 */
	virtual void Flush() = 0;
	
protected:
	virtual ~IAvaPlayableRemoteControlPresetInfoCache() = default;
};