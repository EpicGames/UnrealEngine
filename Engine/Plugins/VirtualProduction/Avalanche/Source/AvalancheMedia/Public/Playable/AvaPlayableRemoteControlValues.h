// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "RemoteControlPreset.h"
#include "AvaPlayableRemoteControlValues.generated.h"

/**
 *	Flags indicating which component of the remote control values have been
 *	modified by an operation.
 */
UENUM()
enum class EAvaPlayableRemoteControlChanges : uint8
{
	None = 0,
	EntityValues		= 1 << 0,
	ControllerValues	= 1 << 1,

	All              = 0xFF,
};
ENUM_CLASS_FLAGS(EAvaPlayableRemoteControlChanges);

USTRUCT()
struct AVALANCHEMEDIA_API FAvaPlayableRemoteControlValue
{
	GENERATED_BODY()

	FAvaPlayableRemoteControlValue() = default;

	FAvaPlayableRemoteControlValue(const FString& InValue, bool bInIsDefault = false)
		: Value(InValue), bIsDefault(bInIsDefault) {}

	void SetValueFrom(const FAvaPlayableRemoteControlValue& InOther)
	{
		Value = InOther.Value;
	}

	/** Returns true if the given value is the same. Ignores the default flag. */
	bool IsSameValueAs(const FAvaPlayableRemoteControlValue& InOther) const
	{
		return Value.Equals(InOther.Value, ESearchCase::CaseSensitive);
	}

	bool Serialize(FArchive& Ar);

	/**
	 * The Remote Control Entity or Controller's Value stored as a Json formatted string.
	 */
	UPROPERTY()
	FString Value;

	/**
	 * Indicate if the value is a default value from a template.
	 * This is used to know which values to update when
	 * updating the page's values from the template (reimport page).
	 * This is set to true only when the values are from the template.
	 * If values are modified by an edit operation, it will be set to false.
	 */
	UPROPERTY()
	bool bIsDefault = false;
};

template<>
struct TStructOpsTypeTraits<FAvaPlayableRemoteControlValue> : public TStructOpsTypeTraitsBase2<FAvaPlayableRemoteControlValue>
{
	enum
	{
		WithSerializer = true,
	};
};

/** Container for the remote control values of a playable. */
USTRUCT()
struct AVALANCHEMEDIA_API FAvaPlayableRemoteControlValues
{
	GENERATED_BODY()

	FAvaPlayableRemoteControlValues() = default;
	
	/** Value as a binary array of the Remote Control Entity. */
	UPROPERTY()
	TMap<FGuid, FAvaPlayableRemoteControlValue> EntityValues;

	/** Controller values. */
	UPROPERTY()
	TMap<FGuid, FAvaPlayableRemoteControlValue> ControllerValues;

	/** Copies the values (properties and controllers) from the given RemoteControlPreset. */
	void CopyFrom(const URemoteControlPreset* InRemoteControlPreset, bool bInIsDefault);

	/**
	 * Compares the remote control EntityValues with another instance
	 * @return true if the other instance has the exact same EntityValues (count and value), false otherwise
	 */
	bool HasSameEntityValues(const FAvaPlayableRemoteControlValues& InOther) const;

	/**
	 * Compares the remote control ControllerValues with another instance
	 * @return true if the other instance has the exact same ControllerValues (count and value), false otherwise
	 */
	bool HasSameControllerValues(const FAvaPlayableRemoteControlValues& InOther) const;

	/**
	 * Removes the extra values compared to the given reference values.
	 * @return flags indicating what changed.
	 */
	EAvaPlayableRemoteControlChanges PruneRemoteControlValues(const FAvaPlayableRemoteControlValues& InReferenceValues);
	
	/**
	 * Update the property/controller values (i.e. add missing, remove extras) from the given reference values.
	 * If bInUpdateDefaults is true, the existing values flagged as "default" will be updated, i.e. the reference values will be applied.
	 * Otherwise, the existing values are not modified.
	 * Also, when adding the missing values from reference default values, the default flag is also set in the destination value. 
	 * For a full copy of all properties and controllers, use CopyFrom instead.
	 * @return flags indicating what changed.
	 */
	EAvaPlayableRemoteControlChanges UpdateRemoteControlValues(const FAvaPlayableRemoteControlValues& InReferenceValues, bool bInUpdateDefaults);

	/**
	 * Reset the values to the reference.
	 * @param InReferenceValues Values to be reset to.
	 * @param bInIsDefaults If true, consider all values as "default". Otherwise, preserves the "default" status from reference.
	 * @return
	 */
	EAvaPlayableRemoteControlChanges ResetRemoteControlValues(const FAvaPlayableRemoteControlValues& InReferenceValues, bool bInIsDefaults);
	
	/**
	 * Reset the controller value to the reference.
	 * @param InReferenceValue Value to be reset to.
	 * @param bInIsDefaults If true, consider all values as "default". Otherwise, preserves the "default" status from reference.
	 * @return
	 */
	EAvaPlayableRemoteControlChanges ResetRemoteControlControllerValue(const FGuid& InId
		, const FAvaPlayableRemoteControlValue& InReferenceValue, bool bInIsDefaults);

	/**
	 * Reset the entity value to the reference.
	 * @param InReferenceValue Value to be reset to.
	 * @param bInIsDefaults If true, consider all values as "default". Otherwise, preserves the "default" status from reference.
	 * @return
	 */
	EAvaPlayableRemoteControlChanges ResetRemoteControlEntityValue(const FGuid& InId
		, const FAvaPlayableRemoteControlValue& InReferenceValue, bool bInIsDefaults);

	/**
	 * Set the entity value from the given preset.
	 * @return true if the value was set.
	 */
	bool SetEntityValue(const FGuid& InId, const URemoteControlPreset* InRemoteControlPreset, bool bInIsDefault);
	bool HasEntityValue(const FGuid& InId) const { return EntityValues.Contains(InId); }
	const FAvaPlayableRemoteControlValue* GetEntityValue(const FGuid& InId) const { return EntityValues.Find(InId); }
	void SetEntityValue(const FGuid& InId, const FAvaPlayableRemoteControlValue& InValue) {EntityValues.Add(InId, InValue);}
	
	bool SetControllerValue(const FGuid& InId, const URemoteControlPreset* InRemoteControlPreset, bool bInIsDefault);
	bool HasControllerValue(const FGuid& InId) const { return ControllerValues.Contains(InId); }
	const FAvaPlayableRemoteControlValue* GetControllerValue(const FGuid& InId) const { return ControllerValues.Find(InId); }
	void SetControllerValue(const FGuid& InId, const FAvaPlayableRemoteControlValue& InValue) {ControllerValues.Add(InId, InValue);}

	/**
	 *	Apply the entity values to the given remote control preset.
	 *	@param InRemoteControlPreset RemoteControlPreset
	 *	@param InSkipEntities set of entities to skip (will not be updated)
	 */
	void ApplyEntityValuesToRemoteControlPreset(URemoteControlPreset* InRemoteControlPreset, const TSet<FGuid>& InSkipEntities = TSet<FGuid>()) const;

	/**
	 *	Apply the controller values to the given remote control preset.
	 *	Remark: controller actions are executed by this operation.
	 */
	void ApplyControllerValuesToRemoteControlPreset(URemoteControlPreset* InRemoteControlPreset, bool bInForceDisableBehaviors = false) const;


	/**
	 * Return true if there are key collisions with the other set of values. 
	 */
	bool HasIdCollisions(const FAvaPlayableRemoteControlValues& InOtherValues) const;
	
	/**
	 * @brief Merge the other values with current ones, combining the keys.
	 * @param InOtherValues Other values to merge with.
	 * @return true if the merge was clean with no collisions. false indicate there was some key collisions and information is lost.
	 */
	bool Merge(const FAvaPlayableRemoteControlValues& InOtherValues);
	
	/** Returns true if the given maps have id collisions. */
	static bool HasIdCollisions(const TMap<FGuid, FAvaPlayableRemoteControlValue>& InValues, const TMap<FGuid, FAvaPlayableRemoteControlValue>& InOtherValues);
	
	static const FAvaPlayableRemoteControlValues& GetDefaultEmpty();

	/**
	 * Collect the referenced asset paths from the given values.
	 */
	static void CollectReferencedAssetPaths(const TMap<FGuid, FAvaPlayableRemoteControlValue>& InValues,TSet<FSoftObjectPath>& OutReferencedPaths);

	/**
	 * Utility function to determine if a controller should be ignored by the playable management layer.
	 * @param InController Controller
	 * @return true if the controller should be ignored.
	 */
	static bool ShouldIgnoreController(const URCVirtualPropertyBase* InController);

	/**
	 * Utility function to determine if a controller is considered an "event" controller, i.e.
	 * it doesn't carry a value to be used to set or restore state, but rather is runtime event based.
	 */
	static bool IsRuntimeEventController(const URCVirtualPropertyBase* InController);
};