// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaPlayableSettings.generated.h"

USTRUCT()
struct FAvaSynchronizedEventsFeatureSelection
{
	GENERATED_BODY()
	
	/**
	 * Select the implementation for synchronizing events.
	 * "Default" will select the most appropriate implementation available.
	 */
	UPROPERTY(Config, EditAnywhere, Category = Settings)
	FString Implementation;
};

/**
 * Settings applied when instancing a Motion Design Playable.
 */
USTRUCT()
struct FAvaPlayableSettings
{
	GENERATED_BODY()

	/**
	 * Select the implementation for synchronizing events.
	 */
	UPROPERTY(Config, EditAnywhere, Category = Settings)
	FAvaSynchronizedEventsFeatureSelection SynchronizedEventsFeature;

	/**
	 * Automatically hide pawn actor in playables.
	 */
	UPROPERTY(Config, EditAnywhere, Category = Settings)
	bool bHidePawnActors = true;

	/**
	 * List of postfix that can be added to controllers to be ignored by playable, playback and rundown implementation.
	 * Those controllers can be used by other control systems at runtime.
	 */
	UPROPERTY(Config, EditAnywhere, Category = Settings)
	TArray<FString> IgnoredControllerPostfix;
};
