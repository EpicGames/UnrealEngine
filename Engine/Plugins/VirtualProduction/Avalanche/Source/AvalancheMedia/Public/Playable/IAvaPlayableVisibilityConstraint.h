// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IAvaPlayableVisibilityConstraint.generated.h"

class UAvaPlayable;

/**
 * Interface for the Playable visibility constraint.
 */
UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UAvaPlayableVisibilityConstraint : public UInterface
{
	GENERATED_BODY()
};

class IAvaPlayableVisibilityConstraint
{
	GENERATED_BODY()

public:
	/**
	 * @brief Verify if a visibility constraint apply to the given playable.
	 * @param InPlayable Playable to evaluate visibility constraint for.
	 * @return true if the playable's visibility is constrained, false otherwise.
	 *
	 * If the visibility is constrained, it means it has to wait for something else to
	 * be ready and therefore can't be made visible just yet.
	 */
	virtual bool IsVisibilityConstrained(const UAvaPlayable* InPlayable) const = 0;
};
