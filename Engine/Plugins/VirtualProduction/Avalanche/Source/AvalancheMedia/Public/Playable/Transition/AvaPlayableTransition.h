// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "Playable/AvaPlayableRemoteControlValues.h"
#include "UObject/Object.h"
#include "UObject/ObjectKey.h"
#include "AvaPlayableTransition.generated.h"

class UAvaPlayable;
class UAvaPlayableGroup;

/**
 * @brief Defines the playable entry role in the transition.
 */
enum class EAvaPlayableTransitionEntryRole : uint8
{
	/** The playable corresponding to this entry is a entering the scene. */
	Enter,
	/** The playable corresponding to this entry is already in the scene and may react to the transition, but is otherwise neutral. */
	Playing,
	/** The playable corresponding to this entry is already in the scene but is commanded to exit. */
	Exit
};

UCLASS()
class AVALANCHEMEDIA_API UAvaPlayableTransition : public UObject
{
	GENERATED_BODY()
	
public:
	virtual bool Start();
	virtual void Stop();
	virtual bool IsRunning() const { return false; }
	virtual void Tick(double InDeltaSeconds) {}

	void SetTransitionId(const FGuid& InTransitionId);
	void SetTransitionFlags(EAvaPlayableTransitionFlags InFlags);
	void SetEnterPlayables(TArray<TWeakObjectPtr<UAvaPlayable>>&& InPlayablesWeak);
	void SetPlayingPlayables(TArray<TWeakObjectPtr<UAvaPlayable>>&& InPlayablesWeak);
	void SetExitPlayables(TArray<TWeakObjectPtr<UAvaPlayable>>&& InPlayablesWeak);
	
	bool IsEnterPlayable(const UAvaPlayable* InPlayable) const;
	bool IsPlayingPlayable(const UAvaPlayable* InPlayable) const;
	bool IsExitPlayable(const UAvaPlayable* InPlayable) const;

	void SetEnterPlayableValues(TArray<TSharedPtr<FAvaPlayableRemoteControlValues>>&& InPlayableValues);

	/**
	 * Finds the stored values for a given playable
	 * @param InPlayable the playable to look for
	 * @param bInIsEnterPlayable whether to look for the Enter Playable Values
	 * @return the remote control values for the playable if found. null otherwise
	 */
	TSharedPtr<FAvaPlayableRemoteControlValues> GetValuesForPlayable(const UAvaPlayable* InPlayable, bool bInIsEnterPlayable);

	/** This is called during the transition evaluation to indicate discarded playables. */
	void MarkPlayableAsDiscard(UAvaPlayable* InPlayable);

	/** Returns information on this transition suitable for logging. */
	virtual FString GetPrettyInfo() const;

	EAvaPlayableTransitionFlags GetTransitionFlags() const { return TransitionFlags; }

	const FGuid& GetTransitionId() const { return TransitionId; }
	
protected:
	UAvaPlayable* FindPlayable(const FGuid& InstanceId) const;
	
protected:
	FGuid TransitionId;
	EAvaPlayableTransitionFlags TransitionFlags = EAvaPlayableTransitionFlags::None;
	
	TArray<TSharedPtr<FAvaPlayableRemoteControlValues>> EnterPlayableValues;

	/** Other Playable Values that are not the Enter Playable's (i.e. Exiting or Playing Playable Values) */
	TMap<TObjectKey<UAvaPlayable>, TSharedPtr<FAvaPlayableRemoteControlValues>> OtherPlayableValues;

	TArray<TWeakObjectPtr<UAvaPlayable>> EnterPlayablesWeak;
	TArray<TWeakObjectPtr<UAvaPlayable>> PlayingPlayablesWeak;
	TArray<TWeakObjectPtr<UAvaPlayable>> ExitPlayablesWeak;

	/** Keep track of the discarded playables so events can be sent when the transition ends. */
	TArray<TWeakObjectPtr<UAvaPlayable>> DiscardPlayablesWeak;

	TSet<TWeakObjectPtr<UAvaPlayableGroup>> PlayableGroupsWeak;
};

class AVALANCHEMEDIA_API FAvaPlayableTransitionBuilder
{
public:
	FAvaPlayableTransitionBuilder();

	void AddEnterPlayableValues(const TSharedPtr<FAvaPlayableRemoteControlValues>& InValues);
	
	bool AddEnterPlayable(UAvaPlayable* InPlayable, bool bInAllowMultipleAdd = false);
	bool AddPlayingPlayable(UAvaPlayable* InPlayable, bool bInAllowMultipleAdd = false);
	bool AddExitPlayable(UAvaPlayable* InPlayable, bool bInAllowMultipleAdd = false);
	
	bool AddPlayable(UAvaPlayable* InPlayable, EAvaPlayableTransitionEntryRole InPlayableRole, bool bInAllowMultipleAdd = false)
	{
		switch (InPlayableRole)
		{
			case EAvaPlayableTransitionEntryRole::Enter:
				return AddEnterPlayable(InPlayable, bInAllowMultipleAdd);
			case EAvaPlayableTransitionEntryRole::Playing:
				return AddPlayingPlayable(InPlayable, bInAllowMultipleAdd);
			case EAvaPlayableTransitionEntryRole::Exit:
				return AddExitPlayable(InPlayable, bInAllowMultipleAdd);
		}
		return false;
	}

	UAvaPlayableTransition* MakeTransition(UObject* InOuter, const FGuid& InTransitionId = FGuid());

private:
	TArray<TSharedPtr<FAvaPlayableRemoteControlValues>> EnterPlayableValues;

	TArray<TWeakObjectPtr<UAvaPlayable>> EnterPlayablesWeak;
	TArray<TWeakObjectPtr<UAvaPlayable>> PlayingPlayablesWeak;
	TArray<TWeakObjectPtr<UAvaPlayable>> ExitPlayablesWeak;
};