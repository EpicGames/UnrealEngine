// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "Delegates/Delegate.h"

class IAvaPlaybackClient;

/**
 * Delegates related to IAvaPlaybackClient.
 * 
 * The delegates have been separated in their own global scope to avoid issues with delegates disconnecting
 * if the playback client is stopped or started during the normal course of operations. Having
 * the delegates outside of IAvaPlaybackClient's implementation simplify the delegate handler
 * management.
 */
namespace UE::AvaPlaybackClient::Delegates
{
	struct FPlaybackStatusChangedArgs
	{
		const FGuid& InstanceId;
		const FSoftObjectPath& AssetPath;
		const FString& ChannelName;
		const FString& ServerName;
		EAvaPlaybackStatus PrevStatus;
		EAvaPlaybackStatus NewStatus;
	};
	
	struct FPlaybackAssetStatusChangedArgs
	{
		const FSoftObjectPath& AssetPath;
		const FString& ServerName;
		EAvaPlaybackAssetStatus Status;
	};

	struct FPlaybackSequenceEventArgs
	{
		const FGuid& InstanceId;
		const FSoftObjectPath& AssetPath;
		const FString& ChannelName;
		const FString& ServerName;
		const FString& SequenceName;
		EAvaPlayableSequenceEventType EventType;
		int32 ServerFrameNumber;
	};

	struct FPlaybackTransitionEventArgs
	{
		const FGuid& TransitionId;
		const FGuid& InstanceId;
		const FString& ChannelName;
		const FString& ServerName;
		EAvaPlayableTransitionEventFlags EventFlags;
		int32 ServerFrameNumber;
	};

	enum class EConnectionEvent : uint8
	{
		ServerConnected = 0,
		ServerDisconnected
	};

	struct FConnectionEventArgs
	{
		const FString& ServerName;
		EConnectionEvent Event;
	};

	/**
	 * Delegate called when a server connection event occurs. 
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnConnectionEvent, IAvaPlaybackClient&, const FConnectionEventArgs&);
	AVALANCHEMEDIA_API FOnConnectionEvent& GetOnConnectionEvent();
	
	/**
	* Delegate called when a playback status changed message is received.
	*/
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPlaybackStatusChanged, IAvaPlaybackClient&, const FPlaybackStatusChangedArgs&);
	AVALANCHEMEDIA_API FOnPlaybackStatusChanged& GetOnPlaybackStatusChanged();

	/**
	* Delegate called when a playback asset status changed message is received.
	*/
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPlaybackAssetStatusChanged, IAvaPlaybackClient&, const FPlaybackAssetStatusChangedArgs&);
	AVALANCHEMEDIA_API FOnPlaybackAssetStatusChanged& GetOnPlaybackAssetStatusChanged();

	/**
	* Delegate called when a playback status changed message is received.
	*/
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPlaybackSequenceEvent, IAvaPlaybackClient&, const FPlaybackSequenceEventArgs&);
	AVALANCHEMEDIA_API FOnPlaybackSequenceEvent& GetOnPlaybackSequenceEvent();

	/**
	* Delegate called when a playback status changed message is received.
	*/
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPlaybackTransitionEvent, IAvaPlaybackClient&, const FPlaybackTransitionEventArgs&);
	AVALANCHEMEDIA_API FOnPlaybackTransitionEvent& GetOnPlaybackTransitionEvent();
};
