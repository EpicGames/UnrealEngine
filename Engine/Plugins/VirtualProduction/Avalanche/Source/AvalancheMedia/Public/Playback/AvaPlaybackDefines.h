// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaPlaybackDefines.generated.h"

UENUM()
enum class EAvaPlaybackAction
{
	/** No op */
	None,
	/** Load the given asset. Used for pre-loading assets. */
	Load,
	/** Start (i.e. start ticking world and rendering) the given asset, loading it if not pre-loaded. */
	Start,
	/** Stop ticking and rendering the world. */
	Stop,
	/** Unload the given asset, i.e. destroy the world, etc. */
	Unload,
	/** Request the status of the asset. No action is actually performed on the asset. */
	Status,
	/** Request to set the user data of the playback instance.*/
	SetUserData,
	/** Request the user data of the playback instance. No action is actually performed on the asset. */
	GetUserData,
};

UENUM()
enum class EAvaPlaybackAnimAction
{
	None,
	Play,
	Continue,
	Stop,
	PreviewFrame,
	CameraCut
};

namespace UE::AvaMedia::Playback
{
	/**
	 * Playback manager needs a channel name as a key.
	 * The default key for a preview channel that may not have a defined broadcast channel.
	 */
	static inline const FLazyName DefaultPreviewChannelName = FLazyName(TEXT("_Preview"));
}