// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "Nodes/AvaPlaybackNodePlayer.h"
#include "Nodes/AvaPlaybackNodeRoot.h"
#include "Nodes/Events/Actions/AvaPlaybackAnimations.h"
#include "Playable/AvaPlayableRemoteControlValues.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "AvaPlaybackGraph.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAvaPlayback, Log, All);

class FAvaPlaybackManager;
class IAvaPlaybackGraphEditor;
class UAvaBroadcast;
class UAvaGameInstance;
class UAvaPlayable;
class UAvaPlayableGroupManager;
class UAvaPlayableTransition;
class UAvaPlaybackNode;
class UAvaPlaybackNodeRoot;
class UEdGraph;
class UTextureRenderTarget2D;
struct FAvaBroadcastOutputChannel;
struct FAvaPlaybackAnimPlaySettings;

UENUM()
enum class EAvaPlaybackStopOptions : uint8
{
	/**
	 * Default option allows for deferred execution of the request when it is safe to do so.
	 */
	None			= 0,
	/**
	 * Forces the execution of the request when it is called. Typically during shut down.
	 */
	ForceImmediate	= 1 << 1,
	/**
	 *	Unload from memory after being stopped.
	 */
	Unload			= 1 << 2,
	/**
	 * Default option allows for deferred execution of the request when it is safe to do so.
	 */
	Default			= None
};
ENUM_CLASS_FLAGS(EAvaPlaybackStopOptions);

UENUM()
enum class EAvaPlaybackUnloadOptions : uint8
{
	/**
	 * Default option allows for deferred execution of the request when it is safe to do so.
	 */
	None			= 0,
	/**
	 * Forces the execution of the request when it is called. Typically during shut down.
	 */
	ForceImmediate	= 1 << 1,
	/**
	 * Default option allows for deferred execution of the request when it is safe to do so.
	 */
	Default			= None
};
ENUM_CLASS_FLAGS(EAvaPlaybackUnloadOptions);

/**
 * Owns a group of playables.
 * Used to group playables per channels, but can be extended to any conceptual grouping.
 */
USTRUCT()
struct FAvaPlaybackPlayableGroup
{
	GENERATED_BODY()

	// TODO: For transition logic within the playback graph, we can't simply index the
	// playables by asset path since we may have to instance the same asset twice for a transition between pages
	// using the same template. For now, rundown does transition with multiple playback graph instances instead.
	UPROPERTY(Transient)
	TMap<FSoftObjectPath, TObjectPtr<UAvaPlayable>> Playables;
	
	UAvaPlayable* FindPlayable(const FSoftObjectPath& InSourceAssetPath) const
	{
		const TObjectPtr<UAvaPlayable>* Playable = Playables.Find(InSourceAssetPath);
		return Playable ? *Playable : nullptr;
	}

	bool HasPlayable(const UAvaPlayable* InPlayable) const
	{
		for (const TPair<FSoftObjectPath, TObjectPtr<UAvaPlayable>>& Playable : Playables)
		{
			if (Playable.Value == InPlayable)
			{
				return true;
			}
		}
		return false;
	}
	
	UAvaPlayable* GetFirstPlayable() const
	{
		for (const TPair<FSoftObjectPath, TObjectPtr<UAvaPlayable>>& Playable : Playables)
		{
			if (Playable.Value)
			{
				return Playable.Value;
			}
		}
		return nullptr;
	}
	
	void GetAllPlayables(TArray<UAvaPlayable*>& OutPlayables) const
	{
		for (const TPair<FSoftObjectPath, TObjectPtr<UAvaPlayable>>& Playable : Playables)
		{
			if (Playable.Value)
			{
				OutPlayables.Add(Playable.Value);
			}
		}
	}

	void ForEachPlayable(TFunctionRef<void(const UAvaPlayable*)> InFunction) const
	{
		for (const TPair<FSoftObjectPath, TObjectPtr<UAvaPlayable>>& Playable : Playables)
		{
			if (Playable.Value)
			{
				InFunction(Playable.Value);
			}
		}
	}
};

/**
 * A Playback Graph is used for playing Motion Design assets integrated with the broadcast framework.
 * It allows the creation of a playback graph with some logic and inputs routed to player nodes, the results of which can be routed
 * to broadcast channels. This is the lowest implementation layer that supports distributed playback (over message bus).
 */
UCLASS(NotBlueprintable, BlueprintType, ClassGroup = "Motion Design Playback", 
	meta = (DisplayName = "Motion Design Playback Graph"))
class AVALANCHEMEDIA_API UAvaPlaybackGraph : public UObject
{
	GENERATED_BODY()

public:
	virtual ~UAvaPlaybackGraph() override;
	
	UFUNCTION(BlueprintCallable, Category = "Motion Design Playback")
	void Play();
	
	UFUNCTION(BlueprintCallable, Category = "Motion Design Playback")
	void Stop(EAvaPlaybackStopOptions InStopOptions);
	
	UFUNCTION(BlueprintCallable, Category = "Motion Design Playback")
	void LoadInstances();

	/** Unload all game instance's worlds from this playback. */
	UFUNCTION(BlueprintCallable, Category = "Motion Design Playback")
	void UnloadInstances(EAvaPlaybackUnloadOptions InUnloadOptions);
	
	UFUNCTION(BlueprintCallable, Category = "Motion Design Playback")
	bool IsPlaying() const { return bIsPlaying; }

	/**
	 * Sets the parent playback manager that will handle ticking. 
	 */
	void SetPlaybackManager(const TSharedPtr<FAvaPlaybackManager>& InPlaybackManager);

	bool IsManaged() const { return PlaybackManagerWeak.IsValid(); }
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPlaybackStateChanged, bool);
	FOnPlaybackStateChanged OnPlaybackStateChanged;

	/** Quick access to the first playable in the playback graph. */ 
	UAvaPlayable* GetFirstPlayable() const;

	void GetAllPlayables(TArray<UAvaPlayable*>& OutPlayables) const;

	void ForEachPlayable(TFunctionRef<void(const UAvaPlayable*)> InFunction) const;

	/** Returns true if the given playable is part of this playback graph. */
	bool HasPlayable(const UAvaPlayable* InPlayable) const;
	
	UAvaPlayable* FindPlayable(const FSoftObjectPath& InSourceAssetPath, const FName& InChannelName) const;
	UAvaPlayable* FindOrLoadPlayable(const FAvaSoftAssetPtr& InSourceAsset, const FName& InChannelName, const FString& InLoadOptions = FString());
	void RemovePlayable(const FSoftObjectPath& InSourceAssetPath, const FName& InChannelName);
	bool UnloadAndRemovePlayable(UAvaPlayable* InPlayable, const FSoftObjectPath& InSourceAssetPath, const FName& InChannelName, bool bInForceImmediate);
	bool UnloadAndRemovePlayable(const FSoftObjectPath& InSourceAssetPath, const FName& InChannelName, bool bInForceImmediate);

	TArray<UAvaGameInstance*> GetActiveGameInstances() const;
	
	/**
	 *	Determines if the graph contains a node with the given source Motion Design Asset.
	 *	This works even if the Motion Design asset running.
	 */
	bool HasPlayerNodeForSourceAsset(const FSoftObjectPath& InSourceAssetPath) const;

	void SetPlayableGroupManager(UAvaPlayableGroupManager* InPlayableGroupManager);

	UAvaPlayableGroupManager* GetPlayableGroupManager() const
	{
		return PlayableGroupManager ? PlayableGroupManager.Get() : GetGlobalPlayableGroupManager();
	}

	static UAvaPlayableGroupManager* GetGlobalPlayableGroupManager();

	/** Resolve the channel name for a playback settings entry. */
	FName ResolveChannelName(const FName InPlaybackChannelName) const
	{
		// For a preview, route everything to the preview channel,
		// otherwise use the resolved channel (from traversing the graph)
		return IsPreviewOnly() ? GetPreviewChannelName() : InPlaybackChannelName;
	}

public:
	bool IsDryRunningGraph() const { return bIsDryRunningGraph; }
	void DryRunGraph(bool bDeferredExecution = false);

	/**
	 *	Sets the preview channel this playback object is dedicated to.
	 *	By doing this, the playback object will render in only that channel regardless
	 *	of the internal playback graph.
	 *	The specified channel may not exist in the broadcast configuration, in which case the
	 *	preview playback will fallback to a channel-less render target.
	 */
	void SetPreviewChannelName(const FName& InPreviewChannelName) { PreviewChannelName = InPreviewChannelName; }

	FName GetPreviewChannelName() const { return PreviewChannelName; }

	/**
	 * Returns true if the playback object is dedicated to a local preview. 
	 */
	bool IsPreviewOnly() const { return !PreviewChannelName.IsNone(); }

	/**
	 * Helper function to implement some of the event nodes logic.
	 * It will return the list of channel names that correspond this the indices.
	 */
	TArray<FName> GetChannelNamesForIndices(const TArray<int32>& InChannelIndices) const;

private:
	void DryRunGraphInternal();
	void RegisterTickDelegate();
	void UnregisterTickDelegate();
	void OnEndFrameTick();
	void ConditionalResolvePlaybackSettings();

public:
	void SetRootNode(UAvaPlaybackNodeRoot* InRoot);
	void AddPlayerNode(UAvaPlaybackNodePlayer* InPlayer);
	
	UAvaPlaybackNodeRoot* GetRootNode() const;
	
	void AddPlaybackNode(UAvaPlaybackNode* Node);
	void RemovePlaybackNode(UAvaPlaybackNode* Node);

	/**
	 * Stops the playable corresponding to the given asset in the given channel.
	 */
	void StopPlayable(const FSoftObjectPath& InSourceAssetPath, const FName& InChannelName);

	/**
	 * Start streaming the asset in a game instance.
	 */
	void LoadAsset(const FAvaSoftAssetPtr& InSourceAsset, const FName& InChannelName, const FString& InLoadOptions);
	
	virtual void Tick(float DeltaTime);
	
	void PushAnimationCommand(const FSoftObjectPath& InSourceAssetPath, const FString& InChannelName, EAvaPlaybackAnimAction InAnimAction, const FAvaPlaybackAnimPlaySettings& InAnimSettings);
	void PushRemoteControlValues(const FSoftObjectPath& InSourceAssetPath, const FString& InChannelName, const TSharedRef<FAvaPlayableRemoteControlValues>& InRemoteControlValues, EAvaPlayableRCUpdateFlags InFlags = EAvaPlayableRCUpdateFlags::None);

	void SetupPlaybackNode(UAvaPlaybackNode* InPlaybackNode, bool bSelectNewNode = true);

#if WITH_EDITOR
	const TArray<TObjectPtr<UAvaPlaybackNode>>& GetPlaybackNodes() const;

	/** Create the basic sound graph */
	void CreateGraph();

	/** Clears all nodes from the graph */
	void ClearGraph();

	void RefreshPlaybackNode(UAvaPlaybackNode* InPlaybackNode);	

	void CompilePlaybackNodesFromGraphNodes();

	/** Get the Playback EdGraph  */
	UEdGraph* GetGraph();

	/** Resets all graph data and nodes */
	void ResetGraph();

	void SetGraphEditor(TSharedPtr<IAvaPlaybackGraphEditor> InGraphEditor);

	TSharedPtr<IAvaPlaybackGraphEditor> GetGraphEditor() const;
#endif

	template<typename InPlaybackNodeType, typename = typename TEnableIf<TIsDerivedFrom<InPlaybackNodeType, UAvaPlaybackNode>::IsDerived>::Type>
	InPlaybackNodeType* ConstructPlaybackNode(TSubclassOf<InPlaybackNodeType> NodeClass = nullptr, bool bSelectNewNode = true)
	{
		//Ensure Class is Valid
		if (!NodeClass)
		{
			NodeClass = InPlaybackNodeType::StaticClass();
		}

		if (!NodeClass->IsChildOf(UAvaPlaybackNodeRoot::StaticClass()) || !IsValid(RootNode))
		{
			//Set flag to be transactional so it registers with undo system
			InPlaybackNodeType* const PlaybackNode = NewObject<InPlaybackNodeType>(this
				, NodeClass
				, NAME_None
				, RF_Transactional);

#if WITH_EDITOR
			PlaybackNodes.Add(PlaybackNode);
#endif
			
			PlaybackNode->PostAllocateNode();
			SetupPlaybackNode(PlaybackNode, bSelectNewNode);
		
			return PlaybackNode;
		}

		checkf(0, TEXT("Root Node has already been created!"));
		return nullptr;
	}

	/** Delegate called when a playable is created in this playback graph. */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPlayableCreated, UAvaPlaybackGraph* /*InPlayback*/, UAvaPlayable* /*InPlayable*/);
	FOnPlayableCreated OnPlayableCreated;

protected:
	void ExecutePendingAnimationCommands();
	void ExecutePendingRemoteControlCommands();
	
	void OnChannelChanged(const FAvaBroadcastOutputChannel& InChannel, EAvaBroadcastChannelChange InChange);
	void OnChannelBroadcastStateChanged(const FAvaBroadcastOutputChannel& InChannel);

	void SetIsPlaying(bool bInIsPlaying);
	
	bool CanRefreshPlayback(const FAvaSoftAssetPtr& InSourceAsset, const FAvaBroadcastOutputChannel& InChannel) const;

	bool RefreshPlayback(const FAvaSoftAssetPtr& InSourceAsset, const FString& InLoadOptions, FAvaBroadcastOutputChannel& InChannel, const FName& InChannelName);
	
	bool CanRefreshPreview(const FAvaSoftAssetPtr& InSourceAsset) const;
	
	bool RefreshPreview(const FAvaSoftAssetPtr& InSourceAsset, const FString& InLoadOptions, FAvaBroadcastOutputChannel& InChannel, const FName& InChannelName);

	static UTextureRenderTarget2D* UpdatePlaybackRenderTarget(const UAvaPlayable* InPlayable, const FAvaBroadcastOutputChannel& InChannel);
	
protected:
	/**
	 * Keep track of the parent manager that handles ticking.
	 * If the playback graph is an asset (not managed), it will handle it's own ticking.
	 */
	TWeakPtr<FAvaPlaybackManager> PlaybackManagerWeak;

	TMap<FName, FAvaPlaybackChannelParameters> PlaybackSettings;
	TMap<FName, FAvaPlaybackChannelParameters> PreviousPlaybackSettings;
	
	UPROPERTY(Transient)
	TObjectPtr<UAvaPlaybackNodeRoot> RootNode;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UAvaPlaybackNodePlayer>> PlayerNodes;

#if WITH_EDITORONLY_DATA
	TWeakPtr<IAvaPlaybackGraphEditor> GraphEditorWeak;
	
	UPROPERTY()
	TObjectPtr<UEdGraph> EdGraph;

	/**
	 * An array of all the Playback Nodes, Only available in Editor (Playback doesn't need to know disconnected nodes,
	 * unless we support runtime changes as well).
	 */
	UPROPERTY()
	TArray<TObjectPtr<UAvaPlaybackNode>> PlaybackNodes;
#endif

	/**
	 * Playback Group Manager for this Playback object.
	 *
	 * This manager defines the scope with which this playback object's playables
	 * can share their group with. In other words, the playables of this playback
	 * (that support sharing a group) can have a shared group (for the given channel)
	 * cached in this given manager.
	 *
	 * So far, the PlayableGroupManager has the same scope as the playable manager,
	 * i.e. global (a.k.a local) or per playback server.
	 */
	UPROPERTY(Transient)
	TObjectPtr<UAvaPlayableGroupManager> PlayableGroupManager;

	/**
	 * Keep track of the playables per channel.
	 * 
	 * The Playback object owns it's playables. However, the playables
	 * can share a playable group from other playback objects. This
	 * is necessary for the playback server that is always creating just
	 * one playback object per playable (atm).
	 */
	UPROPERTY(Transient)
	TMap<FName, FAvaPlaybackPlayableGroup> ChannelPlayableGroups;
	
	bool bIsPlaying = false;
	bool bIsDryRunningGraph = false;
	bool bAsyncDryRunRequested = false;
	bool bIsTicking = false;	// Tick reentrancy guard.
	FDelegateHandle TickDelegateHandle;

	/** Name of the preview channel this playback object is dedicated to. */
	FName PreviewChannelName;
	
	struct FPlaybackCommand
	{
		/** If the command has been pending for too long, it will be discarded. */
		FDateTime Timeout;
		
		FSoftObjectPath SourcePath;
		FString ChannelName;
		FName ChannelFName;

		bool HasTimedOut(const FDateTime& InNow) const
		{
			return InNow > Timeout;	
		}
	};
	struct FAnimationCommand : public FPlaybackCommand
	{
		EAvaPlaybackAnimAction AnimAction;
		FAvaPlaybackAnimPlaySettings AnimPlaySettings;
	};
	struct FRemoteControlCommand : public FPlaybackCommand
	{
		TSharedRef<FAvaPlayableRemoteControlValues> Values;
		EAvaPlayableRCUpdateFlags Flags;
	};
	
	TArray<FAnimationCommand> PendingAnimationCommands;
	TArray<FRemoteControlCommand> PendingRemoteControlCommands;
};

/**
 * Somewhat reusable helper class to build a playback graph. 
 */
class FAvaPlaybackGraphBuilder
{
public:
	FAvaPlaybackGraphBuilder(UAvaPlayableGroupManager* InPlayableGroupManager);
	~FAvaPlaybackGraphBuilder();

	/**
	 * @brief Connect the given node to the root node's pin corresponding to the given channel name.
	 * @param InChannelName Channel name to connect to.
	 * @param InNodeToConnect Node (already constructed) to connect.
	 */
	bool ConnectToRoot(const FString& InChannelName, UAvaPlaybackNode* InNodeToConnect);
	
	int32 GetPinIndexForChannel(const FString& InChannelName) const;

	UAvaPlaybackGraph* FinishBuilding();

	/** Helper function to construct a playback node. This will add the node in the playback's node list. */
	template<typename InPlaybackNodeType, typename = typename TEnableIf<TIsDerivedFrom<InPlaybackNodeType, UAvaPlaybackNode>::IsDerived>::Type>
		InPlaybackNodeType* ConstructPlaybackNode(TSubclassOf<InPlaybackNodeType> InNodeClass = nullptr, bool bInSelectNewNode = true)
	{
		if (Playback)
		{
			bFinished = false;
			return Playback->ConstructPlaybackNode<InPlaybackNodeType>(InNodeClass, bInSelectNewNode);
		}
		checkf(0, TEXT("Playback object is not created."));
		return nullptr;
	}
	
private:
	bool bFinished = false;
	UAvaPlaybackGraph* Playback = nullptr;
	UAvaPlaybackNodeRoot* RootNode = nullptr;
};