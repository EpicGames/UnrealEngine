// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playback/AvaPlaybackGraph.h"
#include "UObject/GCObject.h"
#include "UObject/StrongObjectPtr.h"
#include "AvaPlaybackManager.generated.h"

class FAvaPlaybackManager;
class FAvaPlaybackSourceAssetEntry;
class IAvaMediaSyncProvider;
class UAvaPlayableGroupManager;
class UAvaPlaybackTransition;
struct FAvaPlaybackAnimPlaySettings;	// private

/** Handle to a playback instance for recycling. */
class AVALANCHEMEDIA_API FAvaPlaybackInstance : public FGCObject
{
public:
	FAvaPlaybackInstance() = default;
	FAvaPlaybackInstance(const FGuid& InInstanceId, const FSoftObjectPath& InSourcePath, const FString& InChannelName, UAvaPlaybackGraph* InPlayback);
	virtual ~FAvaPlaybackInstance() override;

	//~ Begin FGCObject
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override;
	//~ End FGCObject

	/**
	 * This is used by the server to track the client's id.
	 * It can also be used when reconciling the client state from the server.
	 */
	void SetInstanceId(const FGuid& InInstanceId);

	/**
	 * Set the instance's user data. This is propagate to the server and can be
	 * used when reconciling the client state.
	 */
	void SetInstanceUserData(const FString& InUserData);
	
	const FGuid& GetInstanceId() const { return InstanceId; }
	const FString& GetInstanceUserData() const { return InstanceUserData;}
	const FString& GetChannelName() const { return ChannelName; }
	const FName& GetChannelFName() const { return ChannelFName; }
	const FSoftObjectPath& GetSourcePath() const { return SourcePath; }
	UAvaPlaybackGraph* GetPlayback() const { return Playback; }
	EAvaPlaybackStatus GetStatus() const { return Status; }
	bool IsPlaying() const { return Playback ? Playback->IsPlaying() : false;}

	bool UpdateStatus();
	void SetStatus(EAvaPlaybackStatus InStatus) { Status = InStatus; }

	void Unload();
	void Recycle();

	TSharedPtr<FAvaPlaybackManager> GetManager() const;

private:
	void OnPlayableCreated(UAvaPlaybackGraph* InPlayback, UAvaPlayable* InPlayable);
	
private:
	/** If the cache slot becomes invalid, it means this instance will be discarded instead of being recycled. */
	TWeakPtr<FAvaPlaybackSourceAssetEntry> AssetEntryWeak;

	FGuid InstanceId;
	FString ChannelName;
	FName ChannelFName;
	FSoftObjectPath SourcePath;
	TObjectPtr<UAvaPlaybackGraph> Playback;
	EAvaPlaybackStatus Status = EAvaPlaybackStatus::Unknown;
	FString InstanceUserData;

	friend class FAvaPlaybackManager;
};

/**
 * For each playback instance, a source asset entry is kept track of to allow the entry to be
 * invalidated and prevent the instance to be recycled.
 */
class AVALANCHEMEDIA_API FAvaPlaybackSourceAssetEntry
{
public:
	FAvaPlaybackSourceAssetEntry(const TSharedPtr<FAvaPlaybackManager>& InParentManager) : ParentManagerWeak(InParentManager.ToWeakPtr()) {}
	~FAvaPlaybackSourceAssetEntry() = default;

	TSharedPtr<FAvaPlaybackInstance> AcquirePlaybackInstance(const FString& InChannelName)
	{
		TSharedPtr<FAvaPlaybackInstance> AcquiredInstance;
		const int32 InstanceIndex = AvailableInstances.IndexOfByPredicate([InChannelName](const TSharedPtr<FAvaPlaybackInstance>& InInstance)
		{
			return InInstance && InInstance->GetChannelName() == InChannelName;
		});

		if (InstanceIndex != INDEX_NONE)
		{
			AcquiredInstance = AvailableInstances[InstanceIndex];
			AvailableInstances.RemoveAtSwap(InstanceIndex, 1);
			UsedInstances.Add(AcquiredInstance.ToWeakPtr());
		}
		return AcquiredInstance;
	}

	TSharedPtr<FAvaPlaybackInstance> FindPlaybackInstance(const FGuid& InInstanceId,
		bool bInAvailableInstances = true, bool bInUsedInstances = true) const;
	
	TSharedPtr<FAvaPlaybackInstance> FindPlaybackInstanceForChannel(const FString& InChannelName,
		bool bInAvailableInstances = true, bool bInUsedInstances = true) const;

	TSharedPtr<FAvaPlaybackInstance> FindPlaybackInstanceByPredicate(TFunctionRef<bool(const FAvaPlaybackInstance& /*InPlaybackInstance*/)> InPredicate,
		bool bInAvailableInstances = true, bool bInUsedInstances = true) const;
	
	void ForAllPlaybackInstances(TFunctionRef<void(FAvaPlaybackInstance& /*InPlaybackInstance*/)> InFunction,
		bool bInAvailableInstances = true, bool bInUsedInstances = true);

	TSharedPtr<FAvaPlaybackManager> GetManager() const { return ParentManagerWeak.Pin(); }

	void DiscardInstance(FAvaPlaybackInstance* InInstanceToRemove);
	void RecycleInstance(FAvaPlaybackInstance* InInstanceToRecycle);

private:
	/** Keeping a weak reference to the manager to recycle the instance. */
	TWeakPtr<FAvaPlaybackManager> ParentManagerWeak;
	
	TArray<TSharedPtr<FAvaPlaybackInstance>> AvailableInstances;
	TArray<TWeakPtr<FAvaPlaybackInstance>> UsedInstances;

	friend class FAvaPlaybackManager;
};

/** Flags indicating what changed in the package event. */
UENUM()
enum class EAvaPlaybackPackageEventFlags : uint8
{
	None			= 0,
	External		= 1 << 0,
	Saved			= 1 << 1,
	AssetDeleted	= 1 << 2,
	All				= 0xFF
};
ENUM_CLASS_FLAGS(EAvaPlaybackPackageEventFlags);

class AVALANCHEMEDIA_API FAvaPlaybackManager : public TSharedFromThis<FAvaPlaybackManager>
{
public:
	FAvaPlaybackManager();
	virtual ~FAvaPlaybackManager();

	void Tick();

	void SetEnablePlaybackCommandsBuffering(bool bInEnable) { bEnablePlaybackCommandsBuffering = bInEnable;}

	UAvaPlayableGroupManager* GetPlayableGroupManager() const { return PlayableGroupManager.Get(); }

	/**
	 * Acquire (recycle) a cached playback instance. Will return null if none available in the cache.
	 * @param InAssetPath Soft object path to the asset to load. This can either be a playback graph itself or a playable asset.
	 * @param InChannelName For a playable asset, specifying the channel name will indicate where to connect the player node in the playback graph
	 */
	TSharedPtr<FAvaPlaybackInstance> AcquirePlaybackInstance(const FSoftObjectPath& InAssetPath, const FString& InChannelName) const;

	/**
	 * Load a new (recyclable) playback instance for the given asset in the given channel.
	 * @remark This function only loads (or creates) the playback graph. The playable assets should be loaded only once the graph is playing.
	 * @param InAssetPath Soft object path to the asset to load. This can either be a playback graph itself or a playable asset.
	 * @param InChannelName For a playable asset, specifying the channel name will indicate where to connect the player node in the playback graph
	 * @param InLoadOptions Optional parameters to be used when loading the playable. 
	 */
	TSharedPtr<FAvaPlaybackInstance> LoadPlaybackInstance(const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FString& InLoadOptions = FString());

	/**
	 * Attempts to acquire (recycle) an existing playback instance. If none available, will load a new one.
	 * @param InAssetPath Soft object path to the asset to load. This can either be a playback graph itself or a playable asset.
	 * @param InChannelName For a playable asset, specifying the channel name will indicate where to connect the player node in the playback graph
	 * @param InLoadOptions Optional parameters to be used when loading the playable.
 	 * @remark Load options will not be applied for recycled instances. (todo)
	 * @see LoadPlaybackInstance, AcquirePlaybackInstance
	 */  
	TSharedPtr<FAvaPlaybackInstance> AcquireOrLoadPlaybackInstance(const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FString& InLoadOptions = FString());

	/**
	 * Finds an existing (either available or used) playback instance.
	 * This will not acquire it (i.e. if available, it will remain so).
	 */
	TSharedPtr<FAvaPlaybackInstance> FindPlaybackInstance(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName) const;

	/**
	 * Unload and discard any available (i.e. not used) instances for this asset/channel entry.
	 * If specified channel name is empty, it will discard all instances of the asset.
	 */
	bool UnloadPlaybackInstances(const FSoftObjectPath& InAssetPath, const FString& InChannelName);
	
	void ForAllPlaybackInstances(TFunctionRef<void(FAvaPlaybackInstance& /*InPlaybackInstance*/)> InFunction);
	
	/**
	 *	Determines the local status of the given asset on this local instance of the playback manager.
	 */
	EAvaPlaybackAssetStatus GetLocalAssetStatus(const FName& InPackageName);
	
	/**
	 * Invalidates the cached local asset status.
	 */
	void InvalidateCachedLocalAssetStatus(const FName& InPackageName);

	/**
	 * Utility function to determine if an asset is locally available.
	 * Remark: Determining the presence of dependencies is somewhat unreliable i.e.
	 * an asset can playback fine even with some dependencies missing and it is hard to figure it out.
	 * Because of that, for now, we consider the asset available even if it is missing some dependencies.
	 */
	bool IsLocalAssetAvailable(const FName& InPackageName)
	{
		const EAvaPlaybackAssetStatus LocalAssetStatus = GetLocalAssetStatus(InPackageName);
		return (LocalAssetStatus == EAvaPlaybackAssetStatus::Available || LocalAssetStatus == EAvaPlaybackAssetStatus::MissingDependencies);
	}

	bool IsLocalAssetAvailable(const FSoftObjectPath& InAssetPath)
	{
		return IsLocalAssetAvailable(InAssetPath.GetLongPackageFName());
	}

	/** Utility function to determine the playback status of an unloaded asset. */
	EAvaPlaybackStatus GetUnloadedPlaybackStatus(const FSoftObjectPath& InAssetPath)
	{
		// When the playback entry is unloaded, we rely on the local asset status to determine the playback status.
		// Note: since there is now an independent asset status, we could remove all the playback states related to the asset.
		return IsLocalAssetAvailable(InAssetPath) ? EAvaPlaybackStatus::Available : EAvaPlaybackStatus::Missing;
	}

	/**
	 *	Invalidate the asset entry. All cached instances will be invalidated along with it.
	 */
	void InvalidatePlaybackAssetEntry(const FSoftObjectPath& InAssetPath);

	/**
	 * Loads a new playback graph for the given asset with the given channel as context.
	 * @remark The playback graph is either loaded or created. The referenced assets (playables) are not loaded yet.
	 * @param InAssetPath Soft object path to the asset to load. This can either be a playback graph itself or a playable asset.
	 * @param InChannelName For a playable asset, specifying the channel name will indicate where to connect the player node in the playback graph
	 * @param InLoadOptions Optional parameters to be used when loading the playable.
	 * @remark Direct loading of playback graphs is not fully supported yet. Experimental use only.
	 */
	UAvaPlaybackGraph* LoadPlaybackObject(const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FString& InLoadOptions = FString()) const;

	/**
	 * Utility function to create a new playback graph instance for the given world asset with the given channel as context.
	 * @param InWorld Soft object path to the world asset to load.
	 * @param InChannelName For a playable asset, specifying the channel name will indicate where to connect the player node in the playback graph
	 * @param InLoadOptions Optional parameters to be used when loading the playable.
	 */
	UAvaPlaybackGraph* BuildPlaybackFromWorld(const TSoftObjectPtr<UWorld>& InWorld, const FString& InChannelName, const FString& InLoadOptions = FString()) const;
	
	/**
	 * Stops all currently playing playback objects.
	 * @param bInUnload if true, will also unload the objects.
	 * @return Returns the list of all source assets that were stopped.
	 **/
	TArray<FSoftObjectPath> StopAllPlaybacks(bool bInUnload);

	bool PushAnimationCommand(const FGuid& InInstanceId, const FSoftObjectPath& InSourcePath, const FString& InChannelName, EAvaPlaybackAnimAction InAction, const FAvaPlaybackAnimPlaySettings& InAnimSettings);
	bool PushRemoteControlCommand(const FGuid& InInstanceId, const FSoftObjectPath& InSourcePath, const FString& InChannelName, const TSharedRef<FAvaPlayableRemoteControlValues>& InRemoteControlValues, EAvaPlayableRCUpdateFlags InFlags = EAvaPlayableRCUpdateFlags::None);
	bool PushPlaybackTransitionStartCommand(UAvaPlaybackTransition* InTransitionToStart);

	/** This is used on the playback server to apply any pending commands to a playback instance. */
	void ApplyPendingCommands(UAvaPlaybackGraph* InPlaybackObject, const FGuid& InInstanceId, const FSoftObjectPath& InSourcePath, const FString& InChannelName);

	/**
	 *	Indicate the manager is in a shutdown sequence and will force game instances to destroy worlds right away.
	 */
	void StartShuttingDown() { bIsShuttingDown = true;}

	bool IsShuttingDown() const { return bIsShuttingDown; }
	
	EAvaPlaybackStopOptions GetPlaybackStopOptions(bool bInUnload) const
	{
		EAvaPlaybackStopOptions Options = bIsShuttingDown ? EAvaPlaybackStopOptions::ForceImmediate : EAvaPlaybackStopOptions::None;
		Options |= (bInUnload || bIsShuttingDown) ? EAvaPlaybackStopOptions::Unload : EAvaPlaybackStopOptions::None;
		return Options;
	}
	
	EAvaPlaybackUnloadOptions GetPlaybackUnloadOptions() const
	{
		return bIsShuttingDown ? EAvaPlaybackUnloadOptions::ForceImmediate : EAvaPlaybackUnloadOptions::None;
	}

	/** Let the Playback manager know that a package has been modified. */
	void OnPackageModified(const FName& InPackageName, EAvaPlaybackPackageEventFlags InFlags = EAvaPlaybackPackageEventFlags::None);

	/**
	 *	TearDown the whole Motion Design Playback system.
	 *	
	 *	When Motion Design Playback is used within a game, we need to tear down everything
	 *	as the parent world is being teared down. The game tear down process will forcibly
	 *	mark as garbage (and GCs) all the GameInstances, including those held by the playback objects,
	 *	despite playback objects holding strong references to them. To avoid issues we preemptively
	 *	destroy all the playback objects.
	 */
	void OnParentWorldBeginTearDown();

	/**
	 * Implements a similar command to Engine::HandleStatCommand, except it
	 * will fetch Motion Design's game viewport client if everything else fails.
	 */
	bool HandleStatCommand(const TArray<FString>& InArgs);

	/** Returns true if the given asset is a playback asset, i.e. either a "playable" asset or a playback graph. */
	static bool IsPlaybackAsset(const FAssetData& InAssetData);
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPlaybackInstanceInvalidated, const FAvaPlaybackInstance&);
	FOnPlaybackInstanceInvalidated OnPlaybackInstanceInvalidated;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPlaybackInstanceStatusChanged, const FAvaPlaybackInstance&);
	FOnPlaybackInstanceStatusChanged OnPlaybackInstanceStatusChanged;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnLocalPlaybackAssetRemoved, const FSoftObjectPath&);
	FOnLocalPlaybackAssetRemoved OnLocalPlaybackAssetRemoved;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBeginTick, float);
	FOnBeginTick OnBeginTick;

protected:
	// Non-copyable (because copy of CachedAssetStatusExtra is not defined)
	FAvaPlaybackManager(const FAvaPlaybackManager&) = delete;
	FAvaPlaybackManager& operator=(const FAvaPlaybackManager&) = delete;

	void OnPackageSaved(const FString& InPackageFileName, UPackage* InPackage, FObjectPostSaveContext InObjectSaveContext);
	void OnAvaSyncPackageModified(IAvaMediaSyncProvider* InAvaMediaSyncProvider, const FName& InPackageName);
	void OnAssetRemoved(const FAssetData& InAssetData);

	TSharedPtr<FAvaPlaybackSourceAssetEntry> FindPlaybackAssetEntry(const FSoftObjectPath& InAssetPath) const
	{
		const TSharedPtr<FAvaPlaybackSourceAssetEntry>* Existing = PlaybackAssetEntries.Find(InAssetPath);
		return Existing ? *Existing : TSharedPtr<FAvaPlaybackSourceAssetEntry>();
	}
	
	TSharedPtr<FAvaPlaybackSourceAssetEntry> GetPlaybackAssetEntry(const FSoftObjectPath& InAssetPath)
	{
		TSharedPtr<FAvaPlaybackSourceAssetEntry>* Existing = PlaybackAssetEntries.Find(InAssetPath);
		if (Existing)
		{
			return *Existing;
		}

		TSharedPtr<FAvaPlaybackSourceAssetEntry> NewAssetEntry = MakeShared<FAvaPlaybackSourceAssetEntry>(SharedThis(this));
		
		PlaybackAssetEntries.Add(InAssetPath, NewAssetEntry);
		return NewAssetEntry;
	}
	
private:
	bool bIsShuttingDown = false;

	/** This is the shared pool of shared playable groups for all the playback objects. */
	TStrongObjectPtr<UAvaPlayableGroupManager> PlayableGroupManager; 
	
	// TODO: Refactor this to support cache capacity with eviction (LRU) like FAvaRundownManagedInstanceCache.
	TMap<FSoftObjectPath, TSharedPtr<FAvaPlaybackSourceAssetEntry>> PlaybackAssetEntries;
	
	/** Cached asset status. */
	TMap<FName, EAvaPlaybackAssetStatus> CachedAssetStatus;

	TArray<TWeakObjectPtr<UAvaPlaybackTransition>> PendingStartTransitions;
	
	/**
	 * Enables the playback command buffering.
	 * 
	 * This is enabled on the playback server to handle the Remote control and animation
	 * commands being received/processed before the playback command itself (where the playback object
	 * is created). If the commands can't be executed because the object is not yet created the
	 * playback manager will buffer the commands and apply them to the object once it is loaded.
	 */
	bool bEnablePlaybackCommandsBuffering = false;
	
	struct FPlaybackObjectCommandBuffers
	{
		struct FAnimationCommand
		{
			EAvaPlaybackAnimAction AnimAction;
			FAvaPlaybackAnimPlaySettings AnimPlaySettings;
		};
		struct FRemoteControlCommand
		{
			TSharedRef<FAvaPlayableRemoteControlValues> Values;
			EAvaPlayableRCUpdateFlags UpdateFlags;
		};
	
		TArray<FAnimationCommand> AnimationCommands;
		TArray<FRemoteControlCommand> RemoteControlCommands;
	};
	// Use FAvaPlaybackManager::MakePlaybackKey for Map key.
	TMap<FString, FPlaybackObjectCommandBuffers> PlaybackObjectCommandBuffers;

	FString MakeCommandBufferKey(const FSoftObjectPath& InAssetPath, const FString& InChannelName) const
	{
		FString CommandBufferKey = InAssetPath.ToString();
		if (!InChannelName.IsEmpty())
		{
			CommandBufferKey += TEXT("_") + InChannelName;
		}
		return CommandBufferKey;
	}

	FString MakeCommandBufferKey(const FGuid& InInstanceId) const
	{
		return InInstanceId.ToString();
	}

	FString MakeCommandBufferKey(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName) const
	{
		return InInstanceId.IsValid() ? MakeCommandBufferKey(InInstanceId) : MakeCommandBufferKey(InAssetPath, InChannelName);
	}
	
	FPlaybackObjectCommandBuffers& GetOrCreatePlaybackCommandBuffers(FString&& InCommandBufferKey);
	const FPlaybackObjectCommandBuffers* GetPlaybackCommandBuffers(const FString& InCommandBufferKey) const;
};
