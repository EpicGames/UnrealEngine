// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "AvaPlaybackDefines.h"
#include "Broadcast/AvaBroadcastDefines.h"
#include "IMessageContext.h"

class UMediaOutput;
struct FAvaPlayableRemoteControlValues;
struct FAvaPlaybackAnimPlaySettings;

class IAvaPlaybackClient
{
public:
	IAvaPlaybackClient() = default;
	virtual ~IAvaPlaybackClient() = default;

	virtual int32 GetNumConnectedServers() const = 0;
	
	virtual TArray<FString> GetServerNames() const = 0;
	
	/** Returns the corresponding server address for the given server name. If server name is not found, returns invalid address. */
	virtual FMessageAddress GetServerAddress(const FString& InServerName) const = 0;

	/** Returns true if the corresponding server user data for the given server name is found. */
	virtual bool HasServerUserData(const FString& InServerName, const FString& InUserDataKey) const = 0;

	/** Returns the corresponding server user data for the given server name and key. Returns empty array if not found. */
	virtual const FString& GetServerUserData(const FString& InServerName, const FString& InUserDataKey) const = 0;

	/** Returns true if the corresponding client user data for the given key is found. */
	virtual bool HasUserData(const FString& InKey) const = 0;

	/** Returns the corresponding client user data for the given key. Returns empty container if not found. */
	virtual const FString& GetUserData(const FString& InKey) const = 0;

	/** Add user data to this client. This is replicated and accessible to the servers. */
	virtual void SetUserData(const FString& InKey, const FString& InData) = 0;

	/** Remove the client's user data entry from the given key. */
	virtual void RemoveUserData(const FString& InKey) = 0;

	/** Send a stat command to all connected servers. */
	virtual void BroadcastStatCommand(const FString& InCommand, bool bInBroadcastLocalState) = 0;
	
	virtual void RequestPlaybackAssetStatus(const FSoftObjectPath& InAssetPath, const FString& InChannelOrServerName, bool bInForceRefresh) = 0;
	virtual void RequestPlayback(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, EAvaPlaybackAction InAction, const FString& InArguments = FString()) = 0;
	virtual void RequestAnimPlayback(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FAvaPlaybackAnimPlaySettings& InAnimSettings) = 0;
	virtual void RequestAnimAction(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FString& InAnimationName, EAvaPlaybackAnimAction InAction) = 0;
	virtual void RequestRemoteControlUpdate(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FAvaPlayableRemoteControlValues& InRemoteControlValues, EAvaPlayableRCUpdateFlags InFlags = EAvaPlayableRCUpdateFlags::None) = 0;
	virtual void RequestPlayableTransitionStart(const FGuid& InTransitionId, TArray<FGuid>&& InEnterInstanceIds, TArray<FGuid>&& InPlayingInstanceIds, TArray<FGuid>&& InExitInstanceIds, TArray<FAvaPlayableRemoteControlValues>&& InEnterValues, const FName& InChannelName, EAvaPlayableTransitionFlags InTransitionFlags) = 0;
	virtual void RequestPlayableTransitionStop(const FGuid& InTransitionId, const FName& InChannelName) = 0;
	virtual void RequestBroadcast(const FString& InProfile, const FName& InChannel, const TArray<UMediaOutput*>& InRemoteMediaOutputs, EAvaBroadcastAction InAction, const FString& InServerName = FString()) = 0;
	/**
	 * Use the device provider to determine if the media output is remote.
	 * This function should only be used as fallback if the channel's media output info is not valid.
	 */
	virtual bool IsMediaOutputRemoteFallback(const UMediaOutput* InMediaOutput) = 0;
	virtual EAvaBroadcastIssueSeverity GetMediaOutputIssueSeverity(const FString& InServerName, const FString& InChannelName, const FGuid& InOutputGuid) const = 0;
	virtual const TArray<FString>& GetMediaOutputIssueMessages(const FString& InServerName, const FString& InChannelName, const FGuid& InOutputGuid) const = 0;
	virtual EAvaBroadcastOutputState GetMediaOutputState(const FString& InServerName, const FString& InChannelName, const FGuid& InOutputGuid) const = 0;

	/**
	 * Returns true if the client has at least one connected server for the given channel.
	 * This does not care if the channel is live or idle. This is needed to send preload
	 * commands to server even if they are not live yet (but connected).
	 */
	virtual bool HasAnyServerOnlineForChannel(const FName& InChannelName) const = 0;

	/**
	 * Returns a list of all connected servers having outputs in the given channel.
	 */
	virtual TArray<FString> GetOnlineServersForChannel(const FName& InChannelName) const = 0;
	
	/**
	 * Returns the status of the playback asset on the given channel on the remote server.
	 * If the return value is not set, it is because it is not yet available, in which case
	 * a status request can be made to make it available.
	 * 
	 * This supports forked channels by allowing to specify the optional server to further precise
	 * the request. In a forked channel situation, a channel will be hosted by multiple servers
	 * and therefore, have a playback status per server. If the server name is left empty,
	 * the playback status of the first server hosting the given channel will be returned.
	 *
	 * @param InInstanceId Specify instance. If not specified, it will look for the most relevant status for the given asset.
	 * @param InAssetPath Full path of the playback asset to fetch the status of.
	 * @param InChannelName Servers can be running different channels, so need to identify the channel as well.
	 * @param InServerName Identifies the server to fetch the status from.
	 */
	virtual TOptional<EAvaPlaybackStatus> GetRemotePlaybackStatus(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FString& InServerName = FString()) const = 0;

	/**
	 * @brief Returns the user data of the specified playback instance.
	* @param InInstanceId Specify instance of the given asset.
	 * @param InAssetPath Full path of the playback asset. Must be valid.
	 * @param InChannelName Servers can be running different channels, so need to identify the channel as well.
	 * @param InServerName Identifies the server to fetch the status from.
	 * @return 
	 */
	virtual const FString* GetRemotePlaybackUserData(const FGuid& InInstanceId, const FSoftObjectPath& InAssetPath, const FString& InChannelName, const FString& InServerName = FString()) const = 0;

	/**
	 *	Returns the status of the playback asset on the given remote server.
	 *	If the return value is not set, it is because it is not yet available, in which case
	 *	a status request must be made explicitly, using RequestPlaybackAssetStatus to make it available.
	 *
	 * @param InAssetPath Full path of the playback asset to fetch the status of.
	 * @param InServerName Identifies the server to fetch the status from.
	 */
	virtual TOptional<EAvaPlaybackAssetStatus> GetRemotePlaybackAssetStatus(const FSoftObjectPath& InAssetPath, const FString& InServerName) const = 0;
};
