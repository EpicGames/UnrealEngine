// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Containers/Set.h"
#include "Math/MathFwd.h"

class UEdGraphNode;
class UEdGraph;
class UAvaPlaybackNode;
class UAvaPlaybackGraph;
class UObject;
class FSlateRect;

class IAvaPlaybackGraphEditor
{
public:
	virtual ~IAvaPlaybackGraphEditor() = default;
	
	virtual UEdGraph* CreatePlaybackGraph(UAvaPlaybackGraph* InPlayback) = 0;
	
	virtual void SetupPlaybackNode(UEdGraph* InGraph, UAvaPlaybackNode* InPlaybackNode, bool bSelectNewNode) = 0;
	
	/** Compiles Playback nodes from graph nodes. */
	virtual void CompilePlaybackNodesFromGraphNodes(UAvaPlaybackGraph* InPlayback) = 0;
	
	virtual void CreateInputPin(UEdGraphNode* InGraphNode) = 0;

	virtual void RefreshNode(UEdGraphNode& InGraphNode) = 0;

	/** Get the bounding area for the currently selected nodes
 	*
 	* @param Rect Final output bounding area, including padding
 	* @param Padding An amount of padding to add to all sides of the bounds
 	*
 	* @return false if nothing is selected
	*/
	virtual bool GetBoundsForSelectedNodes(FSlateRect& Rect, float Padding) = 0;

	virtual TSet<UObject*> GetSelectedNodes() const = 0;

	virtual bool CanPasteNodes() const = 0;
	virtual void PasteNodesHere(const FVector2D& Location) = 0;
};

#endif
