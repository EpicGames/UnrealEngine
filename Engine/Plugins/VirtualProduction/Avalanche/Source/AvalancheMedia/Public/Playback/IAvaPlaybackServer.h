// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageContext.h"
#include "UObject/SoftObjectPath.h"

class FAvaPlaybackManager;
class IAvaBroadcastSettings;
struct FAvaInstanceSettings;
struct FAvaPlayableSettings;

class IAvaPlaybackServer
{
public:
	
	struct FPlaybackInstanceReference
	{
		FGuid Id;
		FSoftObjectPath Path;
	};
	
	virtual TArray<FPlaybackInstanceReference> StopPlaybacks(const FString& InChannelName = FString(), const FSoftObjectPath& InAssetPath = FSoftObjectPath(), bool bInUnload = true) = 0;
	virtual TArray<FPlaybackInstanceReference> StartPlaybacks() = 0;
	
	virtual void StartBroadcast() = 0;
	virtual void StopBroadcast() = 0;
	
	/** Returns the server's name. */
	virtual const FString& GetName() const = 0;

	virtual bool HasUserData(const FString& InKey) const = 0;
	
	virtual const FString& GetUserData(const FString& InKey) const = 0;

	/** Add user data to this server. This is replicated and accessible to the client. */
	virtual void SetUserData(const FString& InKey, const FString& InData) = 0;

	/** Remove the server's user data entry from the given key. */
	virtual void RemoveUserData(const FString& InKey) = 0;

	/** Returns the list of connected clients. */
	virtual TArray<FString> GetClientNames() const = 0;

	/** Returns the client address. */
	virtual FMessageAddress GetClientAddress(const FString& InClientName) const = 0;

	/** Returns true if the corresponding client user data for the given client name is found. */
	virtual bool HasClientUserData(const FString& InClientName, const FString& InKey) const = 0;

	/**
	 * Returns the corresponding client user data for the given client name and key.
	 * Returns empty string if not found.
	 */
	virtual const FString& GetClientUserData(const FString& InClientName, const FString& InKey) const = 0;

	/**
	 * Access broadcast settings replicated from connected client.
	 * Will return nullptr if no clients are connected.
	 */
	virtual const IAvaBroadcastSettings* GetBroadcastSettings() const = 0;

	/**
	 * Access Motion Design Instance settings replicated from connected client(s).
	 * Will return nullptr if no clients are connected.
	 */
	virtual const FAvaInstanceSettings* GetAvaInstanceSettings() const = 0;

	/**
	 * Access Motion Design Playable settings replicated from connected client(s).
	 * Will return nullptr if no clients are connected.
	 */
	virtual const FAvaPlayableSettings* GetPlayableSettings() const = 0;
	
	/** Access the server's playback manager. */
	virtual const FAvaPlaybackManager& GetPlaybackManager() const = 0;
	virtual FAvaPlaybackManager& GetPlaybackManager() = 0;

protected:
	virtual ~IAvaPlaybackServer() = default;
};
