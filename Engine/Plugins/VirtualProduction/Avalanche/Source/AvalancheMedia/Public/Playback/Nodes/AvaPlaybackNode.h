// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "Containers/Array.h"
#include "Delegates/IDelegateInstance.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "AvaPlaybackNode.generated.h"

class FName;
class FText;
class FReferenceCollector;
class UAvaPlaybackGraph;
class UEdGraphNode;
struct FPropertyChangedEvent;

/**
 * Base Class for all the Nodes found in the Motion Design Playback Graph
 */
UCLASS(Abstract)
class AVALANCHEMEDIA_API UAvaPlaybackNode : public UObject
{
	GENERATED_BODY()

public:
	
	struct NodeCategory
	{
		static const FText Default;
		static const FText EventTrigger;
		static const FText EventAction;
		static const FText EventFlowControl;
	};
	static const int32 MaxAllowedChildNodes = 32;

	/**
	 * Called either when the Node has been Constructed (new node) or from Post Load
	 */
	virtual void PostAllocateNode();
	
	virtual void BeginDestroy() override;
	
	UAvaPlaybackGraph* GetPlayback() const;

	/**
	 * Gets the Display Name of the Node
	 */
	virtual FText GetNodeDisplayNameText() const;

	/**
	* Gets the Display Category of the Node
	*/
	virtual FText GetNodeCategoryText() const;
	

	/**
	 * Gets the ToolTip of the Node
	 */
	virtual FText GetNodeTooltipText() const;

	/**
	 * Called when the UAvaPlaybackGraph changes its state from Playing <-> Stopped
	 * @param  bPlaying The new state of Playback
	 */
	virtual void NotifyPlaybackStateChanged(bool bPlaying) {}
	
	double GetLastTimeTicked() const;
	double GetChildLastTimeTicked(int32 ChildIndex) const;

	/**
 	* Handles the 'Dry' run of the Node.
 	* This is called when Compiling the Graph.
 	* @param  InAncestors A list showing the Ancestors of this Node in order from Root to Parent (e.g. [Root, Grandparent, Parent] )
 	*/
	void DryRunNode(TArray<UAvaPlaybackNode*>& InAncestors);

	//Called before Dry Running the Graph. This is called once per node
	virtual void PreDryRun() {}

	//Called while Dry Running the Graph. This can be called multiple times for nodes that are connected to multiple nodes.
	virtual void DryRun(const TArray<UAvaPlaybackNode*>& InAncestors) {}

	//Called after Dry Running the Graph. This is called once per node
	virtual void PostDryRun() {}

	/**
	 * Handles the 'Raw' ticking of the Node.
	 * This shouldn't be called directly for ChildNodes. Instead 'TickChild' should be called
	 * to feed in extra information about the Tick
	 * @param  DeltaTime         Delta Time, in Seconds
	 * @param  ChannelParameters The aggregate group of parameters that the child nodes and this node set for Playback
	 */
	virtual void Tick(float DeltaTime, FAvaPlaybackChannelParameters& ChannelParameters);

	/**
	 * Ticks Childs as well as checking if the Child actually made connection to a Player Node
	 * If the Ticking results in valid Channel Parameters, then it calls 'NotifyChildNodeSucceeded'
	 * @param  DeltaTime         Delta Time, in Seconds
	 * @param  ChildIndex        The Index of the Child Node to Tick
	 * @param  ChannelParameters The aggregate group of parameters that the child nodes set for Playback
	 */
	void TickChild(float DeltaTime, int32 ChildIndex, FAvaPlaybackChannelParameters& ChannelParameters);

	/**
	 * Called when the Child at ChildIndex, made a connection to a Player Node successfully
	 * This updates the Last Ticked Times and stores them both in This Node and the Child Node that caused the connection
	 * @param ChildIndex The Index of the Child Node that succeeded
	 */
	void NotifyChildNodeSucceeded(int32 ChildIndex);

public:
	
	//Recursive call to get all the Nodes connected to this Node (inclusive)
	void GetAllNodes(TArray<UAvaPlaybackNode*>& OutPlaybackNodes);
	
	//UObject interface
	virtual void PostLoad() override;

#if WITH_EDITOR
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	virtual void Serialize(FArchive& Ar) override;
	virtual bool CanBeClusterRoot() const override { return false; }
	virtual bool CanBeInCluster() const override { return false; }
	//~UObject interface

#if WITH_EDITORONLY_DATA
	void SetGraphNode(UEdGraphNode* InGraphNode);
	UEdGraphNode* GetGraphNode() const;
#endif

#if WITH_EDITOR
	virtual bool EditorDryRunGraphOnNodeRefresh(FPropertyChangedEvent& PropertyChangedEvent) const { return false; }
#endif
	
	void SetChildNodes(TArray<UAvaPlaybackNode*>&& InChildNodes);
	
	virtual void ReconstructNode();
	virtual void RefreshNode(bool bDryRunGraph = false);
	
	const TArray<TObjectPtr<UAvaPlaybackNode>>& GetChildNodes() const;
	
	virtual void CreateStartingConnectors();
	virtual void InsertChildNode(int32 Index);
	virtual void RemoveChildNode(int32 Index);
	
	virtual int32 GetMinChildNodes() const { return 0; }
	virtual int32 GetMaxChildNodes() const { return 1; }

#if WITH_EDITOR
	virtual FName GetInputPinName(int32 InputPinIndex) const;
#endif
	
protected:
	UPROPERTY()
	TArray<TObjectPtr<UAvaPlaybackNode>> ChildNodes;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UEdGraphNode> GraphNode;
#endif

	FDelegateHandle PlaybackStateChangedHandle;
	
	TMap<int32, double> ChildLastValidTicks;
	
	double LastValidTick = -1.0;
};
