// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaPlaybackNode.h"
#include "Containers/Array.h"
#include "Framework/AvaSoftAssetPtr.h"
#include "AvaPlaybackNodePlayer.generated.h"

class FText;
class UTextureRenderTarget2D;
struct FAvaPlaybackChannelParameters;
struct FPropertyChangedEvent;

UCLASS(Abstract)
class AVALANCHEMEDIA_API UAvaPlaybackNodePlayer : public UAvaPlaybackNode
{
	GENERATED_BODY()

public:
	UAvaPlaybackNodePlayer();

	virtual void PostAllocateNode() override;

	virtual int32 GetMinChildNodes() const override { return 1; }
	virtual int32 GetMaxChildNodes() const override { return MaxAllowedChildNodes; }

	//Channel Ticking
	virtual void Tick(float DeltaTime, FAvaPlaybackChannelParameters& ChannelParameters) override;

	//Event Ticking
	void TickEventFeed(float DeltaTime);
	void ResetEvents();

	virtual FText GetNodeDisplayNameText() const override;
	virtual FText GetNodeTooltipText() const override;

#if WITH_EDITOR
	UTextureRenderTarget2D* GetPreviewRenderTarget() const;

	// Always Dry Run Graph on Changes in Player Node
	virtual bool EditorDryRunGraphOnNodeRefresh(FPropertyChangedEvent& PropertyChangedEvent) const override { return true; }
#endif

	virtual const FSoftObjectPath& GetAssetPath() const;

	virtual FAvaSoftAssetPtr GetAssetPtr() const { return FAvaSoftAssetPtr(); }

	virtual const FString& GetLoadOptions() const { return LoadOptions;}

	/** Get the channel indices from the last time this node was traversed. */
	const TArray<int32>& GetLastTickChannelIndices() const { return LastTickChannelIndices; }

	/** Set current load options for the asset. */
	virtual void SetLoadOptions(const FString& InLoadOptions);

protected:
	/** Additional information necessary to load an asset. */
	UPROPERTY(EditAnywhere, Category = "Motion Design")
	FString LoadOptions;
	
	FText DisplayNameText;

	/**
	 * Keep track of the channel indices that reach this node during tick traversal.
	 * This is then propagated on the event triggers to know which channel(s) the events are coming from.
	 */
	TArray<int32> ChannelIndices;
	TArray<int32> LastTickChannelIndices;
};
