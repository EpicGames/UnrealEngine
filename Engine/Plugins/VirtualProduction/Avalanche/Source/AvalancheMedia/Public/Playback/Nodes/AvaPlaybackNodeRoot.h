// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "Playback/Nodes/AvaPlaybackNode.h"
#include "AvaPlaybackNodeRoot.generated.h"

class UTextureRenderTarget2D;
struct FAvaBroadcastOutputChannel;

UCLASS()
class AVALANCHEMEDIA_API UAvaPlaybackNodeRoot : public UAvaPlaybackNode
{
	GENERATED_BODY()

public:

	virtual void PostAllocateNode() override;
	virtual void BeginDestroy() override;

	void TickRoot(float InDeltaTime, TMap<FName, FAvaPlaybackChannelParameters>& OutPlaybackSettings);

	virtual FText GetNodeDisplayNameText() const override;
	virtual FText GetNodeTooltipText() const override;	
	
	void OnBroadcastChanged(EAvaBroadcastChange InChange);
	void OnChannelChanged(const FAvaBroadcastOutputChannel& InChannel, EAvaBroadcastChannelChange InChange);
	
	virtual int32 GetMinChildNodes() const override;
	virtual int32 GetMaxChildNodes() const override;

#if WITH_EDITOR
	virtual FName GetInputPinName(int32 InputPinIndex) const override;
#endif
	
	FName GetChannelName(int32 InChannelNameIndex) const;

protected:
	
	FDelegateHandle BroadcastChangedHandle;
	FDelegateHandle ChannelChangedHandle;

	UPROPERTY()
	TArray<TObjectPtr<UTextureRenderTarget2D>> RenderTargets;
};
