// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequenceShared.h"
#include "Containers/Set.h"
#include "Playback/AvaPlaybackDefines.h"
#include "UObject/NameTypes.h"
#include "AvaPlaybackAnimations.generated.h"

USTRUCT()
struct FAvaPlaybackAnimPlaySettings
{
	GENERATED_BODY()

	FAvaPlaybackAnimPlaySettings() : FAvaPlaybackAnimPlaySettings(NAME_None) {} 
	FAvaPlaybackAnimPlaySettings(FName InAnimationName) : AnimationName(InAnimationName) {}
	
	UPROPERTY(EditAnywhere, Category = "Motion Design")
	EAvaPlaybackAnimAction Action = EAvaPlaybackAnimAction::None;
	
	UPROPERTY(VisibleAnywhere, Category = "Motion Design")
	FName AnimationName = NAME_None;

	UPROPERTY(EditAnywhere, Category = "Motion Design")
	float StartAtTime = 0.0f;

	UPROPERTY(EditAnywhere, Category = "Motion Design")
	int32 LoopCount = 0;

	UPROPERTY(EditAnywhere, Category = "Motion Design")
	EAvaSequencePlayMode PlayMode = EAvaSequencePlayMode::Forward;

	UPROPERTY(EditAnywhere, Category = "Motion Design")
	float PlaybackSpeed = 1.0f;
	
	UPROPERTY(EditAnywhere, Category = "Motion Design")
	bool bRestoreState = false;

	friend uint32 GetTypeHash(const FAvaPlaybackAnimPlaySettings& InPlaySettings)
	{
		return GetTypeHash(InPlaySettings.AnimationName);
	}

	bool operator==(const FAvaPlaybackAnimPlaySettings& Other) const
	{
		return AnimationName == Other.AnimationName;
	}

	FAvaSequencePlayParams AsPlayParams() const
	{
		FAvaSequencePlayParams PlayParams;

		PlayParams.Start    = FAvaSequenceTime(StartAtTime);
		PlayParams.PlayMode = PlayMode;

		PlayParams.AdvancedSettings.PlaybackSpeed = PlaybackSpeed;
		PlayParams.AdvancedSettings.LoopCount     = LoopCount;
		PlayParams.AdvancedSettings.bRestoreState = bRestoreState;

		return PlayParams;
	}
};

USTRUCT()
struct FAvaPlaybackAnimations
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = "Motion Design")
	TSet<FAvaPlaybackAnimPlaySettings> AvailableAnimations;
};
