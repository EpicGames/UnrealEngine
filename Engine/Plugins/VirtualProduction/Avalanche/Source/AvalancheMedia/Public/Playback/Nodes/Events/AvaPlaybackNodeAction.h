// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaPlaybackNodeEvent.h"
#include "AvaPlaybackNodeAction.generated.h"

class FText;

/*
 * Node part of the Event Nodes that handles Logic Execution
 */
UCLASS(Abstract)
class AVALANCHEMEDIA_API UAvaPlaybackNodeAction : public UAvaPlaybackNodeEvent
{
	GENERATED_BODY()

public:
	virtual FText GetNodeCategoryText() const override final;
	
	virtual int32 GetMinChildNodes() const override final { return 1; }
	virtual int32 GetMaxChildNodes() const override final { return 1; }
};
