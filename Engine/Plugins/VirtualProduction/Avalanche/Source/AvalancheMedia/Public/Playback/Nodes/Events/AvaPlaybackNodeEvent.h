// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playback/Nodes/AvaPlaybackNode.h"
#include "AvaPlaybackNodeEvent.generated.h"

struct FAvaPlaybackChannelParameters;
struct FAvaPlaybackEventParameters;

/*
 * Base Class for all Event Nodes
 */
UCLASS(Abstract)
class AVALANCHEMEDIA_API UAvaPlaybackNodeEvent : public UAvaPlaybackNode
{
	GENERATED_BODY()

public:
	//UAvaPlaybackNode Interface
	virtual void Tick(float DeltaTime, FAvaPlaybackChannelParameters& ChannelParameters) override final {}
	//~UAvaPlaybackNode Interface
	
	virtual void TickEvent(float DeltaTime, FAvaPlaybackEventParameters& OutEventParameters);
	virtual void OnEventTriggered(const FAvaPlaybackEventParameters& InEventParameters) {}
	virtual void Reset() {}
};
