// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaPlaybackNodeEvent.h"
#include "AvaPlaybackNodeTrigger.generated.h"

class FText;
struct FAvaPlaybackEventParameters;

/*
 * Node part of the Event Nodes that Listens to an Event and Triggers the Event Flow
 */
UCLASS(Abstract)
class AVALANCHEMEDIA_API UAvaPlaybackNodeTrigger : public UAvaPlaybackNodeEvent
{
	GENERATED_BODY()

public:
	virtual FText GetNodeCategoryText() const override final;
	
	virtual int32 GetMinChildNodes() const override final { return 0; }
	virtual int32 GetMaxChildNodes() const override final { return 0; }
	
	virtual void TriggerEvent();
	virtual void Reset() override final;
	virtual void TickEvent(float DeltaTime, FAvaPlaybackEventParameters& OutEventParameters) override;
	
private:
	bool bEventTriggered = false;
};
