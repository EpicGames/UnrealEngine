// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "AvaRundownDefines.h"
#include "AvaRundownPage.h"
#include "UObject/Object.h"
#include "AvaRundown.generated.h"

AVALANCHEMEDIA_API DECLARE_LOG_CATEGORY_EXTERN(LogAvaRundown, Log, All);

class FAvaPlaybackManager;
class FAvaRundownPageTransitionBuilder;
class FAvaRundownPlaybackClientWatcher;
class UAvaPlaybackGraph;
class UAvaRundownPagePlayer;
class UAvaRundownPageTransition;
class UTextureRenderTarget2D;

/**
 * @brief Defines the insertion position in a page list.
 */
struct FAvaRundownPageInsertPosition
{
	/**
	 * The position is defined with a page id because it is coupled with the id generation.
	 * For the id generator, if an insertion position is defined, we also want the generated
	 * ids to be in relation to that.
	 */
	int32 AdjacentId;
		
	/** Defines the insertion position relative to the reference page i.e. above or below the adjacent page. */
	bool bAddBelow;

	explicit FAvaRundownPageInsertPosition(int32 InAdjacentId = FAvaRundownPage::InvalidPageId, bool bInAddBelow = true) : AdjacentId(InAdjacentId), bAddBelow(bInAddBelow) {} 

	bool IsValid() const { return AdjacentId != FAvaRundownPage::InvalidPageId; }

	bool IsAddAbove() const { return !bAddBelow;}
	
	bool IsAddBelow() const { return bAddBelow;}

	/** Update the id only if it was initially valid. */
	void ConditionalUpdateAdjacentId(int32 InNewAdjacentId)
	{
		if (IsValid())
		{
			AdjacentId = InNewAdjacentId;
		}
	}
};

/**
 * Defines the parameters for the page id generator algorithm.
 * The Id generator uses a sequence strategy to search for an unused id.
 * It is defined by a starting id and a search direction.
 */
struct FAvaRundownPageIdGeneratorParams
{
	/** Starting Id for the search. */
	int32 ReferenceId;
		
	/**
	 * @brief (Initial) Search increment.
	 * @remark For negative increment search, the limit of the search space can be reached. If no unique id is found,
	 *		   the search will continue in the positive direction instead.		   
	 */
	int32 Increment;
		
	explicit FAvaRundownPageIdGeneratorParams(int32 InReferenceId = FAvaRundownPage::InvalidPageId, int32 InIncrement = 1)
		: ReferenceId(InReferenceId), Increment(InIncrement) {}

	/** Operation helper: Determines id generation from the insert parameters. */
	static FAvaRundownPageIdGeneratorParams FromInsertPosition(const FAvaRundownPageInsertPosition& InInsertPosition)
	{
		// When used with insertion in a page list, if the element is added above, we will first try to generate
		// the id in decreasing order.
		return FAvaRundownPageIdGeneratorParams( InInsertPosition.AdjacentId, InInsertPosition.bAddBelow ? 1 : -1);
	}

	/** Operation helper: Id generation prefers using insertion parameters (if specified) over source id. */
	static FAvaRundownPageIdGeneratorParams FromInsertPositionOrSourceId(int32 InSourceId, const FAvaRundownPageInsertPosition& InInsertPosition)
	{
		return InInsertPosition.IsValid() ? FromInsertPosition(InInsertPosition) : FAvaRundownPageIdGeneratorParams(InSourceId);
	}
};

struct FAvaRundownPageListChangeParams
{
	UAvaRundown* Rundown;
	FAvaRundownPageListReference PageListReference;
	EAvaRundownPageListChange ChangeType;
	TArray<int32> AffectedPages;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnAvaRundownPageListChanged, const FAvaRundownPageListChangeParams&)
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnAvaRundownPagesChanged, const UAvaRundown*, const FAvaRundownPage&, EAvaRundownPageChanges)

USTRUCT()
struct FAvaRundownPageCollection
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FAvaRundownPage> Pages;

	/** Cache mapping the Page Id to the index where the Page with such Page Id is at */
	TMap<int32, int32> PageIndices;
	
	int32 GetPageIndex(const int32 InPageId) const
	{
		if (InPageId != FAvaRundownPage::InvalidPageId)
		{
			const int32* PageIndex = PageIndices.Find(InPageId);
			return PageIndex ? *PageIndex : INDEX_NONE;
		}
		return INDEX_NONE;
	}

	AVALANCHEMEDIA_API void Empty(UAvaRundown* InRundown, const FAvaRundownPageListReference& InPageListReference);

	/** Complete refresh of the page indices. */
	void RefreshPageIndices()
	{
		PageIndices.Empty(Pages.Num());
		for (int32 Index = 0; Index < Pages.Num(); ++Index)
		{
			PageIndices.Add(Pages[Index].GetPageId(), Index);
		}
	}

	/** Refresh page indices after a new page has been inserted at the given index. */
	void PostInsertRefreshPageIndices(const int32 InStartAtIndex)
	{
		for (int32 Index = InStartAtIndex; Index < Pages.Num(); ++Index)
		{
			PageIndices.Add(Pages[Index].GetPageId(), Index);
		}
	}
};

USTRUCT()
struct FAvaRundownSubList
{
	GENERATED_BODY()

	/** Unique identifier for the page list. */
	UPROPERTY()
	FGuid Id;

	UPROPERTY()
	TArray<int32> PageIds;

	/** Display Name */
	UPROPERTY()
	FText Name;

	bool IsValid() const
	{
		return Id.IsValid();
	}
};

namespace UE::AvaRundown
{
	inline bool IsPreviewPlayType(EAvaRundownPagePlayType InPlayType)
	{
		return InPlayType == EAvaRundownPagePlayType::PreviewFromStart || InPlayType == EAvaRundownPagePlayType::PreviewFromFrame;
	}
}

/**
 * Carries the context for the playback of a page list.
 * In particular, what is the current play head to be able to move to
 * the next page in the page list.
 * Note: a "rundown" has many page lists and each page lists has 2 play type (preview or program),
 * therefore, we have many instances of this page list context for each page lists and play type within
 * a "rundown" object.
 */
struct FAvaRundownPageListPlaybackContext
{
	/** Keeps track of the page id the play head is on, i.e. this is the last played page. */
	int32 PlayHeadPageId = FAvaRundownPage::InvalidPageId;
};

/**
 * This class is a container for all the page list contexts are rundown can have.
 * The current implementation only keeps track of the play type, i.e. preview vs program.
 * The design for this is not settle yet. The requirement for a page list context per preview channel comes
 * from the rundown server as it may have a preview channel dedicated per client connection, which implies a page list
 * context for each one of them. There is only one "program" page list context for now.
 */
struct FAvaRundownPageListPlaybackContextCollection
{
	TSharedPtr<FAvaRundownPageListPlaybackContext> GetContext(bool bInIsPreview, const FName& InPreviewChannelName) const
	{
		if (bInIsPreview)
		{
			const TSharedPtr<FAvaRundownPageListPlaybackContext>* FoundContext = PreviewContexts.Find(InPreviewChannelName);
			return FoundContext ? *FoundContext : TSharedPtr<FAvaRundownPageListPlaybackContext>();
		}
		return ProgramContext;
	}

	FAvaRundownPageListPlaybackContext& GetOrCreateContext(bool bInIsPreview, const FName& InPreviewChannelName)
	{
		const TSharedPtr<FAvaRundownPageListPlaybackContext> ExistingContext = GetContext(bInIsPreview, InPreviewChannelName);
		return ExistingContext ? *ExistingContext : *CreateContext(bInIsPreview, InPreviewChannelName);
	}

protected:
	TSharedPtr<FAvaRundownPageListPlaybackContext> CreateContext(bool bInIsPreview, const FName& InPreviewChannelName)
	{
		if (bInIsPreview)
		{
			TSharedPtr<FAvaRundownPageListPlaybackContext> NewPreviewContext = MakeShared<FAvaRundownPageListPlaybackContext>(); 
			PreviewContexts.Add(InPreviewChannelName, NewPreviewContext);
			return NewPreviewContext;
		}
		ProgramContext = MakeShared<FAvaRundownPageListPlaybackContext>();
		return ProgramContext;
	}

	TSharedPtr<FAvaRundownPageListPlaybackContext> ProgramContext;
	TMap<FName, TSharedPtr<FAvaRundownPageListPlaybackContext>> PreviewContexts;
};

UENUM()
enum class EAvaRundownPageStopOptions : uint8
{
	/**
	 * Default option will stop the page with transitions if available.
	 */
	None				= 0,
	/**
	 * Forces the page to stop without transitions.
	 */
	ForceNoTransition	= 1 << 1,
	/**
	 * Default option will stop the page with transitions if available.
	 */
	Default				= None
};
ENUM_CLASS_FLAGS(EAvaRundownPageStopOptions);

/**
 * @brief Manages page pre-loading.
 */
class IAvaRundownPageLoadingManager
{
public:
	IAvaRundownPageLoadingManager() = default;
	virtual ~IAvaRundownPageLoadingManager() = default;

	virtual bool RequestLoadPage(int32 InPageId, bool bInIsPreview, const FName& InPreviewChannelName) = 0;
};

/**
 * This class is a container for what could be described as a "show" for broadcast purposes.
 *
 * It goes beyond a simple list of items. It contains the following:
 * - a list of Motion Design Template Pages (or just Templates).
 * - a list of Motion Design Instanced Pages (or just Pages).
 * - a list of page views (or just Views).
 *
 * Workflow:
 *
 * 1- Templates
 * 
 * The first step in the work flow consist in importing templates. The source asset is not actually imported
 * in the "show" container, it is just soft referenced. However, the import process will load and cache some information
 * about the template (exposed properties, default values, animations, transition logic layer, etc).
 * Given that this information is cached, it may become stale if the source asset is updated. Therefore, reimporting
 * the templates may be necessary within the normal work flow.
 * Todo: keep a hash of the source asset to determine if it has changed.
 *
 * 2- Pages
 *
 * The pages are instances of the templates, allowing to change the exposed properties and controllers, also selecting
 * an output program channel for the given page. Only one program channel is allowed per page.
 *
 * 3- Page Views
 *
 * Separate page views can be made in order to create "rundowns" for separate segments/parts of a show.
 *
 * "Page Groups" Discussion:
 * "Page Groups" are not implemented. It would be different than page views, i.e.
 * pages could be grouped in either of the page list or page views.
 * Other applications support page grouping to emulate MOS's hierarchy.
 * In the MOS/NCS hierarchies: Rundown -> Stories/Segments -> Parts -> Pieces/Items
 * Although full emulation of MOS schema may not be necessary within the Motion Design playback framework.
 *
 */
UCLASS(NotBlueprintable, BlueprintType, ClassGroup = "Motion Design Rundown", meta = (DisplayName = "Motion Design Rundown"))
class AVALANCHEMEDIA_API UAvaRundown : public UObject
{
	GENERATED_BODY()

public:
	UAvaRundown();

	virtual ~UAvaRundown() override;

	static const FAvaRundownPageListReference TemplatePageList;
	static const FAvaRundownPageListReference InstancePageList;

	static FAvaRundownPageListReference CreateSubListReference(const FGuid& InSubListId)
	{
		return {EAvaRundownPageListType::View, InSubListId};
	}

	static FAvaRundownPageListReference CreateSubListReference(const FAvaRundownSubList& InSubList)
	{
		return CreateSubListReference(InSubList.Id);
	}

protected:
	bool IsPageIdUnique(int32 InPageId) const { return !TemplatePages.PageIndices.Contains(InPageId) && !InstancedPages.PageIndices.Contains(InPageId); }
	int32 GenerateUniquePageId(int32 InReferencePageId = FAvaRundownPage::InvalidPageId, int32 InIncrement = 1) const;
	int32 GenerateUniquePageId(const FAvaRundownPageIdGeneratorParams& InParams) const;
	
	/** Caches the Page's Id to its Index in the Pages Array*/
	void RefreshPageIndices();

	void RefreshSubListIndices();
	
public:
	static const FAvaRundownSubList InvalidSubList;
	static FAvaRundownSubList InvalidSubListMutable;

	//~ Begin UObject Interface
	virtual void BeginDestroy() override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PreEditUndo() override;
	virtual void PostEditUndo() override;
	virtual void OnCookEvent(UE::Cook::ECookEvent InCookEvent, UE::Cook::FCookEventContext& InCookContext) override;
#endif
	//~ End UObject Interface

	/** Returns true if the rundown is empty, i.e. no pages and no templates. */
	bool IsEmpty() const;

	/**
	 * Clear the rundown of all it's content.
	 * @remark Will be prevented if the rundown is playing.
	 */
	bool Empty();

	int32 AddTemplateInternal(const FAvaRundownPageIdGeneratorParams& InIdGeneratorParams, const TFunctionRef<bool(FAvaRundownPage&)> InSetupTemplateFunction);
	
	/** Add empty template. */
	int32 AddTemplate(const FAvaRundownPageIdGeneratorParams& InIdGeneratorParams = FAvaRundownPageIdGeneratorParams());

	/**
	 * Creates a combination template which combines other templates.
	 * @remark This function does not validate the templates, use ValidateTemplateIdsForComboTemplate independently.
	 */ 
	int32 AddComboTemplate(const TArray<int32>& InTemplateIds, const FAvaRundownPageIdGeneratorParams& InIdGeneratorParams = FAvaRundownPageIdGeneratorParams());

	/**
	 * Combo Templates have specific requirement for the sub-templates.
	 * This function validates a given list of templates to use as sub-templates for a combo template.
	 * A combination template can only be created using transition logic templates that are on different
	 * transition layers.
	 * 
	 * @param InTemplateIds Templates to validate.
	 * @param InErrorLog Used to log the reasons for invalid templates. 
	 * @return validated template Ids.
	 */
	TArray<int32> ValidateTemplateIdsForComboTemplate(const TArray<int32>& InTemplateIds, FOutputDevice& InErrorLog);

	/**
	 * @brief Add templates from existing source.
	 * @param InSourceTemplates Source template to add.
	 * @return The new template Ids created.
	 *
	 * For the id generation, it will attempt to reuse the source ids, but
	 * in case of collision, new ids are generated with the positive increment
	 * sequence search method.
	 */
	TArray<int32> AddTemplates(const TArray<FAvaRundownPage>& InSourceTemplates);

	/**
	 * @brief Create new pages in the page list for teh given template Ids.
	 * @param InTemplateIds Templates to use for each page. A new page is created for each entry in that array.
	 * @return The new page Ids created.
	 */
	TArray<int32> AddPagesFromTemplates(const TArray<int32>& InTemplateIds);

	/**
	 * @brief Creates a new page in the page list using the given template.
	 * @param InTemplateId Reference to the template to use for that page.
	 * @param InIdGeneratorParams Defines how the page id is going to be generated.
	 * @param InInsertAt Specifies the insertion location in the page list (i.e. the index in the page list).
	 * @return Returns the page Id of the created page.
	 */
	int32 AddPageFromTemplate(int32 InTemplateId, const FAvaRundownPageIdGeneratorParams& InIdGeneratorParams = FAvaRundownPageIdGeneratorParams(), const FAvaRundownPageInsertPosition& InInsertAt = FAvaRundownPageInsertPosition());

	bool CanAddPage() const;

	bool CanChangePageOrder() const;
	/** Reorders the pages, swapping the old indices for the new ones. */
	bool ChangePageOrder(const FAvaRundownPageListReference& InPageListReference, const TArray<int32>& InPageIndices);

	bool RemovePage(int32 InPageId);
	bool CanRemovePage(int32 InPageId) const;

	/** Remove all the Pages in the Array. Returns the number of Pages removed.*/
	int32 RemovePages(const TArray<int32>& InPageIds);
	bool CanRemovePages(const TArray<int32>& InPageIds) const;

	bool RenumberPageIds(const TArray<int32>& InPageIds, const FAvaRundownPageIdGeneratorParams& InIdParams);
	bool RenumberPageId(int32 InPageId, int32 InNewPageId);
	bool CanRenumberPageId(int32 InPageId) const;
	bool CanRenumberPageId(int32 InPageId, int32 InNewPageId) const;

	bool SetRemoteControlEntityValue(int32 InPageId, const FGuid& InId, const FAvaPlayableRemoteControlValue& InValue);
	bool SetRemoteControlControllerValue(int32 InPageId, const FGuid& InId,const FAvaPlayableRemoteControlValue& InValue);
	EAvaPlayableRemoteControlChanges UpdateRemoteControlValues(int32 InPageId, const FAvaPlayableRemoteControlValues& InRemoteControlValues, bool bInUpdateDefaults);
	
	EAvaPlayableRemoteControlChanges ResetRemoteControlValues(int32 InPageId, bool bInUseTemplateValues, bool bInIsDefault);
	EAvaPlayableRemoteControlChanges ResetRemoteControlControllerValue(int32 InPageId, const FGuid& InControllerId, bool bInUseTemplateValues, bool bInIsDefault);
	EAvaPlayableRemoteControlChanges ResetRemoteControlEntityValue(int32 InPageId, const FGuid& InEntityId, bool bInUseTemplateValues, bool bInIsDefault);

	void InvalidateManagedInstanceCacheForPages(const TArray<int32>& InPageIds) const;

	void UpdateAssetForPages(const TArray<int32>& InPageIds, bool bInReimportPage);
	
	const FAvaRundownPage& GetPage(int32 InPageId) const;
	FAvaRundownPage& GetPage(int32 InPageId);
	const FAvaRundownPageCollection& GetTemplatePages() const { return TemplatePages; }
	const FAvaRundownPageCollection& GetInstancedPages() const { return InstancedPages; }
	static const FAvaRundownPage& GetPageSafe(const UAvaRundown* InRundown, int32 InPageId)
	{
		return InRundown ? InRundown->GetPage(InPageId) : FAvaRundownPage::NullPage;
	}
	/**
	 * Gets the page following the page with the given page id in the given page list.
	 * If the given page is not in the given list, the returned page is invalid.
	 */
	const FAvaRundownPage& GetNextPage(int32 InPageId, const FAvaRundownPageListReference& InPageListReference) const;
	
	/**
	 * Gets the page following the page with the given page id in the given page list.
	 * If the given page is not in the given list, the returned page is invalid.
	 */
	FAvaRundownPage& GetNextPage(int32 InPageId, const FAvaRundownPageListReference& InPageListReference);

	/** Gets the page following the page with the given page id. Using current active page list. */
	const FAvaRundownPage& GetNextPage(int32 InPageId) const { return GetNextPage(InPageId, ActivePageList);}

	/** Gets the page following the page with the given page id. Using current active page list. */
	FAvaRundownPage& GetNextPage(int32 InPageId)  { return GetNextPage(InPageId, ActivePageList);}
	
	FOnAvaRundownPageListChanged& GetOnPageListChanged() { return OnPageListChanged; }
	FOnAvaRundownPagesChanged& GetOnPagesChanged() { return OnPagesChanged; }

	/** Delegate called to determine if the playback context can be closed. */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnCanClosePlaybackContext, const UAvaRundown*, bool& /*bOutResult*/);
	FOnCanClosePlaybackContext::RegistrationType& GetOnCanClosePlaybackContext() const { return OnCanClosePlaybackContext; }

	/**
	 * Since the playback context is part of the asset for now, there is an explicit call to initialize it.
	 * This would be done by the editor (or server).
	 * A future refactor will extract the rundown "player" functionality in another class and it will
	 * be possible to create multiple instance of a rundown player for the same rundown.
	 */
	void InitializePlaybackContext();

	/**
	 * Determines if the playback context can be closed.  
	 */
	bool CanClosePlaybackContext() const;
	
	/**
	 * Similarly, when the editor is done, it can close the playback context. 
	 * This will clean up the internal structures for playback and optionally stop all the pages.
	 */
	void ClosePlaybackContext(bool bInStopAllPages);

	/**
	 * Returns true if the any page is either playing or previewing.
	 */
	bool IsPlaying() const;

	/** Returns true if the page with the given page Id is being previewed (in any preview channel). */
	bool IsPagePreviewing(int32 InPageId) const;

	/** Returns true if the page is playing in it's assigned program channel. */
	bool IsPagePlaying(int32 InPageId) const;
	
	bool IsPagePlaying(const FAvaRundownPage& InPage) const { return IsPagePlaying(InPage.GetPageId()); }

	bool IsPagePlayingOrPreviewing(int32 InPageId) const;

	bool UnloadPage(int32 InPageId, const FString& InChannelName);

	struct FLoadedInstanceInfo
	{
		FGuid InstanceId;
		FSoftObjectPath AssetPath;
	};
	
	/**
	 * @brief Preload the given page so it has an asset ready for playback.
	 * @return UUIDs (and asset paths) of the playback instances that where loaded (or are scheduled for loading).
	 */
	TArray<FLoadedInstanceInfo> LoadPage(int32 InPageId,  bool bInPreview, const FName& InPreviewChannelName);

	/**
	 * @brief Start the playback of the asset defined in the given pages.
	 * @remark If the play type is a preview, the default preview channel is used.
	 * @param InPageIds Rundown's pages to play out.
	 * @param InPlayType Either play on program or preview
	 * @param InSource the source that requested to play pages
	 * @return page Ids that where started.
	 */
	TArray<int32> PlayPages(const TArray<int32>& InPageIds, EAvaRundownPagePlayType InPlayType, EAvaRundownPagePlaySource InSource = EAvaRundownPagePlaySource::Default);

	/**
	 * @brief Start the playback of the asset defined in the given pages.
	 * @remark If the play type is a preview, the given preview channel is used.
	 * @param InPageIds Rundown's pages to play out.
	 * @param InPlayType Either play on program or preview
	 * @param InPreviewChannelName Channel to use for preview. Only used if play type is preview.
	 * @param InSource the source that requested to play pages
	 * @return page Ids that where started.
	 */
	TArray<int32> PlayPages(const TArray<int32>& InPageIds, EAvaRundownPagePlayType InPlayType, const FName& InPreviewChannelName, EAvaRundownPagePlaySource InSource = EAvaRundownPagePlaySource::Default);

	bool PlayPage(int32 InPageId, EAvaRundownPagePlayType InPlayType, EAvaRundownPagePlaySource InSource = EAvaRundownPagePlaySource::Default);
	bool PlayPage(int32 InPageId, EAvaRundownPagePlayType InPlayType, const FName& InPreviewChannelName, EAvaRundownPagePlaySource InSource = EAvaRundownPagePlaySource::Default);

	bool CanPlayPage(int32 InPageId, bool bInPreview) const;
	bool CanPlayPage(int32 InPageId, bool bInPreview, const FName& InPreviewChannelName, FString* OutFailureReason) const;

	TArray<int32> StopPages(const TArray<int32>& InPageIds, EAvaRundownPageStopOptions InOptions, bool bInPreview);
	TArray<int32> StopPages(const TArray<int32>& InPageIds, EAvaRundownPageStopOptions InOptions, bool bInPreview, const FName& InPreviewChannelName);

	bool StopPage(int32 InPageId, EAvaRundownPageStopOptions InOptions, bool bInPreview) { return !StopPages({InPageId}, InOptions, bInPreview).IsEmpty(); }
	bool StopPage(int32 InPageId, EAvaRundownPageStopOptions InOptions, bool bInPreview, const FName& InPreviewChannelName) { return !StopPages({InPageId}, InOptions, bInPreview, InPreviewChannelName).IsEmpty(); }
	
	bool CanStopPage(int32 InPageId, EAvaRundownPageStopOptions InOptions, bool bInPreview) const;
	bool CanStopPage(int32 InPageId, EAvaRundownPageStopOptions InOptions, bool bInPreview, const FName& InPreviewChannelName, FString* OutFailureReason) const;

	/**
	 * Stop the layers of a given channel.
	 * If using transition, it will produce a special transition that has only kick out layers.
	 *
	 * returns the pages that will be affected (totally or partially stopped).
	 */
	TArray<int32> StopLayers(FName InChannelName, const TArray<FAvaTagHandle>& InLayers, EAvaRundownPageStopOptions InOptions);

	/** Determines if a given layer has something playing that can be stopped. */
	bool CanStopLayer(FName InChannelName, const FAvaTagHandle& InLayer) const;
	
	bool StopChannel(const FString& InChannelName);
	bool CanStopChannel(const FString& InChannelName) const;

	bool ContinuePage(int32 InPageId, bool bInPreview);
	bool ContinuePage(int32 InPageId, bool bInPreview, const FName& InPreviewChannelName);

	bool CanContinuePage(int32 InPageId, bool bInPreview) const;
	bool CanContinuePage(int32 InPageId, bool bInPreview, const FName& InPreviewChannelName) const;

	/** Used to reconcile playing state with a remote playback if connection was lost. */
	bool RestorePlaySubPage(int32 InPageId, int32 InSubPageIndex, const FGuid& InExistingInstanceId, bool bInIsPreview, const FName& InPreviewChannelName);

	FAvaRundownPageListReference AddSubList();
	
	bool RemoveSubList(const FAvaRundownPageListReference& InPageListReference);
	
	bool RenameSubList(const FAvaRundownPageListReference& InPageListReference, const FText& InNewName);

	/**
	 * Return the current playing Page Ids on the specified program channel.
	 * If program channel is none, returns all playing pages on all program channels. 
	 */
	TArray<int32> GetPlayingPageIds(const FName InProgramChannelName = NAME_None) const;

	/**
	 * Return the current previewing Page Ids on the specified preview channel.
	 * If preview channel is none, returns all previewing pages on all channels. 
	 */
	TArray<int32> GetPreviewingPageIds(const FName InPreviewChannelName = NAME_None) const;

	const FAvaRundownPageListReference& GetActivePageListReference() const { return ActivePageList; }

	bool SetActivePageList(const FAvaRundownPageListReference& InPageListReference);

	/** Returns true only if a sub list, not the main list, is active. */
	bool HasActiveSubList() const;

	const FAvaRundownSubList& GetActiveSubList() const { return GetSubList(ActivePageList.SubListId); }
	FAvaRundownSubList& GetActiveSubList() { return GetSubList(ActivePageList.SubListId); }

	const FAvaRundownSubList& GetSubList(int32 InSubListIndex) const;
	FAvaRundownSubList& GetSubList(int32 InSubListIndex);

	const FAvaRundownSubList& GetSubList(const FGuid& InSubListId) const;
	FAvaRundownSubList& GetSubList(const FGuid& InSubListId);

	const FAvaRundownSubList& GetSubList(const FAvaRundownPageListReference& InPageListReference) const
	{
		return InPageListReference.Type == EAvaRundownPageListType::View ? GetSubList(InPageListReference.SubListId) : InvalidSubList;
	}
	
	FAvaRundownSubList& GetSubList(const FAvaRundownPageListReference& InPageListReference)
	{
		return InPageListReference.Type == EAvaRundownPageListType::View ? GetSubList(InPageListReference.SubListId) : InvalidSubListMutable;
	}

	int32 GetSubListIndex(const FAvaRundownSubList& InSubList) const;

	bool IsValidSubList(const FAvaRundownPageListReference& InPageListReference) const;

	const TArray<FAvaRundownSubList>& GetSubLists() const { return SubLists; }

	bool AddPageToSubList(const FAvaRundownPageListReference& InPageListReference, int32 InPageId, const FAvaRundownPageInsertPosition& InInsertPosition = FAvaRundownPageInsertPosition());
	bool AddPagesToSubList(const FAvaRundownPageListReference& InPageListReference, const TArray<int32>& InPages);

	int32 RemovePagesFromSubList(const FAvaRundownPageListReference& InPageListReference, const TArray<int32>& InPages);

	DECLARE_MULTICAST_DELEGATE(FOnActiveListChanged)
	FOnActiveListChanged& GetOnActiveListChanged() { return OnActiveListChanged; }

	UTextureRenderTarget2D* GetPreviewRenderTarget() const { return GetPreviewRenderTarget(GetDefaultPreviewChannelName());}
	UTextureRenderTarget2D* GetPreviewRenderTarget(const FName& InPreviewChannel) const;

	/** Returns the currently selected preview channel (from the settings). */
	static FName GetDefaultPreviewChannelName();

	/** Clean up playing status on a system tear down. */
	void OnParentWordBeginTearDown();

	/**
	 * Push an RC event on the given controllers for the given page.
	 * The propagation to channels in determined by rundown settings.
	 * 
	 * @param InControllerIds List of rc controller ids that should be triggered.
	 * @param InPageId Page id these events apply to.
	 * @param bInPushToProgram if true, the command will be pushed to page's program channel (if currently playing).
	 * @param bInPushToPreview if true, the command will be pushed to preview channel(s).
	 * @param InPreviewChannelName Allow to specify a specific preview channel to push to. If left none, will push to all preview channel this page is playing in.
	 * @return true if the event command was pushed to the corresponding playback instance.
	 */
	bool PushRuntimeRemoteControlEvents(TConstArrayView<FGuid> InControllerIds, int32 InPageId, bool bInPushToProgram, bool bInPushToPreview, FName InPreviewChannelName = NAME_None);
	
	/**
	 * Push the page's RC values to the runtime playback instances.
	 * @remark If updating preview, will update all preview channels by default. 
	 */
	bool PushRuntimeRemoteControlValues(int32 InPageId, bool bInIsPreview, const FName& InPreviewChannelName = NAME_None) const;
	
	void NotifyPageRemoteControlValueChanged(int32 InPageId, EAvaPlayableRemoteControlChanges InRemoteControlChanges);
	
	void NotifyPageStopped(int32 InPageId) const
	{
		OnPagesChanged.Broadcast(this, GetPage(InPageId), EAvaRundownPageChanges::Status);
	}
	
	void NotifyPageSequenceFinished(int32 InPageId)
	{
		OnPagesChanged.Broadcast(this, GetPage(InPageId), EAvaRundownPageChanges::Status);
	}

#if WITH_EDITOR
	void NotifyPIEEnded(const bool);
#endif

	FAvaPlaybackManager& GetPlaybackManager() const;

	/** Access the page loading manager for this rundown. */
	IAvaRundownPageLoadingManager& GetPageLoadingManager()
	{
		return PageLoadingManager ? *PageLoadingManager : MakePageLoadingManager();
	}

protected:
	int32 AddPageFromTemplateInternal(int32 InTemplateId, const FAvaRundownPageIdGeneratorParams& InIdGeneratorParams = FAvaRundownPageIdGeneratorParams(), const FAvaRundownPageInsertPosition& InInsertAt = FAvaRundownPageInsertPosition());
	
	void InitializePage(FAvaRundownPage& InOutPage, int32 InPageId, int32 InTemplateId) const;

	/**
	 * Returns true if the selected page can play on the given channel.
	 * This enforces that the channel type (program or preview) is compatible with the requested operation.
	 */
	bool IsChannelTypeCompatibleForRequest(const FAvaRundownPage& InSelectedPage, bool bInIsPreview, const FName& InPreviewChannelName, FString* OutFailureReason) const;

private:
	void OnPlayPages(const TArray<int32>& InPlayedPageIds, EAvaRundownPagePlayType InPlayType, FName InPreviewChannelName, EAvaRundownPagePlaySource InSource);

	IAvaRundownPageLoadingManager& MakePageLoadingManager();

	bool PlayPageWithTransition(FAvaRundownPageTransitionBuilder& InBuilder, const FAvaRundownPage& InPage, EAvaRundownPagePlayType InPlayType, bool bInIsPreview, const FName& InPreviewChannelName);
	bool StopPageNoTransition(const FAvaRundownPage& InPage, bool bInPreview, const FName& InPreviewChannelName);
	bool StopPageWithTransition(FAvaRundownPageTransitionBuilder& InBuilder, const FAvaRundownPage& InPage, bool bInPreview, const FName& InPreviewChannelName);
	
	const FAvaRundownPage& GetNextFromPages(const TArray<FAvaRundownPage>& InPages, int32 InStartingIndex) const;
	FAvaRundownPage& GetNextFromPages(TArray<FAvaRundownPage>& InPages, int32 InStartingIndex) const;

	const FAvaRundownPage& GetNextFromSubList(const TArray<int32>& InSubListIds, int32 InStartingIndex) const;
	FAvaRundownPage& GetNextFromSubList(const TArray<int32>& InSubListIds, int32 InStartingIndex);

protected:
	UPROPERTY()
	TArray<FAvaRundownPage> Pages_DEPRECATED;

	UPROPERTY()
	FAvaRundownPageCollection TemplatePages;

	UPROPERTY()
	FAvaRundownPageCollection InstancedPages;

	UPROPERTY()
	TArray<FAvaRundownSubList> SubLists;

	/** ==InstancePageList Indicates that the entire list is being played, rather than a specific view. */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	FAvaRundownPageListReference ActivePageList = InstancePageList;

	/**
	 * Keeping track of playing pages.
	 * 
	 * Note: For the rundown editor, we can only preview one page on the selected preview channel,
	 * however, rundown server will eventually require that more than one page is previewed on different preview channels
	 * for different connected clients.
	 */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TArray<TObjectPtr<UAvaRundownPagePlayer>> PagePlayers;
	
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TArray<TObjectPtr<UAvaRundownPageTransition>> PageTransitions;

	TUniquePtr<FAvaRundownPageListPlaybackContextCollection> PageListPlaybackContextCollection;

	TUniquePtr<IAvaRundownPageLoadingManager> PageLoadingManager;
	
	/**
	 * Playback Client Watcher ensures external playback events are reconciled.
	 */
	friend class FAvaRundownPlaybackClientWatcher;
	TPimplPtr<FAvaRundownPlaybackClientWatcher> PlaybackClientWatcher;

protected:
	void AddPagePlayer(UAvaRundownPagePlayer* InPagePlayer);
	
public:
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPagePlayerEvent, UAvaRundown*, UAvaRundownPagePlayer*);

	/**
	 * Rundown Player Event - Called when a new page player is added to the playback context.
	 */
	FOnPagePlayerEvent OnPagePlayerAdded;
	
	/**
	 * Rundown Player Event - Called when a stopped page player is about to be removed from the playback context.
	 */
	FOnPagePlayerEvent OnPagePlayerRemoving;

	FOnPagePlayerEvent& GetOnPagePlayerAdded() { return OnPagePlayerAdded; }
	FOnPagePlayerEvent& GetOnPagePlayerRemoving() { return OnPagePlayerRemoving; }

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPageTransitionEvent, UAvaRundown*, UAvaRundownPageTransition*);

	/**
	 * Rundown Transition Event - Called when a new page transition is added to the playback context.
	 */
	FOnPageTransitionEvent OnPageTransitionAdded;

	/**
	 * Rundown Transition Event - Called when a completed page transition is about to be removed from the playback context.
	 */
	FOnPageTransitionEvent OnPageTransitionRemoving;

	FOnPageTransitionEvent& GetOnPageTransitionAdded() { return OnPageTransitionAdded; }
	FOnPageTransitionEvent& GetOnPageTransitionRemoving() { return OnPageTransitionRemoving; }
	
	void AddPageTransition(UAvaRundownPageTransition* InPageTransition)
	{
		PageTransitions.Add(InPageTransition);
		OnPageTransitionAdded.Broadcast(this, InPageTransition);
	}

	void RemovePageTransition(UAvaRundownPageTransition* InPageTransition)
	{
		OnPageTransitionRemoving.Broadcast(this, InPageTransition);
		PageTransitions.Remove(InPageTransition);
	}

	UAvaRundownPageTransition* GetPageTransition(const FGuid& InTransitionId) const;

	bool CanStartTransitionForPage(const FAvaRundownPage& InPage, bool bInIsPreview, const FName& InPreviewChannelName, FString* OutFailureReason) const;
	
	/**
	 * Stop transitions involving the given page playing in the specified channel.
	 * @return number of transitions stopped.
	 */
	int32 StopPageTransitionsForPage(const FAvaRundownPage& InPage, bool bInIsPreview, const FName& InPreviewChannelName);

	/**
	 * Stop all current transitions in the specified channel.
	 * @return number of transitions stopped.
	 */
	int32 StopPageTransitionsForChannel(const FName InChannelName);

	/**
	 * Stop transitions that return true for the given predicate.
	 * @return number of transitions stopped.
	 */
	int32 StopPageTransitionsByPredicate(TFunctionRef<bool(UAvaRundownPageTransition*)> InPredicate);
	
	const TArray<TObjectPtr<UAvaRundownPagePlayer>>& GetPagePlayers() const { return PagePlayers; }
	
	UAvaRundownPagePlayer* FindPlayerForProgramPage(int32 InPageId) const;

	UAvaRundownPagePlayer* FindPlayerForPreviewPage(int32 InPageId, const FName& InPreviewChannelFName) const;
	
	UAvaRundownPagePlayer* FindPlayerForPage(int32 InPageId, bool bInIsPreview, const FName& InPreviewChannelName) const
	{
		return bInIsPreview ?  FindPlayerForPreviewPage(InPageId, InPreviewChannelName) : FindPlayerForProgramPage(InPageId);
	}

	/** Finds the page player for the given pageId and channel name. */
	UAvaRundownPagePlayer* FindPagePlayer(int32 InPageId, FName InChannelName) const;

	void RemoveStoppedPagePlayers();

	FAvaRundownPageListPlaybackContextCollection* GetPageListPlaybackContextCollection() const
	{
		return PageListPlaybackContextCollection.Get();
	}

	FAvaRundownPageListPlaybackContextCollection& GetOrCreatePageListPlaybackContextCollection()
	{
		if (!PageListPlaybackContextCollection.IsValid())
		{
			PageListPlaybackContextCollection = MakeUnique<FAvaRundownPageListPlaybackContextCollection>();
		}
		check(PageListPlaybackContextCollection.IsValid());
		return *PageListPlaybackContextCollection;
	}

protected:
	FOnAvaRundownPagesChanged OnPagesChanged;
	FOnAvaRundownPageListChanged OnPageListChanged;
	FOnActiveListChanged OnActiveListChanged;
	mutable FOnCanClosePlaybackContext OnCanClosePlaybackContext;

	TMap<FGuid, int32> SubListIndices;

#if WITH_EDITOR
	class FPreUndoBackup;
	TPimplPtr<FPreUndoBackup> PreUndoBackup;
#endif
};
