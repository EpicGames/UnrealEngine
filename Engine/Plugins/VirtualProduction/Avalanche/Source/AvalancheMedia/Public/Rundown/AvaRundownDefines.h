// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaRundownDefines.generated.h"

UENUM(BlueprintType, DisplayName = "Motion Design Rundown Page Play Type")
enum class EAvaRundownPagePlayType : uint8
{
	PlayFromStart,
	PreviewFromStart,
	PreviewFromFrame
};

/** Indicates the source that initiated play pages */
UENUM()
enum class EAvaRundownPagePlaySource : uint8
{
	/** Undefined source */
	Undefined,
	/** The rundown editor initiated the request */
	Editor,
	/** The rundown server initiated the request*/
	Server,
	/** The rundown component initiated the request */
	Component,

	Default = Undefined,
};

namespace UE::AvaMedia::Rundown
{
	enum { InvalidPageId = -1};
}