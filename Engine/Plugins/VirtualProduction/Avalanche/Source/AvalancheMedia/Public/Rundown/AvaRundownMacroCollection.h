// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/InputChord.h"
#include "UObject/Object.h"
#include "AvaRundownMacroCollection.generated.h"

USTRUCT()
struct FAvaRundownMacroCommand
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Command")
	FName Name;

	UPROPERTY(EditAnywhere, Category="Command")
	FString Arguments;
};

/**
 * Defines a macro (list of commands) that is assigned to a shortcut key.
 */
USTRUCT()
struct FAvaRundownMacroKeyBinding
{
	GENERATED_BODY()

	/** Macro description */
	UPROPERTY(EditAnywhere, Category="Macro")
	FString Description;

	/** Input Key this Macro is bound to. */
	UPROPERTY(EditAnywhere, Category="Key")
	FInputChord InputChord;

	/** List of commands this macro will run. */
	UPROPERTY(EditAnywhere, Category="Commands")
	TArray<FAvaRundownMacroCommand> Commands;
};

/**
 * Collection of rundown commands and bindings (either keys or other).
 */
UCLASS(NotBlueprintable, BlueprintType, ClassGroup = "Motion Design Rundown", 
	meta = (DisplayName = "Motion Design Rundown Macro Collection"))
class AVALANCHEMEDIA_API UAvaRundownMacroCollection : public UObject
{
	GENERATED_BODY()

public:
	bool HasBindingFor(const FInputChord& InInputChord) const; 

	int32 ForEachBinding(const FInputChord& InInputChord, TFunctionRef<bool(const FAvaRundownMacroKeyBinding&)> InCallback) const;
	int32 ForEachCommand(const FInputChord& InInputChord, TFunctionRef<bool(const FAvaRundownMacroCommand&)> InCallback) const;

protected:
	UPROPERTY(EditAnywhere, Category="Marcos")
	TArray<FAvaRundownMacroKeyBinding> KeyBindings;
};
