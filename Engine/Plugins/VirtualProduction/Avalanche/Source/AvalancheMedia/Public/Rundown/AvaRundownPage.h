// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "AvaTagHandle.h"
#include "AvaTransitionEnums.h"
#include "Playable/AvaPlayableRemoteControlValues.h"
#include "StructUtils/InstancedStruct.h"

#include "AvaRundownPage.generated.h"

class UAvaRundown;
class URemoteControlPreset;
struct FAvaRundownPageCommand;

UENUM()
enum class EAvaRundownPageStatus : uint8
{
	/** Invalid page status. */
	Unknown = 0,
	/** Output is offline. */
	Offline,
	/** When the page is not available, i.e. the asset is not present in the local content. */
	Missing,
	/** Out of date */
	NeedsSync,
	/** Asset is being downloaded. */
	Syncing,
	/** When the page is present in local content, but not loaded. */
	Available,
	/** Load/Start has been requested. */
	Loading,
	/** Page is loaded in memory and ready to play. */
	Loaded,
	/** Page is currently playing in an output channel.*/
	Playing,
	/** Page is currently playing as local preview.*/
	Previewing,
	/** Something bad happened. */
	Error
};

USTRUCT()
struct FAvaRundownChannelPageStatus
{
	GENERATED_BODY()

	UPROPERTY()
	EAvaBroadcastChannelType Type = EAvaBroadcastChannelType::Program;
	
	UPROPERTY()
	EAvaRundownPageStatus Status = EAvaRundownPageStatus::Unknown;
	
	UPROPERTY()
	bool bNeedsSync = false;
};

/** Page command data is stored in json serialized string in the page to be compatible with external apps. */
USTRUCT()
struct FAvaRundownPageCommandData
{
	GENERATED_BODY()

	/** Command name: string. */
	UPROPERTY(VisibleAnywhere, Category = "Motion Design")
	FString Name;

	/** Command payload: json formatted string. */
	UPROPERTY(VisibleAnywhere, Category = "Motion Design")
	FString Payload;
};

USTRUCT(BlueprintType, DisplayName = "Motion Design Rundown Page")
struct AVALANCHEMEDIA_API FAvaRundownPage
{
	friend UAvaRundown;
	GENERATED_BODY()

public:
	static FAvaRundownPage NullPage;
	static const int32 InvalidPageId;

	static bool StatusesContainsStatus(const TArray<FAvaRundownChannelPageStatus>& InStatuses, const TArray<EAvaRundownPageStatus>& InStatusEnums)
	{
		for (const FAvaRundownChannelPageStatus& Status : InStatuses)
		{
			if (InStatusEnums.Contains(Status.Status))
			{
				return true;
			}
		}

		return false;
	}
	
	FAvaRundownPage(int32 InPageId = InvalidPageId, int32 InTemplateId = InvalidPageId);

	bool IsValidPage() const;

	void Rename(const FString& InNewName);
	void RenameFriendlyName(const FString& InNewName);
	
	int32 GetPageId() const { return PageId; }
	void SetPageId(int32 InPageId) { PageId = InPageId; } // Do not use lightly.

	int32 GetTemplateId() const { return TemplateId; }
	void SetTemplateId(int32 InTemplateId) { TemplateId = InTemplateId; } // Do not use lightly.
	bool IsTemplate() const { return TemplateId == InvalidPageId; }

	const TArray<int32>& GetCombinedTemplateIds() const { return CombinedTemplateIds; }
	bool IsComboTemplate() const { return IsTemplate() && !CombinedTemplateIds.IsEmpty();}
	
	const TSet<int32>& GetInstancedIds() const { return Instances; }

	void ResetInstancedIds() { Instances.Reset(); }	// For internal use only.

	const FString& GetPageName() const { return PageName; }
	void SetPageName(const FString& InPageName) { PageName = InPageName; }

	bool HasPageSummary() const { return !PageSummary.IsEmptyOrWhitespace(); }
	FText GetPageSummary() const { return PageSummary; }

	bool HasPageFriendlyName() const { return !FriendlyName.IsEmptyOrWhitespace(); }
	void SetPageFriendlyName(const FText& InPageFriendlyName) { FriendlyName = InPageFriendlyName; }
	FText GetPageFriendlyName() const { return FriendlyName; }

	FText GetPageDescription() const;

	/**
	 * @brief Update the page summary (only if summary is empty)
	 * @param InRundown Owning rundown.
	 * @return true if the summary was generated, false otherwise (either because of an error or it was not needed).
	 */
	bool UpdatePageSummary(const UAvaRundown* InRundown);

	UE_DEPRECATED(5.6, "Use the version that provides the rundown." )
	bool UpdatePageSummary(const TArray<const URemoteControlPreset*>& InPresets, bool bInIsPresetChanged = false);

	/**
	 * @brief Update the page summary from controller values or commands.
	 * @param InRundown Owning rundown. Necessary to resolve templates.
	 * @param InPresets Input presets to get the value from.
	 * @param bInForceUpdate Forces the update of the summary even if there is already one. If false, summary is generated only if empty. 
	 * @return true if the summary was generated, false otherwise (either because of an error or it was not needed).
	 */
	bool UpdatePageSummary(const UAvaRundown* InRundown, const TArray<const URemoteControlPreset*>& InPresets, bool bInForceUpdate = false);

	bool UpdateTransitionLogic();
	bool HasTransitionLogic(const UAvaRundown* InRundown) const;
	
	FAvaTagHandle GetTransitionLayer(const UAvaRundown* InRundown, int32 InTemplateIndex = 0) const;
	TArray<FAvaTagHandle> GetTransitionLayers(const UAvaRundown* InRundown) const;

	EAvaTransitionInstancingMode GetTransitionMode( const UAvaRundown* InRundown, int32 InTemplateIndex = 0) const;
	TArray<EAvaTransitionInstancingMode> GetTransitionModes(const UAvaRundown* InRundown) const;
	
	/**
	 * Appends the page's program status(es).
	 * Supports multiple statuses per page for forked channels (one status per server including local).
	 * The order of the statuses is the same as the outputs (but only once per server) in the channel.
	 * @remark Nothing is added if the page is a template.
	 * @return The number of program statuses added.
	 */
	int32 AppendPageProgramStatuses(const UAvaRundown* InParentRundown, TArray<FAvaRundownChannelPageStatus>& OutPageStatuses) const;
	
	/**
	 * Appends the page's preview status(es) for the given preview channel.
	 * Remote channels are not yet supported for preview channels, only local for now.
	 * @remark If the preview channel is not specified, it will use the rundown's default preview channel.
	 * @remark this api is intended to query extra preview channels (for rundown server).
	 * @return The number of preview statuses added.
	 */
	int32 AppendPagePreviewStatuses(const UAvaRundown* InParentRundown, const FName& InPreviewChannelName, TArray<FAvaRundownChannelPageStatus>& OutPageStatuses) const;

	/**
	 * Returns all the page's "standard" (program and preview) playback statuses. 
	 * The order of the status is the same as the outputs in the channel
	 * and the preview status is always last.
	 * Template pages only have the preview status added.
	 */
	TArray<FAvaRundownChannelPageStatus> GetPageStatuses(const UAvaRundown* InParentRundown) const;

	/**
	 * Returns either the program statuses or preview statuses only depending if the page is instanced (has a program channel),
	 * or a template respectively.
	 */
	TArray<FAvaRundownChannelPageStatus> GetPageContextualStatuses(const UAvaRundown* InParentRundown) const;

	/**
	 * Returns the page's program statuses.
	 * Returns nothing for template pages.
	 */
	TArray<FAvaRundownChannelPageStatus> GetPageProgramStatuses(const UAvaRundown* InParentRundown) const;

	/**
	 * Returns the page's preview statuses for the given preview channel.
	 * If the preview channel is not specified, it will use the rundown's default preview channel.
	 */
	TArray<FAvaRundownChannelPageStatus> GetPagePreviewStatuses(const UAvaRundown* InParentRundown, const FName& InPreviewChannelName = NAME_None) const;
	
	bool IsEnabled() const { return bEnabled; }
	void SetEnabled(bool bInEnabled) { bEnabled = bInEnabled; }

	FSoftObjectPath GetAssetPath(const UAvaRundown* InRundown, int32 InTemplateIndex = 0) const;
	TArray<FSoftObjectPath> GetAssetPaths(const UAvaRundown* InRundown) const;
	FSoftObjectPath GetAssetPathDirect() const { return AssetPath;}

	/** Returns true if the page has some valid assets. */
	bool HasAssets(const UAvaRundown& InRundown) const;

	bool UpdateAsset(const FSoftObjectPath& InAssetPath, bool bInReimportPage = false);

	/** Returns true if the page (or it's template(s)) has commands. */
	bool HasCommands(const UAvaRundown* InRundown) const;

	FName GetChannelName() const;
	int32 GetChannelIndex() const { return OutputChannel; }
	void SetChannelName(FName InChannelName);

	EAvaPlayableRemoteControlChanges PruneRemoteControlValues(const FAvaPlayableRemoteControlValues& InRemoteControlValues);
	EAvaPlayableRemoteControlChanges UpdateRemoteControlValues(const FAvaPlayableRemoteControlValues& InRemoteControlValues, bool bInUpdateDefaults);

	bool HasRemoteControlEntityValue(const FGuid& InId) const { return RemoteControlValues.HasEntityValue(InId); }
	const FAvaPlayableRemoteControlValue* GetRemoteControlEntityValue(const FGuid& InId) const { return RemoteControlValues.GetEntityValue(InId); }
	void SetRemoteControlEntityValue(const FGuid& InId, const FAvaPlayableRemoteControlValue& InValue);

	bool HasRemoteControlControllerValue(const FGuid& InId) const { return RemoteControlValues.HasControllerValue(InId); }
	const FAvaPlayableRemoteControlValue* GetRemoteControlControllerValue(const FGuid& InId) const { return RemoteControlValues.GetControllerValue(InId); }
	void SetRemoteControlControllerValue(const FGuid& InId, const FAvaPlayableRemoteControlValue& InValue);

	const FAvaPlayableRemoteControlValues& GetRemoteControlValues() const { return RemoteControlValues; }
	bool GetDefaultRemoteControlValues(const UAvaRundown* InRundown, bool bInUseTemplateValues, FAvaPlayableRemoteControlValues& OutValues) const;
	bool GetDefaultEntityValue(const UAvaRundown* InRundown, const FGuid& InId, bool bInUseTemplateValues, FAvaPlayableRemoteControlValue& OutValue) const;
	bool GetDefaultControllerValue(const UAvaRundown* InRundown, const FGuid& InId, bool bInUseTemplateValues, FAvaPlayableRemoteControlValue& OutValue) const;

	bool IsDefaultEntityValue(const UAvaRundown* InRundown, const FGuid& InId, bool bInUseTemplateValues) const;
	bool IsDefaultControllerValue(const UAvaRundown* InRundown, const FGuid& InId, bool bInUseTemplateValues) const;

	EAvaPlayableRemoteControlChanges ResetRemoteControlValues(const UAvaRundown* InRundown, bool bInUseTemplateValues, bool bInIsDefault);
	EAvaPlayableRemoteControlChanges ResetRemoteControlEntityValue(const UAvaRundown* InRundown, const FGuid& InId, bool bInUseTemplateValues, bool bInIsDefault);
	EAvaPlayableRemoteControlChanges ResetRemoteControlControllerValue(const UAvaRundown* InRundown, const FGuid& InId, bool bInUseTemplateValues, bool bInIsDefault);

	friend FORCEINLINE uint32 GetTypeHash(const FAvaRundownPage& InPage)
	{
		return GetTypeHash(InPage.PageId);
	}

	bool operator<(const FAvaRundownPage& InOther) const
	{
		return this->PageId < InOther.PageId;
	}

	void PostLoad();

	/** Returns the number of templates this page/combo template has. */
	int32 GetNumTemplates(const UAvaRundown* InRundown) const;

	/**
	 * Returns the template at given index.
	 * This will first resolve the direct template, then lookup combined templates.
	 */
	const FAvaRundownPage& GetTemplate(const UAvaRundown* InRundown, int32 InIndex) const;

	/**
	 * Resolves a page's templates.
	 * For a page, it returns its (direct) template.
	 * For a template, it returns itself. 
	 */
	const FAvaRundownPage& ResolveTemplate(const UAvaRundown* InRundown) const;
	
	/**
	 * @brief Compare the values from another template and determines if there is a match.
	 * @param InTemplatePage Other template page to compare to.
	 * @return true if the templates are matching (ignores ids).
	 */
	bool IsTemplateMatchingByValue(const FAvaRundownPage& InTemplatePage) const;

	/**
	 * Load the instanced commands from the given command data (from web api).
	 * @todo Needed until all json serializer can support FInstancedStruct.
	 */
	void LoadInstancedCommands(const TArray<FAvaRundownPageCommandData>& InCommands);

	/**
	 * Save the instanced commands to command data (suitable for web api).
	 * @todo Needed until all json serializer can support FInstancedStruct.
	 */
	TArray<FAvaRundownPageCommandData> SaveInstancedCommands() const;

	/**
	 * @brief Get the instanced page commands.
	 * @return array of instanced page commands that can be executed.
	 */
	const TArray<FInstancedStruct>& GetInstancedCommands() const
	{
		return InstancedCommands;
	}

	/**
	 * @brief Set the instanced commands.
	 * @param InInstancedCommands new instance commands to copy in the page.
	 */
	void SetInstancedCommands(const TArray<FInstancedStruct>& InInstancedCommands);

	/**
	 * @brief Calls the given function for each commands in the page.
	 * @param InFunction Function to call for each command. The Page parameter is the direct page the command is on, i.e. may be the template.
	 * @param InRundown Owning rundown (used to resolve templates)
	 * @param bInDirectOnly If true, only the current page's command is traversed, otherwise, the templates commands are included as well.
	 */
	void ForEachInstancedCommands(TFunctionRef<void(const FAvaRundownPageCommand&, const FAvaRundownPage&)> InFunction, const UAvaRundown* InRundown, bool bInDirectOnly) const;

protected:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design")
	bool bEnabled = true;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design")
	int32 PageId = InvalidPageId;

	/** Page Instance Property: Template Id for this page. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design")
	int32 TemplateId;

	/**
	 * Template property: For combination template, lists the templates that are combined.
	 * A combination template can only be created using transition logic templates.
	 * In order to create a combination template, the templates must be in different transition layers.
	 */ 
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design")
	TArray<int32> CombinedTemplateIds;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design")
	FString PageName;

	/** Template property: path for this template. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design", meta=(AllowedClasses="/Script/Engine.World"))
	FSoftObjectPath AssetPath;

	/** Template property: List the Ids of all instances. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design")
	TSet<int32> Instances;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design")
	int32 OutputChannel = 0;

	UPROPERTY(EditAnywhere, Category = "Motion Design")
	FAvaPlayableRemoteControlValues RemoteControlValues;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design")
	FText PageSummary = FText::GetEmpty();

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design")
	FText FriendlyName = FText::GetEmpty();

	/** Indicate if the template asset has transition logic. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design")
	bool bHasTransitionLogic = false;

	/** Transition Layer Tag cached from the transition tree. Cached for fast display in page/template list. */
	UPROPERTY(VisibleAnywhere, Category = "Motion Design")
	FAvaTagHandle TransitionLayerTag;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design")
	EAvaTransitionInstancingMode TransitionMode = EAvaTransitionInstancingMode::New;

	/**
	 * Page command that can be executed when playing this page.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design", meta = (ExcludeBaseStruct, BaseStruct = "/Script/AvalancheMedia.AvaRundownPageCommand"))
	TArray<FInstancedStruct> InstancedCommands;
};
