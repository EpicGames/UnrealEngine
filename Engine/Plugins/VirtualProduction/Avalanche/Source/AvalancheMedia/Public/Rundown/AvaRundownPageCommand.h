// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

#include "AvaRundownPageCommand.generated.h"

class FAvaRundownPageTransitionBuilder;
class UAvaRundown;
struct FAvaRundownPage;

/**
 * Base context carrying parameters for page commands execution. 
 */
struct FAvaRundownPageCommandContext
{
	const UAvaRundown& Rundown;
	const FAvaRundownPage& Page;
	const FName ChannelName;
};

/**
 * Base class for page commands.
 * @remark this API is experimental.
 */
USTRUCT(BlueprintType)
struct FAvaRundownPageCommand
{
	GENERATED_BODY()

	virtual ~FAvaRundownPageCommand() = default;

	/** Returns this command's description. Used for page summary. */
	virtual FText GetDescription() const { return FText();}
	
	/** Returns true if the command will act as transition logic. */
	virtual bool HasTransitionLogic() const { return false; }

	/** Returns a string representing the transition layers. This is used for the page list "layer" column. */
	virtual FString GetTransitionLayerString(const FString& InSeparator) const { return FString(); }
	
	/** Returns true if the command can be executed. Used by UI for button status. */
	virtual bool CanExecuteOnPlay(FAvaRundownPageCommandContext& InContext, FString* OutFailureReason) const { return false; }

	/** Execute the command when a page is played. */
	virtual bool ExecuteOnPlay(FAvaRundownPageTransitionBuilder& InTransitionBuilder, FAvaRundownPageCommandContext& InContext) const { return false; }

	/** Returns true if the command can be executed. Used by UI for button status. */
	virtual bool CanExecuteOnLoad(FAvaRundownPageCommandContext& InContext, FString* OutFailureReason) const { return false; }

	/** Execute the command when a page is loaded. */
	virtual bool ExecuteOnLoad(FAvaRundownPageCommandContext& InContext, FString& OutLoadOptions) const { return false; }
};
