// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagHandle.h"
#include "Playable/AvaPlayable.h"
#include "Playback/AvaPlaybackManager.h"
#include "Rundown/AvaRundownDefines.h"
#include "UObject/Object.h"
#include "AvaRundownPagePlayer.generated.h"

class FAvaPlaybackInstance;
class UAvaPlaybackGraph;
class UAvaRundown;
class UAvaRundownPagePlayer;
struct FAvaRundownPage;

/**
 *	To support combo templates, a page is now potentially composed of a number of
 *	playback instances that each have their own player.
 */
UCLASS()
class AVALANCHEMEDIA_API UAvaRundownPlaybackInstancePlayer : public UObject
{
	GENERATED_BODY()

public:
	UAvaRundownPlaybackInstancePlayer();
	virtual ~UAvaRundownPlaybackInstancePlayer() override;

	bool Load(const UAvaRundownPagePlayer& InPagePlayer, const UAvaRundown* InRundown, const FAvaRundownPage& InPage, int32 InSubPageIndex, const FGuid& InInstanceId);
	bool IsLoaded() const;

	UE_DEPRECATED(5.6, "Playing a page without a transition is no longuer supported.")
	void Play(const UAvaRundownPagePlayer& InPagePlayer, const UAvaRundown* InRundown, EAvaRundownPagePlayType InPlayType, bool bInIsUsingTransitionLogic)
	{
		Play(InPagePlayer, InRundown, InPlayType);
	}
	
	void Play(const UAvaRundownPagePlayer& InPagePlayer, const UAvaRundown* InRundown, EAvaRundownPagePlayType InPlayType);
	bool IsPlaying() const;

	bool Continue(const FString& InChannelName);
	bool Stop();
	
	FAvaPlaybackInstance* GetPlaybackInstance() const { return PlaybackInstance ? PlaybackInstance.Get() : nullptr; }

	FGuid GetPlaybackInstanceId() const { return PlaybackInstance ? PlaybackInstance->GetInstanceId() : FGuid(); }

	bool HasPlayable(const UAvaPlayable* InPlayable) const;

	UAvaPlayable* GetFirstPlayable() const;

	UAvaRundownPagePlayer* GetPagePlayer() const;

	void SetPagePlayer(UAvaRundownPagePlayer* InPagePlayer);

public:
	UPROPERTY()
	FAvaTagHandle TransitionLayer;

	UPROPERTY()
	FSoftObjectPath SourceAssetPath;
	
	UPROPERTY()
	TObjectPtr<UAvaPlaybackGraph> Playback;

	TSharedPtr<FAvaPlaybackInstance> PlaybackInstance;

	TWeakObjectPtr<UAvaRundownPagePlayer> ParentPagePlayer;
};

UCLASS()
class AVALANCHEMEDIA_API UAvaRundownPagePlayer : public UObject
{
	GENERATED_BODY()
	
public:
	UAvaRundownPagePlayer();
	virtual ~UAvaRundownPagePlayer() override;

	/** Initialize Page player without loading any playback instances. */
	bool Initialize(UAvaRundown* InRundown, const FAvaRundownPage& InPage, bool bInIsPreview, const FName& InPreviewChannel);

	/** Initialize the Page player and loads all playback instances. */
	bool InitializeAndLoad(UAvaRundown* InRundown, const FAvaRundownPage& InPage, bool bInIsPreview, const FName& InPreviewChannel);

	/**
	 * Load an instance player for the "sub page" at the given index.
	 * The index is the template index in the combo template.
	 */
	UAvaRundownPlaybackInstancePlayer* LoadInstancePlayer(int32 InSubPageIndex, const FGuid& InInstanceId);

	/** Add a pre-existing instance player to this page player. */
	void AddInstancePlayer(UAvaRundownPlaybackInstancePlayer* InExistingInstancePlayer);

	/** Returns true if at least one of the instance player is loaded, false otherwise. */
	bool IsLoaded() const;

	UE_DEPRECATED(5.6, "Playing a page without a transition is no longuer supported.")
	bool Play(EAvaRundownPagePlayType InPlayType, bool bInIsUsingTransitionLogic)
	{
		return Play(InPlayType);
	}
	
	/**
	 * Ensure all the playback instances are playing.
	 * Returns true if at least one playback instance is playing, false otherwise.
	 */
	bool Play(EAvaRundownPagePlayType InPlayType);

	/** Returns true if at least one of the instance player is playing, false otherwise. */
	bool IsPlaying() const;
	
	bool Continue();
	bool Stop();

	int32 GetNumInstancePlayers() const { return InstancePlayers.Num(); }

	FAvaPlaybackInstance* GetPlaybackInstance(int32 InIndex = 0) const
	{
		return InstancePlayers.IsValidIndex(InIndex) && IsValid(InstancePlayers[InIndex]) ? InstancePlayers[InIndex]->GetPlaybackInstance() : nullptr;
	}

	FGuid GetPlaybackInstanceId(int32 InIndex = 0) const
	{
		return InstancePlayers.IsValidIndex(InIndex) && IsValid(InstancePlayers[InIndex]) ? InstancePlayers[InIndex]->GetPlaybackInstanceId() : FGuid();
	}

	UAvaPlaybackGraph* GetPlayback(int32 InIndex = 0) const
	{
		return InstancePlayers.IsValidIndex(InIndex) && IsValid(InstancePlayers[InIndex]) ? InstancePlayers[InIndex]->Playback : nullptr;
	}

	FSoftObjectPath GetSourceAssetPath(int32 InIndex = 0) const
	{
		return InstancePlayers.IsValidIndex(InIndex) && IsValid(InstancePlayers[InIndex]) ? InstancePlayers[InIndex]->SourceAssetPath : FSoftObjectPath();
	}

	UAvaRundownPlaybackInstancePlayer* GetInstancePlayer(int32 InIndex) const
	{
		return InstancePlayers.IsValidIndex(InIndex)? InstancePlayers[InIndex] : nullptr;
	}

	void ForEachInstancePlayer(TFunctionRef<void(UAvaRundownPlaybackInstancePlayer*)> InFunction)
	{
		for (const TObjectPtr<UAvaRundownPlaybackInstancePlayer>& InstancePlayer : InstancePlayers)
		{
			InFunction(InstancePlayer.Get());
		}
	}

	void ForEachInstancePlayer(TFunctionRef<void(const UAvaRundownPlaybackInstancePlayer*)> InFunction) const
	{
		for (const TObjectPtr<UAvaRundownPlaybackInstancePlayer>& InstancePlayer : InstancePlayers)
		{
			InFunction(InstancePlayer.Get());
		}
	}
	
	UAvaRundown* GetRundown() const { return RundownWeak.Get(); }

	static int32 GetPageIdFromInstanceUserData(const FString& InUserData);

	static void SetInstanceUserDataFromPage(FAvaPlaybackInstance& InPlaybackInstance, const FAvaRundownPage& InPage);

	bool HasPlayable(const UAvaPlayable* InPlayable) const;

	UAvaRundownPlaybackInstancePlayer* FindInstancePlayerForPlayable(const UAvaPlayable* InPlayable) const;

	UAvaRundownPlaybackInstancePlayer* FindInstancePlayerByInstanceId(const FGuid& InInstanceId) const;

	UAvaRundownPlaybackInstancePlayer* FindInstancePlayerByAssetPath(const FSoftObjectPath& InAssetPath) const;
	
protected:
	UAvaRundownPlaybackInstancePlayer* CreateAndLoadInstancePlayer(UAvaRundown* InRundown, const FAvaRundownPage& InPage, int32 InSubPageIndex, const FGuid& InInstanceId);

	void RemoveInstancePlayer(UAvaRundownPlaybackInstancePlayer* InInstancePlayer);
	
	void HandleOnPlayableSequenceEvent(UAvaPlayable* InPlayable, FName InSequenceLabel, EAvaPlayableSequenceEventType InEventType);

public:
	UPROPERTY()
	int32 PageId = INDEX_NONE;

	UPROPERTY()
	bool bIsPreview = false;

	/** @remark For previews, the channel name will not be the one set in the page. */
	UPROPERTY()
	FName ChannelFName;

	/** @remark For previews, the channel name will not be the one set in the page. */
	UPROPERTY()
	FString ChannelName;

	UPROPERTY()
	TArray<TObjectPtr<UAvaRundownPlaybackInstancePlayer>> InstancePlayers;

protected:
	TWeakObjectPtr<UAvaRundown> RundownWeak;
};
