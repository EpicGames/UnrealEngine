// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageContext.h"

class IAvaRundownServer
{
public:
	virtual ~IAvaRundownServer() = default;
	
	/** Returns the server's name. */
	virtual const FString& GetName() const = 0;

	/** Returns the endpoint's message address. */
	virtual const FMessageAddress& GetMessageAddress() const = 0;

	/** Returns the list of connected client addresses. */
	virtual TArray<FMessageAddress> GetClientAddresses() const = 0;
};