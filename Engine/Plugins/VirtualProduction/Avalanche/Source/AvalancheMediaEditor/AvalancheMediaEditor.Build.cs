// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheMediaEditor : ModuleRules
{
	public AvalancheMediaEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AvalancheCore",
				"AvalancheMedia",
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"MediaIOCore",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"AssetDefinition",
				"AssetTools",
				"Avalanche",
				"AvalancheEditorCore",
				"AvalancheRemoteControl",
				"AvalancheTag",
				"ContentBrowser",
				"EditorWidgets",
				"GraphEditor",
				"InputCore",
				"Json",
				"JsonSerialization",
				"JsonUtilities",
				"Kismet",
				"MediaIOEditor",
				"Messaging",
				"Projects",
				"PropertyEditor",
				"RHI",
				"RemoteControl",
				"RemoteControlLogic",
				"RemoteControlUI", 
				"RenderCore",
				"Serialization",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"ToolWidgets",
				"UnrealEd",
				"XmlSerialization",
			}
		);
	}
}
