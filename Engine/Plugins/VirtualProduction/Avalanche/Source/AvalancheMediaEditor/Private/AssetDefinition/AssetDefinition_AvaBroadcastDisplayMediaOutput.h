// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinition/AssetDefinition_MediaOutput.h"
#include "Broadcast/OutputDevices/AvaBroadcastDisplayMediaOutput.h"
#include "AssetDefinition_AvaBroadcastDisplayMediaOutput.generated.h"

UCLASS()
class UAssetDefinition_AvaBroadcastDisplayMediaOutput : public UAssetDefinition_MediaOutput
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_AvaBroadcastDisplayMediaOutput", "Motion Design Broadcast Display Media Output"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UAvaBroadcastDisplayMediaOutput::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Media, NSLOCTEXT("AssetDefinition", "AssetDefinition_AvaBroadcastDisplayMediaOutputSubMenu", "Media Sources + Outputs"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
