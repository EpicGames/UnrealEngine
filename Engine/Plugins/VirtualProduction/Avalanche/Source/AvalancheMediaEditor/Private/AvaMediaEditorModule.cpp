// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMediaEditorModule.h"

#include "AvaMediaEditorStyle.h"
#include "AvaMediaSettings.h"
#include "Broadcast/AvaBroadcastEditor.h"
#include "Broadcast/OutputDevices/AvaBroadcastMediaIOOutputConfigurationCustomization.h"
#include "Customization/AvaPlaybackServerLoggingEntryCustomization.h"
#include "Customization/AvaSynchronizedEventsFeatureSelectionCustomization.h"
#include "Editor.h"
#include "Engine/GameViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "IAvaMediaModule.h"
#include "LevelEditor.h"
#include "LevelEditorViewport.h"
#include "Playback/AvaPlaybackCommands.h"
#include "Playback/Graph/AvaPlaybackConnectionDrawingPolicy.h"
#include "Rundown/AvaRundownCommands.h"
#include "Rundown/AvaRundownEditorSettings.h"
#include "Rundown/AvaRundownMacroCollection.h"
#include "Rundown/Customization/AvaRundownMacroCommandCustomization.h"
#include "Rundown/Customization/AvaRundownMacroKeyBindingCustomization.h"
#include "Rundown/Factories/Filters/AvaRundownFactoriesUtils.h"
#include "Rundown/Factories/Filters/AvaRundownFilterChannelExpressionFactory.h"
#include "Rundown/Factories/Filters/AvaRundownFilterComboPageExpressionFactory.h"
#include "Rundown/Factories/Filters/AvaRundownFilterIdExpressionFactory.h"
#include "Rundown/Factories/Filters/AvaRundownFilterNameExpressionFactory.h"
#include "Rundown/Factories/Filters/AvaRundownFilterPathExpressionFactory.h"
#include "Rundown/Factories/Filters/AvaRundownFilterStatusExpressionFactory.h"
#include "Rundown/Factories/Filters/AvaRundownFilterTransitionLayerExpressionFactory.h"
#include "Rundown/Factories/Filters/IAvaRundownFilterExpressionFactory.h"
#include "Rundown/Factories/Filters/IAvaRundownFilterSuggestionFactory.h"
#include "Rundown/Factories/Suggestions/AvaRundownFilterChannelSuggestionFactory.h"
#include "Rundown/Factories/Suggestions/AvaRundownFilterComboPageSuggestionFactory.h"
#include "Rundown/Factories/Suggestions/AvaRundownFilterIdSuggestionFactory.h"
#include "Rundown/Factories/Suggestions/AvaRundownFilterNameSuggestionFactory.h"
#include "Rundown/Factories/Suggestions/AvaRundownFilterPathSuggestionFactory.h"
#include "Rundown/Factories/Suggestions/AvaRundownFilterStatusSuggestionFactory.h"
#include "Rundown/Factories/Suggestions/AvaRundownFilterTransitionLayerSuggestionFactory.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AvaMediaEditorModule"

DEFINE_LOG_CATEGORY(LogAvaMediaEditor);

namespace UE::AvaMediaEditorModule::Private
{
	namespace BroadcastEditorEntry
	{
		static const FName MenuName(TEXT("LevelEditor.StatusBar.ToolBar"));
		static const FName SectionName(TEXT("MotionDesign"));
	}

	static void GetEditorViewportClient(FCommonViewportClient** OutViewportClient)
	{
		// Replicating the logic from UUnrealEdEngine::Exec.
		FCommonViewportClient* ViewportClient = nullptr;
		if (!GStatProcessingViewportClient && (!GEngine->GameViewport || GEngine->GameViewport->IsSimulateInEditorViewport() ))
		{
			ViewportClient = GLastKeyLevelEditingViewportClient ? GLastKeyLevelEditingViewportClient : GCurrentLevelEditingViewportClient;
		}

		if (ViewportClient)
		{
			*OutViewportClient = ViewportClient;
		}
	}
}

void FAvaMediaEditorModule::StartupModule()
{
	using namespace UE::AvaMediaEditorModule::Private;

	InitExtensibilityManagers();

	FAvaPlaybackCommands::Register();
	FAvaRundownCommands::Register();

	if (FSlateApplication::IsInitialized())
	{
		AddEditorToolbarButtons();
	}

	// Register the Motion Design Playback Graph connection policy with the graph editor
	PlaybackConnectionFactory = MakeShared<FAvaPlaybackConnectionDrawingPolicyFactory>();
	FEdGraphUtilities::RegisterVisualPinConnectionFactory(PlaybackConnectionFactory);

	// MediaIOEditor module is loaded in PostEngineInit phase,
	// so we need in order to have our customizations override theirs, we need to
	// register ours after, i.e. once all modules are loaded.
	FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddRaw(this, &FAvaMediaEditorModule::RegisterCustomizations);
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FAvaMediaEditorModule::PostEngineInit);

	// Register Map Change Events
	FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditor.OnMapChanged().AddRaw(this, &FAvaMediaEditorModule::HandleMapChanged);

	IAvaMediaModule::Get().GetEditorViewportClientDelegate().BindStatic(&GetEditorViewportClient);

	RegisterRundownFilterExpressionFactories();
	RegisterRundownFilterSuggestionFactories();
}

void FAvaMediaEditorModule::ShutdownModule()
{
	FCoreDelegates::OnAllModuleLoadingPhasesComplete.RemoveAll(this);
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	// Unregister Map Change Events
	if (FLevelEditorModule* LevelEditor = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		LevelEditor->OnMapChanged().RemoveAll(this);
	}

	ResetExtensibilityManagers();
	if (FSlateApplication::IsInitialized())
	{
		RemoveEditorToolbarButtons();
	}
	if (UObjectInitialized() && !IsEngineExitRequested())
	{
		UnregisterCustomizations();
	}

	FAvaPlaybackCommands::Unregister();
	FAvaRundownCommands::Unregister();

	if (PlaybackConnectionFactory.IsValid())
	{
		FEdGraphUtilities::UnregisterVisualPinConnectionFactory(PlaybackConnectionFactory);
		PlaybackConnectionFactory.Reset();
	}
}

TSharedPtr<FExtensibilityManager> FAvaMediaEditorModule::GetBroadcastToolBarExtensibilityManager()
{
	return BroadcastToolBarExtensibility;
}

TSharedPtr<FExtensibilityManager> FAvaMediaEditorModule::GetPlaybackToolBarExtensibilityManager()
{
	return PlaybackToolBarExtensibility;
}

TSharedPtr<FExtensibilityManager> FAvaMediaEditorModule::GetRundownToolBarExtensibilityManager()
{
	return RundownToolBarExtensibility;
}

TSharedPtr<FExtensibilityManager> FAvaMediaEditorModule::GetRundownMenuExtensibilityManager()
{
	return RundownMenuExtensibility;
}

FSlateIcon FAvaMediaEditorModule::GetToolbarBroadcastButtonIcon() const
{
	const IAvaMediaModule& MediaModule = IAvaMediaModule::Get();

	if (MediaModule.IsPlaybackClientStarted())
	{
		return FSlateIcon(FAvaMediaEditorStyle::Get().GetStyleSetName(), "AvaMediaEditor.BroadcastClient", "AvaMediaEditor.BroadcastClient.Small");
	}
	else if (MediaModule.IsPlaybackServerStarted())
	{
		return FSlateIcon(FAvaMediaEditorStyle::Get().GetStyleSetName(), "AvaMediaEditor.BroadcastServer", "AvaMediaEditor.BroadcastServer.Small");
	}
	return FSlateIcon(FAvaMediaEditorStyle::Get().GetStyleSetName(), "AvaMediaEditor.BroadcastIcon");
}

bool FAvaMediaEditorModule::CanFilterSupportComparisonOperation(const FName& InFilterKey, ETextFilterComparisonOperation InOperation, EAvaRundownSearchListType InRundownSearchListType) const
{
	if (const TSharedPtr<IAvaRundownFilterExpressionFactory>* FilterExpressionFactory = FilterExpressionFactories.Find(InFilterKey))
	{
		return FilterExpressionFactory->Get()->SupportsComparisonOperation(InOperation, InRundownSearchListType);
	}
	return false;
}

bool FAvaMediaEditorModule::FilterExpression(const FName& InFilterKey, const FAvaRundownPage& InItem, const FAvaRundownTextFilterArgs& InArgs) const
{
	if (const TSharedPtr<IAvaRundownFilterExpressionFactory>* FilterExpressionFactory = FilterExpressionFactories.Find(InFilterKey))
	{
		return FilterExpressionFactory->Get()->FilterExpression(InItem, InArgs);
	}
	return false;
}

TArray<TSharedPtr<IAvaRundownFilterSuggestionFactory>> FAvaMediaEditorModule::GetSimpleSuggestions(EAvaRundownSearchListType InSuggestionType) const
{
	TArray<TSharedPtr<IAvaRundownFilterSuggestionFactory>> OutArray;

	for (const TPair<FName, TSharedPtr<IAvaRundownFilterSuggestionFactory>>& Suggestion : FilterSuggestionFactories)
	{
		if (Suggestion.Value->SupportSuggestionType(InSuggestionType) && Suggestion.Value->IsSimpleSuggestion())
		{
			OutArray.Add(Suggestion.Value);
		}
	}

	return OutArray;
}

TArray<TSharedPtr<IAvaRundownFilterSuggestionFactory>> FAvaMediaEditorModule::GetComplexSuggestions(EAvaRundownSearchListType InSuggestionType) const
{
	TArray<TSharedPtr<IAvaRundownFilterSuggestionFactory>> OutArray;

	for (const TPair<FName, TSharedPtr<IAvaRundownFilterSuggestionFactory>>& Suggestion : FilterSuggestionFactories)
	{
		if (Suggestion.Value->SupportSuggestionType(InSuggestionType) && !Suggestion.Value->IsSimpleSuggestion())
		{
			OutArray.Add(Suggestion.Value);
		}
	}

	return OutArray;
}

void FAvaMediaEditorModule::AddEditorToolbarButtons()
{
	using namespace UE::AvaMediaEditorModule::Private;

	FToolMenuEntry OpenBroadcastButtonEntry = FToolMenuEntry::InitToolBarButton(TEXT("OpenBroadcastToolbarButton")
		, FExecuteAction::CreateStatic(&FAvaBroadcastEditor::OpenBroadcastEditor)
		, LOCTEXT("OpenBroadcast_Title", "Broadcast")
		, LOCTEXT("OpenBroadcast_Tooltip", "Opens the Motion Design Broadcast Editor Window")
		, TAttribute<FSlateIcon>::Create([]() { return IAvaMediaEditorModule::Get().GetToolbarBroadcastButtonIcon(); })
	);
	OpenBroadcastButtonEntry.StyleNameOverride = TEXT("CalloutToolbar"); // Display Labels
	
	if (UToolMenu* const Menu = UToolMenus::Get()->ExtendMenu(BroadcastEditorEntry::MenuName))
	{
		FToolMenuSection& Section = Menu->FindOrAddSection(BroadcastEditorEntry::SectionName);
		Section.AddEntry(OpenBroadcastButtonEntry);
	}
}

void FAvaMediaEditorModule::RemoveEditorToolbarButtons()
{
	using namespace UE::AvaMediaEditorModule::Private;

	if (GIsEditor && UObjectInitialized())
	{
		UToolMenus::Get()->RemoveSection(BroadcastEditorEntry::MenuName, BroadcastEditorEntry::SectionName);
	}
}

void FAvaMediaEditorModule::OpenBroadcastEditor(const TArray<FString>& InArguments)
{
	FAvaBroadcastEditor::OpenBroadcastEditor();
}

void FAvaMediaEditorModule::InitExtensibilityManagers()
{
	BroadcastToolBarExtensibility = MakeShared<FExtensibilityManager>();
	PlaybackToolBarExtensibility  = MakeShared<FExtensibilityManager>();
	RundownToolBarExtensibility  = MakeShared<FExtensibilityManager>();
	RundownMenuExtensibility     = MakeShared<FExtensibilityManager>();
}

void FAvaMediaEditorModule::ResetExtensibilityManagers()
{
	BroadcastToolBarExtensibility = nullptr;
	PlaybackToolBarExtensibility  = nullptr;
	RundownToolBarExtensibility  = nullptr;
	RundownMenuExtensibility     = nullptr;
}

void FAvaMediaEditorModule::RegisterCustomizations() const
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout(FMediaIOOutputConfiguration::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAvaBroadcastMediaIOOutputConfigurationCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(FAvaRundownMacroCommand::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAvaRundownMacroCommandCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(FAvaRundownMacroKeyBinding::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAvaRundownMacroKeyBindingCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(FAvaSynchronizedEventsFeatureSelection::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAvaSynchronizedEventsFeatureSelectionCustomization::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(FAvaPlaybackServerLoggingEntry::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAvaPlaybackServerLoggingEntryCustomization::MakeInstance));
}

void FAvaMediaEditorModule::UnregisterCustomizations() const
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.UnregisterCustomPropertyTypeLayout(FMediaIOOutputConfiguration::StaticStruct()->GetFName());
	PropertyModule.UnregisterCustomPropertyTypeLayout(FAvaRundownMacroCommand::StaticStruct()->GetFName());
	PropertyModule.UnregisterCustomPropertyTypeLayout(FAvaRundownMacroKeyBinding::StaticStruct()->GetFName());
	PropertyModule.UnregisterCustomPropertyTypeLayout(FAvaSynchronizedEventsFeatureSelection::StaticStruct()->GetFName());
	PropertyModule.UnregisterCustomPropertyTypeLayout(FAvaPlaybackServerLoggingEntry::StaticStruct()->GetFName());
}

void FAvaMediaEditorModule::PostEngineInit()
{
	using namespace UE::AvaMediaEditorModule::Private;

	const UAvaRundownEditorSettings* Settings = UAvaRundownEditorSettings::Get();

	if (Settings && Settings->bAutoStartRundownServer)
	{
		IAvaMediaModule& AvaModule = IAvaMediaModule::Get();
		if (!AvaModule.IsRundownServerStarted())
		{
			AvaModule.StartRundownServer(Settings->RundownServerName);
		}
	}
}

void FAvaMediaEditorModule::HandleMapChanged(UWorld* InWorld, EMapChangeType InMapChangeType)
{
	EAvaMediaMapChangeType EventType;
	switch (InMapChangeType)
	{
		case EMapChangeType::LoadMap:
			EventType = EAvaMediaMapChangeType::LoadMap;
			break;

		case EMapChangeType::SaveMap:
			EventType = EAvaMediaMapChangeType::SaveMap;
			break;

		case EMapChangeType::NewMap:
			EventType = EAvaMediaMapChangeType::NewMap;
			break; 

		case EMapChangeType::TearDownWorld:
			EventType = EAvaMediaMapChangeType::TearDownWorld;
			break; 

		default:
			EventType = EAvaMediaMapChangeType::None;
			break; 
	}

	// Propagate the editor event to the runtime module.
	IAvaMediaModule::Get().NotifyMapChangedEvent(InWorld, EventType);
}

template <
	typename InRundownFilterExpressionFactoryType,
	typename ... InArgsType
	UE_REQUIRES_DEFINITION(std::is_base_of_v<IAvaRundownFilterExpressionFactory, InRundownFilterExpressionFactoryType>)
>
void FAvaMediaEditorModule::RegisterRundownFilterExpressionFactory(InArgsType&&... InArgs)
{
	const TSharedRef<IAvaRundownFilterExpressionFactory> FilterExpressionFactory =
		IAvaRundownFilterExpressionFactory::MakeInstance<InRundownFilterExpressionFactoryType>(Forward<InArgsType>(InArgs)...);

	if (!FilterExpressionFactories.Contains(FilterExpressionFactory->GetFilterIdentifier()))
	{
		FilterExpressionFactories.Add(FilterExpressionFactory->GetFilterIdentifier(), FilterExpressionFactory);
	}
}

template <
	typename InRundownSuggestionFactoryType,
	typename ... InArgsType
	UE_REQUIRES_DEFINITION(std::is_base_of_v<IAvaRundownFilterSuggestionFactory, InRundownSuggestionFactoryType>)
>
void FAvaMediaEditorModule::RegisterRundownFilterSuggestionFactory(InArgsType&&... InArgs)
{
	const TSharedRef<IAvaRundownFilterSuggestionFactory> FilterSuggestionFactory =
		IAvaRundownFilterSuggestionFactory::MakeInstance<InRundownSuggestionFactoryType>(Forward<InArgsType>(InArgs)...);

	if (!FilterSuggestionFactories.Contains(FilterSuggestionFactory->GetSuggestionIdentifier()))
	{
		FilterSuggestionFactories.Add(FilterSuggestionFactory->GetSuggestionIdentifier(), FilterSuggestionFactory);
	}
}

void FAvaMediaEditorModule::RegisterRundownFilterExpressionFactories()
{
	RegisterRundownFilterExpressionFactory<FAvaRundownFilterChannelExpressionFactory>();
	RegisterRundownFilterExpressionFactory<FAvaRundownFilterComboPageExpressionFactory>();
	RegisterRundownFilterExpressionFactory<FAvaRundownFilterIdExpressionFactory>();
	RegisterRundownFilterExpressionFactory<FAvaRundownFilterNameExpressionFactory>();
	RegisterRundownFilterExpressionFactory<FAvaRundownFilterPathExpressionFactory>();
	RegisterRundownFilterExpressionFactory<FAvaRundownFilterStatusExpressionFactory>();
	RegisterRundownFilterExpressionFactory<FAvaRundownFilterTransitionLayerExpressionFactory>();
}

void FAvaMediaEditorModule::RegisterRundownFilterSuggestionFactories()
{
	RegisterRundownFilterSuggestionFactory<FAvaRundownFilterChannelSuggestionFactory>();
	RegisterRundownFilterSuggestionFactory<FAvaRundownFilterComboPageSuggestionFactory>();
	RegisterRundownFilterSuggestionFactory<FAvaRundownFilterIdSuggestionFactory>();
	RegisterRundownFilterSuggestionFactory<FAvaRundownFilterNameSuggestionFactory>();
	RegisterRundownFilterSuggestionFactory<FAvaRundownFilterPathSuggestionFactory>();
	RegisterRundownFilterSuggestionFactory<FAvaRundownFilterStatusSuggestionFactory>();
	RegisterRundownFilterSuggestionFactory<FAvaRundownFilterTransitionLayerSuggestionFactory>();
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FAvaMediaEditorModule, AvalancheMediaEditor)
