// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaMediaEditorModule.h"
#include "Templates/UnrealTypeTraits.h"

#include <type_traits>

class IAvaRundownFilterExpressionFactory;
class IAvaRundownFilterSuggestionFactory;
enum class EAvaRundownSearchListType : uint8;
struct FGraphPanelPinConnectionFactory;

class FAvaMediaEditorModule : public IAvaMediaEditorModule
{

public:

	//IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~IModuleInterface

	//IAvaMediaEditorModule
	virtual TSharedPtr<FExtensibilityManager> GetBroadcastToolBarExtensibilityManager() override;
	virtual TSharedPtr<FExtensibilityManager> GetPlaybackToolBarExtensibilityManager() override;
	virtual TSharedPtr<FExtensibilityManager> GetRundownToolBarExtensibilityManager() override;
	virtual TSharedPtr<FExtensibilityManager> GetRundownMenuExtensibilityManager() override;
	virtual bool CanFilterSupportComparisonOperation(const FName& InFilterKey, ETextFilterComparisonOperation InOperation, EAvaRundownSearchListType InRundownSearchListType) const override;
	virtual bool FilterExpression(const FName& InFilterKey, const FAvaRundownPage& InItem, const FAvaRundownTextFilterArgs& InArgs) const override;
	virtual TArray<TSharedPtr<IAvaRundownFilterSuggestionFactory>> GetSimpleSuggestions(EAvaRundownSearchListType InSuggestionType) const override;
	virtual TArray<TSharedPtr<IAvaRundownFilterSuggestionFactory>> GetComplexSuggestions(EAvaRundownSearchListType InSuggestionType) const override;
	//~IAvaMediaEditorModule

	void AddEditorToolbarButtons();
	void RemoveEditorToolbarButtons();
	virtual FSlateIcon GetToolbarBroadcastButtonIcon() const override;

	static void OpenBroadcastEditor(const TArray<FString>& InArguments);

protected:
	void InitExtensibilityManagers();
	void ResetExtensibilityManagers();

	/** Register details view customizations. */
	void RegisterCustomizations() const;

	/** Unregister details view customizations. */
	void UnregisterCustomizations() const;

private:
	void PostEngineInit();
	void HandleMapChanged(UWorld* InWorld, EMapChangeType InMapChangeType);
	
	template <
		typename InRundownFilterExpressionFactoryType,
		typename... InArgsType
		UE_REQUIRES(std::is_base_of_v<IAvaRundownFilterExpressionFactory, InRundownFilterExpressionFactoryType>)
	>
	void RegisterRundownFilterExpressionFactory(InArgsType&&... InArgs);

	template <
		typename InRundownSuggestionFactoryType,
		typename... InArgsType
		UE_REQUIRES(std::is_base_of_v<IAvaRundownFilterSuggestionFactory, InRundownSuggestionFactoryType>)
	>
	void RegisterRundownFilterSuggestionFactory(InArgsType&&... InArgs);

	void RegisterRundownFilterExpressionFactories();

	void RegisterRundownFilterSuggestionFactories();

private:
	TSharedPtr<FExtensibilityManager> BroadcastToolBarExtensibility;
	TSharedPtr<FExtensibilityManager> PlaybackToolBarExtensibility;
	TSharedPtr<FExtensibilityManager> RundownToolBarExtensibility;
	TSharedPtr<FExtensibilityManager> RundownMenuExtensibility;

	TSharedPtr<FGraphPanelPinConnectionFactory> PlaybackConnectionFactory;

	/** Holds all the RundownFilterExpressionFactory */
	TMap<FName, TSharedPtr<IAvaRundownFilterExpressionFactory>> FilterExpressionFactories;

	/** Holds all the RundownFilterSuggestionFactory */
	TMap<FName, TSharedPtr<IAvaRundownFilterSuggestionFactory>> FilterSuggestionFactories;
};
