// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMediaEditorSettings.h"

UAvaMediaEditorSettings::UAvaMediaEditorSettings()
{
	CategoryName = TEXT("Motion Design");
	SectionName  = TEXT("Playback & Broadcast");
	
	PlaybackDefaultNodeColor  = FLinearColor(1.f, 1.f, 1.f, 1.f);
	PlaybackChannelsNodeColor = FLinearColor(1.f, 0.1f, 0.f, 1.f);
	PlaybackPlayerNodeColor   = FLinearColor(0.f, 1.f, 0.f, 1.f);
	PlaybackEventNodeColor    = FLinearColor(1.f, 0.f, 0.f, 1.f);
	PlaybackActionNodeColor   = FLinearColor(0.190525f, 0.583898f, 1.0f, 1.0f);
}

UAvaMediaEditorSettings* UAvaMediaEditorSettings::GetSingletonInstance()
{
	UAvaMediaEditorSettings* const DefaultSettings = GetMutableDefault<UAvaMediaEditorSettings>();
	static bool bInitialized = false;
	if (!bInitialized)
	{
		bInitialized = true;
		DefaultSettings->SetFlags(RF_Transactional);
	}
	return DefaultSettings;
}
