// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "AvaMediaEditorSettings.generated.h"

UCLASS(config=EditorPerProjectUserSettings, meta=(DisplayName="Playback & Broadcast"))
class UAvaMediaEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UAvaMediaEditorSettings();
	
	static const UAvaMediaEditorSettings& Get() {return *GetSingletonInstance();}
	static UAvaMediaEditorSettings& GetMutable() {return *GetSingletonInstance();}

	UPROPERTY(Config, EditAnywhere, Category = "Broadcast", meta = (InlineEditConditionToggle))
	bool bBroadcastEnforceMaxChannelCount = true;
	
	UPROPERTY(Config, EditAnywhere, Category = "Broadcast", meta = (DisplayName = "Max Channel Count", EditCondition = "bBroadcastEnforceMaxChannelCount"))
	int32 BroadcastMaxChannelCount = 9;

	/**
	 * By default, only the media output classes with a device provider are listed as output devices.
	 * If this is set to true, then all the media output classes will be listed, thus allowing use of
	 * outputs that don't correspond to a physical device, such as NDI, File output, etc.
	 * Caution: enabling this mode may expose Media Output classes that are not usable.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Broadcast", meta = (DisplayName = "Show All Media Output Classes"))
	bool bBroadcastShowAllMediaOutputClasses = false;

	UPROPERTY(Config, EditAnywhere, Category = "Playback", meta = (DisplayName = "Default Node Color"))
	FLinearColor PlaybackDefaultNodeColor;

	UPROPERTY(Config, EditAnywhere, Category = "Playback", meta = (DisplayName = "Channels Node Color"))
	FLinearColor PlaybackChannelsNodeColor;
	
	UPROPERTY(Config, EditAnywhere, Category = "Playback", meta = (DisplayName = "Player Node Color"))
	FLinearColor PlaybackPlayerNodeColor;
	
	UPROPERTY(Config, EditAnywhere, Category = "Playback", meta = (DisplayName = "Event Node Color"))
	FLinearColor PlaybackEventNodeColor;

	UPROPERTY(Config, EditAnywhere, Category = "Playback", meta = (DisplayName = "Action Node Color"))
	FLinearColor PlaybackActionNodeColor;

private:
	static UAvaMediaEditorSettings* GetSingletonInstance();
};
