// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMediaEditorStyle.h"
#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Containers/StringFwd.h"
#include "Interfaces/IPluginManager.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/ToolBarStyle.h"

FAvaMediaEditorStyle::FAvaMediaEditorStyle()
	: FSlateStyleSet(TEXT("AvaMediaEditor"))
{
	SetParentStyleName(FAppStyle::GetAppStyleSetName());

	const FVector2D Icon12x12(12.0f, 12.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	SetContentRoot(Plugin->GetBaseDir() / TEXT("Resources"));

	// Asset Class Icons
	Set("ClassIcon.AvaPlaybackGraph", new IMAGE_BRUSH("Icons/MediaIcons/AvaPlaybackIcon_16x"     , Icon16x16));
	Set("ClassThumbnail.AvaPlaybackGraph", new IMAGE_BRUSH("Icons/MediaIcons/AvaPlaybackIcon_64x", Icon64x64));
	Set("ClassIcon.AvaRundown", new IMAGE_BRUSH("Icons/MediaIcons/AvaRundownIcon_16x"     , Icon16x16));
	Set("ClassThumbnail.AvaRundown", new IMAGE_BRUSH("Icons/MediaIcons/AvaRundownIcon_64x", Icon64x64));
	Set("ClassIcon.AvaRundownMacroCollection", new IMAGE_BRUSH("Icons/MediaIcons/AvaRundownMacroCollectionIcon_16x"     , Icon16x16));
	Set("ClassThumbnail.AvaRundownMacroCollection", new IMAGE_BRUSH("Icons/MediaIcons/AvaRundownMacroCollectionIcon_64x", Icon64x64));

	//Rundown Commands
	Set("AvaRundown.AddPage"         , new IMAGE_BRUSH_SVG("Icons/MediaIcons/AddPage"      , Icon16x16));
	Set("AvaRundown.AddTemplate"     , new IMAGE_BRUSH_SVG("Icons/MediaIcons/AddPage"      , Icon16x16));
	Set("AvaRundown.CreatePageInstanceFromTemplate"  , new IMAGE_BRUSH_SVG("Icons/MediaIcons/AddPage"      , Icon16x16));
	Set("AvaRundown.CreateComboTemplate", new IMAGE_BRUSH_SVG("Icons/MediaIcons/AddPage"   , Icon16x16));
	Set("AvaRundown.RemovePage"      , new IMAGE_BRUSH_SVG("Icons/MediaIcons/RemovePage"   , Icon16x16));
	Set("AvaRundown.RenumberPage"    , new IMAGE_BRUSH_SVG("Icons/MediaIcons/RenumberPage" , Icon16x16));
	Set("AvaRundown.ReimportPage"    , new IMAGE_BRUSH_SVG("Icons/MediaIcons/ReimportPage" , Icon16x16));
	Set("AvaRundown.EditPageSource"  , new IMAGE_BRUSH_SVG("Icons/MediaIcons/OpenAsset"	   , Icon16x16));
	Set("AvaRundown.Play"            , new IMAGE_BRUSH_SVG("Icons/MediaIcons/Play"         , Icon16x16));
	Set("AvaRundown.UpdateValues"    , new IMAGE_BRUSH_SVG("Icons/MediaIcons/UpdateValues" , Icon16x16));
	Set("AvaRundown.Stop"            , new IMAGE_BRUSH_SVG("Icons/MediaIcons/Stop"         , Icon16x16));
	Set("AvaRundown.ForceStop"       , new IMAGE_BRUSH_SVG("Icons/MediaIcons/Stop"         , Icon16x16));
	Set("AvaRundown.Continue"        , new IMAGE_BRUSH_SVG("Icons/MediaIcons/Continue"     , Icon16x16));
	Set("AvaRundown.PlayNext"        , new IMAGE_BRUSH_SVG("Icons/MediaIcons/PlayNext"     , Icon16x16));
	Set("AvaRundown.PreviewFrame"    , new IMAGE_BRUSH_SVG("Icons/MediaIcons/PreviewFrame" , Icon16x16));
	Set("AvaRundown.PreviewPlay"     , new IMAGE_BRUSH_SVG("Icons/MediaIcons/Play"         , Icon16x16));
	Set("AvaRundown.PreviewStop"     , new IMAGE_BRUSH_SVG("Icons/MediaIcons/Stop"         , Icon16x16));
	Set("AvaRundown.PreviewForceStop", new IMAGE_BRUSH_SVG("Icons/MediaIcons/Stop"         , Icon16x16));
	Set("AvaRundown.PreviewContinue" , new IMAGE_BRUSH_SVG("Icons/MediaIcons/Continue"     , Icon16x16));
	Set("AvaRundown.PreviewPlayNext" , new IMAGE_BRUSH_SVG("Icons/MediaIcons/PlayNext"     , Icon16x16));
	Set("AvaRundown.TakeToProgram"   , new IMAGE_BRUSH_SVG("Icons/MediaIcons/Play"		   , Icon16x16));

	// All load page commands have the same icon. If this is ever going to change consider enabling icon visibility in MakeLoadPageOptionsMenu() (SAvaRundownShowControl.cpp)
	Set("AvaRundown.LoadAllPages"     , new IMAGE_BRUSH_SVG("Icons/MediaIcons/LoadPage", Icon16x16));
	Set("AvaRundown.LoadNextPages"    , new IMAGE_BRUSH_SVG("Icons/MediaIcons/LoadPage", Icon16x16));
	Set("AvaRundown.LoadSelectedPages", new IMAGE_BRUSH_SVG("Icons/MediaIcons/LoadPage", Icon16x16));

	//Broadcast
	Set("AvaMediaEditor.OutputIcon"           , new IMAGE_BRUSH("Icons/MediaIcons/MediaOutput"     , Icon20x20));
	Set("AvaMediaEditor.BroadcastIcon"        , new IMAGE_BRUSH("Icons/MediaIcons/MediaOutput"     , Icon20x20));
	Set("AvaMediaEditor.BroadcastClient"      , new IMAGE_BRUSH("Icons/MediaIcons/BroadcastClient" , Icon64x64));
	Set("AvaMediaEditor.BroadcastServer"      , new IMAGE_BRUSH("Icons/MediaIcons/BroadcastServer" , Icon64x64));
	Set("AvaMediaEditor.BroadcastClient.Small", new IMAGE_BRUSH("Icons/MediaIcons/BroadcastClient" , Icon20x20));
	Set("AvaMediaEditor.BroadcastServer.Small", new IMAGE_BRUSH("Icons/MediaIcons/BroadcastServer" , Icon20x20));
	Set("AvaMediaEditor.BroadcastOffline"     , new IMAGE_BRUSH_SVG("Icons/MediaIcons/BroadcastOffline"	, Icon16x16));
	Set("AvaMediaEditor.BroadcastError"       , new IMAGE_BRUSH_SVG("Icons/MediaIcons/BroadcastError"		, Icon16x16));
	Set("AvaMediaEditor.BroadcastWarning"     , new IMAGE_BRUSH_SVG("Icons/MediaIcons/BroadcastWarning"	, Icon16x16));
	Set("AvaMediaEditor.BroadcastLive"        , new IMAGE_BRUSH_SVG("Icons/MediaIcons/BroadcastLive"		, Icon16x16)); 
	Set("AvaMediaEditor.BroadcastIdle"        , new IMAGE_BRUSH_SVG("Icons/MediaIcons/BroadcastIdle"		, Icon16x16));

	// Broadcast Channel Types
	Set("AvaMediaEditor.ChannelTypeProgram"	, new IMAGE_BRUSH_SVG("Icons/MediaIcons/ChannelTypeProgram"	, Icon16x16)); 
	Set("AvaMediaEditor.ChannelTypePreview"	, new IMAGE_BRUSH_SVG("Icons/MediaIcons/ChannelTypePreview"	, Icon16x16));
	
	// Media Status
	/* For when we have SVGs
	Set("AvaMediaEditor.MediaAssetStatus", new IMAGE_BRUSH_SVG("Icons/MediaIcons/MediaLoaded", Icon16x16));
	Set("AvaMediaEditor.MediaSyncStatus",  new IMAGE_BRUSH_SVG("Icons/MediaIcons/MediaLoaded", Icon16x16));
	Set("AvaMediaEditor.MediaPreviewing",  new IMAGE_BRUSH_SVG("Icons/MediaIcons/MediaLoaded", Icon16x16));
	Set("AvaMediaEditor.MediaPlaying",     new IMAGE_BRUSH_SVG("Icons/MediaIcons/MediaLoaded", Icon16x16));
	*/
	Set("AvaMediaEditor.MediaAssetStatus", new IMAGE_BRUSH("Icons/MediaIcons/MediaStatus",     Icon16x16));
	Set("AvaMediaEditor.MediaSyncStatus",  new IMAGE_BRUSH("Icons/MediaIcons/MediaSyncStatus", Icon16x16));
	Set("AvaMediaEditor.MediaPreviewing",  new IMAGE_BRUSH("Icons/MediaIcons/MediaPreviewing", Icon16x16));
	Set("AvaMediaEditor.MediaPlaying",     new IMAGE_BRUSH("Icons/MediaIcons/MediaPlaying",    Icon16x16));

	// Media Output Status
	Set("AvaMediaEditor.MediaOutputOffline"   , new IMAGE_BRUSH_SVG("Icons/MediaIcons/MediaOutputOffline"	, Icon16x16));
	Set("AvaMediaEditor.MediaOutputIdle"      , new IMAGE_BRUSH_SVG("Icons/MediaIcons/MediaOutputIdle"		, Icon16x16));
	Set("AvaMediaEditor.MediaOutputPreparing" , new IMAGE_BRUSH_SVG("Icons/MediaIcons/MediaOutputPreparing", Icon16x16));
	Set("AvaMediaEditor.MediaOutputLive"      , new IMAGE_BRUSH_SVG("Icons/MediaIcons/MediaOutputLive"		, Icon16x16));
	Set("AvaMediaEditor.MediaOutputLiveWarn"  , new IMAGE_BRUSH_SVG("Icons/MediaIcons/MediaOutputLiveWarn"	, Icon16x16));
	Set("AvaMediaEditor.MediaOutputError"     , new IMAGE_BRUSH_SVG("Icons/MediaIcons/MediaOutputError"	, Icon16x16));

	// Transition Logic Mode
	Set("AvaMediaEditor.TransitionModeReuse", new IMAGE_BRUSH_SVG("Icons/MediaIcons/Recycling", Icon16x16));
	
	// Motion Design Preview
	Set("AvaMediaEditor.Checkerboard" , new IMAGE_BRUSH("Images/AvaPreviewCheckerboard", Icon16x16, FLinearColor::White, ESlateBrushTileType::Both));

	// Button styles
	const FButtonStyle& AppStyle_SimpleButton = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton");

	Set("AvaMediaEditor.BorderlessButton", FButtonStyle(AppStyle_SimpleButton));

	const FButtonStyle& MenuButtonStyleGreen = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("FlatButton.Success");
	Set("AvaMediaEditor.ButtonGreen", MenuButtonStyleGreen);
	const FButtonStyle& MenuButtonStyleRed = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("FlatButton.Danger");
	Set("AvaMediaEditor.ButtonRed", MenuButtonStyleRed);

	const FToolBarStyle& SlimToolbarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar");
	Set("AvaMediaEditor.ToolBar", SlimToolbarStyle);
	const FButtonStyle& MenuButtonStyle = SlimToolbarStyle.ButtonStyle;

	FToolBarStyle SlimToolbarStyleOverrideRedButton = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar");
	SlimToolbarStyleOverrideRedButton.SetButtonStyle(GetWidgetStyle<FButtonStyle>("AvaMediaEditor.ButtonRed"));
	SlimToolbarStyleOverrideRedButton.ButtonStyle.SetDisabled(MenuButtonStyle.Disabled);
	Set("AvaMediaEditor.ToolBarRedButtonOverride", SlimToolbarStyleOverrideRedButton);

	FToolBarStyle SlimToolbarStyleOverrideGreenButton = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar");
	SlimToolbarStyleOverrideGreenButton.SetButtonStyle(GetWidgetStyle<FButtonStyle>("AvaMediaEditor.ButtonGreen"));
	SlimToolbarStyleOverrideGreenButton.ButtonStyle.SetDisabled(MenuButtonStyle.Disabled);
	Set("AvaMediaEditor.ToolBarGreenButtonOverride", SlimToolbarStyleOverrideGreenButton);

	const FToolBarStyle& CalloutToolbarStyleOverride = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("CalloutToolbar");
	Set("AvaMediaEditor.CalloutToolbar", CalloutToolbarStyleOverride);

	Set("TableView.ActivePageBrush", new FSlateColorBrush(FLinearColor(0.05f, 0.1f, 0.05f, 0.5f)));
	Set("TableView.Hovered.ActivePageBrush", new FSlateColorBrush(FLinearColor(0.05f, 0.2f, 0.05f, 0.5f)));
	Set("TableView.Selected.ActivePageBrush", new FSlateColorBrush(FLinearColor(0.05f, 0.3f, 0.05f, 0.5f)));
	Set("TableView.DisabledPageBrush", new FSlateColorBrush(FLinearColor(0.1f, 0.05f, 0.05f, 0.5f)));
	Set("TableView.Hovered.DisabledPageBrush", new FSlateColorBrush(FLinearColor(0.2f, 0.05f, 0.05f, 0.5f)));
	Set("TableView.Selected.DisabledPageBrush", new FSlateColorBrush(FLinearColor(0.3f, 0.05f, 0.05f, 0.5f)));

	// Asset Colors - in the shades of yellow (hue = 36, saturation = 221)
	Set("AvaMediaEditor.AssetColors.Rundown", FLinearColor::MakeFromHSV8(36, 221, 170));
	Set("AvaMediaEditor.AssetColors.RundownMacroCollection", FLinearColor::MakeFromHSV8(36, 221, 124));
	Set("AvaMediaEditor.AssetColors.Playback", FLinearColor::MakeFromHSV8(36, 221, 105));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FAvaMediaEditorStyle::~FAvaMediaEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
