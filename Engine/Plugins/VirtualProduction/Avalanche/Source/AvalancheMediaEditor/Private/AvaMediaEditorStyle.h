// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FAvaMediaEditorStyle final : public FSlateStyleSet
{
public:
	static FAvaMediaEditorStyle& Get()
	{
		static FAvaMediaEditorStyle Instance;
		return Instance;
	}

	FAvaMediaEditorStyle();
	virtual ~FAvaMediaEditorStyle() override;
};
