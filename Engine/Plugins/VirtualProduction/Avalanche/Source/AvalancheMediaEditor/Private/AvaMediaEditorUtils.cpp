// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMediaEditorUtils.h"

const FSlateBrush* FAvaMediaEditorUtils::GetChannelStatusBrush(EAvaBroadcastChannelState InChannelState, EAvaBroadcastIssueSeverity InChannelIssueSeverity)
{
	if (InChannelState == EAvaBroadcastChannelState::Live)
	{
		switch (InChannelIssueSeverity)
		{
		case EAvaBroadcastIssueSeverity::None:
			return FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.BroadcastLive");
		case EAvaBroadcastIssueSeverity::Warnings:
			return FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.BroadcastWarning");
		case EAvaBroadcastIssueSeverity::Errors:
			return FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.BroadcastError");
		}
	}
	else if (InChannelState == EAvaBroadcastChannelState::Idle)
	{
		return FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.BroadcastIdle");
	}
	else
	{
		return FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.BroadcastOffline");
	}
	
	return FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.BroadcastIdle");
}

FText FAvaMediaEditorUtils::GetChannelStatusText(EAvaBroadcastChannelState InChannelState, EAvaBroadcastIssueSeverity InChannelIssueSeverity)
{
	//TODO: Possibly also add Issue Severity (if not none)
	return StaticEnum<EAvaBroadcastChannelState>()->GetDisplayNameTextByIndex(static_cast<int32>(InChannelState));
}
