// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "AvaMediaEditorStyle.h"

class FAvaMediaEditorUtils
{
public:
	static const FSlateBrush* GetChannelStatusBrush(EAvaBroadcastChannelState InChannelState, EAvaBroadcastIssueSeverity InChannelIssueSeverity);
	static FText GetChannelStatusText(EAvaBroadcastChannelState InChannelState, EAvaBroadcastIssueSeverity InChannelIssueSeverity);
};
