// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaBroadcastAppMode.h"

#include "Broadcast/AvaBroadcastEditor.h"

const FName FAvaBroadcastAppMode::DefaultMode("DefaultName");

#define LOCTEXT_NAMESPACE "AvaBroadcastAppMode"

FAvaBroadcastAppMode::FAvaBroadcastAppMode(const TSharedPtr<FAvaBroadcastEditor>& InBroadcastEditor, const FName& InModeName)
	: FApplicationMode(InModeName, FAvaBroadcastAppMode::GetLocalizedMode)
	, BroadcastEditorWeak(InBroadcastEditor)
{
}

void FAvaBroadcastAppMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FAvaBroadcastEditor> BroadcastEditor = BroadcastEditorWeak.Pin();
	BroadcastEditor->PushTabFactories(TabFactories);
}

FText FAvaBroadcastAppMode::GetLocalizedMode(const FName InMode)
{
	static TMap<FName, FText> LocModes;

	if (LocModes.Num() == 0)
	{
		LocModes.Add(DefaultMode, LOCTEXT("Broadcast_DefaultMode", "Default"));
	}

	check(InMode != NAME_None);
	const FText* OutDesc = LocModes.Find(InMode);
	check(OutDesc);
	
	return *OutDesc;
}

#undef LOCTEXT_NAMESPACE
