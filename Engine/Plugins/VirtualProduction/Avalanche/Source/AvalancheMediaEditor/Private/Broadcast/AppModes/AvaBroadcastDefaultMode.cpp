// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaBroadcastDefaultMode.h"

#include "Broadcast/AvaBroadcastEditor.h"
#include "Broadcast/TabFactories/AvaBroadcastChannelsTabFactory.h"
#include "Broadcast/TabFactories/AvaBroadcastDetailsTabFactory.h"
#include "Broadcast/TabFactories/AvaBroadcastOutputsTabFactory.h"
#include "IAvaMediaEditorModule.h"

#define LOCTEXT_NAMESPACE "AvaBroadcastDefaultMode"

FAvaBroadcastDefaultMode::FAvaBroadcastDefaultMode(const TSharedPtr<FAvaBroadcastEditor>& InBroadcastEditor)
	: FAvaBroadcastAppMode(InBroadcastEditor, FAvaBroadcastAppMode::DefaultMode)
{
	WorkspaceMenuCategory = FWorkspaceItem::NewGroup(LOCTEXT("WorkspaceMenu_AvaBroadcast", "Motion Design Broadcast"));
	
	check(InBroadcastEditor.IsValid());
	TabLayout = FTabManager::NewLayout("MotionDesignBroadcastEditor_Default_Layout_V1")
		->AddArea
		(
			FTabManager::NewPrimaryArea() ->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.75f)
					->AddTab(FAvaBroadcastChannelsTabFactory::TabID, ETabState::OpenedTab)
					->SetForegroundTab(FAvaBroadcastChannelsTabFactory::TabID)
					->SetHideTabWell(true)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.25f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(FAvaBroadcastOutputsTabFactory::TabID, ETabState::OpenedTab)
						->SetForegroundTab(FAvaBroadcastOutputsTabFactory::TabID)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(FAvaBroadcastDetailsTabFactory::TabID, ETabState::OpenedTab)
						->SetForegroundTab(FAvaBroadcastDetailsTabFactory::TabID)
					)
				)
			)
		);

	// Add Tab Spawners
	TabFactories.RegisterFactory(MakeShared<FAvaBroadcastChannelsTabFactory>(InBroadcastEditor));
	TabFactories.RegisterFactory(MakeShared<FAvaBroadcastOutputsTabFactory>(InBroadcastEditor));
	TabFactories.RegisterFactory(MakeShared<FAvaBroadcastDetailsTabFactory>(InBroadcastEditor));

	//Make sure we start with our existing list of extenders instead of creating a new one
	IAvaMediaEditorModule& AvaMediaEditorModule = IAvaMediaEditorModule::Get();
	ToolbarExtender = AvaMediaEditorModule.GetBroadcastToolBarExtensibilityManager()->GetAllExtenders();
	InBroadcastEditor->ExtendToolBar(ToolbarExtender);
}

#undef LOCTEXT_NAMESPACE
