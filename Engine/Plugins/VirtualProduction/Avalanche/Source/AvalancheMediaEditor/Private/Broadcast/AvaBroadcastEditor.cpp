// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/AvaBroadcastEditor.h"
#include "AppModes/AvaBroadcastDefaultMode.h"
#include "AvaMediaEditorSettings.h"
#include "AvaMediaSettings.h"
#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/ChannelGrid/AvaBroadcastOutputTileItem.h"
#include "Containers/UnrealString.h"
#include "ChannelGrid/Slate/SAvaBroadcastProfileEntry.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "IAvaMediaEditorModule.h"
#include "IAvaMediaModule.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "MediaOutput.h"
#include "Misc/MessageDialog.h"
#include "Playback/IAvaPlaybackClient.h"
#include "ScopedTransaction.h"
#include "Styling/SlateBrush.h"
#include "UObject/NameTypes.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "UObject/NoExportTypes.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "AvaBroadcastEditor"

TSharedPtr<FAvaBroadcastEditor> FAvaBroadcastEditor::BroadcastEditor;

FAvaBroadcastEditor::~FAvaBroadcastEditor()
{
	if (UObjectInitialized())
	{
		UAvaMediaSettings& AvaMediaSettings = UAvaMediaSettings::GetMutable();
		AvaMediaSettings.OnSettingChanged().RemoveAll(this);
	}
	
	if (BroadcastWeak.IsValid())
	{
		BroadcastWeak->RemoveChangeListener(this);
	}
}

void FAvaBroadcastEditor::OpenBroadcastEditor()
{
	if (BroadcastEditor.IsValid() && BroadcastEditor->IsHosted())
	{
		BroadcastEditor->BringToolkitToFront();
	}
	else
	{
		BroadcastEditor.Reset();
		BroadcastEditor = MakeShared<FAvaBroadcastEditor>(FPrivateToken());
		BroadcastEditor->InitBroadcastEditor(&UAvaBroadcast::Get());
	}
}

void FAvaBroadcastEditor::SelectOutputTile(const TSharedPtr<FAvaBroadcastOutputTileItem>& InOutputTile)
{
	if (SelectedOutputTileWeak != InOutputTile)
	{
		SelectedOutputTileWeak = InOutputTile;
		OnOutputTileSelectionChanged.Broadcast(InOutputTile);
	}
}

TSharedPtr<FAvaBroadcastOutputTileItem> FAvaBroadcastEditor::GetSelectedOutputTile() const
{
	return SelectedOutputTileWeak.Pin();
}

void FAvaBroadcastEditor::OnClose()
{
	FWorkflowCentricApplication::OnClose();
	if (BroadcastEditor.IsValid() && this == BroadcastEditor.Get())
	{
		BroadcastEditor.Reset();
	}
}

void FAvaBroadcastEditor::InitBroadcastEditor(UAvaBroadcast* InBroadcast)
{
	BroadcastWeak = InBroadcast;
	
	if (InBroadcast)
	{
		InBroadcast->AddChangeListener(FOnAvaBroadcastChanged::FDelegate::CreateSP(this
			, &FAvaBroadcastEditor::OnBroadcastChanged));
	}

	UAvaMediaSettings& AvaMediaSettings = UAvaMediaSettings::GetMutable();
	AvaMediaSettings.OnSettingChanged().AddRaw(this, &FAvaBroadcastEditor::OnAvaMediaSettingsChanged);

	CreateDefaultCommands();
	
	const FName BroadcastEditorAppName(TEXT("MotionDesignBroadcastEditorApp"));
	constexpr bool bCreateDefaultStandaloneMenu = true;
	constexpr bool bCreateDefaultToolbar = true;

	InitAssetEditor(EToolkitMode::Standalone
		, nullptr
		, BroadcastEditorAppName
		, FTabManager::FLayout::NullLayout
		, bCreateDefaultStandaloneMenu
		, bCreateDefaultToolbar
		, InBroadcast);
	
	RegisterApplicationModes();
}

void FAvaBroadcastEditor::OnBroadcastChanged(EAvaBroadcastChange ChangedEvent)
{
	if (EnumHasAnyFlags(ChangedEvent, EAvaBroadcastChange::CurrentProfile))
	{
		TSharedPtr<FAvaBroadcastOutputTileItem> NullItem;
		SelectedOutputTileWeak = NullItem;
		OnOutputTileSelectionChanged.Broadcast(NullItem);
	}
}

void FAvaBroadcastEditor::OnAvaMediaSettingsChanged(UObject*, FPropertyChangedEvent&)
{
	if (BroadcastWeak.IsValid())
	{
		BroadcastWeak->GetCurrentProfile().UpdateChannels(true);
	}
}

void FAvaBroadcastEditor::SaveAsset_Execute()
{
	FWorkflowCentricApplication::SaveAsset_Execute();
	if (BroadcastWeak.IsValid())
	{
		BroadcastWeak->SaveBroadcast();
	}
}

FName FAvaBroadcastEditor::GetToolkitFName() const
{
	return TEXT("AvaBroadcastEditor");
}

FText FAvaBroadcastEditor::GetBaseToolkitName() const
{
	return LOCTEXT("BroadcastAppLabel", "Motion Design Broadcast Editor");
}

FText FAvaBroadcastEditor::GetToolkitName() const
{
	return LOCTEXT("BroadcastAppName", "Broadcast");
}

FText FAvaBroadcastEditor::GetToolkitToolTipText() const
{
	if (IAvaMediaModule::Get().IsPlaybackServerStarted())
	{
		return LOCTEXT("BroadcastServerAppToolTip", "Motion Design Broadcast: Playback Server running");
	}
	else if (IAvaMediaModule::Get().IsPlaybackClientStarted())
	{
		return LOCTEXT("BroadcastClientAppToolTip", "Motion Design Broadcast: Playback Client running");
	}
	return LOCTEXT("BroadcastAppToolTip", "Motion Design Broadcast");
}

FString FAvaBroadcastEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("BroadcastScriptPrefix", "Script ").ToString();
}

FLinearColor FAvaBroadcastEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.0f, 0.0f, 0.3f, 0.5f);
}

const FSlateBrush* FAvaBroadcastEditor::GetDefaultTabIcon() const
{
	return IAvaMediaEditorModule::Get().GetToolbarBroadcastButtonIcon().GetIcon();
}

UAvaBroadcast* FAvaBroadcastEditor::GetBroadcastObject() const
{
	return BroadcastWeak.Get();
}

void FAvaBroadcastEditor::ExtendToolBar(TSharedPtr<FExtender> Extender)
{
	Extender->AddToolBarExtension("Asset"
		, EExtensionHook::After
		, ToolkitCommands
		, FToolBarExtensionDelegate::CreateSP(this, &FAvaBroadcastEditor::FillPlayToolBar));
}

void FAvaBroadcastEditor::FillPlayToolBar(FToolBarBuilder& ToolBarBuilder)
{
	TWeakObjectPtr<UAvaBroadcast> Broadcast = GetBroadcastObject();

	//TODO: Change Lambdas to their Own Command Action
	ToolBarBuilder.BeginSection(TEXT("Player"));
	{
		ToolBarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &FAvaBroadcastEditor::AddChannel)
				, FCanExecuteAction::CreateSP(this, &FAvaBroadcastEditor::CanAddChannel))
			, NAME_None
			, LOCTEXT("NewChannel_Label", "New Channel")
			, LOCTEXT("NewChannel_ToolTip", "New Channel")
			, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus")
		);

		ToolBarBuilder.AddToolBarButton(FUIAction(FExecuteAction::CreateLambda([Broadcast]
				{
					Broadcast->StartBroadcast();
				}),
				FCanExecuteAction::CreateLambda([Broadcast]
				{
					return Broadcast.IsValid() && !Broadcast->IsBroadcastingAllChannels();
				}))
			, NAME_None
			, LOCTEXT("Play_Label", "Start All Channels")
			, LOCTEXT("Play_ToolTip", "Starts Broadcast on all Idle Channels")
			, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Play")
		);

		ToolBarBuilder.AddToolBarButton(FUIAction(FExecuteAction::CreateLambda([Broadcast]
				{
					Broadcast->StopBroadcast();
				}),
				FCanExecuteAction::CreateLambda([Broadcast]
				{
					return Broadcast.IsValid() && Broadcast->IsBroadcastingAnyChannel();
				}))
			, NAME_None
			, LOCTEXT("Stop_Label", "Stop All Channels")
			, LOCTEXT("Stop_ToolTip", "Stops Broadcast on all Live Channels")
			, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Stop")
		);
		
	}
	ToolBarBuilder.EndSection();
	ToolBarBuilder.AddSeparator();
	
	ToolBarBuilder.BeginSection(TEXT("Preview"));
	{
		ToolBarBuilder.AddToolBarButton(FUIAction(FExecuteAction::CreateLambda([Broadcast]
				{
					Broadcast->SetCanShowPreview(!Broadcast->CanShowPreview());
				}),
				FCanExecuteAction::CreateLambda([Broadcast]
				{
					return Broadcast.IsValid();
				}),
				FIsActionChecked::CreateLambda([Broadcast]
				{
					return Broadcast.IsValid() && Broadcast->CanShowPreview();
				}))
			, NAME_None
			, LOCTEXT("Preview_Label", "Show Channel Preview")
			, LOCTEXT("Preview_ToolTip", "Show Channel Preview")
			, FSlateIcon(FAppStyle::GetAppStyleSetName(), "StaticMeshEditor.SetRealtimePreview")
			, EUserInterfaceActionType::RadioButton
		);		
	}
	ToolBarBuilder.EndSection();
	ToolBarBuilder.AddSeparator();

	MakeProfilesToolbar(ToolBarBuilder);

	ToolBarBuilder.BeginSection(TEXT("PlaybackServices"));
	{
		// Add Start button for the playback client service.
		ToolBarBuilder.AddToolBarButton(FUIAction(FExecuteAction::CreateLambda([]
				{
					StartPlaybackClientAction();
				})
				, FCanExecuteAction::CreateLambda([]
				{
					return !IAvaMediaModule::Get().IsPlaybackClientStarted();
				})
				, FGetActionCheckState()
				, FIsActionButtonVisible::CreateLambda([]
				{
					return !IAvaMediaModule::Get().IsPlaybackClientStarted();
				})
			)
			, NAME_None
			, LOCTEXT("PlaybackClientStart_Label", "Start Client")
			, LOCTEXT("PlaybackClientStart_ToolTip", "Start Playback Client")
			, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Play")
		);

		// Add Stop button for the playback client service.
		ToolBarBuilder.AddToolBarButton(FUIAction(FExecuteAction::CreateLambda([]
				{
					IAvaMediaModule::Get().StopPlaybackClient();
				})
				, FCanExecuteAction::CreateLambda([]
				{
					return IAvaMediaModule::Get().IsPlaybackClientStarted();
				})
				, FGetActionCheckState()
				, FIsActionButtonVisible::CreateLambda([]
				{
					return IAvaMediaModule::Get().IsPlaybackClientStarted();
				})
			)
			, NAME_None
			, LOCTEXT("PlaybackClientStop_Label", "Stop Client")
			, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&FAvaBroadcastEditor::GetStopPlaybackClientTooltip))
			, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Stop") 
		);
		
    	ToolBarBuilder.AddToolBarButton(FUIAction(FExecuteAction::CreateLambda([]
    			{
    				IAvaMediaModule::Get().LaunchGameModeLocalPlaybackServer();
    			})
    			, FCanExecuteAction::CreateLambda([]
    			{
    				const IAvaMediaModule& AvaMediaModule = IAvaMediaModule::Get();
    				return AvaMediaModule.IsPlaybackClientStarted() && !AvaMediaModule.IsGameModeLocalPlaybackServerLaunched();
    			})
    			, FGetActionCheckState()
				, FIsActionButtonVisible::CreateLambda([]
				{
					return !IAvaMediaModule::Get().IsGameModeLocalPlaybackServerLaunched();
				})
			)
    		, NAME_None
    		, LOCTEXT("LaunchLocalServer_Label", "Launch Local Server")
    		, TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&FAvaBroadcastEditor::GetLaunchLocalServerTooltip))
    		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Play")
    	);

    	ToolBarBuilder.AddToolBarButton(FUIAction(FExecuteAction::CreateLambda([]
    			{
    				IAvaMediaModule::Get().StopGameModeLocalPlaybackServer();
    			})
    			, FCanExecuteAction::CreateLambda([]
    			{
    				return IAvaMediaModule::Get().IsGameModeLocalPlaybackServerLaunched();
    			})
    			, FGetActionCheckState()
				, FIsActionButtonVisible::CreateLambda([]
				{
					return IAvaMediaModule::Get().IsGameModeLocalPlaybackServerLaunched();
				})
			)
    		, NAME_None
    		, LOCTEXT("StopLocalServer_Label", "Stop Local Server")
    		, LOCTEXT("StopLocalServer_ToolTip", "Stops Game Mode Local Server Process")
    		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Stop")
    	);

		// Add "Reset Server" button for the playback client service.
		// Todo: Eventually, there will be a "servers" tab, showing connected servers
		// and we'll be able to have per-server reset. But for now, this UI only allows
		// to reset all the servers.
		ToolBarBuilder.AddToolBarButton(FUIAction(FExecuteAction::CreateLambda([]
				{
					IAvaPlaybackClient& PlaybackClient = IAvaMediaModule::Get().GetPlaybackClient();
					// Stop all transitions.
					PlaybackClient.RequestPlayableTransitionStop(FGuid(), NAME_None);
					// Stop and unload all playbacks.
					PlaybackClient.RequestPlayback(FGuid(), FSoftObjectPath(), FString(), EAvaPlaybackAction::Unload);
					// Stop all broadcast channels.
					PlaybackClient.RequestBroadcast(TEXT(""), FName(), TArray<UMediaOutput*>(), EAvaBroadcastAction::Stop);
				})
				, FCanExecuteAction::CreateLambda([]
				{
					IAvaMediaModule& MediaModule = IAvaMediaModule::Get();
					return MediaModule.IsPlaybackClientStarted() && MediaModule.GetPlaybackClient().GetNumConnectedServers() > 0;
				})
				, FGetActionCheckState()
				, FIsActionButtonVisible()
			)
			, NAME_None
			, LOCTEXT("PlaybackClientResetServers_Label", "Reset Server(s)")
			, LOCTEXT("PlaybackClientResetServers_ToolTip", "Reset (i.e. stop and unload) all playback instances and broadcast channels on all connected servers.")
			, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Stop")
		);
    }
    ToolBarBuilder.EndSection();
    ToolBarBuilder.AddSeparator();

	ToolBarBuilder.MakeWidget();
}

FText FAvaBroadcastEditor::GetCurrentProfileName() const
{
	return FText::FromName(UAvaBroadcast::Get().GetCurrentProfileName());
}

void FAvaBroadcastEditor::MakeProfilesToolbar(FToolBarBuilder& ToolBarBuilder)
{
	ToolBarBuilder.BeginSection(TEXT("Profiles"));
	{
		static const FCanExecuteAction DefaultCanExecuteAction(FCanExecuteAction::CreateLambda([]()
		{
			return !UAvaBroadcast::Get().IsBroadcastingAnyChannel();
		}));
		
		ToolBarBuilder.AddComboButton(FUIAction(FExecuteAction(), DefaultCanExecuteAction)
			, FOnGetContent::CreateSP(this, &FAvaBroadcastEditor::MakeProfileComboButton)
			, TAttribute<FText>(this, &FAvaBroadcastEditor::GetCurrentProfileName)
			, LOCTEXT("Profile_ToolTip", "Pick between different Profiles")
			, FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.Profile")
			, false);

		ToolBarBuilder.SetLabelVisibility(EVisibility::Collapsed);
		ToolBarBuilder.AddToolBarButton(FUIAction(FExecuteAction::CreateLambda([]()
				{
					FScopedTransaction Transaction(LOCTEXT("DuplicateProfile", "Duplicate Profile"));
					UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
					Broadcast.Modify();
					const bool bResult = Broadcast.DuplicateCurrentProfile();
					if (!bResult)
					{
						Transaction.Cancel();
					}
				})
				, DefaultCanExecuteAction)
			, NAME_None
			, TAttribute<FText>()
			, LOCTEXT("BroadcastDuplicateProfile_ToolTip", "Duplicate Profile")
			, FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Duplicate"));
		ToolBarBuilder.SetLabelVisibility(EVisibility::Visible);
	}
	ToolBarBuilder.EndSection();
	ToolBarBuilder.AddSeparator();
}

TSharedRef<SWidget> FAvaBroadcastEditor::MakeProfileComboButton()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection(TEXT("Profiles"), LOCTEXT("Profiles_Title", "Profiles"));
	{
		TArray<FName> ProfileNames = UAvaBroadcast::Get().GetProfileNames();
		for (const FName& ProfileName : ProfileNames)
		{
			MenuBuilder.AddWidget(SNew(SAvaBroadcastProfileEntry, ProfileName)
					.OnProfileEntrySelected(this, &FAvaBroadcastEditor::OnProfileSelected)
				, FText::GetEmpty()
				, false
				, true);
		}
	}
	MenuBuilder.EndSection();
	MenuBuilder.AddSeparator();

	MenuBuilder.BeginSection(TEXT("AddProfile"));
	{
		MenuBuilder.AddMenuEntry(LOCTEXT("AddProfile_Label", "New Profile...")
			, FText::GetEmpty()
			, FSlateIcon()
			, FUIAction(FExecuteAction::CreateSP(this, &FAvaBroadcastEditor::CreateNewProfile)));
	}	
	MenuBuilder.EndSection();
	
	return MenuBuilder.MakeWidget();
}

void FAvaBroadcastEditor::CreateNewProfile()
{
	if (UAvaBroadcast* const Broadcast = BroadcastWeak.Get())
	{
		FScopedTransaction Transaction(LOCTEXT("CreateProfile", "Create Profile"));
		Broadcast->Modify();
		Broadcast->CreateProfile(NAME_None);
	}
}

FReply FAvaBroadcastEditor::OnProfileSelected(FName InProfileName)
{
	FSlateApplication::Get().DismissAllMenus();

	FScopedTransaction Transaction(LOCTEXT("SetCurrentProfile", "Set Current Profile"));
	
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	Broadcast.Modify();

	if (Broadcast.SetCurrentProfile(InProfileName))
	{
		return FReply::Handled();
	}
	else
	{
		Transaction.Cancel();
		return FReply::Unhandled();
	}
}

void FAvaBroadcastEditor::RegisterApplicationModes()
{
	TArray<TSharedRef<FApplicationMode>> ApplicationModes;
	TSharedPtr<FAvaBroadcastEditor> This = SharedThis(this);
	
	ApplicationModes.Add(MakeShared<FAvaBroadcastDefaultMode>(This));
	//Can add more App Modes here
	
	for (const TSharedRef<FApplicationMode>& AppMode : ApplicationModes)
	{
		AddApplicationMode(AppMode->GetModeName(), AppMode);
	}

	SetCurrentMode(FAvaBroadcastAppMode::DefaultMode);
}

void FAvaBroadcastEditor::CreateDefaultCommands()
{
}

FText FAvaBroadcastEditor::GetStopPlaybackClientTooltip()
{
	if (UAvaMediaSettings::Get().bAutoStartPlaybackClient)
	{
		return LOCTEXT("PlaybackClientStopAuto_ToolTip", "Stop Playback Client (auto started)");
	}
	return LOCTEXT("PlaybackClientStopManual_ToolTip", "Stop Playback Client (manually started)");
}

void FAvaBroadcastEditor::StartPlaybackClientAction()
{
	IAvaMediaModule& AvaMediaModule = IAvaMediaModule::Get();

	// We can't run both playback client and server at the same time in the same process.
	// If the user ask for the client to start, we need to stop the server, but we'll
	// ask user confirmation.
	if (AvaMediaModule.IsPlaybackServerStarted())
	{
		const EAppReturnType::Type Reply = FMessageDialog::Open(EAppMsgType::YesNo
			, LOCTEXT("PlaybackServerStop_Message", "This action will stop the playback server, continue anyway?"));

		if (Reply != EAppReturnType::Type::Yes)
		{
			return;
		}
	}

	AvaMediaModule.StartPlaybackClient();
}

FText FAvaBroadcastEditor::GetLaunchLocalServerTooltip()
{
	if (IAvaMediaModule::Get().IsPlaybackClientStarted())
	{
		return LOCTEXT("LaunchLocalServerAuto_ToolTip", "Launches Game Mode Local Server Process");
	}
	return LOCTEXT("LaunchLocalServerManual_ToolTip", "Launches Game Mode Local Server Process (disabled, start client first)");
}

bool FAvaBroadcastEditor::CanAddChannel()
{
	const int32 ChannelCount = UAvaBroadcast::Get().GetCurrentProfile().GetChannels().Num();

	const UAvaMediaEditorSettings& MediaSettings = UAvaMediaEditorSettings::Get();

	return !MediaSettings.bBroadcastEnforceMaxChannelCount
		|| ChannelCount < MediaSettings.BroadcastMaxChannelCount;
}

void FAvaBroadcastEditor::AddChannel()
{
	if (UAvaBroadcast* const Broadcast = BroadcastWeak.Get())
	{
		FScopedTransaction Transaction(LOCTEXT("AddChannel", "Add Channel"));
		Broadcast->Modify();
		Broadcast->GetCurrentProfile().AddChannel();
	}
}

#undef LOCTEXT_NAMESPACE
