// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FAvaBroadcastOutputTileItem;
class FName;
class FReply;
class FText;
class FToolBarBuilder;
class FExtender;
class UAvaBroadcast;
class UMediaOutput;
class UObject;
struct FLinearColor;
struct FPropertyChangedEvent;
struct FSlateBrush;

class FAvaBroadcastEditor : public FWorkflowCentricApplication
{
	struct FPrivateToken { explicit FPrivateToken() = default; };
	
public:
	explicit FAvaBroadcastEditor(FPrivateToken) {}
	
	virtual ~FAvaBroadcastEditor() override;

	static void OpenBroadcastEditor();

	void SelectOutputTile(const TSharedPtr<FAvaBroadcastOutputTileItem>& InOutputTile);
	TSharedPtr<FAvaBroadcastOutputTileItem> GetSelectedOutputTile() const;
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnOutputItemSelectionChanged, const TSharedPtr<FAvaBroadcastOutputTileItem>&);
	FOnOutputItemSelectionChanged OnOutputTileSelectionChanged;
	
protected:
	void InitBroadcastEditor(UAvaBroadcast* InBroadcast);

	void OnBroadcastChanged(EAvaBroadcastChange ChangedEvent);
	void OnAvaMediaSettingsChanged(UObject*, FPropertyChangedEvent&);

	virtual void SaveAsset_Execute() override;
	virtual bool CanSaveAssetAs() const override { return false; }
	virtual bool CanFindInContentBrowser() const override { return false; }

	virtual void OnClose() override;

	//~ Begin IToolkit Interface
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	//~ End IToolkit Interface

	//~ Begin FAssetEditorToolkit Interface
	virtual const FSlateBrush* GetDefaultTabIcon() const override;
	//~ End FAssetEditorToolkit Interface

public:
	UAvaBroadcast* GetBroadcastObject() const;

	void ExtendToolBar(TSharedPtr<FExtender> Extender);
	void FillPlayToolBar(FToolBarBuilder& ToolBarBuilder);

	FText GetCurrentProfileName() const;
	void MakeProfilesToolbar(FToolBarBuilder& ToolBarBuilder);
	TSharedRef<SWidget> MakeProfileComboButton();
	
	void CreateNewProfile();
	FReply OnProfileSelected(FName InProfileName);

	bool CanAddChannel();
	void AddChannel();
	
protected:
	void RegisterApplicationModes();
	void CreateDefaultCommands();

	static FText GetStopPlaybackClientTooltip();
	static FText GetLaunchLocalServerTooltip();
	static void StartPlaybackClientAction();

protected:
	static TSharedPtr<FAvaBroadcastEditor> BroadcastEditor;
	
	TWeakObjectPtr<UAvaBroadcast> BroadcastWeak;
	
	TWeakPtr<FAvaBroadcastOutputTileItem> SelectedOutputTileWeak;	
};
