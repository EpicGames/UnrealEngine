// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/ChannelGrid/AvaBroadcastOutputTileItem.h"
#include "AvaMediaDefines.h"
#include "AvaMediaEditorStyle.h"
#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/Channel/AvaBroadcastOutputChannel.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputUtils.h"
#include "ClassIconFinder.h"
#include "DragDropOps/AvaBroadcastOutputTileItemDragDropOp.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "Layout/Geometry.h"
#include "MediaOutput.h"
#include "Styling/SlateBrush.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "AvaBroadcastOutputTileItem"

FAvaBroadcastOutputTileItem::FAvaBroadcastOutputTileItem(FName InChannelName, UMediaOutput* InMediaOutput)
	: ChannelName(InChannelName)
	, MediaOutput(InMediaOutput)
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FAvaBroadcastOutputTileItem::OnMediaOutputPropertyChanged);
	FAvaBroadcastOutputChannel::GetOnChannelChanged().AddRaw(this, &FAvaBroadcastOutputTileItem::OnChannelChanged);
	FAvaBroadcastOutputChannel::GetOnMediaOutputStateChanged().AddRaw(this, &FAvaBroadcastOutputTileItem::OnMediaOutputStateChanged);
	BroadcastChangedHandle = UAvaBroadcast::Get().AddChangeListener(
		FOnAvaBroadcastChanged::FDelegate::CreateRaw(this, &FAvaBroadcastOutputTileItem::OnBroadcastChanged));

	UpdateInfo();
}

FAvaBroadcastOutputTileItem::~FAvaBroadcastOutputTileItem()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
	FAvaBroadcastOutputChannel::GetOnChannelChanged().RemoveAll(this);
	FAvaBroadcastOutputChannel::GetOnMediaOutputStateChanged().RemoveAll(this);
	UAvaBroadcast::Get().RemoveChangeListener(BroadcastChangedHandle);
}

const FAvaBroadcastOutputChannel& FAvaBroadcastOutputTileItem::GetChannel() const
{
	return UAvaBroadcast::Get().GetCurrentProfile().GetChannel(ChannelName);
}

FAvaBroadcastOutputChannel& FAvaBroadcastOutputTileItem::GetChannel()
{
	return UAvaBroadcast::Get().GetCurrentProfile().GetChannelMutable(ChannelName);
}

TSharedRef<SWidget> FAvaBroadcastOutputTileItem::GenerateTile() const
{
	// Although not norminal, we allow the tile to be created. It will show status as "invalid".
	if (!MediaOutput.IsValid())
	{
		UE_LOGF(LogAvaBroadcast, Error, "Broadcast Output Tile generated with invalid output.");
	}

	return SNew(SHorizontalBox)
		.ToolTipText(this, &FAvaBroadcastOutputTileItem::GetToolTipText)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SImage)
			.Image(GetMediaOutputIcon())
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.f, 0.f, 0.f, 0.f)
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		.VAlign(EVerticalAlignment::VAlign_Fill)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(0.75f)
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			.VAlign(EVerticalAlignment::VAlign_Fill)
			[
				SNew(SScaleBox)
				.Stretch(EStretch::UserSpecified)
				.UserSpecifiedScale(1.25f)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &FAvaBroadcastOutputTileItem::GetDisplayText)
					.Justification(ETextJustify::Center)
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(0.25f)
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			.VAlign(EVerticalAlignment::VAlign_Fill)
			[
				SNew(SScaleBox)
				.Stretch(EStretch::ScaleToFitY)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(EVerticalAlignment::VAlign_Center)
					[
						SNew(SImage)
						.Image(this, &FAvaBroadcastOutputTileItem::GetMediaOutputStatusBrush)
					]
					+SHorizontalBox::Slot()
					.Padding(2.f, 0.f, 0.f, 0.f)
					.VAlign(EVerticalAlignment::VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(this, &FAvaBroadcastOutputTileItem::GetMediaOutputStatusText)
					]
				]
			]
		];
}

FText FAvaBroadcastOutputTileItem::GetDisplayText() const
{
	return MediaOutputDisplayText;
}

FText FAvaBroadcastOutputTileItem::GetMediaOutputStatusText() const
{
	return MediaOutputStatusText;
}

FText FAvaBroadcastOutputTileItem::GetToolTipText() const
{
	return MediaOutputToolTipText;
}

const FSlateBrush* FAvaBroadcastOutputTileItem::GetMediaOutputIcon() const
{
	const UClass* MediaOutputClass = MediaOutput.IsValid() ? MediaOutput->GetClass() : UMediaOutput::StaticClass();   
	return FClassIconFinder::FindThumbnailForClass(MediaOutputClass);
}

const FSlateBrush* FAvaBroadcastOutputTileItem::GetMediaOutputStatusBrush() const
{
	return MediaOutputStatusBrush;
}

FReply FAvaBroadcastOutputTileItem::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!UAvaBroadcast::Get().IsBroadcastingAnyChannel())
	{
		const bool bShouldDuplicate = MouseEvent.IsAltDown();
		
		const TSharedRef<FAvaBroadcastOutputTileItemDragDropOp> DragDropOp = FAvaBroadcastOutputTileItemDragDropOp::New(SharedThis(this)
				, bShouldDuplicate);
			
		return FReply::Handled().BeginDragDrop(DragDropOp);
	}
	return FReply::Unhandled();
}

void FAvaBroadcastOutputTileItem::OnMediaOutputPropertyChanged(UObject* InObject, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (MediaOutput.IsValid() && InObject == MediaOutput)
	{
		UpdateInfo();
	}
}

void FAvaBroadcastOutputTileItem::OnChannelChanged(const FAvaBroadcastOutputChannel& InChannel, EAvaBroadcastChannelChange InChange)
{
	if (EnumHasAnyFlags(InChange, EAvaBroadcastChannelChange::State)
		&& InChannel.IsValidChannel()
		&& InChannel.GetChannelName() == ChannelName)
	{
		UpdateInfo();
	}
}

void FAvaBroadcastOutputTileItem::OnMediaOutputStateChanged(const FAvaBroadcastOutputChannel& InChannel, const UMediaOutput* InMediaOutput)
{
	if (InChannel.IsValidChannel() && InChannel.GetChannelName() == ChannelName)
	{
		UpdateInfo();
	}
}

void FAvaBroadcastOutputTileItem::OnBroadcastChanged(EAvaBroadcastChange InChange)
{
	// When output devices are updated, some of their status can change from offline/idle.
	if (EnumHasAnyFlags(InChange, EAvaBroadcastChange::OutputDevices))
	{
		UpdateInfo();
	}
}


void FAvaBroadcastOutputTileItem::UpdateInfo()
{
	// The capture backend may fire events during tear down.
	if (!MediaOutput.IsValid())
	{
		MediaOutputStatusText = LOCTEXT("MediaOutputStatus_Invalid", "Invalid");
		MediaOutputStatusBrush = FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.MediaOutputOffline");

		// Only update if it was uninitialized, otherwise leave as is (may help debugging).
		if (MediaOutputDisplayText.IsEmpty())
		{
			MediaOutputDisplayText = LOCTEXT("MediaOutputDisplayName_Invalid", "Invalid Output");
		}
		return;
	}

	MediaOutputDisplayText = FindLatestDisplayText();

	const FAvaBroadcastOutputChannel& Channel = GetChannel();
	const EAvaBroadcastOutputState OutputState = Channel.GetMediaOutputState(MediaOutput.Get());
	const EAvaBroadcastIssueSeverity Severity = Channel.GetMediaOutputIssueSeverity(OutputState, MediaOutput.Get());
	switch(OutputState)
	{
	case EAvaBroadcastOutputState::Offline:
		MediaOutputStatusText = LOCTEXT("MediaOutput_Offline", "Offline");
		MediaOutputStatusBrush = FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.MediaOutputOffline");
		break;
	case EAvaBroadcastOutputState::Idle:
		MediaOutputStatusText = LOCTEXT("MediaOutput_Idle", "Idle");
		MediaOutputStatusBrush = FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.MediaOutputIdle");
		break;
	case EAvaBroadcastOutputState::Preparing:
		MediaOutputStatusText = LOCTEXT("MediaOutput_Preparing", "Preparing");
		MediaOutputStatusBrush = FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.MediaOutputPreparing");
		break;
	case EAvaBroadcastOutputState::Live:
		MediaOutputStatusText = LOCTEXT("MediaOutput_Live", "Live");
		if (Severity == EAvaBroadcastIssueSeverity::Errors || Severity == EAvaBroadcastIssueSeverity::Warnings)
		{
			// If severity is warning or error, add a secondary icon (yellow or red exclamation mark)
			// so the user knows to lookup the error in the tooltip.
			MediaOutputStatusBrush = FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.MediaOutputLiveWarn");
		}
		else
		{
			MediaOutputStatusBrush = FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.MediaOutputLive");
		}
		break;
	case EAvaBroadcastOutputState::Error:
	default:
		MediaOutputStatusText = LOCTEXT("MediaOutput_Errors", "Error(s)");
		MediaOutputStatusBrush = FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.MediaOutputError");
		break;
	}

	FString AllMessages;

	// Include the status of the server in the tooltip.
	const FAvaBroadcastMediaOutputInfo& OutputInfo = Channel.GetMediaOutputInfo(MediaOutput.Get());
	if (OutputInfo.IsValid())
	{
		if (OutputInfo.IsRemote())
		{
			AllMessages.Append(FString::Printf(TEXT("Remote Output on server \"%s\".\n"), *OutputInfo.ServerName));
		}
		else
		{
			AllMessages.Append(TEXT("Local Output.\n"));
		}
	}
	else
	{
		AllMessages.Append(TEXT("Invalid Output Info.\n"));
	}
	
	
	switch(Severity)
	{
	case EAvaBroadcastIssueSeverity::Errors:
		AllMessages.Append("Errors: \n");
		break;
	case EAvaBroadcastIssueSeverity::Warnings:
		AllMessages.Append(TEXT("Warning: \n"));
		break;
	default:
		if (OutputState == EAvaBroadcastOutputState::Offline)
		{
			AllMessages.Append(TEXT("Offline\n"));
		}
		else
		{
			AllMessages.Append(TEXT("Healthy\n"));
		}
		break;
	}
	
	const TArray<FString>& Messages = GetChannel().GetMediaOutputIssueMessages(MediaOutput.Get());
	for (const FString& Message : Messages)
	{
		AllMessages.Append(Message);
		AllMessages.Append(TEXT("\n"));
	}
	MediaOutputToolTipText = FText::FromString(AllMessages);
}

FText FAvaBroadcastOutputTileItem::FindLatestDisplayText() const
{
	check(MediaOutput.IsValid());

	FString ServerName;
	
	const FAvaBroadcastOutputChannel& Channel = GetChannel();
	if (Channel.IsValidChannel())
	{
		const FAvaBroadcastMediaOutputInfo& OutputInfo = Channel.GetMediaOutputInfo(MediaOutput.Get());
		if (OutputInfo.IsValid() && OutputInfo.IsRemote())
		{
			ServerName = OutputInfo.ServerName + TEXT(": ");
		}
	}

	FString DeviceName = UE::AvaBroadcastOutputUtils::GetDeviceName(MediaOutput.Get());
	
	if (DeviceName.IsEmpty())
	{
		DeviceName = MediaOutput->GetFName().ToString();
	}

	return ServerName.IsEmpty() ? FText::FromString(DeviceName) : FText::FromString(ServerName + DeviceName);
}

#undef LOCTEXT_NAMESPACE
