// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Broadcast/Channel/AvaBroadcastOutputChannel.h"
#include "Delegates/IDelegateInstance.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FAvaBroadcastOutputTileItem;
class FReply;
class SWidget;
class UObject;
class UMediaOutput;
struct FAvaBroadcastOutputChannel;
struct FGeometry;
struct FPointerEvent;
struct FPropertyChangedEvent;
struct FSlateBrush;
enum class EAvaBroadcastChange : uint8;

using FAvaBroadcastOutputTileItemPtr = TSharedPtr<FAvaBroadcastOutputTileItem>;

class FAvaBroadcastOutputTileItem : public TSharedFromThis<FAvaBroadcastOutputTileItem>
{
public:
	FAvaBroadcastOutputTileItem(FName InChannelName, UMediaOutput* InMediaOutput);
	~FAvaBroadcastOutputTileItem();
	
	const FAvaBroadcastOutputChannel& GetChannel() const;
	FAvaBroadcastOutputChannel& GetChannel();
	
	UMediaOutput* GetMediaOutput() const { return MediaOutput.Get(); }

	TSharedRef<SWidget> GenerateTile() const;	

	FText GetDisplayText() const;
	FText GetMediaOutputStatusText() const;
	FText GetToolTipText() const;

	const FSlateBrush* GetMediaOutputIcon() const;
	const FSlateBrush* GetMediaOutputStatusBrush() const;

	FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	
protected:
	void OnMediaOutputPropertyChanged(UObject* InObject, FPropertyChangedEvent& PropertyChangedEvent);
	void OnChannelChanged(const FAvaBroadcastOutputChannel& InChannel, EAvaBroadcastChannelChange InChange);
	void OnMediaOutputStateChanged(const FAvaBroadcastOutputChannel& InChannel, const UMediaOutput* InMediaOutput);
	void OnBroadcastChanged(EAvaBroadcastChange InChange);

	void UpdateInfo();
	
	FText FindLatestDisplayText() const;

protected:
	FName ChannelName = NAME_None;

	TWeakObjectPtr<UMediaOutput> MediaOutput;
	
	FText MediaOutputDisplayText;
	FText MediaOutputStatusText;
	FText MediaOutputToolTipText;
	
	const FSlateBrush* MediaOutputStatusBrush = nullptr;

	FDelegateHandle BroadcastChangedHandle;
};
