// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaBroadcastOutputTileItemDragDropOp.h"
#include "Broadcast/AvaBroadcast.h"
#include "Input/Reply.h"
#include "MediaOutput.h"
#include "ScopedTransaction.h"
#include "UObject/NameTypes.h"

#define LOCTEXT_NAMESPACE "AvaBroadcastOutputTileItemDragDropOp"

TSharedRef<FAvaBroadcastOutputTileItemDragDropOp> FAvaBroadcastOutputTileItemDragDropOp::New(const FAvaBroadcastOutputTileItemPtr& InOutputClassItem, bool bInIsDuplicating)
{
	TSharedRef<FAvaBroadcastOutputTileItemDragDropOp> DragDropOp = MakeShared<FAvaBroadcastOutputTileItemDragDropOp>();
	DragDropOp->Init(InOutputClassItem, bInIsDuplicating);
	return DragDropOp;
}

bool FAvaBroadcastOutputTileItemDragDropOp::IsValidToDropInChannel(FName InTargetChannelName) const
{
	if (!OutputTileItem.IsValid())
	{
		return false;
	}

	// Don't allow dropping a remote output in a preview channel
	if (UAvaBroadcast::Get().GetChannelType(InTargetChannelName) == EAvaBroadcastChannelType::Preview
		&& OutputTileItem->GetChannel().IsMediaOutputRemote(OutputTileItem->GetMediaOutput()))
	{
		return false;
	}
	
	// Allow Duplicate on Same Channel
	return (bIsDuplicating || OutputTileItem->GetChannel().GetChannelName() != InTargetChannelName);
}

FReply FAvaBroadcastOutputTileItemDragDropOp::OnChannelDrop(FName InTargetChannelName)
{
	if (TSharedPtr<FAvaBroadcastOutputTileItem> Tile = GetOutputTileItem())
	{
		UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
		
		FScopedTransaction Transaction(LOCTEXT("OnChannelDrop", "Drop Media Output"));
		Broadcast.Modify();
		
		UMediaOutput* const SourceMediaOutput = Tile->GetMediaOutput();

		if (!IsValid(SourceMediaOutput))
		{
			Transaction.Cancel();
			return FReply::Unhandled();
		}

		FAvaBroadcastOutputChannel& TargetChannel = Broadcast.GetCurrentProfile().GetChannelMutable(InTargetChannelName);
		FAvaBroadcastOutputChannel& SourceChannel = Tile->GetChannel();
		FAvaBroadcastMediaOutputInfo MediaOutputInfo = SourceChannel.GetMediaOutputInfo(SourceMediaOutput);

		// Don't allow dropping a remote output in a preview channel
		if (Broadcast.GetChannelType(InTargetChannelName) == EAvaBroadcastChannelType::Preview && MediaOutputInfo.IsRemote())
		{
			Transaction.Cancel();
			return FReply::Unhandled();
		}
		
		if (bIsDuplicating)
		{
			UMediaOutput* const NewMediaOutput = DuplicateObject<UMediaOutput>(SourceMediaOutput
				, SourceMediaOutput->GetOuter()
				, NAME_None);
			
			check(NewMediaOutput);
			MediaOutputInfo.Guid = FGuid::NewGuid();	// Allocate a new guid for the duplicate.
			TargetChannel.AddMediaOutput(NewMediaOutput, MediaOutputInfo);
		}
		else
		{
			SourceChannel.RemoveMediaOutput(SourceMediaOutput);
			TargetChannel.AddMediaOutput(SourceMediaOutput, MediaOutputInfo);
		}

		return FReply::Handled();
	}
	return FReply::Unhandled();
}


void FAvaBroadcastOutputTileItemDragDropOp::Init(const FAvaBroadcastOutputTileItemPtr& InOutputTileItem, bool bInIsDuplicating)
{
	OutputTileItem = InOutputTileItem;
	bIsDuplicating = bInIsDuplicating;
	
	CurrentHoverText = InOutputTileItem->GetDisplayText();
	CurrentIconBrush = InOutputTileItem->GetMediaOutputIcon();
	
	CurrentIconColorAndOpacity = FSlateColor::UseForeground();
	MouseCursor = EMouseCursor::GrabHandClosed;

	SetupDefaults();
	Construct();
}

#undef LOCTEXT_NAMESPACE
