// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Broadcast/ChannelGrid/AvaBroadcastOutputTileItem.h"
#include "DragAndDrop/DecoratedDragDropOp.h"

class FName;
class FReply;

class FAvaBroadcastOutputTileItemDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FAvaOutputTileDragDropOp, FDecoratedDragDropOp)
	
	static TSharedRef<FAvaBroadcastOutputTileItemDragDropOp> New(const FAvaBroadcastOutputTileItemPtr& InOutputClassItem, bool bInIsDuplicating);

	bool IsValidToDropInChannel(FName InTargetChannelName) const;
	
	FAvaBroadcastOutputTileItemPtr GetOutputTileItem() const { return OutputTileItem; }

	FReply OnChannelDrop(FName InTargetChannelName);
	
protected:
	void Init(const FAvaBroadcastOutputTileItemPtr& InOutputTileItem, bool bInIsDuplicating);

	//Keep Reference Count while Drag Dropping
	FAvaBroadcastOutputTileItemPtr OutputTileItem;

	bool bIsDuplicating = false;
};
