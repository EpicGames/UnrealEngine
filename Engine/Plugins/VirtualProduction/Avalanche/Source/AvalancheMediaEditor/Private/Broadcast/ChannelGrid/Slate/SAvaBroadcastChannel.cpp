// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaBroadcastChannel.h"
#include "AvaMediaEditorStyle.h"
#include "AvaMediaEditorUtils.h"
#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/AvaBroadcastEditor.h"
#include "Broadcast/Channel/AvaBroadcastOutputChannel.h"
#include "Broadcast/ChannelGrid/DragDropOps/AvaBroadcastOutputTileItemDragDropOp.h"
#include "Broadcast/OutputDevices/DragDropOps/AvaBroadcastOutputTreeItemDragDropOp.h"
#include "Broadcast/OutputDevices/Slate/SAvaBroadcastCaptureImage.h"
#include "Brushes/SlateImageBrush.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Input/DragAndDrop.h"
#include "Input/Events.h"
#include "Internationalization/Text.h"
#include "IStructureDetailsView.h"
#include "Layout/Geometry.h"
#include "Layout/Visibility.h"
#include "MediaOutput.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"
#include "UObject/NameTypes.h"

#define LOCTEXT_NAMESPACE "SAvaBroadcastChannel"

void SAvaBroadcastChannel::Construct(const FArguments& InArgs, const TSharedPtr<FAvaBroadcastEditor>& InBroadcastEditor)
{
	BroadcastEditorWeak = InBroadcastEditor;
	ChannelName = InArgs._ChannelName;
	CanMaximize = InArgs._CanMaximize;
	OnMaximizeClicked = InArgs._OnMaximizeClicked;
	
	RegisterCommands();
	
	FAvaBroadcastOutputChannel::GetOnChannelChanged().AddRaw(this, &SAvaBroadcastChannel::OnChannelChanged);
	InBroadcastEditor->OnOutputTileSelectionChanged.AddRaw(this, &SAvaBroadcastChannel::OnOutputTileSelectionChanged);

	FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetChannelMutable(ChannelName);
	QualitySettingsStruct = MakeShared<FStructOnScope>(FAvaBroadcastOutputChannel::StaticStruct(), reinterpret_cast<uint8*>(&Channel));

	CreateQualitySettingsDetails();

	const FMargin ButtonPadding = FMargin(2.f);
	
	ChildSlot
	[
		SNew(SOverlay)
		.Clipping(EWidgetClipping::ClipToBounds)
		+ SOverlay::Slot()
		[
			SNew(SBorder)
			.BorderImage(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>(TEXT("SceneOutliner.TableViewRow")).DropIndicator_Onto)
			.Visibility(this, &SAvaBroadcastChannel::GetChannelDragVisibility)
		]
		+ SOverlay::Slot()
		.Padding(2.0f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.WidgetBorder"))
			[
				MakeChannelVerticalBox()
			]
		]
		//Min/Maximize and X button
		+ SOverlay::Slot()
		.VAlign(EVerticalAlignment::VAlign_Top)
		.HAlign(EHorizontalAlignment::HAlign_Right)
		.Padding(0.0f, 5.f, 5.f, 0.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SComboButton)
				.ToolTipText(LOCTEXT("ChannelSettingToolTip", "Open the channel viewport quality settings"))
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ContentPadding(ButtonPadding)
				.HasDownArrow(false)
				.MenuPlacement(MenuPlacement_BelowAnchor)
				.OnGetMenuContent(this, &SAvaBroadcastChannel::GetQualitySettingsDetails)
				.ButtonContent()
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ContentPadding(ButtonPadding)
				.OnClicked(this, &SAvaBroadcastChannel::OnChannelPinButtonClicked)
				.ToolTipText(LOCTEXT("ChannelPinToolTip", "Pin the channel across all profiles."))
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(this, &SAvaBroadcastChannel::GetChannelPinBrush)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ContentPadding(ButtonPadding)
				.OnClicked(this, &SAvaBroadcastChannel::OnChannelMaximizeButtonClicked)
				.ToolTipText(this, &SAvaBroadcastChannel::GetChannelMaximizeRestoreTooltipText)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(this, &SAvaBroadcastChannel::GetChannelMaximizeRestoreBrush)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ContentPadding(ButtonPadding)
				.OnClicked(this, &SAvaBroadcastChannel::OnChannelRemoveButtonClicked)
				.IsEnabled(this, &SAvaBroadcastChannel::CanEditChanges)
				.ToolTipText(LOCTEXT("ChannelRemoveToolTip", "Remove this channel."))
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("Icons.X"))
				]
			]
		]
		//Status Button
		+ SOverlay::Slot()
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.VAlign(EVerticalAlignment::VAlign_Top)
		.Padding(5.f, 5.f, 0.f, 0.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ContentPadding(ButtonPadding)
				.OnClicked(this, &SAvaBroadcastChannel::OnChannelStatusButtonClicked)
				.ToolTipText(LOCTEXT("ChannelStatusChangeToolTip", "Change the status of this channel."))
				[
					SAssignNew(ChannelStatusOptions, SMenuAnchor)
					.OnGetMenuContent(this, &SAvaBroadcastChannel::GetChannelStatusOptions)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(this, &SAvaBroadcastChannel::GetChannelStatusBrush)
					]				
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(2.f, 0.f, 0.f, 0.f)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SAvaBroadcastChannel::GetChannelStatusText)
				.ToolTipText_Lambda([this]()
					{
						return FText::Format(LOCTEXT("ChannelStatusToolTip", "Current Channel Status: {0}"), GetChannelStatusText());
					})
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ContentPadding(ButtonPadding)
				.OnClicked(this, &SAvaBroadcastChannel::OnChannelTypeToggleButtonClicked)
				.ToolTipText(LOCTEXT("ChannelTypeToggleToolTip", "Toggles between \"Program\" or \"Preview\" channel type."))
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(this, &SAvaBroadcastChannel::GetChannelTypeBrush)
				]
			]
		]
	];

	OutputTileListView->SetStyle(nullptr);	

	OnChannelBroadcastStateChanged(Channel);
	OnChannelMediaOutputsChanged(Channel);
}

SAvaBroadcastChannel::~SAvaBroadcastChannel()
{
	FAvaBroadcastOutputChannel::GetOnChannelChanged().RemoveAll(this);	
	if (TSharedPtr<FAvaBroadcastEditor> BroadcastEditor = BroadcastEditorWeak.Pin())
	{
		BroadcastEditor->OnOutputTileSelectionChanged.RemoveAll(this);
	}
}

void SAvaBroadcastChannel::RegisterCommands()
{
	MediaOutputCommandList = MakeShared<FUICommandList>();
	
	MediaOutputCommandList->MapAction(FGenericCommands::Get().Delete
		, FExecuteAction::CreateSP(this, &SAvaBroadcastChannel::DeleteSelectedOutputTiles)
		, FCanExecuteAction::CreateSP(this, &SAvaBroadcastChannel::CanEditChanges));
}

void SAvaBroadcastChannel::SetPosition(int32 InColumnIndex, int32 InRowIndex)
{
	ColumnIndex= InColumnIndex;
	RowIndex = InRowIndex;
}

void SAvaBroadcastChannel::OnChannelChanged(const FAvaBroadcastOutputChannel& InChannel, EAvaBroadcastChannelChange InChange)
{
	if (EnumHasAnyFlags(InChange, EAvaBroadcastChannelChange::State))
	{
		OnChannelBroadcastStateChanged(InChannel);
	}
	if (EnumHasAnyFlags(InChange, EAvaBroadcastChannelChange::RenderTarget))
	{
		OnChannelRenderTargetChanged(InChannel);
	}
	if (EnumHasAnyFlags(InChange, EAvaBroadcastChannelChange::MediaOutputs))
	{
		OnChannelMediaOutputsChanged(InChannel);
	}
}

void SAvaBroadcastChannel::OnChannelBroadcastStateChanged(const FAvaBroadcastOutputChannel& InChannel)
{	
	if (InChannel.IsValidChannel() && InChannel.GetChannelName() == ChannelName)
	{
		ChannelState = InChannel.GetState();
		ChannelStateText = FAvaMediaEditorUtils::GetChannelStatusText(ChannelState, InChannel.GetIssueSeverity());
		ChannelStatusBrush = FAvaMediaEditorUtils::GetChannelStatusBrush(ChannelState, InChannel.GetIssueSeverity());
		
		if (ChannelState != EAvaBroadcastChannelState::Live)
		{
			ChannelPreviewBrush.Reset();
		}

		//Update ChannelPreviewBrush when Visibility Changes. It might be we haven't created it yet.
		OnChannelRenderTargetChanged(InChannel);
	}
}

void SAvaBroadcastChannel::OnChannelRenderTargetChanged(const FAvaBroadcastOutputChannel& InChannel)
{
	if (InChannel.IsValidChannel() && InChannel.GetChannelName() == ChannelName)
	{
		UTextureRenderTarget2D* const RenderTarget = InChannel.GetCurrentRenderTarget(true);
		
		//Only Invert Alpha if the Current Render Target isn't the Placeholder RT, since Placeholder uses Widget Rendering.
		//TODO: Need to check whether the Channel or Media Outputs is doing Invert Alpha rather than figuring it out here
		bShouldInvertAlpha = InChannel.GetPlaceholderRenderTarget() != RenderTarget;

		if (RenderTarget)
		{
			const FVector2D RenderTargetSize(RenderTarget->SizeX, RenderTarget->SizeY);
			
			//If Brush is invalid, or the Brush's Texture Target doesn't match the new Render Target, reset the Brush.
			if (!ChannelPreviewBrush.IsValid() || ChannelPreviewBrush->GetResourceObject() != RenderTarget)
			{
				ChannelPreviewBrush = MakeShared<FSlateImageBrush>(RenderTarget, RenderTargetSize);
			}
			//If Sizes mismatch, just resizes rather than recreating the Brush with same underlying Resource
			else if (ChannelPreviewBrush->GetImageSize() !=  RenderTargetSize)
			{
				ChannelPreviewBrush->SetImageSize(RenderTargetSize);
			}
		}
	}
}

void SAvaBroadcastChannel::OnChannelMediaOutputsChanged(const FAvaBroadcastOutputChannel& InChannel)
{
	if (InChannel.IsValidChannel() && InChannel.GetChannelName() == ChannelName)
	{
		TSet<UMediaOutput*> CurrentMediaOutputs(InChannel.GetMediaOutputs());
		
		TSet<UMediaOutput*> SeenMediaOutputs;
		SeenMediaOutputs.Reserve(OutputTileItems.Num());
		
		//Remove Items no longer in the Media Outputs List
		for (TArray<FAvaBroadcastOutputTileItemPtr>::TIterator Iter = OutputTileItems.CreateIterator(); Iter; ++Iter)
		{
			const FAvaBroadcastOutputTileItemPtr& Item = *Iter;

			//Remove if Item or Underlying Media Output is Invalid, 
			if (!Item.IsValid())
			{
				Iter.RemoveCurrent();
				continue;
			}

			UMediaOutput* const MediaOutput = Item->GetMediaOutput();
			if (!IsValid(MediaOutput) || !CurrentMediaOutputs.Contains(MediaOutput))
			{
				Iter.RemoveCurrent();
			}
			else
			{
				SeenMediaOutputs.Add(MediaOutput);
			}
		}

		//Add New Media Outputs as Item in the List
		{
			TArray<UMediaOutput*> NewMediaOutputs = CurrentMediaOutputs.Difference(SeenMediaOutputs).Array();
			OutputTileItems.Reserve(OutputTileItems.Num() + NewMediaOutputs.Num());
			
			for (UMediaOutput* const MediaOutput : NewMediaOutputs)
			{
				TSharedPtr<FAvaBroadcastOutputTileItem> Item = MakeShared<FAvaBroadcastOutputTileItem>(ChannelName, MediaOutput);
				OutputTileItems.Add(Item);
			}
		}
	
		if (OutputTileListView.IsValid())
		{
			OutputTileListView->RequestListRefresh();
		}
	}
}

void SAvaBroadcastChannel::OnOutputTileSelectionChanged(const TSharedPtr<FAvaBroadcastOutputTileItem>& Item)
{
	if (OutputTileListView.IsValid() && !OutputTileItems.Contains(Item))
	{
		//Clear Selection without Notifying
		OutputTileListView->Private_ClearSelection();
	}
}

void SAvaBroadcastChannel::DeleteSelectedOutputTiles()
{
	if (OutputTileListView.IsValid() && OutputTileListView->GetNumItemsSelected() > 0)
	{
		TArray<FAvaBroadcastOutputTileItemPtr> SelectedTiles = OutputTileListView->GetSelectedItems();
		TArray<UMediaOutput*> MediaOutputs;
		MediaOutputs.Reserve(SelectedTiles.Num());

		for (const FAvaBroadcastOutputTileItemPtr& SelectedTile : SelectedTiles)
		{
			if (SelectedTile.IsValid())
			{
				MediaOutputs.Add(SelectedTile->GetMediaOutput());
			}
		}

		FScopedTransaction Transaction(LOCTEXT("RemoveMediaOutputs", "Remove Media Outputs"));
		
		UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
		Broadcast.Modify();
		
		const int32 RemovedCount = Broadcast.GetCurrentProfile().RemoveChannelMediaOutputs(ChannelName, MediaOutputs);
		
		if (RemovedCount == 0)
		{
			Transaction.Cancel();
		}
	}
}

FText SAvaBroadcastChannel::GetChannelStatusText() const
{
	return ChannelStateText;
}

TSharedRef<SWidget> SAvaBroadcastChannel::GetChannelStatusOptions()
{
	FMenuBuilder Builder(true, nullptr);
	
	Builder.BeginSection("Status", LOCTEXT("BroadcastStatus", "Broadcast Status"));
	{
		const FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetChannel(ChannelName);
		// An offline channel can't be made live from here.
		if (Channel.GetState() != EAvaBroadcastChannelState::Offline)
		{
			for (uint8 Index = 0; Index < static_cast<uint8>(EAvaBroadcastChannelState::Max); ++Index)
			{
				const EAvaBroadcastChannelState State = static_cast<EAvaBroadcastChannelState>(Index);

				// Only add other states that the broadcast isn't currently in (or offline).
				if (Channel.GetState() != State && State != EAvaBroadcastChannelState::Offline)
				{
					Builder.AddMenuEntry(StaticEnum<EAvaBroadcastChannelState>()->GetDisplayNameTextByIndex(Index)
						, FText::GetEmpty()
						, FSlateIcon()
						, FUIAction(FExecuteAction::CreateSP(this, &SAvaBroadcastChannel::OnChannelStatusSelected, State)));	
				}
			}
		}
	}
	Builder.EndSection();
	
	return Builder.MakeWidget();
}

TSharedRef<SWidget> SAvaBroadcastChannel::MakeChannelVerticalBox()
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(0.1f)
		.Padding(0.f, 2.f, 0.f, 2.f)
		.VAlign(EVerticalAlignment::VAlign_Fill)
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		.MaxHeight(30.f)
		[
			SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFitY)
			[
				SAssignNew(ChannelNameTextBlock, SInlineEditableTextBlock)
				.Text(this, &SAvaBroadcastChannel::GetChannelNameText)
				.OnVerifyTextChanged(this, &SAvaBroadcastChannel::OnVerifyChannelNameTextChanged)
				.OnTextCommitted(this, &SAvaBroadcastChannel::OnChannelNameTextCommitted)
				.IsReadOnly(this, &SAvaBroadcastChannel::IsReadOnly)
				.Justification(ETextJustify::Center)
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(0.15f)
		.MaxHeight(50.f)
		.VAlign(EVerticalAlignment::VAlign_Fill)
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Center)
			[
				SNew(SScaleBox)
				.Stretch(EStretch::ScaleToFitY)
				[
					MakeMediaOutputsTileView()
				]
			]
			+ SOverlay::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				SNew(STextBlock)
				.Visibility(this, &SAvaBroadcastChannel::GetMediaOutputEmptyTextVisibility)
				.Text(LOCTEXT("NoMediaOutputsFound", "No Media Outputs. Drag and Drop from the Output Devices List."))
				.TextStyle(FAppStyle::Get(), "HintText")
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(0.75f)
		.VAlign(EVerticalAlignment::VAlign_Fill)
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		[
			SNew(SBorder)
			.Visibility(this, &SAvaBroadcastChannel::GetChannelPreviewVisibility)
			[
				SNew(SScaleBox)
				.Stretch(EStretch::ScaleToFit)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.HAlign(EHorizontalAlignment::HAlign_Center)
				[
					SNew(SAvaBroadcastCaptureImage)
					.ImageArgs(SImage::FArguments()
						.Image(this, &SAvaBroadcastChannel::GetChannelPreviewBrush))
					.ShouldInvertAlpha(this, &SAvaBroadcastChannel::ShouldInvertAlpha)
					.EnableGammaCorrection(false)
					.EnableBlending(false)
				]
			]
		];		
}

TSharedRef<SWidget> SAvaBroadcastChannel::MakeMediaOutputsTileView()
{
	return SAssignNew(OutputTileListView, SListView<FAvaBroadcastOutputTileItemPtr>)
		.Orientation(EOrientation::Orient_Horizontal)
		.ScrollbarVisibility(EVisibility::Collapsed)
		.SelectionMode(ESelectionMode::Single)
		.ListItemsSource(&OutputTileItems)
		.OnGenerateRow(this, &SAvaBroadcastChannel::OnGenerateMediaOutputTile)
		.OnSelectionChanged(this, &SAvaBroadcastChannel::OnMediaOutputTileSelectionChanged)
		.OnContextMenuOpening(this, &SAvaBroadcastChannel::OnMediaOutputTileContextMenuOpening)
	;
}

TSharedRef<ITableRow> SAvaBroadcastChannel::OnGenerateMediaOutputTile(FAvaBroadcastOutputTileItemPtr Item, const TSharedRef<STableViewBase>& InOwnerTable) const
{
	check(Item.IsValid());
	return SNew(STableRow<FAvaBroadcastOutputTileItemPtr>, InOwnerTable)
		.Padding(FMargin(5.f, 5.f))
		.OnDragDetected(Item.ToSharedRef(), &FAvaBroadcastOutputTileItem::OnDragDetected)
		[
			Item->GenerateTile()
		];
}

void SAvaBroadcastChannel::OnMediaOutputTileSelectionChanged(FAvaBroadcastOutputTileItemPtr Item, ESelectInfo::Type SelectInfo)
{
	if (TSharedPtr<FAvaBroadcastEditor> BroadcastEditor = BroadcastEditorWeak.Pin())
	{
		BroadcastEditor->SelectOutputTile(Item);
	}
}

TSharedPtr<SWidget> SAvaBroadcastChannel::OnMediaOutputTileContextMenuOpening() const
{
	FMenuBuilder MenuBuilder(true, MediaOutputCommandList);
	MenuBuilder.BeginSection("Edit", LOCTEXT("Edit", "Edit"));
	{
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
	}
	MenuBuilder.EndSection();
	return MenuBuilder.MakeWidget();
}

void SAvaBroadcastChannel::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	SetDragging(false);
	if (CanEditChanges())
	{
		if (const TSharedPtr<FAvaBroadcastOutputTreeItemDragDropOp> OutputClassDragDropOp = DragDropEvent.GetOperationAs<FAvaBroadcastOutputTreeItemDragDropOp>())
		{
			if (OutputClassDragDropOp->IsValidToDropInChannel(ChannelName))
			{
				SetDragging(true);
			}
		}
		else if (const TSharedPtr<FAvaBroadcastOutputTileItemDragDropOp> OutputTileDragDropOp = DragDropEvent.GetOperationAs<FAvaBroadcastOutputTileItemDragDropOp>())
		{
			if (OutputTileDragDropOp->IsValidToDropInChannel(ChannelName))
			{
				SetDragging(true);
			}
		}
	}
}

void SAvaBroadcastChannel::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	SetDragging(false);
}

FReply SAvaBroadcastChannel::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	SetDragging(false);
	if (CanEditChanges())
	{
		if (const TSharedPtr<FAvaBroadcastOutputTreeItemDragDropOp> OutputItemDragDropOp = DragDropEvent.GetOperationAs<FAvaBroadcastOutputTreeItemDragDropOp>())
		{
			OutputItemDragDropOp->OnChannelDrop(ChannelName);
		}
		else if (const TSharedPtr<FAvaBroadcastOutputTileItemDragDropOp> OutputTileDragDropOp = DragDropEvent.GetOperationAs<FAvaBroadcastOutputTileItemDragDropOp>())
		{
			OutputTileDragDropOp->OnChannelDrop(ChannelName);
		}
	}
	return FReply::Unhandled();
}

FReply SAvaBroadcastChannel::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (MediaOutputCommandList.IsValid() && MediaOutputCommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SAvaBroadcastChannel::SetDragging(bool bIsDragging)
{
	DragVisibility = bIsDragging
		? EVisibility::SelfHitTestInvisible
		: EVisibility::Hidden;
}

FText SAvaBroadcastChannel::GetChannelNameText() const
{
	return FText::FromName(ChannelName);
}

bool SAvaBroadcastChannel::OnVerifyChannelNameTextChanged(const FText& InText, FText& OutErrorMessage)
{
	return UAvaBroadcast::Get().CanRenameChannel(ChannelName, FName(InText.ToString()), OutErrorMessage);
}

void SAvaBroadcastChannel::OnChannelNameTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	const FName NewChannelName(InText.ToString());

	FScopedTransaction Transaction(LOCTEXT("RenameChannel", "Rename Channel"));
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	Broadcast.Modify();
	
	if (Broadcast.RenameChannel(ChannelName, NewChannelName))
	{
		ChannelName = NewChannelName;
	}
	else
	{
		Transaction.Cancel();
	}
}

FReply SAvaBroadcastChannel::OnChannelStatusButtonClicked()
{
	if (ChannelStatusOptions.IsValid() && !ChannelStatusOptions->IsOpen())
	{
		ChannelStatusOptions->SetIsOpen(true);
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SAvaBroadcastChannel::OnChannelPinButtonClicked()
{
	FScopedTransaction Transaction(LOCTEXT("ToggleChannelPin", "Toggle Channel Pin"));
	
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();	
	Broadcast.Modify();
	if (!Broadcast.IsChannelPinned(ChannelName))
	{
		Broadcast.PinChannel(ChannelName, Broadcast.GetCurrentProfileName());
	}
	else
	{
		Broadcast.UnpinChannel(ChannelName);
	}
	Broadcast.RebuildProfiles();
	Broadcast.QueueNotifyChange(EAvaBroadcastChange::ChannelGrid);
	Broadcast.GetOnChannelsListChanged().Broadcast(Broadcast.GetCurrentProfile());
	
	return FReply::Handled();
}

FReply SAvaBroadcastChannel::OnChannelTypeToggleButtonClicked()
{
	FScopedTransaction Transaction(LOCTEXT("ToggleChannelType", "Toggle Channel Type"));

	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	Broadcast.Modify();

	const EAvaBroadcastChannelType NewChannelType = (Broadcast.GetChannelType(ChannelName) == EAvaBroadcastChannelType::Preview)
		? EAvaBroadcastChannelType::Program : EAvaBroadcastChannelType::Preview;
	Broadcast.SetChannelType(ChannelName, NewChannelType);
	Broadcast.QueueNotifyChange(EAvaBroadcastChange::ChannelType);
	Broadcast.GetOnChannelsListChanged().Broadcast(Broadcast.GetCurrentProfile());

	return FReply::Handled();
}

FReply SAvaBroadcastChannel::OnChannelMaximizeButtonClicked()
{
	if (OnMaximizeClicked.IsBound())
	{
		OnMaximizeClicked.Execute(SharedThis(this));
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SAvaBroadcastChannel::OnChannelRemoveButtonClicked()
{
	FScopedTransaction Transaction(LOCTEXT("RemoveChannel", "Remove Channel"));
	
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	Broadcast.Modify();
	
	if (Broadcast.GetCurrentProfile().RemoveChannel(ChannelName))
	{
		return FReply::Handled();
	}

	Transaction.Cancel();
	return FReply::Unhandled();
}

void SAvaBroadcastChannel::OnChannelStatusSelected(EAvaBroadcastChannelState State)
{
	FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetChannelMutable(ChannelName);
	if (Channel.IsValidChannel())
	{
		switch (State)
		{
		case EAvaBroadcastChannelState::Idle:
			Channel.StopChannelBroadcast();
			break;
		case EAvaBroadcastChannelState::Live:
			Channel.StartChannelBroadcast();
			break;
		}
	}
}

const FSlateBrush* SAvaBroadcastChannel::GetChannelStatusBrush() const
{
	return ChannelStatusBrush;
}

const FSlateBrush* SAvaBroadcastChannel::GetChannelPreviewBrush() const
{
	return ChannelPreviewBrush.Get();
}

const FSlateBrush* SAvaBroadcastChannel::GetChannelPinBrush() const
{
	return UAvaBroadcast::Get().IsChannelPinned(ChannelName) ?
		FAppStyle::Get().GetBrush("Icons.Pinned") : FAppStyle::Get().GetBrush("Icons.Unpinned");
}

const FSlateBrush* SAvaBroadcastChannel::GetChannelMaximizeRestoreBrush() const
{
	FName BrushName = TEXT("EditorViewport.Front");
	if (CanMaximize.IsBound() && !CanMaximize.Get())
	{
		BrushName = TEXT("LevelEditor.Tabs.Viewports");
	}
	return FAppStyle::Get().GetBrush(BrushName);
}

const FSlateBrush* SAvaBroadcastChannel::GetChannelTypeBrush() const
{
	return UAvaBroadcast::Get().GetChannelType(ChannelName) == EAvaBroadcastChannelType::Preview ?
		FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.ChannelTypePreview")
		: FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.ChannelTypeProgram");
}

EVisibility SAvaBroadcastChannel::GetMediaOutputEmptyTextVisibility() const
{
	return OutputTileItems.IsEmpty()
		? EVisibility::Visible
		: EVisibility::Hidden;
}

EVisibility SAvaBroadcastChannel::GetChannelPreviewVisibility() const
{
	return UAvaBroadcast::Get().CanShowPreview()
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

EVisibility SAvaBroadcastChannel::GetChannelDragVisibility() const
{
	return DragVisibility;
}

bool SAvaBroadcastChannel::ShouldInvertAlpha() const
{
	return bShouldInvertAlpha;
}

bool SAvaBroadcastChannel::IsReadOnly() const
{
	return ChannelState == EAvaBroadcastChannelState::Live;
}

bool SAvaBroadcastChannel::CanEditChanges() const
{
	return !IsReadOnly();
}

FText SAvaBroadcastChannel::GetChannelMaximizeRestoreTooltipText() const
{
	FText TooltipText = LOCTEXT("ChannelMaximizeToolTip", "Maximize this channel in the view. All other channels will be hidden.");
	if (CanMaximize.IsBound() && !CanMaximize.Get())
	{
		TooltipText = LOCTEXT("ChannelRestoreToolTip", "Restore this channel in the view. All other channels will be visible.");
	}
	return TooltipText;
}

void SAvaBroadcastChannel::CreateQualitySettingsDetails()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowOptions = false;

	QualitySettingsDetails = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, FStructureDetailsViewArgs(), QualitySettingsStruct);
	QualitySettingsDetails->GetOnFinishedChangingPropertiesDelegate().AddSP(this, &SAvaBroadcastChannel::OnChannelQualitySettingsChanged);
}

void SAvaBroadcastChannel::OnChannelQualitySettingsChanged(const FPropertyChangedEvent& InEvent)
{
	if (QualitySettingsStruct.IsValid())
	{
		const FAvaBroadcastOutputChannel& Channel = *reinterpret_cast<const FAvaBroadcastOutputChannel*>(QualitySettingsStruct->GetStructMemory());
		FAvaBroadcastOutputChannel::GetOnChannelChanged().Broadcast(Channel, EAvaBroadcastChannelChange::Settings);
	}
}

TSharedRef<SWidget> SAvaBroadcastChannel::GetQualitySettingsDetails()
{
	// We need to recreate the details view to update presets from the config the user may have changed
	CreateQualitySettingsDetails();

	return SNew(SBox)
		.MaxDesiredHeight(500.f)
		.MinDesiredWidth(120.f)
		[
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			[
				QualitySettingsDetails->GetWidget().ToSharedRef()
			]
		];
}

#undef LOCTEXT_NAMESPACE
