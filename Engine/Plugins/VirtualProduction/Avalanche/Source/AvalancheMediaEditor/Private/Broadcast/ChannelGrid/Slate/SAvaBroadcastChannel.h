// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "Broadcast/Channel/AvaBroadcastOutputChannel.h"
#include "Broadcast/ChannelGrid/AvaBroadcastOutputTileItem.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/SCompoundWidget.h"

class FAvaBroadcastEditor;
class FName;
class FReply;
class FStructOnScope;
class FText;
class FUICommandList;
class IStructureDetailsView;
class ITableRow;
class SAvaBroadcastChannel;
class SAvaViewportQualitySettings;
class SInlineEditableTextBlock;
class SlateBrush;
class SMenuAnchor;
class STableViewBase;
class SWidget;
class UMediaOutput;
struct EVisibility;
struct FAvaBroadcastOutputChannel;
struct FGeometry;
struct FSlateImageBrush;
template<typename ItemType> class SListView;

DECLARE_DELEGATE_RetVal(bool, FAvaBroadcastCanMaximizeChannel);
DECLARE_DELEGATE_OneParam(FAvaBroadcastOnChannelMaximizeClicked, const TSharedRef<SAvaBroadcastChannel>&);

class SAvaBroadcastChannel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaBroadcastChannel) {}
		SLATE_ARGUMENT(FName, ChannelName)
		SLATE_ATTRIBUTE(bool, CanMaximize)
		SLATE_EVENT(FAvaBroadcastOnChannelMaximizeClicked, OnMaximizeClicked)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedPtr<FAvaBroadcastEditor>& InBroadcastEditor);
	virtual ~SAvaBroadcastChannel() override;

	void RegisterCommands();
	FName GetChannelName() const { return ChannelName; }

	void SetPosition(int32 InColumnIndex, int32 InRowIndex);
	int32 GetRowIndex() const { return RowIndex; }
	int32 GetColumnIndex() const { return ColumnIndex; }

	void OnChannelChanged(const FAvaBroadcastOutputChannel& InChannel, EAvaBroadcastChannelChange InChange);
	
	void OnChannelBroadcastStateChanged(const FAvaBroadcastOutputChannel& InChannel);
	void OnChannelRenderTargetChanged(const FAvaBroadcastOutputChannel& InChannel);
	void OnChannelMediaOutputsChanged(const FAvaBroadcastOutputChannel& InChannel);
	
	void OnOutputTileSelectionChanged(const TSharedPtr<FAvaBroadcastOutputTileItem>& InMediaOutputItem);
	
	void DeleteSelectedOutputTiles();

	FText GetChannelStatusText() const;
	
	TSharedRef<SWidget> GetChannelStatusOptions();
	TSharedRef<SWidget> MakeChannelVerticalBox();
	TSharedRef<SWidget> MakeMediaOutputsTileView();

	TSharedRef<ITableRow> OnGenerateMediaOutputTile(FAvaBroadcastOutputTileItemPtr Item, const TSharedRef<STableViewBase>& InOwnerTable) const;
	void OnMediaOutputTileSelectionChanged(FAvaBroadcastOutputTileItemPtr Item,	ESelectInfo::Type SelectInfo);
	TSharedPtr<SWidget> OnMediaOutputTileContextMenuOpening() const;
	
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;;
	
	void SetDragging(bool bIsDragging);

	FText GetChannelNameText() const;
	bool OnVerifyChannelNameTextChanged(const FText& InText, FText& OutErrorMessage);
	void OnChannelNameTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);

	FReply OnChannelStatusButtonClicked();
	FReply OnChannelPinButtonClicked();
	FReply OnChannelTypeToggleButtonClicked();
	FReply OnChannelMaximizeButtonClicked();
	FReply OnChannelRemoveButtonClicked();

	void OnChannelStatusSelected(EAvaBroadcastChannelState State);

	const FSlateBrush* GetChannelStatusBrush() const;
	const FSlateBrush* GetChannelPreviewBrush() const;
	const FSlateBrush* GetChannelPinBrush() const;
	const FSlateBrush* GetChannelMaximizeRestoreBrush() const;
	const FSlateBrush* GetChannelTypeBrush() const;

	EVisibility GetMediaOutputEmptyTextVisibility() const;
	EVisibility GetChannelPreviewVisibility() const;
	EVisibility GetChannelDragVisibility() const;

	bool ShouldInvertAlpha() const;

	bool IsReadOnly() const;
	bool CanEditChanges() const;
	
protected:
	FText GetChannelMaximizeRestoreTooltipText() const;

	void CreateQualitySettingsDetails();

	void OnChannelQualitySettingsChanged(const FPropertyChangedEvent& InEvent);

	TSharedRef<SWidget> GetQualitySettingsDetails();

	FName ChannelName = NAME_None;

	TSharedPtr<SMenuAnchor> ChannelStatusOptions;
	
	TSharedPtr<SListView<FAvaBroadcastOutputTileItemPtr>> OutputTileListView;
	
	TSharedPtr<FStructOnScope> QualitySettingsStruct;

	TSharedPtr<IStructureDetailsView> QualitySettingsDetails;

	TArray<FAvaBroadcastOutputTileItemPtr> OutputTileItems;
	
	TSharedPtr<FSlateImageBrush> ChannelPreviewBrush;

	const FSlateBrush* ChannelStatusBrush = nullptr;
	
	TSharedPtr<SInlineEditableTextBlock> ChannelNameTextBlock;
	
	TWeakPtr<FAvaBroadcastEditor> BroadcastEditorWeak;

	TSharedPtr<FUICommandList> MediaOutputCommandList;
	
	EVisibility DragVisibility = EVisibility::Hidden;

	EAvaBroadcastChannelState ChannelState = EAvaBroadcastChannelState::Idle;
	
	FText ChannelStateText;

	TAttribute<bool> CanMaximize;
	FAvaBroadcastOnChannelMaximizeClicked OnMaximizeClicked;

	int32 RowIndex = INDEX_NONE;
	
	int32 ColumnIndex= INDEX_NONE;
	
	bool bShouldInvertAlpha = false;
};
