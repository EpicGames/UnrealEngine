// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaBroadcastChannels.h"

#include "AvaMediaEditorSettings.h"
#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/AvaBroadcastEditor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Input/Reply.h"
#include "SAvaBroadcastChannel.h"
#include "ScopedTransaction.h"
#include "Widgets/Layout/SGridPanel.h"

#define LOCTEXT_NAMESPACE "SAvaBroadcastChannels"

SAvaBroadcastChannels::FAvaChannelMaximizer::FAvaChannelMaximizer()
{
	MaximizeSequence.AddCurve(0.f, 0.25f, ECurveEaseFunction::CubicInOut);
}

void SAvaBroadcastChannels::FAvaChannelMaximizer::Reset()
{
	ChannelWidgetWeak.Reset();
	bMaximizing = false;
}

void SAvaBroadcastChannels::FAvaChannelMaximizer::ToggleMaximize(const TSharedRef<SAvaBroadcastChannel>& InChannelWidget)
{
	ChannelWidgetWeak = InChannelWidget;
	bMaximizing = !bMaximizing;

	//Do reverse here so that we don't have to calculate 1.f - Lerp everytime
	if (bMaximizing)
	{
		MaximizeSequence.PlayReverse(InChannelWidget);
	}
	else
	{
		MaximizeSequence.Play(InChannelWidget);
	}
}

float SAvaBroadcastChannels::FAvaChannelMaximizer::GetRowFill(int32 InRowIndex) const
{
	if (!ChannelWidgetWeak.IsValid() || ChannelWidgetWeak.Pin()->GetRowIndex() == InRowIndex)
	{
		return 1.f;
	}

	if (MaximizeSequence.IsPlaying())
	{
		return FMath::Max(KINDA_SMALL_NUMBER, MaximizeSequence.GetLerp());
	}

	return bMaximizing ? KINDA_SMALL_NUMBER : 1.f;
}

float SAvaBroadcastChannels::FAvaChannelMaximizer::GetColumnFill(int32 InColumnIndex) const
{
	if (!ChannelWidgetWeak.IsValid() || ChannelWidgetWeak.Pin()->GetColumnIndex() == InColumnIndex)
	{
		return 1.f;
	}

	if (MaximizeSequence.IsPlaying())
	{
		return FMath::Max(KINDA_SMALL_NUMBER, MaximizeSequence.GetLerp());
	}

	return bMaximizing ? KINDA_SMALL_NUMBER : 1.f;
}

void SAvaBroadcastChannels::Construct(const FArguments& InArgs, const TSharedPtr<FAvaBroadcastEditor>& InBroadcastEditor)
{
	BroadcastEditorWeak = InBroadcastEditor;
	check(InBroadcastEditor.IsValid());
	
	UAvaBroadcast* const Broadcast = InBroadcastEditor->GetBroadcastObject();
	BroadcastWeak = Broadcast;
	check(IsValid(Broadcast));
	
	Broadcast->AddChangeListener(FOnAvaBroadcastChanged::FDelegate::CreateSP(this
		, &SAvaBroadcastChannels::OnBroadcastChanged));
	
	ChannelGrid = SNew(SGridPanel);
	
	ChildSlot
	.HAlign(EHorizontalAlignment::HAlign_Fill)
	.VAlign(EVerticalAlignment::VAlign_Fill)
	[
		ChannelGrid.ToSharedRef()
	];

	RefreshChannelGrid();
}

SAvaBroadcastChannels::~SAvaBroadcastChannels()
{
	if (BroadcastWeak.IsValid())
	{
		BroadcastWeak->RemoveChangeListener(this);
	}
}

float SAvaBroadcastChannels::GetRowFill(int32 RowId) const
{
	return ChannelMaximizer.GetRowFill(RowId);
}

float SAvaBroadcastChannels::GetColumnFill(int32 ColumnId) const
{
	return ChannelMaximizer.GetColumnFill(ColumnId);
}

bool SAvaBroadcastChannels::CanMaximizeChannel() const
{
	const int32 ChannelCount = Channels.Num();
	return ChannelCount == 1 || (ChannelCount > 1 && !ChannelMaximizer.bMaximizing);
}

void SAvaBroadcastChannels::ToggleMaximizeChannel(const TSharedRef<SAvaBroadcastChannel>& InWidget)
{
	if (Channels.Num() > 1)
	{
		ChannelMaximizer.ToggleMaximize(InWidget);
	}
}

void SAvaBroadcastChannels::OnBroadcastChanged(EAvaBroadcastChange ChangedEvent)
{
	if (EnumHasAnyFlags(ChangedEvent, EAvaBroadcastChange::CurrentProfile))
	{
		Channels.Reset();
		ChannelGrid->ClearChildren();
		ChannelMaximizer.Reset();
	}
	RefreshChannelGrid();
}

void SAvaBroadcastChannels::RefreshChannelGrid()
{
	if (BroadcastWeak.IsValid())
	{
		const FAvaBroadcastProfile& Profile = BroadcastWeak->GetCurrentProfile();
		
		//Remove Items that are not in the New Channel List
		for (TMap<FName, TSharedPtr<SAvaBroadcastChannel>>::TIterator Iter = Channels.CreateIterator(); Iter; ++Iter)
		{
			//Remove Invalid Channels
			if (!Iter->Value.IsValid())
			{
				Iter.RemoveCurrent();
				continue;
			}
			
			//TODO: Currently Remove all Slots and Re-add them in Order
			ChannelGrid->RemoveSlot(Iter->Value.ToSharedRef());

			const FAvaBroadcastOutputChannel& Channel = Profile.GetChannel(Iter->Key);
			
			//Refresh the Media Tiles as they might've been deleted in an Undo operation
			Iter->Value->OnChannelMediaOutputsChanged(Channel);

			const bool bInvalidChannel = !Channel.IsValidChannel();
			const bool bMismatchChannelName = Iter->Key != Iter->Value->GetChannelName();
			
			if (bInvalidChannel || bMismatchChannelName)
			{
				//Reset Maximizer State if we're removing the Channel that is Maximized
				if (Iter->Value == ChannelMaximizer.ChannelWidgetWeak)
				{
					ChannelMaximizer.Reset();
				}
				Iter.RemoveCurrent();
			}
		}

		int32 ItemCount = 0;

		const TArray<FAvaBroadcastOutputChannel*>& BroadcastChannels = Profile.GetChannels();
		
		const int32 ItemsPerRow = FMath::RoundHalfToZero(FMath::Sqrt(static_cast<float>(BroadcastChannels.Num())));

		TSharedPtr<FAvaBroadcastEditor> BroadcastEditor = BroadcastEditorWeak.Pin();

		ChannelGrid->ClearFill();
		
		//Re-add existing Widgets or add new ones
		for (const FAvaBroadcastOutputChannel* InChannel : BroadcastChannels)
		{
			const FName ChannelName = InChannel->GetChannelName();
			if (!Channels.Contains(ChannelName))
			{
				TSharedRef<SAvaBroadcastChannel> NewChannel = SNew(SAvaBroadcastChannel, BroadcastEditor)
					.ChannelName(ChannelName)
					.CanMaximize(this, &SAvaBroadcastChannels::CanMaximizeChannel)
					.OnMaximizeClicked(this, &SAvaBroadcastChannels::ToggleMaximizeChannel);
				
				Channels.Add(ChannelName, NewChannel);
			}
			
			const int32 Row = ItemCount / ItemsPerRow;
			const int32 Column = ItemCount % ItemsPerRow;
			
			TSharedRef<SAvaBroadcastChannel> Channel = Channels.Find(ChannelName)->ToSharedRef();
			Channel->SetPosition(Column, Row);

			ChannelGrid->AddSlot(Column, Row)
				.VAlign(EVerticalAlignment::VAlign_Fill)
				.HAlign(EHorizontalAlignment::HAlign_Fill)
				[
					Channel
				];

			++ItemCount;
		}

		//Set Fill Attributes
		if (ItemsPerRow > 0)
		{
			const int32 ColumnCount = ItemsPerRow;
			const int32 RowCount = FMath::CeilToInt(static_cast<float>(ItemCount) / static_cast<float>(ItemsPerRow));
			
			for (int32 Index = 0; Index < ColumnCount; ++Index)
			{
				ChannelGrid->SetColumnFill(Index, TAttribute<float>::CreateSP(this, &SAvaBroadcastChannels::GetColumnFill, Index));
			}
			
			for (int32 Index = 0; Index < RowCount; ++Index)
			{
				ChannelGrid->SetRowFill(Index, TAttribute<float>::CreateSP(this, &SAvaBroadcastChannels::GetRowFill, Index));
			}
		}		
	}
}

FReply SAvaBroadcastChannels::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && BroadcastEditorWeak.IsValid())
	{
		BroadcastEditorWeak.Pin()->SelectOutputTile(nullptr);
	}
	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
