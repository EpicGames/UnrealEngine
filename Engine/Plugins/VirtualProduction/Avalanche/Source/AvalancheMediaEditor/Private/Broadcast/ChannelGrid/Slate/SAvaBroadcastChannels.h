// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/CurveSequence.h"
#include "AvaMediaDefines.h"
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FAvaBroadcastEditor;
class FReply;
class SAvaBroadcastChannel;
class SGridPanel;
class SWidget;
class UAvaBroadcast;

class SAvaBroadcastChannels : public SCompoundWidget
{
	struct FAvaChannelMaximizer
	{
		FAvaChannelMaximizer();

		void Reset();
		void ToggleMaximize(const TSharedRef<SAvaBroadcastChannel>& InChannelWidget);
		
		float GetRowFill(int32 InRowIndex) const;
		float GetColumnFill(int32 InColumnIndex) const;

		TWeakPtr<SAvaBroadcastChannel> ChannelWidgetWeak;
		FCurveSequence MaximizeSequence;
		bool bMaximizing  = false;
	};
	
public:
	SLATE_BEGIN_ARGS(SAvaBroadcastChannels) {}
	SLATE_END_ARGS()
	
	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedPtr<FAvaBroadcastEditor>& InBroadcastEditor);
	
	virtual ~SAvaBroadcastChannels() override;
	
	float GetRowFill(int32 RowId) const;
	float GetColumnFill(int32 ColumnId) const;
	
	bool CanMaximizeChannel() const;
	void ToggleMaximizeChannel(const TSharedRef<SAvaBroadcastChannel>& InWidget);
	
	void OnBroadcastChanged(EAvaBroadcastChange ChangedEvent);
	void RefreshChannelGrid();
	
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;;
	
protected:
	TWeakObjectPtr<UAvaBroadcast> BroadcastWeak;
	
	TWeakPtr<FAvaBroadcastEditor> BroadcastEditorWeak;
	
	TSharedPtr<SGridPanel> ChannelGrid;
	
	TMap<FName, TSharedPtr<SAvaBroadcastChannel>> Channels;

	FAvaChannelMaximizer ChannelMaximizer;
};
