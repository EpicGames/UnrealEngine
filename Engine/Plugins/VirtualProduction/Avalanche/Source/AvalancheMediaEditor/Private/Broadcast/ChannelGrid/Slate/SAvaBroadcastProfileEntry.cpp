// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaBroadcastProfileEntry.h"
#include "Broadcast/AvaBroadcast.h"
#include "Framework/Application/SlateApplication.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Geometry.h"
#include "ScopedTransaction.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "UObject/NameTypes.h"

#define LOCTEXT_NAMESPACE "SAvaBroadcastProfileEntry"

const FButtonStyle* SAvaBroadcastProfileEntry::MenuButtonStyle = nullptr;

void SAvaBroadcastProfileEntry::Construct(const FArguments& InArgs, FName InProfileName)
{
	UpdateProfileName(InProfileName);
	OnProfileEntrySelected = InArgs._OnProfileEntrySelected;
	
	if (MenuButtonStyle == nullptr)
	{
		MenuButtonStyle = &FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Menu.Button");
	}
	
	ChildSlot
	[
		SNew(SBox)
		.MinDesiredWidth(120.f)
		.MinDesiredHeight(24.f)
		[
			SNew(SBorder)
			.BorderImage(this, &SAvaBroadcastProfileEntry::GetBorderImage)
			.Padding(FMargin(12.f, 1.f, 12.f, 1.f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SAssignNew(ProfileTextBlock, SInlineEditableTextBlock)
					.Text(this, &SAvaBroadcastProfileEntry::GetProfileNameText)
					.Style(&FCoreStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("InlineEditableTextBlockStyle"))
					.OnTextCommitted(this, &SAvaBroadcastProfileEntry::OnProfileTextCommitted)
					.OnVerifyTextChanged(this, &SAvaBroadcastProfileEntry::OnVerifyProfileTextChanged)
					.OnEnterEditingMode(this, &SAvaBroadcastProfileEntry::OnEnterEditingMode)
					.OnExitEditingMode(this, &SAvaBroadcastProfileEntry::OnExitEditingMode)
					.IsReadOnly(false)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SSpacer)
					.Size(FVector2D(5.f, 1.f))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText(LOCTEXT("RenameProfileToolTip", "Rename Profile"))
					.OnClicked(this, &SAvaBroadcastProfileEntry::RenameProfile)
					.ClickMethod(EButtonClickMethod::Type::MouseUp)
					[
						SNew(SScaleBox)
						.Stretch(EStretch::ScaleToFit)
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("GenericCommands.Rename"))
							.DesiredSizeOverride(FVector2D(12.f))
						]
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText(LOCTEXT("DeleteProfileToolTip", "Delete Profile"))
					.OnClicked(this, &SAvaBroadcastProfileEntry::DeleteProfile)
					[
						SNew(SScaleBox)
						.Stretch(EStretch::ScaleToFit)
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("GenericCommands.Delete"))
							.DesiredSizeOverride(FVector2D(12.f))
						]
					]
				]
			]
		]
	];	
}

void SAvaBroadcastProfileEntry::UpdateProfileName(FName InProfileName)
{
	ProfileName = InProfileName;
	ProfileNameText = FText::FromName(InProfileName);
}

void SAvaBroadcastProfileEntry::OnProfileTextCommitted(const FText& InProfileText, ETextCommit::Type InCommitType)
{
	const FName DesiredProfileName(*InProfileText.ToString());
	
	FScopedTransaction Transaction(LOCTEXT("RenameProfile", "Rename Profile"));
	
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	Broadcast.Modify();
	
	if (Broadcast.RenameProfile(ProfileName, DesiredProfileName))
	{
		UpdateProfileName(DesiredProfileName);
	}
	else
	{
		Transaction.Cancel();
	}
}

bool SAvaBroadcastProfileEntry::OnVerifyProfileTextChanged(const FText& InProfileText, FText& OutErrorMessage)
{
	if (InProfileText.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("EmptyProfileName", "Profile name entry is empty.");
		return false;
	}
	
	const FName DesiredProfileName(*InProfileText.ToString());
	
	if (!UAvaBroadcast::Get().CanRenameProfile(ProfileName, DesiredProfileName))
	{
		OutErrorMessage = LOCTEXT("BroadcastCannotRename", "Broadcast cannot rename the Profile to the given name");
		return false;
	}
	
	return true;
}

void SAvaBroadcastProfileEntry::OnEnterEditingMode()
{
	bInEditingMode = true;
}

void SAvaBroadcastProfileEntry::OnExitEditingMode()
{
	bInEditingMode = false;
}

FReply SAvaBroadcastProfileEntry::RenameProfile()
{
	if (ProfileTextBlock.IsValid() && !ProfileTextBlock->IsInEditMode())
	{
		bRenameRequested = true;
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SAvaBroadcastProfileEntry::DeleteProfile()
{
	FScopedTransaction Transaction(LOCTEXT("RemoveProfile", "Remove Profile"));
	
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	Broadcast.Modify();
	
	const bool bResult = Broadcast.RemoveProfile(ProfileName);

	if (!bResult)
	{
		Transaction.Cancel();
	}
	
	FSlateApplication::Get().DismissAllMenus();
	return FReply::Handled();
}

void SAvaBroadcastProfileEntry::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime
	, const float InDeltaTime)
{
	if (bRenameRequested)
	{
		bRenameRequested = false;
		ProfileTextBlock->EnterEditingMode();
	}
}

FReply SAvaBroadcastProfileEntry::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (OnProfileEntrySelected.IsBound())
	{
		return OnProfileEntrySelected.Execute(ProfileName);
	}
	return FReply::Handled();
}

const FSlateBrush* SAvaBroadcastProfileEntry::GetBorderImage() const
{
	if (IsHovered())
	{
		return &MenuButtonStyle->Hovered;
	}
	return &MenuButtonStyle->Normal;
}

#undef LOCTEXT_NAMESPACE 
