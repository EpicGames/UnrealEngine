// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/SCompoundWidget.h"

class FName;
class FReply;
class FText;
class SInlineEditableTextBlock;
struct FButtonStyle;
struct FGeometry;
struct FPointerEvent;
struct FSlateBrush;

DECLARE_DELEGATE_RetVal_OneParam(FReply, FOnAvaBroadcastProfileEntrySelected, FName ProfileName);

class SAvaBroadcastProfileEntry : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaBroadcastProfileEntry){}
		SLATE_EVENT(FOnAvaBroadcastProfileEntrySelected, OnProfileEntrySelected)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, FName InProfileName);

	void UpdateProfileName(FName InProfileName);
	FText GetProfileNameText() const { return ProfileNameText; }

	void OnProfileTextCommitted(const FText& InProfileText, ETextCommit::Type InCommitType);
	bool OnVerifyProfileTextChanged(const FText& InProfileText, FText& OutErrorMessage);

	void OnEnterEditingMode();
	void OnExitEditingMode();

	FReply RenameProfile();
	FReply DeleteProfile();

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	const FSlateBrush* GetBorderImage() const;

protected:
	FName ProfileName;
	
	FText ProfileNameText;

	FOnAvaBroadcastProfileEntrySelected OnProfileEntrySelected;
	
	TSharedPtr<SInlineEditableTextBlock> ProfileTextBlock;
	
	static const FButtonStyle* MenuButtonStyle;
	
	bool bRenameRequested = false;
	bool bInEditingMode = false;
};
