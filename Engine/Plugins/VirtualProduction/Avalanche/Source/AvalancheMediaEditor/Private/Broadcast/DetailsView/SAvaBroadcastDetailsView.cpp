// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaBroadcastDetailsView.h"

#include "Broadcast/AvaBroadcastEditor.h"
#include "Broadcast/Channel/AvaBroadcastOutputChannel.h"
#include "Broadcast/ChannelGrid/AvaBroadcastOutputTileItem.h"
#include "DetailsViewArgs.h"
#include "Layout/Visibility.h"
#include "MediaOutput.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaBroadcastDetailsView"

void SAvaBroadcastDetailsView::Construct(const FArguments& InArgs, const TSharedPtr<FAvaBroadcastEditor>& InBroadcastEditor)
{
	BroadcastEditorWeak = InBroadcastEditor;
	
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	InBroadcastEditor->OnOutputTileSelectionChanged.AddRaw(this, &SAvaBroadcastDetailsView::OnMediaOutputSelectionChanged);
	
	DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateRaw(this
		, &SAvaBroadcastDetailsView::IsMediaOutputEditingEnabled));

	ChildSlot
	[
		SNew(SOverlay)
		+SOverlay::Slot()
		[
			DetailsView.ToSharedRef()
		]
		+SOverlay::Slot()
		.Padding(2.0f, 24.0f, 2.0f, 2.0f)
		.HAlign(HAlign_Center)
		[
			SNew(STextBlock)
			.Visibility(this, &SAvaBroadcastDetailsView::GetEmptySelectionTextVisibility)
			.Text(LOCTEXT("NoMediaOutputsSelected", "Select a Channel Media Output to view its details."))
			.TextStyle(FAppStyle::Get(), "HintText")
		]
	];

	OnMediaOutputSelectionChanged(nullptr);
}

SAvaBroadcastDetailsView::~SAvaBroadcastDetailsView()
{
	if (TSharedPtr<FAvaBroadcastEditor> BroadcastEditor = BroadcastEditorWeak.Pin())
	{
		BroadcastEditor->OnOutputTileSelectionChanged.RemoveAll(this);
	}
}

bool SAvaBroadcastDetailsView::IsMediaOutputEditingEnabled() const
{
	if (TSharedPtr<FAvaBroadcastEditor> BroadcastEditor = BroadcastEditorWeak.Pin())
	{
		if (TSharedPtr<FAvaBroadcastOutputTileItem> MediaOutputItem = BroadcastEditor->GetSelectedOutputTile())
		{
			const FAvaBroadcastOutputChannel& Channel = MediaOutputItem->GetChannel();
			return Channel.IsValidChannel() && Channel.IsEditingEnabled();
		}
	}
	return false;
}

void SAvaBroadcastDetailsView::OnMediaOutputSelectionChanged(const TSharedPtr<FAvaBroadcastOutputTileItem>& InSelectedItem)
{
	check(DetailsView.IsValid());

	TArray<UObject*> SelectedObjects;
	SelectedObjects.Reserve(1);
	
	if (InSelectedItem.IsValid())
	{
		SelectedObjects.Add(InSelectedItem->GetMediaOutput());
	}

	//Using array as SetObject makes an Array with NULL Element
	DetailsView->SetObjects(SelectedObjects);
}

EVisibility SAvaBroadcastDetailsView::GetEmptySelectionTextVisibility() const
{
	return DetailsView->GetSelectedObjects().IsEmpty()
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
