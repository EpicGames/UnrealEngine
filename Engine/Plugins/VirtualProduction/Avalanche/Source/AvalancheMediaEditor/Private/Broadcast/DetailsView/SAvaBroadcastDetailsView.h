// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FAvaBroadcastEditor;
class FAvaBroadcastOutputTileItem;
class IDetailsView;
struct EVisibility;

class SAvaBroadcastDetailsView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaBroadcastDetailsView) {}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedPtr<FAvaBroadcastEditor>& InBroadcastEditor);
	virtual ~SAvaBroadcastDetailsView() override;

	bool IsMediaOutputEditingEnabled() const;
	void OnMediaOutputSelectionChanged(const TSharedPtr<FAvaBroadcastOutputTileItem>& InSelectedItem);

	EVisibility GetEmptySelectionTextVisibility() const;
	
protected:
	TWeakPtr<FAvaBroadcastEditor> BroadcastEditorWeak;
	
	TSharedPtr<IDetailsView> DetailsView;
};
