// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaBroadcastDisplayMediaOutputFactory.h"
#include "AssetTypeCategories.h"
#include "Broadcast/OutputDevices/AvaBroadcastDisplayMediaOutput.h"
#include "Misc/FeedbackContext.h"
#include "UObject/NoExportTypes.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"

UAvaBroadcastDisplayMediaOutputFactory::UAvaBroadcastDisplayMediaOutputFactory()
{
	SupportedClass = UAvaBroadcastDisplayMediaOutput::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UAvaBroadcastDisplayMediaOutputFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName
	, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UAvaBroadcastDisplayMediaOutput>(InParent, InClass, InName, Flags);
}

uint32 UAvaBroadcastDisplayMediaOutputFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}

bool UAvaBroadcastDisplayMediaOutputFactory::ShouldShowInNewMenu() const
{
	return true;
}
