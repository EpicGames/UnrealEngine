// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "AvaBroadcastDisplayMediaOutputFactory.generated.h"

class FFeedbackContext;
class FName;
class UClass;
class UObject;
enum EObjectFlags : uint32;

/**
 * Implements a factory for UAvaBroadcastDisplayMediaOutput objects.
 */
UCLASS()
class UAvaBroadcastDisplayMediaOutputFactory : public UFactory
{
	GENERATED_BODY()

public:
	UAvaBroadcastDisplayMediaOutputFactory();

	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual bool ShouldShowInNewMenu() const override;
};
