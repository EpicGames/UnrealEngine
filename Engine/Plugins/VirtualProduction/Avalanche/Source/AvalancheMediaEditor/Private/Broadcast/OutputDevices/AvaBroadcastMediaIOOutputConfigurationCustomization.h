// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/MediaIOCustomizationBase.h"
#include "Input/Reply.h"
#include "MediaIOCoreDefinitions.h"

/**
 * Implements a details view customization for the FMediaIOOutputConfiguration
 */
class FAvaBroadcastMediaIOOutputConfigurationCustomization : public FMediaIOCustomizationBase
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

private:
	virtual TAttribute<FText> GetContentText() override;
	virtual TSharedRef<SWidget> HandleSourceComboButtonMenuContent() override;

	void OnSelectionChanged(FMediaIOOutputConfiguration SelectedItem);
	FReply OnButtonClicked() const;

	TWeakPtr<SWidget> PermutationSelector;
	FMediaIOOutputConfiguration SelectedConfiguration;
};
