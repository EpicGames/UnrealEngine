// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaBroadcastOutputTreeItemDragDropOp.h"

#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputTreeItem.h"
#include "MediaOutput.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "AvaBroadcastOutputTreeItemDragDropOp"

TSharedRef<FAvaBroadcastOutputTreeItemDragDropOp> FAvaBroadcastOutputTreeItemDragDropOp::New(const TSharedPtr<IAvaBroadcastOutputTreeItem>& InOutputTreeItem)
{
	TSharedRef<FAvaBroadcastOutputTreeItemDragDropOp> DragDropOp = MakeShared<FAvaBroadcastOutputTreeItemDragDropOp>();
	DragDropOp->Init(InOutputTreeItem);
	return DragDropOp;
}

bool FAvaBroadcastOutputTreeItemDragDropOp::IsValidToDropInChannel(FName InTargetChannelName) const
{
	return OutputTreeItem.IsValid() && OutputTreeItem->IsValidToDropInChannel(InTargetChannelName);
}

FReply FAvaBroadcastOutputTreeItemDragDropOp::OnChannelDrop(FName InTargetChannelName)
{
	if (const TSharedPtr<IAvaBroadcastOutputTreeItem> Item = GetOutputTreeItem())
	{
		UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
		
		FScopedTransaction Transaction(LOCTEXT("OnChannelDrop", "Drop Media Output"));
		Broadcast.Modify();
		
		const UMediaOutput* const MediaOutput = Item->AddMediaOutputToChannel(InTargetChannelName, FAvaBroadcastMediaOutputInfo());
		if (IsValid(MediaOutput))
		{
			return FReply::Handled();
		}
		else
		{
			Transaction.Cancel();
		}
	}
	return FReply::Unhandled();
}

void FAvaBroadcastOutputTreeItemDragDropOp::Init(const TSharedPtr<IAvaBroadcastOutputTreeItem>& InOutputTreeItem)
{
	OutputTreeItem = InOutputTreeItem;
	CurrentHoverText = InOutputTreeItem->GetDisplayName();
	CurrentIconBrush = InOutputTreeItem->GetIconBrush();
	CurrentIconColorAndOpacity = FSlateColor::UseForeground();
	MouseCursor = EMouseCursor::GrabHandClosed;

	SetupDefaults();
	Construct();
}

#undef LOCTEXT_NAMESPACE
