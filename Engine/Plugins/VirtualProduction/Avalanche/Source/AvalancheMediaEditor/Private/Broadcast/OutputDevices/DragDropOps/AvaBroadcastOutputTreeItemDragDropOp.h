// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"

class IAvaBroadcastOutputTreeItem;

class FAvaBroadcastOutputTreeItemDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FAvaBroadcastOutputTreeItemDragDropOp, FDecoratedDragDropOp)

	static TSharedRef<FAvaBroadcastOutputTreeItemDragDropOp> New(const TSharedPtr<IAvaBroadcastOutputTreeItem>& InOutputClassItem);

	bool IsValidToDropInChannel(FName InTargetChannelName) const;

	TSharedPtr<IAvaBroadcastOutputTreeItem> GetOutputTreeItem() const { return OutputTreeItem; }

	FReply OnChannelDrop(FName InTargetChannelName);

protected:
	void Init(const TSharedPtr<IAvaBroadcastOutputTreeItem>& InOutputClassItem);

	/** Keep Reference Count while Drag Dropping */
	TSharedPtr<IAvaBroadcastOutputTreeItem> OutputTreeItem;
};
