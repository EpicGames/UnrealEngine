// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaBroadcastOutputDevices.h"

#include "AvaMediaEditorSettings.h"
#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputRootItem.h"
#include "Broadcast/OutputDevices/DragDropOps/AvaBroadcastOutputTreeItemDragDropOp.h"
#include "Widgets/Views/STreeView.h"

TMap<uint32, bool> SAvaBroadcastOutputDevices::ItemExpansionStates = {};

SAvaBroadcastOutputDevices::~SAvaBroadcastOutputDevices()
{
	if (UObjectInitialized())
	{
		UAvaMediaEditorSettings::GetMutable().OnSettingChanged().RemoveAll(this);
	}
}

void SAvaBroadcastOutputDevices::Construct(const FArguments& InArgs, const TSharedPtr<FAvaBroadcastEditor>& InBroadcastEditor)
{
	RootItem = MakeShared<FAvaBroadcastOutputRootItem>();
	
	SAssignNew(OutputTree, STreeView<FAvaOutputTreeItemPtr>)
		.SelectionMode(ESelectionMode::Single)
		.OnGenerateRow(this, &SAvaBroadcastOutputDevices::OnGenerateItemRow)
		.OnGetChildren(this, &SAvaBroadcastOutputDevices::OnGetRowChildren)
		.OnExpansionChanged(this, &SAvaBroadcastOutputDevices::OnRowExpansionChanged)
		.TreeItemsSource(&TopLevelItems);
	
	ChildSlot
	[
		OutputTree.ToSharedRef()
	];

	RefreshOutputDevices();
	
	BroadcastChangedHandle = UAvaBroadcast::Get().AddChangeListener(
		FOnAvaBroadcastChanged::FDelegate::CreateSP(this, &SAvaBroadcastOutputDevices::OnBroadcastChanged));
	UAvaMediaEditorSettings::GetMutable().OnSettingChanged().AddSP(this, &SAvaBroadcastOutputDevices::OnAvaMediaSettingsChanged);
}

TSharedRef<ITableRow> SAvaBroadcastOutputDevices::OnGenerateItemRow(FAvaOutputTreeItemPtr InItem
	, const TSharedRef<STableViewBase>& InOwnerTable)
{
	check(InItem.IsValid());

	InItem->OnCreateDragDropOperation().BindLambda([](const FAvaOutputTreeItemPtr& InItem)
	{
		return FAvaBroadcastOutputTreeItemDragDropOp::New(InItem);
	});

	return SNew(STableRow<FAvaOutputTreeItemPtr>, InOwnerTable)
		.ShowWires(false)
		.Padding(FMargin(5.f, 5.f))
		.OnDragDetected(InItem.ToSharedRef(), &IAvaBroadcastOutputTreeItem::OnDragDetected)
		[
			InItem->GenerateRowWidget().ToSharedRef()
		];
}

void SAvaBroadcastOutputDevices::OnGetRowChildren(FAvaOutputTreeItemPtr InItem, TArray<FAvaOutputTreeItemPtr>& OutChildren) const
{
	if (InItem.IsValid())
	{
		OutChildren.Append(InItem->GetChildren());
	}
}

void SAvaBroadcastOutputDevices::OnRowExpansionChanged(FAvaOutputTreeItemPtr InItem, const bool bInIsExpanded)
{
	const uint32 Hash = ItemHashRecursive(InItem);

	if (ItemExpansionStates.Contains(Hash))
	{
		ItemExpansionStates[Hash] = bInIsExpanded;
	}
	else
	{
		ItemExpansionStates.Add(Hash, bInIsExpanded);
	}
}

uint32 SAvaBroadcastOutputDevices::ItemHashRecursive(const FAvaOutputTreeItemPtr& InItem) const
{
	if (!InItem.IsValid() || InItem->IsA<FAvaBroadcastOutputRootItem>())
	{
		return 0;
	}

	const uint32 Hash = GetTypeHash(InItem->GetDisplayName().ToString());

	if (const TSharedPtr<FAvaBroadcastOutputTreeItem>& Parent = InItem->GetParent().Pin())
	{
		const uint32 ParentHash = ItemHashRecursive(Parent);
		if (ParentHash != 0)
		{
			const uint32 CombinedHash = HashCombine(ParentHash, Hash);
			return CombinedHash;
		}
	}
		
	return Hash;
}

void SAvaBroadcastOutputDevices::RefreshOutputDevices()
{
	check(RootItem.IsValid());
	
	//Refresh Items
	IAvaBroadcastOutputTreeItem::FRefreshChildrenParams RefreshDevicesParams;
	RefreshDevicesParams.bShowAllMediaOutputClasses = UAvaMediaEditorSettings::Get().bBroadcastShowAllMediaOutputClasses;
	FAvaBroadcastOutputTreeItem::RefreshTree(RootItem, RefreshDevicesParams);
	TopLevelItems = RootItem->GetChildren();
	OutputTree->RequestTreeRefresh();

	SetExpansionStatesRecursive(RootItem);
}

void SAvaBroadcastOutputDevices::OnBroadcastChanged(EAvaBroadcastChange InChange)
{
	if (EnumHasAnyFlags(InChange, EAvaBroadcastChange::OutputDevices))
	{
		RefreshOutputDevices();
	}
}

void SAvaBroadcastOutputDevices::OnAvaMediaSettingsChanged(UObject*, FPropertyChangedEvent& InPropertyChangeEvent)
{
	static const FName BroadcastShowAllMediaOutputClassesName = GET_MEMBER_NAME_CHECKED(UAvaMediaEditorSettings, bBroadcastShowAllMediaOutputClasses);
	if (InPropertyChangeEvent.GetPropertyName() == BroadcastShowAllMediaOutputClassesName)
	{
		RefreshOutputDevices();
	}
}

void SAvaBroadcastOutputDevices::SetExpansionStatesRecursive(const FAvaOutputTreeItemPtr& InRootItem)
{
	if (!InRootItem.IsValid())
	{
		return;
	}

	for (const FAvaOutputTreeItemPtr& Child : InRootItem->GetChildren())
	{
		if (Child.IsValid())
		{
			const uint32 ChildHash = ItemHashRecursive(Child);
			if (ItemExpansionStates.Contains(ChildHash))
			{
				OutputTree->SetItemExpansion(Child, ItemExpansionStates[ChildHash]);
			}

			SetExpansionStatesRecursive(Child);
		}
	}
}
