// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputTreeItem.h"
#include "Widgets/SCompoundWidget.h"

class FAvaBroadcastEditor;
class FAvaBroadcastOutputRootItem;
class ITableRow;
class STableViewBase;
template<typename ItemType> class STreeView;

class SAvaBroadcastOutputDevices : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaBroadcastOutputDevices){}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedPtr<FAvaBroadcastEditor>& InBroadcastEditor);
	virtual ~SAvaBroadcastOutputDevices() override;
	
	TSharedRef<ITableRow> OnGenerateItemRow(FAvaOutputTreeItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable);
	
	void OnGetRowChildren(FAvaOutputTreeItemPtr InItem, TArray<FAvaOutputTreeItemPtr>& OutChildren) const;
	void OnRowExpansionChanged(FAvaOutputTreeItemPtr InItem, const bool bInIsExpanded);

	void RefreshOutputDevices();

protected:
	void OnBroadcastChanged(EAvaBroadcastChange InChange);
	void OnAvaMediaSettingsChanged(UObject*, FPropertyChangedEvent& InPropertyChangeEvent);

	uint32 ItemHashRecursive(const FAvaOutputTreeItemPtr& InItem) const;
	void SetExpansionStatesRecursive(const FAvaOutputTreeItemPtr& InRootItem);

	static TMap<uint32, bool> ItemExpansionStates;

	FDelegateHandle BroadcastChangedHandle;

	TSharedPtr<STreeView<FAvaOutputTreeItemPtr>> OutputTree;
	
	TArray<FAvaOutputTreeItemPtr> TopLevelItems;

	TSharedPtr<FAvaBroadcastOutputRootItem> RootItem;
};
