// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaBroadcastChannelsTabFactory.h"

#include "AvaMediaEditorStyle.h"
#include "Broadcast/AvaBroadcastEditor.h"
#include "Broadcast/ChannelGrid/Slate/SAvaBroadcastChannels.h"
#include "IAvaMediaEditorModule.h"

const FName FAvaBroadcastChannelsTabFactory::TabID(TEXT("MotionDesignBroadcastChannels"));

#define LOCTEXT_NAMESPACE "AvaBroadcastChannelsTabFactory"

FAvaBroadcastChannelsTabFactory::FAvaBroadcastChannelsTabFactory(const TSharedPtr<FAvaBroadcastEditor>& InBroadcastEditor)
	: FAvaBroadcastTabFactory(TabID, InBroadcastEditor)
{
	TabLabel = LOCTEXT("BroadcastChannels_TabLabel", "Channels");
	TabIcon = FSlateIcon(FAvaMediaEditorStyle::Get().GetStyleSetName(), "AvaMediaEditor.BroadcastIcon");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("BroadcastChannels_ViewMenu_Desc", "Grid View of the Broadcast Channels");
	ViewMenuTooltip = LOCTEXT("BroadcastChannels_ViewMenu_ToolTip", "Grid View of the Broadcast Channels");
}

TSharedRef<SWidget> FAvaBroadcastChannelsTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FAvaBroadcastEditor> BroadcastEditor = BroadcastEditorWeak.Pin();
	check(BroadcastEditor.IsValid());
	return SNew(SAvaBroadcastChannels, BroadcastEditor);
}

const FSlateBrush* FAvaBroadcastChannelsTabFactory::GetTabIcon(const FWorkflowTabSpawnInfo& Info) const
{
	return IAvaMediaEditorModule::Get().GetToolbarBroadcastButtonIcon().GetIcon();
}

#undef LOCTEXT_NAMESPACE
