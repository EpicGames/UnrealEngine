// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaBroadcastDetailsTabFactory.h"

#include "Broadcast/AvaBroadcastEditor.h"
#include "Broadcast/DetailsView/SAvaBroadcastDetailsView.h"

const FName FAvaBroadcastDetailsTabFactory::TabID(TEXT("MotionDesignBroadcastDetails"));

#define LOCTEXT_NAMESPACE "AvaBroadcastDetailsTabFactory"

FAvaBroadcastDetailsTabFactory::FAvaBroadcastDetailsTabFactory(const TSharedPtr<FAvaBroadcastEditor>& InBroadcastEditor)
	: FAvaBroadcastTabFactory(TabID, InBroadcastEditor)
{
	TabLabel = LOCTEXT("BroadcastDetails_TabLabel", "Details");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("BroadcastDetails_ViewMenu_Desc", "Broadcast Details");
	ViewMenuTooltip = LOCTEXT("BroadcastDetails_ViewMenu_ToolTip", "Broadcast Details");
}

TSharedRef<SWidget> FAvaBroadcastDetailsTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FAvaBroadcastEditor> BroadcastEditor = BroadcastEditorWeak.Pin();
	check(BroadcastEditor.IsValid());	
	return SNew(SAvaBroadcastDetailsView, BroadcastEditor);
}

#undef LOCTEXT_NAMESPACE
