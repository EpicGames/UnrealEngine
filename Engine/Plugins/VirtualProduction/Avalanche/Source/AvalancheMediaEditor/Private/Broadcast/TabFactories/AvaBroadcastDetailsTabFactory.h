// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaBroadcastTabFactory.h"

class FAvaBroadcastDetailsTabFactory : public FAvaBroadcastTabFactory
{
public:
	static const FName TabID;
	
	FAvaBroadcastDetailsTabFactory(const TSharedPtr<FAvaBroadcastEditor>& InBroadcastEditor);
	
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
};
