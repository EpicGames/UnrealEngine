// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaBroadcastOutputsTabFactory.h"

#include "Broadcast/AvaBroadcastEditor.h"
#include "Broadcast/OutputDevices/Slate/SAvaBroadcastOutputDevices.h"

const FName FAvaBroadcastOutputsTabFactory::TabID(TEXT("MotionDesignBroadcastOutputList"));

#define LOCTEXT_NAMESPACE "AvaBroadcastOutputsTabFactory"

FAvaBroadcastOutputsTabFactory::FAvaBroadcastOutputsTabFactory(const TSharedPtr<FAvaBroadcastEditor>& InBroadcastEditor)
	: FAvaBroadcastTabFactory(TabID, InBroadcastEditor)
{
	TabLabel = LOCTEXT("BroadcastOutputList_TabLabel", "Output Devices");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.VirtualProduction");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("BroadcastOutputList_ViewMenu_Desc", "A list of found Media Outputs");
	ViewMenuTooltip = LOCTEXT("BroadcastOutputList_ViewMenu_ToolTip", "A list of found Media Outputs");
}

TSharedRef<SWidget> FAvaBroadcastOutputsTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FAvaBroadcastEditor> BroadcastEditor = BroadcastEditorWeak.Pin();
	check(BroadcastEditor.IsValid());
	return SNew(SAvaBroadcastOutputDevices, BroadcastEditor);
}

#undef LOCTEXT_NAMESPACE
