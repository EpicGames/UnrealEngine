// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaBroadcastTabFactory.h"

class FAvaBroadcastOutputsTabFactory : public FAvaBroadcastTabFactory
{
public:
	static const FName TabID;
	
	FAvaBroadcastOutputsTabFactory(const TSharedPtr<FAvaBroadcastEditor>& InBroadcastEditor);
	
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	
};
