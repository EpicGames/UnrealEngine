// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaBroadcastTabFactory.h"

#include "Broadcast/AvaBroadcastEditor.h"

FAvaBroadcastTabFactory::FAvaBroadcastTabFactory(const FName& InTabID, const TSharedPtr<FAvaBroadcastEditor>& InBroadcastEditor)
	: FWorkflowTabFactory(InTabID, InBroadcastEditor)
	, BroadcastEditorWeak(InBroadcastEditor)
{
}
