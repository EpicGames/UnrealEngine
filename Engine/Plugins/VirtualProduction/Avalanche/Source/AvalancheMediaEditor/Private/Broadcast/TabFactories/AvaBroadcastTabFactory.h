// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class FAvaBroadcastEditor;

//Base class for all Tab Factories in Ava Broadcast Editor
class FAvaBroadcastTabFactory : public FWorkflowTabFactory
{
public:
	
	FAvaBroadcastTabFactory(const FName& InTabID, const TSharedPtr<FAvaBroadcastEditor>& InBroadcastEditor);

protected:

	TWeakPtr<FAvaBroadcastEditor> BroadcastEditorWeak;
};

