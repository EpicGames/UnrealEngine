// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customization/AvaPlaybackServerLoggingEntryCustomization.h"

#include "AvaMediaSettings.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "AvaPlaybackServerLoggingEntryCustomization"

TSharedRef<IPropertyTypeCustomization> FAvaPlaybackServerLoggingEntryCustomization::MakeInstance()
{
	return MakeShared<FAvaPlaybackServerLoggingEntryCustomization>();
}

void FAvaPlaybackServerLoggingEntryCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	const TSharedPtr<IPropertyHandle> CategoryHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaPlaybackServerLoggingEntry, Category));
	const TSharedPtr<IPropertyHandle> VerbosityHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaPlaybackServerLoggingEntry, VerbosityLevel));
	
	const FMargin PropertyPadding(2.0f, 0.0f, 2.0f, 0.0f);

	InHeaderRow
			.NameContent()
			[
				InStructPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(PropertyPadding)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SBox)
					.HAlign(EHorizontalAlignment::HAlign_Fill)
					.VAlign(EVerticalAlignment::VAlign_Center)
					.MinDesiredWidth(200.f) // Note: Minimum size enough for most log categories so the verbosity is visually aligned.
					[
						CategoryHandle->CreatePropertyValueWidget()
					]
				]
				+ SHorizontalBox::Slot()
				.Padding(PropertyPadding)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					VerbosityHandle->CreatePropertyValueWidget()
				]
			];
}

void FAvaPlaybackServerLoggingEntryCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InStructBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
}

#undef LOCTEXT_NAMESPACE