// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSynchronizedEventsFeatureSelectionCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "Playable/AvaPlayableSettings.h"
#include "SAvaSynchronizedEventsImplementationSelector.h"

TSharedRef<IPropertyTypeCustomization> FAvaSynchronizedEventsFeatureSelectionCustomization::MakeInstance()
{
	return MakeShared<FAvaSynchronizedEventsFeatureSelectionCustomization>();
}

void FAvaSynchronizedEventsFeatureSelectionCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InPropertyTypeCustomizationUtils)
{

}

void FAvaSynchronizedEventsFeatureSelectionCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InStructBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	const TSharedPtr<IPropertyHandle> ImplementationHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSynchronizedEventsFeatureSelection, Implementation));
	
	if (ImplementationHandle.IsValid())
	{
		InStructBuilder.AddProperty(ImplementationHandle.ToSharedRef()).CustomWidget()
			.NameContent()
			[
				InStructPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SAvaSynchronizedEventsImplementationSelector, ImplementationHandle.ToSharedRef())
			];
	}
}