// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaSynchronizedEventsImplementationSelector.h"

#include "AvaMediaSettings.h"
#include "DetailLayoutBuilder.h"
#include "ModularFeature/AvaMediaSynchronizedEventsFeature.h"
#include "Widgets/Input/SComboBox.h"

void SAvaSynchronizedEventsImplementationSelector::Construct(const FArguments& InArgs, TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	UpdateItems();
	PropertyHandle = InPropertyHandle;
	
	ChildSlot
	[
		SAssignNew(Combo, SComboBox<FName>)
		.InitiallySelectedItem(GetItemFromProperty())
		.OptionsSource(&Items)
		.OnGenerateWidget(this, &SAvaSynchronizedEventsImplementationSelector::GenerateWidget)
		.OnSelectionChanged(this, &SAvaSynchronizedEventsImplementationSelector::HandleSelectionChanged)
		.OnComboBoxOpening(this, &SAvaSynchronizedEventsImplementationSelector::OnComboBoxOpening)
		[
			SNew(STextBlock)
			.Text(this, &SAvaSynchronizedEventsImplementationSelector::GetDisplayTextFromProperty)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];
}

TSharedRef<SWidget> SAvaSynchronizedEventsImplementationSelector::GenerateWidget(FName InItem)
{
	return SNew(STextBlock)
		.Text(GetDisplayTextFromItem(InItem))
		.ToolTipText(GetDisplayDescriptionFromItem(InItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void SAvaSynchronizedEventsImplementationSelector::HandleSelectionChanged(FName InProposedSelection, ESelectInfo::Type InSelectInfo)
{
	if (PropertyHandle)
	{
		PropertyHandle->SetValue(InProposedSelection.ToString());
	}
}

FText SAvaSynchronizedEventsImplementationSelector::GetDisplayTextFromProperty() const
{
	return GetDisplayTextFromItem(GetItemFromProperty());
}

void SAvaSynchronizedEventsImplementationSelector::OnComboBoxOpening()
{
	check(Combo.IsValid());
	Combo->SetSelectedItem(GetItemFromProperty());
}

FName SAvaSynchronizedEventsImplementationSelector::GetItemFromProperty() const
{
	if (PropertyHandle)
	{
		FString Value;
		PropertyHandle->GetValue(Value);
		return FName(*Value);
	}
	return FName();
}

FText SAvaSynchronizedEventsImplementationSelector::GetDisplayTextFromItem(FName InItem) const
{
	if (const IAvaMediaSynchronizedEventsFeature* Implementation = FAvaMediaSynchronizedEventsFeature::FindImplementation(InItem))
	{
		if (InItem == UAvaMediaSettings::SynchronizedEventsFeatureSelection_Default)
		{
			return FText::Format(INVTEXT("{0} ({1})"), FText::FromName(InItem), Implementation->GetDisplayName());
		}
		else
		{
			return Implementation->GetDisplayName();
		}
	}
	return FText::FromName(InItem);
}

FText SAvaSynchronizedEventsImplementationSelector::GetDisplayDescriptionFromItem(FName InItem) const
{
	if (const IAvaMediaSynchronizedEventsFeature* Implementation = FAvaMediaSynchronizedEventsFeature::FindImplementation(InItem))
	{
		return Implementation->GetDisplayDescription();
	}
	return FText::FromName(InItem);
}

void SAvaSynchronizedEventsImplementationSelector::UpdateItems()
{
	Items.Reset();

	FAvaMediaSynchronizedEventsFeature::EnumerateImplementations([this](const IAvaMediaSynchronizedEventsFeature* InFeature)
	{
		Items.Add(InFeature->GetName());
	});

	Items.Add(UAvaMediaSettings::SynchronizedEventsFeatureSelection_Default);
}
