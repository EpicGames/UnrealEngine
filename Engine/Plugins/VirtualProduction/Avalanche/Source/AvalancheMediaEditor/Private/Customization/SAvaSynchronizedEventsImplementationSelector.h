// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class IPropertyHandle;
template <typename OptionType> class SComboBox;

class SAvaSynchronizedEventsImplementationSelector : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaSynchronizedEventsImplementationSelector){}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TSharedPtr<IPropertyHandle> InPropertyHandle);
	
	TSharedRef<SWidget> GenerateWidget(FName InItem);
	
	void HandleSelectionChanged(FName InProposedSelection, ESelectInfo::Type InSelectInfo);
	
	FText GetDisplayTextFromProperty() const;
	
	void OnComboBoxOpening();
	
protected:
	FName GetItemFromProperty() const;
	
	FText GetDisplayTextFromItem(FName InItem) const;
	FText GetDisplayDescriptionFromItem(FName InItem) const;
	
	void UpdateItems();
	
	TSharedPtr<IPropertyHandle> PropertyHandle;
	
	TSharedPtr<SComboBox<FName>> Combo;

	TArray<FName> Items;
};
