// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPlaybackAppMode.h"

#include "Playback/AvaPlaybackGraphEditor.h"

const FName FAvaPlaybackAppMode::DefaultMode("DefaultName");

#define LOCTEXT_NAMESPACE "AvaPlaybackAppMode"

FAvaPlaybackAppMode::FAvaPlaybackAppMode(const TSharedPtr<FAvaPlaybackGraphEditor>& InPlaybackEditor, const FName& InModeName)
	: FApplicationMode(InModeName, FAvaPlaybackAppMode::GetLocalizedMode)
	, PlaybackEditorWeak(InPlaybackEditor)
{
}

void FAvaPlaybackAppMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FAvaPlaybackGraphEditor> PlaybackEditor = PlaybackEditorWeak.Pin();
	PlaybackEditor->PushTabFactories(TabFactories);
}

FText FAvaPlaybackAppMode::GetLocalizedMode(const FName InMode)
{
	static TMap<FName, FText> LocModes;

	if (LocModes.Num() == 0)
	{
		LocModes.Add(DefaultMode, LOCTEXT("Playback_DefaultMode", "Default"));
	}

	check(InMode != NAME_None);
	const FText* OutDesc = LocModes.Find(InMode);
	check(OutDesc);
	
	return *OutDesc;
}

#undef LOCTEXT_NAMESPACE
