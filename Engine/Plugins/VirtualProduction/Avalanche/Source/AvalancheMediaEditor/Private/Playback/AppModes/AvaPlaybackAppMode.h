// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

class FAvaPlaybackGraphEditor;

class FAvaPlaybackAppMode : public FApplicationMode
{
public:

	// Mode constants
	static const FName DefaultMode;
	//Add more here
	
	FAvaPlaybackAppMode(const TSharedPtr<FAvaPlaybackGraphEditor>& InPlaybackEditor, const FName& InModeName);
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;

protected:
	
	static FText GetLocalizedMode(const FName InMode);

	TWeakPtr<FAvaPlaybackGraphEditor> PlaybackEditorWeak;
	
	// Set of spawnable tabs in the mode
	FWorkflowAllowedTabSet TabFactories;
};
