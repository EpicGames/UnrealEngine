// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPlaybackDefaultMode.h"

#include "IAvaMediaEditorModule.h"
#include "Playback/AvaPlaybackGraphEditor.h"
#include "Playback/TabFactories/AvaPlaybackDetailsTabFactory.h"
#include "Playback/TabFactories/AvaPlaybackEditorGraphTabFactory.h"

#define LOCTEXT_NAMESPACE "AvaPlaybackDefaultMode"

FAvaPlaybackDefaultMode::FAvaPlaybackDefaultMode(const TSharedPtr<FAvaPlaybackGraphEditor>& InPlaybackEditor)
	: FAvaPlaybackAppMode(InPlaybackEditor, FAvaPlaybackAppMode::DefaultMode)
{
	WorkspaceMenuCategory = FWorkspaceItem::NewGroup(LOCTEXT("WorkspaceMenu_AvaPlayback", "Motion Design Playback"));
	
	check(InPlaybackEditor.IsValid());
	TabLayout = FTabManager::NewLayout("MotionDesignPlaybackEditor_Default_Layout_V1")
		->AddArea
		(
			FTabManager::NewPrimaryArea() ->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.75f)
					->AddTab(FAvaPlaybackEditorGraphTabFactory::TabID, ETabState::OpenedTab)
					->SetForegroundTab(FAvaPlaybackEditorGraphTabFactory::TabID)
					->SetHideTabWell(true)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.25f)
					->AddTab(FAvaPlaybackDetailsTabFactory::TabID, ETabState::OpenedTab)
				)
			)
		);

	// Add Tab Spawners
	TabFactories.RegisterFactory(MakeShared<FAvaPlaybackEditorGraphTabFactory>(InPlaybackEditor));
	TabFactories.RegisterFactory(MakeShared<FAvaPlaybackDetailsTabFactory>(InPlaybackEditor));

	//Make sure we start with our existing list of extenders instead of creating a new one
	IAvaMediaEditorModule& AvaMediaEditorModule = IAvaMediaEditorModule::Get();
	ToolbarExtender = AvaMediaEditorModule.GetPlaybackToolBarExtensibilityManager()->GetAllExtenders();
	InPlaybackEditor->ExtendToolBar(ToolbarExtender);
}

#undef LOCTEXT_NAMESPACE
