// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaPlaybackAppMode.h"

class FAvaPlaybackDefaultMode : public FAvaPlaybackAppMode
{
public:
	FAvaPlaybackDefaultMode(const TSharedPtr<FAvaPlaybackGraphEditor>& InPlaybackEditor);
};
