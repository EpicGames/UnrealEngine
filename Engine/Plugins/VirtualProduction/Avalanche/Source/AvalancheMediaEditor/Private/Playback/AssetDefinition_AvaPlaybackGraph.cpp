// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_AvaPlaybackGraph.h"

#include "AvaMediaEditorStyle.h"
#include "ContentBrowserMenuContexts.h"
#include "Playback/AvaPlaybackGraph.h"
#include "Playback/AvaPlaybackGraphEditor.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_AvaPlaybackGraph"

FText UAssetDefinition_AvaPlaybackGraph::GetAssetDisplayName() const
{
	return LOCTEXT("AvaMediaEditorPlaybackAction_Name", "Motion Design Playback Graph");
}

TSoftClassPtr<UObject> UAssetDefinition_AvaPlaybackGraph::GetAssetClass() const
{
	return UAvaPlaybackGraph::StaticClass();
}

FLinearColor UAssetDefinition_AvaPlaybackGraph::GetAssetColor() const
{
	static const FName PlaybackAssetColorName(TEXT("AvaMediaEditor.AssetColors.Playback"));
	return FAvaMediaEditorStyle::Get().GetColor(PlaybackAssetColorName);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_AvaPlaybackGraph::GetAssetCategories() const
{
	static const TArray<FAssetCategoryPath> Categories = {EAssetCategoryPaths::Blueprint};
	return Categories;
}

EAssetCommandResult UAssetDefinition_AvaPlaybackGraph::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	EAssetCommandResult CommandResult = EAssetCommandResult::Unhandled;
	if (OpenArgs.OpenMethod == EAssetOpenMethod::Edit)
	{
		for (UAvaPlaybackGraph* Playback : OpenArgs.LoadObjects<UAvaPlaybackGraph>())
		{
			if (Playback)
			{
				const TSharedRef<FAvaPlaybackGraphEditor> PlaybackEditor = MakeShared<FAvaPlaybackGraphEditor>();
				PlaybackEditor->InitPlaybackEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, Playback);
				CommandResult = EAssetCommandResult::Handled;
			}
		}
	}
	return CommandResult;
}

namespace UE::AvaMediaEditor::PlaybackGraph::Private
{
	/** Count selected assets of the given type, loaded or not. */
	template<typename ExpectedAssetType>
	int32 GetNumSelectedAssets(const UContentBrowserAssetContextMenuContext* Context)
	{
		int32 NumSelectedAssets = 0;
		for (const FAssetData& Asset : Context->SelectedAssets)
		{
			if (Asset.IsInstanceOf(ExpectedAssetType::StaticClass()))
			{
				++NumSelectedAssets;
			}
		}
		return NumSelectedAssets;
	}

	/** Returns the count of playbacks that are loaded and playing. Will not load unloaded assets. */
	static int32 GetNumSelectedPlaybackPlaying(const UContentBrowserAssetContextMenuContext* Context)
	{
		int32 NumPlaying = 0;
		for (const FAssetData& Asset : Context->SelectedAssets)
		{
			if (Asset.IsInstanceOf(UAvaPlaybackGraph::StaticClass()))
			{
				if (const UAvaPlaybackGraph* Playback = Cast<UAvaPlaybackGraph>(Asset.FastGetAsset(false)))
				{
					if (Playback->IsPlaying())
					{
						++NumPlaying;
					}
				}
			}
		}
		return NumPlaying;
	}
	
	static bool CanExecutePlay(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		return GetNumSelectedAssets<UAvaPlaybackGraph>(Context) > GetNumSelectedPlaybackPlaying(Context);
	}

	static void ExecutePlay(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		for (UAvaPlaybackGraph* Playback : Context->LoadSelectedObjects<UAvaPlaybackGraph>())
		{
			if (Playback && !Playback->IsPlaying())
			{
				Playback->Play();
			}
		}
	}

	static bool CanExecuteStop(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		return GetNumSelectedPlaybackPlaying(Context) > 0;
	}

	static void ExecuteStop(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		for (UAvaPlaybackGraph* Playback : Context->LoadSelectedObjects<UAvaPlaybackGraph>())
		{
			if (Playback && Playback->IsPlaying())
			{
				Playback->Stop(EAvaPlaybackStopOptions::Default);
			}
		}
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []
	{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UAvaPlaybackGraph::StaticClass());

			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				{
					const TAttribute<FText> Label = LOCTEXT("MotionDesignPlayback_Play", "Play");
					const TAttribute<FText> ToolTip = LOCTEXT("MotionDesignPlayback_PlayTooltip", "Play");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericPlay");

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecutePlay);
					UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanExecutePlay);
					InSection.AddMenuEntry("MotionDesignPlayback_Play", Label, ToolTip, Icon, UIAction);
				}
				{
					const TAttribute<FText> Label = LOCTEXT("MotionDesignPlayback_Stop", "Stop");
					const TAttribute<FText> ToolTip = LOCTEXT("MotionDesignPlayback_Stop", "Stop");
					const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericPause");

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteStop);
					UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanExecuteStop);
					InSection.AddMenuEntry("MotionDesignPlayback_Stop", Label, ToolTip, Icon, UIAction);
				}
			}));
		}));
	});
}


#undef LOCTEXT_NAMESPACE
