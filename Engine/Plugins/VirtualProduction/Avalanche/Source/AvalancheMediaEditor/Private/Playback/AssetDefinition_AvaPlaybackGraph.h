// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "AssetDefinition_AvaPlaybackGraph.generated.h"

UCLASS()
class UAssetDefinition_AvaPlaybackGraph : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	//~ Begin UAssetDefinition
	virtual FText GetAssetDisplayName() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	//~ End UAssetDefinition
};
