// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/AvaPlaybackCommands.h"

#define LOCTEXT_NAMESPACE "AvaPlaybackCommands"

void FAvaPlaybackCommands::RegisterCommands()
{
	UI_COMMAND(AddInputPin
		, "Add Input Pin", "Adds an input pin to the node"
		, EUserInterfaceActionType::Button
		, FInputChord());
	
	UI_COMMAND(RemoveInputPin
		, "Remove Input Pin"
		, "Removes an input pin from the node"
		, EUserInterfaceActionType::Button
		, FInputChord());
}

#undef LOCTEXT_NAMESPACE
