// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

class FAvaPlaybackCommands : public TCommands<FAvaPlaybackCommands>
{
public:
	
	FAvaPlaybackCommands() : TCommands<FAvaPlaybackCommands>(TEXT("AvaPlayback")
		, NSLOCTEXT("MotionDesignPlaybackCommands", "MotionDesignPlaybackCommands", "Motion Design Playback")
		, NAME_None
		, FAppStyle::GetAppStyleSetName())
	{
	}

	/** Initialize commands */
	virtual void RegisterCommands() override;
	
	TSharedPtr<FUICommandInfo> AddInputPin;
	TSharedPtr<FUICommandInfo> RemoveInputPin;
};
