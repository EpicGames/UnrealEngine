// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/AvaPlaybackGraphEditor.h"

#include "AppModes/AvaPlaybackDefaultMode.h"
#include "Broadcast/AvaBroadcastEditor.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphUtilities.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GraphEditor.h"
#include "GraphEditorActions.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IAvaMediaEditorModule.h"
#include "Internationalization/Text.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Math/Color.h"
#include "Playback/AvaPlaybackCommands.h"
#include "Playback/AvaPlaybackGraph.h"
#include "Playback/Graph/AvaPlaybackEditorGraph.h"
#include "Playback/Graph/AvaPlaybackEditorGraphSchema.h"
#include "Playback/Graph/Nodes/AvaPlaybackEditorGraphNode.h"
#include "Playback/Graph/Nodes/AvaPlaybackEditorGraphNode_Root.h"
#include "Playback/Graph/SchemaActions/AvaPlaybackAction_NewComment.h"
#include "Playback/Nodes/AvaPlaybackNode.h"
#include "ScopedTransaction.h"
#include "SNodePanel.h"
#include "UObject/NameTypes.h"
#include "UObject/NoExportTypes.h"
#include "UObject/UObjectIterator.h"
#include "WorkflowOrientedApp/ApplicationMode.h"

#define LOCTEXT_NAMESPACE "AvaPlaybackGraphEditor"

TMap<TSubclassOf<UAvaPlaybackNode>, TSubclassOf<UAvaPlaybackEditorGraphNode>> FAvaPlaybackGraphEditor::OverrideNodeClasses;

void FAvaPlaybackGraphEditor::InitPlaybackEditor(const EToolkitMode::Type InMode
	, const TSharedPtr<IToolkitHost>& InitToolkitHost
	, UAvaPlaybackGraph* InPlayback)
{
	CacheOverrideNodeClasses();
	
	PlaybackGraphWeak = InPlayback;
	PlaybackGraphWeak->SetGraphEditor(SharedThis(this));
	PlaybackGraphWeak->CreateGraph();
	
	CreateDefaultCommands();
	CreateGraphCommands();
	
	const FName PlaybackEditorAppName(TEXT("MotionDesignPlaybackEditorApp"));
	constexpr bool bCreateDefaultStandaloneMenu = true;
	constexpr bool bCreateDefaultToolbar = true;
	
	InitAssetEditor(InMode
		, InitToolkitHost
		, PlaybackEditorAppName
		, FTabManager::FLayout::NullLayout
		, bCreateDefaultStandaloneMenu
		, bCreateDefaultToolbar
		, InPlayback);
	
	RegisterApplicationModes();
	PlaybackGraphWeak->DryRunGraph();
}

FName FAvaPlaybackGraphEditor::GetToolkitFName() const
{
	return TEXT("AvaPlaybackGraphEditor");
}

FText FAvaPlaybackGraphEditor::GetBaseToolkitName() const
{
	return LOCTEXT("PlaybackAppLabel", "Motion Design Playback Editor");
}

FString FAvaPlaybackGraphEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("PlaybackScriptPrefix", "Script ").ToString();
}

FLinearColor FAvaPlaybackGraphEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.0f, 0.0f, 0.3f, 0.5f);
}

UAvaPlaybackGraph* FAvaPlaybackGraphEditor::GetPlaybackObject() const
{
	return PlaybackGraphWeak.Get();
}

void FAvaPlaybackGraphEditor::ExtendToolBar(TSharedPtr<FExtender> Extender)
{
	Extender->AddToolBarExtension("Asset"
		, EExtensionHook::After
		, ToolkitCommands
		, FToolBarExtensionDelegate::CreateSP(this, &FAvaPlaybackGraphEditor::FillPlayToolBar));
}

void FAvaPlaybackGraphEditor::FillPlayToolBar(FToolBarBuilder& ToolBarBuilder)
{
	TWeakObjectPtr<UAvaPlaybackGraph> Playback = GetPlaybackObject();
	
	ToolBarBuilder.BeginSection(TEXT("Player"));
	{
		ToolBarBuilder.AddToolBarButton(FUIAction(FExecuteAction::CreateLambda([Playback]
				{
					if (Playback.IsValid())
					{
						Playback->Play();
					}
				}),
				FCanExecuteAction::CreateLambda([Playback]
				{
					return Playback.IsValid() && !Playback->IsPlaying();
				}))
			, NAME_None
			, LOCTEXT("Play_Label", "Play")
			, LOCTEXT("Play_ToolTip", "Play")
			, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Play")
		);

		ToolBarBuilder.AddToolBarButton(FUIAction(FExecuteAction::CreateLambda([Playback]
				{
					if (Playback.IsValid())
					{
						Playback->Stop(EAvaPlaybackStopOptions::ForceImmediate | EAvaPlaybackStopOptions::Unload);
					}
				}),
				FCanExecuteAction::CreateLambda([Playback]
				{
					return Playback.IsValid() && Playback->IsPlaying();
				}))
			, NAME_None
			, LOCTEXT("Stop_Label", "Stop")
			, LOCTEXT("Stop_ToolTip", "Stop")
			, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Stop")
		);
	}
	ToolBarBuilder.EndSection();
	
	ToolBarBuilder.BeginSection(TEXT("Broadcast"));
	{
		ToolBarBuilder.AddToolBarButton(FExecuteAction::CreateStatic(&FAvaBroadcastEditor::OpenBroadcastEditor)
		, NAME_None
		, LOCTEXT("Broadcast_Label", "Broadcast")
		, LOCTEXT("Broadcast_ToolTip", "Opens the Motion Design Broadcast Editor Window")
		, TAttribute<FSlateIcon>::Create([]() { return IAvaMediaEditorModule::Get().GetToolbarBroadcastButtonIcon(); }));
	}
	ToolBarBuilder.EndSection();
}

void FAvaPlaybackGraphEditor::CacheOverrideNodeClasses()
{
	if (OverrideNodeClasses.IsEmpty())
	{
		for (UClass* Class : TObjectRange<UClass>())
		{
			if (Class->IsChildOf(UAvaPlaybackEditorGraphNode::StaticClass()))
			{
				const UAvaPlaybackEditorGraphNode* const GraphNode = Cast<UAvaPlaybackEditorGraphNode>(Class->GetDefaultObject());
				if (TSubclassOf<UAvaPlaybackNode> PlaybackNodeClass = GraphNode->GetPlaybackNodeClass())
				{
					OverrideNodeClasses.Add(PlaybackNodeClass, Class);
				}
			}
		}
	}	
}

TSharedRef<SGraphEditor> FAvaPlaybackGraphEditor::CreateGraphEditor()
{
	UAvaPlaybackGraph* const Playback = GetPlaybackObject();
	check(Playback);
	
	FGraphAppearanceInfo AppearanceInfo;
	AppearanceInfo.CornerText = LOCTEXT("AppearanceCornerText_Playback", "Motion Design PLAYBACK");

	SGraphEditor::FGraphEditorEvents InEvents;
	InEvents.OnSelectionChanged  = SGraphEditor::FOnSelectionChanged::CreateSP(this, &FAvaPlaybackGraphEditor::OnSelectedNodesChanged);
	InEvents.OnTextCommitted     = FOnNodeTextCommitted::CreateSP(this, &FAvaPlaybackGraphEditor::OnNodeTitleCommitted);

	GraphEditor = SNew(SGraphEditor)
		.AdditionalCommands(GraphEditorCommands)
		.IsEditable(true)
		.Appearance(AppearanceInfo)
		.GraphToEdit(Playback->GetGraph())
		.GraphEvents(InEvents)
		.AutoExpandActionMenu(true)
		.ShowGraphStateOverlay(false);
	
	return GraphEditor.ToSharedRef();
}

UEdGraph* FAvaPlaybackGraphEditor::CreatePlaybackGraph(UAvaPlaybackGraph* InPlayback)
{
	UAvaPlaybackEditorGraph* NewPlaybackGraph = CastChecked<UAvaPlaybackEditorGraph>(FBlueprintEditorUtils::CreateNewGraph(InPlayback
		, NAME_None
		, UAvaPlaybackEditorGraph::StaticClass()
		, UAvaPlaybackEditorGraphSchema::StaticClass()));
	
	return NewPlaybackGraph;
}

void FAvaPlaybackGraphEditor::SetupPlaybackNode(UEdGraph* InGraph
	, UAvaPlaybackNode* InPlaybackNode
	, bool bSelectNewNode)
{
	TSubclassOf<UAvaPlaybackEditorGraphNode> NodeClass = UAvaPlaybackEditorGraphNode::StaticClass();
	
	//Find Most Relevant Override Node Class
	{
		UClass* StartingClass = InPlaybackNode->GetClass();
		while (StartingClass && StartingClass->IsChildOf(UAvaPlaybackNode::StaticClass()))
		{
			if (TSubclassOf<UAvaPlaybackEditorGraphNode>* const OverrideClass = OverrideNodeClasses.Find(StartingClass))
			{
				NodeClass = *OverrideClass;
				break;
			}
			StartingClass = StartingClass->GetSuperClass();
		}

		if (!NodeClass)
		{
			NodeClass = UAvaPlaybackEditorGraphNode::StaticClass();
		}
	}
	
	UAvaPlaybackEditorGraph* const PlaybackGraph = Cast<UAvaPlaybackEditorGraph>(InGraph);
	check(PlaybackGraph  && NodeClass);
	
	UAvaPlaybackEditorGraphNode* const Node = PlaybackGraph->CreatePlaybackEditorGraphNode(NodeClass, bSelectNewNode);
	check(Node);
	
	Node->SetPlaybackNode(InPlaybackNode);
	Node->CreateNewGuid();
	Node->PostPlacedNewNode();
	
	if (Node->Pins.Num() == 0)
	{
		Node->AllocateDefaultPins();
	}
}

void FAvaPlaybackGraphEditor::CompilePlaybackNodesFromGraphNodes(UAvaPlaybackGraph* InPlayback)
{
	UEdGraph* const PlaybackGraph = InPlayback->GetGraph();
	check(PlaybackGraph);

	TArray<UAvaPlaybackNode*> PlaybackNodes;
	PlaybackNodes.Reserve(PlaybackGraph->Nodes.Num());
	
	for (UEdGraphNode* const EdGraphNode : PlaybackGraph->Nodes)
	{
		UAvaPlaybackEditorGraphNode* const GraphNode = Cast<UAvaPlaybackEditorGraphNode>(EdGraphNode);
		if (GraphNode && GraphNode->GetPlaybackNode())
		{
			UAvaPlaybackNode* const PlaybackNode = GraphNode->GetPlaybackNode();
			
			// Set ChildNodes of each PlaybackNode
			TArray<UEdGraphPin*> InputPins = GraphNode->GetInputPins();
			
			TArray<UAvaPlaybackNode*> ChildNodes;
			ChildNodes.Reserve(InputPins.Num());
			
			for (UEdGraphPin* const InputPin : InputPins)
			{
				if (InputPin->LinkedTo.Num() > 0)
				{
					//Note: A Playback Node Input Pin can't be connected to multiple sources.
					UAvaPlaybackEditorGraphNode* GraphChildNode = CastChecked<UAvaPlaybackEditorGraphNode>(InputPin->LinkedTo[0]->GetOwningNode());
					ChildNodes.Add(GraphChildNode->GetPlaybackNode());
				}
				else
				{
					ChildNodes.AddZeroed();
				}
			}

			PlaybackNode->SetFlags(RF_Transactional);
			PlaybackNode->Modify();
			PlaybackNode->SetChildNodes(MoveTemp(ChildNodes));
			
			PlaybackNodes.Emplace(PlaybackNode);
		}
	}	
	
	InPlayback->DryRunGraph();
	
	for (UAvaPlaybackNode* const PlaybackNode : PlaybackNodes)
	{
		PlaybackNode->PostEditChange();
	}
}

void FAvaPlaybackGraphEditor::CreateInputPin(UEdGraphNode* InGraphNode)
{
	CastChecked<UAvaPlaybackEditorGraphNode>(InGraphNode)->CreateInputPin();
}

void FAvaPlaybackGraphEditor::RefreshNode(UEdGraphNode& InNode)
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->RefreshNode(InNode);
	}
}

bool FAvaPlaybackGraphEditor::GetBoundsForSelectedNodes(FSlateRect& Rect, float Padding)
{
	if (GraphEditor.IsValid())
	{
		return GraphEditor->GetBoundsForSelectedNodes(Rect, Padding);
	}
	return false;
}

TSet<UObject*> FAvaPlaybackGraphEditor::GetSelectedNodes() const
{
	if (GraphEditor.IsValid())
	{
		return GraphEditor->GetSelectedNodes();
	}
	return TSet<UObject*>();
}

void FAvaPlaybackGraphEditor::OnSelectedNodesChanged(const TSet<UObject*>& NewSelection)
{
	TArray<UObject*> Selection;
	if (NewSelection.Num())
	{
		for (TSet<UObject*>::TConstIterator Iter(NewSelection); Iter; ++Iter)
		{
			if (UAvaPlaybackEditorGraphNode* const GraphNode = Cast<UAvaPlaybackEditorGraphNode>(*Iter))
			{
				Selection.Add(GraphNode->GetPlaybackNode());
			}
			else
			{
				Selection.Add(*Iter);
			}
		}
	}
	OnPlaybackSelectionChanged.Broadcast(Selection);
}

void FAvaPlaybackGraphEditor::OnNodeTitleCommitted(const FText& NewText
	, ETextCommit::Type CommitInfo
	, UEdGraphNode* NodeBeingChanged)
{
	if (NodeBeingChanged)
	{
		const FScopedTransaction Transaction(LOCTEXT("RenameNode", "Rename Node"));
		NodeBeingChanged->Modify();
		NodeBeingChanged->OnRenameNode(NewText.ToString());
	}
}

void FAvaPlaybackGraphEditor::RegisterApplicationModes()
{
	TArray<TSharedRef<FApplicationMode>> ApplicationModes;
	TSharedPtr<FAvaPlaybackGraphEditor> This = SharedThis(this);
	
	ApplicationModes.Add(MakeShared<FAvaPlaybackDefaultMode>(This));
	//Can add more App Modes here
	
	for (const TSharedRef<FApplicationMode>& AppMode : ApplicationModes)
	{
		AddApplicationMode(AppMode->GetModeName(), AppMode);
	}

	SetCurrentMode(FAvaPlaybackAppMode::DefaultMode);
}

void FAvaPlaybackGraphEditor::CreateDefaultCommands()
{
}

void FAvaPlaybackGraphEditor::CreateGraphCommands()
{
	GraphEditorCommands = MakeShared<FUICommandList>();
	{
		const FGraphEditorCommandsImpl& GraphCommands = FGraphEditorCommands::Get();
		const FGenericCommands& GenericCommands = FGenericCommands::Get();
		const FAvaPlaybackCommands& PlaybackCommands = FAvaPlaybackCommands::Get();
			
		// Playback Commands
		GraphEditorCommands->MapAction(PlaybackCommands.AddInputPin,
			FExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::AddInputPin),
			FCanExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::CanAddInputPin));

		GraphEditorCommands->MapAction(PlaybackCommands.RemoveInputPin,
			FExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::RemoveInputPin),
			FCanExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::CanRemoveInputPin));
		
		// Graph Editor Commands
		GraphEditorCommands->MapAction(GraphCommands.CreateComment
			, FExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::CreateComment));

		// Editing commands
		GraphEditorCommands->MapAction(GenericCommands.SelectAll
			, FExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::SelectAllNodes)
			, FCanExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::CanSelectAllNodes)
		);

		GraphEditorCommands->MapAction(GenericCommands.Delete
			, FExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::DeleteSelectedNodes)
			, FCanExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::CanDeleteSelectedNodes)
		);
		
		GraphEditorCommands->MapAction(GenericCommands.Copy
			, FExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::CopySelectedNodes)
			, FCanExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::CanCopySelectedNodes)
		);

		GraphEditorCommands->MapAction(GenericCommands.Cut
			, FExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::CutSelectedNodes)
			, FCanExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::CanCutSelectedNodes)
		);

		GraphEditorCommands->MapAction(GenericCommands.Paste
			, FExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::PasteNodes)
			, FCanExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::CanPasteNodes)
		);

		GraphEditorCommands->MapAction(GenericCommands.Duplicate
			, FExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::DuplicateSelectedNodes)
			, FCanExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::CanDuplicateSelectedNodes)
		);

		// Alignment Commands
		GraphEditorCommands->MapAction(GraphCommands.AlignNodesTop
			, FExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::OnAlignTop)
		);

		GraphEditorCommands->MapAction(GraphCommands.AlignNodesMiddle
			, FExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::OnAlignMiddle)
		);

		GraphEditorCommands->MapAction(GraphCommands.AlignNodesBottom
			, FExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::OnAlignBottom)
		);

		GraphEditorCommands->MapAction(GraphCommands.AlignNodesLeft
			, FExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::OnAlignLeft)
		);

		GraphEditorCommands->MapAction(GraphCommands.AlignNodesCenter
			, FExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::OnAlignCenter)
		);

		GraphEditorCommands->MapAction(GraphCommands.AlignNodesRight
			, FExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::OnAlignRight)
		);

		GraphEditorCommands->MapAction(GraphCommands.StraightenConnections
			, FExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::OnStraightenConnections)
		);

		// Distribution Commands
		GraphEditorCommands->MapAction(GraphCommands.DistributeNodesHorizontally
			, FExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::OnDistributeNodesH)
		);

		GraphEditorCommands->MapAction(GraphCommands.DistributeNodesVertically
			, FExecuteAction::CreateSP(this, &FAvaPlaybackGraphEditor::OnDistributeNodesV)
		);
	}
}

bool FAvaPlaybackGraphEditor::CanAddInputPin() const
{
	return GetSelectedNodes().Num() == 1;
}

void FAvaPlaybackGraphEditor::AddInputPin()
{
	const TSet<UObject*> SelectedNodes = GetSelectedNodes();
	// Iterator used but should only contain one node
	for (TSet<UObject*>::TConstIterator It(SelectedNodes); It; ++It)
	{
		if (UAvaPlaybackEditorGraphNode* SelectedNode = Cast<UAvaPlaybackEditorGraphNode>(*It))
		{
			SelectedNode->AddInputPin();
			break;
		}
	}
}

bool FAvaPlaybackGraphEditor::CanRemoveInputPin() const
{
	return true;
}

void FAvaPlaybackGraphEditor::RemoveInputPin()
{
	if (GraphEditor.IsValid())
	{
		if (UEdGraphPin* const SelectedPin = GraphEditor->GetGraphPinForMenu())
		{
			UAvaPlaybackEditorGraphNode* const SelectedNode = Cast<UAvaPlaybackEditorGraphNode>(SelectedPin->GetOwningNode());
			if (SelectedNode && SelectedNode == SelectedPin->GetOwningNode())
			{
				SelectedNode->RemoveInputPin(SelectedPin);
			}
		}
	}
}

void FAvaPlaybackGraphEditor::CreateComment()
{
	if (GraphEditor.IsValid())
	{
		FAvaPlaybackAction_NewComment CommentAction;
		CommentAction.PerformAction(GraphEditor->GetCurrentGraph(), nullptr, GraphEditor->GetPasteLocation2f());
	}
}

void FAvaPlaybackGraphEditor::SelectAllNodes()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->SelectAllNodes();
	}
}

bool FAvaPlaybackGraphEditor::CanDeleteSelectedNodes() const
{
	const TSet<UObject*> SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() == 1)
	{
		for (TSet<UObject*>::TConstIterator It(SelectedNodes); It; ++It)
		{
			if (Cast<UAvaPlaybackEditorGraphNode_Root>(*It))
			{
				// Return false if only root node is selected, as it can't be deleted
				return false;
			}
		}
	}
	return SelectedNodes.Num() > 0;
}

void FAvaPlaybackGraphEditor::DeleteSelectedNodes()
{
	if (!GraphEditor.IsValid() || !GraphEditor->GetCurrentGraph())
	{
		return;
	}
	
	const FScopedTransaction Transaction(LOCTEXT("DeleteSelectedNodes", "Delete Selected Nodes"));
	GraphEditor->GetCurrentGraph()->Modify();

	const TSet<UObject*> SelectedNodes = GetSelectedNodes();
	GraphEditor->ClearSelectionSet();

	check(PlaybackGraphWeak.IsValid());
	for (TSet<UObject*>::TConstIterator It(SelectedNodes); It; ++It)
	{
		UEdGraphNode* const Node = CastChecked<UEdGraphNode>(*It);
		if (Node->CanUserDeleteNode())
		{
			if (UAvaPlaybackEditorGraphNode* const PlaybackGraphNode = Cast<UAvaPlaybackEditorGraphNode>(Node))
			{
				UAvaPlaybackNode* const NodeToDelete = PlaybackGraphNode->GetPlaybackNode();
				FBlueprintEditorUtils::RemoveNode(nullptr, PlaybackGraphNode, true);

				// Make sure Playback is updated to match graph
				PlaybackGraphWeak->CompilePlaybackNodesFromGraphNodes();

				// Remove this node from the PlaybackGraphWeak's list of all Playback Nodes
				PlaybackGraphWeak->RemovePlaybackNode(NodeToDelete);
				PlaybackGraphWeak->MarkPackageDirty();
			}
			else
			{
				FBlueprintEditorUtils::RemoveNode(nullptr, Node, true);
			}
		}
	}
}

void FAvaPlaybackGraphEditor::DeleteSelectedDuplicatableNodes()
{
	if (!GraphEditor.IsValid())
	{
		return;
	}
	
	// Cache off the old selection
	const TSet<UObject*> OldSelectedNodes = GetSelectedNodes();

	// Clear the selection and only select the nodes that can be duplicated
	TSet<UObject*> RemainingNodes;
	GraphEditor->ClearSelectionSet();

	for (TSet<UObject*>::TConstIterator Iter(OldSelectedNodes); Iter; ++Iter)
	{
		UEdGraphNode* const Node = Cast<UEdGraphNode>(*Iter);
		if (Node && Node->CanDuplicateNode())
		{
			GraphEditor->SetNodeSelection(Node, true);
		}
		else
		{
			RemainingNodes.Add(Node);
		}
	}

	// Delete the duplicatable nodes
	DeleteSelectedNodes();

	// Reselect whatever's left from the original selection after the deletion
	GraphEditor->ClearSelectionSet();

	for (TSet<UObject*>::TConstIterator Iter(RemainingNodes); Iter; ++Iter)
	{
		if (UEdGraphNode* const Node = Cast<UEdGraphNode>(*Iter))
		{
			GraphEditor->SetNodeSelection(Node, true);
		}
	}
}

bool FAvaPlaybackGraphEditor::CanCopySelectedNodes() const
{
	const TSet<UObject*> SelectedNodes = GetSelectedNodes();
	
	for (TSet<UObject*>::TConstIterator Iter(SelectedNodes); Iter; ++Iter)
	{
		UEdGraphNode* const Node = Cast<UEdGraphNode>(*Iter);
		
		//If at least one node can be duplicated, we should allow copy
		if (Node && Node->CanDuplicateNode())
		{
			return true;
		}
	}
	
	return false;
}

void FAvaPlaybackGraphEditor::CopySelectedNodes()
{
	// Export the selected nodes and place the text on the clipboard
	const TSet<UObject*> SelectedNodes = GetSelectedNodes();
	
	for (TSet<UObject*>::TConstIterator Iter(SelectedNodes); Iter; ++Iter)
	{
		if (UAvaPlaybackEditorGraphNode* const Node = Cast<UAvaPlaybackEditorGraphNode>(*Iter))
		{
			Node->PrepareForCopying();
		}
	}
	
	FString ExportedText;
	FEdGraphUtilities::ExportNodesToText(SelectedNodes, ExportedText);
	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);
	
	for (TSet<UObject*>::TConstIterator Iter(SelectedNodes); Iter; ++Iter)
	{
		if (UAvaPlaybackEditorGraphNode* const Node = Cast<UAvaPlaybackEditorGraphNode>(*Iter))
		{
			Node->PostCopyNode();
		}
	}
}

bool FAvaPlaybackGraphEditor::CanCutSelectedNodes() const
{
	return CanCopySelectedNodes() && CanDeleteSelectedNodes();
}

void FAvaPlaybackGraphEditor::CutSelectedNodes()
{
	CopySelectedNodes();
	// Cut should only delete nodes that can be duplicated
	DeleteSelectedDuplicatableNodes();
}

bool FAvaPlaybackGraphEditor::CanPasteNodes() const
{
	if (PlaybackGraphWeak.IsValid() && PlaybackGraphWeak->GetGraph())
	{
		FString ClipboardContent;
		FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);
		return FEdGraphUtilities::CanImportNodesFromText(PlaybackGraphWeak->GetGraph(), ClipboardContent);
	}
	return false;
}

void FAvaPlaybackGraphEditor::PasteNodes()
{
	if (GraphEditor.IsValid())
	{
		PasteNodesHere(GraphEditor->GetPasteLocation2f());
	}
}

void FAvaPlaybackGraphEditor::PasteNodesHere(const FVector2D& Location)
{
	if (!GraphEditor.IsValid() || !PlaybackGraphWeak.IsValid() || !PlaybackGraphWeak->GetGraph())
	{
		return;
	}
	
	// Undo/Redo support
	const FScopedTransaction Transaction(LOCTEXT("PasteNodes", "Paste Playback Nodes"));
	PlaybackGraphWeak->GetGraph()->Modify();
	PlaybackGraphWeak->Modify();

	// Clear the selection set (newly pasted stuff will be selected)
	GraphEditor->ClearSelectionSet();

	// Grab the text to paste from the clipboard.
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);

	// Import the nodes
	TSet<UEdGraphNode*> PastedNodes;
	FEdGraphUtilities::ImportNodesFromText(PlaybackGraphWeak->GetGraph(), TextToImport, /*out*/ PastedNodes);

	//Average position of nodes so we can move them while still maintaining relative distances to each other
	FVector2D AverageNodePosition(0.0f,0.0f);

	for (TSet<UEdGraphNode*>::TIterator It(PastedNodes); It; ++It)
	{
		UEdGraphNode* Node = *It;
		AverageNodePosition.X += Node->NodePosX;
		AverageNodePosition.Y += Node->NodePosY;
	}

	if (PastedNodes.Num() > 0)
	{
		FVector2D::FReal InverseNumNodes = 1.0 / static_cast<FVector2D::FReal>(PastedNodes.Num());
		AverageNodePosition.X *= InverseNumNodes;
		AverageNodePosition.Y *= InverseNumNodes;
	}

	for (TSet<UEdGraphNode*>::TIterator It(PastedNodes); It; ++It)
	{
		UEdGraphNode* const Node = *It;
		if (UAvaPlaybackEditorGraphNode* PlaybackGraphNode = Cast<UAvaPlaybackEditorGraphNode>(Node))
		{
			PlaybackGraphWeak->AddPlaybackNode(PlaybackGraphNode->GetPlaybackNode());
		}

		// Select the newly pasted stuff
		GraphEditor->SetNodeSelection(Node, true);

		Node->NodePosX = (Node->NodePosX - AverageNodePosition.X) + Location.X ;
		Node->NodePosY = (Node->NodePosY - AverageNodePosition.Y) + Location.Y ;

		Node->SnapToGrid(SNodePanel::GetSnapGridSize());

		// Give new node a different Guid from the old one
		Node->CreateNewGuid();
	}

	// Force new pasted nodes to have same connections as graph nodes
	PlaybackGraphWeak->CompilePlaybackNodesFromGraphNodes();

	// Update UI
	GraphEditor->NotifyGraphChanged();

	PlaybackGraphWeak->PostEditChange();
	PlaybackGraphWeak->MarkPackageDirty();
}

bool FAvaPlaybackGraphEditor::CanDuplicateSelectedNodes() const
{
	return CanCopySelectedNodes();
}

void FAvaPlaybackGraphEditor::DuplicateSelectedNodes()
{
	CopySelectedNodes();
	PasteNodes();
}

void FAvaPlaybackGraphEditor::OnAlignTop()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnAlignTop();
	}
}

void FAvaPlaybackGraphEditor::OnAlignMiddle()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnAlignMiddle();
	}
}

void FAvaPlaybackGraphEditor::OnAlignBottom()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnAlignBottom();
	}
}

void FAvaPlaybackGraphEditor::OnAlignLeft()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnAlignLeft();
	}
}

void FAvaPlaybackGraphEditor::OnAlignCenter()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnAlignCenter();
	}
}

void FAvaPlaybackGraphEditor::OnAlignRight()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnAlignRight();
	}
}

void FAvaPlaybackGraphEditor::OnStraightenConnections()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnStraightenConnections();
	}
}

void FAvaPlaybackGraphEditor::OnDistributeNodesH()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnDistributeNodesH();
	}
}

void FAvaPlaybackGraphEditor::OnDistributeNodesV()
{
	if (GraphEditor.IsValid())
	{
		GraphEditor->OnDistributeNodesV();
	}
}

#undef LOCTEXT_NAMESPACE
