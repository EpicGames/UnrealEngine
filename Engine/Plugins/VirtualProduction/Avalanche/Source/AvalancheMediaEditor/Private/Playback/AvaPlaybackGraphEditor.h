// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StringFwd.h"
#include "Math/MathFwd.h"
#include "Playback/IAvaPlaybackGraphEditor.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Toolkits/IToolkit.h"
#include "Toolkits/IToolkitHost.h"
#include "Types/SlateEnums.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FExtender;
class FName;
class FSlateRect;
class FText;
class FToolBarBuilder;
class FUICommandList;
class SGraphEditor;
class UAvaPlaybackEditorGraphNode;
class UAvaPlaybackGraph;
class UAvaPlaybackNode;
class UEdGraph;
class UEdGraphNode;
class UObject;
struct FLinearColor;

class FAvaPlaybackGraphEditor
	: public FWorkflowCentricApplication
	, public IAvaPlaybackGraphEditor
{
public:
	void InitPlaybackEditor(const EToolkitMode::Type InMode
		, const TSharedPtr<IToolkitHost>& InitToolkitHost
		, UAvaPlaybackGraph* InPlayback);

	//~ Begin IToolkit
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	//~ End IToolkit

	UAvaPlaybackGraph* GetPlaybackObject() const;
	
	void ExtendToolBar(TSharedPtr<FExtender> Extender);
	void FillPlayToolBar(FToolBarBuilder& ToolBarBuilder);

public:
	static void CacheOverrideNodeClasses();
	
	TSharedRef<SGraphEditor> CreateGraphEditor();
	
	//~ Begin IAvaPlaybackGraphEditor
	virtual UEdGraph* CreatePlaybackGraph(UAvaPlaybackGraph* InPlayback) override;
	virtual void SetupPlaybackNode(UEdGraph* InGraph, UAvaPlaybackNode* InPlaybackNode, bool bSelectNewNode) override;
	virtual void CompilePlaybackNodesFromGraphNodes(UAvaPlaybackGraph* InPlayback) override;
	virtual void CreateInputPin(UEdGraphNode* InGraphNode) override;
	virtual void RefreshNode(UEdGraphNode& InGraphNode) override;
	virtual bool GetBoundsForSelectedNodes(FSlateRect& Rect, float Padding) override;
	virtual TSet<UObject*> GetSelectedNodes() const override;
	//~ End IAvaPlaybackGraphEditor
	
	/** Called when the selection changes in the GraphEditor */
	void OnSelectedNodesChanged(const TSet<UObject*>& NewSelection);

	/**
	 * Called when a node's title is committed for a rename
	 *
	 * @param	NewText				New title text
	 * @param	CommitInfo			How text was committed
	 * @param	NodeBeingChanged	The node being changed
	 */
	void OnNodeTitleCommitted(const FText& NewText, ETextCommit::Type CommitInfo, UEdGraphNode* NodeBeingChanged);
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPlaybackSelectionChanged, const TArray<UObject*>&);
	FOnPlaybackSelectionChanged OnPlaybackSelectionChanged;
	
protected:
	void RegisterApplicationModes();
	
	void CreateDefaultCommands();
	void CreateGraphCommands();

public:
	bool CanAddInputPin() const;
	void AddInputPin();

	bool CanRemoveInputPin() const;
	void RemoveInputPin();
	
	void CreateComment();
	
	bool CanSelectAllNodes() const { return true; }
	void SelectAllNodes();

	bool CanDeleteSelectedNodes() const;
	void DeleteSelectedNodes();
	void DeleteSelectedDuplicatableNodes();
	
	bool CanCopySelectedNodes() const;
	void CopySelectedNodes();
	
	bool CanCutSelectedNodes() const;
	void CutSelectedNodes();
	
	virtual bool CanPasteNodes() const override;
	void PasteNodes();
	virtual void PasteNodesHere(const FVector2D& Location) override;
	
	bool CanDuplicateSelectedNodes() const;
	void DuplicateSelectedNodes();

	void OnAlignTop();
	void OnAlignMiddle();
	void OnAlignBottom();
	void OnAlignLeft();
	void OnAlignCenter();
	void OnAlignRight();

	void OnStraightenConnections();
	void OnDistributeNodesH();
	void OnDistributeNodesV();
	
protected:
	TWeakObjectPtr<UAvaPlaybackGraph> PlaybackGraphWeak;
	
	TSharedPtr<FUICommandList> GraphEditorCommands;

	TSharedPtr<SGraphEditor> GraphEditor;
	
	static TMap<TSubclassOf<UAvaPlaybackNode>, TSubclassOf<UAvaPlaybackEditorGraphNode>> OverrideNodeClasses;
};
