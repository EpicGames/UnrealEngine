// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/AvaPlaybackGraphFactory.h"

#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "Playback/AvaPlaybackGraph.h"

#define LOCTEXT_NAMESPACE "AvaPlaybackGraphFactory"

namespace UE::AvaMediaEditor::PlaybackGraphFactory::Private
{
	/** 
	 * CVar to specify if new playback graph assets can be created.
	 * Default is false.
	 */
	bool bEnableCreateNewPlaybackGraph = false;
	FAutoConsoleVariableRef CVarEnableCreateNewPlaybackGraph(
		TEXT("MotionDesignPlaybackGraphFactory.EnableCreateNew"),
		bEnableCreateNewPlaybackGraph,
		TEXT("Specify if new playback graph assets can be created."),
		ECVF_Default
	);
}

UAvaPlaybackGraphFactory::UAvaPlaybackGraphFactory()
{
	// Provide the factory with information about how to handle our asset
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UAvaPlaybackGraph::StaticClass();
}

UAvaPlaybackGraphFactory::~UAvaPlaybackGraphFactory()
{
}

bool UAvaPlaybackGraphFactory::CanCreateNew() const
{
	return UE::AvaMediaEditor::PlaybackGraphFactory::Private::bEnableCreateNewPlaybackGraph;	
}

uint32 UAvaPlaybackGraphFactory::GetMenuCategories() const
{
	const IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	return AssetTools.FindAdvancedAssetCategory("MotionDesignCategory");
}

UObject* UAvaPlaybackGraphFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags
  , UObject* Context, FFeedbackContext* Warn)
{
	UAvaPlaybackGraph* Playback = nullptr;
	if (ensure(SupportedClass == Class))
	{
		Playback = NewObject<UAvaPlaybackGraph>(InParent, Name, Flags);
	}
	return Playback;
}

FString UAvaPlaybackGraphFactory::GetDefaultNewAssetName() const
{
	return TEXT("NewPlaybackGraph");
}

#undef LOCTEXT_NAMESPACE
