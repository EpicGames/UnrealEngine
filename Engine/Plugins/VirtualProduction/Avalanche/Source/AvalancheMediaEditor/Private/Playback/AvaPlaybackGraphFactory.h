// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "AvaPlaybackGraphFactory.generated.h"

UCLASS()
class UAvaPlaybackGraphFactory : public UFactory
{
	GENERATED_BODY()

public:

	UAvaPlaybackGraphFactory();
	virtual ~UAvaPlaybackGraphFactory() override;

protected:
	
	//~ Begin UFactory Interface
	virtual bool CanCreateNew() const override;
	virtual uint32 GetMenuCategories() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual FString GetDefaultNewAssetName() const override;
	//~ Begin UFactory Interface
};
