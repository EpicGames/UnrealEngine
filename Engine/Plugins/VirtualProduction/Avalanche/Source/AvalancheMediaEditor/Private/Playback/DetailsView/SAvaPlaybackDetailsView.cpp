// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaPlaybackDetailsView.h"
#include "DetailsViewArgs.h"
#include "Modules/ModuleManager.h"
#include "Playback/AvaPlaybackGraphEditor.h"
#include "PropertyEditorModule.h"

void SAvaPlaybackDetailsView::Construct(const FArguments& InArgs, const TSharedPtr<FAvaPlaybackGraphEditor>& InPlaybackEditor)
{
	PlaybackEditorWeak = InPlaybackEditor;
	
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	InPlaybackEditor->OnPlaybackSelectionChanged.AddRaw(this, &SAvaPlaybackDetailsView::OnPlaybackNodeSelectionChanged);
	
	ChildSlot
	[
		DetailsView.ToSharedRef()
	];

	OnPlaybackNodeSelectionChanged({});
}

SAvaPlaybackDetailsView::~SAvaPlaybackDetailsView()
{
	if (TSharedPtr<FAvaPlaybackGraphEditor> PlaybackEditor = PlaybackEditorWeak.Pin())
	{
		PlaybackEditor->OnPlaybackSelectionChanged.RemoveAll(this);
	}
}

void SAvaPlaybackDetailsView::OnPlaybackNodeSelectionChanged(const TArray<UObject*>& InSelectedObjects)
{
	DetailsView->SetObjects(InSelectedObjects);
}
