// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FAvaPlaybackGraphEditor;
class IDetailsView;
class UObject;

class SAvaPlaybackDetailsView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaPlaybackDetailsView){}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedPtr<FAvaPlaybackGraphEditor>& InPlaybackEditor);
	virtual ~SAvaPlaybackDetailsView() override;
	
	void OnPlaybackNodeSelectionChanged(const TArray<UObject*>& InSelectedObjects);

protected:
	TWeakPtr<FAvaPlaybackGraphEditor> PlaybackEditorWeak;
	
	TSharedPtr<IDetailsView> DetailsView;
};
