// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPlaybackConnectionDrawingPolicy.h"

#include "Playback/AvaPlaybackGraph.h"
#include "Playback/Graph/AvaPlaybackEditorGraph.h"
#include "Playback/Graph/AvaPlaybackEditorGraphSchema.h"
#include "Playback/Graph/Nodes/AvaPlaybackEditorGraphNode_Root.h"

FConnectionDrawingPolicy* FAvaPlaybackConnectionDrawingPolicyFactory::CreateConnectionPolicy(const UEdGraphSchema* Schema
	, int32 InBackLayerID
	, int32 InFrontLayerID
	, float ZoomFactor
	, const FSlateRect& InClippingRect
	, FSlateWindowElementList& InDrawElements
	, UEdGraph* InGraphObj) const
{
	if (Schema->IsA(UAvaPlaybackEditorGraphSchema::StaticClass()))
	{
		return new FAvaPlaybackConnectionDrawingPolicy(InBackLayerID
			, InFrontLayerID
			, ZoomFactor
			, InClippingRect
			, InDrawElements
			, InGraphObj);
	}
	return nullptr;
}

FAvaPlaybackConnectionDrawingPolicy::FAvaPlaybackConnectionDrawingPolicy(int32 InBackLayerID
	, int32 InFrontLayerID
	, float ZoomFactor
	, const FSlateRect& InClippingRect
	, FSlateWindowElementList& InDrawElements
	, UEdGraph* InGraphObject)

	: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements)
	, GraphObject(InGraphObject)
{
	// Cache off the editor options
	ActiveColor = Settings->TraceAttackColor;
	InactiveColor = Settings->TraceReleaseColor;

	ActiveWireThickness = Settings->TraceAttackWireThickness;
	InactiveWireThickness = Settings->TraceReleaseWireThickness;

	// Don't want to draw ending arrowheads
	ArrowImage = nullptr;
	ArrowRadius = FVector2D::ZeroVector;
}

void FAvaPlaybackConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* OutputPin
	, UEdGraphPin* InputPin
	, FConnectionParams& Params)
{
	Params.AssociatedPin1 = OutputPin;
	Params.AssociatedPin2 = InputPin;

	// Get the schema and grab the default color from it
	check(OutputPin);
	check(GraphObject);
	const UEdGraphSchema* const Schema = GraphObject->GetSchema();

	Params.WireColor = Schema->GetPinTypeColor(OutputPin->PinType);
	UAvaPlaybackEditorGraph* const Graph = Cast<UAvaPlaybackEditorGraph>(GraphObject);
	
	if (!InputPin || !Graph)
	{
		return;
	}
	
	bool bExecuted = false;

	const UAvaPlaybackGraph* const Playback = Graph->GetPlaybackGraph();
	const bool bIsPlaying = Playback ? Playback->IsPlaying() : false;
	
	UAvaPlaybackEditorGraphNode* const InputNode  = Cast<UAvaPlaybackEditorGraphNode>(InputPin->GetOwningNode());
	UAvaPlaybackEditorGraphNode* const OutputNode = Cast<UAvaPlaybackEditorGraphNode>(OutputPin->GetOwningNode());

	if (bIsPlaying && InputNode && OutputNode)
	{
		const int32 InputIndex  = InputNode->GetInputPinIndex(InputPin);

		const double ChildTickedTime            = OutputNode->GetLastTimeTicked();
		const double ChildTickedTimeFromParent  = InputNode->GetChildLastTimeTicked(InputIndex);
		const double RootTickedTime             = Playback->GetRootNode()->GetLastTimeTicked();
		constexpr float GracePeriod             = 0.25f; //Time in Seconds to allow for the Child to still be considered Active after receiving its last signal
		
		if (ChildTickedTime > 0.f 
			&& FMath::IsNearlyEqual(ChildTickedTime, RootTickedTime, GracePeriod) // Child Ticked Recently
			&& FMath::IsNearlyEqual(ChildTickedTime, ChildTickedTimeFromParent, GracePeriod)) // Child Ticked through this Input Node (Parent) Recently
		{
			bExecuted = true;
			Params.WireThickness = ActiveWireThickness;
			Params.WireColor = ActiveColor;
			Params.bDrawBubbles = true;
		}
	}

	if (!bExecuted)
	{
		// It's not followed, fade it and keep it thin
		Params.WireColor = InactiveColor;
		Params.WireThickness = InactiveWireThickness;
	}
}
