// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConnectionDrawingPolicy.h"
#include "EdGraphUtilities.h"

struct FAvaPlaybackConnectionDrawingPolicyFactory : public FGraphPanelPinConnectionFactory
{
public:
	
	virtual ~FAvaPlaybackConnectionDrawingPolicyFactory() override
	{
	}

	// FGraphPanelPinConnectionFactory
	virtual FConnectionDrawingPolicy* CreateConnectionPolicy(const UEdGraphSchema* Schema
		, int32 InBackLayerID
		, int32 InFrontLayerID
		, float ZoomFactor
		, const FSlateRect& InClippingRect
		, FSlateWindowElementList& InDrawElements
		, UEdGraph* InGraphObj) const override;
	// ~FGraphPanelPinConnectionFactory
};

class FAvaPlaybackConnectionDrawingPolicy  : public FConnectionDrawingPolicy
{
protected:
	
	// Times for one execution pair within the current graph
	struct FTimePair
	{
		double PredExecTime;
		double ThisExecTime;

		FTimePair()
			: PredExecTime(0.0)
			, ThisExecTime(0.0)
		{
		}
	};

	// Map of pairings
	using FExecPairingMap = TMap<UEdGraphNode*, FTimePair>;

	// Map of nodes that preceded before a given node in the execution sequence (one entry for each pairing)
	TMap<UEdGraphNode*, FExecPairingMap> PredecessorNodes;

	UEdGraph* GraphObject;

	FLinearColor ActiveColor;
	FLinearColor InactiveColor;

	float ActiveWireThickness;
	float InactiveWireThickness;

public:
	
	FAvaPlaybackConnectionDrawingPolicy(int32 InBackLayerID
		, int32 InFrontLayerID
		, float ZoomFactor
		, const FSlateRect& InClippingRect
		, FSlateWindowElementList& InDrawElements
		, UEdGraph* InGraphObject);
	
	// FConnectionDrawingPolicy interface
	virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params) override;
	// End of FConnectionDrawingPolicy interface
};
