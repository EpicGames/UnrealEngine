// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Graph/AvaPlaybackEditorGraph.h"
#include "Playback/AvaPlaybackGraph.h"
#include "Playback/Graph/Nodes/AvaPlaybackEditorGraphNode.h"

UAvaPlaybackGraph* UAvaPlaybackEditorGraph::GetPlaybackGraph() const
{
	return CastChecked<UAvaPlaybackGraph>(GetOuter());
}

UAvaPlaybackEditorGraphNode* UAvaPlaybackEditorGraph::CreatePlaybackEditorGraphNode(TSubclassOf<UAvaPlaybackEditorGraphNode> NewNodeClass, bool bSelectNewNode)
{
	return CastChecked<UAvaPlaybackEditorGraphNode>(CreateNode(NewNodeClass, false, bSelectNewNode));
}
