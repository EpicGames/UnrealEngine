// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraph.h"
#include "Templates/SubclassOf.h"
#include "AvaPlaybackEditorGraph.generated.h"

class UAvaPlaybackGraph;
class UAvaPlaybackEditorGraphNode;

UCLASS()
class UAvaPlaybackEditorGraph : public UEdGraph
{
	GENERATED_BODY()

public:
	UAvaPlaybackGraph* GetPlaybackGraph() const;
	
	UAvaPlaybackEditorGraphNode* CreatePlaybackEditorGraphNode(TSubclassOf<UAvaPlaybackEditorGraphNode> NewNodeClass
		, bool bSelectNewNode = true);
};
