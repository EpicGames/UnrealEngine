// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Graph/AvaPlaybackEditorGraphSchema.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "GraphEditorActions.h"
#include "Math/Color.h"
#include "Playback/AvaPlaybackGraph.h"
#include "Playback/Graph/AvaPlaybackEditorGraph.h"
#include "Playback/Graph/Nodes/AvaPlaybackEditorGraphNode_Root.h"
#include "Playback/Graph/SchemaActions/AvaPlaybackAction_NewComment.h"
#include "Playback/Graph/SchemaActions/AvaPlaybackAction_NewNode.h"
#include "Playback/Graph/SchemaActions/AvaPlaybackAction_PasteNode.h"
#include "Playback/IAvaPlaybackGraphEditor.h"
#include "Playback/Nodes/AvaPlaybackNodeRoot.h"
#include "ScopedTransaction.h"
#include "ToolMenu.h"
#include "UObject/NameTypes.h"
#include "UObject/NoExportTypes.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "AvaPlaybackEditorGraphSchema"

const FLinearColor UAvaPlaybackEditorGraphSchema::ActivePinColor = FLinearColor::White;
const FLinearColor UAvaPlaybackEditorGraphSchema::InactivePinColor = FLinearColor(0.05f, 0.05f, 0.05f);

// Allowable PinType.PinCategory values
const FName UAvaPlaybackEditorGraphSchema::PC_ChannelFeed(TEXT("channelfeed"));
const FName UAvaPlaybackEditorGraphSchema::PC_Event(TEXT("event"));

TArray<TSubclassOf<UAvaPlaybackNode>> UAvaPlaybackEditorGraphSchema::PlaybackNodeClasses;

TSharedPtr<IAvaPlaybackGraphEditor> UAvaPlaybackEditorGraphSchema::GetPlaybackGraphEditor(const UEdGraph* Graph) const
{
	if (Graph && Cast<UAvaPlaybackEditorGraph>(Graph))
	{
		if (UAvaPlaybackGraph* const Playback = Cast<UAvaPlaybackEditorGraph>(Graph)->GetPlaybackGraph())
		{
			return Playback->GetGraphEditor();
		}
	}
	return nullptr;
}

void UAvaPlaybackEditorGraphSchema::CompilePlaybackNodesFromGraphNodes(UEdGraphNode& Node) const
{
	CastChecked<UAvaPlaybackEditorGraph>(Node.GetGraph())->GetPlaybackGraph()->CompilePlaybackNodesFromGraphNodes();
}

bool UAvaPlaybackEditorGraphSchema::ConnectionCausesLoop(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin) const
{
	UAvaPlaybackEditorGraphNode* const InputNode = Cast<UAvaPlaybackEditorGraphNode>(InputPin->GetOwningNode());

	if (InputNode)
	{
		// Only nodes representing SoundNodes have outputs
		UAvaPlaybackEditorGraphNode* const OutputNode = CastChecked<UAvaPlaybackEditorGraphNode>(OutputPin->GetOwningNode());

		if (OutputNode->GetPlaybackNode())
		{
			// Grab all child nodes. We can't just test the output because 
			// the loop could happen from any additional child nodes. 
			TArray<UAvaPlaybackNode*> Nodes;
			OutputNode->GetPlaybackNode()->GetAllNodes(Nodes);

			// If our test input is in that set, return true.
			return Nodes.Contains(InputNode->GetPlaybackNode());
		}
	}

	// Simple connection to root node
	return false;
}

void UAvaPlaybackEditorGraphSchema::GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const
{
	GetPlaybackNodeActions(ContextMenuBuilder, true);
	GetCommentAction(ContextMenuBuilder, ContextMenuBuilder.CurrentGraph);

	bool bCanPasteNodes = false;
	if (TSharedPtr<IAvaPlaybackGraphEditor> GraphEditor = GetPlaybackGraphEditor(ContextMenuBuilder.CurrentGraph))
	{
		bCanPasteNodes = GraphEditor->CanPasteNodes();
	}
	
	if (!ContextMenuBuilder.FromPin && bCanPasteNodes)
	{
		TSharedPtr<FAvaPlaybackAction_PasteNode> NewAction = MakeShared<FAvaPlaybackAction_PasteNode>(FText::GetEmpty()
			, LOCTEXT("PasteHereAction", "Paste here")
			, FText::GetEmpty()
			, 0);
		
		ContextMenuBuilder.AddAction(NewAction);
	}
}

void UAvaPlaybackEditorGraphSchema::GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (Context->Node)
	{
		const UAvaPlaybackEditorGraphNode* const GraphNode = Cast<const UAvaPlaybackEditorGraphNode>(Context->Node);
		{
			FToolMenuSection& Section = Menu->AddSection("AvaGraphSchemaNodeActions", LOCTEXT("NodeActionsMenuHeader", "Node Actions"));
			Section.AddMenuEntry(FGraphEditorCommands::Get().BreakNodeLinks);
		}
	}
	Super::GetContextMenuActions(Menu, Context);
}

void UAvaPlaybackEditorGraphSchema::CreateDefaultNodesForGraph(UEdGraph& Graph) const
{
	UAvaPlaybackGraph* const Playback = CastChecked<UAvaPlaybackEditorGraph>(&Graph)->GetPlaybackGraph();
	UAvaPlaybackNodeRoot* const RootNode = Playback->ConstructPlaybackNode<UAvaPlaybackNodeRoot>();
	check(RootNode);
	
	SetNodeMetaData(RootNode->GetGraphNode(), FNodeMetadata::DefaultGraphNode);
}

const FPinConnectionResponse UAvaPlaybackEditorGraphSchema::CanCreateConnection(const UEdGraphPin* PinA
	, const UEdGraphPin* PinB) const
{
	// Make sure the pins are not on the same node
	if (PinA->GetOwningNode() == PinB->GetOwningNode())
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionSameNode", "Both are on the same node"));
	}

	// Pin mismatch in Pin Category
	if (PinA->PinType.PinCategory != PinB->PinType.PinCategory)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("IncompatibleCategories", "Pin Types are not Compatible"));
	}
	
	// Compare the directions
	const UEdGraphPin* InputPin = nullptr;
	const UEdGraphPin* OutputPin = nullptr;

	if (!CategorizePinsByDirection(PinA, PinB, InputPin, OutputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("IncompatibleDirections", "Directions are not compatible"));
	}

	if (ConnectionCausesLoop(InputPin, OutputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, LOCTEXT("ConnectionLoop", "Connection would cause loop"));
	}

	// Break existing connections on inputs only for Channel Feed only
	// multiple Output connections are acceptable
	if (InputPin->LinkedTo.Num() > 0)
	{
		ECanCreateConnectionResponse ReplyBreakOutputs;
		if (InputPin == PinA)
		{
			ReplyBreakOutputs = CONNECT_RESPONSE_BREAK_OTHERS_A;
		}
		else
		{
			ReplyBreakOutputs = CONNECT_RESPONSE_BREAK_OTHERS_B;
		}
		return FPinConnectionResponse(ReplyBreakOutputs, LOCTEXT("ConnectionReplace", "Replace existing connections"));
	}

	return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT(""));
}

bool UAvaPlaybackEditorGraphSchema::TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
	if (UEdGraphSchema::TryCreateConnection(PinA, PinB))
	{
		CompilePlaybackNodesFromGraphNodes(*PinA->GetOwningNode());
		return true;
	}
	return false;
}

FLinearColor UAvaPlaybackEditorGraphSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	if (PinType.PinCategory == PC_ChannelFeed)
	{
		return FLinearColor::White;
	}
	
	if (PinType.PinCategory == PC_Event)
	{
		return FLinearColor::White;
	}

	return Super::GetPinTypeColor(PinType);
}

bool UAvaPlaybackEditorGraphSchema::ShouldHidePinDefaultValue(UEdGraphPin* Pin) const
{
	return Super::ShouldHidePinDefaultValue(Pin);
}

void UAvaPlaybackEditorGraphSchema::BreakNodeLinks(UEdGraphNode& TargetNode) const
{
	Super::BreakNodeLinks(TargetNode);
	CompilePlaybackNodesFromGraphNodes(TargetNode);
}

void UAvaPlaybackEditorGraphSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const
{
	const FScopedTransaction Transaction(LOCTEXT("PlaybackGraph_BreakPinLinks", "Break Pin Links"));
	Super::BreakPinLinks(TargetPin, bSendsNodeNotifcation);

	// if this would notify the node then we need to compile the SoundCue
	if (bSendsNodeNotifcation)
	{
		CompilePlaybackNodesFromGraphNodes(*TargetPin.GetOwningNode());
	}
}

int32 UAvaPlaybackEditorGraphSchema::GetNodeSelectionCount(const UEdGraph* Graph) const
{
	return Super::GetNodeSelectionCount(Graph);
}

TSharedPtr<FEdGraphSchemaAction> UAvaPlaybackEditorGraphSchema::GetCreateCommentAction() const
{
	return MakeShared<FAvaPlaybackAction_NewComment>();
}

void UAvaPlaybackEditorGraphSchema::CachePlaybackNodeClasses()
{
	if (PlaybackNodeClasses.IsEmpty())
	{
		// Construct list of non-abstract sound node classes.
		for (UClass* Class : TObjectRange<UClass>())
		{
			if (Class->IsChildOf(UAvaPlaybackNode::StaticClass()) && !Class->HasAnyClassFlags(CLASS_Abstract))
			{
				PlaybackNodeClasses.Add(Class);
			}
		}
	}
}

void UAvaPlaybackEditorGraphSchema::GetPlaybackNodeActions(FGraphActionMenuBuilder& ActionMenuBuilder, bool bShowSelectedActions) const
{
	CachePlaybackNodeClasses();
	
	for (TSubclassOf<UAvaPlaybackNode> PlaybackNodeClass : PlaybackNodeClasses)
	{		
		UAvaPlaybackNode* PlaybackNode = PlaybackNodeClass->GetDefaultObject<UAvaPlaybackNode>();
		
		if (!ActionMenuBuilder.FromPin
			|| ActionMenuBuilder.FromPin->Direction == EGPD_Input
			|| PlaybackNode->GetMaxChildNodes() > 0)
		{
			const FText NodeTitle = PlaybackNode->GetNodeDisplayNameText();
			const FText NodeCategory = PlaybackNode->GetNodeCategoryText();
			
			//New Node Action (for anything other than Playback Node Root)
			if (!PlaybackNodeClass->IsChildOf(UAvaPlaybackNodeRoot::StaticClass()))
			{				
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Desc"), NodeTitle);
				
				TSharedPtr<FAvaPlaybackAction_NewNode> NewNodeAction = MakeShared<FAvaPlaybackAction_NewNode>(NodeCategory
					, NodeTitle
					, FText::Format(LOCTEXT("NewPlaybackNodeTooltip", "Adds {Desc} node here"), Arguments)
					, 0);
				
				ActionMenuBuilder.AddAction(NewNodeAction);
				NewNodeAction->SetPlaybackNodeClass(PlaybackNodeClass);
			}
		}
	}
}

void UAvaPlaybackEditorGraphSchema::GetCommentAction(FGraphActionMenuBuilder& ActionMenuBuilder, const UEdGraph* CurrentGraph) const
{
	if (!ActionMenuBuilder.FromPin)
	{
		TSharedPtr<FAvaPlaybackAction_NewComment> NewAction = MakeShared<FAvaPlaybackAction_NewComment>(FText::GetEmpty()
			, LOCTEXT("AddCommentAction", "Add Comment...")
			, LOCTEXT("CreateCommentToolTip", "Creates a comment.")
			, 0);
		
		ActionMenuBuilder.AddAction(NewAction);
	}
}

#undef LOCTEXT_NAMESPACE 
