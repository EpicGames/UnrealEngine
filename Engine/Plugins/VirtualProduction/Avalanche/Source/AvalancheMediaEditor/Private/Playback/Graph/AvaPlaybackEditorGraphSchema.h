// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/StringFwd.h"
#include "EdGraph/EdGraphSchema.h"
#include "Math/MathFwd.h"
#include "Templates/SubclassOf.h"
#include "Templates/SharedPointer.h"
#include "AvaPlaybackEditorGraphSchema.generated.h"

class IAvaPlaybackGraphEditor;
class FName;
class UAvaPlaybackGraph;
class UAvaPlaybackNode;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
class UGraphNodeContextMenuContext;
class UToolMenu;
struct FAssetData;
struct FEdGraphPinType;
struct FGraphContextMenuBuilder;
struct FLinearColor;
struct FPinConnectionResponse;

UCLASS()
class UAvaPlaybackEditorGraphSchema : public UEdGraphSchema
{
	GENERATED_BODY()

public:
	static const FLinearColor ActivePinColor;
	static const FLinearColor InactivePinColor;
	
	// Allowable PinType.PinCategory values
	static const FName PC_ChannelFeed;
	static const FName PC_Event;

	void CompilePlaybackNodesFromGraphNodes(UEdGraphNode& Node) const;
	
	/** Check whether connecting these pins would cause a loop */
	bool ConnectionCausesLoop(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin) const;
	
	//~ Begin UEdGraphSchema
	virtual void GetGraphContextActions(FGraphContextMenuBuilder& ContextMenuBuilder) const override;
	virtual void GetContextMenuActions(UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual void CreateDefaultNodesForGraph(UEdGraph& Graph) const override;
	virtual bool CanDuplicateGraph(UEdGraph* InSourceGraph) const override { return false; }
	virtual bool ShouldAlwaysPurgeOnModification() const override { return true; }
	virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const override;
	virtual bool TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const override;
	virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	virtual bool ShouldHidePinDefaultValue(UEdGraphPin* Pin) const override;
	virtual void BreakNodeLinks(UEdGraphNode& TargetNode) const override;
	virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const override;
	virtual int32 GetNodeSelectionCount(const UEdGraph* Graph) const override;
	virtual TSharedPtr<FEdGraphSchemaAction> GetCreateCommentAction() const override;
	//~ End UEdGraphSchema

	static void CachePlaybackNodeClasses();
	
	/** Adds actions for creating every type of Playback Node */
	void GetPlaybackNodeActions(FGraphActionMenuBuilder& ActionMenuBuilder, bool bShowSelectedActions) const;
	void GetCommentAction(FGraphActionMenuBuilder& ActionMenuBuilder, const UEdGraph* CurrentGraph = nullptr) const;

protected:
	static TArray<TSubclassOf<UAvaPlaybackNode>> PlaybackNodeClasses;

	TSharedPtr<IAvaPlaybackGraphEditor> GetPlaybackGraphEditor(const UEdGraph* Graph) const;
};
