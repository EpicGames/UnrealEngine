// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Graph/Nodes/AvaPlaybackEditorGraphNode.h"
#include "AvaMediaEditorSettings.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "Framework/Commands/GenericCommands.h"
#include "GraphEditorActions.h"
#include "Math/Color.h"
#include "Playback/AvaPlaybackCommands.h"
#include "Playback/AvaPlaybackGraph.h"
#include "Playback/Graph/AvaPlaybackEditorGraph.h"
#include "Playback/Graph/AvaPlaybackEditorGraphSchema.h"
#include "Playback/Graph/Nodes/AvaPlaybackEditorGraphNode_Root.h"
#include "Playback/Graph/Nodes/Slate/SAvaPlaybackEditorGraphNode.h"
#include "Playback/Nodes/AvaPlaybackNode.h"
#include "ScopedTransaction.h"
#include "SGraphNode.h"
#include "ToolMenu.h"
#include "UObject/NameTypes.h"
#include "UObject/NoExportTypes.h"

#define LOCTEXT_NAMESPACE "AvaPlaybackEditorGraphNode"

UAvaPlaybackGraph* UAvaPlaybackEditorGraphNode::GetPlayback() const
{
	return CastChecked<UAvaPlaybackEditorGraph>(GetGraph())->GetPlaybackGraph();
}

void UAvaPlaybackEditorGraphNode::SetPlaybackNode(UAvaPlaybackNode* InPlaybackNode)
{
	PlaybackNode = InPlaybackNode;
	PlaybackNode->SetGraphNode(this);
}

double UAvaPlaybackEditorGraphNode::GetLastTimeTicked()
{
	return PlaybackNode ? PlaybackNode->GetLastTimeTicked() : -1.0;
}

double UAvaPlaybackEditorGraphNode::GetChildLastTimeTicked(int32 ChildIndex) const
{
	return PlaybackNode ? PlaybackNode->GetChildLastTimeTicked(ChildIndex) : -1.0;
}

bool UAvaPlaybackEditorGraphNode::CanAddInputPin() const
{
	if (PlaybackNode)
	{
		// Check if adding another input would exceed max child nodes.
		return PlaybackNode->GetChildNodes().Num() < PlaybackNode->GetMaxChildNodes();
	}
	else
	{
		return false;
	}
}

void UAvaPlaybackEditorGraphNode::AddInputPin()
{
	const FScopedTransaction Transaction(LOCTEXT("PlaybackGraphNodeAddInput", "Add Playback Input"));
	Modify();
	CreateInputPin();

	UAvaPlaybackGraph* const Playback = GetPlayback();
	Playback->CompilePlaybackNodesFromGraphNodes();
	Playback->MarkPackageDirty();

	// Refresh the current graph, so the pins can be updated
	GetGraph()->NotifyGraphChanged();
}

void UAvaPlaybackEditorGraphNode::CreateInputPin()
{
	const int32 InputIndex = GetInputPinCount();
	
	UEdGraphPin* const Pin = CreatePin(EGPD_Input
		, GetInputPinCategory(InputIndex)
		, GetInputPinSubCategory(InputIndex)
		, GetInputPinSubCategoryObject(InputIndex)
		, GetInputPinName(InputIndex));

	check(Pin);
	
	if (Pin->PinName.IsNone())
	{
		// Makes sure pin has a name for lookup purposes but user will never see it
		Pin->PinName = CreateUniquePinName(TEXT("Input"));
		Pin->PinFriendlyName = FText::FromString(TEXT(" "));
	}
}

void UAvaPlaybackEditorGraphNode::RemoveInputPin(UEdGraphPin* InGraphPin)
{
	const FScopedTransaction Transaction(LOCTEXT("PlaybackGraphNodeRemoveInput", "Remove Playback Input") );
	Modify();
	
	const TArray<UEdGraphPin*> InputPins = GetInputPins();
	
	for (int32 InputIndex = 0; InputIndex < InputPins.Num(); InputIndex++)
	{
		if (InGraphPin == InputPins[InputIndex])
		{
			InGraphPin->MarkAsGarbage();
			Pins.Remove(InGraphPin);
			
			// also remove the PlaybackNode child node so ordering matches
			PlaybackNode->Modify();
			PlaybackNode->RemoveChildNode(InputIndex);
			break;
		}
	}

	UAvaPlaybackGraph* const Playback = GetPlayback();
	Playback->CompilePlaybackNodesFromGraphNodes();
	Playback->MarkPackageDirty();

	// Refresh the current graph, so the pins can be updated
	GetGraph()->NotifyGraphChanged();
}

void UAvaPlaybackEditorGraphNode::PrepareForCopying()
{
	if (PlaybackNode)
	{
		// Temporarily take ownership of the PlaybackNode, so that it is not deleted when cutting
		PlaybackNode->Rename(nullptr, this, REN_DontCreateRedirectors);
	}
}

void UAvaPlaybackEditorGraphNode::PostPasteNode()
{
	if (PlaybackNode)
	{
		PlaybackNode->PostAllocateNode();
	}
}

void UAvaPlaybackEditorGraphNode::PostCopyNode()
{
	ResetPlaybackNodeOwner();
}

void UAvaPlaybackEditorGraphNode::ResetPlaybackNodeOwner()
{
	if (PlaybackNode)
	{
		UAvaPlaybackGraph* const Playback = GetPlayback();
		if (PlaybackNode->GetOuter() != Playback)
		{
			// Ensures PlaybackNode is owned by the Playback Object
			PlaybackNode->Rename(nullptr, Playback, REN_DontCreateRedirectors);
		}

		// Set up the back pointer for newly created sound nodes
		PlaybackNode->SetGraphNode(this);
	}
}

TSubclassOf<UAvaPlaybackNode> UAvaPlaybackEditorGraphNode::GetPlaybackNodeClass() const
{
	return UAvaPlaybackNode::StaticClass();
}

void UAvaPlaybackEditorGraphNode::CreateInputPins()
{
	if (PlaybackNode)
	{
		for (int32 ChildIndex = 0; ChildIndex < PlaybackNode->GetChildNodes().Num(); ++ChildIndex)
		{
			CreateInputPin();
		}
	}
}

void UAvaPlaybackEditorGraphNode::CreateOutputPin()
{
	if (!IsRootNode())
	{
		UEdGraphPin* const Pin = CreatePin(EGPD_Output
			, GetOutputPinCategory()
			, NAME_None);
		
		if (Pin->PinName.IsNone())
		{
			// Makes sure pin has a name for lookup purposes but user will never see it
			Pin->PinName = CreateUniquePinName(TEXT("Output"));
			Pin->PinFriendlyName = FText::FromString(TEXT(" "));;
		}
	}
}

bool UAvaPlaybackEditorGraphNode::IsRootNode() const
{
	return !!Cast<UAvaPlaybackEditorGraphNode_Root>(this);
}

UEdGraphPin* UAvaPlaybackEditorGraphNode::GetOutputPin() const
{
	for (UEdGraphPin* const Pin : Pins)
	{
		if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
		{
			return Pin;
		}
	}
	return nullptr;
}

TArray<UEdGraphPin*> UAvaPlaybackEditorGraphNode::GetInputPins() const
{
	TArray<UEdGraphPin*> InputPins;
	
	//Reserve with the Expected Number of Inputs (since we assume only 1 output pin, reserve PinNum - 1)
	InputPins.Reserve(FMath::Max(0, Pins.Num() - 1));
	
	for (UEdGraphPin* const Pin : Pins)
	{
		if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
		{
			InputPins.Add(Pin);
		}
	}

	return InputPins;
}

UEdGraphPin* UAvaPlaybackEditorGraphNode::GetInputPin(int32 InputIndex)  const
{
	TArray<UEdGraphPin*> InputPins = GetInputPins();
	
	if (InputPins.IsValidIndex(InputIndex))
	{
		return InputPins[InputIndex];
	}
	
	checkf(0, TEXT("PlaybackGraphNode::GetInputPin. Index out of Bounds"));
	return nullptr;
}

int32 UAvaPlaybackEditorGraphNode::GetInputPinIndex(UEdGraphPin* InInputPin) const
{
	if (InInputPin && InInputPin->Direction == EGPD_Input)
	{
		int32 Index = 0;
		for (const UEdGraphPin* const Pin : Pins)
		{
			if (Pin && Pin->Direction == EGPD_Input)
			{
				if (Pin == InInputPin)
				{
					return Index;
				}
				++Index;
			}
		}
	}
	return INDEX_NONE;
}

int32 UAvaPlaybackEditorGraphNode::GetInputPinCount() const
{
	return GetInputPins().Num();
}

void UAvaPlaybackEditorGraphNode::InsertNewNode(UEdGraphPin* FromPin, UEdGraphPin* NewLinkPin, TSet<UEdGraphNode*>& OutNodeList)
{
	const UAvaPlaybackEditorGraphSchema* const Schema = CastChecked<UAvaPlaybackEditorGraphSchema>(GetSchema());

	// The pin we are creating from already has a connection that needs to be broken. We want to "insert" the new node in between, so that the output of the new node is hooked up too
	UEdGraphPin* OldLinkedPin = FromPin->LinkedTo[0];
	check(OldLinkedPin);

	FromPin->BreakAllPinLinks();

	// Hook up the old linked pin to the first valid output pin on the new node
	for (int32 OutputPinIndex=0; OutputPinIndex < Pins.Num(); OutputPinIndex++)
	{
		UEdGraphPin* OutputPin = Pins[OutputPinIndex];
		check(OutputPin);
		
		if (Schema->CanCreateConnection(OldLinkedPin, OutputPin).Response == ECanCreateConnectionResponse::CONNECT_RESPONSE_MAKE)
		{
			if (Schema->TryCreateConnection(OldLinkedPin, OutputPin))
			{
				OutNodeList.Add(OldLinkedPin->GetOwningNode());
				OutNodeList.Add(this);
			}
			break;
		}
	}

	if (Schema->TryCreateConnection(FromPin, NewLinkPin))
	{
		OutNodeList.Add(FromPin->GetOwningNode());
		OutNodeList.Add(this);
	}
}

void UAvaPlaybackEditorGraphNode::PostLoad()
{
	Super::PostLoad();

	// Fixup any Playback Node back pointers that may be out of date
	if (PlaybackNode)
	{
		PlaybackNode->SetGraphNode(this);
	}

	for (UEdGraphPin* const Pin : Pins)
	{
		if (Pin && Pin->PinName.IsNone())
		{
			// Makes sure pin has a name for lookup purposes but user will never see it
			if (Pin->Direction == EGPD_Input)
			{
				Pin->PinName = CreateUniquePinName(TEXT("Input"));
			}
			else
			{
				Pin->PinName = CreateUniquePinName(TEXT("Output"));
			}
			Pin->PinFriendlyName = FText::FromString(TEXT(" "));
		}
	}
}

void UAvaPlaybackEditorGraphNode::PostEditImport()
{
	Super::PostEditImport();
	ResetPlaybackNodeOwner();
}

void UAvaPlaybackEditorGraphNode::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
	if (!bDuplicateForPIE)
	{
		CreateNewGuid();
	}
}

FText UAvaPlaybackEditorGraphNode::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (PlaybackNode)
	{
		return PlaybackNode->GetNodeDisplayNameText();
	}
	return Super::GetNodeTitle(TitleType);
}

FText UAvaPlaybackEditorGraphNode::GetTooltipText() const
{
	if (PlaybackNode)
	{
		return PlaybackNode->GetNodeTooltipText();
	}
	return Super::GetTooltipText();
}

void UAvaPlaybackEditorGraphNode::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if (Context->Pin)
	{
		// If on an input that can be deleted, show option
		if (Context->Pin->Direction == EGPD_Input && PlaybackNode->GetChildNodes().Num() > PlaybackNode->GetMinChildNodes())
		{
			FToolMenuSection& Section = Menu->AddSection("PlaybackGraphRemovePin"
				, LOCTEXT("PlaybackGraphRemovePin", "Input Pin Actions"));
			Section.AddMenuEntry(FAvaPlaybackCommands::Get().AddInputPin);
			Section.AddMenuEntry(FAvaPlaybackCommands::Get().RemoveInputPin);
		}
	}
	else if (Context->Node)
	{
		{
			FToolMenuSection& Section = Menu->AddSection("PlaybackGraphNodeEdit");
			Section.AddMenuEntry(FGenericCommands::Get().Delete);
			Section.AddMenuEntry(FGenericCommands::Get().Cut);
			Section.AddMenuEntry(FGenericCommands::Get().Copy);
			Section.AddMenuEntry(FGenericCommands::Get().Duplicate);
			Section.AddSeparator(NAME_None);
		}
		
		{
			FToolMenuSection& Section = Menu->AddSection("PlaybackGraphNodeOrganization", LOCTEXT("PlaybackGraphNodeOrganization", "Organization"));
			
			Section.AddSubMenu("Alignment"
				, LOCTEXT("AlignmentHeader", "Alignment")
				, FText::GetEmpty()
				, FNewToolMenuDelegate::CreateLambda([](UToolMenu* SubMenu)
				{
					{
						FToolMenuSection& SubMenuSection = SubMenu->AddSection("PlaybackGraphSchemaAlignment", LOCTEXT("AlignHeader", "Align"));
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesTop);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesMiddle);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesBottom);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesLeft);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesCenter);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesRight);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().StraightenConnections);
					}
	
					{
						FToolMenuSection& SubMenuSection = SubMenu->AddSection("PlaybackGraphSchemaDistribution", LOCTEXT("DistributionHeader", "Distribution"));
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesHorizontally);
						SubMenuSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesVertically);
					}
				}));
			
			Section.AddSeparator(NAME_None);
		}
	}
}

void UAvaPlaybackEditorGraphNode::AllocateDefaultPins()
{
	check(Pins.Num() == 0);
	CreateInputPins();
	CreateOutputPin();
}

void UAvaPlaybackEditorGraphNode::ReconstructNode()
{
	// Break any links to 'orphan' pins
	for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
	{
		UEdGraphPin* Pin = Pins[PinIndex];
		TArray<class UEdGraphPin*>& LinkedToRef = Pin->LinkedTo;
		for (int32 LinkIdx=0; LinkIdx < LinkedToRef.Num(); LinkIdx++)
		{
			UEdGraphPin* OtherPin = LinkedToRef[LinkIdx];
			// If we are linked to a pin that its owner doesn't know about, break that link
			if (!OtherPin->GetOwningNode()->Pins.Contains(OtherPin))
			{
				Pin->LinkedTo.Remove(OtherPin);
			}
		}
	}

	// Store the old Input and Output pins
	TArray<UEdGraphPin*> OldInputPins = GetInputPins();
	UEdGraphPin* OldOutputPin = GetOutputPin();
	
	// Move the existing pins to a saved array
	TArray OldPins(Pins);
	Pins.Reset();

	// Recreate the new pins
	AllocateDefaultPins();

	// Get new Input and Output pins
	TArray<UEdGraphPin*> NewInputPins = GetInputPins();
	UEdGraphPin* NewOutputPin = GetOutputPin();

	for (int32 PinIndex = 0; PinIndex < OldInputPins.Num(); ++PinIndex)
	{
		if (PinIndex < NewInputPins.Num())
		{
			NewInputPins[PinIndex]->MovePersistentDataFromOldPin(*OldInputPins[PinIndex]);
		}
	}
	
	if (OldOutputPin)
	{
		NewOutputPin->MovePersistentDataFromOldPin(*OldOutputPin);
	}

	// Throw away the original pins
	for (UEdGraphPin* OldPin : OldPins)
	{
		OldPin->Modify();
		UEdGraphNode::DestroyPin(OldPin);
	}
}

void UAvaPlaybackEditorGraphNode::AutowireNewNode(UEdGraphPin* FromPin)
{
	if (FromPin)
	{
		const UAvaPlaybackEditorGraphSchema* const Schema = CastChecked<UAvaPlaybackEditorGraphSchema>(GetSchema());
		TSet<UEdGraphNode*> NodeList;

		// auto-connect from dragged pin to first compatible pin on the new node
		for (int32 Index = 0; Index < Pins.Num(); ++Index)
		{
			UEdGraphPin* Pin = Pins[Index];
			check(Pin);
			
			FPinConnectionResponse Response = Schema->CanCreateConnection(FromPin, Pin);
			
			if (Response.Response == ECanCreateConnectionResponse::CONNECT_RESPONSE_MAKE)
			{
				if (Schema->TryCreateConnection(FromPin, Pin))
				{
					NodeList.Add(FromPin->GetOwningNode());
					NodeList.Add(this);
				}
				break;
			}
			
			if (Response.Response == ECanCreateConnectionResponse::CONNECT_RESPONSE_BREAK_OTHERS_A)
			{
				InsertNewNode(FromPin, Pin, NodeList);
				break;
			}
		}

		// Send all nodes that received a new pin connection a notification
		for (TSet<UEdGraphNode*>::TConstIterator It = NodeList.CreateConstIterator(); It; ++It)
		{
			UEdGraphNode* Node = *It;
			Node->NodeConnectionListChanged();
		}
	}
}

bool UAvaPlaybackEditorGraphNode::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const
{
	return Schema->IsA(UAvaPlaybackEditorGraphSchema::StaticClass());
}

FLinearColor UAvaPlaybackEditorGraphNode::GetNodeTitleColor() const
{
	return UAvaMediaEditorSettings::Get().PlaybackDefaultNodeColor;
}

TSharedPtr<SGraphNode> UAvaPlaybackEditorGraphNode::CreateVisualWidget()
{
	return SNew(SAvaPlaybackEditorGraphNode, this);
}

FName UAvaPlaybackEditorGraphNode::GetInputPinName(int32 InputPinIndex) const
{
	if (PlaybackNode)
	{
		return PlaybackNode->GetInputPinName(InputPinIndex);
	}
	return NAME_None;
}

FName UAvaPlaybackEditorGraphNode::GetInputPinCategory(int32 InputPinIndex) const
{
	return UAvaPlaybackEditorGraphSchema::PC_ChannelFeed;
}

FName UAvaPlaybackEditorGraphNode::GetInputPinSubCategory(int32 InputPinIndex) const
{
	return NAME_None;
}

UObject* UAvaPlaybackEditorGraphNode::GetInputPinSubCategoryObject(int32 InputPinIndex) const
{
	return nullptr;
}

FName UAvaPlaybackEditorGraphNode::GetOutputPinCategory() const
{
	return UAvaPlaybackEditorGraphSchema::PC_ChannelFeed;
}

#undef LOCTEXT_NAMESPACE
