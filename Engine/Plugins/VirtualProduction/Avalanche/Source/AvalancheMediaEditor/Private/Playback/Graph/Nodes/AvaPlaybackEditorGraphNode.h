// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "EdGraph/EdGraphNode.h"
#include "Templates/SubclassOf.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"
#include "AvaPlaybackEditorGraphNode.generated.h"

class FName;
class SGraphNode;
class UAvaPlaybackGraph;
class UAvaPlaybackNode;
class UEdGraphPin;
class UEdGraphSchema;
class UGraphNodeContextMenuContext;
class UToolMenu;
class UObject;
struct FLinearColor;

UCLASS()
class UAvaPlaybackEditorGraphNode : public UEdGraphNode
{
	GENERATED_BODY()

public:
	UAvaPlaybackGraph* GetPlayback() const;
	
	void SetPlaybackNode(UAvaPlaybackNode* InPlaybackNode);

	double GetLastTimeTicked();
	double GetChildLastTimeTicked(int32 ChildIndex) const;
	
	bool CanAddInputPin() const;
	void AddInputPin();
	void CreateInputPin();

	void RemoveInputPin(UEdGraphPin* InGraphPin);

	//~ Begin UEdGraphNode interface
	virtual void PrepareForCopying() override;
	virtual void PostPasteNode() override;
	//~ End UEdGraphNode interface

	void PostCopyNode();

	/** Make sure the PlaybackNode is owned by the Playback Object */
	void ResetPlaybackNodeOwner();
	
	//Return which Playback Node this Graph Node should Represent
	virtual TSubclassOf<UAvaPlaybackNode> GetPlaybackNodeClass() const;

	UAvaPlaybackNode* GetPlaybackNode() const
	{
		return PlaybackNode;
	}
	
protected:
	/** Create all of the input pins required */
	void CreateInputPins();
	void CreateOutputPin();

public:
	/** Is this the non-deletable root node */
	bool IsRootNode() const;
	
	UEdGraphPin* GetOutputPin() const;
	TArray<UEdGraphPin*> GetInputPins() const;
	
	UEdGraphPin* GetInputPin(int32 InputIndex) const;
	int32 GetInputPinIndex(UEdGraphPin* InInputPin) const;
	int32 GetInputPinCount() const;

	/**
	 * Handles inserting the node between the FromPin and what the FromPin was original connected to
	 *
	 * @param FromPin			The pin this node is being spawned from
	 * @param NewLinkPin		The new pin the FromPin will connect to
	 * @param OutNodeList		Any nodes that are modified will get added to this list for notification purposes
	 */
	void InsertNewNode(UEdGraphPin* FromPin, UEdGraphPin* NewLinkPin, TSet<UEdGraphNode*>& OutNodeList);

	//~ Begin UObject
	virtual void PostLoad() override;
	virtual void PostEditImport() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	//~ End UObject
	
	//~ Begin UEdGraphNode
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual void GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const override;
	virtual void AllocateDefaultPins() override;
	virtual void ReconstructNode() override;
	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* Schema) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	//~ End UEdGraphNode
	
	virtual FName GetInputPinName(int32 InputPinIndex) const;
	virtual FName GetInputPinCategory(int32 InputPinIndex) const;
	virtual FName GetInputPinSubCategory(int32 InputPinIndex) const;
	virtual UObject* GetInputPinSubCategoryObject(int32 InputPinIndex) const;

	virtual FName GetOutputPinCategory() const;
	
protected:
	// The Playback Node this represents
	UPROPERTY(VisibleAnywhere, Instanced, Category=Playback)
	TObjectPtr<UAvaPlaybackNode> PlaybackNode;
};
