// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Graph/Nodes/AvaPlaybackEditorGraphNode_EventAction.h"

#include "AvaMediaEditorSettings.h"
#include "Playback/Graph/AvaPlaybackEditorGraphSchema.h"
#include "Playback/Nodes/Events/AvaPlaybackNodeAction.h"

TSubclassOf<UAvaPlaybackNode> UAvaPlaybackEditorGraphNode_EventAction::GetPlaybackNodeClass() const
{
	return UAvaPlaybackNodeAction::StaticClass();
}

FName UAvaPlaybackEditorGraphNode_EventAction::GetInputPinCategory(int32 InputPinIndex) const
{
	return UAvaPlaybackEditorGraphSchema::PC_Event;
}

FName UAvaPlaybackEditorGraphNode_EventAction::GetOutputPinCategory() const
{
	return UAvaPlaybackEditorGraphSchema::PC_Event;
}

FLinearColor UAvaPlaybackEditorGraphNode_EventAction::GetNodeTitleColor() const
{
	return UAvaMediaEditorSettings::Get().PlaybackActionNodeColor;
}
