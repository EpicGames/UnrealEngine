// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Graph/Nodes/AvaPlaybackEditorGraphNode_EventFlow.h"

#include "Playback/Graph/AvaPlaybackEditorGraphSchema.h"
#include "Playback/Nodes/Events/AvaPlaybackNodeFlow.h"

TSubclassOf<UAvaPlaybackNode> UAvaPlaybackEditorGraphNode_EventFlow::GetPlaybackNodeClass() const
{
	return UAvaPlaybackNodeFlow::StaticClass();
}

FName UAvaPlaybackEditorGraphNode_EventFlow::GetInputPinCategory(int32 InputPinIndex) const
{
	return UAvaPlaybackEditorGraphSchema::PC_Event;
}

FName UAvaPlaybackEditorGraphNode_EventFlow::GetOutputPinCategory() const
{
	return UAvaPlaybackEditorGraphSchema::PC_Event;
}
