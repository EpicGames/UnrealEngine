// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Graph/Nodes/AvaPlaybackEditorGraphNode_EventTrigger.h"

#include "AvaMediaEditorSettings.h"
#include "Playback/Graph/AvaPlaybackEditorGraphSchema.h"
#include "Playback/Nodes/Events/AvaPlaybackNodeTrigger.h"

TSubclassOf<UAvaPlaybackNode> UAvaPlaybackEditorGraphNode_EventTrigger::GetPlaybackNodeClass() const
{
	return UAvaPlaybackNodeTrigger::StaticClass();
}

FName UAvaPlaybackEditorGraphNode_EventTrigger::GetOutputPinCategory() const
{
	return UAvaPlaybackEditorGraphSchema::PC_Event;
}

FLinearColor UAvaPlaybackEditorGraphNode_EventTrigger::GetNodeTitleColor() const
{
	return UAvaMediaEditorSettings::Get().PlaybackEventNodeColor;
}
