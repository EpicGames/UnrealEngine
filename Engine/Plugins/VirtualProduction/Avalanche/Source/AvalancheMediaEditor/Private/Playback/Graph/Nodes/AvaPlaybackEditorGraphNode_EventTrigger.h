// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaPlaybackEditorGraphNode.h"
#include "AvaPlaybackEditorGraphNode_EventTrigger.generated.h"

UCLASS()
class UAvaPlaybackEditorGraphNode_EventTrigger : public UAvaPlaybackEditorGraphNode
{
	GENERATED_BODY()

public:
	
	virtual TSubclassOf<UAvaPlaybackNode> GetPlaybackNodeClass() const override;
	virtual FName GetOutputPinCategory() const override;

	//UEdGraphNode interface
	virtual FLinearColor GetNodeTitleColor() const override;
	//~UEdGraphNode interface
};
