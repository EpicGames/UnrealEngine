// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Graph/Nodes/AvaPlaybackEditorGraphNode_LevelPlayer.h"

#include "AvaMediaEditorSettings.h"
#include "Playback/Graph/AvaPlaybackEditorGraphSchema.h"
#include "Playback/Nodes/AvaPlaybackNodeLevelPlayer.h"
#include "Slate/SAvaPlaybackEditorGraphNode_Player.h"

TSubclassOf<UAvaPlaybackNode> UAvaPlaybackEditorGraphNode_LevelPlayer::GetPlaybackNodeClass() const
{
	return UAvaPlaybackNodeLevelPlayer::StaticClass();
}

FName UAvaPlaybackEditorGraphNode_LevelPlayer::GetInputPinCategory(int32 InputPinIndex) const
{
	return UAvaPlaybackEditorGraphSchema::PC_Event;
}

FLinearColor UAvaPlaybackEditorGraphNode_LevelPlayer::GetNodeTitleColor() const
{
	return UAvaMediaEditorSettings::Get().PlaybackPlayerNodeColor;
}

TSharedPtr<SGraphNode> UAvaPlaybackEditorGraphNode_LevelPlayer::CreateVisualWidget()
{
	return SNew(SAvaPlaybackEditorGraphNode_Player, this);
}
