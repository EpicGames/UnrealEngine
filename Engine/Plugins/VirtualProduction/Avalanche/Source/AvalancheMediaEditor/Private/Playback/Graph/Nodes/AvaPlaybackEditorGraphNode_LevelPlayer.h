// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaPlaybackEditorGraphNode.h"
#include "AvaPlaybackEditorGraphNode_LevelPlayer.generated.h"

UCLASS()
class UAvaPlaybackEditorGraphNode_LevelPlayer : public UAvaPlaybackEditorGraphNode
{
	GENERATED_BODY()
	
	virtual TSubclassOf<UAvaPlaybackNode> GetPlaybackNodeClass() const override;
	virtual FName GetInputPinCategory(int32 InputPinIndex) const override;
	
	//UEdGraphNode interface
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	//~UEdGraphNode interface
};
