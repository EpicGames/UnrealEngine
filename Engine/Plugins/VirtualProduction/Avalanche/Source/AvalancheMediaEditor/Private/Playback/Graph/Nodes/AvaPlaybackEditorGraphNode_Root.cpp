// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Graph/Nodes/AvaPlaybackEditorGraphNode_Root.h"

#include "AvaMediaEditorSettings.h"
#include "Playback/Nodes/AvaPlaybackNodeRoot.h"
#include "Slate/SAvaPlaybackEditorGraphNode_Root.h"

#define LOCTEXT_NAMESPACE "AvaPlaybackEditorGraphNode_Root"

TSubclassOf<UAvaPlaybackNode> UAvaPlaybackEditorGraphNode_Root::GetPlaybackNodeClass() const
{
	return UAvaPlaybackNodeRoot::StaticClass();
}

FLinearColor UAvaPlaybackEditorGraphNode_Root::GetNodeTitleColor() const
{
	return UAvaMediaEditorSettings::Get().PlaybackChannelsNodeColor;
}

bool UAvaPlaybackEditorGraphNode_Root::CanUserDeleteNode() const
{
	return false;
}

bool UAvaPlaybackEditorGraphNode_Root::CanDuplicateNode() const
{
	return false;
}

TSharedPtr<SGraphNode> UAvaPlaybackEditorGraphNode_Root::CreateVisualWidget()
{
	return SNew(SAvaPlaybackEditorGraphNode_Root, this);
}

#undef LOCTEXT_NAMESPACE
