// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaPlaybackEditorGraphNode.h"
#include "AvaPlaybackEditorGraphNode_Root.generated.h"

UCLASS()
class UAvaPlaybackEditorGraphNode_Root : public UAvaPlaybackEditorGraphNode
{
	GENERATED_BODY()
	
	virtual TSubclassOf<UAvaPlaybackNode> GetPlaybackNodeClass() const override;

	//UEdGraphNode interface
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual bool CanUserDeleteNode() const override;
	virtual bool CanDuplicateNode() const override;
	virtual TSharedPtr<SGraphNode> CreateVisualWidget() override;
	//~UEdGraphNode interface
};
