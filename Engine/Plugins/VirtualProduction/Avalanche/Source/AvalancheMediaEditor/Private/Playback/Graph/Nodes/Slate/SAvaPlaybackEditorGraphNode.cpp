// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Graph/Nodes/Slate/SAvaPlaybackEditorGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "GraphEditorSettings.h"
#include "Input/Reply.h"
#include "KismetPins/SGraphPinExec.h"
#include "Layout/Visibility.h"
#include "Playback/Graph/AvaPlaybackEditorGraphSchema.h"
#include "Playback/Graph/Nodes/AvaPlaybackEditorGraphNode.h"
#include "SGraphPin.h"
#include "Widgets/SBoxPanel.h"
#include "Layout/Margin.h"

#define LOCTEXT_NAMESPACE "SAvaPlaybackEditorGraphNode"

void SAvaPlaybackEditorGraphNode::Construct(const FArguments& InArgs, UAvaPlaybackEditorGraphNode* InGraphNode)
{
	PlaybackGraphNode = InGraphNode;
	GraphNode = InGraphNode;
	
	SetCursor(EMouseCursor::CardinalCross);
	UpdateGraphNode();
	
	PostConstruct();
}

void SAvaPlaybackEditorGraphNode::CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox)
{
	TSharedRef<SWidget> AddPinButton = AddPinButtonContent(LOCTEXT("AvaPlaybackNodeAddPinButton", "Add Input")
		, LOCTEXT("AvaPlaybackNodeAddPinButton_Tooltip", "Adds an input to the Playback node"));

	FMargin AddPinPadding = Settings->GetOutputPinPadding();
	AddPinPadding.Top += 6.0f;

	OutputBox->AddSlot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		.Padding(AddPinPadding)
		[
			AddPinButton
		];
}

EVisibility SAvaPlaybackEditorGraphNode::IsAddPinButtonVisible() const
{
	EVisibility ButtonVisibility = SGraphNode::IsAddPinButtonVisible();
	if (ButtonVisibility == EVisibility::Visible)
	{
		if (PlaybackGraphNode.IsValid() && !PlaybackGraphNode->CanAddInputPin())
		{
			ButtonVisibility = EVisibility::Collapsed;
		}
	}
	return ButtonVisibility;
}

FReply SAvaPlaybackEditorGraphNode::OnAddPin()
{
	if (PlaybackGraphNode.IsValid())
	{
		PlaybackGraphNode->AddInputPin();
		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

TSharedPtr<SGraphPin> SAvaPlaybackEditorGraphNode::CreatePinWidget(UEdGraphPin* Pin) const
{
	if (Pin->PinType.PinCategory == UAvaPlaybackEditorGraphSchema::PC_Event)
	{
		return SNew(SGraphPinExec, Pin);
	}
	return SGraphNode::CreatePinWidget(Pin);
}

#undef LOCTEXT_NAMESPACE 
