// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphNode.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FReply;
class SGraphPin;
class SVerticalBox;
class UAvaPlaybackEditorGraphNode;
class UEdGraphPin;
struct EVisibility;

class SAvaPlaybackEditorGraphNode : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SAvaPlaybackEditorGraphNode) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, UAvaPlaybackEditorGraphNode* InGraphNode);
	virtual void PostConstruct() {};
	
protected:
	//~ Begin SGraphNode
	virtual void CreateOutputSideAddButton(TSharedPtr<SVerticalBox> OutputBox) override;
	virtual EVisibility IsAddPinButtonVisible() const override;
	virtual FReply OnAddPin() override;
	virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const override;
	//~ End SGraphNode
	
protected:
	TWeakObjectPtr<UAvaPlaybackEditorGraphNode> PlaybackGraphNode;
};
