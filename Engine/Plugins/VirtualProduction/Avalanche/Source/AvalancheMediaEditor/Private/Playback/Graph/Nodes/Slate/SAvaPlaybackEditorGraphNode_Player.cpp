// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaPlaybackEditorGraphNode_Player.h"
#include "Broadcast/Channel/AvaBroadcastOutputChannel.h"
#include "Broadcast/OutputDevices/Slate/SAvaBroadcastCaptureImage.h"
#include "Brushes/SlateImageBrush.h"
#include "Engine/TextureRenderTarget2D.h"
#include "GraphEditorSettings.h"
#include "Playback/Graph/Nodes/AvaPlaybackEditorGraphNode.h"
#include "Playback/Nodes/AvaPlaybackNodePlayer.h"
#include "Styling/SlateBrush.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/SBoxPanel.h"

/**
 * Wrapper to encapsulate a FSlateImageBrush that can be updated with a render target.
 * If the render target becomes null, the brush is updated to be a solid color.
 */
struct SAvaPlaybackEditorGraphNode_Player::FPreviewBrush
{
	FSlateImageBrush Brush;
	FVector2D RenderTargetSize;

	FPreviewBrush(const FVector2D& InRenderTargetSize)
		: Brush(NAME_None, InRenderTargetSize, FLinearColor::Black)
		, RenderTargetSize(InRenderTargetSize)
	{

	}

	void Update(UTextureRenderTarget2D* InRenderTarget)
	{
		if (InRenderTarget)
		{
			RenderTargetSize = FVector2D(InRenderTarget->SizeX, InRenderTarget->SizeY);

			//If Brush is invalid, or the Brush's Texture Target doesn't match the new Render Target, reset the Brush.
			if (Brush.GetResourceObject() != InRenderTarget)
			{
				Brush = FSlateImageBrush(InRenderTarget, RenderTargetSize);
			}
			//If Sizes mismatch, just resizes rather than recreating the Brush with same underlying Resource
			else if (Brush.GetImageSize() != RenderTargetSize)
			{
				Brush.SetImageSize(RenderTargetSize);
			}
		}
		else if (Brush.GetResourceObject() != nullptr)
		{
			// Preserve the size of the preview brush so the checkerboard remains the same.
			Brush = FSlateImageBrush(NAME_None, RenderTargetSize, FLinearColor::Black);
		}
	}
};

SAvaPlaybackEditorGraphNode_Player::SAvaPlaybackEditorGraphNode_Player() = default;
SAvaPlaybackEditorGraphNode_Player::~SAvaPlaybackEditorGraphNode_Player() = default;

void SAvaPlaybackEditorGraphNode_Player::PostConstruct()
{
	check(PlaybackGraphNode.IsValid());
	PlayerNode = Cast<UAvaPlaybackNodePlayer>(PlaybackGraphNode->GetPlaybackNode());
	check(PlayerNode.IsValid());
	
	FIntPoint RenderTargetSize;
	if (const UTextureRenderTarget2D* RenderTarget = PlayerNode->GetPreviewRenderTarget())
	{
		RenderTargetSize = FIntPoint(RenderTarget->SizeX, RenderTarget->SizeY);
	}
	else
	{
		RenderTargetSize = FAvaBroadcastOutputChannel::GetDefaultMediaOutputSize(EAvaBroadcastChannelType::Program);
	}	
	PlayerPreviewBrush = MakeUnique<FPreviewBrush>(FVector2d(RenderTargetSize.X, RenderTargetSize.Y));
}

TSharedRef<SWidget> SAvaPlaybackEditorGraphNode_Player::CreatePreviewWidget()
{
	return SNew(SBox)
		.WidthOverride(106.f)
		.HeightOverride(106.f)
		[
			SNew(SBorder)
			.Padding(5.0f)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				SNew(SScaleBox)
				.Stretch(EStretch::ScaleToFitY)
				[
					SNew(SAvaBroadcastCaptureImage)
					.ImageArgs(SImage::FArguments()
						.Image(this, &SAvaPlaybackEditorGraphNode_Player::GetPlayerPreviewBrush))
					.ShouldInvertAlpha(true)
					.EnableGammaCorrection(false)
					.EnableBlending(false)
				]
			]
		];
}

const FSlateBrush* SAvaPlaybackEditorGraphNode_Player::GetPlayerPreviewBrush() const
{
	// Update the brush here to ensure the cached image attribute is updated with the correct render target.
	PlayerPreviewBrush->Update(PlayerNode.IsValid() ? PlayerNode->GetPreviewRenderTarget() : nullptr);
	return &PlayerPreviewBrush->Brush;
}

void SAvaPlaybackEditorGraphNode_Player::CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox)
{
	if (GraphNode && MainBox.IsValid())
	{
		int32 LeftPinCount = InputPins.Num();
		int32 RightPinCount = OutputPins.Num();

		const float NegativeHPad = FMath::Max<float>(-Settings->PaddingTowardsNodeEdge, 0.0f);
		const float ExtraPad = 0.0f;

		// Place preview widget based on where the least pins are
		if (LeftPinCount < RightPinCount || RightPinCount == 0)
		{
			LeftNodeBox->AddSlot()
			.Padding(FMargin(NegativeHPad + ExtraPad, 0.0f, 0.0f, 0.0f))
			.AutoHeight()
			.HAlign(HAlign_Left)
			[
				CreatePreviewWidget()
			];
		}
		else if (LeftPinCount > RightPinCount)
		{
			RightNodeBox->AddSlot()
			.Padding(FMargin(NegativeHPad + ExtraPad, 0.0f, 0.0f, 0.0f))
			.AutoHeight()
			.HAlign(HAlign_Right)
			[
				CreatePreviewWidget()
			];
		}
		else
		{
			MainBox->AddSlot()
			.Padding(Settings->GetNonPinNodeBodyPadding())
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					CreatePreviewWidget()
				]
			];
		}
	}
}

TSharedRef<SWidget> SAvaPlaybackEditorGraphNode_Player::CreateNodeContentArea()
{
	// NODE CONTENT AREA
	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.FillWidth(1.0f)
			[
				// LEFT
				SAssignNew(LeftNodeBox, SVerticalBox)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			[
				// RIGHT
				SAssignNew(RightNodeBox, SVerticalBox)
			]
		];
}
