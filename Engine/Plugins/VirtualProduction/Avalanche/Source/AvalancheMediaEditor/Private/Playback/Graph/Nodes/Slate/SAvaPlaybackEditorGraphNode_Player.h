// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Playback/Graph/Nodes/Slate/SAvaPlaybackEditorGraphNode.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UAvaPlaybackNodePlayer;
class SWidget;
struct FSlateBrush;
class SVerticalBox;

class SAvaPlaybackEditorGraphNode_Player : public SAvaPlaybackEditorGraphNode
{
public:
	SAvaPlaybackEditorGraphNode_Player();
	virtual ~SAvaPlaybackEditorGraphNode_Player() override;

	virtual void PostConstruct() override;

	/** Creates a preview viewport if necessary */
	TSharedRef<SWidget> CreatePreviewWidget();

	const FSlateBrush* GetPlayerPreviewBrush() const;

	//SGraphNode Interface
	virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;
	virtual TSharedRef<SWidget> CreateNodeContentArea() override;
	//~SGraphNode Interface

protected:
	TWeakObjectPtr<UAvaPlaybackNodePlayer> PlayerNode;

	struct FPreviewBrush;
	TUniquePtr<FPreviewBrush> PlayerPreviewBrush;
};
