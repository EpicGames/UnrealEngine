// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaPlaybackEditorGraphNode_Root.h"
#include "EdGraph/EdGraphPin.h"
#include "Playback/Graph/AvaPlaybackEditorGraphSchema.h"
#include "Playback/Graph/Pins/Slate/SAvaPlaybackEditorGraphPin_Channel.h"
#include "SGraphNode.h"
#include "SGraphPin.h"

TSharedPtr<SGraphPin> SAvaPlaybackEditorGraphNode_Root::CreatePinWidget(UEdGraphPin* Pin) const
{
	if (Pin->PinType.PinCategory == UAvaPlaybackEditorGraphSchema::PC_ChannelFeed)
	{
		return SNew(SAvaPlaybackEditorGraphPin_Channel, Pin);
	}
	return SGraphNode::CreatePinWidget(Pin);
}
