// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Playback/Graph/Nodes/Slate/SAvaPlaybackEditorGraphNode.h"

class SGraphPin;
class UAvaPlaybackEditorGraphNode_Root;
class UEdGraphPin;

class SAvaPlaybackEditorGraphNode_Root : public SAvaPlaybackEditorGraphNode
{
public:
	/** Hook that allows derived classes to supply their own SGraphPin derivatives for any pin. */
	virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const override;
};
