// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaPlaybackEditorGraphPin_Channel.h"
#include "AvaMediaEditorUtils.h"
#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/Channel/AvaBroadcastOutputChannel.h"
#include "EdGraph/EdGraphPin.h"
#include "Internationalization/Text.h"
#include "Playback/Graph/AvaPlaybackEditorGraphSchema.h"
#include "SGraphPin.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateBrush.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SWidget.h"

void SAvaPlaybackEditorGraphPin_Channel::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	SGraphPin::Construct(SGraphPin::FArguments()
			.UsePinColorForText(true)
		, InPin);
	
	UpdateChannelState();
}

const FAvaBroadcastOutputChannel* SAvaPlaybackEditorGraphPin_Channel::GetChannel() const
{
	if (const UEdGraphPin* const ChannelPin = GetPinObj())
	{
		const FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetChannel(ChannelPin->GetFName());
		if (Channel.IsValidChannel())
		{
			return &Channel;
		}
	}
	return nullptr;
}

void SAvaPlaybackEditorGraphPin_Channel::UpdateChannelState()
{
	bPinEnabled = false;
	
	if (const FAvaBroadcastOutputChannel* const Channel = GetChannel())
	{
		bPinEnabled = true;
		ChannelStateText = FAvaMediaEditorUtils::GetChannelStatusText(Channel->GetState(), Channel->GetIssueSeverity());
		ChannelStateBrush = FAvaMediaEditorUtils::GetChannelStatusBrush(Channel->GetState(), Channel->GetIssueSeverity());
	}
}

TSharedRef<SWidget> SAvaPlaybackEditorGraphPin_Channel::GetDefaultValueWidget()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SImage)
			.Image(this, &SAvaPlaybackEditorGraphPin_Channel::GetChannelStatusBrush)
		]
		+ SHorizontalBox::Slot()
		.Padding(2.f, 0.f, 0.f, 0.f)
		.HAlign(EHorizontalAlignment::HAlign_Left)
		.VAlign(EVerticalAlignment::VAlign_Center)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(this, &SAvaPlaybackEditorGraphPin_Channel::GetChannelStatusText)
		];
}

FText SAvaPlaybackEditorGraphPin_Channel::GetChannelStatusText() const
{
	return ChannelStateText;
}

const FSlateBrush* SAvaPlaybackEditorGraphPin_Channel::GetChannelStatusBrush() const
{
	return ChannelStateBrush;
}

FSlateColor SAvaPlaybackEditorGraphPin_Channel::GetPinColor() const
{
	return bPinEnabled
		? UAvaPlaybackEditorGraphSchema::ActivePinColor
		: UAvaPlaybackEditorGraphSchema::InactivePinColor;
}	
