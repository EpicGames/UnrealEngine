// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphPin.h"
#include "Templates/SharedPointer.h"

struct FAvaBroadcastOutputChannel;
class UEdGraphPin;
class SWidget;
class FText;
struct FSlateBrush;
struct FSlateColor;

class SAvaPlaybackEditorGraphPin_Channel : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SAvaPlaybackEditorGraphPin_Channel){}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, UEdGraphPin* InPin);
	
	const FAvaBroadcastOutputChannel* GetChannel() const;
	
	void UpdateChannelState();
	
	/** Build the widget we should put into the 'default value' space, shown when nothing connected */
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;

	FText GetChannelStatusText() const;
	
	const FSlateBrush* GetChannelStatusBrush() const;

	virtual FSlateColor GetPinColor() const override;
	
protected:
	const FSlateBrush* ChannelStateBrush = nullptr;
	
	FText ChannelStateText;
	
	bool bPinEnabled = true;
};
