// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Graph/SchemaActions/AvaPlaybackAction_NewComment.h"

#include "EdGraphNode_Comment.h"
#include "Playback/AvaPlaybackGraph.h"
#include "Playback/Graph/AvaPlaybackEditorGraph.h"
#include "Playback/IAvaPlaybackGraphEditor.h"

UEdGraphNode* FAvaPlaybackAction_NewComment::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin
	, const FVector2f& Location, bool bSelectNewNode)
{
	// Add menu item for creating comment boxes
	UEdGraphNode_Comment* const CommentTemplate = NewObject<UEdGraphNode_Comment>();

	FVector2f SpawnLocation = Location;
	
	UAvaPlaybackGraph* const Playback = CastChecked<UAvaPlaybackEditorGraph>(ParentGraph)->GetPlaybackGraph();
	check(Playback);
	
	TSharedPtr<IAvaPlaybackGraphEditor> GraphEditor = Playback->GetGraphEditor();

	FSlateRect Bounds;
	if (GraphEditor && GraphEditor->GetBoundsForSelectedNodes(Bounds, 50.f))
	{
		CommentTemplate->SetBounds(Bounds);
		SpawnLocation.X = CommentTemplate->NodePosX;
		SpawnLocation.Y = CommentTemplate->NodePosY;
	}
	
	return FEdGraphSchemaAction_NewNode::SpawnNodeFromTemplate<UEdGraphNode_Comment>(ParentGraph, CommentTemplate, SpawnLocation);
}
