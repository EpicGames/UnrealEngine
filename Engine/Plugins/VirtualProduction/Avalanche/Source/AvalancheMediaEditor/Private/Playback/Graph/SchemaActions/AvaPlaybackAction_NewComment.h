// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"
#include "AvaPlaybackAction_NewComment.generated.h"

USTRUCT()
struct FAvaPlaybackAction_NewComment : public FEdGraphSchemaAction
{
	GENERATED_BODY()

	FAvaPlaybackAction_NewComment() = default;

	FAvaPlaybackAction_NewComment(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
	{
	}

	//~ Begin FEdGraphSchemaAction Interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph
		, UEdGraphPin* FromPin
		, const FVector2f& Location
		, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction Interface
};
