// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Graph/SchemaActions/AvaPlaybackAction_NewNode.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "Playback/AvaPlaybackGraph.h"
#include "Playback/Graph/AvaPlaybackEditorGraph.h"
#include "Playback/Nodes/AvaPlaybackNode.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "AvaPlaybackAction_NewNode"

FAvaPlaybackAction_NewNode::FAvaPlaybackAction_NewNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
	: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
{
}

UEdGraphNode* FAvaPlaybackAction_NewNode::PerformAction(UEdGraph* ParentGraph
	, UEdGraphPin* FromPin
	, const FVector2f& Location
	, bool bSelectNewNode)
{
	check(IsValid(PlaybackNodeClass));

	UAvaPlaybackGraph* Playback = CastChecked<UAvaPlaybackEditorGraph>(ParentGraph)->GetPlaybackGraph();
	check(Playback);
	
	const FScopedTransaction Transaction(LOCTEXT("NewPlaybackNode", "Motion Design Playback: New Node"));
	ParentGraph->Modify();
	Playback->Modify();

	UAvaPlaybackNode* NewNode = Playback->ConstructPlaybackNode<UAvaPlaybackNode>(PlaybackNodeClass, bSelectNewNode);

	// Attempt to connect inputs to selected nodes, unless we're already dragging from a single output
	if (FromPin == nullptr || FromPin->Direction == EGPD_Input)
	{
		ConnectToSelectedNodes(NewNode, ParentGraph);
	}

	UEdGraphNode* GraphNode = NewNode->GetGraphNode();
	GraphNode->NodePosX = Location.X;
	GraphNode->NodePosY = Location.Y;
	GraphNode->AutowireNewNode(FromPin);

	Playback->PostEditChange();
	Playback->MarkPackageDirty();

	return GraphNode;
}

void FAvaPlaybackAction_NewNode::SetPlaybackNodeClass(TSubclassOf<UAvaPlaybackNode> InPlaybackNodeClass)
{
	PlaybackNodeClass = InPlaybackNodeClass;
}

void FAvaPlaybackAction_NewNode::ConnectToSelectedNodes(UAvaPlaybackNode* NewNode, UEdGraph* ParentGraph) const
{
}

#undef LOCTEXT_NAMESPACE
