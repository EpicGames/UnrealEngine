// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"
#include "Math/MathFwd.h"
#include "Playback/Nodes/AvaPlaybackNode.h"
#include "Templates/SubclassOf.h"
#include "AvaPlaybackAction_NewNode.generated.h"

class FText;
class UAvaPlaybackNode;
class UEdGraph;
class UEdGraphNode;

USTRUCT()
struct FAvaPlaybackAction_NewNode : public FEdGraphSchemaAction
{
	GENERATED_BODY()

public:
	FAvaPlaybackAction_NewNode() = default;
	
	FAvaPlaybackAction_NewNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping);

	//~ Begin FEdGraphSchemaAction
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph
		, UEdGraphPin* FromPin
		, const FVector2f& Location
		, bool bSelectNewNode = true) override;
	//~ End FEdGraphSchemaAction

	void SetPlaybackNodeClass(TSubclassOf<UAvaPlaybackNode> InPlaybackNodeClass);

protected:
	/** Connects new node to output of selected nodes */
	void ConnectToSelectedNodes(UAvaPlaybackNode* NewNode, UEdGraph* ParentGraph) const;

	/** Class of node we want to create */
	UPROPERTY()
	TSubclassOf<UAvaPlaybackNode> PlaybackNodeClass;
};
