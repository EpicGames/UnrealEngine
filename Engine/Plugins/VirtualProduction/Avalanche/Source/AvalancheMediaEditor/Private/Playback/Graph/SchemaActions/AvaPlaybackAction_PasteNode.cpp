// Copyright Epic Games, Inc. All Rights Reserved.

#include "Playback/Graph/SchemaActions/AvaPlaybackAction_PasteNode.h"

#include "Playback/AvaPlaybackGraph.h"
#include "Playback/Graph/AvaPlaybackEditorGraph.h"
#include "Playback/IAvaPlaybackGraphEditor.h"

UEdGraphNode* FAvaPlaybackAction_PasteNode::PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin
	, const FVector2f& Location, bool bSelectNewNode)
{
	UAvaPlaybackGraph* const Playback = CastChecked<UAvaPlaybackEditorGraph>(ParentGraph)->GetPlaybackGraph();
	check(Playback);
	
	if (TSharedPtr<IAvaPlaybackGraphEditor> GraphEditor = Playback->GetGraphEditor())
	{
		GraphEditor->PasteNodesHere(FDeprecateSlateVector2D(Location));
	}
	
	return nullptr;
}
