// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPlaybackDetailsTabFactory.h"

#include "Playback/AvaPlaybackGraphEditor.h"
#include "Playback/DetailsView/SAvaPlaybackDetailsView.h"

const FName FAvaPlaybackDetailsTabFactory::TabID(TEXT("MotionDesignPlaybackDetails"));

#define LOCTEXT_NAMESPACE "AvaPlaybackDetailsTabFactory"

FAvaPlaybackDetailsTabFactory::FAvaPlaybackDetailsTabFactory(const TSharedPtr<FAvaPlaybackGraphEditor>& InPlaybackEditor)
	: FAvaPlaybackTabFactory(TabID, InPlaybackEditor)
{
	TabLabel = LOCTEXT("PlaybackDetails_TabLabel", "Details");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("PlaybackDetails_ViewMenu_Desc", "Playback Node Details");
	ViewMenuTooltip = LOCTEXT("PlaybackDetails_ViewMenu_ToolTip", "Playback Node Details");
}

TSharedRef<SWidget> FAvaPlaybackDetailsTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FAvaPlaybackGraphEditor> PlaybackEditor = PlaybackEditorWeak.Pin();
	check(PlaybackEditor.IsValid());
	return SNew(SAvaPlaybackDetailsView, PlaybackEditor);
}

#undef LOCTEXT_NAMESPACE
