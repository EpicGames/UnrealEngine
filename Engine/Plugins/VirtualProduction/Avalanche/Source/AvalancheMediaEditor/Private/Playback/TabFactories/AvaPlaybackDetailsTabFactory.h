// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaPlaybackTabFactory.h"

class FAvaPlaybackDetailsTabFactory : public FAvaPlaybackTabFactory
{
public:
	static const FName TabID;
	
	FAvaPlaybackDetailsTabFactory(const TSharedPtr<FAvaPlaybackGraphEditor>& InPlaybackEditor);
	
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
};
