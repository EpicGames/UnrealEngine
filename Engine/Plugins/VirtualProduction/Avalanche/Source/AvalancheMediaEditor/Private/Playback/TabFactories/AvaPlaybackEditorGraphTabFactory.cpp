// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPlaybackEditorGraphTabFactory.h"

#include "GraphEditor.h"
#include "Playback/AvaPlaybackGraphEditor.h"

const FName FAvaPlaybackEditorGraphTabFactory::TabID(TEXT("MotionDesignPlaybackGraph"));

#define LOCTEXT_NAMESPACE "AvaPlaybackEditorGraphTabFactory"

FAvaPlaybackEditorGraphTabFactory::FAvaPlaybackEditorGraphTabFactory(const TSharedPtr<FAvaPlaybackGraphEditor>& InPlaybackEditor)
	: FAvaPlaybackTabFactory(TabID, InPlaybackEditor)
{
	TabLabel = LOCTEXT("PlaybackGraph_TabLabel", "Playback Graph");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("PlaybackGraph_ViewMenu_Desc", "Playback Graph");
	ViewMenuTooltip = LOCTEXT("PlaybackGraph_ViewMenu_ToolTip", "Playback Graph");
}

TSharedRef<SWidget> FAvaPlaybackEditorGraphTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedRef<FAvaPlaybackGraphEditor> PlaybackEditor = PlaybackEditorWeak.Pin().ToSharedRef();	
	return PlaybackEditor->CreateGraphEditor();
}

#undef LOCTEXT_NAMESPACE
