// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaPlaybackTabFactory.h"

class FAvaPlaybackEditorGraphTabFactory : public FAvaPlaybackTabFactory
{
public:
	static const FName TabID;
	
	FAvaPlaybackEditorGraphTabFactory(const TSharedPtr<FAvaPlaybackGraphEditor>& InPlaybackEditor);
	
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
};
