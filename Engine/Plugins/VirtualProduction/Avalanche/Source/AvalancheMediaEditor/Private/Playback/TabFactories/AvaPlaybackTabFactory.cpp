// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPlaybackTabFactory.h"

#include "Playback/AvaPlaybackGraphEditor.h"

FAvaPlaybackTabFactory::FAvaPlaybackTabFactory(const FName& InTabID, const TSharedPtr<FAvaPlaybackGraphEditor>& InPlaybackEditor)
	: FWorkflowTabFactory(InTabID, InPlaybackEditor)
	, PlaybackEditorWeak(InPlaybackEditor)
{
}
