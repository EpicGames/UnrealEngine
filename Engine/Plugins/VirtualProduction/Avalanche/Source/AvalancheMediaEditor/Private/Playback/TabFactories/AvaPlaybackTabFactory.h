// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class FAvaPlaybackGraphEditor;

//Base class for all Tab Factories in Ava Playback Editor
class FAvaPlaybackTabFactory : public FWorkflowTabFactory
{
public:
	
	FAvaPlaybackTabFactory(const FName& InTabID, const TSharedPtr<FAvaPlaybackGraphEditor>& InPlaybackEditor);

protected:

	TWeakPtr<FAvaPlaybackGraphEditor> PlaybackEditorWeak;
};
