// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownAppMode.h"

#include "Rundown/AvaRundownEditor.h"

#define LOCTEXT_NAMESPACE "AvaRundownAppMode"

const FName FAvaRundownAppMode::DefaultMode(TEXT("DefaultMode"));

FAvaRundownAppMode::FAvaRundownAppMode(const TSharedPtr<FAvaRundownEditor>& InRundownEditor, const FName& InModeName)
	: FApplicationMode(InModeName, FAvaRundownAppMode::GetLocalizedMode)
	, RundownEditorWeak(InRundownEditor)
{
}

void FAvaRundownAppMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin();
	RundownEditor->PushTabFactories(TabFactories);
}

TSharedPtr<FDocumentTabFactory> FAvaRundownAppMode::GetDocumentTabFactory(const FName& InName) const
{
	if (const TSharedRef<FDocumentTabFactory>* DocFactory = DocumentTabFactories.Find(InName))
	{
		return *DocFactory;
	}
	return nullptr;
}

void FAvaRundownAppMode::RegisterDocumentTabFactory(const TSharedPtr<FDocumentTabFactory>& InDocumentTabFactory, const TSharedPtr<FTabManager>& InTabManager)
{
	if (!InDocumentTabFactory)
	{
		return;
	}
	
	// Keep track of this "document" factory so we don't add it again.
	DocumentTabFactories.Add(InDocumentTabFactory->GetIdentifier(), InDocumentTabFactory.ToSharedRef());

	// Add to the tab manager as a tab spawner.
	if (InTabManager)
	{
		if (InTabManager->HasTabSpawner(InDocumentTabFactory->GetIdentifier()))
		{
			InTabManager->UnregisterTabSpawner(InDocumentTabFactory->GetIdentifier());
		}
		
		InDocumentTabFactory->RegisterTabSpawner(InTabManager.ToSharedRef(), this);
	}

	// Update "Allowed" tab factories. (May not be necessary)
	if (!TabFactories.GetFactory(InDocumentTabFactory->GetIdentifier()).IsValid())
	{
		TabFactories.RegisterFactory(InDocumentTabFactory);
	}
}

void FAvaRundownAppMode::UnregisterDocumentTabFactory(const FName& InTabId, const TSharedPtr<FTabManager>& InTabManager)
{
	if (InTabManager && InTabManager->HasTabSpawner(InTabId))
	{
		InTabManager->UnregisterTabSpawner(InTabId);
	}

	if (TabFactories.GetFactory(InTabId).IsValid())
	{
		TabFactories.UnregisterFactory(InTabId);
	}

	DocumentTabFactories.Remove(InTabId);
}

FText FAvaRundownAppMode::GetLocalizedMode(const FName InMode)
{
	static TMap<FName, FText> LocModes;

	if (LocModes.Num() == 0)
	{
		LocModes.Add(DefaultMode, LOCTEXT("Rundown_DefaultMode", "Default"));
	}

	check(InMode != NAME_None);
	const FText* OutDesc = LocModes.Find(InMode);
	check(OutDesc);
	
	return *OutDesc;
}

#undef LOCTEXT_NAMESPACE
