// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

class FAvaRundownEditor;

class FAvaRundownAppMode : public FApplicationMode
{
public:

	// Mode constants
	static const FName DefaultMode;
	//Add more here
	
	FAvaRundownAppMode(const TSharedPtr<FAvaRundownEditor>& InRundownEditor, const FName& InModeName);
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;

	TSharedPtr<FDocumentTabFactory> GetDocumentTabFactory(const FName& InName) const;

	void RegisterDocumentTabFactory(const TSharedPtr<FDocumentTabFactory>& InDocumentTabFactory, const TSharedPtr<FTabManager>& InTabManager = nullptr);
	void UnregisterDocumentTabFactory(const FName& InTabId, const TSharedPtr<FTabManager>& InTabManager);

protected:
	
	static FText GetLocalizedMode(const FName InMode);

	TWeakPtr<FAvaRundownEditor> RundownEditorWeak;
	
	// Set of spawnable tabs in the mode
	FWorkflowAllowedTabSet TabFactories;

	TMap<FName, TSharedRef<FDocumentTabFactory>> DocumentTabFactories;
};
