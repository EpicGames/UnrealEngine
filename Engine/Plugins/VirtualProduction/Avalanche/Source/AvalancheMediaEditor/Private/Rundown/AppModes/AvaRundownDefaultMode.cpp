// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownDefaultMode.h"

#include "IAvaMediaEditorModule.h"
#include "Rundown/AvaRundownEditor.h"
#include "Rundown/TabFactories/AvaRundownChannelLayerStatusListTabFactory.h"
#include "Rundown/TabFactories/AvaRundownChannelStatusListTabFactory.h"
#include "Rundown/TabFactories/AvaRundownInstancedPageListTabFactory.h"
#include "Rundown/TabFactories/AvaRundownPageDetailsTabFactory.h"
#include "Rundown/TabFactories/AvaRundownPageViewerTabFactory.h"
#include "Rundown/TabFactories/AvaRundownShowControlTabFactory.h"
#include "Rundown/TabFactories/AvaRundownSubListDocumentTabFactory.h"
#include "Rundown/TabFactories/AvaRundownSubListTabFactory.h"
#include "Rundown/TabFactories/AvaRundownTemplatePageListTabFactory.h"

#define LOCTEXT_NAMESPACE "AvaRundownDefaultMode"

FAvaRundownDefaultMode::FAvaRundownDefaultMode(const TSharedPtr<FAvaRundownEditor>& InRundownEditor)
	: FAvaRundownAppMode(InRundownEditor, FAvaRundownAppMode::DefaultMode)
{
	WorkspaceMenuCategory = FWorkspaceItem::NewGroup(LOCTEXT("WorkspaceMenu_AvaRundown", "Motion Design Rundown"));
	
	check(InRundownEditor.IsValid());
	TabLayout = FTabManager::NewLayout("MotionDesignRundownEditor_Default_Layout_V2.4")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.5f)
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.25f)
					->AddTab(FAvaRundownTemplatePageListTabFactory::TabID, ETabState::OpenedTab)
					->SetForegroundTab(FAvaRundownTemplatePageListTabFactory::TabID)
					->SetHideTabWell(true)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.05f)
					->AddTab(FAvaRundownShowControlTabFactory::TabID, ETabState::OpenedTab)
					->SetForegroundTab(FAvaRundownShowControlTabFactory::TabID)
					->SetHideTabWell(true)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.35f)
					->AddTab(FAvaRundownInstancedPageListTabFactory::TabID, ETabState::OpenedTab)
					->SetForegroundTab(FAvaRundownInstancedPageListTabFactory::TabID)
					->SetHideTabWell(false)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.35f)
					->AddTab(FAvaRundownSubListTabFactory::TabID, ETabState::ClosedTab)
					->SetForegroundTab(FAvaRundownSubListTabFactory::TabID)
					->SetHideTabWell(false)
				)
			)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.60f)
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.3f)
					->SetOrientation(Orient_Vertical)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.7f)
						->AddTab(FAvaRundownPageDetailsTabFactory::TabID, ETabState::OpenedTab)
						->SetForegroundTab(FAvaRundownPageDetailsTabFactory::TabID)
						->SetHideTabWell(false)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.3f)
						->AddTab(FAvaRundownChannelLayerStatusListTabFactory::TabID, ETabState::OpenedTab)
						->SetForegroundTab(FAvaRundownChannelLayerStatusListTabFactory::TabID)
						->SetHideTabWell(false)
					)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.02f)
					->AddTab(FAvaRundownChannelStatusListTabFactory::TabID, ETabState::OpenedTab)
					->SetForegroundTab(FAvaRundownChannelStatusListTabFactory::TabID)
					->SetHideTabWell(true)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.6f)
					->AddTab(FAvaRundownPageViewerTabFactory::TabID, ETabState::OpenedTab)
					->SetForegroundTab(FAvaRundownPageViewerTabFactory::TabID)
					->SetHideTabWell(false)
				)
			)
		);

	// Add Tab Spawners
	TabFactories.RegisterFactory(MakeShared<FAvaRundownTemplatePageListTabFactory>(InRundownEditor));
	TabFactories.RegisterFactory(MakeShared<FAvaRundownInstancedPageListTabFactory>(InRundownEditor));
	TabFactories.RegisterFactory(MakeShared<FAvaRundownPageDetailsTabFactory>(InRundownEditor));
	TabFactories.RegisterFactory(MakeShared<FAvaRundownShowControlTabFactory>(InRundownEditor));
	TabFactories.RegisterFactory(MakeShared<FAvaRundownSubListTabFactory>(InRundownEditor));	
	TabFactories.RegisterFactory(MakeShared<FAvaRundownPageViewerTabFactory>(InRundownEditor));
	TabFactories.RegisterFactory(MakeShared<FAvaRundownChannelStatusListTabFactory>(InRundownEditor));
	TabFactories.RegisterFactory(MakeShared<FAvaRundownChannelLayerStatusListTabFactory>(InRundownEditor));

	// Document factories for existing "documents" within the rundown.
	if (const UAvaRundown* Rundown = InRundownEditor->GetRundown())
	{
		for (const FAvaRundownSubList& SubList : Rundown->GetSubLists())
		{
			TSharedPtr<FAvaRundownSubListDocumentTabFactory> PageViewDocumentFactory =
				MakeShared<FAvaRundownSubListDocumentTabFactory>(UAvaRundown::CreateSubListReference(SubList), InRundownEditor);

			RegisterDocumentTabFactory(PageViewDocumentFactory);
		}
	}

	// Make sure we start with our existing list of extenders instead of creating a new one
	IAvaMediaEditorModule& AvaMediaEditorModule = IAvaMediaEditorModule::Get();
	ToolbarExtender = AvaMediaEditorModule.GetRundownToolBarExtensibilityManager()->GetAllExtenders(
		InRundownEditor->GetToolkitCommands(),
		*InRundownEditor->GetObjectsCurrentlyBeingEdited()
	);
	InRundownEditor->ExtendToolBar(ToolbarExtender);
}

#undef LOCTEXT_NAMESPACE
