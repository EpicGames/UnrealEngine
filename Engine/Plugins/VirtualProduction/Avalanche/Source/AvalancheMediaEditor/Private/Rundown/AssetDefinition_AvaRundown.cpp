// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_AvaRundown.h"

#include "AvaMediaEditorStyle.h"
#include "ContentBrowserMenuContexts.h"
#include "Framework/Commands/UIAction.h"
#include "IAvaMediaEditorModule.h"
#include "Misc/MessageDialog.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownEditor.h"
#include "Rundown/AvaRundownEditorUtils.h"
#include "Rundown/AvaRundownSerializationUtils.h"
#include "ToolMenu.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "UAssetDefinition_AvaRundown"

FText UAssetDefinition_AvaRundown::GetAssetDisplayName() const
{
	return LOCTEXT("AvaRundownAction_Name", "Motion Design Rundown");
}

TSoftClassPtr<UObject> UAssetDefinition_AvaRundown::GetAssetClass() const
{
	return UAvaRundown::StaticClass();
}

FLinearColor UAssetDefinition_AvaRundown::GetAssetColor() const
{
	static const FName RundownAssetColorName(TEXT("AvaMediaEditor.AssetColors.Rundown"));
	return FAvaMediaEditorStyle::Get().GetColor(RundownAssetColorName);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_AvaRundown::GetAssetCategories() const
{
	static const TArray<FAssetCategoryPath> Categories = {EAssetCategoryPaths::Blueprint};
	return Categories;
}

EAssetCommandResult UAssetDefinition_AvaRundown::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	EAssetCommandResult CommandResult = EAssetCommandResult::Unhandled;
	if (OpenArgs.OpenMethod == EAssetOpenMethod::Edit)
	{
		for (UAvaRundown* Rundown : OpenArgs.LoadObjects<UAvaRundown>())
		{
			if (Rundown)
			{
				const TSharedRef<FAvaRundownEditor> RundownEditor = MakeShared<FAvaRundownEditor>();
				RundownEditor->InitRundownEditor(EToolkitMode::Standalone, OpenArgs.ToolkitHost, Rundown);
				CommandResult = EAssetCommandResult::Handled;
			}
		}
	}
	return CommandResult;
}

namespace UE::AvaMediaEditor::Rundown::Private
{
	static bool CanExportToJson(const FToolMenuContext& InContext)
	{
		return true;
	}
	
	static void ExportToJson(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		for (UAvaRundown* Rundown : Context->LoadSelectedObjects<UAvaRundown>())
		{
			if (Rundown)
			{
				if (Rundown->GetOutermost() && Rundown->GetOutermost()->HasAnyPackageFlags(PKG_DisallowExport))
				{
					UE_LOGF(LogAvaRundown, Error, "Package disallow export.");
					continue;
				}

				using namespace UE::AvaRundownEditor::Utils;
				using namespace UE::AvaMedia::RundownSerializationUtils;
				FString ExportFilename = GetExportFilepath(Rundown, TEXT("json file"), TEXT("json"));
				if (!ExportFilename.IsEmpty())
				{
					FText ErrorMessage;
					if (!SaveRundownToJson(Rundown, *ExportFilename, ErrorMessage))
					{
						UE_LOGF(LogAvaRundown, Error,
							"Failed to export rundown \"%ls\" to file \"%ls\". Reason: %ls",
							*Rundown->GetFullName(), *ExportFilename, *ErrorMessage.ToString());
					}
				}
			}
		}
	}

	static bool CanImportFromJson(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
		for (const UAvaRundown* Rundown : Context->GetSelectedObjectsInMemory<UAvaRundown>())
		{
			if (Rundown && Rundown->IsPlaying())
			{
				return false;
			}
		}
		return true;
	}
	
	static void ImportFromJson(const FToolMenuContext& InContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);

		for (UAvaRundown* Rundown : Context->LoadSelectedObjects<UAvaRundown>())
		{
			if (Rundown)
			{
				// Ask user to confirm stopping the rundown if it is playing.
				if (Rundown->IsPlaying())
				{
					const FText MessageText = LOCTEXT("StopPagesOnImportQuestion",
						"All pages must be stopped before importing. Some pages are still playing, do you want to stop all pages?");
			
					const EAppReturnType::Type Reply = FMessageDialog::Open(EAppMsgType::YesNoCancel, EAppReturnType::Yes, MessageText);

					if (Reply == EAppReturnType::Cancel)
					{
						return;	// cancel the whole operation.
					}

					if (Reply == EAppReturnType::No)
					{
						UE_LOGF(LogAvaRundown, Warning, "Skipping import of rundown \"%ls\"", *Rundown->GetFullName());
						continue;
					}
					
					if (Reply == EAppReturnType::Yes)
					{
						Rundown->ClosePlaybackContext(true);
					}
				}
				
				// Ask user to confirm stomping the existing rundown.
				if (!Rundown->IsEmpty())
				{
					const FText MessageText = LOCTEXT("ClearRundownOnImportQuestion",
						"The rundown is not empty, all existing content will be overwritten. Are you sure?");
			
					const EAppReturnType::Type Reply = FMessageDialog::Open(EAppMsgType::YesNoCancel, EAppReturnType::Yes, MessageText);
					if (Reply == EAppReturnType::Cancel)
					{
						return;
					}
					if (Reply == EAppReturnType::No)
					{
						continue;
					}
				}
				
				using namespace UE::AvaRundownEditor::Utils;
				using namespace UE::AvaMedia::RundownSerializationUtils;
				FString ImportFilename = GetImportFilepath(TEXT("json file"), TEXT("json"));
				if (!ImportFilename.IsEmpty())
				{
					FText ErrorMessage;
					if (!LoadRundownFromJson(Rundown, *ImportFilename, ErrorMessage))
					{
						UE_LOGF(LogAvaRundown, Error,
							"Failed to import rundown \"%ls\" from file \"%ls\". Reason: %ls",
							*Rundown->GetFullName(), *ImportFilename, *ErrorMessage.ToString());
					}
				}
			}
		}
	}

	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []
	{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
		{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UAvaRundown::StaticClass());

			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				{
					const TAttribute<FText> Label = LOCTEXT("AvaRundown_ExportToJson", "Export to Json");
					const TAttribute<FText> ToolTip = LOCTEXT("AvaRundown_ExportToJsonTooltip", "Export To Json");

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExportToJson);
					UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanExportToJson);
					InSection.AddMenuEntry("AvaRundown_ExportToJson", Label, ToolTip, FSlateIcon(), UIAction);
				}
				{
					const TAttribute<FText> Label = LOCTEXT("AvaRundown_ImportFromJson", "Import from Json");
					const TAttribute<FText> ToolTip = LOCTEXT("AvaRundown_ImportFromJsonTooltip", "Import from Json");

					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ImportFromJson);
					UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateStatic(&CanImportFromJson);
					InSection.AddMenuEntry("AvaRundown_ImportFromJson", Label, ToolTip, FSlateIcon(), UIAction);
				}
			}));
		}));
	});
}

#undef LOCTEXT_NAMESPACE
