// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_AvaRundownMacroCollection.h"

#include "AvaMediaEditorStyle.h"
#include "Rundown/AvaRundownMacroCollection.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_AvaRundownMacroCollection"

FText UAssetDefinition_AvaRundownMacroCollection::GetAssetDisplayName() const
{
	return LOCTEXT("AvaRundownMacroCollectionAction_Name", "Motion Design Rundown Macros");
}

TSoftClassPtr<UObject> UAssetDefinition_AvaRundownMacroCollection::GetAssetClass() const
{
	return UAvaRundownMacroCollection::StaticClass();
}

FLinearColor UAssetDefinition_AvaRundownMacroCollection::GetAssetColor() const
{
	static const FName RundownMacroCollectionAssetColorName(TEXT("AvaMediaEditor.AssetColors.RundownMacroCollection"));
	return FAvaMediaEditorStyle::Get().GetColor(RundownMacroCollectionAssetColorName);
}

#undef LOCTEXT_NAMESPACE
