// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "AssetDefinition_AvaRundownMacroCollection.generated.h"

UCLASS()
class UAssetDefinition_AvaRundownMacroCollection : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	//~ Begin UAssetDefinition
	virtual FText GetAssetDisplayName() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual FLinearColor GetAssetColor() const override;
	//~ End UAssetDefinition
};
