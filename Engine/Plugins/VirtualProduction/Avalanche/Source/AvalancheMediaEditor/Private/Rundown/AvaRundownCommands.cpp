// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownCommands.h"

#define LOCTEXT_NAMESPACE "AvaRundownCommands"

void FAvaRundownCommands::RegisterCommands()
{
	UI_COMMAND(AddTemplate
		, "Add Template"
		, "Adds a new Template to the Template List"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Insert));
	
	UI_COMMAND(CreatePageInstanceFromTemplate
		, "Create Page(s)"
		, "Adds a new copy of a Template into the main Page List"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Control, EKeys::Insert));

	UI_COMMAND(CreateComboTemplate
		, "Create Combo Template"
		, "Create a combination template that contains all selected templates. This is only relevant for transition logic templates. All templates that are to be merged into Combination Templates must be in different transition logic layers"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(RemovePage
		, "Remove Pages"
		, "Removes the Selected Pages from the Page List"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Delete));

	UI_COMMAND(RenumberPage
		, "Renumber Pages"
		, "Renumbers the Id of the Selected Pages"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::F3));
	
	UI_COMMAND(ReimportPage
		, "Reimport Page"
		, "Re-imports the selected page"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(EditPageSource
		, "Edit Scene"
		, "Edit source Motion Design Asset of the selected page"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(ExportPagesToRundown
		, "Export To Rundown"
		, "Export selected pages to a Rundown asset"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(ExportPagesToJson
		, "Export To Json"
		, "Export selected pages to an external json file"
		, EUserInterfaceActionType::Button
		, FInputChord());
	
	UI_COMMAND(ExportPagesToXml
		, "Export To Xml"
		, "Export selected pages to an external xml file"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(PreviewFrame
		, "Preview Frame"
		, "Runs the Selected Page without playing animations, but jumping to the Preview Frame set in each animation"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(PreviewPlay
		, "Preview In"
		, "Runs the Selected Page displayed in the Page Preview"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Control, EKeys::Enter));

	UI_COMMAND(PreviewStop
		, "Preview Out"
		, "Stops the Page displayed in the Page Preview"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Control, EKeys::Multiply));

	UI_COMMAND(PreviewForceStop
		, "Force Preview Out"
		, "Forcibly stops the Page displayed in the Page Preview regardless of it's current state."
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(PreviewContinue
		, "Continue"
		, "Continues to play the Animation in the Current Page being Played if it has stopped"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Control, EKeys::Divide));

	UI_COMMAND(PreviewPlayNext
		, "Preview Next"
		, "Previews the Next Page in the Page List"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Control, EKeys::Subtract));

	UI_COMMAND(TakeToProgram
		, "Take To Program"
		, "Runs the current previewed page in the selected channel"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Enter));
	
	UI_COMMAND(Play
		, "Take In"
		, "Runs the Selected Page displayed in the selected channel"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Shift, EKeys::Enter));

	UI_COMMAND(UpdateValues
		, "Update Values"
		, "Update Values of the Selected Page displayed in the selected channel"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::F5));
	
	UI_COMMAND(Stop
		, "Take Out"
		, "Stops the Page displayed in the selected channel"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Multiply));

	UI_COMMAND(ForceStop
		, "Force Take Out"
		, "Forcibly stops the Page displayed in the selected channel regardless of it's current state."
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(Continue
		, "Continue"
		, "Continues to play the Animation in the Current Page being Played if it has stopped"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Divide));

	UI_COMMAND(PlayNext
		, "Take Next"
		, "Takes the Next Page in the Page List"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::Subtract));

	UI_COMMAND(LoadAllPages
		, "Load All Pages"
		, "Loads all the pages for play in their set channels"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::L, EModifierKey::Control | EModifierKey::Shift | EModifierKey::Alt));

	UI_COMMAND(LoadNextPages
		, "Load Next Pages"
		, "Loads the next pages in program for play in its set channels"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::L, EModifierKey::Control | EModifierKey::Shift));

	UI_COMMAND(LoadSelectedPages
		, "Load Selected Pages"
		, "Loads the selected pages for play in their set channels"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::L, EModifierKey::Control));

	UI_COMMAND(ResetValuesToDefaults
		, "Reset To Defaults"
		, "Reset selected values to default values"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(ResetValuesToTemplate
		, "Reset To Template"
		, "Reset selected values to template values"
		, EUserInterfaceActionType::Button
		, FInputChord());
}

#undef LOCTEXT_NAMESPACE
