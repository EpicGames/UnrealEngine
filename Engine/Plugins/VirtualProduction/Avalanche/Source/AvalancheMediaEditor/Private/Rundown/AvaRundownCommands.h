// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaEditorStyle.h"
#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "AvaRundownCommands"

class FAvaRundownCommands : public TCommands<FAvaRundownCommands>
{
public:
	
	FAvaRundownCommands()
		: TCommands<FAvaRundownCommands>(TEXT("AvaRundown")
		, LOCTEXT("MotionDesignRundownCommands", "Motion Design Rundown")
		, NAME_None
		, FAvaMediaEditorStyle::Get().GetStyleSetName())
	{
	}

	/** Initialize commands */
	virtual void RegisterCommands() override;

	/** Page List Commands */
	TSharedPtr<FUICommandInfo> AddTemplate;
	TSharedPtr<FUICommandInfo> CreatePageInstanceFromTemplate;
	TSharedPtr<FUICommandInfo> CreateComboTemplate;
	TSharedPtr<FUICommandInfo> RemovePage;
	TSharedPtr<FUICommandInfo> RenumberPage;
	TSharedPtr<FUICommandInfo> ReimportPage;
	TSharedPtr<FUICommandInfo> EditPageSource;
	TSharedPtr<FUICommandInfo> ExportPagesToRundown;
	TSharedPtr<FUICommandInfo> ExportPagesToJson;
	TSharedPtr<FUICommandInfo> ExportPagesToXml;

	/** Show Control / Rundown Commands */
	TSharedPtr<FUICommandInfo> Play;
	TSharedPtr<FUICommandInfo> UpdateValues;
	TSharedPtr<FUICommandInfo> Stop;
	TSharedPtr<FUICommandInfo> ForceStop;
	TSharedPtr<FUICommandInfo> Continue;
	TSharedPtr<FUICommandInfo> PlayNext;
	TSharedPtr<FUICommandInfo> LoadAllPages;
	TSharedPtr<FUICommandInfo> LoadNextPages;
	TSharedPtr<FUICommandInfo> LoadSelectedPages;

	/** Preview Control / Rundown Commands */
	TSharedPtr<FUICommandInfo> PreviewFrame;
	TSharedPtr<FUICommandInfo> PreviewPlay;
	TSharedPtr<FUICommandInfo> PreviewStop;
	TSharedPtr<FUICommandInfo> PreviewForceStop;
	TSharedPtr<FUICommandInfo> PreviewContinue;
	TSharedPtr<FUICommandInfo> PreviewPlayNext;
	TSharedPtr<FUICommandInfo> TakeToProgram;

	/** Remote Control Commands */
	TSharedPtr<FUICommandInfo> ResetValuesToDefaults;
	TSharedPtr<FUICommandInfo> ResetValuesToTemplate;
};

#undef LOCTEXT_NAMESPACE
