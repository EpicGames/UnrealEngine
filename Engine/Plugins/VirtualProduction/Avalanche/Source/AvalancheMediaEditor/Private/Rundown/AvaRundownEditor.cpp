// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownEditor.h"

#include "AppModes/AvaRundownDefaultMode.h"
#include "Async/Async.h"
#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/AvaBroadcastEditor.h"
#include "Framework/Application/IInputProcessor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IAvaMediaEditorModule.h"
#include "IAvaMediaModule.h"
#include "Misc/MessageDialog.h"
#include "Pages/Slate/SAvaRundownInstancedPageList.h"
#include "Pages/Slate/SAvaRundownReadPage.h"
#include "Pages/Slate/SAvaRundownTemplatePageList.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownCommands.h"
#include "Rundown/AvaRundownEditorMacroCommands.h"
#include "Rundown/AvaRundownEditorSettings.h"
#include "Rundown/AvaRundownEditorUtils.h"
#include "Rundown/AvaRundownMacroCollection.h"
#include "Rundown/AvaRundownManagedInstanceCache.h"
#include "Rundown/AvaRundownPagePlayer.h"
#include "Rundown/AvaRundownPlaybackUtils.h"
#include "Rundown/Pages/PageViews/IAvaRundownPageView.h"
#include "Rundown/Pages/Slate/SAvaRundownPageList.h"
#include "ScopedTransaction.h"
#include "TabFactories/AvaRundownInstancedPageListTabFactory.h"
#include "TabFactories/AvaRundownSubListDocumentTabFactory.h"
#include "TabFactories/AvaRundownTemplatePageListTabFactory.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkflowOrientedApp/ApplicationMode.h"

#define LOCTEXT_NAMESPACE "AvaRundownEditor"

namespace UE::AvaRundownEditor::Private
{
	/** Return true if there's an Active Tab and if it is part of this Rundown Editor. */
	inline bool IsActiveTabPartOfEditor(FAvaRundownEditor& InRundownEditor)
	{
		const TSharedPtr<SDockTab> ActiveTab = FGlobalTabmanager::Get()->GetActiveTab();
		return ActiveTab.IsValid() && InRundownEditor.GetAssociatedTabManager() == ActiveTab->GetTabManagerPtr();
	}

	inline bool ShouldStopPagesOnClose()
	{
		const UAvaRundownEditorSettings* RundownEditorSettings = UAvaRundownEditorSettings::Get();
		return RundownEditorSettings ? RundownEditorSettings->bShouldStopPagesOnClose : false;
	}

	inline const UAvaRundownMacroCollection* GetMacroCollection()
	{
		const UAvaRundownEditorSettings* RundownEditorSettings = UAvaRundownEditorSettings::Get();
		if (!RundownEditorSettings || RundownEditorSettings->MacroCollection.IsNull())
		{
			return nullptr;
		}

		return RundownEditorSettings->MacroCollection.LoadSynchronous();
	}

	FInputChord MakeInputChord(const FKeyEvent& InKeyEvent)
	{
		return FInputChord(InKeyEvent.GetKey(),InKeyEvent.IsShiftDown(),  InKeyEvent.IsControlDown(), InKeyEvent.IsAltDown(), InKeyEvent.IsCommandDown());
	}

	FName GetName(EAvaRundownEditorMacroCommand InCommand)
	{
		return FAvaRundownEditorMacroCommands::GetShortCommandName(InCommand);
	}
}

class FAvaRundownEditorInputProcessor : public IInputProcessor
{
public:
	FAvaRundownEditorInputProcessor(const TWeakPtr<FAvaRundownEditor>& InRundownEditorWeak)
		: RundownEditorWeak(InRundownEditorWeak)
	{
	}

	virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor) override {}

	virtual bool HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent) override
	{
		if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
		{
			if (!UE::AvaRundownEditor::Private::IsActiveTabPartOfEditor(*RundownEditor))
			{
				return false;
			}

			// Do not pre-process keys that are not relevant to the editor
			if (!RundownEditor->IsKeyRelevant(InKeyEvent))
			{
				return false;
			}

			// Next, Check if the Keyboard Focused Widget (if valid) actually handles these Key Events
			const TSharedPtr<SWidget> FocusedWidget = FSlateApplicationBase::Get().GetKeyboardFocusedWidget();

			if (FocusedWidget.IsValid() && FocusedWidget->OnKeyDown(FocusedWidget->GetTickSpaceGeometry(), InKeyEvent).IsEventHandled())
			{
				// if handled, then just return as we don't want to process the same key again for the widget
				return true;
			}

			// If not handled then forward it to Rundown Editor
			return RundownEditor->HandleKeyDownEvent(InKeyEvent);
		}
		return false;
	}

private:
	TWeakPtr<FAvaRundownEditor> RundownEditorWeak;
};

/**
 *	Console commands are shared between all editors.
 *	However, only the editor that is the active tab will receive the commands.
 */
class FAvaRundownEditor::FSharedConsoleCommands
{
	struct FPrivateToken { explicit FPrivateToken() = default; };
public:
	static void RegisterEditor(const TSharedPtr<FAvaRundownEditor>& InRundownEditor);
	static void UnregisterEditor(FAvaRundownEditor* InRundownEditor);

	static TSharedPtr<FSharedConsoleCommands> GetSharedInstance();

	explicit FSharedConsoleCommands(FPrivateToken)
	{
		RegisterConsoleCommands();
	}

	~FSharedConsoleCommands()
	{
		UnregisterConsoleCommands();
	}

private:
	void RegisterConsoleCommands();
	void UnregisterConsoleCommands();

	TSharedPtr<FAvaRundownEditor> GetActiveRundownEditor() const;

	template <typename InFunctionType>
	void InvokeOnActiveEditor(InFunctionType InFunctionToInvoke) const
	{
		if (const TSharedPtr<FAvaRundownEditor> ActiveRundownEditor = GetActiveRundownEditor())
		{
			::Invoke(InFunctionToInvoke, ActiveRundownEditor);
		}
	}

	void StartAutoPlayCommand(const TArray<FString>& InArgs) const
	{
		InvokeOnActiveEditor([&InArgs](const TSharedPtr<FAvaRundownEditor>& InRundownEditor){InRundownEditor->StartAutoPlayCommand(InArgs);});
	}

	void StopAutoPlayCommand(const TArray<FString>& InArgs) const
	{
		InvokeOnActiveEditor([&InArgs](const TSharedPtr<FAvaRundownEditor>& InRundownEditor){InRundownEditor->StopAutoPlayCommand(InArgs);});
	}

	void LoadPageCommand(const TArray<FString>& InArgs) const
	{
		InvokeOnActiveEditor([&InArgs](const TSharedPtr<FAvaRundownEditor>& InRundownEditor){InRundownEditor->LoadPageCommand(InArgs);});
	}
	
	void UnloadPageCommand(const TArray<FString>& InArgs) const
	{
		InvokeOnActiveEditor([&InArgs](const TSharedPtr<FAvaRundownEditor>& InRundownEditor){InRundownEditor->UnloadPageCommand(InArgs);});
	}

	void StopLayersPreviewCommand(const TArray<FString>& InArgs) const
	{
		InvokeOnActiveEditor([&InArgs](const TSharedPtr<FAvaRundownEditor>& InRundownEditor){InRundownEditor->StopLayersCommand(InArgs, /*bInPreview*/ true, /*bInForce*/ false);});
	}

	void ForceStopLayersPreviewCommand(const TArray<FString>& InArgs) const
	{
		InvokeOnActiveEditor([&InArgs](const TSharedPtr<FAvaRundownEditor>& InRundownEditor){InRundownEditor->StopLayersCommand(InArgs, /*bInPreview*/ true, /*bInForce*/ true);});
	}

	void StopLayersProgramCommand(const TArray<FString>& InArgs) const
	{
		InvokeOnActiveEditor([&InArgs](const TSharedPtr<FAvaRundownEditor>& InRundownEditor){InRundownEditor->StopLayersCommand(InArgs, /*bInPreview*/ false, /*bInForce*/ false);});
	}

	void ForceStopLayersProgramCommand(const TArray<FString>& InArgs) const
	{
		InvokeOnActiveEditor([&InArgs](const TSharedPtr<FAvaRundownEditor>& InRundownEditor){InRundownEditor->StopLayersCommand(InArgs, /*bInPreview*/ false, /*bInForce*/ true);});
	}
	

private:
	TArray<IConsoleObject*> ConsoleCommands;
	TArray<TWeakPtr<FAvaRundownEditor>> RundownEditors;
};

FAvaRundownEditor::FAvaRundownEditor()
{}

FAvaRundownEditor::~FAvaRundownEditor()
{
	UAvaRundown* Rundown = AvaRundown.Get();

	if (IsValid(Rundown))
	{
		Rundown->GetOnActiveListChanged().RemoveAll(this);
		Rundown->GetOnPageListChanged().RemoveAll(this);
		Rundown->GetOnPagePlayerAdded().RemoveAll(this);
		Rundown->GetOnCanClosePlaybackContext().RemoveAll(this);

		if (Rundown->CanClosePlaybackContext())
		{
			const bool bStopPages = bStopPagesOnCloseOverride.IsSet() ?
				bStopPagesOnCloseOverride.GetValue() : UE::AvaRundownEditor::Private::ShouldStopPagesOnClose();

			Rundown->ClosePlaybackContext(bStopPages);
		}
	}

	if (InputProcessor.IsValid() && FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().UnregisterInputPreProcessor(InputProcessor);
	}

	if (IAvaMediaModule::IsModuleLoaded())
	{
		// This is a safety flush to provide the possibility to reset all loaded assets.
		// This is to force a reload of the assets if they are modified in another editor.
		IAvaMediaModule::Get().GetManagedInstanceCache().Flush();
	}

	FSharedConsoleCommands::UnregisterEditor(this);
}

void FAvaRundownEditor::InitRundownEditor(const EToolkitMode::Type InMode
	, const TSharedPtr<IToolkitHost>& InInitToolkitHost
	, UAvaRundown* InRundown)
{
	if (FSlateApplication::IsInitialized())
	{
		InputProcessor = MakeShared<FAvaRundownEditorInputProcessor>(SharedThis(this));
		FSlateApplication::Get().RegisterInputPreProcessor(InputProcessor);
	}

	AvaRundown = InRundown;

	CreateRundownCommands();

	if (IsValid(InRundown))
	{
		InRundown->GetOnActiveListChanged().AddSP(this, &FAvaRundownEditor::OnActiveListChanged);
		InRundown->GetOnPageListChanged().AddSP(this, &FAvaRundownEditor::OnPageListChanged);
		InRundown->GetOnPagePlayerAdded().AddSP(this, &FAvaRundownEditor::HandleOnPagePlayerAdded);
		InRundown->GetOnCanClosePlaybackContext().AddSP(this, &FAvaRundownEditor::OnCanClosePlaybackContext);
		InRundown->InitializePlaybackContext();
	}

	const FName RundownEditorAppName(TEXT("MotionDesignRundownEditorApp"));
	constexpr bool bCreateDefaultStandaloneMenu = true;
	constexpr bool bCreateDefaultToolbar = true;

	InitAssetEditor(InMode
		, InInitToolkitHost
		, RundownEditorAppName
		, FTabManager::FLayout::NullLayout
		, bCreateDefaultStandaloneMenu
		, bCreateDefaultToolbar
		, InRundown);

	RegisterApplicationModes();
	FSharedConsoleCommands::RegisterEditor(SharedThis(this));
}

bool FAvaRundownEditor::IsKeyRelevant(const FKeyEvent& InKeyEvent) const
{
	if (ReadPageWidget.IsValid() && ReadPageWidget->IsKeyRelevant(InKeyEvent))
	{
		return true;
	}

	using namespace UE::AvaRundownEditor::Private;
	const UAvaRundownMacroCollection* MacroCollection = GetMacroCollection();
	if (MacroCollection && MacroCollection->HasBindingFor(MakeInputChord(InKeyEvent)))
	{
		return true;
	}
	
	return false;
}

bool FAvaRundownEditor::HandleKeyDownEvent(const FKeyEvent& InKeyEvent)
{
	if (ReadPageWidget.IsValid() && ReadPageWidget->ProcessRundownKeyDown(InKeyEvent))
	{
		return true;
	}

	// Check if we have a binding for this key
	using namespace UE::AvaRundownEditor::Private;
	const UAvaRundownMacroCollection* MacroCollection = GetMacroCollection();
	if (!MacroCollection)
	{
		return false;
	}

	const FInputChord InputChord = MakeInputChord(InKeyEvent);
	const int32 NumCommandsFound = MacroCollection->ForEachCommand(InputChord, [this](const FAvaRundownMacroCommand& InCommand)
	{
		if (const FMacroCommandFunction* CommandFunction = GetBindableMacroCommands().Find(InCommand.Name))
		{
			TArray<FString> Args;
			InCommand.Arguments.ParseIntoArrayWS(Args, TEXT(","));
			(*CommandFunction)(Args);
			return true;
		}
		return false;
	});
	
	return NumCommandsFound > 0;
}

TSharedPtr<SAvaRundownPageList> FAvaRundownEditor::GetTemplateListWidget() const
{
	return GetListWidget(FAvaRundownTemplatePageListTabFactory::TabID);
}

TSharedPtr<SAvaRundownPageList> FAvaRundownEditor::GetInstanceListWidget() const
{
	return GetListWidget(FAvaRundownInstancedPageListTabFactory::TabID);
}

TSharedPtr<SAvaRundownPageList> FAvaRundownEditor::GetListWidget(const FAvaRundownPageListReference& InPageListReference) const
{
	if (InPageListReference.Type == EAvaRundownPageListType::Template)
	{
		return GetListWidget(FAvaRundownTemplatePageListTabFactory::TabID);
	}

	if (InPageListReference.Type == EAvaRundownPageListType::Instance)
	{
		return GetListWidget(FAvaRundownInstancedPageListTabFactory::TabID);
	}

	return GetListWidget(FAvaRundownSubListDocumentTabFactory::GetTabId(InPageListReference));
}

TSharedPtr<SAvaRundownPageList> FAvaRundownEditor::GetListWidget(const FName& InTabId) const
{
	if (TabManager.IsValid())
	{
		TSharedPtr<SDockTab> FoundTab = TabManager->FindExistingLiveTab(InTabId);

		if (FoundTab.IsValid())
		{
			TSharedPtr<SWidget> Content = FoundTab->GetContent();

			if (Content->GetWidgetClass().GetWidgetType() == SAvaRundownPageList::StaticWidgetClass().GetWidgetType()
				|| Content->GetWidgetClass().GetWidgetType() == SAvaRundownInstancedPageList::StaticWidgetClass().GetWidgetType()
				|| Content->GetWidgetClass().GetWidgetType() == SAvaRundownTemplatePageList::StaticWidgetClass().GetWidgetType())
			{
				return StaticCastSharedPtr<SAvaRundownPageList>(Content);
			}
		}
	}

	return nullptr;
}

TSharedPtr<SAvaRundownInstancedPageList> FAvaRundownEditor::GetActiveListWidget() const
{
	const UAvaRundown* Rundown = AvaRundown.Get();

	if (!IsValid(Rundown))
	{
		return nullptr;
	}

	const FAvaRundownPageListReference& ActiveList = Rundown->GetActivePageListReference();

	if (ActiveList.Type == EAvaRundownPageListType::Template)
	{
		return nullptr;
	}

	TSharedPtr<SAvaRundownPageList> PageList;

	if (ActiveList.Type == EAvaRundownPageListType::Instance)
	{
		PageList = GetListWidget(FAvaRundownInstancedPageListTabFactory::TabID);
	}
	else if (ActiveList.Type == EAvaRundownPageListType::View)
	{
		PageList = GetListWidget(FAvaRundownSubListDocumentTabFactory::GetTabId(ActiveList));
	}

	if (PageList.IsValid() && PageList->GetWidgetClass().GetWidgetType() == SAvaRundownInstancedPageList::StaticWidgetClass().GetWidgetType())
	{
		return StaticCastSharedPtr<SAvaRundownInstancedPageList>(PageList);
	}

	return nullptr;
}

TSharedPtr<SAvaRundownPageList> FAvaRundownEditor::GetFocusedListWidget() const
{
	if (TabManager.IsValid())
	{
		const TSharedPtr<SDockTab> ActiveTab = FGlobalTabmanager::Get()->GetActiveTab();

		if (ActiveTab.IsValid() && const_cast<FAvaRundownEditor*>(this)->GetAssociatedTabManager() == ActiveTab->GetTabManagerPtr())
		{
			const TSharedPtr<SWidget> Content = ActiveTab->GetContent();

			if (Content->GetWidgetClass().GetWidgetType() == SAvaRundownPageList::StaticWidgetClass().GetWidgetType()
				|| Content->GetWidgetClass().GetWidgetType() == SAvaRundownInstancedPageList::StaticWidgetClass().GetWidgetType()
				|| Content->GetWidgetClass().GetWidgetType() == SAvaRundownTemplatePageList::StaticWidgetClass().GetWidgetType())
			{
				return StaticCastSharedPtr<SAvaRundownPageList>(Content);
			}
		}
	}
	return nullptr;
}

int32 FAvaRundownEditor::GetFirstSelectedPageOnActiveSubListWidget() const
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		return ActiveWidget->GetFirstSelectedPageId();
	}

	return FAvaRundownPage::InvalidPageId;
}

TConstArrayView<int32> FAvaRundownEditor::GetSelectedPagesOnActiveSubListWidget() const
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		return ActiveWidget->GetSelectedPageIds();
	}

	return {};
}

TConstArrayView<int32> FAvaRundownEditor::GetSelectedPagesOnFocusedWidget() const
{
	const TSharedPtr<SAvaRundownPageList> FocusedWidget = GetFocusedListWidget();
	if (FocusedWidget.IsValid())
	{
		return FocusedWidget->GetSelectedPageIds();
	}

	return {};
}

bool FAvaRundownEditor::CanAddTemplate() const
{
	const UAvaRundown* Rundown = AvaRundown.Get();
	if (IsValid(Rundown))
	{
		return Rundown->CanAddPage();
	}
	return false;
}

void FAvaRundownEditor::AddTemplate()
{
	UAvaRundown* Rundown = AvaRundown.Get();

	if (IsValid(Rundown))
	{
		FScopedTransaction Transaction(LOCTEXT("AddTemplate", "Add Template"));
		Rundown->Modify();
		
		if (Rundown->AddTemplate() == FAvaRundownPage::InvalidPageId)
		{
			Transaction.Cancel();
		}
	}
}

bool FAvaRundownEditor::CanPlaySelectedPage() const
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		return ActiveWidget->CanPlaySelectedPage();
	}

	return false;
}

void FAvaRundownEditor::PlaySelectedPage()
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		ActiveWidget->PlaySelectedPage();
	}
}

bool FAvaRundownEditor::CanUpdateValuesOnSelectedPage() const
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();
	return ActiveWidget.IsValid() ? ActiveWidget->CanUpdateValuesOnSelectedPage() : false;
}

void FAvaRundownEditor::UpdateValuesOnSelectedPage()
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		ActiveWidget->UpdateValuesOnSelectedPage();
	}
}

bool FAvaRundownEditor::CanStopSelectedPage(bool bInForce) const
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		return ActiveWidget->CanStopSelectedPage(bInForce);
	}

	return false;
}

void FAvaRundownEditor::StopSelectedPage(bool bInForce)
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		ActiveWidget->StopSelectedPage(bInForce);
	}
}

bool FAvaRundownEditor::CanContinueSelectedPage() const
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		return ActiveWidget->CanContinueSelectedPage();
	}

	return false;
}

void FAvaRundownEditor::ContinueSelectedPage()
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		ActiveWidget->ContinueSelectedPage();
	}
}

bool FAvaRundownEditor::CanPlayNextPage() const
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		return ActiveWidget->CanPlayNextPage();
	}

	return false;
}

void FAvaRundownEditor::PlayNextPage()
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		ActiveWidget->PlayNextPage();
	}
}

bool FAvaRundownEditor::CanLoadAllPages() const
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActivePageList = GetActiveListWidget();
	return ActivePageList && ActivePageList->CanLoadAllPages();
}

void FAvaRundownEditor::LoadAllPages()
{
	if (const TSharedPtr<SAvaRundownInstancedPageList> ActivePageList = GetActiveListWidget())
	{
		ActivePageList->LoadAllPages();
	}
}

bool FAvaRundownEditor::CanLoadNextPages() const
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActivePageList = GetActiveListWidget();
	return ActivePageList && ActivePageList->CanLoadNextPages();
}

void FAvaRundownEditor::LoadNextPages()
{
	if (const TSharedPtr<SAvaRundownInstancedPageList> ActivePageList = GetActiveListWidget())
	{
		ActivePageList->LoadNextPages();
	}
}

bool FAvaRundownEditor::CanLoadSelectedPages() const
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActivePageList = GetActiveListWidget();
	return ActivePageList && ActivePageList->CanLoadSelectedPages();
}

void FAvaRundownEditor::LoadSelectedPages()
{
	if (const TSharedPtr<SAvaRundownInstancedPageList> ActivePageList = GetActiveListWidget())
	{
		ActivePageList->LoadSelectedPages();
	}
}

bool FAvaRundownEditor::CanPreviewPlaySelectedPage() const
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		if (ActiveWidget->CanPreviewPlaySelectedPage())
		{
			return true;
		}

		// If instanced page list can't preview play the page it may be that you selected a template page.
		if (const TSharedPtr<SAvaRundownPageList> TemplateWidget = GetTemplateListWidget())
		{
			return TemplateWidget->CanPreviewPlaySelectedPage();
		}
	}

	return false;
}

void FAvaRundownEditor::PreviewPlaySelectedPage(bool bInToMark)
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		// If instanced page list can't preview play but you reached here than it is for sure a template page selected
		if (ActiveWidget->CanPreviewPlaySelectedPage())
		{
			ActiveWidget->PreviewPlaySelectedPage(bInToMark);
		}
		else if (const TSharedPtr<SAvaRundownPageList> TemplateWidget = GetTemplateListWidget())
		{
			TemplateWidget->PreviewPlaySelectedPage(bInToMark);
		}
	}
}

bool FAvaRundownEditor::CanPreviewStopSelectedPage(bool bInForce) const
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		return ActiveWidget->CanPreviewStopSelectedPage(bInForce);
	}

	return false;
}

void FAvaRundownEditor::PreviewStopSelectedPage(bool bInForce)
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		ActiveWidget->PreviewStopSelectedPage(bInForce);
	}
}

bool FAvaRundownEditor::CanPreviewContinueSelectedPage() const
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		return ActiveWidget->CanPreviewContinueSelectedPage();
	}

	return false;
}

void FAvaRundownEditor::PreviewContinueSelectedPage()
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		ActiveWidget->PreviewContinueSelectedPage();
	}
}

bool FAvaRundownEditor::CanPreviewPlayNextPage() const
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		return ActiveWidget->CanPreviewPlayNextPage();
	}

	return false;
}

void FAvaRundownEditor::PreviewPlayNextPage()
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		ActiveWidget->PreviewPlayNextPage();
	}
}

bool FAvaRundownEditor::CanTakeToProgram() const
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		return ActiveWidget->CanTakeToProgram();
	}

	return false;
}

void FAvaRundownEditor::TakeToProgram() const
{
	const TSharedPtr<SAvaRundownInstancedPageList> ActiveWidget = GetActiveListWidget();

	if (ActiveWidget.IsValid())
	{
		ActiveWidget->TakeToProgram();
	}
}

bool FAvaRundownEditor::CanCreateInstancesFromSelectedTemplates() const
{
	UAvaRundown* Rundown = AvaRundown.Get();

	if (!IsValid(Rundown))
	{
		return false;
	}

	const TSharedPtr<SAvaRundownPageList> TemplateListWidget = GetTemplateListWidget();

	if (!TemplateListWidget)
	{
		return false;
	}

	const bool bHasSelectedPages = TemplateListWidget.IsValid() && (TemplateListWidget->GetSelectedPageIds().IsEmpty() == false);
	const bool bCanAddPage = Rundown->CanAddPage();

	return bHasSelectedPages && bCanAddPage;
}

void FAvaRundownEditor::CreateInstancesFromSelectedTemplates()
{
	UAvaRundown* Rundown = AvaRundown.Get();

	if (!IsValid(Rundown))
	{
		return;
	}

	const TSharedPtr<SAvaRundownPageList> TemplateListWidget = GetTemplateListWidget();

	if (!TemplateListWidget)
	{
		return;
	}
	
	FScopedTransaction Transaction(LOCTEXT("CreateInstancesFromSelectedTemplates", "Create Instances From Selected Templates"));
	Rundown->Modify();

	const TArray<int32> AddedPages = Rundown->AddPagesFromTemplates(TemplateListWidget->GetSelectedPageIds());

	if (!AddedPages.IsEmpty())
	{
		const TSharedPtr<SAvaRundownInstancedPageList> PageListWidget = GetActiveListWidget();

		if (PageListWidget.IsValid() && PageListWidget->GetPageListReference().Type == EAvaRundownPageListType::View)
		{
			Rundown->AddPagesToSubList(PageListWidget->GetPageListReference(), AddedPages);
		}
	}
	else
	{
		Transaction.Cancel();
	}
}

bool FAvaRundownEditor::CanRemoveSelectedPages() const
{
	const UAvaRundown* Rundown = AvaRundown.Get();

	if (!IsValid(Rundown))
	{
		return false;
	}

	const TSharedPtr<SAvaRundownPageList> PageListWidget = GetFocusedListWidget();

	return PageListWidget.IsValid() && (PageListWidget->GetSelectedPageIds().IsEmpty() == false)
		&& Rundown->CanRemovePages(PageListWidget->GetSelectedPageIds());
}

void FAvaRundownEditor::RemoveSelectedPages()
{
	UAvaRundown* Rundown = AvaRundown.Get();

	if (!IsValid(Rundown))
	{
		return;
	}

	const TSharedPtr<SAvaRundownPageList> PageListWidget = GetFocusedListWidget();
	if (PageListWidget.IsValid())
	{
		FScopedTransaction Transaction(LOCTEXT("RemoveSelectedPages", "Remove Selected Pages"));
		Rundown->Modify();

		const int32 RemovedCount = Rundown->RemovePages(PageListWidget->GetSelectedPageIds());
		
		if (RemovedCount == 0)
		{
			Transaction.Cancel();
		}
	}
}

void FAvaRundownEditor::RefreshSubListTabs()
{
	const UAvaRundown* Rundown = AvaRundown.Get();
	if (IsValid(Rundown))
	{
		for (const FAvaRundownSubList& SubList : Rundown->GetSubLists())
		{
			RefreshSubListTab(UAvaRundown::CreateSubListReference(SubList), /*bInSetActive*/false);
		}
	}
}

bool FAvaRundownEditor::RequestCloseDocumentTab(const FName& InDocumentTabId)
{
	if (TabManager)
	{
		if (const TSharedPtr<SDockTab> DocumentTab = TabManager->FindExistingLiveTab(InDocumentTabId))
		{
			return DocumentTab->RequestCloseTab();
		}
	}
	return false;
}

void FAvaRundownEditor::UnregisterDocumentTabFactory(const FName& InDocumentTabId)
{
	const TSharedPtr<FApplicationMode> AppMode = GetCurrentModePtr();
	if (AppMode.IsValid())
	{
		const TSharedRef<FAvaRundownAppMode> RundownAppMode = StaticCastSharedRef<FAvaRundownAppMode>(AppMode.ToSharedRef());
		RundownAppMode->UnregisterDocumentTabFactory(InDocumentTabId, TabManager);
	}
}

FName FAvaRundownEditor::GetToolkitFName() const
{
	return TEXT("AvaRundownEditor");
}

FText FAvaRundownEditor::GetBaseToolkitName() const
{
	return LOCTEXT("RundownAppLabel", "Motion Design Rundown Editor");
}

FString FAvaRundownEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("RundownScriptPrefix", "Script ").ToString();
}

FLinearColor FAvaRundownEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.0f, 0.0f, 0.3f, 0.5f);
}

bool FAvaRundownEditor::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	const UAvaRundown* Rundown = AvaRundown.Get();

	// Ask user if they want pages to be stopped.
	// Only ask if the editor is closed by the user directly.
	if (IsValid(Rundown) && InCloseReason == EAssetEditorCloseReason::AssetEditorHostClosed)
	{
		if (Rundown->IsPlaying())
		{
			
			const FText MessageText = LOCTEXT("StopPagesOnExitQuestion",
				"The editor is closing and some pages are still playing, do you want to stop all pages?");
			
			const EAppReturnType::Type Reply = FMessageDialog::Open(EAppMsgType::YesNoCancel, EAppReturnType::Yes, MessageText);

			if (Reply == EAppReturnType::Cancel)
			{
				return false;	// Don't close the editor.
			}
			
			bStopPagesOnCloseOverride = (Reply == EAppReturnType::Yes);
		}
	}

	return FWorkflowCentricApplication::OnRequestClose(InCloseReason);
}

TSharedRef<SWidget> FAvaRundownEditor::MakeReadPageWidget()
{
	return SAssignNew(ReadPageWidget, SAvaRundownReadPage, SharedThis(this));
}

void FAvaRundownEditor::ExtendToolBar(TSharedPtr<FExtender> InExtender)
{
	InExtender->AddToolBarExtension("Asset"
		, EExtensionHook::After
		, ToolkitCommands
		, FToolBarExtensionDelegate::CreateSP(this, &FAvaRundownEditor::FillPageToolBar));
}

void FAvaRundownEditor::FillPageToolBar(FToolBarBuilder& OutToolBarBuilder)
{
	const FAvaRundownCommands& RundownCommands = FAvaRundownCommands::Get();

	OutToolBarBuilder.BeginSection(TEXT("Pages"));
	{
		OutToolBarBuilder.AddToolBarButton(RundownCommands.AddTemplate);
		OutToolBarBuilder.AddToolBarButton(RundownCommands.CreatePageInstanceFromTemplate);
		OutToolBarBuilder.AddToolBarButton(RundownCommands.RemovePage);
	}
	OutToolBarBuilder.EndSection();

	OutToolBarBuilder.AddSeparator();

	OutToolBarBuilder.BeginSection(TEXT("Broadcast"));
	{
		OutToolBarBuilder.AddToolBarButton(FExecuteAction::CreateStatic(&FAvaBroadcastEditor::OpenBroadcastEditor)
			, NAME_None
			, LOCTEXT("Broadcast_Label", "Broadcast")
			, LOCTEXT("Broadcast_ToolTip", "Opens the Motion Design Broadcast Editor Window")
			, TAttribute<FSlateIcon>::Create([]() { return IAvaMediaEditorModule::Get().GetToolbarBroadcastButtonIcon(); }));
		OutToolBarBuilder.AddComboButton(
			FUIAction(FExecuteAction()
			, FCanExecuteAction::CreateLambda([](){ return !UAvaBroadcast::Get().IsBroadcastingAnyChannel();}))
			, FOnGetContent::CreateSP(this, &FAvaRundownEditor::MakeProfileComboButton)
			, TAttribute<FText>(this, &FAvaRundownEditor::GetCurrentProfileName)
			, LOCTEXT("Profile_ToolTip", "Pick between different Profiles")
			, FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.Profile")
			, false);
		OutToolBarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateLambda([]{ UAvaBroadcast::Get().StartBroadcast();})
				, FCanExecuteAction::CreateLambda([]{ return !UAvaBroadcast::Get().IsBroadcastingAllChannels();}))
			, NAME_None
			, LOCTEXT("Play_Label", "Start All Channels")
			, LOCTEXT("Play_ToolTip", "Starts Broadcast on all Idle Channels")
			, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Play"));
		OutToolBarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateLambda([]{ UAvaBroadcast::Get().StopBroadcast();})
				, FCanExecuteAction::CreateLambda([]{ return UAvaBroadcast::Get().IsBroadcastingAnyChannel();}))
			, NAME_None
			, LOCTEXT("Stop_Label", "Stop All Channels")
			, LOCTEXT("Stop_ToolTip", "Stops Broadcast on all Live Channels")
			, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Stop"));
	}
	OutToolBarBuilder.EndSection();
}

UAvaRundown* FAvaRundownEditor::GetRundown() const
{
	return AvaRundown.Get();
}

void FAvaRundownEditor::BeginModify()
{
	if (AvaRundown.IsValid())
	{
		AvaRundown->Modify(/*bAlwaysMarkDirty*/false);
	}
}

void FAvaRundownEditor::MarkAsModified()
{
	if (AvaRundown.IsValid() && AvaRundown->CanModify())
	{
		AvaRundown->MarkPackageDirty();
	}
}

void FAvaRundownEditor::OnPageListChanged(const FAvaRundownPageListChangeParams& InParams)
{
	// Undo Support - Create missing Page View Tabs.
	RefreshSubListTab(InParams.PageListReference);
}

void FAvaRundownEditor::OnActiveListChanged()
{
	const UAvaRundown* Rundown = AvaRundown.Get();
	if (!IsValid(Rundown))
	{
		return;
	}

	const FAvaRundownPageListReference& ActiveList = Rundown->GetActivePageListReference();
	RefreshSubListTab(ActiveList, /*bInSetActive*/true);
}

void FAvaRundownEditor::HandleOnPagePlayerAdded(UAvaRundown* InRundown, UAvaRundownPagePlayer* InPagePlayer)
{
	UAvaRundown* Rundown = AvaRundown.Get();

	if (!IsValid(Rundown) || InRundown != Rundown || !IsValid(InPagePlayer))
	{
		return;
	}

	// Auto start preview channel.
	if (InPagePlayer->bIsPreview)
	{
		const UAvaRundownEditorSettings* Settings = UAvaRundownEditorSettings::Get();
		if (Settings && Settings->bAutoStartPreviewBroadcastChannel)
		{
			UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
			if (Broadcast.GetCurrentProfile().GetChannel(InPagePlayer->ChannelFName).IsValidChannel())
			{
				Broadcast.ConditionalStartBroadcastChannel(InPagePlayer->ChannelFName);
			}
			else if (InPagePlayer->ChannelFName != UE::AvaMedia::Playback::DefaultPreviewChannelName)
			{
				UE_LOGF(LogAvaBroadcast, Warning,
					"Start Broadcast failed: Channel \"%ls\" of Profile \"%ls\" is not valid.",
					*InPagePlayer->ChannelFName.ToString(), *GetCurrentProfileName().ToString());
			}
		}
	}
}

void FAvaRundownEditor::OnCanClosePlaybackContext(const UAvaRundown* InRundown, bool& bOutResult) const
{
	const UAvaRundown* Rundown = AvaRundown.Get();
	if (Rundown != nullptr && InRundown == Rundown)
	{
		bOutResult = false;
	}
}

TSharedPtr<SDockTab> FAvaRundownEditor::CreateSubListTab(const FAvaRundownPageListReference& InSubListReference)
{
	const TSharedPtr<FApplicationMode> AppMode = GetCurrentModePtr();

	if (AppMode.IsValid())
	{
		const FName TabId = FAvaRundownSubListDocumentTabFactory::GetTabId(InSubListReference);
		const TSharedRef<FAvaRundownAppMode> RundownAppMode = StaticCastSharedRef<FAvaRundownAppMode>(AppMode.ToSharedRef());
		TSharedPtr<FDocumentTabFactory> DocTabFactory = RundownAppMode->GetDocumentTabFactory(TabId);

		if (!DocTabFactory.IsValid())
		{
			DocTabFactory = MakeShared<FAvaRundownSubListDocumentTabFactory>(InSubListReference, SharedThis(this));
			RundownAppMode->RegisterDocumentTabFactory(DocTabFactory, TabManager);
		}

		return TabManager->TryInvokeTab(TabId);
	}

	return nullptr;
}

void FAvaRundownEditor::RefreshSubListTab(const FAvaRundownPageListReference& InSubListReference, bool bInSetActive)
{
	if (InSubListReference.Type != EAvaRundownPageListType::View)
	{
		return;
	}

	const UAvaRundown* Rundown = AvaRundown.Get();
	if (!IsValid(Rundown))
	{
		return;
	}

	const FName TabId = FAvaRundownSubListDocumentTabFactory::GetTabId(InSubListReference);

	TSharedPtr<SDockTab> SubListTab = TabManager->FindExistingLiveTab(TabId);

	if (Rundown->IsValidSubList(InSubListReference))
	{
		if (!SubListTab.IsValid())
		{
			SubListTab = CreateSubListTab(InSubListReference);
		}

		if (SubListTab.IsValid() && bInSetActive)
		{
			SubListTab->ActivateInParent(ETabActivationCause::SetDirectly);
			SubListTab->DrawAttention();
		}
	}
	else
	{
		if (SubListTab.IsValid())
		{
			SubListTab->RequestCloseTab();
			UnregisterDocumentTabFactory(TabId);
		}
	}
}

void FAvaRundownEditor::RegisterApplicationModes()
{
	TArray<TSharedRef<FApplicationMode>> ApplicationModes;
	TSharedPtr<FAvaRundownEditor> This = SharedThis(this);

	ApplicationModes.Add(MakeShared<FAvaRundownDefaultMode>(This));
	//Can add more App Modes here

	for (const TSharedRef<FApplicationMode>& AppMode : ApplicationModes)
	{
		AddApplicationMode(AppMode->GetModeName(), AppMode);
	}

	SetCurrentMode(FAvaRundownDefaultMode::DefaultMode);
}

void FAvaRundownEditor::CreateRundownCommands()
{
	//Rundown Commands
	{
		const FAvaRundownCommands& RundownCommands = FAvaRundownCommands::Get();

		ToolkitCommands->MapAction(RundownCommands.Play,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::PlaySelectedPage),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanPlaySelectedPage));

		ToolkitCommands->MapAction(RundownCommands.UpdateValues,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::UpdateValuesOnSelectedPage),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanUpdateValuesOnSelectedPage));
		
		ToolkitCommands->MapAction(RundownCommands.Stop,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::StopSelectedPage, false),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanStopSelectedPage, false));

		ToolkitCommands->MapAction(RundownCommands.ForceStop,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::StopSelectedPage, true),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanStopSelectedPage, true));

		ToolkitCommands->MapAction(RundownCommands.Continue,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::ContinueSelectedPage),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanContinueSelectedPage));

		ToolkitCommands->MapAction(RundownCommands.PlayNext,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::PlayNextPage),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanPlayNextPage));

		ToolkitCommands->MapAction(RundownCommands.LoadAllPages,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::LoadAllPages),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanLoadAllPages));

		ToolkitCommands->MapAction(RundownCommands.LoadNextPages,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::LoadNextPages),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanLoadNextPages));

		ToolkitCommands->MapAction(RundownCommands.LoadSelectedPages,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::LoadSelectedPages),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanLoadSelectedPages));

		ToolkitCommands->MapAction(RundownCommands.PreviewFrame,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::PreviewPlaySelectedPage, true),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanPreviewPlaySelectedPage));

		ToolkitCommands->MapAction(RundownCommands.PreviewPlay,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::PreviewPlaySelectedPage, false),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanPreviewPlaySelectedPage));

		ToolkitCommands->MapAction(RundownCommands.PreviewContinue,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::PreviewContinueSelectedPage),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanPreviewContinueSelectedPage));

		ToolkitCommands->MapAction(RundownCommands.PreviewStop,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::PreviewStopSelectedPage, false),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanPreviewStopSelectedPage, false));

		ToolkitCommands->MapAction(RundownCommands.PreviewForceStop,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::PreviewStopSelectedPage, true),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanPreviewStopSelectedPage, true));

		ToolkitCommands->MapAction(RundownCommands.PreviewPlayNext,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::PreviewPlayNextPage),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanPreviewPlayNextPage));

		ToolkitCommands->MapAction(RundownCommands.TakeToProgram,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::TakeToProgram),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanTakeToProgram));

		ToolkitCommands->MapAction(RundownCommands.AddTemplate,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::AddTemplate),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanAddTemplate));

		ToolkitCommands->MapAction(RundownCommands.CreatePageInstanceFromTemplate,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::CreateInstancesFromSelectedTemplates),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanCreateInstancesFromSelectedTemplates));

		ToolkitCommands->MapAction(RundownCommands.RemovePage,
			FExecuteAction::CreateSP(this, &FAvaRundownEditor::RemoveSelectedPages),
			FCanExecuteAction::CreateSP(this, &FAvaRundownEditor::CanRemoveSelectedPages));
	}
}

FText FAvaRundownEditor::GetCurrentProfileName() const
{
	return FText::FromName(UAvaBroadcast::Get().GetCurrentProfileName());
}

TSharedRef<SWidget> FAvaRundownEditor::MakeProfileComboButton()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	TArray<FName> ProfileNames = UAvaBroadcast::Get().GetProfileNames();
	for (FName ProfileName : ProfileNames)
	{
		MenuBuilder.AddMenuEntry(FText::FromName(ProfileName)
			, FText::GetEmpty()
			, FSlateIcon()
			, FUIAction(FExecuteAction::CreateLambda([ProfileName](){ UAvaBroadcast::Get().SetCurrentProfile(ProfileName);}))
		);
	}

	return MenuBuilder.MakeWidget();
}

void FAvaRundownEditor::StartAutoPlayCommand(const TArray<FString>& InArgs)
{
	const double PlayInterval = (InArgs.Num() > 0) ? FCString::Atod(*InArgs[0]) : FAutoPlayTicker::DefaultPlayInterval;
	UE_LOGF(LogAvaRundown, Log, "Rundown auto play started, interval: %f seconds.", PlayInterval);
	AutoPlayTicker = MakeUnique<FAutoPlayTicker>(SharedThis(this), PlayInterval);
}

void FAvaRundownEditor::StopAutoPlayCommand(const TArray<FString>& InArgs)
{
	CancelAutoPlay();
}

void FAvaRundownEditor::CancelAutoPlay()
{
	if (AutoPlayTicker && !AutoPlayTicker->bIsCancelled)
	{
		// Mark as cancelled.
		AutoPlayTicker->bIsCancelled = true;
		// We can't delete the object immediately because it might still be ticking so we defer with an async task.
		TWeakPtr<FAvaRundownEditor> RundownEditorWeak(SharedThis(this));
		AsyncTask(ENamedThreads::GameThread, [RundownEditorWeak]()
		{
			if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
			{
				RundownEditor->AutoPlayTicker.Reset();
			}
		});
	}
}

void FAvaRundownEditor::LoadPageCommand(const TArray<FString>& InArgs)
{
	FString Errors;
	const TArray<int32> PageIds = ArgumentsToPageIds(InArgs, /*bInPreview*/ false, Errors);

	if (!Errors.IsEmpty())
	{
		UE_LOGF(LogAvaRundown, Error, "Errors in Load Page command: %ls", *Errors);
	}

	UAvaRundown* Rundown = AvaRundown.Get();

	// Log the status of the operation.
	auto LoadPage = [Rundown](int32 InPageId, bool bInIsPreview, FName InChannelName)
	{
		if (Rundown->GetPageLoadingManager().RequestLoadPage(InPageId, bInIsPreview, InChannelName))
		{
			UE_LOGF(LogAvaRundown, Display, "Loaded page %d for channel \"%ls\".", InPageId, *InChannelName.ToString());
		}
	};

	const FName PreviewChannelName = UAvaRundown::GetDefaultPreviewChannelName();
	
	for (const int32 PageId : PageIds)
	{
		const FAvaRundownPage& Page =  AvaRundown->GetPage(PageId);
		if (Page.IsValidPage())
		{
			if (!Page.IsTemplate())
			{
				LoadPage(PageId, false, Page.GetChannelName());
			}
			LoadPage(PageId, true, PreviewChannelName);
		}
	}
}

void FAvaRundownEditor::UnloadPageCommand(const TArray<FString>& InArgs)
{
	FString Errors;
	const TArray<int32> PageIds = ArgumentsToPageIds(InArgs, /*bInPreview*/ false, Errors);

	if (!Errors.IsEmpty())
	{
		UE_LOGF(LogAvaRundown, Error, "Errors in Unload Page command: %ls", *Errors);
	}

	UAvaRundown* Rundown = AvaRundown.Get();

	// Log the status of the operation.
	auto UnloadPage = [Rundown](int32 InPageId, const FName& InChannelName)
	{
		if (Rundown->UnloadPage(InPageId, InChannelName.ToString()))
		{
			UE_LOGF(LogAvaRundown, Display, "Unloaded page %d for channel \"%ls\".", InPageId, *InChannelName.ToString());
		}
	};

	const FName PreviewChannelName = UAvaRundown::GetDefaultPreviewChannelName();
	
	for (const int32 PageId : PageIds)
	{
		const FAvaRundownPage& Page =  AvaRundown->GetPage(PageId);
		if (Page.IsValidPage())
		{
			if (!Page.IsTemplate())
			{
				UnloadPage(PageId, Page.GetChannelName());
			}
			UnloadPage(PageId, PreviewChannelName);
		}
	}
}

void FAvaRundownEditor::PlayPageCommand(const TArray<FString>& InArgs, EAvaRundownPagePlayType InPlayType)
{
	const bool bPreview = UE::AvaRundown::IsPreviewPlayType(InPlayType);

	FString Errors;
	const TArray<int32> PageIds = ArgumentsToPageIds(InArgs, bPreview, Errors);

	if (!Errors.IsEmpty())
	{
		UE_LOGF(LogAvaRundown, Error, "Errors in Play Page command: %ls", *Errors);
	}

	UAvaRundown* Rundown = AvaRundown.Get();

	if (Rundown && !PageIds.IsEmpty())
	{
		Rundown->PlayPages(PageIds, InPlayType, EAvaRundownPagePlaySource::Editor);
	}
}

void FAvaRundownEditor::ContinuePageCommand(const TArray<FString>& InArgs, bool bInPreview)
{
	FString Errors;	
	const TArray<int32> PageIds = ArgumentsToPageIds(InArgs, bInPreview, Errors);

	if (!Errors.IsEmpty())
	{
		UE_LOGF(LogAvaRundown, Error, "Errors in Continue Page command: %ls", *Errors);
	}

	UAvaRundown* Rundown = AvaRundown.Get();

	for (const int32 PageId : PageIds)
	{
		if (Rundown->CanContinuePage(PageId, bInPreview))
		{
			Rundown->ContinuePage(PageId, bInPreview);
		}
	}
}

void FAvaRundownEditor::PlayNextPageCommand(const TArray<FString>& InArgs, bool bInPreview)
{
	UAvaRundown* Rundown = AvaRundown.Get();
	if (!AvaRundown.IsValid())
	{
		UE_LOGF(LogAvaRundown, Error, "Errors in Continue Page command: invalid rundown.");
		return;
	}

	const int32 NextPageId = FAvaRundownPlaybackUtils::GetPageIdToPlayNext(
		Rundown, UAvaRundown::InstancePageList, bInPreview, bInPreview ? UAvaRundown::GetDefaultPreviewChannelName() : NAME_None);

	if (FAvaRundownPlaybackUtils::IsPageIdValid(NextPageId))
	{
		Rundown->PlayPage(NextPageId, bInPreview ? EAvaRundownPagePlayType::PreviewFromFrame : EAvaRundownPagePlayType::PlayFromStart, EAvaRundownPagePlaySource::Editor);
	}
}

void FAvaRundownEditor::StopPageCommand(const TArray<FString>& InArgs, bool bInPreview, bool bInForce)
{
	FString Errors;
	const TArray<int32> PageIds = ArgumentsToPageIds(InArgs, bInPreview, Errors);

	if (!Errors.IsEmpty())
	{
		UE_LOGF(LogAvaRundown, Error, "Errors in \"Stop Page\" command: %ls", *Errors);
	}

	UAvaRundown* Rundown = AvaRundown.Get();
	if (!Rundown)
	{
		return;
	}

	if (Rundown && !PageIds.IsEmpty())
	{
		Rundown->StopPages(PageIds, bInForce ? EAvaRundownPageStopOptions::ForceNoTransition : EAvaRundownPageStopOptions::Default, bInPreview);
	}
}

void FAvaRundownEditor::StopLayersCommand(const TArray<FString>& InArgs, bool bInPreview, bool bInForce)
{
	// Validate arguments
	if (InArgs.Num() < 2)
	{
		UE_LOGF(LogAvaRundown, Error, "Errors in \"Stop Layers\" command: Too few arguments, need at least ChannelName and one layer name");
		return;
	}

	const FName ChannelName(*InArgs[0]);

	bool bAnyChannel = false;
	if (InArgs[0] == TEXT("all") || InArgs[0] == TEXT("any"))
	{
		bAnyChannel = true;
	}
	
	UAvaRundown* Rundown = AvaRundown.Get();
	if (!Rundown)
	{
		return;
	}

	TArray<FName> LayerNames;
	LayerNames.Reserve(InArgs.Num());
	for (const FString& Arg : InArgs)
	{
		LayerNames.Add(FName(*Arg));
	}
	LayerNames.RemoveAtSwap(0);

	// Using a map to gather layers per channel.
	TMap<FName, TArray<FAvaTagHandle>> AllLayers;
	
	// Search for the layers in specified channel
	for (const UAvaRundownPagePlayer* PagePlayer : Rundown->GetPagePlayers())
	{
		if (!PagePlayer
			|| PagePlayer->bIsPreview != bInPreview
			|| (!bAnyChannel && PagePlayer->ChannelFName != ChannelName))
		{
			continue;
		}

		TArray<FAvaTagHandle>& Layers = AllLayers.FindOrAdd(PagePlayer->ChannelFName);

		PagePlayer->ForEachInstancePlayer([&Layers, &LayerNames](const UAvaRundownPlaybackInstancePlayer* InInstancePlayer)
		{
			if (LayerNames.Contains(InInstancePlayer->TransitionLayer.ToName()))
			{
				FAvaRundownPlaybackUtils::AddTagHandleUnique(Layers, InInstancePlayer->TransitionLayer);
			}
		});
	}

	const EAvaRundownPageStopOptions StopOptions = bInForce ? EAvaRundownPageStopOptions::ForceNoTransition : EAvaRundownPageStopOptions::Default;

	for (TPair<FName, TArray<FAvaTagHandle>>& Layers : AllLayers)
	{
		UE_LOGF(LogAvaRundown, Log, "Stopping layers: %ls in channel %ls.", *FString::JoinBy(Layers.Value, TEXT(", "), [](const FAvaTagHandle& InTag){return InTag.ToString();}), *Layers.Key.ToString());
		Rundown->StopLayers(Layers.Key, Layers.Value, StopOptions);
	}
}

void FAvaRundownEditor::TakeToProgramCommand(const TArray<FString>& InArgs)
{
	FString Errors;
	const TArray<int32> PageIds = ArgumentsToPageIds(InArgs, /*bInPreview*/ true, Errors);

	if (!Errors.IsEmpty())
	{
		UE_LOGF(LogAvaRundown, Error, "Errors in \"Take To Program\" command: %ls", *Errors);
	}

	UAvaRundown* Rundown = AvaRundown.Get();
	if (!Rundown)
	{
		return;
	}

	const TArray<int32> PageIdsToPlay = FAvaRundownPlaybackUtils::GetPagesToTakeToProgram(Rundown, PageIds);

	if (!PageIdsToPlay.IsEmpty())
	{
		Rundown->PlayPages(PageIdsToPlay, EAvaRundownPagePlayType::PlayFromStart, EAvaRundownPagePlaySource::Editor);
	}
}

void FAvaRundownEditor::StopPageTransitionCommand(const TArray<FString>& InArgs)
{
	UAvaRundown* Rundown = AvaRundown.Get();
	if (!Rundown)
	{
		UE_LOGF(LogAvaRundown, Error, "No valid rundown available.");
		return;
	}

	const UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	TArray<FString> ChannelNames;
	ChannelNames.Reserve(Broadcast.GetChannelNameCount());
	for (int32 ChannelIndex = 0; ChannelIndex < Broadcast.GetChannelNameCount(); ++ChannelIndex)
	{
		ChannelNames.Add(Broadcast.GetChannelName(ChannelIndex).ToString());
	}
	
	// Arguments have to be channel names, but we support partial name match.
	for (const FString& Argument : InArgs)
	{
		for (const FString& ChannelName : ChannelNames)
		{
			if (ChannelName == Argument || ChannelName.Contains(Argument))
			{
				const int32 NumTransitionsStopped = Rundown->StopPageTransitionsForChannel(FName(*ChannelName)); 
				if (NumTransitionsStopped > 0)
				{
					UE_LOGF(LogAvaRundown, Log, "Stopped %d transition(s) on channel %ls.", NumTransitionsStopped, *ChannelName);
				}
			}
		}
	}
}

void FAvaRundownEditor::StartChannelCommand(const TArray<FString>& InArgs)
{
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	if (InArgs.IsEmpty() || InArgs[0] == TEXT("all"))
	{
		Broadcast.StartBroadcast();
		return;
	}
	
	for (const FString& Argument : InArgs)
	{
		FAvaBroadcastOutputChannel& Channel = Broadcast.GetCurrentProfile().GetChannelMutable(FName(Argument));
		if (Channel.IsValidChannel())
		{
			Channel.StartChannelBroadcast();
		}
		else
		{
			UE_LOGF(LogAvaRundown, Error, "Argument \"%ls\" is not a valid channel name.", *Argument);
		}
	}
}

void FAvaRundownEditor::StopChannelCommand(const TArray<FString>& InArgs)
{
	UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
	if (InArgs.IsEmpty() || InArgs[0] == TEXT("all"))
	{
		Broadcast.StopBroadcast();
		return;
	}
	
	for (const FString& Argument : InArgs)
	{
		FAvaBroadcastOutputChannel& Channel = Broadcast.GetCurrentProfile().GetChannelMutable(FName(Argument));
		if (Channel.IsValidChannel())
		{
			Channel.StopChannelBroadcast();
		}
		else
		{
			UE_LOGF(LogAvaRundown, Error, "Argument \"%ls\" is not a valid channel name.", *Argument);
		}
	}
}

TArray<int32> FAvaRundownEditor::ArgumentsToPageIds(const TArray<FString>& InArgs, bool bInPreview, FString& OutErrors) const
{
	if (!AvaRundown.IsValid())
	{
		OutErrors = TEXT("invalid rundown.");
		return {};
	}

	UAvaRundown* Rundown = AvaRundown.Get();

	TSet<int32> PageIds;

	for (const FString& Arg : InArgs)
	{
		if (Arg.IsNumeric())
		{
			const int32 PageId = FCString::Atoi(*Arg);
			const FAvaRundownPage& Page =  Rundown->GetPage(PageId);
			if (!Page.IsValidPage())
			{
				OutErrors += FString::Printf(TEXT("argument \"%s\" is not a valid page Id. "), *Arg);
				continue;
			}
			PageIds.Add(PageId);
		}
		else if (Arg == TEXT("all"))	// all pages in the list.
		{
			for (const FAvaRundownPage& Page : Rundown->GetInstancedPages().Pages)
			{
				PageIds.Add(Page.GetPageId());
			}
			for (const FAvaRundownPage& Page : Rundown->GetTemplatePages().Pages)
			{
				PageIds.Add(Page.GetPageId());
			}
		}
		else if (Arg == TEXT("any"))	// any playing pages
		{
			if (bInPreview)
			{
				PageIds.Append(Rundown->GetPreviewingPageIds());
			}
			else
			{
				PageIds.Append(Rundown->GetPlayingPageIds());
			}
		}
		else if (Arg == TEXT("selected")) // selected pages for the current list (in focus)
		{
			PageIds.Append(GetSelectedPagesOnFocusedWidget());
		}
		else if (UAvaBroadcast::Get().GetChannelIndex(FName(Arg)) != INDEX_NONE)
		{
			// Any playing pages on the specified channel.
			if (bInPreview)
			{
				PageIds.Append(Rundown->GetPreviewingPageIds(FName(Arg)));
			}
			else
			{
				PageIds.Append(Rundown->GetPlayingPageIds(FName(Arg)));
			}
		}
		else
		{
			OutErrors += FString::Printf(TEXT("argument \"%s\" is not recognized. Supported arguments: pageId, channelName, \"all\" or \"any\". "), *Arg);
		}
	}
	return PageIds.Array();
}

const FAvaRundownEditor::FBindableMacroCommands& FAvaRundownEditor::GetBindableMacroCommands()
{
	if (BindableMacroCommands.IsEmpty())
	{
		using namespace UE::AvaRundownEditor::Private;
		BindableMacroCommands.Add(GetName(EAvaRundownEditorMacroCommand::LoadPage), [this](const TArray<FString>& InArgs){LoadPageCommand(InArgs);});
		BindableMacroCommands.Add(GetName(EAvaRundownEditorMacroCommand::UnloadPage), [this](const TArray<FString>& InArgs){UnloadPageCommand(InArgs);});
		BindableMacroCommands.Add(GetName(EAvaRundownEditorMacroCommand::TakeIn), [this](const TArray<FString>& InArgs){PlayPageCommand(InArgs, EAvaRundownPagePlayType::PlayFromStart);});
		BindableMacroCommands.Add(GetName(EAvaRundownEditorMacroCommand::TakeOut), [this](const TArray<FString>& InArgs){StopPageCommand(InArgs, /*bInPreview*/ false, /*bInForce*/ false);});
		BindableMacroCommands.Add(GetName(EAvaRundownEditorMacroCommand::ForceTakeOut), [this](const TArray<FString>& InArgs){StopPageCommand(InArgs, /*bInPreview*/ false, /*bInForce*/ true);});
		BindableMacroCommands.Add(GetName(EAvaRundownEditorMacroCommand::TakeNext), [this](const TArray<FString>& InArgs){PlayNextPageCommand(InArgs, /*bInPreview*/ false);});
		BindableMacroCommands.Add(GetName(EAvaRundownEditorMacroCommand::Continue), [this](const TArray<FString>& InArgs){ContinuePageCommand(InArgs, /*bInPreview*/ false);});
		BindableMacroCommands.Add(GetName(EAvaRundownEditorMacroCommand::PreviewIn), [this](const TArray<FString>& InArgs){PlayPageCommand(InArgs, EAvaRundownPagePlayType::PreviewFromStart);});
		BindableMacroCommands.Add(GetName(EAvaRundownEditorMacroCommand::PreviewOut), [this](const TArray<FString>& InArgs){StopPageCommand(InArgs, /*bInPreview*/ true, /*bInForce*/ false);});
		BindableMacroCommands.Add(GetName(EAvaRundownEditorMacroCommand::ForcePreviewOut), [this](const TArray<FString>& InArgs){StopPageCommand(InArgs, /*bInPreview*/ true, /*bInForce*/ true);});
		BindableMacroCommands.Add(GetName(EAvaRundownEditorMacroCommand::PreviewNext), [this](const TArray<FString>& InArgs){PlayNextPageCommand(InArgs, /*bInPreview*/ true);});
		BindableMacroCommands.Add(GetName(EAvaRundownEditorMacroCommand::ContinuePreview), [this](const TArray<FString>& InArgs){ContinuePageCommand(InArgs, /*bInPreview*/ true);});
		BindableMacroCommands.Add(GetName(EAvaRundownEditorMacroCommand::TakeToProgram), [this](const TArray<FString>& InArgs){TakeToProgramCommand(InArgs);});
		BindableMacroCommands.Add(GetName(EAvaRundownEditorMacroCommand::StopPageTransition), [this](const TArray<FString>& InArgs){StopPageTransitionCommand(InArgs);});
		BindableMacroCommands.Add(GetName(EAvaRundownEditorMacroCommand::StartChannel), [this](const TArray<FString>& InArgs){StartChannelCommand(InArgs);});
		BindableMacroCommands.Add(GetName(EAvaRundownEditorMacroCommand::StopChannel), [this](const TArray<FString>& InArgs){StopChannelCommand(InArgs);});
		BindableMacroCommands.Add(GetName(EAvaRundownEditorMacroCommand::PreviewFrame), [this](const TArray<FString>& InArgs){PlayPageCommand(InArgs, EAvaRundownPagePlayType::PreviewFromFrame);});
	}
	return BindableMacroCommands;
}

FAvaRundownEditor::FAutoPlayTicker::FAutoPlayTicker(TWeakPtr<FAvaRundownEditor> InRundownEditorWeak, double InTickInterval)
	: RundownEditorWeak(InRundownEditorWeak)
	, PlayInterval(InTickInterval)
{
	// Start current page or select and play the first page of the list.
	if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		UAvaRundown* Rundown = RundownEditor->GetRundown();

		if (Rundown && Rundown->GetInstancedPages().Pages.Num() > 0)
		{
			TMap<FName, int32> PageIdsToPlay = TMap<FName, int32>();

			if (const TSharedPtr<SAvaRundownInstancedPageList> ActivePageList = RundownEditor->GetActiveListWidget())
			{
				if (ActivePageList->GetPageViews().IsEmpty() == false)
				{
					if (ActivePageList->GetSelectedPageIds().IsEmpty())
					{
						const FAvaRundownPageViewPtr& FirstPage = ActivePageList->GetPageViews()[0];
						PageIdsToPlay.Add(Rundown->GetPage(FirstPage->GetPageId()).GetChannelName(), FirstPage->GetPageId());
						ActivePageList->SelectPage(FirstPage->GetPageId());
					}
					else
					{
						for (const int32 PageId : ActivePageList->GetSelectedPageIds())
						{
							if (PageIdsToPlay.Find(Rundown->GetPage(PageId).GetChannelName()) == nullptr)
							{
								PageIdsToPlay.Add(Rundown->GetPage(PageId).GetChannelName(), PageId);
							}
						}
					}
				}
			}

			if (PageIdsToPlay.IsEmpty())
			{
				// Just go direct to the instanced pages.
				PageIdsToPlay.Add(Rundown->GetInstancedPages().Pages[0].GetChannelName(), Rundown->GetInstancedPages().Pages[0].GetPageId());

				// And select it if possible.
				if (TSharedPtr<SAvaRundownPageList> MainPageList = RundownEditor->GetInstanceListWidget())
				{
					if (MainPageList->GetPageViews().IsEmpty() == false)
					{
						if (MainPageList->GetSelectedPageIds().IsEmpty())
						{
							MainPageList->SelectPage(Rundown->GetInstancedPages().Pages[0].GetPageId());
						}
						else
						{
							for (const int32 PageId : MainPageList->GetSelectedPageIds())
							{
								if (PageIdsToPlay.Find(Rundown->GetPage(PageId).GetChannelName()) == nullptr)
								{
									PageIdsToPlay.Add(Rundown->GetPage(PageId).GetChannelName(), PageId);
								}
							}
						}
					}
				}
			}

			for (const TPair<FName, int32> PageId : PageIdsToPlay)
			{
				Rundown->PlayPage(PageId.Value, EAvaRundownPagePlayType::PlayFromStart, EAvaRundownPagePlaySource::Editor);
			}
		}
	}
}

void FAvaRundownEditor::FAutoPlayTicker::Tick(float DeltaTime)
{
	const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin();
	if (!RundownEditor || bIsCancelled)
	{
		return;
	}

	const UAvaRundown* Rundown = RundownEditor->GetRundown();
	if (!Rundown)
	{
		UE_LOGF(LogAvaRundown, Error, "Invalid rundown. Cancelling auto play.");
		RundownEditor->CancelAutoPlay();
		return;
	}

	if (!bIsPagePlaying)
	{
		// Check the status of the page to see if it started playing.
		const TArray<int32> SelectedPagesId = RundownEditor->GetRundown()->GetPlayingPageIds();
		if (SelectedPagesId.IsEmpty())
		{
			return;
		}

		int32 NumberOfPagesPlaying = 0;
		for (const int32 SelectedPageId : SelectedPagesId)
		{
			const FAvaRundownPage* SelectedPage = (SelectedPageId != FAvaRundownPage::InvalidPageId) ? &Rundown->GetPage(SelectedPageId) : nullptr;

			if (SelectedPage && SelectedPage->IsValidPage())
			{
				const TArray<FAvaRundownChannelPageStatus> Statuses = SelectedPage->GetPageContextualStatuses(Rundown);

				for (const FAvaRundownChannelPageStatus& Status : Statuses)
				{
					if (Status.Status == EAvaRundownPageStatus::Playing)
					{
						// bIsPagePlaying = true;
						NumberOfPagesPlaying++;
						NextPageStartTime = FApp::GetCurrentTime() + PlayInterval;
						break;
					}
					else if (Status.Status == EAvaRundownPageStatus::Error)
					{
						UE_LOGF(LogAvaRundown, Error, "Playback error. Cancelling auto play.");
						RundownEditor->CancelAutoPlay();
						return;
					}
				}
			}
			else
			{
				UE_LOGF(LogAvaRundown, Error, "Current Page is Invalid. Cancelling auto play.");
				RundownEditor->CancelAutoPlay();
				return;
			}
		}
		bIsPagePlaying = (NumberOfPagesPlaying == SelectedPagesId.Num());
	}

	if (bIsPagePlaying && FApp::GetCurrentTime() > NextPageStartTime)
	{
		// This will request the page to play.
		if (const TSharedPtr<SAvaRundownInstancedPageList> ActivePageList = RundownEditor->GetActiveListWidget())
		{
			const TArray<int32> NextPages = ActivePageList->PlayNextPage();
			if (NextPages.Num() > 0)
			{
				ActivePageList->DeselectPages();
				ActivePageList->SelectPages(NextPages);
			}
		}

		// But we will wait for it to actually start before measuring play time.
		bIsPagePlaying = false;
	}
}

TStatId FAvaRundownEditor::FAutoPlayTicker::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FAutoPlayTicker, STATGROUP_Tickables);
}

void FAvaRundownEditor::FSharedConsoleCommands::RegisterEditor(const TSharedPtr<FAvaRundownEditor>& InRundownEditor)
{
	if (InRundownEditor.IsValid() && !InRundownEditor->SharedConsoleCommands.IsValid())
	{
		InRundownEditor->SharedConsoleCommands = GetSharedInstance();
		InRundownEditor->SharedConsoleCommands->RundownEditors.Add(InRundownEditor);
	}
}

void FAvaRundownEditor::FSharedConsoleCommands::UnregisterEditor(FAvaRundownEditor* InRundownEditor)
{
	if (InRundownEditor && InRundownEditor->SharedConsoleCommands.IsValid())
	{
		InRundownEditor->SharedConsoleCommands->RundownEditors.RemoveAll([InRundownEditor](const TWeakPtr<FAvaRundownEditor>& Element)
		{
			return !Element.IsValid() || Element.HasSameObject(InRundownEditor);
		});
		InRundownEditor->SharedConsoleCommands.Reset();
	}
}

TSharedPtr<FAvaRundownEditor::FSharedConsoleCommands> FAvaRundownEditor::FSharedConsoleCommands::GetSharedInstance()
{
	static TWeakPtr<FSharedConsoleCommands> GlobalInstanceWeak;
	if (GlobalInstanceWeak.IsValid())
	{
		return GlobalInstanceWeak.Pin();
	}
	TSharedPtr<FSharedConsoleCommands> NewInstance = MakeShared<FSharedConsoleCommands>(FPrivateToken());
	GlobalInstanceWeak = NewInstance;
	return NewInstance;
}

void FAvaRundownEditor::FSharedConsoleCommands::RegisterConsoleCommands()
{
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("MotionDesignRundown.StartAutoPlay"),
		TEXT("Starts auto play of the currently active rundown editor."),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FSharedConsoleCommands::StartAutoPlayCommand),
		ECVF_Default
	));
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("MotionDesignRundown.StopAutoPlay"),
		TEXT("Stops auto play of the currently active rundown editor."),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FSharedConsoleCommands::StopAutoPlayCommand),
		ECVF_Default
	));
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("MotionDesignRundown.LoadPage"),
		TEXT("Preload the specified page from the current rundown to memory."),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FSharedConsoleCommands::LoadPageCommand),
		ECVF_Default
	));
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("MotionDesignRundown.UnloadPage"),
		TEXT("Unload the specified page from the current rundown."),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FSharedConsoleCommands::UnloadPageCommand),
		ECVF_Default
	));
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("MotionDesignRundown.StopLayersPreview"),
		TEXT("Stops currently previewing layers."),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FSharedConsoleCommands::StopLayersPreviewCommand),
		ECVF_Default
	));
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("MotionDesignRundown.ForceStopLayersPreview"),
		TEXT("Stops currently previewing layers."),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FSharedConsoleCommands::ForceStopLayersPreviewCommand),
		ECVF_Default
	));
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("MotionDesignRundown.StopLayersProgram"),
		TEXT("Stops currently playing layers."),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FSharedConsoleCommands::StopLayersProgramCommand),
		ECVF_Default
	));
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("MotionDesignRundown.ForceStopLayersProgram"),
		TEXT("Stops currently playing layers."),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FSharedConsoleCommands::ForceStopLayersProgramCommand),
		ECVF_Default
	));
}

void FAvaRundownEditor::FSharedConsoleCommands::UnregisterConsoleCommands()
{
	for (IConsoleObject* ConsoleCmd : ConsoleCommands)
	{
		IConsoleManager::Get().UnregisterConsoleObject(ConsoleCmd);
	}
	ConsoleCommands.Empty();
}

TSharedPtr<FAvaRundownEditor> FAvaRundownEditor::FSharedConsoleCommands::GetActiveRundownEditor() const
{
	for (const TWeakPtr<FAvaRundownEditor>& RundownEditorWeak : RundownEditors)
	{
		TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin();
		if (RundownEditor && UE::AvaRundownEditor::Private::IsActiveTabPartOfEditor(*RundownEditor))
		{
			return RundownEditor;
		}
	}
	return TSharedPtr<FAvaRundownEditor>();
}

#undef LOCTEXT_NAMESPACE
