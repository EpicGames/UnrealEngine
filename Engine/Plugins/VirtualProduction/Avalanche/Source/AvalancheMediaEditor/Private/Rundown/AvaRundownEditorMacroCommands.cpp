// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownEditorMacroCommands.h"

#include "UObject/Class.h"

FName FAvaRundownEditorMacroCommands::GetCommandName(EAvaRundownEditorMacroCommand InCommand)
{
	return StaticEnum<EAvaRundownEditorMacroCommand>()->GetNameByValue(static_cast<int64>(InCommand));
}

FName FAvaRundownEditorMacroCommands::GetShortCommandName(EAvaRundownEditorMacroCommand InCommand)
{
	return FName(StaticEnum<EAvaRundownEditorMacroCommand>()->GetNameStringByValue(static_cast<int64>(InCommand)));
}