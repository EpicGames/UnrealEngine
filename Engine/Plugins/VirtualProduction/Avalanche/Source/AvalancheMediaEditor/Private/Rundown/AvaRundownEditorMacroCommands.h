// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaRundownEditorMacroCommands.generated.h"

/** Available macro commands for Ava rundown editor. */
UENUM()
enum class EAvaRundownEditorMacroCommand : uint8
{
	None,
	// Playback commands
	LoadPage,
	UnloadPage,
	TakeIn,
	TakeOut,
	ForceTakeOut,
	TakeNext,
	Continue,
	PreviewIn,
	PreviewOut,
	ForcePreviewOut,
	PreviewNext,
	ContinuePreview,
	TakeToProgram,
	StopPageTransition,
	StartChannel,
	StopChannel,
	PreviewFrame
};

struct FAvaRundownEditorMacroCommands
{
	static FName GetCommandName(EAvaRundownEditorMacroCommand InCommand);
	static FName GetShortCommandName(EAvaRundownEditorMacroCommand InCommand);
};
