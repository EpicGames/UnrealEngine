// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownEditorSettings.h"
#include "IAvaMediaModule.h"
#include "Misc/MessageDialog.h"

#define LOCTEXT_NAMESPACE "AvaRundownEditorSettings"

UAvaRundownEditorSettings::UAvaRundownEditorSettings()
{
	CategoryName = TEXT("Motion Design");
	SectionName  = TEXT("Rundown Editor");
}

const UAvaRundownEditorSettings* UAvaRundownEditorSettings::Get()
{
	return GetMutable();
}

UAvaRundownEditorSettings* UAvaRundownEditorSettings::GetMutable()
{
	UAvaRundownEditorSettings* const DefaultSettings = GetMutableDefault<UAvaRundownEditorSettings>();
	static bool bInitialized = false;
	if (!bInitialized)
	{
		bInitialized = true;
		DefaultSettings->SetFlags(RF_Transactional);
	}
	return DefaultSettings;
}

void UAvaRundownEditorSettings::PostEditChangeProperty(struct FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	// Check if server is auto-start.
	static const FName AutoStartRundownServerPropertyName = GET_MEMBER_NAME_CHECKED(UAvaRundownEditorSettings, bAutoStartRundownServer);

	if (InPropertyChangedEvent.MemberProperty->GetFName() == AutoStartRundownServerPropertyName)
	{
		IAvaMediaModule& AvaMediaModule = IAvaMediaModule::Get();
		if (bAutoStartRundownServer && !AvaMediaModule.IsRundownServerStarted())
		{
			const FText MessageText = LOCTEXT("StartRundownServerQuestion", "Do you want to start rundown server now?");
			const EAppReturnType::Type Reply = FMessageDialog::Open(EAppMsgType::YesNo, EAppReturnType::Yes, MessageText);

			if (Reply == EAppReturnType::Yes)
			{
				AvaMediaModule.StartRundownServer(RundownServerName);
			}
		}
		else if (!bAutoStartRundownServer && AvaMediaModule.IsRundownServerStarted())
		{
			const FText MessageText = LOCTEXT("StopRundownServerQuestion", "Rundown Server is currently running. Do you want to stop it now?");
			const EAppReturnType::Type Reply = FMessageDialog::Open(EAppMsgType::YesNo, EAppReturnType::Yes, MessageText);

			if (Reply == EAppReturnType::Yes)
			{
				AvaMediaModule.StopRundownServer();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
