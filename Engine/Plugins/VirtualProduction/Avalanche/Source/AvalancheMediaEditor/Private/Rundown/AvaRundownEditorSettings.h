// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Rundown/AvaRundownEditorDefines.h"
#include "AvaRundownEditorSettings.generated.h"

class UAvaRundownMacroCollection;

UCLASS(config=EditorPerProjectUserSettings, meta=(DisplayName="Rundown"))
class UAvaRundownEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UAvaRundownEditorSettings();

	static const UAvaRundownEditorSettings* Get();
	static UAvaRundownEditorSettings* GetMutable();

	//~ Begin UObject
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End UObject
	
	/**
	 * Configuring the default page action when closing the editor.
	 * By default it will stop the pages.
	 */
	UPROPERTY(Config, EditAnywhere, Category="Rundown")
	bool bShouldStopPagesOnClose = true;
	
	/** Shows the checker board in preview */
	UPROPERTY(Config, EditAnywhere, Category="Preview")
	bool bPreviewCheckerBoard = false;

	/** Automatically start the currently selected preview broadcast channel when a preview playback starts. */
	UPROPERTY(Config, EditAnywhere, Category="Preview")
	bool bAutoStartPreviewBroadcastChannel = true;

	/** Whether to show controller properties in Rundown Page Details */
	UPROPERTY(Config, EditAnywhere, Category = "Page Details", meta = (DisplayName = "Show RC Controlled Properties"))
	bool bPageDetailsShowProperties = false;

	/** Current macro collection used by the rundown editor. */
	UPROPERTY(Config, EditAnywhere, Category="Macros")
	TSoftObjectPtr<UAvaRundownMacroCollection> MacroCollection;
	
	UPROPERTY(Config, EditAnywhere, Category="Page Actions")
	EAvaRundownPageSet PreviewContinueActionPageSet = EAvaRundownPageSet::SelectedOrPlaying;

	UPROPERTY(Config, EditAnywhere, Category="Page Actions")
	EAvaRundownPageSet PreviewOutActionPageSet = EAvaRundownPageSet::SelectedOrPlaying;
	
	UPROPERTY(Config, EditAnywhere, Category="Page Actions")
	EAvaRundownPageSet ContinueActionPageSet = EAvaRundownPageSet::SelectedOrPlaying;

	UPROPERTY(Config, EditAnywhere, Category="Page Actions")
	EAvaRundownPageSet TakeOutActionPageSet = EAvaRundownPageSet::SelectedOrPlaying;

	UPROPERTY(Config, EditAnywhere, Category="Page Actions")
	EAvaRundownPageSet UpdateValuesActionPageSet = EAvaRundownPageSet::SelectedOrPlaying;

	/**
	 * Whether rundown server is started automatically when the editor is launched.
	 * For game mode or packaged games, the rundown server can be launched with
	 * the command line -MotionDesignRundownServerStart[=ServerName].
	 */
	UPROPERTY(Config, EditAnywhere, Category="Server")
	bool bAutoStartRundownServer = false;

	/** Name given to the rundown server. If empty, the server name will be the host name. */
	UPROPERTY(Config, EditAnywhere, Category="Server")
	FString RundownServerName;
};
