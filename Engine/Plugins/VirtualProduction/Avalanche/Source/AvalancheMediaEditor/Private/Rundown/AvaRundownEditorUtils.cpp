// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownEditorUtils.h"

#include "Backends/XmlStructSerializerBackend.h"
#include "ContentBrowserModule.h"
#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/FileManager.h"
#include "IAssetTools.h"
#include "IAvaMediaModule.h"
#include "IContentBrowserSingleton.h"
#include "JsonObjectConverter.h"
#include "Misc/PathViews.h"
#include "RCVirtualProperty.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownManagedInstanceCache.h"
#include "StructDeserializer.h"
#include "StructSerializer.h"

#define LOCTEXT_NAMESPACE "AvaRundownEditorUtils"

namespace UE::AvaRundownEditor::Utils::Private
{
	static FString LastExportPath;
	static FString LastImportPath;
	
	static const FString PageEntriesName = TEXT("Pages");

	// Don't serialize the transient properties.
	static auto TransientPropertyFilter = [](const FProperty* InCurrentProp, const FProperty* InParentProp)
	{
		const bool bIsTransient = InCurrentProp && InCurrentProp->HasAnyPropertyFlags(CPF_Transient); 
		return !bIsTransient; 
	};

	struct FRundownSerializerPolicies : public FStructSerializerPolicies
	{
		FRundownSerializerPolicies()
		{
			PropertyFilter = TransientPropertyFilter;
		}
	};

	struct FRundownDeserializerPolicies : public FStructDeserializerPolicies
	{
		FRundownDeserializerPolicies()
		{
			PropertyFilter = TransientPropertyFilter;
		}
	};
	
	TArray<TSharedPtr<FJsonValue>> PagesToJsonObjects(UAvaRundown* InRundown, const TArray<int32>& InPageIds)
	{
		check(InRundown);
		TArray<TSharedPtr<FJsonValue>> PageEntries;
		PageEntries.Reserve(InPageIds.Num());

		for (const int32 PageId : InPageIds)
		{
			const FAvaRundownPage& Page = InRundown->GetPage(PageId);

			if (Page.IsValidPage())
			{
				TSharedRef<FJsonObject> PageObject = MakeShared<FJsonObject>();
				FJsonObjectConverter::UStructToJsonObject(FAvaRundownPage::StaticStruct(), &Page, PageObject, 0 /* CheckFlags */, 0 /* SkipFlags */);
				PageEntries.Add(MakeShared<FJsonValueObject>(PageObject));
			}
		}

		return PageEntries;
	}

	const FAvaRundownPage& FindPage(const TArray<FAvaRundownPage>& InPages, int32 InPageIdToFind)
	{
		for (const FAvaRundownPage& Page : InPages)
		{
			if (Page.GetPageId() == InPageIdToFind)
			{
				return Page;
			}
		}
		return FAvaRundownPage::NullPage;
	}
	
	const FAvaRundownPage& FindTemplateForSourcePage(const UAvaRundown* InRundown, const FAvaRundownPage& InSourcePage,
		const TArray<FAvaRundownPage>& InSourceTemplates, FImportTemplateMap& InOutImportedTemplateIds)
	{
		{
			// Check if the template is already imported/existing at the given TemplateId.
			const FAvaRundownPage& ExistingTemplate = InRundown->GetPage(InOutImportedTemplateIds.GetTemplateId(InSourcePage.GetTemplateId()));
			if (ExistingTemplate.IsValidPage() && ExistingTemplate.IsTemplate() && ExistingTemplate.GetAssetPathDirect() == InSourcePage.GetAssetPathDirect())
			{
				return ExistingTemplate;
			}
		}

		// Fallback: Try to find a match using the source template if available.
		const FAvaRundownPage& SourceTemplate = FindPage(InSourceTemplates, InSourcePage.GetTemplateId());
		if (SourceTemplate.IsValidPage())
		{
			// Try to find that template in the rundown with an exact match (rc values, asset, etc).
			const FAvaRundownPageCollection& PageCollection = InRundown->GetTemplatePages();
			for (const FAvaRundownPage& ExistingTemplate : PageCollection.Pages)
			{
				if (ExistingTemplate.IsTemplateMatchingByValue(SourceTemplate))
				{
					// Keep track of the match we made for next time.
					InOutImportedTemplateIds.Add(InSourcePage.GetTemplateId(), ExistingTemplate.GetPageId());
					return ExistingTemplate;
				}
			}
		}
		
		return FAvaRundownPage::NullPage;
	}

	bool CopyPageInPlace(UAvaRundown* InRundown, int32 InPageId, const FAvaRundownPage& InSourcePage, int32 InTemplateId)
	{
		FAvaRundownPage& DestinationPage = InRundown->GetPage(InPageId);
		if (DestinationPage.IsValidPage())
		{
			DestinationPage = InSourcePage;
			DestinationPage.SetPageId(InPageId);		// Restore page id.
			DestinationPage.SetTemplateId(InTemplateId);
			return true;
		}
		UE_LOGF(LogAvaRundown, Error, "Failed to copy page in place: page id %d is not found in destination rundown.", InPageId);
		return false;
	}
}

FString UE::AvaRundownEditor::Utils::GetImportFilepath(const TCHAR* InFileDescription, const TCHAR* InExtension)
{
	// Reference: UAssetToolsImpl::ExportAssetsInternal.
	TArray<FString> OpenFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bOpened = false;
	if (DesktopPlatform)
	{
		if (Private::LastImportPath.IsEmpty())
		{
			Private::LastImportPath = FPaths::ProjectSavedDir();
		}
		
		const FString FileType = FString::Printf(TEXT("%s (*.%s)|*.%s"), InFileDescription, InExtension, InExtension);
		
		bOpened = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			FText::Format(LOCTEXT("Import_F", "Import {0}"), FText::FromString(InFileDescription)).ToString(),
			*Private::LastImportPath,
			TEXT(""),
			*FileType,
			EFileDialogFlags::None,
			OpenFilenames
		);
	}
	if (bOpened && OpenFilenames.Num() > 0 && OpenFilenames[0].IsEmpty() == false)
	{
		Private::LastImportPath = OpenFilenames[0];
		return OpenFilenames[0];
	}
	return FString();
}

FString UE::AvaRundownEditor::Utils::GetExportFilepath(const UObject* InObjectToExport, const TCHAR* InFileDescription, const TCHAR* InExtension)
{
	// Reference: UAssetToolsImpl::ExportAssetsInternal.
	TArray<FString> SaveFilenames;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bSave = false;
	if (DesktopPlatform)
	{
		if (Private::LastExportPath.IsEmpty())
		{
			Private::LastExportPath = FPaths::ProjectSavedDir();
		}
		
		const FString FileType = FString::Printf(TEXT("%s (*.%s)|*.%s"), InFileDescription, InExtension, InExtension);
		
		bSave = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			FText::Format(LOCTEXT("Export_F", "Export: {0}"), FText::FromString(InObjectToExport->GetName())).ToString(),
			*Private::LastExportPath,
			*InObjectToExport->GetName(),
			*FileType,
			EFileDialogFlags::None,
			SaveFilenames
		);
	}
	if (bSave && SaveFilenames.Num() > 0)
	{
		Private::LastExportPath = SaveFilenames[0];
		return SaveFilenames[0];
	}
	return FString();
}

FString UE::AvaRundownEditor::Utils::GetSaveAssetAsPath(const FString& InDefaultPath, const FString& InDefaultAssetName)
{
	FSaveAssetDialogConfig SaveAssetDialogConfig;
	{
		SaveAssetDialogConfig.DefaultPath = InDefaultPath;
		SaveAssetDialogConfig.DefaultAssetName = InDefaultAssetName;
		SaveAssetDialogConfig.AssetClassNames.Add(UAvaRundown::StaticClass()->GetClassPathName());
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
	}
	
	const FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	return ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
}

FString UE::AvaRundownEditor::Utils::SerializePagesToJson(UAvaRundown* InRundown, const TArray<int32>& InPageIds)
{
	if (!IsValid(InRundown))
	{
		return FString();
	}
	
	const TArray<TSharedPtr<FJsonValue>> PageEntries = Private::PagesToJsonObjects(InRundown, InPageIds);
	
	if (PageEntries.IsEmpty())
	{
		return FString();
	}

	const TSharedRef<FJsonObject> RootJsonObject = MakeShared<FJsonObject>();
	RootJsonObject->SetArrayField(Private::PageEntriesName, PageEntries);

	FString SerializedString;
	const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&SerializedString);

	FJsonSerializer::Serialize(RootJsonObject, Writer);
	
	return SerializedString;
}

TArray<FAvaRundownPage> UE::AvaRundownEditor::Utils::DeserializePagesFromJson(const FString& InJsonString)
{
	TSharedPtr<FJsonObject> RootJsonObject;
	const TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(InJsonString);

	if (!FJsonSerializer::Deserialize(Reader, RootJsonObject))
	{
		UE_LOGF(LogAvaRundown, Warning, "Unable to serialize the pasted text into Json format");
		return {};
	}

	const TArray<TSharedPtr<FJsonValue>>* PageEntries;
	if (!RootJsonObject->TryGetArrayField(Private::PageEntriesName, PageEntries))
	{
		UE_LOGF(LogAvaRundown, Warning, "Missing %ls entry field in pasted text", *Private::PageEntriesName);
		return {};
	}

	TArray<FAvaRundownPage> Pages;
	Pages.Reserve(PageEntries->Num());
	for (const TSharedPtr<FJsonValue>& PageEntry : *PageEntries)
	{
		if (!PageEntry.IsValid() || PageEntry->Type != EJson::Object)
		{
			UE_LOGF(LogAvaRundown, Warning, "Invalid page entry. Not an object");
			continue;
		}

		const TSharedPtr<FJsonObject>& PageObject = PageEntry->AsObject();
		check(PageObject.IsValid());

		FAvaRundownPage Page;
		if (FJsonObjectConverter::JsonObjectToUStruct(PageObject.ToSharedRef(), FAvaRundownPage::StaticStruct(), &Page, 0 /* CheckFlags */, 0 /* SkipFlags */))
		{
			Pages.Emplace(MoveTemp(Page));
		}
		else
		{
			UE_LOGF(LogAvaRundown, Warning, "Unable to convert Page Entry Json Object to Motion Design Page Struct");
		}
	}
	return Pages;
}

TStrongObjectPtr<UAvaRundown> UE::AvaRundownEditor::Utils::ExportPagesToRundown(const UAvaRundown* InRundown, const TArray<int32>& InPageIds)
{
	if (!InRundown || InPageIds.IsEmpty())
	{
		return nullptr;
	}

	TSet<int32> AddedTemplates;
	TArray<FAvaRundownPage> SourcePages;
	TArray<FAvaRundownPage> SourceTemplates;
	bool bExportTemplates = false;
	
	for (const int32 PageId : InPageIds)
	{
		const FAvaRundownPage& Page = InRundown->GetPage(PageId);
		if (!Page.IsValidPage())
		{
			continue;
		}

		if (Page.IsTemplate() && !AddedTemplates.Contains(Page.GetPageId()))
		{
			SourceTemplates.Add(Page);
			AddedTemplates.Add(Page.GetTemplateId());
			bExportTemplates = true; 
			continue;
		}
		
		SourcePages.Add(Page);
		if (!AddedTemplates.Contains(Page.GetTemplateId()))
		{
			const FAvaRundownPage& Template = InRundown->GetPage(Page.GetTemplateId());
			if (Template.IsValidPage())
			{
				SourceTemplates.Add(Template);
				AddedTemplates.Add(Template.GetPageId());
			}
		}
	}

	if (!SourcePages.IsEmpty())
	{
		const TStrongObjectPtr<UAvaRundown> NewRundown(NewObject<UAvaRundown>());
		FImportTemplateMap ImportedTemplateIds;
		
		if (bExportTemplates)
		{
			ImportTemplatePages(NewRundown.Get(), SourceTemplates, ImportedTemplateIds);
		}
		
		ImportInstancedPages(NewRundown.Get(), UAvaRundown::InstancePageList, SourcePages, SourceTemplates, ImportedTemplateIds);
		return NewRundown;
	}

	return nullptr;
}

bool UE::AvaRundownEditor::Utils::SaveRundownToXml(const UAvaRundown* InRundown, FArchive& InArchive, EXmlSerializationEncoding InXmlEncoding)
{
	if (IsValid(InRundown))
	{
		// Note: using the StructSerializerBackend produces a more compact format and is more suitable for exporting compared to
		// FXmlArchiveOutputFormatter. However, it doesn't support serializing UObject in place and the xml deserializer hasn't been
		// implemented yet. Support for serialization of UObjects in place is not planned to be needed for rundown at the moment.
		FXmlStructSerializerBackend Backend(InArchive, EStructSerializerBackendFlags::Default);
		FStructSerializer::Serialize(InRundown, *InRundown->GetClass(), Backend, Private::FRundownSerializerPolicies());
		Backend.SaveDocument(InXmlEncoding);
		return true;
	}
	return false;
}

bool UE::AvaRundownEditor::Utils::SaveRundownToXml(const UAvaRundown* InRundown, const TCHAR* InFilepath, EXmlSerializationEncoding InXmlEncoding)
{
	if (IsValid(InRundown))
	{
		const TUniquePtr<FArchive> FileWriter(IFileManager::Get().CreateFileWriter(InFilepath));
		if (FileWriter)	
		{
			const bool bSaved = SaveRundownToXml(InRundown, *FileWriter, InXmlEncoding);
			FileWriter->Close();
			return bSaved;
		}
	}
	return false;
}

bool UE::AvaRundownEditor::Utils::CanLoadRundownFromFile(const TCHAR* InFilepath)
{
	const FStringView Extension = FPathViews::GetExtension(InFilepath);
	
	// We only support loading from json file (LoadRundownFromJson).
	return Extension.Equals(TEXT("json"), ESearchCase::IgnoreCase);
}

TArray<int32> UE::AvaRundownEditor::Utils::ImportTemplatePages(UAvaRundown* InRundown, const TArray<FAvaRundownPage>& InSourceTemplates, FImportTemplateMap& OutImportedTemplateIds)
{
	if (!IsValid(InRundown))
	{
		return {};
	}

	TArray<int32> OutTemplateIds;
	OutTemplateIds.Reserve(InSourceTemplates.Num());

	for (const FAvaRundownPage& SourceTemplate : InSourceTemplates)
	{
		const int32 SourceTemplateId = SourceTemplate.IsTemplate() ? SourceTemplate.GetPageId() : SourceTemplate.GetTemplateId();
		// Try to add the template with the id it had in the original list.
		int32 ImportedTemplateId = InRundown->AddTemplate(FAvaRundownPageIdGeneratorParams(SourceTemplateId));
		
		if (ImportedTemplateId != FAvaRundownPage::InvalidPageId)
		{
			OutImportedTemplateIds.Add(SourceTemplateId, ImportedTemplateId);
			ensure(Private::CopyPageInPlace(InRundown, ImportedTemplateId, SourceTemplate, FAvaRundownPage::InvalidPageId));
			InRundown->GetPage(ImportedTemplateId).ResetInstancedIds();	// We know this template has no instances yet.
			OutTemplateIds.Add(ImportedTemplateId);
		}
	}
	return OutTemplateIds;
}

TArray<int32> UE::AvaRundownEditor::Utils::ImportInstancedPages(
	UAvaRundown* InRundown,
	const FAvaRundownPageListReference& InPageListReference,
	const TArray<FAvaRundownPage>& InSourcePages,
	const TArray<FAvaRundownPage>& InSourceTemplates,
	FImportTemplateMap& InOutImportedTemplateIds,
	const FAvaRundownPageInsertPosition& InInsertPosition)
{
	if (!IsValid(InRundown))
	{
		return {};
	}
	
	TArray<int32> OutPageIds;

	FAvaRundownPageInsertPosition InsertPosition = InInsertPosition;

	// If we are adding above, iteration should be reversed, so the last is added first
	// and the next to last added above that, etc.
	const bool bReverseIteration = InsertPosition.IsValid() && !InsertPosition.bAddBelow;
	
	for (int32 PageIndex = 0; PageIndex < InSourcePages.Num(); ++PageIndex)
	{
		const FAvaRundownPage& SourcePage = InSourcePages[bReverseIteration ? InSourcePages.Num() - PageIndex - 1 : PageIndex];
		
		if (!SourcePage.IsValidPage())
		{
			continue;
		}

		// Attempt to find/create template for this page
		int32 ImportedTemplateId = FAvaRundownPage::InvalidPageId;

		if (SourcePage.IsTemplate())
		{
			// Todo: untested case.
			// Suspect this of being wrong. if the page is a template, then it needs to be imported
			// as a template, i.e. ImportTemplatePages.
			ImportedTemplateId = SourcePage.GetPageId();
		}
		else
		{
			// The source page has a source template id. It may not match the destination, this is
			// why we rely on the ImportedTemplateIds to translate that. As a fallback, if the source
			// templates are provided, it will try to match templates by values.
			const FAvaRundownPage& ExistingTemplate = Private::FindTemplateForSourcePage(InRundown, SourcePage, InSourceTemplates, InOutImportedTemplateIds);

			if (ExistingTemplate.IsValidPage())
			{
				ImportedTemplateId = ExistingTemplate.GetPageId();
			}
			else
			{
				// Try to add the template with the id it had in the original list.
				ImportedTemplateId = InRundown->AddTemplate(FAvaRundownPageIdGeneratorParams(SourcePage.GetTemplateId()));
				
				if (ImportedTemplateId != FAvaRundownPage::InvalidPageId)
				{
					// Keep track of correspondence for next pages in the list.
					InOutImportedTemplateIds.Add(SourcePage.GetTemplateId(), ImportedTemplateId);
					
					// We either add the current page "as template", or use the source templates if provided.
					const FAvaRundownPage& SourceTemplate = Private::FindPage(InSourceTemplates, SourcePage.GetTemplateId());
					
					ensure(Private::CopyPageInPlace(InRundown, ImportedTemplateId, SourceTemplate.IsValidPage() ? SourceTemplate : SourcePage, FAvaRundownPage::InvalidPageId));
					InRundown->GetPage(ImportedTemplateId).ResetInstancedIds();	// We know this template has no instances yet.
				}
				else
				{
					// We are unable to find/create a template for this page.
					continue;
				}
			}
		}

		// There is a valid imported template at this point.
		check(ImportedTemplateId != FAvaRundownPage::InvalidPageId);

		// We're pasting to the instance list, so just add the page
		if (InPageListReference.Type == EAvaRundownPageListType::Instance)
		{
			// We want to preserve the source PageId if possible.
			const FAvaRundownPageIdGeneratorParams NewPageIdParams =
				FAvaRundownPageIdGeneratorParams::FromInsertPositionOrSourceId(SourcePage.GetPageId(), InsertPosition);
			
			int32 ImportedPageId = InRundown->AddPageFromTemplate(ImportedTemplateId, NewPageIdParams, InsertPosition);

			if (ImportedPageId != FAvaRundownPage::InvalidPageId)
			{
				ensure(Private::CopyPageInPlace(InRundown, ImportedPageId, SourcePage, ImportedTemplateId));
				OutPageIds.Add(ImportedPageId);			
				InsertPosition.ConditionalUpdateAdjacentId(ImportedPageId); // Update for next insertion.
			}

			continue;
		}

		// We're pasting to a sub list, so we need to check if the page exists in the instance list first
		const FAvaRundownPage& InstancedPage = InRundown->GetPage(SourcePage.GetPageId());
		int32 InstancedPageId = InstancedPage.GetPageId();

		// Add it if it's missing
		if (!InstancedPage.IsValidPage() || InstancedPage.IsTemplate())
		{
			// We want to preserve the source PageId if possible.
			const FAvaRundownPageIdGeneratorParams NewPageIdParams =
				FAvaRundownPageIdGeneratorParams::FromInsertPositionOrSourceId(!InstancedPage.IsTemplate() ? SourcePage.GetPageId() : FAvaRundownPage::InvalidPageId, InsertPosition);

			// Note: If the page is a template, it will already be imported as templateId.
			InstancedPageId = InRundown->AddPageFromTemplate(ImportedTemplateId, NewPageIdParams, InsertPosition);

			if (InstancedPageId != FAvaRundownPage::InvalidPageId)
			{
				ensure(Private::CopyPageInPlace(InRundown, InstancedPageId, SourcePage, ImportedTemplateId));
				InsertPosition.ConditionalUpdateAdjacentId(InstancedPageId); // Update for next insertion.
			}
			else
			{
				// We were unable to create the instance page, so it cannot be added to a sub list
				continue;
			}
		}

		// There is a valid imported instanced page at this point.
		check(InstancedPageId != FAvaRundownPage::InvalidPageId);

		// Now we have our instance page reference, add it to the sublist
		if (InRundown->AddPageToSubList(InPageListReference, InstancedPageId))
		{
			OutPageIds.Add(InstancedPageId);
		}
	}

	if (bReverseIteration)
	{
		Algo::Reverse(OutPageIds);
	}
	
	return OutPageIds;
}

TArray<int32> UE::AvaRundownEditor::Utils::ImportInstancedPagesFromRundown(UAvaRundown* InRundown, const UAvaRundown* InSourceRundown, const FAvaRundownPageInsertPosition& InInsertPosition)
{
	if (!IsValid(InRundown) || !IsValid(InSourceRundown))
	{
		return {};
	}
	
	const FAvaRundownPageCollection& SourceTemplates = InSourceRundown->GetTemplatePages();
	const FAvaRundownPageCollection& SourcePages = InSourceRundown->GetInstancedPages();
	FImportTemplateMap ImportedTemplateIds;
	
	return ImportInstancedPages(InRundown, UAvaRundown::InstancePageList, SourcePages.Pages, SourceTemplates.Pages, ImportedTemplateIds, InInsertPosition);
}

UAvaRundown* UE::AvaRundownEditor::Utils::SaveDuplicateRundown(UAvaRundown* InSourceRundown, const FString& InAssetName, const FString& InPackagePath)
{
	return Cast<UAvaRundown>(IAssetTools::Get().DuplicateAsset(InAssetName, InPackagePath, InSourceRundown));
}

bool UE::AvaRundownEditor::Utils::MergeDefaultRemoteControlValues(const TArray<TSharedPtr<FAvaRundownManagedInstance>>& InManagedInstances, FAvaPlayableRemoteControlValues& OutMergedValues)
{
	bool bAllUniqueIds = true;
	
	for (const TSharedPtr<FAvaRundownManagedInstance>& ManagedInstance : InManagedInstances)
	{
		if (ManagedInstance.IsValid())
		{
			bAllUniqueIds &= OutMergedValues.Merge(ManagedInstance->GetDefaultRemoteControlValues());
		}
	}
	
	return bAllUniqueIds;
}

EAvaPlayableRemoteControlChanges UE::AvaRundownEditor::Utils::UpdateDefaultRemoteControlValues(UAvaRundown* InRundown, const TArray<int32>& InSelectedPageIds)
{
	EAvaPlayableRemoteControlChanges Changes = EAvaPlayableRemoteControlChanges::None;

	if (!InRundown)
	{
		return Changes;
	}

	FAvaRundownManagedInstanceCache& ManagedInstanceCache = IAvaMediaModule::Get().GetManagedInstanceCache();

	for (const int32 PageId : InSelectedPageIds)
	{
		FAvaRundownPage& Page = InRundown->GetPage(PageId);

		if (Page.IsValidPage())
		{
			TArray<TSharedPtr<FAvaRundownManagedInstance>> ManagedInstances = ManagedInstanceCache.GetManagedInstancesForPage(InRundown, Page);
					
			if (!ManagedInstances.IsEmpty())
			{
				FAvaPlayableRemoteControlValues MergedDefaultRCValues;
				MergeDefaultRemoteControlValues(ManagedInstances, MergedDefaultRCValues);

				// Using the rundown API for event propagation.
				constexpr bool bUpdateDefaults = true;
				Changes |= InRundown->UpdateRemoteControlValues(PageId, MergedDefaultRCValues, bUpdateDefaults);
			}

			// Combo templates will also update the values of the sub-templates.
			if (Page.IsComboTemplate())
			{
				Changes |= UpdateDefaultRemoteControlValues(InRundown, Page.GetCombinedTemplateIds());
			}
		}
	}
	return Changes;
}

// Todo: merge with FAvaRundownPage::UpdatePageSummary(InRundown).
// This will also be needed for rundown server.
// It would need to add a "force update" parameter.
bool UE::AvaRundownEditor::Utils::ForceUpdatePageSummary(int32 InPageId, UAvaRundown* InRundown)
{
	FAvaRundownPage& Page = InRundown ? InRundown->GetPage(InPageId) : FAvaRundownPage::NullPage;
	if (!Page.IsValidPage())
	{
		return false;
	}

	// Important: the managed instance cache is used by the page details panel, which assumes the instances it holds
	// for the current page are not modified by something else.
	// This is why we need to back up the controllers and restore them at the end of the function.
	FAvaRundownManagedInstanceCache& ManagedInstanceCache = IAvaMediaModule::Get().GetManagedInstanceCache();
	TArray<TSharedPtr<FAvaRundownManagedInstance>> ManagedInstances = ManagedInstanceCache.GetManagedInstancesForPage(InRundown, Page);
	TArray<const URemoteControlPreset*> Presets;
	Presets.Reserve(ManagedInstances.Num());

	FAvaPlayableRemoteControlValues AllControllerValuesBackup;
	
	for (const TSharedPtr<FAvaRundownManagedInstance>& Instance : ManagedInstances)
	{
		if (URemoteControlPreset* Preset = Instance->GetRemoteControlPreset())
		{
			// Backup controllers
			for (const URCVirtualPropertyBase* Controller : Preset->GetControllers())
			{
				if (!AllControllerValuesBackup.SetControllerValue(Controller->Id, Preset, /*bInIsDefault*/ false))
				{
					// Don't risk stomping preset controllers.
					UE_LOGF(LogAvaRundown, Error, "Unable to backup preset controllers. Aborting page summary update.");
					return false;
				}
			}
			
			// Apply page controller value to preset without running the behaviors. We only care about controllers for page summary. 
			Page.GetRemoteControlValues().ApplyControllerValuesToRemoteControlPreset(Preset, /*bInForceDisableBehaviors*/ true);
			
			Presets.Add(Preset);
		}
	}

	const bool bSummaryUpdated = Page.UpdatePageSummary(InRundown, Presets, /*bInForceUpdate*/ true);

	// Restore managed instances controllers
	for (const TSharedPtr<FAvaRundownManagedInstance>& Instance : ManagedInstances)
	{
		if (URemoteControlPreset* Preset = Instance->GetRemoteControlPreset())
		{
			AllControllerValuesBackup.ApplyControllerValuesToRemoteControlPreset(Preset, /*bInForceDisableBehaviors*/ true);
		}
	}

	return bSummaryUpdated;
}

#undef LOCTEXT_NAMESPACE
