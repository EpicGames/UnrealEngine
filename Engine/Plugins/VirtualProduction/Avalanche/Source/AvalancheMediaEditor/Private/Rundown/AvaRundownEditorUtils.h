// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownPage.h"
#include "Templates/SharedPointer.h"
#include "UObject/StrongObjectPtr.h"
#include "XmlSerializationDefines.h"

class FAvaRundownManagedInstance;
class UAvaRundown;

namespace UE::AvaRundownEditor::Utils
{
	FString GetImportFilepath(const TCHAR* InFileDescription, const TCHAR* InExtension);
	FString GetExportFilepath(const UObject* InObjectToExport, const TCHAR* InFileDescription, const TCHAR* InExtension);
	FString GetSaveAssetAsPath(const FString& InDefaultPath, const FString& InDefaultAssetName);
	
	FString SerializePagesToJson(UAvaRundown* InRundown, const TArray<int32>& InPageIds);
	TArray<FAvaRundownPage> DeserializePagesFromJson(const FString& InJsonString);

	/**
	 * @brief Creates a transient rundown with only the specified pages.
	 * @param InRundown Source Rundown.
	 * @param InPageIds Pages to copy to the new rundown.
	 * @return Created rundown.
	 */
	TStrongObjectPtr<UAvaRundown> ExportPagesToRundown(const UAvaRundown* InRundown, const TArray<int32>& InPageIds);

	bool SaveRundownToXml(const UAvaRundown* InRundown, FArchive& InArchive, EXmlSerializationEncoding InXmlEncoding);
	bool SaveRundownToXml(const UAvaRundown* InRundown, const TCHAR* InFilepath, EXmlSerializationEncoding InXmlEncoding);

	/**
	 * Check the file extenstion to see if it is a supported format to use 
	 * with the deserializing functions (LoadRundownFrom...).
	 */
	bool CanLoadRundownFromFile(const TCHAR* InFilepath);

	/**
	 * Map of "Source TemplateId" to "Destination Template Id".
	 * 
	 * During the import of new pages, we need to keep track of the
	 * destination template Id, i.e. Source Page's template id -> actual id in rundown.
	 */
	struct FImportTemplateMap
	{
		TMap<int32, int32> TemplateIds;

		void Add(int32 InSourceTemplateId, int32 InDestinationTemplateId)
		{
			TemplateIds.Add(InSourceTemplateId, InDestinationTemplateId);
		}

		bool HasSourceTemplateId(int32 InSourceTemplateId) const
		{
			return TemplateIds.Contains(InSourceTemplateId);
		}

		int32 GetTemplateId(int32 InSourceTemplateId) const
		{
			const int32* FoundId = TemplateIds.Find(InSourceTemplateId);
			return FoundId ? *FoundId : InSourceTemplateId;
		}
	};
	
	TArray<int32> ImportTemplatePages(UAvaRundown* InRundown, const TArray<FAvaRundownPage>& InSourceTemplates, FImportTemplateMap& OutImportedTemplateIds);

	/**
	 * @brief Import the given pages to the rundown.
	 * @param InRundown Rundown that receives the new pages. 
	 * @param InPageListReference Reference to the page list to which the pages will be added.
	 * @param InSourcePages Source pages
	 * @param InSourceTemplates Source templates, if available, for the source pages. 
	 * @param InOutImportedTemplateIds Imported template ids to make the correspondence between the source page template id and what was imported.
	 * @param InInsertPosition Indicate where in the page list to insert the new pages.
	 * @return Array of added page ids.
	 *
	 * Importing pages is a non-trivial operation. It includes adding missing templates. The biggest issue
	 * is maintaining the link between the page's template id and the imported templates. If available, the
	 * source templates can be used to reconstruct this relationship by finding an exact match with the templates
	 * of the destination rundown.
	 */
	TArray<int32> ImportInstancedPages(
		UAvaRundown* InRundown,
		const FAvaRundownPageListReference& InPageListReference,
		const TArray<FAvaRundownPage>& InSourcePages,
		const TArray<FAvaRundownPage>& InSourceTemplates,
		FImportTemplateMap& InOutImportedTemplateIds,
		const FAvaRundownPageInsertPosition& InInsertPosition = FAvaRundownPageInsertPosition());

	/**
	 * @brief Import/Merge a rundown into another rundown.
	 * @param InRundown Destination rundown.
	 * @param InSourceRundown Source rundown to merge in the destination.
	 * @param InInsertPosition Indicate where in the page list to insert the new pages.
	 * @return Array of added page ids.
	 */
	TArray<int32> ImportInstancedPagesFromRundown(
		UAvaRundown* InRundown,
		const UAvaRundown* InSourceRundown,
		const FAvaRundownPageInsertPosition& InInsertPosition = FAvaRundownPageInsertPosition());
	
	UAvaRundown* SaveDuplicateRundown(UAvaRundown* InSourceRundown, const FString& InAssetName, const FString& InPackagePath);

	bool MergeDefaultRemoteControlValues(const TArray<TSharedPtr<FAvaRundownManagedInstance>>& InManagedInstances, FAvaPlayableRemoteControlValues& OutMergedValues);
	EAvaPlayableRemoteControlChanges UpdateDefaultRemoteControlValues(UAvaRundown* InRundown, const TArray<int32>& InSelectedPageIds);
	
	/**
     * Utility function to force update the page summary from the controller values. 
     */
    bool ForceUpdatePageSummary(int32 InPageId, UAvaRundown* InRundown);
}