// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownExporter.h"

#include "AvaMediaSerializationUtils.h"
#include "Misc/FeedbackContext.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownEditorUtils.h"
#include "Rundown/AvaRundownSerializationUtils.h"
#include "Serialization/MemoryWriter.h"

UAvaRundownExporter::UAvaRundownExporter()
{
	FormatExtension.Add(TEXT("json"));
	FormatDescription.Add(TEXT("JavaScript Object Notation file"));
	FormatExtension.Add(TEXT("xml"));
	FormatDescription.Add(TEXT("eXtensible Markup Language file"));
	SupportedClass = UAvaRundown::StaticClass();
	bText = true;
}

bool UAvaRundownExporter::ExportText(const FExportObjectInnerContext* InContext, UObject* InObject, const TCHAR* InType, FOutputDevice& InAr, FFeedbackContext* InWarn, uint32 InPortFlags)
{
	const UAvaRundown* Rundown = Cast<UAvaRundown>(InObject);
	if (!Rundown)
	{
		return false;
	}

	TArray<uint8> OutputBytes;
	FMemoryWriter Writer(OutputBytes);
	bool bSavedToBytes = false;

	if ( FCString::Stricmp(InType, TEXT("json")) == 0 )
	{
		FText ErrorMessage;
		bSavedToBytes = UE::AvaMedia::RundownSerializationUtils::SaveRundownToJson(Rundown, Writer, ErrorMessage);
		
		if (!bSavedToBytes && InWarn)
		{
			InWarn->CategorizedLogf(LogAvaRundown.GetCategoryName(), ELogVerbosity::Error,
				TEXT("Failed to export rundown \"%s\". Reason: %s"),
				*Rundown->GetFullName(), *ErrorMessage.ToString());
		}
	}
	else if (FCString::Stricmp(InType, TEXT("xml")) == 0 )
	{
		// Note: using WChar encoding for compatibility with BytesToString below.
		bSavedToBytes = UE::AvaRundownEditor::Utils::SaveRundownToXml(Rundown, Writer, EXmlSerializationEncoding::WChar);
	}

	if (bSavedToBytes)
	{
		FString OutputString;
		UE::AvaMediaSerializationUtils::JsonValueConversion::BytesToString(OutputBytes, OutputString);
		InAr.Log(OutputString);
		return true;
	}
	
	return false;	
}