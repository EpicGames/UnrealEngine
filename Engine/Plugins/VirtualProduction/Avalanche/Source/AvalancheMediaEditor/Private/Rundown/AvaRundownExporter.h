// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Exporters/Exporter.h"
#include "AvaRundownExporter.generated.h"

UCLASS()
class UAvaRundownExporter : public UExporter
{
	GENERATED_BODY()

public:
	UAvaRundownExporter();
	
	//~ Begin UExporter Interface
	virtual bool ExportText(const FExportObjectInnerContext* InContext, UObject* InObject, const TCHAR* InType, FOutputDevice& InAr, FFeedbackContext* InWarn, uint32 InPortFlags) override;
	//~ End UExporter Interface
};