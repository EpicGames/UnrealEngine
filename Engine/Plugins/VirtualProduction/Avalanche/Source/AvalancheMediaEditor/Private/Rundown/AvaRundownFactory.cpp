// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownFactory.h"

#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "Rundown/AvaRundown.h"

UAvaRundownFactory::UAvaRundownFactory()
{
	// Provide the factory with information about how to handle our asset
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UAvaRundown::StaticClass();
}

UAvaRundownFactory::~UAvaRundownFactory()
{
}

uint32 UAvaRundownFactory::GetMenuCategories() const
{
	const IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	return AssetTools.FindAdvancedAssetCategory("MotionDesignCategory");
}

UObject* UAvaRundownFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags
	, UObject* Context, FFeedbackContext* Warn)
{
	UAvaRundown* Rundown = nullptr;
	if (ensure(SupportedClass == Class))
	{
		Rundown = NewObject<UAvaRundown>(InParent, Name, Flags);
	}
	return Rundown;
}

FString UAvaRundownFactory::GetDefaultNewAssetName() const
{
	return TEXT("NewRundown");
}

