// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "AvaRundownFactory.generated.h"

UCLASS()
class UAvaRundownFactory : public UFactory
{
	GENERATED_BODY()

public:

	UAvaRundownFactory();
	virtual ~UAvaRundownFactory() override;

protected:
	
	//~ Begin UFactory Interface
	virtual uint32 GetMenuCategories() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual FString GetDefaultNewAssetName() const override;
	//~ Begin UFactory Interface
};
