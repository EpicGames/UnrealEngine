// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/AvaRundownMacroCollectionFactory.h"

#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "Rundown/AvaRundownMacroCollection.h"

UAvaRundownMacroCollectionFactory::UAvaRundownMacroCollectionFactory()
{
	SupportedClass = UAvaRundownMacroCollection::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
	bEditorImport = false;
}

uint32 UAvaRundownMacroCollectionFactory::GetMenuCategories() const
{
	const IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	return AssetTools.FindAdvancedAssetCategory("MotionDesignCategory");
}

UObject* UAvaRundownMacroCollectionFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags
	, UObject* Context, FFeedbackContext* Warn)
{
	if (ensure(SupportedClass == Class))
	{
		return NewObject<UAvaRundownMacroCollection>(InParent, Name, Flags);
	}
	return nullptr;
}

FString UAvaRundownMacroCollectionFactory::GetDefaultNewAssetName() const
{
	return TEXT("NewRundownMacros");
}
