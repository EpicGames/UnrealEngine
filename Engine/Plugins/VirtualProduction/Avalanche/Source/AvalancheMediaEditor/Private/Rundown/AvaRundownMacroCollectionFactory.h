// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "AvaRundownMacroCollectionFactory.generated.h"

UCLASS()
class UAvaRundownMacroCollectionFactory : public UFactory
{
	GENERATED_BODY()
	
	UAvaRundownMacroCollectionFactory();
	
	//~ Begin UFactory Interface
	virtual uint32 GetMenuCategories() const override;
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual FString GetDefaultNewAssetName() const override;
	//~ End UFactory Interface
};
