// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownChannelLayerStatusList.h"

#include "AvaTag.h"
#include "Misc/CoreDelegates.h"
#include "Rundown/AvaRundownEditor.h"
#include "Rundown/AvaRundownEditorDefines.h"
#include "Rundown/AvaRundownPagePlayer.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SWrapBox.h"

#define LOCTEXT_NAMESPACE "SAvaRundownChannelLayerStatusList"

namespace UE::AvaMediaEditor::Private
{
	const TArray<FLinearColor> ComboPageColorPalette = {
		FStyleColors::AccentRed.GetSpecifiedColor(),
		FStyleColors::AccentGreen.GetSpecifiedColor(),
		FStyleColors::AccentBlue.GetSpecifiedColor(),
		FStyleColors::AccentYellow.GetSpecifiedColor(),
		FStyleColors::AccentPurple.GetSpecifiedColor(),
		FStyleColors::AccentOrange.GetSpecifiedColor(),
		FStyleColors::AccentGray.GetSpecifiedColor(),
		FStyleColors::AccentBrown.GetSpecifiedColor(),
		FStyleColors::AccentPink.GetSpecifiedColor(),
		FStyleColors::AccentWhite.GetSpecifiedColor()
	};
}

void SAvaRundownChannelLayerStatusList::Construct(const FArguments& InArgs, const TSharedPtr<FAvaRundownEditor>& InRundownEditor)
{
	RundownEditorWeak = InRundownEditor;

	ChildSlot
	[
		SNew(SScrollBox)
		.Orientation(EOrientation::Orient_Vertical)
		+ SScrollBox::Slot()
		.FillSize(1)
		[
			SAssignNew(Container, SWrapBox)
			.UseAllottedSize(true)
			.HAlign(HAlign_Left)
			.Orientation(EOrientation::Orient_Horizontal)
			.InnerSlotPadding(FVector2D(5.f))
		]
	];

	RefreshList();

	UpdateHandle = FCoreDelegates::OnEndFrame.AddSPLambda(this, [this]() { RefreshList(); });

	if (UAvaRundown* Rundown = InRundownEditor->GetRundown())
	{
		// @TODO: When events work, remove end of frame!
		//Rundown->GetPlaybackManager().OnPlaybackInstanceStatusChanged.AddSP(this, &SAvaRundownChannelLayerStatusList::OnPlaybackInstanceStatusChanged);
	}
}

void SAvaRundownChannelLayerStatusList::RefreshList()
{
	Container->ClearChildren();
	ChannelLayerStatusList.Empty();

	TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin();

	if (!RundownEditor.IsValid())
	{
		return;
	}

	UAvaRundown* Rundown = RundownEditor->GetRundown();

	if (!IsValid(Rundown))
	{
		return;
	}

	//   Channel,    Layer, List of pages trying to use this layer
	TMap<FName, TMap<FName, TSet<int32>>> LayerPageRequest;

	// Iterate over the active page players to find what's playing on each layer
	for (const TObjectPtr<UAvaRundownPagePlayer>& PagePlayer : Rundown->GetPagePlayers())
	{
		if (!PagePlayer->IsPlaying())
		{
			continue;
		}

		const int32 PageId = PagePlayer->PageId;
		const FAvaRundownPage& Page = Rundown->GetPage(PageId);

		if (!Page.IsValidPage())
		{
			continue;
		}

		const FName& ChannelName = PagePlayer->ChannelFName;

		for (const TObjectPtr<UAvaRundownPlaybackInstancePlayer>& InstancePlayer : PagePlayer->InstancePlayers)
		{
			if (!InstancePlayer->IsPlaying())
			{
				continue;
			}

			if (!InstancePlayer->TransitionLayer.IsValid())
			{
				continue;
			}

			const FName LayerName = InstancePlayer->TransitionLayer.ToName();
			if (LayerName.IsNone())
			{
				continue;
			}

			const FAvaRundownPage& TemplatePage = Rundown->GetPage(Page.GetTemplateId());

			if (!TemplatePage.IsValidPage())
			{
				continue;
			}

			const FText LayerDescription = FText::FromName(LayerName);
			FLinearColor ComboPageColor = FLinearColor::Transparent;

			// If we're not a combo page, just add our id to the layer request
			if (!TemplatePage.IsComboTemplate())
			{
				//UE_LOGF(LogAvaRundown, Verbose, "Non-Template page playing on: %ls: %ls: %d", *ChannelName.ToString(), *LayerName.ToString(), PageId);
				LayerPageRequest.FindOrAdd(ChannelName).FindOrAdd(LayerName).Add(PageId);
			}
			/**
			 * If this is a combo page then we need to work out if our pages are active/overridden.
			 * For each playing page, record the pages that the entire combo template wanted to play
			 * and compare later.
			 */
			else
			{
				// For each layer that this template uses, mark off a request for that layer.
				for (const FAvaTagHandle& TemplateTag : TemplatePage.GetTransitionLayers(Rundown))
				{
					if (!TemplateTag.IsValid())
					{
						continue;
					}

					const FName TemplateLayerName = TemplateTag.ToName();

					if (TemplateLayerName == NAME_None)
					{
						continue;
					}

					//UE_LOGF(LogAvaRundown, Warning, "Template page playing on: %ls: %ls: %d", *ChannelName.ToString(), *TemplateLayerName.ToString(), PageId);
					LayerPageRequest.FindOrAdd(ChannelName).FindOrAdd(TemplateLayerName).Add(PageId);
				}
			}

			ChannelLayerStatusList.FindOrAdd(ChannelName).FindOrAdd(LayerName) = {
				ChannelName,
				LayerName,
				PageId,
				LayerDescription,
				ComboPageColor,
				false
			};
		}
	}

	int32 ColorIndex = 0;

	//   PageId, Color
	TMap<int32, FLinearColor> ComboPageColors;

	LayerPageRequest.KeySort(
		[](const FName& InA, const FName& InB)
		{
			return InA.Compare(InB) < 0;
		});

	// Now iterate through the actually playing pages and see if they match the requested combo page pages.
	for (TPair<FName, TMap<FName, TSet<int32>>>& ChannelPair : LayerPageRequest)
	{
		const FName& ChannelName = ChannelPair.Key;

		ChannelPair.Value.KeySort(
			[](const FName& InA, const FName& InB)
			{
				return InA.Compare(InB) < 0;
			});

		for (const TPair<FName, TSet<int32>>& LayerPair : ChannelPair.Value)
		{
			const FName& ValueName = LayerPair.Key;

			if (TMap<FName, FAvaRundownChannelLayerEntry>* LiveChannelMap = ChannelLayerStatusList.Find(ChannelName))
			{
				if (FAvaRundownChannelLayerEntry* LiveLayerEntry = LiveChannelMap->Find(ValueName))
				{
					if (LayerPair.Value.IsEmpty())
					{
						continue;
					}

					// No conflict
					if (LayerPair.Value.Num() == 1)
					{
						const int32 PageId = *LayerPair.Value.FindArbitraryElement();

						if (const FLinearColor* CurrentColor = ComboPageColors.Find(PageId))
						{
							LiveLayerEntry->ComboPageColor = *CurrentColor;
						}
						else
						{
							using namespace UE::AvaMediaEditor::Private;

							LiveLayerEntry->ComboPageColor = ComboPageColorPalette[ColorIndex];

							++ColorIndex;

							// If we have too many, just repeat them
							if (ColorIndex >= ComboPageColorPalette.Num())
							{
								ColorIndex = 0;
							}

							ComboPageColors.Add(PageId, LiveLayerEntry->ComboPageColor);
						}
					}
					// If more than 1 playing page has requested this entry, it must have a conflict.
					else
					{
						LiveLayerEntry->bOverridden = true;
					}
				}
			}
		}
	}

	constexpr float Padding = 5.f;

	ChannelLayerStatusList.KeySort(
		[](const FName& InA, const FName& InB)
		{
			return InA.Compare(InB) < 0;
		});

	// Add new widgets
	for (const TPair<FName, TMap<FName, FAvaRundownChannelLayerEntry>>& ChannelPair : ChannelLayerStatusList)
	{
		const FName& ChannelName = ChannelPair.Key;

		constexpr int32 ColumnStatus = 0;
		constexpr int32 ColumnLayer = 1;
		constexpr int32 ColumnPageId = 2;
		constexpr int32 ColumnCount = 3;

		const FName PreviewChannelName =  UE::AvaMedia::Playback::DefaultPreviewChannelName;
		static const FText PreviewChannelDescription = LOCTEXT("Preview", "Preview");

		int32 RowIndex = 0;

		TSharedRef<SGridPanel> ChannelInfo = SNew(SGridPanel)
			+ SGridPanel::Slot(ColumnStatus, RowIndex)
			.ColumnSpan(ColumnCount)
			.HAlign(HAlign_Fill)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SColorBlock)
					.Color(FSlateColor(EStyleColor::Primary).GetSpecifiedColor())
				]
				+ SOverlay::Slot()
				.Padding(Padding)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(ChannelName == PreviewChannelName ? PreviewChannelDescription : FText::FromName(ChannelName))
					.ColorAndOpacity(FSlateColor(EStyleColor::ForegroundHeader))
				]
			];

		++RowIndex;

		ChannelInfo->AddSlot(ColumnStatus, RowIndex)
			.Padding(0.f)
			[
				SNew(SColorBlock)
				.Color(FSlateColor(EStyleColor::Header).GetSpecifiedColor())
			];

		ChannelInfo->AddSlot(ColumnLayer, RowIndex)
			.Padding(1.f, 0.f, 0.f, 0.f)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SColorBlock)
					.Color(FSlateColor(EStyleColor::Header).GetSpecifiedColor())
				]
				+ SOverlay::Slot()
				.Padding(Padding)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Layer", "Layer"))
				]
			];

		ChannelInfo->AddSlot(ColumnPageId, RowIndex)
			.Padding(1.f, 0.f, 0.f, 0.f)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SColorBlock)
					.Color(FSlateColor(EStyleColor::Header).GetSpecifiedColor())
				]
				+ SOverlay::Slot()
				.Padding(Padding)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PageId", "Page ID"))
					.MinDesiredWidth(50.f)
				]
			];

		for (const TPair<FName, FAvaRundownChannelLayerEntry>& LayerPair : ChannelPair.Value)
		{
			++RowIndex;

			const FAvaRundownChannelLayerEntry& LayerEntry = LayerPair.Value;

			if (LayerEntry.bOverridden)
			{
				ChannelInfo->AddSlot(ColumnStatus, RowIndex)
					.Padding(Padding, Padding, 0.f, Padding)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Next"))
					];
			}
			else
			{
				ChannelInfo->AddSlot(ColumnStatus, RowIndex)
					.Padding(Padding)
					[
						SNew(SColorBlock)
						.Color(LayerEntry.ComboPageColor)
						.Size(FVector2D(16.0))
						.Visibility(LayerEntry.ComboPageColor != FLinearColor::Transparent ? EVisibility::Visible : EVisibility::Hidden)
					];
			}

			ChannelInfo->AddSlot(ColumnLayer, RowIndex)
				.Padding(Padding, Padding, 0.f, Padding)
				[
					SNew(STextBlock)
					.Text(LayerEntry.LayerDescription)
				];

			ChannelInfo->AddSlot(ColumnPageId, RowIndex)
				.Padding(Padding, Padding, 0.f, Padding)
				[
					SNew(STextBlock)
					.Text(FText::AsNumber(LayerEntry.PageId, &UE::AvaRundown::FEditorMetrics::PageIdFormattingOptions))
				];
		}

		Container->AddSlot()
			[
				ChannelInfo
			];
	}

	/*
	 * @TODO: When events work, re-enable this.
	if (UpdateHandle.IsValid())
	{
		FCoreDelegates::OnEndFrame.Remove(UpdateHandle);
		UpdateHandle.Reset();
	}
	*/
}

void SAvaRundownChannelLayerStatusList::OnPlaybackInstanceStatusChanged(const FAvaPlaybackInstance& InPlaybackInstance)
{
	if (UpdateHandle.IsValid())
	{
		return;
	}

	UpdateHandle = FCoreDelegates::OnEndFrame.AddSPLambda(this, [this]() { RefreshList(); });
}

#undef LOCTEXT_NAMESPACE
