// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Delegates/IDelegateInstance.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FAvaPlaybackInstance;
class FAvaRundownEditor;
class SAvaRundownChannelLayerStatus;
class SWrapBox;

struct FAvaRundownChannelLayerEntry
{
	FName ChannelName;
	FName LayerName;
	int32 PageId;
	FText LayerDescription;
	FLinearColor ComboPageColor;
	bool bOverridden;
};

/*
 * Widget containing the Status of all Channel Layers in Broadcast. It contains a list of SAvaRundownChannelLayerStatus widgets
 */
class SAvaRundownChannelLayerStatusList : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaRundownChannelLayerStatusList) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedPtr<FAvaRundownEditor>& InRundownEditor);

protected:
	TWeakPtr<FAvaRundownEditor> RundownEditorWeak;
	TSharedPtr<SWrapBox> Container;
	FDelegateHandle UpdateHandle;

	//   Channel,    Layer, Details
	TMap<FName, TMap<FName, FAvaRundownChannelLayerEntry>> ChannelLayerStatusList;

	void RefreshList();

	void OnPlaybackInstanceStatusChanged(const FAvaPlaybackInstance& InPlaybackInstance);
};
