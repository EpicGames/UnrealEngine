// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownChannelStatus.h"
#include "AvaMediaDefines.h"
#include "AvaMediaEditorUtils.h"
#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/Channel/AvaBroadcastOutputChannel.h"
#include "Internationalization/Text.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "UObject/NameTypes.h"

void SAvaRundownChannelStatus::Construct(const FArguments& InArgs, const FAvaBroadcastOutputChannel& InChannel)
{
	ChannelName = InChannel.GetChannelName();
	FAvaBroadcastOutputChannel::GetOnChannelChanged().AddSP(this, &SAvaRundownChannelStatus::OnChannelChanged);

	ChildSlot
	.Padding(FMargin(10.f, 3.f))
	[
		SNew(SHorizontalBox)		
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SAssignNew(ChannelStatusIcon, SImage)
			.DesiredSizeOverride(FVector2D(16.f))
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			SNew(STextBlock)
			.Text(FText::FromName(ChannelName))
		]
	];

	OnChannelChanged(InChannel, EAvaBroadcastChannelChange::State);
}

SAvaRundownChannelStatus::~SAvaRundownChannelStatus()
{
	FAvaBroadcastOutputChannel::GetOnChannelChanged().RemoveAll(this);
}

void SAvaRundownChannelStatus::OnChannelChanged(const FAvaBroadcastOutputChannel& InChannel, EAvaBroadcastChannelChange InChange)
{
	const bool bMatchingChannels = InChannel.GetChannelName() == ChannelName;
	const bool bStateChanged     = EnumHasAnyFlags(InChange, EAvaBroadcastChannelChange::State);
	
	if (!bMatchingChannels || !bStateChanged)
	{
		return;
	}
	
	ChannelStatusIcon->SetImage(FAvaMediaEditorUtils::GetChannelStatusBrush(InChannel.GetState(), InChannel.GetIssueSeverity()));
}
