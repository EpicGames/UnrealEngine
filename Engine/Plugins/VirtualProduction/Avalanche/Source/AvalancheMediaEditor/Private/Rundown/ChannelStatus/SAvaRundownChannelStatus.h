// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Broadcast/Channel/AvaBroadcastOutputChannel.h"
#include "Widgets/SCompoundWidget.h"
#include "Templates/SharedPointer.h"

class SImage;
class UAvaBroadcast;
class FName;
struct FAvaBroadcastOutputChannel;
enum class EAvaBroadcastChannelChange : uint8;

/*
 * Widget displaying the Status of a Single Channel
 */
class SAvaRundownChannelStatus : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaRundownChannelStatus) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FAvaBroadcastOutputChannel& InChannel);

	virtual ~SAvaRundownChannelStatus() override;
	
	void OnChannelChanged(const FAvaBroadcastOutputChannel& InChannel, EAvaBroadcastChannelChange InChange);
	
protected:
	FName ChannelName;

	TSharedPtr<SImage> ChannelStatusIcon;
};
