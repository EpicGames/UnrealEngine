// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownChannelStatusList.h"
#include "AvaMediaDefines.h"
#include "Broadcast/AvaBroadcast.h"
#include "Misc/EnumClassFlags.h"
#include "SAvaRundownChannelStatus.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SWrapBox.h"
#include "UObject/NameTypes.h"

void SAvaRundownChannelStatusList::Construct(const FArguments& InArgs)
{
	BroadcastWeak = UAvaBroadcast::GetBroadcast();
	
	check(BroadcastWeak.IsValid());
	
	BroadcastWeak->AddChangeListener(FOnAvaBroadcastChanged::FDelegate::CreateSP(this
		, &SAvaRundownChannelStatusList::OnBroadcastChanged));
	
	ChildSlot
	[
		SAssignNew(WrapBox, SWrapBox)
		.UseAllottedSize(true)
		.HAlign(EHorizontalAlignment::HAlign_Center)
	];

	RefreshList();
}

SAvaRundownChannelStatusList::~SAvaRundownChannelStatusList()
{
	if (BroadcastWeak.IsValid())
	{
		BroadcastWeak->RemoveChangeListener(this);
	}
}

void SAvaRundownChannelStatusList::RefreshList()
{
	UAvaBroadcast* const Broadcast = BroadcastWeak.Get();
	
	if (!Broadcast || !WrapBox.IsValid())
	{
		return;
	}

	WrapBox->ClearChildren();

	const TArray<FAvaBroadcastOutputChannel*>& Channels = Broadcast->GetCurrentProfile().GetChannels();
	
	for (const FAvaBroadcastOutputChannel* Channel : Channels)
	{
		WrapBox->AddSlot()
			[
				SNew(SAvaRundownChannelStatus, *Channel)
			];
	}
}

void SAvaRundownChannelStatusList::OnBroadcastChanged(EAvaBroadcastChange InChange)
{
	if (EnumHasAnyFlags(InChange, EAvaBroadcastChange::CurrentProfile
		| EAvaBroadcastChange::ChannelGrid
		| EAvaBroadcastChange::ChannelRename))
	{
		RefreshList();
	}
}
