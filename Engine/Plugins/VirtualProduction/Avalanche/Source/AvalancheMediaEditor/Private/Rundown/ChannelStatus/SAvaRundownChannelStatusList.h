// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/WeakObjectPtrTemplates.h"

class SAvaRundownChannelStatus;
class SWrapBox;
class UAvaBroadcast;
class FName;
enum class EAvaBroadcastChange : uint8;

/*
 * Widget containing the Status of all Channels in Broadcast. It contains a list of SAvaRundownChannelStatus widgets
 */
class SAvaRundownChannelStatusList : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaRundownChannelStatusList){}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);
	
	virtual ~SAvaRundownChannelStatusList() override;

	void RefreshList();
	
	void OnBroadcastChanged(EAvaBroadcastChange InChange);

protected:
	TSharedPtr<SWrapBox> WrapBox;

	TWeakObjectPtr<UAvaBroadcast> BroadcastWeak;

	TMap<FName, TSharedPtr<SAvaRundownChannelStatus>> ChannelStatusSlots;
};
