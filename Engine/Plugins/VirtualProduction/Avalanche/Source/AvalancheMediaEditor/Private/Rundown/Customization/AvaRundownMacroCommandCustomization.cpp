// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/Customization/AvaRundownMacroCommandCustomization.h"

#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "Rundown/AvaRundownMacroCollection.h"
#include "Rundown/Customization/SAvaRundownMacroCommandSelector.h"

#define LOCTEXT_NAMESPACE "AvaRundownMacroCommandCustomization"

TSharedRef<IPropertyTypeCustomization> FAvaRundownMacroCommandCustomization::MakeInstance()
{
	return MakeShared<FAvaRundownMacroCommandCustomization>();
}

void FAvaRundownMacroCommandCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	MacroCommandHandle = InStructPropertyHandle;
}

void FAvaRundownMacroCommandCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InStructBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	const TSharedPtr<IPropertyHandle> NameHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaRundownMacroCommand, Name));
	const TSharedPtr<IPropertyHandle> ArgumentsHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaRundownMacroCommand, Arguments));
	
	const TSharedPtr<IPropertyHandleArray> ParentArrayHandle = InStructPropertyHandle->GetParentHandle()->AsArray();

	const TSharedRef<SWidget> RemoveButton = (ParentArrayHandle.IsValid() ? PropertyCustomizationHelpers::MakeDeleteButton(FSimpleDelegate::CreateSP(this, &FAvaRundownMacroCommandCustomization::RemoveMacroCommandButton_OnClick),
		LOCTEXT("RemoveMacroCommandToolTip", "Removes Macro Command")) : SNullWidget::NullWidget);

	const FMargin PropertyPadding(2.0f, 0.0f, 2.0f, 0.0f);
	
	InStructBuilder.AddCustomRow(LOCTEXT("MacroCommandName", "Name"))
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SAvaRundownMacroCommandSelector, NameHandle)
		]
		+ SHorizontalBox::Slot()
		.Padding(PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			ArgumentsHandle->CreatePropertyNameWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			ArgumentsHandle->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			RemoveButton
		]
	];
}

void FAvaRundownMacroCommandCustomization::RemoveMacroCommandButton_OnClick()
{
	if (MacroCommandHandle->IsValidHandle())
	{
		const TSharedPtr<IPropertyHandle> ParentHandle = MacroCommandHandle->GetParentHandle();
		const TSharedPtr<IPropertyHandleArray> ParentArrayHandle = ParentHandle->AsArray();

		ParentArrayHandle->DeleteItem(MacroCommandHandle->GetIndexInArray());
	}
}

#undef LOCTEXT_NAMESPACE