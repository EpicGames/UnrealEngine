// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "Widgets/SWidget.h"

class FAvaRundownMacroCommandCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();
	
	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& PropertyTypeCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& PropertyTypeCustomizationUtils) override;
	//~ End IPropertyTypeCustomization

private:
	void RemoveMacroCommandButton_OnClick();

private:
	TSharedPtr<IPropertyHandle> MacroCommandHandle;
};