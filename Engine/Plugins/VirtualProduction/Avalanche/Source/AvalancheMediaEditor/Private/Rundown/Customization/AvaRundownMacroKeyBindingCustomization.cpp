// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/Customization/AvaRundownMacroKeyBindingCustomization.h"

#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "Rundown/AvaRundownMacroCollection.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "AvaRundownMacroKeyBindingCustomization"

TSharedRef<IPropertyTypeCustomization> FAvaRundownMacroKeyBindingCustomization::MakeInstance()
{
	return MakeShared<FAvaRundownMacroKeyBindingCustomization>();
}

void FAvaRundownMacroKeyBindingCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InStructPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	const TSharedPtr<IPropertyHandle> DescriptionHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaRundownMacroKeyBinding, Description));

	const FMargin PropertyPadding(2.0f, 0.0f, 2.0f, 0.0f);

	InHeaderRow.NameContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			DescriptionHandle->CreatePropertyNameWidget(FText::FromString(TEXT("Name")))
		]
		+ SHorizontalBox::Slot()
		.Padding(PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			DescriptionHandle->CreatePropertyValueWidget()
		]
	];
}

void FAvaRundownMacroKeyBindingCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& InStructBuilder, IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	const TSharedPtr<IPropertyHandle> InputChordHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaRundownMacroKeyBinding, InputChord));
	const TSharedPtr<IPropertyHandle> CommandsHandle = InStructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaRundownMacroKeyBinding, Commands));

	if (InputChordHandle.IsValid())
	{
		AddInputChord(InputChordHandle.ToSharedRef(), InStructBuilder);
	}

	if (CommandsHandle.IsValid())
	{
		InStructBuilder.AddProperty(CommandsHandle.ToSharedRef());
	}
}

void FAvaRundownMacroKeyBindingCustomization::AddInputChord(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InStructBuilder)
{
	const TSharedPtr<IPropertyHandle> KeyHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputChord, Key));
	const TSharedPtr<IPropertyHandle> ShiftHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputChord, bShift));
	const TSharedPtr<IPropertyHandle> CtrlHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputChord, bCtrl));
	const TSharedPtr<IPropertyHandle> AltHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputChord, bAlt));
	const TSharedPtr<IPropertyHandle> CmdHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FInputChord, bCmd));
	
	const FMargin PropertyPadding(2.0f, 0.0f, 2.0f, 0.0f);
	constexpr float TextBoxWidth = 250.0f;
	
	InStructBuilder.AddCustomRow(LOCTEXT("KeySearchStr", "Key"))
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(PropertyPadding)
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(TextBoxWidth)
			[
				InStructBuilder.GenerateStructValueWidget(KeyHandle.ToSharedRef())
			]
		]
		+ SHorizontalBox::Slot()
		.Padding(PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			ShiftHandle->CreatePropertyNameWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			ShiftHandle->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			CtrlHandle->CreatePropertyNameWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			CtrlHandle->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			AltHandle->CreatePropertyNameWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			AltHandle->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			CmdHandle->CreatePropertyNameWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(PropertyPadding)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			CmdHandle->CreatePropertyValueWidget()
		]
	];
}

#undef LOCTEXT_NAMESPACE