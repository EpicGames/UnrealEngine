// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownMacroCommandSelector.h"

#include "DetailLayoutBuilder.h"
#include "Rundown/AvaRundownEditorMacroCommands.h"

TArray<SAvaRundownMacroCommandSelector::FCommandItemPtr> SAvaRundownMacroCommandSelector::CommandItems;

void SAvaRundownMacroCommandSelector::Construct(const FArguments& InArgs, TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	UpdateCommandItems();
	PropertyHandle = InPropertyHandle;
	
	ChildSlot
	[
		SAssignNew(CommandCombo, SComboBox<FCommandItemPtr>)
		.InitiallySelectedItem(GetItemFromProperty())
		.OptionsSource(&CommandItems)
		.OnGenerateWidget(this, &SAvaRundownMacroCommandSelector::GenerateWidget)
		.OnSelectionChanged(this, &SAvaRundownMacroCommandSelector::HandleSelectionChanged)
		.OnComboBoxOpening(this, &SAvaRundownMacroCommandSelector::OnComboBoxOpening)
		[
			SNew(STextBlock)
			.Text(this, &SAvaRundownMacroCommandSelector::GetDisplayTextFromProperty)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];
}

TSharedRef<SWidget> SAvaRundownMacroCommandSelector::GenerateWidget(FCommandItemPtr InItem)
{
	return SNew(STextBlock)
		.Text(GetDisplayTextFromItem(InItem))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

void SAvaRundownMacroCommandSelector::HandleSelectionChanged(SComboBox<FCommandItemPtr>::NullableOptionType InProposedSelection, ESelectInfo::Type InSelectInfo)
{
	if (PropertyHandle && InProposedSelection.IsValid())
	{
		PropertyHandle->SetValue(InProposedSelection->Name);
	}
}

FText SAvaRundownMacroCommandSelector::GetDisplayTextFromProperty() const
{
	return GetDisplayTextFromItem(GetItemFromProperty());
}

void SAvaRundownMacroCommandSelector::OnComboBoxOpening()
{
	check(CommandCombo.IsValid());
	CommandCombo->SetSelectedItem(GetItemFromProperty());
}

SAvaRundownMacroCommandSelector::FCommandItemPtr SAvaRundownMacroCommandSelector::GetItemFromProperty() const
{
	if (PropertyHandle)
	{
		FName Value;
		PropertyHandle->GetValue(Value);
		for (const FCommandItemPtr& Item : CommandItems)
		{
			if (Item && Item->Name == Value)
			{
				return Item;
			}
		}
	}
	return nullptr;
}

FText SAvaRundownMacroCommandSelector::GetDisplayTextFromItem(const FCommandItemPtr& InItem) const
{
	FText DisplayText;
	
	const UEnum* CommandEnum = StaticEnum<EAvaRundownEditorMacroCommand>();
	if (ensure(CommandEnum) && InItem.IsValid())
	{
		DisplayText = CommandEnum->GetDisplayNameTextByIndex(InItem->EnumIndex);
	}
	
	return DisplayText;
}

void SAvaRundownMacroCommandSelector::UpdateCommandItems()
{
	if (!CommandItems.IsEmpty())
	{
		return;
	}
	
	const UEnum* CommandEnum = StaticEnum<EAvaRundownEditorMacroCommand>();
	if (ensure(CommandEnum))
	{
		CommandItems.Reset(CommandEnum->NumEnums() - 1);

		for (int32 EnumIndex = 0; EnumIndex < CommandEnum->NumEnums() - 1; ++EnumIndex)
		{
			FName ShortCommandName(CommandEnum->GetNameStringByIndex(EnumIndex));
			CommandItems.Add(MakeShared<FCommandItem>(EnumIndex, ShortCommandName));
		}
	}
}
