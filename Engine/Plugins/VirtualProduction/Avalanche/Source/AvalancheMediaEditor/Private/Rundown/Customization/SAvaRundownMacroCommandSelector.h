// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"

class IPropertyHandle;

class SAvaRundownMacroCommandSelector : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaRundownMacroCommandSelector){}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TSharedPtr<IPropertyHandle> InPropertyHandle);

	struct FCommandItem
	{
		int32 EnumIndex = 0;
		FName Name;

		FCommandItem(int32 InEnumIndex, FName InName) : EnumIndex(InEnumIndex), Name(InName) {}
	};

	using FCommandItemPtr = TSharedPtr<FCommandItem>;
	
	TSharedRef<SWidget> GenerateWidget(FCommandItemPtr InItem);
	
	void HandleSelectionChanged(SComboBox<FCommandItemPtr>::NullableOptionType InProposedSelection, ESelectInfo::Type InSelectInfo);
	
	FText GetDisplayTextFromProperty() const;
	
	void OnComboBoxOpening();
	
protected:
	FCommandItemPtr GetItemFromProperty() const;
	
	FText GetDisplayTextFromItem(const FCommandItemPtr& InItem) const;
	
	void UpdateCommandItems();
	
	TSharedPtr<IPropertyHandle> PropertyHandle;
	
	TSharedPtr<SComboBox<FCommandItemPtr>> CommandCombo;

	static TArray<FCommandItemPtr> CommandItems;
};
