// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "StructUtils/InstancedStruct.h"

#include "AvaRundownPageDetailView.generated.h"


/** Class used to expose part of the rundown page data to the detail panel. */
UCLASS()
class UAvaRundownPageDetailView : public UObject
{
	GENERATED_BODY()

public:
	/** Page Commands */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Page Data", meta = (ExcludeBaseStruct, BaseStruct = "/Script/AvalancheMedia.AvaRundownPageCommand"))
	TArray<FInstancedStruct> Commands;
};
