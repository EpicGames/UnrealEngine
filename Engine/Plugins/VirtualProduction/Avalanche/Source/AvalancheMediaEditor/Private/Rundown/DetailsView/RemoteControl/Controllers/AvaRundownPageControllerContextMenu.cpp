// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageControllerContextMenu.h"
#include "AvaRundownRCControllerItem.h"
#include "Controller/RCController.h"
#include "Controller/RCCustomControllerUtilities.h"
#include "IAvaMediaEditorModule.h"
#include "Playable/AvaPlayableRemoteControl.h"
#include "Rundown/AvaRundownCommands.h"
#include "Rundown/AvaRundownEditor.h"
#include "Rundown/AvaRundownPage.h"
#include "Rundown/DetailsView/RemoteControl/Properties/SAvaRundownPageRemoteControlProps.h"
#include "SAvaRundownRCControllerPanel.h"
#include "ScopedTransaction.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define LOCTEXT_NAMESPACE "AvaRundownPageControllerContextMenu"

FAvaRundownPageControllerContextMenu::FAvaRundownPageControllerContextMenu(const TWeakPtr<FUICommandList>& InCommandListWeak)
{
	BindCommands(InCommandListWeak);
}

void FAvaRundownPageControllerContextMenu::BindCommands(const TWeakPtr<FUICommandList>& InCommandListWeak)
{
	TSharedPtr<FUICommandList> CommandList = InCommandListWeak.Pin();
	if (!CommandList.IsValid())
	{
		return;
	}

	CommandListWeak = InCommandListWeak;

	const FAvaRundownCommands& RundownCommands = FAvaRundownCommands::Get();

	CommandList->MapAction(RundownCommands.ResetValuesToDefaults,
		FExecuteAction::CreateRaw(this, &FAvaRundownPageControllerContextMenu::ResetValuesToDefaults, false),
		FCanExecuteAction::CreateRaw(this, &FAvaRundownPageControllerContextMenu::CanResetValuesToDefaults, false));

	CommandList->MapAction(RundownCommands.ResetValuesToTemplate,
		FExecuteAction::CreateRaw(this, &FAvaRundownPageControllerContextMenu::ResetValuesToDefaults, true),
		FCanExecuteAction::CreateRaw(this, &FAvaRundownPageControllerContextMenu::CanResetValuesToDefaults, true));
}

TSharedRef<SWidget> FAvaRundownPageControllerContextMenu::GeneratePageContextMenuWidget(const TWeakPtr<FAvaRundownEditor>& InRundownEditorWeak
	, const FAvaRundownPage& InRundownPage
	, const TWeakPtr<SAvaRundownRCControllerPanel>& InControllerListWidgetWeak)
{
	UToolMenus* const ToolMenus = UToolMenus::Get();
	check(ToolMenus);
	
	static const FName ContextMenuName = TEXT("RundownRCControllerPanel");

	if (!ToolMenus->IsMenuRegistered(ContextMenuName))
	{
		UToolMenu* const ContextMenu = ToolMenus->RegisterMenu(ContextMenuName, NAME_None, EMultiBoxType::Menu);
		check(ContextMenu);

		ContextMenu->AddDynamicSection(TEXT("PopulateContextMenu"), FNewToolMenuDelegate::CreateLambda(
			[this](UToolMenu* const InMenu)
			{
				if (IsValid(InMenu))
				{
					PopulatePageContextMenu(*InMenu, InMenu->FindContext<UAvaRundownPageControllerContext>());
				}
			}));
	}

	UAvaRundownPageControllerContext* const ContextObject = NewObject<UAvaRundownPageControllerContext>();
	check(ContextObject);
	ContextObject->InitContext(InRundownEditorWeak, InRundownPage.GetPageId(), InControllerListWidgetWeak);

	TSharedPtr<FExtender> Extender;

	// Compatibility with IAvaMediaEditorModule Rundown Menu Extensibility Manager
	TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin();
	if (CommandList.IsValid())
	{
		TSharedPtr<FExtensibilityManager> MenuExtensibility = IAvaMediaEditorModule::Get().GetRundownMenuExtensibilityManager();
		if (MenuExtensibility.IsValid())
		{
			if (TSharedPtr<FAvaRundownEditor> RundownEditor = InRundownEditorWeak.Pin())
			{
				if (const TArray<UObject*>* EditingObjects = RundownEditor->GetObjectsCurrentlyBeingEdited())
				{
					Extender = MenuExtensibility->GetAllExtenders(CommandList.ToSharedRef(), *EditingObjects);
				}
			}
		}
	}

	FToolMenuContext Context(CommandList, Extender, ContextObject);
	return ToolMenus->GenerateWidget(ContextMenuName, Context);
}

void FAvaRundownPageControllerContextMenu::PopulatePageContextMenu(UToolMenu& InMenu, UAvaRundownPageControllerContext* const InContext)
{
	CurrentContext = InContext;

	if (!IsValid(InContext))
	{
		return;
	}

	TSharedPtr<SAvaRundownRCControllerPanel> ListWidget = CurrentContext->GetControllerListWidget();
	if (ListWidget.IsValid() && ListWidget->GetSelectedControllerItems().Num() == 0)
	{
		return;
	}

	UAvaRundown* const Rundown = GetContextRundown();
	if (!IsValid(Rundown))
	{
		return;
	}

	const FAvaRundownPage& RundownPage = Rundown->GetPage(CurrentContext->GetRundownPageId());
	if (!RundownPage.IsValidPage())
	{
		return;
	}

	const FAvaRundownCommands& RundownCommands = FAvaRundownCommands::Get();

	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("ControllerActions"), LOCTEXT("ControllerActions", "Controller Actions"));

	Section.AddMenuEntry(TEXT("ResetControllerToDefaults"),
		RundownCommands.ResetValuesToDefaults,
		LOCTEXT("ResetControllerToDefaults", "Reset to Defaults"),
		LOCTEXT("ResetControllerToDefaultsToolTip", "Reset selected controller values to the defaults."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault")));

	if (!RundownPage.IsTemplate())
	{
		Section.AddMenuEntry(TEXT("ResetControllerToTemplate"),
			RundownCommands.ResetValuesToTemplate,
			LOCTEXT("ResetControllerToTemplate", "Reset to Template"),
			LOCTEXT("ResetControllerToTemplateToolTip", "Reset selected controller values to the defaults."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault")));
	}
}

TArray<TWeakPtr<FAvaRundownRCControllerItem>> FAvaRundownPageControllerContextMenu::GetSelectedControllerItems() const
{
	TArray<TWeakPtr<FAvaRundownRCControllerItem>> OutItems;

	if (CurrentContext.IsValid())
	{
		if (TSharedPtr<SAvaRundownRCControllerPanel> ControllerListWidget = CurrentContext->GetControllerListWidget())
		{
			for (const FAvaRundownRCControllerItemPtr& ControllerItem : ControllerListWidget->GetSelectedControllerItems())
			{
				if (ControllerItem.IsValid())
				{
					OutItems.Add(ControllerItem);
				}
			}
		}
	}

	return OutItems;
}

bool FAvaRundownPageControllerContextMenu::HasValidSelectedItems() const
{
	return GetSelectedControllerItems().Num() > 0;
}

void FAvaRundownPageControllerContextMenu::ResetValuesToDefaults(const bool bInUseTemplateValues)
{
	const TSharedPtr<FAvaRundownEditor> RundownEditor = CurrentContext.IsValid() ? CurrentContext->GetRundownEditor() : nullptr;
	UAvaRundown* const Rundown = RundownEditor.IsValid() ? RundownEditor->GetRundown() : nullptr;
	if (!IsValid(Rundown))
	{
		return;
	}

	const FAvaRundownPage& Page = Rundown->GetPage(CurrentContext->GetRundownPageId());
	if (!Page.IsValidPage())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ResetControllerValuesTransaction", "Reset RC Controller Values"));
	Rundown->Modify();

	bool bControllerValueModified = false;
	
	TArray<TWeakPtr<FAvaRundownRCControllerItem>> SelectedControllerItems = GetSelectedControllerItems();
	for (const TWeakPtr<FAvaRundownRCControllerItem>& ControllerItemWeak : SelectedControllerItems)
	{
		if (const TSharedPtr<FAvaRundownRCControllerItem> ControllerItem = ControllerItemWeak.Pin())
		{
			URCController* const Controller = ControllerItem->GetController();
			if (IsValid(Controller))
			{
				const EAvaPlayableRemoteControlChanges Changes = Rundown->ResetRemoteControlControllerValue(
					CurrentContext->GetRundownPageId(), Controller->Id, bInUseTemplateValues, /*bInIsDefault=*/false);

				if (Changes != EAvaPlayableRemoteControlChanges::None)
				{
					RundownEditor->MarkAsModified();
					
					// Apply the reset controller values from the page to the RCP so it updates the behaviors (and controlled entities).
					using namespace UE::AvaPlayableRemoteControl;
					if (const FAvaPlayableRemoteControlValue* ControllerValue = Page.GetRemoteControlValues().GetControllerValue(Controller->Id))
					{
						// This will execute the behaviours.
						if (SetValueOfController(Controller, ControllerValue->Value) == EAvaPlayableRemoteControlResult::Completed)
						{
							bControllerValueModified = true;
							
							// Read values of the controlled entities and save that back in the page.
							if (const URemoteControlPreset* Preset = Controller->PresetWeakPtr.Get())
							{
								TSet<FGuid> EntityIds;
								UE::RCCustomControllers::GetEntitiesControlledByController(Preset, Controller, EntityIds);
								SAvaRundownPageRemoteControlProps::SaveRemoteControlEntitiesToPage(Preset, EntityIds, Rundown, CurrentContext->GetRundownPageId());
							}
						}
					}
				}
			}
		}
	}

	if (bControllerValueModified)
	{
		if (const TSharedPtr<SAvaRundownRCControllerPanel> ControllerListWidget = CurrentContext->GetControllerListWidget())
		{
			ControllerListWidget->UpdatePageSummary(/*bInForceUpdate*/ true);
		}
	}
}

bool FAvaRundownPageControllerContextMenu::CanResetValuesToDefaults(const bool bInUseTemplateValues) const
{
	UAvaRundown* const Rundown = GetContextRundown();
	if (!IsValid(Rundown))
	{
		return false;
	}

	const FAvaRundownPage& RundownPage = Rundown->GetPage(CurrentContext->GetRundownPageId());
	if (!RundownPage.IsValidPage())
	{
		return false;
	}

	const TArray<TWeakPtr<FAvaRundownRCControllerItem>> SelectedItems = GetSelectedControllerItems();
	if (SelectedItems.Num() == 0)
	{
		return false;
	}

	for (const TWeakPtr<FAvaRundownRCControllerItem>& ControllerItemWeak : SelectedItems)
	{
		if (const TSharedPtr<FAvaRundownRCControllerItem> ControllerItem = ControllerItemWeak.Pin())
		{
			const URCController* const Controller = ControllerItem->GetController();
			if (IsValid(Controller))
			{
				if (!RundownPage.IsDefaultControllerValue(Rundown, Controller->Id, bInUseTemplateValues))
				{
					return true;
				}
			}
		}
	}

	return false;
}

UAvaRundown* FAvaRundownPageControllerContextMenu::GetContextRundown() const
{
	if (!CurrentContext.IsValid())
	{
		return nullptr;
	}

	const TSharedPtr<FAvaRundownEditor> RundownEditor = CurrentContext->GetRundownEditor();
	if (!RundownEditor.IsValid())
	{
		return nullptr;
	}

	return RundownEditor->GetRundown();
}

#undef LOCTEXT_NAMESPACE
