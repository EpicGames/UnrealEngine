// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/Pages/AvaRundownPageControllerContext.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplatesFwd.h"

class FAvaRundownEditor;
class FAvaRundownRCControllerItem;
class FUICommandList;
class SAvaRundownRCControllerPanel;
class SWidget;
class UAvaRundownPageControllerContext;
class UToolMenu;
struct FAvaRundownPage;

class FAvaRundownPageControllerContextMenu : public TSharedFromThis<FAvaRundownPageControllerContextMenu>
{
public:
	FAvaRundownPageControllerContextMenu(const TWeakPtr<FUICommandList>& InCommandList);

	TSharedRef<SWidget> GeneratePageContextMenuWidget(const TWeakPtr<FAvaRundownEditor>& InRundownEditorWeak
		, const FAvaRundownPage& InRundownPage
		, const TWeakPtr<SAvaRundownRCControllerPanel>& InControllerListWidgetWeak);

	bool HasValidSelectedItems() const;

private:
	void PopulatePageContextMenu(UToolMenu& InMenu, UAvaRundownPageControllerContext* const InContext);

	void BindCommands(const TWeakPtr<FUICommandList>& InCommandListWeak);

	void ResetValuesToDefaults(const bool bInUseTemplateValues);
	bool CanResetValuesToDefaults(const bool bInUseTemplateValues) const;

	TArray<TWeakPtr<FAvaRundownRCControllerItem>> GetSelectedControllerItems() const;

	UAvaRundown* GetContextRundown() const;

	TWeakPtr<FUICommandList> CommandListWeak;

	TWeakObjectPtr<UAvaRundownPageControllerContext> CurrentContext;
};
