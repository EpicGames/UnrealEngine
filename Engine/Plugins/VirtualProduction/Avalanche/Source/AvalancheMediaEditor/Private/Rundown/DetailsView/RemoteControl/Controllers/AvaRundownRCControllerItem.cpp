// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownRCControllerItem.h"
#include "Controller/RCController.h"
#include "IRemoteControlUIModule.h"
#include "RCVirtualProperty.h"
#include "RemoteControlPreset.h"
#include "SAvaRundownRCControllerItemRow.h"
#include "Playable/AvaPlayableRemoteControlPresetInfo.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define LOCTEXT_NAMESPACE "AvaRundownRCControllerItem"

namespace UE::AvaRundownRCControllerItem::Utils
{
	FText GetControllerDisplayName(URemoteControlPreset* InPreset, const FGuid& InControllerId)
	{
		if (InPreset)
		{
			if (URCVirtualPropertyBase* Controller = InPreset->GetController(InControllerId))
			{
				const FName DisplayName = Controller->DisplayName.IsNone() ? Controller->PropertyName : Controller->DisplayName;
				return FText::FromName(DisplayName);
			}
		}
		return FText::FromString(InControllerId.ToString());
	}

	FText GetEntityDisplayName(URemoteControlPreset* InPreset, const FGuid& InEntityId)
	{
		if (InPreset)
		{
			TWeakPtr<FRemoteControlEntity> EntityWeak = InPreset->GetExposedEntity(InEntityId);
			if (TSharedPtr<FRemoteControlEntity> Entity = EntityWeak.Pin())
			{
				return FText::FromName(Entity->GetLabel());
			}
		}
		return FText::FromString(InEntityId.ToString());
	}
}

FAvaRundownRCControllerItem::FAvaRundownRCControllerItem(int32 InInstanceIndex, FName InAssetName, URCController* InController, const TSharedRef<IDetailTreeNode>& InTreeNode, const FAvaPlayableRemoteControlPresetInfo& InPresetInfo)
{
	AssetName = InAssetName;
	InstanceIndex = InInstanceIndex;
	ControllerWeak = InController;
	
	if (InController)
	{
		DisplayIndex = InController->DisplayIndex;
		
		const FName DisplayName = InController->DisplayName.IsNone()
			? InController->PropertyName
			: InController->DisplayName;
		
		DisplayNameText = FText::FromName(DisplayName);
		DescriptionText = InController->Description;

		{
			// Build formatted tool tip lines with information from the controller.
			TArray<FText> ToolTipLines;
			ToolTipLines.Reserve(5 + InController->Metadata.Num());

			ToolTipLines.Add(FText::Format(LOCTEXT("Controller_ToolTipLine_Name", "Name: \"{0}\""), FText::FromString(InController->PropertyName.ToString())));
			ToolTipLines.Add(FText::Format(LOCTEXT("Controller_ToolTipLine_Id", "Id: {0}"), FText::FromString(InController->Id.ToString())));
			if (!InController->FieldId.IsNone())
			{
				ToolTipLines.Add(FText::Format(LOCTEXT("Controller_ToolTipLine_FieldId", "FieldId: \"{0}\""), FText::FromString(InController->FieldId.ToString())));
			}
			if (!InController->Description.IsEmpty())
			{
				ToolTipLines.Add(FText::Format(LOCTEXT("Controller_ToolTipLine_Desc", "Description: {0}"), InController->Description));
			}
			ToolTipLines.Add(FText::Format(LOCTEXT("Controller_ToolTipLine_DisplayIndex", "Display Index: {0}"), InController->DisplayIndex));
			
			if (InPresetInfo.IsControllerOverlapping(InController->Id))
			{
				// We want to report to the user which controller is overlapping and why.
				using namespace UE::AvaRundownRCControllerItem::Utils;
				if (URemoteControlPreset* Preset = InController->PresetWeakPtr.Get())
				{
					for (const TPair<FGuid, FAvaPlayableRemoteControlControlledEntityInfo>& ControlledEntity : InPresetInfo.EntitiesControlledByController)
					{
						if (ControlledEntity.Value.ControlledBy.Contains(InController->Id) && ControlledEntity.Value.ControlledBy.Num() > 1)
						{
							for (const FGuid& OtherController : ControlledEntity.Value.ControlledBy)
							{
								if (OtherController != InController->Id)
								{
									ToolTipLines.Add(
										FText::Format(LOCTEXT("Controller_ToolTipLine_OverlappingWith", "Controller is overlapping with \"{0}\" because of entity \"{1}\""),
											GetControllerDisplayName(Preset, OtherController), GetEntityDisplayName(Preset, ControlledEntity.Key)));
								}
							}
						}
					}
				}
			}
			else
			{
				ToolTipLines.Add(LOCTEXT("Controller_ToolTipLine_NonOverlapping", "Controller is non-overlapping"));
			}

			for (const TPair<FName, FString>& MetaDataEntry : InController->Metadata)
			{
				if (!MetaDataEntry.Value.IsEmpty())
				{
					ToolTipLines.Add(FText::Format(LOCTEXT("Controller_ToolTipLine_Metadata", "Metadata: \"{0}\", Value: \"{1}\""),
						FText::FromName(MetaDataEntry.Key), FText::FromString(MetaDataEntry.Value)));
				}
			}

			ToolTipText = FText::Join(LOCTEXT("Controller_ToolTipLine_Delimiter", "\n"), ToolTipLines);
		}
		
		NodeWidgets = InTreeNode->CreateNodeWidgets();

		// Check if the Controller has a custom widget. In that case, overwrite the value widget with it
		if (const TSharedPtr<SWidget>& CustomControllerWidget = IRemoteControlUIModule::Get().CreateCustomControllerWidget(InController, InTreeNode->CreatePropertyHandle()))
		{
			NodeWidgets.ValueWidget = CustomControllerWidget;
		}
	}
}

TSharedRef<ITableRow> FAvaRundownRCControllerItem::CreateWidget(TSharedRef<SAvaRundownRCControllerPanel> InControllerPanel, const TSharedRef<STableViewBase>& InOwnerTable) const
{
	return SNew(SAvaRundownRCControllerItemRow, InControllerPanel, InOwnerTable, SharedThis(this));
}

URCController* FAvaRundownRCControllerItem::GetController() const
{
	return ControllerWeak.Get();
}

void FAvaRundownRCControllerItem::ExecuteController() const
{
	if (URCController* Controller = GetController())
	{
		Controller->ExecuteBehaviours();

		if (URemoteControlPreset* Preset = Controller->PresetWeakPtr.Get())
		{
			Preset->OnControllerModified().Broadcast(Preset, {Controller->Id});
		}
	}
}

#undef LOCTEXT_NAMESPACE
