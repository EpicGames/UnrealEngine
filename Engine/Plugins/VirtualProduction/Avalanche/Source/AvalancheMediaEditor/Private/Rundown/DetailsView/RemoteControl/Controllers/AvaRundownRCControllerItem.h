// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailTreeNode.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplatesFwd.h"

class ITableRow;
class SAvaRundownRCControllerPanel;
class STableViewBase;
class URCController;
struct FAvaPlayableRemoteControlPresetInfo;

class FAvaRundownRCControllerItem : public TSharedFromThis<FAvaRundownRCControllerItem>
{
public:
	FAvaRundownRCControllerItem(int32 InInstanceIndex, FName InAssetName, URCController* InController, const TSharedRef<IDetailTreeNode>& InTreeNode, const FAvaPlayableRemoteControlPresetInfo& InPresetInfo);
	
	TSharedRef<ITableRow> CreateWidget(TSharedRef<SAvaRundownRCControllerPanel> InControllerPanel, const TSharedRef<STableViewBase>& InOwnerTable) const;

	FText GetDisplayName() const { return DisplayNameText; }
	FText GetDescriptionText() const { return DescriptionText; }
	FText GetToolTipText() const { return ToolTipText; }
	const FNodeWidgets& GetNodeWidgets() const { return NodeWidgets; }
	
	int32 GetInstanceIndex() const { return InstanceIndex; }
	int32 GetDisplayIndex() const { return DisplayIndex; }
	FName GetAssetName() const { return AssetName; }

	URCController* GetController() const;

	/** Executes the behaviors of the controller that this item references */
	void ExecuteController() const;

private:
	int32 InstanceIndex = 0;
	int32 DisplayIndex = 0;
	FName AssetName;
	FText DisplayNameText;
	FText DescriptionText;
	FText ToolTipText;
	FNodeWidgets NodeWidgets;

	TWeakObjectPtr<URCController> ControllerWeak;
};
