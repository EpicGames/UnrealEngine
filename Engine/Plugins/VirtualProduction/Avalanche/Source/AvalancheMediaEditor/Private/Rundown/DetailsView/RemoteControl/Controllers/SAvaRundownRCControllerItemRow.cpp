// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownRCControllerItemRow.h"
#include "AvaRundownRCControllerItem.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScissorRectBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaRundownRCControllerItemRow"

void SAvaRundownRCControllerItemRow::Construct(const FArguments& InArgs, TSharedRef<SAvaRundownRCControllerPanel> InControllerPanel,
	const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<const FAvaRundownRCControllerItem>& InRowItem)
{
	ItemPtrWeak = InRowItem;
	ControllerPanelWeak = InControllerPanel;

	SMultiColumnTableRow<FAvaRundownRCControllerItemPtr>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

TSharedRef<SWidget> SAvaRundownRCControllerItemRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	TSharedPtr<const FAvaRundownRCControllerItem> ItemPtr = ItemPtrWeak.Pin();

	if (ItemPtr.IsValid())
	{
		if (InColumnName == SAvaRundownRCControllerPanel::ControllerColumnName)
		{
			return SNew(SScissorRectBox)
				[
					SNew(SBox)
					.VAlign(EVerticalAlignment::VAlign_Center)
					.Padding(3.f, 2.f, 3.f, 2.f)
					[
						SNew(STextBlock)
						.Text(ItemPtr->GetDisplayName())
						.ToolTipText(ItemPtr->GetToolTipText())
					]
				];
		}
		else if (InColumnName == SAvaRundownRCControllerPanel::DescriptionColumnName)
		{
			return SNew(SScissorRectBox)
				[
					SNew(SBox)
					.VAlign(EVerticalAlignment::VAlign_Center)
					.Padding(3.f, 2.f, 3.f, 2.f)
					[
						SNew(STextBlock)
						.Text(ItemPtr->GetDescriptionText())
						.ToolTipText(ItemPtr->GetToolTipText())
					]
				];
		}
		else if (InColumnName == SAvaRundownRCControllerPanel::ValueColumnName)
		{
			if (ItemPtr->GetNodeWidgets().ValueWidget.IsValid())
			{
				return ItemPtr->GetNodeWidgets().ValueWidget.ToSharedRef();
			}
			else if (ItemPtr->GetNodeWidgets().WholeRowWidget.IsValid())
			{
				return ItemPtr->GetNodeWidgets().WholeRowWidget.ToSharedRef();
			}
		}
		else if (InColumnName == SAvaRundownRCControllerPanel::ExecuteColumnName)
		{
			return SNew(SButton)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText(LOCTEXT("ExecuteController_Tooltip", "Executes the controller's behaviors, as if its value had been committed."))
				.ContentPadding(FMargin(4, 0))
				.Visibility(this, &SAvaRundownRCControllerItemRow::GetExecuteButtonVisibility)
				.OnClicked(this, &SAvaRundownRCControllerItemRow::OnExecuteController)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("Icons.Play"))
				];
		}
	}

	return SNullWidget::NullWidget;
}

EVisibility SAvaRundownRCControllerItemRow::GetExecuteButtonVisibility() const
{
	return IsHovered() ? EVisibility::Visible : EVisibility::Hidden;
}

FReply SAvaRundownRCControllerItemRow::OnExecuteController()
{
	if (TSharedPtr<const FAvaRundownRCControllerItem> Item = ItemPtrWeak.Pin())
	{
		Item->ExecuteController();
	}
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
