// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaRundownRCControllerItem.h"
#include "SAvaRundownRCControllerPanel.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class SAvaRundownRCControllerItemRow : public SMultiColumnTableRow<FAvaRundownRCControllerItemPtr>
{
public:
	SLATE_BEGIN_ARGS(SAvaRundownRCControllerItemRow) { }
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<SAvaRundownRCControllerPanel> InControllerPanel, 
		const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<const FAvaRundownRCControllerItem>& InRowItem);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

protected:
	EVisibility GetExecuteButtonVisibility() const;
	FReply OnExecuteController();

	TWeakPtr<const FAvaRundownRCControllerItem> ItemPtrWeak;
	TWeakPtr<SAvaRundownRCControllerPanel> ControllerPanelWeak;
};
