// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownRCControllerPanel.h"
#include "AvaRundownRCControllerItem.h"
#include "Behaviour/Builtin/Path/RCSetAssetByPathBehaviour.h"
#include "Controller/RCController.h"
#include "IAvaMediaModule.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Modules/ModuleManager.h"
#include "Playable/AvaPlayableRemoteControl.h"
#include "RCVirtualProperty.h"
#include "Rundown/AvaRundownEditor.h"
#include "Rundown/AvaRundownEditorUtils.h"
#include "Rundown/AvaRundownManagedInstanceCache.h"
#include "Rundown/AvaRundownPage.h"
#include "Rundown/DetailsView/RemoteControl/Controllers/AvaRundownPageControllerContextMenu.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SAvaRundownRCControllerPanel"

const FLazyName SAvaRundownRCControllerPanel::ControllerColumnName = "ControllerColumn";
const FLazyName SAvaRundownRCControllerPanel::DescriptionColumnName = "DescriptionColumn";
const FLazyName SAvaRundownRCControllerPanel::ValueColumnName = "ValueColumn";
const FLazyName SAvaRundownRCControllerPanel::ExecuteColumnName = "ExecuteColumn";

void SAvaRundownRCControllerPanel::Construct(const FArguments& InArgs, const TSharedPtr<FAvaRundownEditor>& InRundownEditor)
{
	RundownEditorWeak = InRundownEditor;
	ActivePageId = FAvaRundownPage::InvalidPageId;

	CommandList = MakeShared<FUICommandList>();

	ContextMenu = MakeShared<FAvaRundownPageControllerContextMenu>(CommandList);

	ChildSlot
	[
		SNew(SBorder)
		.Padding(8.0f)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SAssignNew(ControllerContainer, SListView<FAvaRundownRCControllerItemPtr>)
			.ListItemsSource(&ControllerItems)
			.SelectionMode(ESelectionMode::Multi)
			.OnContextMenuOpening(this, &SAvaRundownRCControllerPanel::GetContextMenuContent)
			.OnGenerateRow(this, &SAvaRundownRCControllerPanel::OnGenerateControllerRow)
			.HeaderRow(
				SNew(SHeaderRow)
				.CanSelectGeneratedColumn(true)
				+ SHeaderRow::Column(ControllerColumnName)
				.DefaultLabel(LOCTEXT("Controller", "Controller"))
				.FillWidth(0.1f)
				+ SHeaderRow::Column(DescriptionColumnName)
				.DefaultLabel(LOCTEXT("Description", "Description"))
				.FillWidth(0.2f)
				+ SHeaderRow::Column(ValueColumnName)
				.DefaultLabel(LOCTEXT("Value", "Value"))
				.FillWidth(0.7f)
				+ SHeaderRow::Column(ExecuteColumnName)
				.DefaultLabel(LOCTEXT("Execute", "Execute"))
				.ManualWidth(64)
				.HeaderContent()
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText(LOCTEXT("ExecuteController_Tooltip", "Executes all the controllers' behaviors in this list from top to bottom order, as if their values had been committed."))
					.ContentPadding(FMargin(4, 0))
					.OnClicked(this, &SAvaRundownRCControllerPanel::OnExecuteAllControllers)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::Get().GetBrush("Icons.Play"))
					]
				]
			)
		]
	];
	
	Refresh({});
}

bool SAvaRundownRCControllerPanel::HasRemoteControlPreset(const URemoteControlPreset* InPreset) const
{
	for (const TSharedPtr<FAvaRundownManagedInstance>& ManagedInstance : ManagedHandles.Instances)
	{
		if (ManagedInstance && ManagedInstance->GetRemoteControlPreset() == InPreset)
		{
			return true;
		}
	}
	return false;
}

void SAvaRundownRCControllerPanel::UpdatePropertyRowGenerators(int32 InNumGenerators)
{
	if (PropertyRowGenerators.Num() != InNumGenerators)
	{
		TArray<TUniquePtr<FPropertyRowGeneratorWrapper>> GeneratorPool = MoveTemp(PropertyRowGenerators);

		PropertyRowGenerators.Empty(InNumGenerators);

		for (int32 Index = 0; Index < InNumGenerators; ++Index)
		{
			if (Index < GeneratorPool.Num())
			{
				PropertyRowGenerators.Add(MoveTemp(GeneratorPool[Index]));
			}
			else
			{
				PropertyRowGenerators.Add(MakeUnique<FPropertyRowGeneratorWrapper>(this));
			}
		}
	}
}

void SAvaRundownRCControllerPanel::RefreshForManagedInstance(int32 InInstanceIndex, const FAvaRundownManagedInstance& InManagedInstance, const FAvaRundownPage& InPage)
{
	URemoteControlPreset* const Preset = InManagedInstance.GetRemoteControlPreset();

	if (!Preset || !PropertyRowGenerators.IsValidIndex(InInstanceIndex)
		|| !PropertyRowGenerators[InInstanceIndex] || !PropertyRowGenerators[InInstanceIndex]->PropertyRowGenerator)
	{
		return;
	}
	
	BindRemoteControlDelegates(Preset);

	if (const TSharedPtr<FStructOnScope> StructOnScope = Preset->GetControllerContainerStructOnScope())
	{
		IPropertyRowGenerator& PropertyRowGenerator = *PropertyRowGenerators[InInstanceIndex]->PropertyRowGenerator;
		
		// We need one of those for each Preset.
		PropertyRowGenerator.SetStructure(StructOnScope);
		PropertyRowGenerators[InInstanceIndex]->PresetWeak = Preset;	// Keep track for proper event routing.
		
		const TArray<TSharedRef<IDetailTreeNode>>& RootTreeNodes = PropertyRowGenerator.GetRootTreeNodes();
		check(RootTreeNodes.Num() <= 1);

		for (const TSharedRef<IDetailTreeNode>& RootTreeNode : RootTreeNodes)
		{
			TArray<TSharedRef<IDetailTreeNode>> Children;
			RootTreeNode->GetChildren(Children);

			for (TSharedRef<IDetailTreeNode>& Child : Children)
			{
				FProperty* const Property = Child->CreatePropertyHandle()->GetProperty();
				check(Property);

				if (Property->IsA<FStrProperty>() || Property->IsA<FTextProperty>())
				{
					Property->SetMetaData(TEXT("MultiLine"), TEXT("true"));
				}

				URCVirtualPropertyBase* const VirtualProperty = Preset->GetController(Property->GetFName());

				if (!VirtualProperty || FAvaPlayableRemoteControlValues::ShouldIgnoreController(VirtualProperty))
				{
					continue;
				}

				// Apply the page value to the controller (sync the managed RCP's controller to the page value).
				{
					const FAvaPlayableRemoteControlValue* ControllerValueFromPage = InPage.GetRemoteControlControllerValue(VirtualProperty->Id);
					if (!ControllerValueFromPage)
					{
						// If the value is not set in the page, fallback to default value from the template.
						if (const FAvaPlayableRemoteControlValue* const DefaultControllerValue = InManagedInstance.GetDefaultRemoteControlValues().GetControllerValue(VirtualProperty->Id))
						{
							// Ensure the default values have the default flag.
							ensure(DefaultControllerValue->bIsDefault == true);

							// WYSIWYG (Solution):
							// Because Deterministic and "special" controllers are applied at runtime, 
							// we need to add the default value (flagged as default) in the current page.
							if (!SetSelectedPageControllerValue(VirtualProperty, *DefaultControllerValue))
							{
								UE_LOGF(LogAvaPlayableRemoteControl, Error, "Controller \"%ls\" (id:%ls): failed to set value in currently selected page.",
									*VirtualProperty->DisplayName.ToString(), *VirtualProperty->Id.ToString());
							}

							ControllerValueFromPage = DefaultControllerValue;
						}
						else
						{
							UE_LOGF(LogAvaPlayableRemoteControl, Error, "Controller \"%ls\" (id:%ls) doesn't have a template default value.",
								*VirtualProperty->DisplayName.ToString(), *VirtualProperty->Id.ToString());
						}
					}

					if (ControllerValueFromPage)
					{
						using namespace UE::AvaPlayableRemoteControl;
						FAvaPlayableRemoteControlValue CurrentControllerValue;
						EAvaPlayableRemoteControlResult RemoteControlResult = GetValueOfController(VirtualProperty, CurrentControllerValue.Value);
						if (!Failed(RemoteControlResult))
						{
							if (!CurrentControllerValue.IsSameValueAs(*ControllerValueFromPage))
							{
								// This serves only to sync the Managed RCP's controller values to the page values.
								// We want to be able to do this without affecting the entity values (preserve WYSIWYG), thus we disable behaviors.
								if (Failed(RemoteControlResult = SetValueOfController(VirtualProperty, ControllerValueFromPage->Value, /*bInBehaviorsEnabled*/ false)))
								{
									UE_LOGF(LogAvaPlayableRemoteControl, Error,
										"Controller \"%ls\" (id:%ls): failed to set value in currently selected page: %ls.",
										*VirtualProperty->DisplayName.ToString(), *VirtualProperty->Id.ToString(), *EnumToString(RemoteControlResult));
								}
							}
						}
						else
						{
							UE_LOGF(LogAvaPlayableRemoteControl, Error,
								"Controller \"%ls\" (id:%ls): failed to get value in currently selected page: %ls.",
								*VirtualProperty->DisplayName.ToString(), *VirtualProperty->Id.ToString(), *EnumToString(RemoteControlResult));
						}
					}
				}

				if (URCController* const Controller = Cast<URCController>(VirtualProperty))
				{
					for (URCBehaviour* const Behavior : Controller->GetBehaviors())
					{
						if (URCSetAssetByPathBehaviour* const AssetByPathBehavior = Cast<URCSetAssetByPathBehaviour>(Behavior))
						{
							AssetByPathBehavior->UpdateTargetEntity();
						}
					}

					FName SourceAssetName = InManagedInstance.GetSourceAssetPath().GetAssetFName();
					const FAvaPlayableRemoteControlPresetInfo& PresetInfo = InManagedInstance.GetRemoteControlPresetInfo();
					ControllerItems.Add(MakeShared<FAvaRundownRCControllerItem>(InInstanceIndex, SourceAssetName, Controller, Child, PresetInfo));
				}
			}
		}
	}
}


void SAvaRundownRCControllerPanel::Refresh(const TArray<int32>& InSelectedPageIds)
{
	// Request to Rebuild on next tick
	ControllerContainer->RebuildList();

	ActivePageId = InSelectedPageIds.IsEmpty() ? FAvaRundownPage::InvalidPageId : InSelectedPageIds[0];

	UAvaRundown* Rundown = GetRundown();
	const FAvaRundownPage& Page = GetActivePage(Rundown);

	if (!Page.IsValidPage())
	{
		ControllerItems.Empty();
		ManagedHandles.Reset();
		return;
	}

	ManagedHandles = IAvaMediaModule::Get().GetManagedInstanceCache().GetManagedHandlesForPage(Rundown, Page);

	int32 NumItems = 0;
	for (const TSharedPtr<FAvaRundownManagedInstance>& ManagedInstance : ManagedHandles.Instances)
	{
		if (const URemoteControlPreset* Preset = ManagedInstance ? ManagedInstance->GetRemoteControlPreset() : nullptr)
		{
			// Only count the controllers that are not ignored.
			for (const URCVirtualPropertyBase* PropertyBase : Preset->GetControllers())
			{
				if (!FAvaPlayableRemoteControlValues::ShouldIgnoreController(PropertyBase))
				{
					NumItems++;
				}
			}
		}
	}

	ControllerItems.Empty(NumItems);
	
	UpdatePropertyRowGenerators(ManagedHandles.Num());
	UpdatePageSummary(/*bInForceUpdate*/ false); // Generate only if missing.

	for (int32 Index = 0; Index < ManagedHandles.Num(); ++Index)
	{
		RefreshForManagedInstance(Index, *ManagedHandles.Instances[Index], Page);
	}

	ControllerItems.Sort([](const FAvaRundownRCControllerItemPtr& A, const FAvaRundownRCControllerItemPtr& B)
	{
		if (A->GetInstanceIndex() == B->GetInstanceIndex())
		{
			return A->GetDisplayIndex() < B->GetDisplayIndex();
		}
		return A->GetInstanceIndex() < B->GetInstanceIndex();
	});
}

TSharedRef<ITableRow> SAvaRundownRCControllerPanel::OnGenerateControllerRow(FAvaRundownRCControllerItemPtr InItem
	, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return InItem->CreateWidget(SharedThis(this), InOwnerTable);
}

void SAvaRundownRCControllerPanel::UpdateDefaultValuesAndRefresh(const TArray<int32>& InSelectedPageIds)
{
	// Remark: The RC values might be already updated in SAvaRundownPageRemoteControlProps.
	// But order of callback is not guaranteed. Code could reach here first, so
	// it needs to update and refresh just in case. Calling UpdateDefaultRemoteControlValues
	// multiple time (from different code paths) is harmless (fast if nothing changed).
	
	if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		if (UAvaRundown* Rundown = RundownEditor->GetRundown())
		{
			using namespace UE::AvaRundownEditor::Utils;
			if (UpdateDefaultRemoteControlValues(Rundown, InSelectedPageIds) != EAvaPlayableRemoteControlChanges::None)
			{
				RundownEditor->MarkAsModified();
			}
		}
	}
	
	Refresh(InSelectedPageIds);
}

void SAvaRundownRCControllerPanel::OnRemoteControlControllerModified(URemoteControlPreset* InPreset, const TSet<FGuid>& InModifiedControllerIds)
{
	// Note: Ignore changes from the RCP Transaction listener.
	if (!IsValid(InPreset) || !HasRemoteControlPreset(InPreset) || GIsTransacting)
	{
		return;
	}

	if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		bool bModified = false;

		for (const FGuid& Id : InModifiedControllerIds)
		{
			if (URCVirtualPropertyBase* Controller = InPreset->GetController(Id))
			{
				using namespace UE::AvaPlayableRemoteControl;
				FAvaPlayableRemoteControlValue ControllerValue;
				const EAvaPlayableRemoteControlResult Result = GetValueOfController(Controller, ControllerValue.Value);

				if (Failed(Result))
				{
					UE_LOGF(LogAvaPlayableRemoteControl, Error, "Unable to get value of controller \"%ls\" (id:%ls): %ls.",
						   *Controller->DisplayName.ToString(), *Controller->Id.ToString(), *EnumToString(Result));
					continue;
				}

				const FAvaPlayableRemoteControlValue* StoredControllerValue = GetSelectedPageControllerValue(Controller);

				if (StoredControllerValue && ControllerValue.IsSameValueAs(*StoredControllerValue))
				{
					continue;	// Skip if value is identical.
				}

				if (!SetSelectedPageControllerValue(Controller, ControllerValue))
				{
					UE_LOGF(LogAvaPlayableRemoteControl, Error, "Unable to set page controller value for: \"%ls\"", *Controller->DisplayName.ToString());
					continue;
				}

				bModified = true;
			}
		}

		if (bModified)
		{
			RundownEditor->MarkAsModified();
		}
	}
}


void SAvaRundownRCControllerPanel::BindRemoteControlDelegates(URemoteControlPreset* InPreset)
{
	if (IsValid(InPreset))
	{
		if (!InPreset->OnControllerAdded().IsBoundToObject(this))
		{
			InPreset->OnControllerAdded().AddSP(this, &SAvaRundownRCControllerPanel::OnRemoteControlControllerAdded);
		}

		if (!InPreset->OnControllerRemoved().IsBoundToObject(this))
		{
			InPreset->OnControllerRemoved().AddSP(this, &SAvaRundownRCControllerPanel::OnRemoteControlControllerRemoved);
		}

		if (!InPreset->OnControllerRenamed().IsBoundToObject(this))
		{
			InPreset->OnControllerRenamed().AddSP(this, &SAvaRundownRCControllerPanel::OnRemoteControlControllerRenamed);
		}

		if (!InPreset->OnControllerModified().IsBoundToObject(this))
		{
			InPreset->OnControllerModified().AddSP(this, &SAvaRundownRCControllerPanel::OnRemoteControlControllerModified);
		}
	}
}

const FAvaPlayableRemoteControlValue* SAvaRundownRCControllerPanel::GetSelectedPageControllerValue(const URCVirtualPropertyBase* InController) const
{
	const FAvaRundownPage& Page = GetActivePage();
	if (IsValid(InController) && Page.IsValidPage())
	{
		return Page.GetRemoteControlControllerValue(InController->Id);
	}
	return nullptr;
}

bool SAvaRundownRCControllerPanel::SetSelectedPageControllerValue(const URCVirtualPropertyBase* InController, const FAvaPlayableRemoteControlValue& InValue) const
{
	UAvaRundown* Rundown = GetRundown();
	if (IsValid(InController) && Rundown)
	{
		// Using the rundown API for event propagation.
		return Rundown->SetRemoteControlControllerValue(ActivePageId, InController->Id, InValue);
	}
	return false;
}

void SAvaRundownRCControllerPanel::UpdatePageSummary(bool bInForceUpdate)
{
	FAvaRundownPage& Page = GetActivePageMutable();
	if (Page.IsValidPage())
	{
		TArray<const URemoteControlPreset*> Presets;
		Presets.Reserve(ManagedHandles.Num());
		for (const TSharedPtr<FAvaRundownManagedInstance>& ManagedInstance : ManagedHandles.Instances)
		{
			if (ManagedInstance && ManagedInstance->GetRemoteControlPreset())
			{
				Presets.Add(ManagedInstance->GetRemoteControlPreset());
			}
		}
		Page.UpdatePageSummary(GetRundown(), Presets, bInForceUpdate);
	}
}

FReply SAvaRundownRCControllerPanel::OnExecuteAllControllers()
{
	for (FAvaRundownRCControllerItemPtr& ControllerItem : ControllerItems)
	{
		if (ControllerItem.IsValid())
		{
			ControllerItem->ExecuteController();
		}
	}
	return FReply::Handled();
}

UAvaRundown* SAvaRundownRCControllerPanel::GetRundown() const
{
	if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		return RundownEditor->GetRundown();
	}
	return nullptr;
}

const FAvaRundownPage& SAvaRundownRCControllerPanel::GetActivePage(const UAvaRundown* InRundown) const
{
	if (IsValid(InRundown) && ActivePageId != FAvaRundownPage::InvalidPageId)
	{
		return InRundown->GetPage(ActivePageId);
	}
	return FAvaRundownPage::NullPage;
}

FAvaRundownPage& SAvaRundownRCControllerPanel::GetActivePageMutable(UAvaRundown* InRundown) const
{
	if (IsValid(InRundown) && ActivePageId != FAvaRundownPage::InvalidPageId)
	{
		return InRundown->GetPage(ActivePageId);
	}
	return FAvaRundownPage::NullPage;
}

const TArray<FAvaRundownRCControllerItemPtr> SAvaRundownRCControllerPanel::GetSelectedControllerItems() const
{
	return ControllerContainer->GetSelectedItems();
}

TSharedPtr<SWidget> SAvaRundownRCControllerPanel::GetContextMenuContent()
{
	const TArray<FAvaRundownRCControllerItemPtr> SelectedItems = GetSelectedControllerItems();
	if (SelectedItems.Num() > 0)
	{
		return ContextMenu->GeneratePageContextMenuWidget(RundownEditorWeak, GetActivePageMutable(), SharedThis(this));
	}
	return SNullWidget::NullWidget;
}

SAvaRundownRCControllerPanel::FPropertyRowGeneratorWrapper::FPropertyRowGeneratorWrapper(SAvaRundownRCControllerPanel* InParentPanel)
	: ParentPanel(InParentPanel)
{
	FPropertyRowGeneratorArgs Args;
	Args.bShouldShowHiddenProperties   = true;
	Args.bAllowMultipleTopLevelObjects = false;
	Args.NotifyHook = this;
	PropertyRowGenerator = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreatePropertyRowGenerator(Args);
}

SAvaRundownRCControllerPanel::FPropertyRowGeneratorWrapper::~FPropertyRowGeneratorWrapper()
{
	if (PropertyRowGenerator)
	{
		PropertyRowGenerator->OnFinishedChangingProperties().RemoveAll(this);
	}
}

void SAvaRundownRCControllerPanel::FPropertyRowGeneratorWrapper::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged)
{
	FNotifyHook::NotifyPostChange(InPropertyChangedEvent, InPropertyThatChanged);
	
	if (URemoteControlPreset* Preset = PresetWeak.Get())
	{
		const TSharedPtr<FAvaRundownEditor> RundownEditor = ParentPanel ? ParentPanel->RundownEditorWeak.Pin() : nullptr;

		// Only capture a modification when scrubbing starts.
		if (!OngoingPropertyChanges.Contains(InPropertyThatChanged) && RundownEditor)
		{
			OngoingPropertyChanges.Add(InPropertyThatChanged);
			RundownEditor->BeginModify();
		}

		Preset->OnModifyController(InPropertyChangedEvent);
		if (ParentPanel && InPropertyChangedEvent.ChangeType & EPropertyChangeType::ValueSet)
		{
			ParentPanel->UpdatePageSummary(/*bInForceUpdate*/ true);
			OngoingPropertyChanges.Remove(InPropertyThatChanged);
		}
	}
}

#undef LOCTEXT_NAMESPACE
