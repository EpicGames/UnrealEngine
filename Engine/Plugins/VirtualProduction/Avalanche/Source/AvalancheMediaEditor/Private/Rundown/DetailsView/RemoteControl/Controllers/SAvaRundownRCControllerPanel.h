// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "Rundown/AvaRundownManagedInstanceHandle.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FAvaRundownEditor;
class FAvaRundownManagedInstance;
class FAvaRundownPageControllerContextMenu;
class FAvaRundownRCControllerItem;
class FUICommandList;
class IPropertyRowGenerator;
class ITableRow;
class SAvaRundownRCControllerPanel;
class SHeaderRow;
class STableViewBase;
class UAvaRundown;
class URCVirtualPropertyBase;
class URemoteControlPreset;
struct FAvaPlayableRemoteControlValue;
struct FAvaRundownPage;
struct FPropertyChangedEvent;
template <typename ItemType> class SListView;

using FAvaRundownRCControllerItemPtr = TSharedPtr<FAvaRundownRCControllerItem>;

class SAvaRundownRCControllerPanel : public SCompoundWidget
{
public:
	static const FLazyName ControllerColumnName;
	static const FLazyName DescriptionColumnName;
	static const FLazyName ValueColumnName;
	static const FLazyName ExecuteColumnName;

	SLATE_BEGIN_ARGS(SAvaRundownRCControllerPanel) {}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedPtr<FAvaRundownEditor>& InRundownEditor);
	
	bool HasRemoteControlPreset(const URemoteControlPreset* InPreset) const;
	
	void Refresh(const TArray<int32>& InSelectedPageIds);
	
	TSharedRef<ITableRow> OnGenerateControllerRow(FAvaRundownRCControllerItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTable);

	const TArray<FAvaRundownRCControllerItemPtr> GetSelectedControllerItems() const;

	void UpdatePageSummary(bool bInForceUpdate);

private:
	/** Executes all the controllers from the items in this panel, in array order */
	FReply OnExecuteAllControllers();

	void UpdatePropertyRowGenerators(int32 InNumGenerators);
	
	void RefreshForManagedInstance(int32 InInstanceIndex, const FAvaRundownManagedInstance& InManagedInstance, const FAvaRundownPage& InPage);
	
	/** Update the given pages remote control values from the defaults then refresh the widget. */
	void UpdateDefaultValuesAndRefresh(const TArray<int32>& InSelectedPageIds);

	void OnRemoteControlControllerAdded(URemoteControlPreset* InPreset, const FName NewControllerName, const FGuid& InControllerId) { UpdateDefaultValuesAndRefresh({ActivePageId}); }
	void OnRemoteControlControllerRemoved(URemoteControlPreset* InPreset, const FGuid& InControllerId) { UpdateDefaultValuesAndRefresh({ActivePageId}); }
	void OnRemoteControlControllerRenamed(URemoteControlPreset* InPreset, const FName InOldLabel, const FName InNewLabel) { UpdateDefaultValuesAndRefresh({ActivePageId}); }
	void OnRemoteControlControllerModified(URemoteControlPreset* InPreset, const TSet<FGuid>& InModifiedControllerIds);
	void BindRemoteControlDelegates(URemoteControlPreset* InPreset);

	const FAvaPlayableRemoteControlValue* GetSelectedPageControllerValue(const URCVirtualPropertyBase* InController) const;
	bool SetSelectedPageControllerValue(const URCVirtualPropertyBase* InController, const FAvaPlayableRemoteControlValue& InValue) const;

	UAvaRundown* GetRundown() const;
	const FAvaRundownPage& GetActivePage(const UAvaRundown* InRundown) const;
	const FAvaRundownPage& GetActivePage() const { return GetActivePage(GetRundown()); }
	FAvaRundownPage& GetActivePageMutable(UAvaRundown* InRundown) const;
	FAvaRundownPage& GetActivePageMutable() const { return GetActivePageMutable(GetRundown()); }

	TSharedPtr<SWidget> GetContextMenuContent();

	TWeakPtr<FAvaRundownEditor> RundownEditorWeak;

	class FPropertyRowGeneratorWrapper : public FNotifyHook
	{
	public:
		TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;
		TWeakObjectPtr<URemoteControlPreset> PresetWeak;
		SAvaRundownRCControllerPanel* ParentPanel = nullptr;
		TSet<FProperty*> OngoingPropertyChanges;

		FPropertyRowGeneratorWrapper(SAvaRundownRCControllerPanel* InParentPanel);
		virtual ~FPropertyRowGeneratorWrapper();

		//~ Begin FNotifyHook
		virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged) override;
		//~ End FNotifyHook
	};
	
	TArray<TUniquePtr<FPropertyRowGeneratorWrapper>> PropertyRowGenerators;

	FAvaRundownManagedInstanceHandles ManagedHandles;
	
	TSharedPtr<SListView<FAvaRundownRCControllerItemPtr>> ControllerContainer;
	
	TArray<FAvaRundownRCControllerItemPtr> ControllerItems;

	int32 ActivePageId = -1;

	TSharedPtr<FUICommandList> CommandList;

	TSharedPtr<FAvaRundownPageControllerContextMenu> ContextMenu;
};
