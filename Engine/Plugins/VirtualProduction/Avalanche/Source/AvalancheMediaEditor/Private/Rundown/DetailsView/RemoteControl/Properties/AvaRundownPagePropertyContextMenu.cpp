// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPagePropertyContextMenu.h"
#include "IAvaMediaEditorModule.h"
#include "Item/AvaRundownRCFieldItem.h"
#include "Rundown/AvaRundownCommands.h"
#include "Rundown/AvaRundownEditor.h"
#include "Rundown/AvaRundownPage.h"
#include "Rundown/Pages/AvaRundownPagePropertyContext.h"
#include "SAvaRundownPageRemoteControlProps.h"
#include "ScopedTransaction.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define LOCTEXT_NAMESPACE "AvaRundownPagePropertyContextMenu"

FAvaRundownPagePropertyContextMenu::FAvaRundownPagePropertyContextMenu(const TWeakPtr<FUICommandList>& InCommandListWeak)
{
	BindCommands(InCommandListWeak);
}

void FAvaRundownPagePropertyContextMenu::BindCommands(const TWeakPtr<FUICommandList>& InCommandListWeak)
{
	TSharedPtr<FUICommandList> CommandList = InCommandListWeak.Pin();
	if (!CommandList.IsValid())
	{
		return;
	}

	CommandListWeak = InCommandListWeak;

	const FAvaRundownCommands& RundownCommands = FAvaRundownCommands::Get();

	CommandList->MapAction(RundownCommands.ResetValuesToDefaults,
		FExecuteAction::CreateRaw(this, &FAvaRundownPagePropertyContextMenu::ResetValuesToDefaults, false),
		FCanExecuteAction::CreateRaw(this, &FAvaRundownPagePropertyContextMenu::CanResetValuesToDefaults, false));

	CommandList->MapAction(RundownCommands.ResetValuesToTemplate,
		FExecuteAction::CreateRaw(this, &FAvaRundownPagePropertyContextMenu::ResetValuesToDefaults, true),
		FCanExecuteAction::CreateRaw(this, &FAvaRundownPagePropertyContextMenu::CanResetValuesToDefaults, true));
}

TSharedRef<SWidget> FAvaRundownPagePropertyContextMenu::GeneratePageContextMenuWidget(const TWeakPtr<FAvaRundownEditor>& InRundownEditorWeak
	, const FAvaRundownPage& InRundownPage
	, const TWeakPtr<SAvaRundownPageRemoteControlProps>& InPropertyListWidgetWeak)
{
	UToolMenus* const ToolMenus = UToolMenus::Get();
	check(ToolMenus);

	const FAvaRundownCommands& RundownCommands = FAvaRundownCommands::Get();

	static const FName ContextMenuName = TEXT("RundownPageRemoteControlProps");

	if (!ToolMenus->IsMenuRegistered(ContextMenuName))
	{
		UToolMenu* const ContextMenu = ToolMenus->RegisterMenu(ContextMenuName, NAME_None, EMultiBoxType::Menu);
		check(ContextMenu);

		ContextMenu->AddDynamicSection(TEXT("PopulateContextMenu"), FNewToolMenuDelegate::CreateLambda(
			[this](UToolMenu* const InMenu)
			{
				if (IsValid(InMenu))
				{
					PopulatePageContextMenu(*InMenu, InMenu->FindContext<UAvaRundownPagePropertyContext>());
				}
			}));
	}
	
	UAvaRundownPagePropertyContext* const ContextObject = NewObject<UAvaRundownPagePropertyContext>();
	check(ContextObject);
	ContextObject->InitContext(InRundownEditorWeak, InRundownPage.GetPageId(), InPropertyListWidgetWeak);

	TSharedPtr<FExtender> Extender;

	// Compatibility with IAvaMediaEditorModule Rundown Menu Extensibility Manager
	TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin();
	if (CommandList.IsValid())
	{
		TSharedPtr<FExtensibilityManager> MenuExtensibility = IAvaMediaEditorModule::Get().GetRundownMenuExtensibilityManager();
		if (MenuExtensibility.IsValid())
		{
			if (TSharedPtr<FAvaRundownEditor> RundownEditor = InRundownEditorWeak.Pin())
			{
				if (const TArray<UObject*>* EditingObjects = RundownEditor->GetObjectsCurrentlyBeingEdited())
				{
					Extender = MenuExtensibility->GetAllExtenders(CommandList.ToSharedRef(), *EditingObjects);
				}
			}
		}
	}

	FToolMenuContext Context(CommandList, Extender, ContextObject);
	return ToolMenus->GenerateWidget(ContextMenuName, Context);
}

void FAvaRundownPagePropertyContextMenu::PopulatePageContextMenu(UToolMenu& InMenu, UAvaRundownPagePropertyContext* const InContext)
{
	CurrentContext = InContext;

	if (!IsValid(InContext))
	{
		return;
	}

	const TSharedPtr<SAvaRundownPageRemoteControlProps> ListWidget = CurrentContext->GetPropertyListWidget();
	if (ListWidget.IsValid() && ListWidget->GetSelectedPropertyItems().Num() == 0)
	{
		return;
	}

	UAvaRundown* const Rundown = GetContextRundown();
	if (!IsValid(Rundown))
	{
		return;
	}

	const FAvaRundownPage& RundownPage = Rundown->GetPage(CurrentContext->GetRundownPageId());
	if (!RundownPage.IsValidPage())
	{
		return;
	}

	const FAvaRundownCommands& RundownCommands = FAvaRundownCommands::Get();

	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("PropertyActions"), LOCTEXT("PropertyActions", "Property Actions"));

	Section.AddMenuEntry(TEXT("ResetPropertyToDefaults"),
		RundownCommands.ResetValuesToDefaults,
		LOCTEXT("ResetPropertyToDefaults", "Reset to Defaults"),
		LOCTEXT("ResetPropertyToDefaultsToolTip", "Reset selected entity property values to the defaults."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault")));

	if (!RundownPage.IsTemplate())
	{
		Section.AddMenuEntry(TEXT("ResetPropertyToTemplate"),
			RundownCommands.ResetValuesToTemplate,
			LOCTEXT("ResetPropertyToTemplate", "Reset to Template"),
			LOCTEXT("ResetPropertyToTemplateToolTip", "Reset selected entity property values to the defaults."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault")));
	}
}

TArray<TWeakPtr<FAvaRundownRCFieldItem>> FAvaRundownPagePropertyContextMenu::GetSelectedPropertyItems() const
{
	TArray<TWeakPtr<FAvaRundownRCFieldItem>> OutItems;

	if (CurrentContext.IsValid())
	{
		if (TSharedPtr<SAvaRundownPageRemoteControlProps> PropertyListWidget = CurrentContext->GetPropertyListWidget())
		{
			for (const TSharedPtr<FAvaRundownRCFieldItem>& FieldItem : PropertyListWidget->GetSelectedPropertyItems())
			{
				if (FieldItem.IsValid())
				{
					OutItems.Add(FieldItem);
				}
			}
		}
	}

	return OutItems;
}

bool FAvaRundownPagePropertyContextMenu::HasValidSelectedItems() const
{
	return GetSelectedPropertyItems().Num() > 0;
}

void FAvaRundownPagePropertyContextMenu::ResetValuesToDefaults(const bool bInUseTemplateValues)
{
	UAvaRundown* const Rundown = GetContextRundown();
	if (!IsValid(Rundown))
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ResetPropertyValuesTransaction", "Reset RC Property Values"));
	Rundown->Modify();

	const TArray<TWeakPtr<FAvaRundownRCFieldItem>> SelectedPropertyItems = GetSelectedPropertyItems();
	for (const TWeakPtr<FAvaRundownRCFieldItem>& PropertyItemWeak : SelectedPropertyItems)
	{
		if (const TSharedPtr<FAvaRundownRCFieldItem> PropertyItem = PropertyItemWeak.Pin())
		{
			const TSharedPtr<FRemoteControlEntity> Entity = PropertyItem->GetEntity();
			if (Entity.IsValid())
			{
				Rundown->ResetRemoteControlEntityValue(CurrentContext->GetRundownPageId()
					, Entity->GetId(), bInUseTemplateValues, /*bInIsDefault=*/false);
			}
		}
	}
}

bool FAvaRundownPagePropertyContextMenu::CanResetValuesToDefaults(const bool bInUseTemplateValues) const
{
	UAvaRundown* const Rundown = GetContextRundown();
	if (!IsValid(Rundown))
	{
		return false;
	}

	const FAvaRundownPage& RundownPage = Rundown->GetPage(CurrentContext->GetRundownPageId());
	if (!RundownPage.IsValidPage())
	{
		return false;
	}

	const TArray<TWeakPtr<FAvaRundownRCFieldItem>> SelectedItems = GetSelectedPropertyItems();
	if (SelectedItems.Num() == 0)
	{
		return false;
	}

	bool bContainsDifferentValues = false;

	for (const TWeakPtr<FAvaRundownRCFieldItem>& PropertyItemWeak : SelectedItems)
	{
		const TSharedPtr<FAvaRundownRCFieldItem> PropertyItem = PropertyItemWeak.Pin();
		if (PropertyItem.IsValid())
		{
			if (PropertyItem->IsEntityControlled())
			{
				return false;
			}

			const TSharedPtr<FRemoteControlEntity> Entity = PropertyItem->GetEntity();
			if (Entity.IsValid() && !RundownPage.IsDefaultEntityValue(Rundown, Entity->GetId(), bInUseTemplateValues))
			{
				bContainsDifferentValues = true;
			}
		}
	}

	return bContainsDifferentValues;
}

UAvaRundown* FAvaRundownPagePropertyContextMenu::GetContextRundown() const
{
	if (!CurrentContext.IsValid())
	{
		return nullptr;
	}

	const TSharedPtr<FAvaRundownEditor> RundownEditor = CurrentContext->GetRundownEditor();
	if (!RundownEditor.IsValid())
	{
		return nullptr;
	}

	return RundownEditor->GetRundown();
}

#undef LOCTEXT_NAMESPACE
