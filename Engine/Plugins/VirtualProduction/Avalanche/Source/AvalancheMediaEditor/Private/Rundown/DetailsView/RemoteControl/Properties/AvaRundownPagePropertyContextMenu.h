// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/Pages/AvaRundownPageControllerContext.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplatesFwd.h"

class FAvaRundownEditor;
class FAvaRundownRCFieldItem;
class FUICommandList;
class SAvaRundownPageRemoteControlProps;
class SWidget;
class UAvaRundownPagePropertyContext;
class UToolMenu;
struct FAvaRundownPage;

class FAvaRundownPagePropertyContextMenu : public TSharedFromThis<FAvaRundownPagePropertyContextMenu>
{
public:
	FAvaRundownPagePropertyContextMenu(const TWeakPtr<FUICommandList>& InCommandListWeak);

	TSharedRef<SWidget> GeneratePageContextMenuWidget(const TWeakPtr<FAvaRundownEditor>& InRundownEditorWeak
		, const FAvaRundownPage& InRundownPage
		, const TWeakPtr<SAvaRundownPageRemoteControlProps>& InPropertyListWidgetWeak);

	bool HasValidSelectedItems() const;

private:
	void PopulatePageContextMenu(UToolMenu& InMenu, UAvaRundownPagePropertyContext* const InContext);

	void BindCommands(const TWeakPtr<FUICommandList>& InCommandListWeak);

	void ResetValuesToDefaults(const bool bInUseTemplateValues);
	bool CanResetValuesToDefaults(const bool bInUseTemplateValues) const;

	TArray<TWeakPtr<FAvaRundownRCFieldItem>> GetSelectedPropertyItems() const;

	UAvaRundown* GetContextRundown() const;

	TWeakPtr<FUICommandList> CommandListWeak;

	TWeakObjectPtr<UAvaRundownPagePropertyContext> CurrentContext;
};
