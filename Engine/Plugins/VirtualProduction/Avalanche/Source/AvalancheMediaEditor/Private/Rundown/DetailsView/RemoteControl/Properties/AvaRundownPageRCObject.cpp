// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageRCObject.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Item/AvaRundownRCDetailTreeNodeItem.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"

FAvaRundownPageRCObject::FAvaRundownPageRCObject(UObject* InObject)
	: ObjectKey(InObject)
{
}

void FAvaRundownPageRCObject::Initialize(FNotifyHook* InNotifyHook)
{
	UObject* const ResolvedObject = ObjectKey.ResolveObjectPtr();

	FPropertyRowGeneratorArgs Args;
	Args.NotifyHook = InNotifyHook;
	PropertyRowGenerator = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreatePropertyRowGenerator(Args);
	PropertyRowGenerator->SetObjects({ ResolvedObject });

	CacheTreeNodes();
}

TSharedPtr<IDetailTreeNode> FAvaRundownPageRCObject::FindTreeNode(const FString& InPathInfo) const
{
	if (const TSharedRef<IDetailTreeNode>* FoundTreeNode = TreeNodeMap.Find(InPathInfo))
	{
		return *FoundTreeNode;
	}
	return nullptr;
}

void FAvaRundownPageRCObject::CacheTreeNodes()
{
	TreeNodeMap.Reset();

	if (!PropertyRowGenerator.IsValid())
	{
		return;
	}

	TArray<TSharedRef<IDetailTreeNode>> TreeNodes = PropertyRowGenerator->GetRootTreeNodes();
	TArray<TSharedRef<IDetailTreeNode>> ChildTreeNodes;

	while (!TreeNodes.IsEmpty())
	{
		TSharedRef<IDetailTreeNode> TreeNode = TreeNodes.Pop(EAllowShrinking::No);

		if (TSharedPtr<IPropertyHandle> PropertyHandle = TreeNode->CreatePropertyHandle())
		{
			constexpr bool bCleanDuplicates = true;
			const FRCFieldPathInfo FieldPathInfo(PropertyHandle->GeneratePathToProperty(), bCleanDuplicates);
			TreeNodeMap.Add(FieldPathInfo.ToString(), TreeNode);
		}

		ChildTreeNodes.Reset();
		TreeNode->GetChildren(ChildTreeNodes);
		TreeNodes.Append(MoveTemp(ChildTreeNodes));
	}
}
