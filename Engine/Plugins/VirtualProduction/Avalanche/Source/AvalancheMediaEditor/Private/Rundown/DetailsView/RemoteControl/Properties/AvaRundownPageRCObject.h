// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "RemoteControlFieldPath.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectKey.h"

class FAvaRundownRCDetailTreeNodeItem;
class FNotifyHook;
class IDetailTreeNode;
class IPropertyRowGenerator;
class UObject;
class URemoteControlPreset;

/** A class containing the information of a UObject exposed to RC */
class FAvaRundownPageRCObject
{
public:
	explicit FAvaRundownPageRCObject(UObject* InObject);

	void Initialize(FNotifyHook* InNotifyHook);

	TSharedPtr<IDetailTreeNode> FindTreeNode(const FString& InPathInfo) const;

	bool operator==(const FAvaRundownPageRCObject& InOther) const
	{
		return ObjectKey == InOther.ObjectKey;
	}

	friend uint32 GetTypeHash(const FAvaRundownPageRCObject& InObject)
	{
		return GetTypeHash(InObject.ObjectKey);
	}

	void CacheTreeNodes();

private:
	FObjectKey ObjectKey;

	TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;

	TMap<FString, TSharedRef<IDetailTreeNode>> TreeNodeMap;
};
