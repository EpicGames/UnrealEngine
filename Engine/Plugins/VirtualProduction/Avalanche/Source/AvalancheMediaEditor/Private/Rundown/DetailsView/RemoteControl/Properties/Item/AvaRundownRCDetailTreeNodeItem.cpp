// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownRCDetailTreeNodeItem.h"
#include "DetailLayoutBuilder.h"
#include "IRemoteControlPropertyHandle.h"
#include "RemoteControlField.h"
#include "Rundown/DetailsView/RemoteControl/Properties/AvaRundownPageRCObject.h"
#include "Rundown/DetailsView/RemoteControl/Properties/SAvaRundownPageRemoteControlProps.h"

TSharedPtr<FAvaRundownRCDetailTreeNodeItem> FAvaRundownRCDetailTreeNodeItem::CreateItem(const TSharedRef<SAvaRundownPageRemoteControlProps>& InPropertyPanel
	, const TSharedRef<FRemoteControlProperty>& InPropertyEntity
	, bool bInControlled)
{
	TSharedRef<FAvaRundownRCDetailTreeNodeItem> PropertyItem = MakeShared<FAvaRundownRCDetailTreeNodeItem>();
	PropertyItem->PropertyEntityWeak = InPropertyEntity;
	PropertyItem->EntityOwnerWeak = InPropertyEntity;
	PropertyItem->bEntityControlled = bInControlled;
	PropertyItem->Refresh(InPropertyPanel);

	PropertyItem->NodeWidgets.NameWidget = SNew(STextBlock)
		.Margin(FMargin(8.f, 2.f, 0.f, 2.f))
		.Text(FText::FromName(InPropertyEntity->GetLabel()))
		.Font(IDetailLayoutBuilder::GetDetailFont());

	return PropertyItem;
}

void FAvaRundownRCDetailTreeNodeItem::Initialize(TSharedRef<IDetailTreeNode> InDetailTreeNode)
{
	DetailTreeNode = InDetailTreeNode;
	NodeWidgets = InDetailTreeNode->CreateNodeWidgets();
	RefreshChildren();
}

FStringView FAvaRundownRCDetailTreeNodeItem::GetPath() const
{
	return FieldPath;
}

void FAvaRundownRCDetailTreeNodeItem::Refresh(const TSharedRef<SAvaRundownPageRemoteControlProps>& InPropertyPanel)
{
	FieldPath.Reset();

	TSharedPtr<FRemoteControlProperty> PropertyEntity = PropertyEntityWeak.Pin();
	if (!PropertyEntity.IsValid())
	{
		return;
	}

	TArray<UObject*> Objects = PropertyEntity->GetBoundObjects();
	if (Objects.IsEmpty())
	{
		return;
	}

	FAvaRundownPageRCObject& PageRCObject = InPropertyPanel->FindOrAddPageRCObject(Objects[0]);
	FieldPath = PropertyEntity->FieldPathInfo.ToString();

	TSharedPtr<IDetailTreeNode> PropertyNode = PageRCObject.FindTreeNode(FieldPath);
	if (!PropertyNode.IsValid())
	{
		return;
	}

	Initialize(PropertyNode.ToSharedRef());
}

void FAvaRundownRCDetailTreeNodeItem::RefreshChildren()
{
	Children.Reset();
	if (!DetailTreeNode.IsValid())
	{
		return;
	}

	TArray<TSharedRef<IDetailTreeNode>> ChildNodes;
	DetailTreeNode->GetChildren(ChildNodes);

	Children.Reserve(ChildNodes.Num());

	for (const TSharedRef<IDetailTreeNode>& ChildNode : ChildNodes)
	{
		TSharedRef<FAvaRundownRCDetailTreeNodeItem> ChildPropertyItem = MakeShared<FAvaRundownRCDetailTreeNodeItem>();
		ChildPropertyItem->PropertyEntityWeak = PropertyEntityWeak;
		ChildPropertyItem->EntityOwnerWeak = EntityOwnerWeak;
		ChildPropertyItem->bEntityControlled = bEntityControlled;
		ChildPropertyItem->Initialize(ChildNode);
		Children.Add(MoveTemp(ChildPropertyItem));
	}
}
