// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaRundownRCFieldItem.h"
#include "IDetailTreeNode.h"

struct FRemoteControlProperty;

class FAvaRundownRCDetailTreeNodeItem : public FAvaRundownRCFieldItem
{
	friend class FAvaRundownRCFieldItem;

public:
	static TSharedPtr<FAvaRundownRCDetailTreeNodeItem> CreateItem(const TSharedRef<SAvaRundownPageRemoteControlProps>& InPropertyPanel
		, const TSharedRef<FRemoteControlProperty>& InPropertyEntity
		, bool bInControlled);

	void Initialize(TSharedRef<IDetailTreeNode> InDetailTreeNode);

	//~ Begin FAvaRundownRCFieldItem
	virtual FStringView GetPath() const override;
	virtual void Refresh(const TSharedRef<SAvaRundownPageRemoteControlProps>& InPropertyPanel) override;
	//~ End FAvaRundownRCFieldItem

private:
	void RefreshChildren();

	TWeakPtr<FRemoteControlProperty> PropertyEntityWeak;

	TSharedPtr<IDetailTreeNode> DetailTreeNode;

	/** Cached path of the Field */
	FString FieldPath;
};
