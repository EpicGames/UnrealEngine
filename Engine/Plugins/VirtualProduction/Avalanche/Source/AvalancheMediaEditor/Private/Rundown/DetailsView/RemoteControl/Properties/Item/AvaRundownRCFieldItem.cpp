// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownRCFieldItem.h"
#include "AvaRundownRCDetailTreeNodeItem.h"
#include "AvaRundownRCFunctionItem.h"
#include "RemoteControlEntity.h"
#include "RemoteControlField.h"
#include "Rundown/DetailsView/RemoteControl/Properties/SAvaRundownRCPropertyItemRow.h"

TSharedPtr<FAvaRundownRCFieldItem> FAvaRundownRCFieldItem::CreateItem(const TSharedRef<SAvaRundownPageRemoteControlProps>& InPropertyPanel, const TSharedRef<FRemoteControlEntity>& InEntity, bool bInControlled)
{
	const UScriptStruct* EntityStruct = InEntity->GetStruct();
	if (!EntityStruct || !EntityStruct->IsChildOf<FRemoteControlField>())
	{
		return nullptr;
	}

	TSharedRef<FRemoteControlField> FieldEntity = StaticCastSharedRef<FRemoteControlField>(InEntity);

	if (FieldEntity->FieldType == EExposedFieldType::Function)
	{
		return FAvaRundownRCFunctionItem::CreateItem(InPropertyPanel
			, StaticCastSharedRef<FRemoteControlFunction>(InEntity)
			, bInControlled);
	}

	if (FieldEntity->FieldType == EExposedFieldType::Property)
	{
		return FAvaRundownRCDetailTreeNodeItem::CreateItem(InPropertyPanel
			, StaticCastSharedRef<FRemoteControlProperty>(InEntity)
			, bInControlled);
	}

	return nullptr;
}

FAvaRundownRCFieldItem::FAvaRundownRCFieldItem()
{
	NodeWidgets.NameWidgetLayoutData.HorizontalAlignment = HAlign_Fill;
	NodeWidgets.NameWidgetLayoutData.VerticalAlignment = VAlign_Fill;
	NodeWidgets.ValueWidgetLayoutData.HorizontalAlignment = HAlign_Fill;
	NodeWidgets.ValueWidgetLayoutData.VerticalAlignment = VAlign_Fill;
}

TSharedRef<ITableRow> FAvaRundownRCFieldItem::CreateWidget(const TSharedRef<SAvaRundownPageRemoteControlProps>& InPropertyPanel, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SAvaRundownRCPropertyItemRow, InPropertyPanel, InOwnerTable, SharedThis(this));
}

FStringView FAvaRundownRCFieldItem::GetPath() const
{
	return FStringView();
}
