// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailTreeNode.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

class ITableRow;
class SAvaRundownPageRemoteControlProps;
class STableViewBase;
class SWidget;
struct FRemoteControlEntity;

class FAvaRundownRCFieldItem : public TSharedFromThis<FAvaRundownRCFieldItem>
{
public:
	static TSharedPtr<FAvaRundownRCFieldItem> CreateItem(const TSharedRef<SAvaRundownPageRemoteControlProps>& InPropertyPanel
		, const TSharedRef<FRemoteControlEntity>& InEntity
		, bool bInControlled);

	FAvaRundownRCFieldItem();

	virtual ~FAvaRundownRCFieldItem() = default;

	TSharedPtr<FRemoteControlEntity> GetEntity() const
	{
		return EntityOwnerWeak.Pin();
	}

	bool IsEntityControlled() const
	{
		return bEntityControlled;
	}

	virtual void Refresh(const TSharedRef<SAvaRundownPageRemoteControlProps>& InPropertyPanel) {}

	TConstArrayView<TSharedPtr<FAvaRundownRCFieldItem>> GetChildren() const
	{
		return Children;
	}

	TSharedRef<ITableRow> CreateWidget(const TSharedRef<SAvaRundownPageRemoteControlProps>& InPropertyPanel, const TSharedRef<STableViewBase>& InOwnerTable);

	const FNodeWidgets& GetNodeWidgets() const
	{
		return NodeWidgets;
	}

	virtual FStringView GetPath() const;

protected:
	TWeakPtr<FRemoteControlEntity> EntityOwnerWeak;

	TArray<TSharedPtr<FAvaRundownRCFieldItem>> Children;

	FNodeWidgets NodeWidgets;

	bool bEntityControlled = false;
};
