// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownRCFunctionItem.h"
#include "DetailLayoutBuilder.h"
#include "RemoteControlField.h"
#include "ScopedTransaction.h"
#include "UObject/Script.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AvaRundownRCFunctionItem"

TSharedPtr<FAvaRundownRCFunctionItem> FAvaRundownRCFunctionItem::CreateItem(const TSharedRef<SAvaRundownPageRemoteControlProps>& InPropertyPanel
	, const TSharedRef<FRemoteControlFunction>& InFunctionEntity
	, bool bInControlled)
{
	TSharedRef<FAvaRundownRCFunctionItem> FunctionItem = MakeShared<FAvaRundownRCFunctionItem>();
	FunctionItem->EntityOwnerWeak = InFunctionEntity;
	FunctionItem->bEntityControlled = bInControlled;
	FunctionItem->Initialize();
	return FunctionItem;
}

void FAvaRundownRCFunctionItem::Initialize()
{
	NodeWidgets.NameWidgetLayoutData.VerticalAlignment = VAlign_Center;
	NodeWidgets.ValueWidgetLayoutData.HorizontalAlignment = HAlign_Left;

	NodeWidgets.NameWidget = SNew(STextBlock)
		.Margin(FMargin(8.f, 2.f, 0.f, 2.f))
		.Text(GetLabel())
		.Font(IDetailLayoutBuilder::GetDetailFont());

	NodeWidgets.ValueWidget = SNew(SBox)
		.Padding(0.f, 3.f)
		[
			SNew(SButton)
			.VAlign(VAlign_Center)
			.OnClicked(this, &FAvaRundownRCFunctionItem::OnFunctionButtonClicked)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CallFunctionLabel", "Call Function"))
			]
		];
}

FText FAvaRundownRCFunctionItem::GetLabel() const
{
	if (TSharedPtr<FRemoteControlEntity> Entity = EntityOwnerWeak.Pin())
	{
		return FText::FromName(Entity->GetLabel());
	}
	return FText::GetEmpty();
}

FReply FAvaRundownRCFunctionItem::OnFunctionButtonClicked() const
{
	TSharedPtr<FRemoteControlEntity> Entity = GetEntity();
	if (!Entity.IsValid())
	{
		return FReply::Unhandled();
	}

	const UScriptStruct* EntityStruct = Entity->GetStruct();
	if (!EntityStruct || !EntityStruct->IsChildOf<FRemoteControlFunction>())
	{
		return FReply::Unhandled();
	}

	TSharedRef<FRemoteControlFunction> FunctionEntity = StaticCastSharedRef<FRemoteControlFunction>(Entity.ToSharedRef());

	FScopedTransaction Transaction(LOCTEXT("CallExposedFunction", "Called a function through Rundown."));
	FEditorScriptExecutionGuard ScriptGuard;

	bool bObjectsModified = false;

	for (UObject* Object : FunctionEntity->GetBoundObjects())
	{
		if (FunctionEntity->FunctionArguments && FunctionEntity->FunctionArguments->IsValid())
		{
			Object->Modify();
			Object->ProcessEvent(FunctionEntity->GetFunction(), FunctionEntity->FunctionArguments->GetStructMemory());
			bObjectsModified = true;
		}
		else
		{
			ensureAlwaysMsgf(false, TEXT("Function default arguments could not be resolved."));
		}
	}

	if (!bObjectsModified)
	{
		Transaction.Cancel();
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
