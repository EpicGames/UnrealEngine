// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaRundownRCFieldItem.h"

struct FRemoteControlFunction;
class FReply;

class FAvaRundownRCFunctionItem : public FAvaRundownRCFieldItem
{
public:
	static TSharedPtr<FAvaRundownRCFunctionItem> CreateItem(const TSharedRef<SAvaRundownPageRemoteControlProps>& InPropertyPanel
		, const TSharedRef<FRemoteControlFunction>& InFunctionEntity
		, bool bInControlled);

private:
	void Initialize();

	FText GetLabel() const;

	FReply OnFunctionButtonClicked() const;
};
