// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownPageRemoteControlProps.h"

#include "Controller/RCCustomControllerUtilities.h"
#include "IAvaMediaModule.h"
#include "Item/AvaRundownRCFieldItem.h"
#include "Playable/AvaPlayableRemoteControl.h"
#include "Playable/AvaPlayableRemoteControlPresetInfo.h"
#include "Playable/AvaPlayableRemoteControlValues.h"
#include "Playback/AvaPlaybackUtils.h"
#include "RemoteControlEntity.h"
#include "RemoteControlPreset.h"
#include "Rundown/AvaRundownEditor.h"
#include "Rundown/AvaRundownEditorSettings.h"
#include "Rundown/AvaRundownEditorUtils.h"
#include "Rundown/AvaRundownManagedInstanceCache.h"
#include "Rundown/AvaRundownPage.h"
#include "Rundown/DetailsView/RemoteControl/Properties/AvaRundownPagePropertyContextMenu.h"
#include "Rundown/DetailsView/SAvaRundownPageDetails.h"
#include "SAvaRundownRCPropertyItemRow.h"
#include "UObject/NameTypes.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "SAvaRundownPageRemoteControlProps"

const FName SAvaRundownPageRemoteControlProps::PropertyColumnName = TEXT("PropertyColumn");
const FName SAvaRundownPageRemoteControlProps::ValueColumnName = TEXT("ValueColumn");

FAvaRundownRCPropertyHeaderRowExtensionDelegate SAvaRundownPageRemoteControlProps::HeaderRowExtensionDelegate;
TMap<FName, TArray<FAvaRundownRCPropertyTableRowExtensionDelegate>> SAvaRundownPageRemoteControlProps::TableRowExtensionDelegates;

TArray<FAvaRundownRCPropertyTableRowExtensionDelegate>& SAvaRundownPageRemoteControlProps::GetTableRowExtensionDelegates(FName InExtensionName)
{
	return TableRowExtensionDelegates.FindOrAdd(InExtensionName);
}

class FAvaRundownPageRCPropsNotifyHookImpl : public FAvaRundownPageRCPropsNotifyHook
{
public:
	FAvaRundownPageRCPropsNotifyHookImpl(const TWeakPtr<SAvaRundownPageRemoteControlProps>& InPanelWeak)
		: PanelWeak(InPanelWeak)
	{
	}

	virtual ~FAvaRundownPageRCPropsNotifyHookImpl() override = default;
	
	//~ Begin FNotifyHook
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged) override
	{
		if (const TSharedPtr<SAvaRundownPageRemoteControlProps> Panel = PanelWeak.Pin())
		{
			const TSharedPtr<FAvaRundownEditor> RundownEditor = Panel->RundownEditorWeak.Pin();
			
			// Only capture a modification when scrubbing starts.
			if (!OngoingPropertyChanges.Contains(InPropertyThatChanged) && RundownEditor)
			{
				OngoingPropertyChanges.Add(InPropertyThatChanged);
				RundownEditor->BeginModify();
			}

			// Apply change to page immediately to capture it in the transaction.
			Panel->OnPostPropertyChanged(InPropertyThatChanged);

			if (InPropertyChangedEvent.ChangeType & EPropertyChangeType::ValueSet)
			{
				OngoingPropertyChanges.Remove(InPropertyThatChanged);
			}
		}
	}
	//~ End FNotifyHook

	TWeakPtr<SAvaRundownPageRemoteControlProps> PanelWeak;
	TSet<FProperty*> OngoingPropertyChanges;
};

void SAvaRundownPageRemoteControlProps::Construct(const FArguments& InArgs, const TSharedRef<SAvaRundownPageDetails>& InPageDetailPanel, const TSharedPtr<FAvaRundownEditor>& InRundownEditor)
{
	RundownEditorWeak = InRundownEditor;
	ActivePageId = FAvaRundownPage::InvalidPageId;
	AsyncAssetLoader = MakeShared<UE::AvaPlayback::Utils::FAsyncAssetLoader>();
	AsyncAssetLoader->OnLoadingCompleted.AddSPLambda(&InPageDetailPanel.Get(), [PageDetailPanelWeak = InPageDetailPanel.ToWeakPtr()]()
	{
		if (const TSharedPtr<SAvaRundownPageDetails> PageDetailPanel = PageDetailPanelWeak.Pin())
		{
			PageDetailPanel->RefreshSelectedPage();	// Refresh the detail panel when assets are loaded.
		}
	});

	CommandList = MakeShared<FUICommandList>();
	NotifyHook = MakeShared<FAvaRundownPageRCPropsNotifyHookImpl>(SharedThis<SAvaRundownPageRemoteControlProps>(this).ToWeakPtr());

	ContextMenu = MakeShared<FAvaRundownPagePropertyContextMenu>(CommandList);

	const UAvaRundownEditorSettings* RundownEditorSettings = UAvaRundownEditorSettings::Get();
	bShowControlledProperties = !RundownEditorSettings || RundownEditorSettings->bPageDetailsShowProperties;

	TSharedRef<SHeaderRow> HeaderRow =
		SNew(SHeaderRow)
		+ SHeaderRow::Column(PropertyColumnName)
		.DefaultLabel(LOCTEXT("Property", "Property"))
		.FillWidth(0.2f)
		+ SHeaderRow::Column(ValueColumnName)
		.DefaultLabel(LOCTEXT("Value", "Value"))
		.FillWidth(0.8f);

	HeaderRowExtensionDelegate.Broadcast(SharedThis(this), HeaderRow);

	ChildSlot
	[
		SNew(SBorder)
		.Padding(8.0f)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SAssignNew(FieldContainer, STreeView<TSharedPtr<FAvaRundownRCFieldItem>>)
			.TreeItemsSource(&FieldItems)
			.SelectionMode(ESelectionMode::Multi)
			.OnContextMenuOpening(this, &SAvaRundownPageRemoteControlProps::GetContextMenuContent)
			.OnGenerateRow(this, &SAvaRundownPageRemoteControlProps::OnGenerateControllerRow)
			.OnGetChildren(this, &SAvaRundownPageRemoteControlProps::OnGetEntityChildren)
			.OnExpansionChanged(this, &SAvaRundownPageRemoteControlProps::OnExpansionChanged)
			.HeaderRow(HeaderRow)
		]
	];

	Refresh({});
}

SAvaRundownPageRemoteControlProps::~SAvaRundownPageRemoteControlProps()
{
}

void SAvaRundownPageRemoteControlProps::UpdateDefaultValuesAndRefresh(const TArray<int32>& InSelectedPageIds)
{
	if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		if (UAvaRundown* Rundown = RundownEditor->GetRundown())
		{
			using namespace UE::AvaRundownEditor::Utils;
			if (UpdateDefaultRemoteControlValues(Rundown, InSelectedPageIds) != EAvaPlayableRemoteControlChanges::None)
			{
				RundownEditor->MarkAsModified();
			}
		}
	}

	Refresh(InSelectedPageIds);
}

TSharedRef<ITableRow> SAvaRundownPageRemoteControlProps::OnGenerateControllerRow(TSharedPtr<FAvaRundownRCFieldItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return InItem->CreateWidget(SharedThis(this), InOwnerTable);
}

void SAvaRundownPageRemoteControlProps::OnGetEntityChildren(TSharedPtr<FAvaRundownRCFieldItem> InItem, TArray<TSharedPtr<FAvaRundownRCFieldItem>>& OutChildren)
{
	if (InItem.IsValid())
	{
		OutChildren.Append(InItem->GetChildren());
	}
}

void SAvaRundownPageRemoteControlProps::OnExpansionChanged(TSharedPtr<FAvaRundownRCFieldItem> InItem, bool bInIsExpanded)
{
	if (InItem.IsValid())
	{
		const FStringView Path = InItem->GetPath();
		if (bInIsExpanded)
		{
			ExpandedPaths.Emplace(Path);
		}
		else
		{
			ExpandedPaths.RemoveByHash(GetTypeHash(Path), Path);
		}
	}
}

void SAvaRundownPageRemoteControlProps::UpdateItemExpansionsRecursive(TConstArrayView<TSharedPtr<FAvaRundownRCFieldItem>> InItems)
{
	for (const TSharedPtr<FAvaRundownRCFieldItem>& Item : InItems)
	{
		if (Item.IsValid())
		{
			const FStringView Path = Item->GetPath();
			const bool bShouldExpand = ExpandedPaths.ContainsByHash(GetTypeHash(Path), Path);
			FieldContainer->SetItemExpansion(Item, bShouldExpand);
			UpdateItemExpansionsRecursive(Item->GetChildren());
		}
	}
}

void SAvaRundownPageRemoteControlProps::RefreshTable(const TSet<FGuid>& InEntityIds)
{
	for (const TSharedPtr<FAvaRundownRCFieldItem>& PropertyItem : FieldItems)
	{
		const TSharedPtr<FRemoteControlEntity> Entity = PropertyItem->GetEntity();
		if (Entity && (InEntityIds.IsEmpty() || InEntityIds.Contains(Entity->GetId())))
		{
			TSharedPtr<ITableRow> TableRow = FieldContainer->WidgetFromItem(PropertyItem);

			if (TableRow.IsValid())
			{
				const TSharedRef<SAvaRundownRCPropertyItemRow> ItemRow = StaticCastSharedRef<SAvaRundownRCPropertyItemRow>(TableRow.ToSharedRef());
				ItemRow->UpdateValue();
			}
		}
	}
}

void SAvaRundownPageRemoteControlProps::Refresh(const TArray<int32>& InSelectedPageIds)
{
	if (!FieldContainer.IsValid())
	{
		return;
	}

	int32 PreviousPageId = ActivePageId;
	ActivePageId = InSelectedPageIds.IsEmpty() ? FAvaRundownPage::InvalidPageId : InSelectedPageIds[0];
	
	if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		const UAvaRundown* Rundown = RundownEditor->GetRundown();

		if (IsValid(Rundown))
		{
			ManagedInstances.Reset();

			if (FAvaRundownPage* ActivePage = GetActivePage())
			{
				using namespace UE::AvaRundownEditor::Utils;
				// Note: we don't use the handles api here because we are not acquiring the events.
				// Todo: the managed instances should be more unified between the Entity/Prop and Controller panels.
				ManagedInstances = IAvaMediaModule::Get().GetManagedInstanceCache().GetManagedInstancesForPage(Rundown, *ActivePage);

				for (const TSharedPtr<FAvaRundownManagedInstance>& ManagedInstance : ManagedInstances)
				{
					BindRemoteControlDelegates(ManagedInstance ? ManagedInstance->GetRemoteControlPreset() : nullptr);	
				}

				if (!ManagedInstances.IsEmpty())
				{
					FAvaPlayableRemoteControlValues MergedDefaultRCValues;
					MergeDefaultRemoteControlValues(ManagedInstances, MergedDefaultRCValues);
					
					// Prune any extra stale values. This happens if templates are changed.
					if (ActivePage->PruneRemoteControlValues(MergedDefaultRCValues) != EAvaPlayableRemoteControlChanges::None)
					{
						UE_LOGF(LogAvaPlayableRemoteControl, Log, "Page %d had stale values that where pruned.", ActivePage->GetPageId());
						RundownEditor->MarkAsModified();
					}

					if (PreviousPageId != ActivePageId)
					{
						TSet<FSoftObjectPath> Assets;
						const FAvaPlayableRemoteControlValues& Values = ActivePage->GetRemoteControlValues();
						FAvaPlayableRemoteControlValues::CollectReferencedAssetPaths(Values.ControllerValues, Assets);
						FAvaPlayableRemoteControlValues::CollectReferencedAssetPaths(Values.EntityValues, Assets);

						// Synchronously load the assets as this refresh assigns the value of these assets to the remote control entities below.
						// Doing an async load would mean that the relevant entities are likely to get assigned a 'none' value instead of the actual asset.
						for (FSoftObjectPath& Asset : Assets)
						{
							Asset.TryLoad();
						}
					}
				}
			}
		}

		struct FAvaPropertyDetails
		{
			TSharedRef<FRemoteControlEntity> Entity;
			bool bEntityControlled;
		};

		TArray<FAvaPropertyDetails> NewItems;

		for (const TSharedPtr<FAvaRundownManagedInstance>& ManagedInstance : ManagedInstances)
		{
			URemoteControlPreset* RemoteControlPreset = ManagedInstance ? ManagedInstance->GetRemoteControlPreset() : nullptr;
			if (!RemoteControlPreset)
			{
				continue;
			}

			const TArray<TWeakPtr<FRemoteControlEntity>> ExposedEntities = RemoteControlPreset->GetExposedEntities<FRemoteControlEntity>();

			TArray<FAvaPropertyDetails> ItemsToAdd;
			ItemsToAdd.Reserve(ExposedEntities.Num());

			for (const TWeakPtr<FRemoteControlEntity>& EntityWeakPtr : ExposedEntities)
			{
				const TSharedPtr<FRemoteControlEntity> Entity = EntityWeakPtr.Pin();
				if (!Entity.IsValid())
				{
					continue;
				}

				const UScriptStruct* EntityStruct = Entity->GetStruct();
				if (!EntityStruct || !EntityStruct->IsChildOf<FRemoteControlField>())
				{
					continue;
				}

				TSharedRef<FRemoteControlField> FieldEntity = StaticCastSharedRef<FRemoteControlField>(Entity.ToSharedRef());

				if (FieldEntity->FieldType == EExposedFieldType::Function)
				{
					ItemsToAdd.Add({ FieldEntity, false });
				}
				else if (FieldEntity->FieldType == EExposedFieldType::Property)
				{
					const FAvaPlayableRemoteControlValue* EntityValue = GetSelectedPageEntityValue(FieldEntity);

					if (!EntityValue)
					{
						// If the page doesn't already have a value, we get it from the template's default values.
						const FAvaPlayableRemoteControlValue* const DefaultEntityValue = ManagedInstance->GetDefaultRemoteControlValues().GetEntityValue(FieldEntity->GetId()); 
						
						if (!DefaultEntityValue)
						{
							FString AccessError;
							// Check if the property is correctly bound, that would explain why the value is missing.
							if (!UE::AvaPlayableRemoteControl::HasReadAccess(FieldEntity, AccessError))
							{
								UE_LOGF(LogAvaPlayableRemoteControl, Error, "Entity \"%ls\" (id:%ls) doesn't have a template default value. Reason: %ls.",
									*FieldEntity->GetLabel().ToString(), *FieldEntity->GetId().ToString(), *AccessError);
							}
							else
							{
								UE_LOGF(LogAvaPlayableRemoteControl, Error, "Entity \"%ls\" (id:%ls) doesn't have a template default value.",
									*FieldEntity->GetLabel().ToString(), *FieldEntity->GetId().ToString());
							}
							// TODO: UX improvement: instead of skipping, could add empty element, with error mark (and error message in tooltip).
							continue;
						}

						// Ensure the default values have the default flag.
						ensure(DefaultEntityValue->bIsDefault == true);
						
						// WYSIWYG (Solution):
						// Capture the default value (flagged as default) in the current page to ensure all values will be applied to runtime RCP.
						if (!SetSelectedPageEntityValue(FieldEntity, *DefaultEntityValue))
						{
							UE_LOGF(LogAvaPlayableRemoteControl, Error, "Entity \"%ls\" (id:%ls): failed to set value in currently selected page.",
								*FieldEntity->GetLabel().ToString(), *FieldEntity->GetId().ToString());
						}

						EntityValue = DefaultEntityValue;
					}
					
					// Update Exposed entity value with value from page.
					using namespace UE::AvaPlayableRemoteControl;
					if (const EAvaPlayableRemoteControlResult Result = SetValueOfEntity(FieldEntity, EntityValue->Value); Failed(Result))
					{
						UE_LOGF(LogAvaPlayableRemoteControl, Error, "Entity \"%ls\" (id:%ls): failed to set entity value: %ls.",
							*FieldEntity->GetLabel().ToString(), *FieldEntity->GetId().ToString(), *EnumToString(Result));
						// TODO: UX improvement: instead of skipping, could add empty element, with error mark (and error message in tooltip).
						continue;
					}

					const bool bEntityControlled = ManagedInstance->GetRemoteControlPresetInfo().EntitiesControlledByController.Contains(FieldEntity->GetId());

					if (!bEntityControlled || bShowControlledProperties)
					{
						ItemsToAdd.Add({ FieldEntity, bEntityControlled });
					}
				}
			}

			const TArray<FGuid>& DefaultGroupOrder = RemoteControlPreset->Layout.GetDefaultGroupOrder();

			// Before adding to new Items, sort the entities based on the Default Group Order.
			ItemsToAdd.Sort(
				[&DefaultGroupOrder](const FAvaPropertyDetails& InLeftItem, const FAvaPropertyDetails& InRightItem)
				{
					const int32 LeftIndex = DefaultGroupOrder.IndexOfByKey(InLeftItem.Entity->GetId());
					const int32 RightIndex = DefaultGroupOrder.IndexOfByKey(InRightItem.Entity->GetId());
					return LeftIndex < RightIndex;
				});

			NewItems.Append(MoveTemp(ItemsToAdd));
		}

		FieldContainer->RebuildList();

		bool bRecreateList = NewItems.Num() != FieldItems.Num();

		if (!bRecreateList)
		{
			for (int32 PropertyIdx = 0; PropertyIdx < NewItems.Num(); ++PropertyIdx)
			{
				const TSharedPtr<FAvaRundownRCFieldItem>& FieldItem = FieldItems[PropertyIdx];
				const FAvaPropertyDetails& NewItem = NewItems[PropertyIdx];

				if (!FieldItem.IsValid())
				{
					bRecreateList = true;
					break;
				}

				TSharedPtr<FRemoteControlEntity> FieldEntity = FieldItem->GetEntity();
				if (!FieldEntity.IsValid())
				{
					bRecreateList = true;
					break;
				}

				if (FieldEntity.Get() != NewItem.Entity.ToSharedPtr().Get())
				{
					bRecreateList = true;
					break;
				}
			}
		}

		if (!bRecreateList)
		{
			RefreshTable();
			return;
		}

		FieldItems.Empty(NewItems.Num());

		TSharedRef<SAvaRundownPageRemoteControlProps> This = SharedThis(this);
		for (const FAvaPropertyDetails& NewItem : NewItems)
		{
			if (TSharedPtr<FAvaRundownRCFieldItem> Item =  FAvaRundownRCFieldItem::CreateItem(This, NewItem.Entity, NewItem.bEntityControlled))
			{
				FieldItems.Add(MoveTemp(Item));
			}
		}

		FieldItems.StableSort(
			[](const TSharedPtr<FAvaRundownRCFieldItem>& InItemA, const TSharedPtr<FAvaRundownRCFieldItem>& InItemB)
			{
				return InItemA->IsEntityControlled() < InItemB->IsEntityControlled();
			});
	}
}

void SAvaRundownPageRemoteControlProps::SetShowControlledProperties(bool bInShowControlledProperties)
{
	bShowControlledProperties = bInShowControlledProperties;
	Refresh({ActivePageId});
}

void SAvaRundownPageRemoteControlProps::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// This tick happens before Tree View tick
	// Ensure all the items have their hierarchies up-to-date prior to Tree Refresh
	if (bRefreshRequested)
	{
		for (FAvaRundownPageRCObject& PageRCObject : PageRCObjects)
		{
			PageRCObject.CacheTreeNodes();
		}

		TSharedRef<SAvaRundownPageRemoteControlProps> This = SharedThis(this);
		for (const TSharedPtr<FAvaRundownRCFieldItem>& FieldItem : FieldItems)
		{
			FieldItem->Refresh(This);
		}
		UpdateItemExpansionsRecursive(FieldItems);
		bRefreshRequested = false;
	}
}

// Remarks:
// This is called by URemoteControlPreset::OnEndFrame() as a result of an entity being modified.
// However, it doesn't seem to be called (or not always) if the entity is modified by a controller action. 
void SAvaRundownPageRemoteControlProps::OnRemoteControlExposedPropertiesModified(URemoteControlPreset* InPreset, const TSet<FGuid>& InModifiedProperties)
{
	// Note: Ignore changes from the RCP Transaction listener.
	if (!IsValid(InPreset) || !HasRemoteControlPreset(InPreset) || GIsTransacting)
	{
		return;
	}

	if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		if (SaveRemoteControlEntitiesToPage(InPreset, InModifiedProperties, RundownEditor->GetRundown(), ActivePageId))
		{
			RundownEditor->MarkAsModified();
		}
	}

	RequestRefresh();
}

void SAvaRundownPageRemoteControlProps::OnRemoteControlControllerModified(URemoteControlPreset* InPreset, const TSet<FGuid>& InModifiedControllerIds)
{
	// Note: Ignore changes from the RCP Transaction listener.
	if (!IsValid(InPreset) || !HasRemoteControlPreset(InPreset) || GIsTransacting)
	{
		return;
	}

	TSet<FGuid> EntityIds;
	for (const FGuid& ControllerId : InModifiedControllerIds)
	{
		UE::RCCustomControllers::GetEntitiesControlledByController(InPreset, InPreset->GetController(ControllerId), EntityIds);
	}

	// If a controller changed, we need to propagate the refresh of the field's widgets.
	// Optimization: only refresh the widgets that are related to the modified controllers.
	RefreshTable(EntityIds);

	// It seems OnPropertyChangedDelegate (OnExposedPropertiesModified()) is not called when properties are
	// changed by controllers. Ensure the values are saved by calling our handler directly.
	OnRemoteControlExposedPropertiesModified(InPreset, EntityIds);

	RequestRefresh();
}

void SAvaRundownPageRemoteControlProps::OnPostPropertyChanged(FProperty* InPropertyThatChanged)
{
	// Find which property of which preset this is.
	for (const TSharedPtr<FAvaRundownManagedInstance>& ManagedInstance : ManagedInstances)
	{
		if (URemoteControlPreset* Preset = ManagedInstance ? ManagedInstance->GetRemoteControlPreset() : nullptr)
		{
			for (const TWeakPtr<FRemoteControlProperty>& ExposedPropertyWeak : Preset->GetExposedEntities<FRemoteControlProperty>())
			{
				if (const TSharedPtr<FRemoteControlProperty> ExposedProperty = ExposedPropertyWeak.Pin())
				{
					if (ExposedProperty->GetProperty() == InPropertyThatChanged)
					{
						OnRemoteControlExposedPropertiesModified(Preset, {ExposedProperty->GetId()});
					}
				}
			}
		}
	}

	RequestRefresh();
}

void SAvaRundownPageRemoteControlProps::RequestRefresh()
{
	if (FieldContainer.IsValid())
	{
		FieldContainer->RequestTreeRefresh();
	}
	bRefreshRequested = true;
}

void SAvaRundownPageRemoteControlProps::BindRemoteControlDelegates(URemoteControlPreset* InPreset)
{
	if (IsValid(InPreset))
	{
		if (!InPreset->OnEntityExposed().IsBoundToObject(this))
		{
			InPreset->OnEntityExposed().AddSP(this, &SAvaRundownPageRemoteControlProps::OnRemoteControlEntitiesExposed);
		}

		if (!InPreset->OnEntityUnexposed().IsBoundToObject(this))
		{
			InPreset->OnEntityUnexposed().AddSP(this, &SAvaRundownPageRemoteControlProps::OnRemoteControlEntitiesUnexposed);
		}

		if (!InPreset->OnEntitiesUpdated().IsBoundToObject(this))
		{
			InPreset->OnEntitiesUpdated().AddSP(this, &SAvaRundownPageRemoteControlProps::OnRemoteControlEntitiesUpdated);
		}

		if (!InPreset->OnExposedPropertiesModified().IsBoundToObject(this))
		{
			InPreset->OnExposedPropertiesModified().AddSP(this, &SAvaRundownPageRemoteControlProps::OnRemoteControlExposedPropertiesModified);
		}

		if (!InPreset->OnControllerModified().IsBoundToObject(this))
		{
			InPreset->OnControllerModified().AddSP(this, &SAvaRundownPageRemoteControlProps::OnRemoteControlControllerModified);
		}
	}
}

bool SAvaRundownPageRemoteControlProps::HasRemoteControlPreset(const URemoteControlPreset* InPreset) const
{
	for (const TSharedPtr<FAvaRundownManagedInstance>& ManagedInstance : ManagedInstances)
	{
		if (ManagedInstance && ManagedInstance->GetRemoteControlPreset() == InPreset)
		{
			return true;
		}
	}
	return false;
}

FAvaRundownPage* SAvaRundownPageRemoteControlProps::GetActivePage() const
{
	if (ActivePageId == FAvaRundownPage::InvalidPageId)
	{
		return nullptr;
	}

	if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		if (RundownEditor->IsRundownValid())
		{
			FAvaRundownPage& Page = RundownEditor->GetRundown()->GetPage(ActivePageId);
			return Page.IsValidPage() ? &Page : nullptr;
		}
	}

	return nullptr;
}

const FAvaPlayableRemoteControlValue* SAvaRundownPageRemoteControlProps::GetSelectedPageEntityValue(const TSharedPtr<FRemoteControlEntity>& InRemoteControlEntity) const
{
	if (InRemoteControlEntity.IsValid())
	{
		if (const FAvaRundownPage* Page = GetActivePage())
		{
			return Page->GetRemoteControlEntityValue(InRemoteControlEntity->GetId()); 
		}
	}

	return nullptr;
}

bool SAvaRundownPageRemoteControlProps::SetSelectedPageEntityValue(const TSharedPtr<FRemoteControlEntity>& InRemoteControlEntity, const FAvaPlayableRemoteControlValue& InValue) const
{
	if (InRemoteControlEntity.IsValid())
	{
		if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
		{
			if (RundownEditor->IsRundownValid())
			{
				// Using the rundown API for event propagation.
				UAvaRundown* Rundown = RundownEditor->GetRundown(); 
				return Rundown->SetRemoteControlEntityValue(ActivePageId, InRemoteControlEntity->GetId(), InValue);
			}
		}
	}

	return false;
}

const TArray<TSharedPtr<FAvaRundownRCFieldItem>> SAvaRundownPageRemoteControlProps::GetSelectedPropertyItems() const
{
	return FieldContainer->GetSelectedItems();
}

bool SAvaRundownPageRemoteControlProps::SaveRemoteControlEntitiesToPage(const URemoteControlPreset* InPreset, const TSet<FGuid>& InPropertyIds, UAvaRundown* InRundown, int32 InPageId)
{
	using namespace UE::AvaPlayableRemoteControl;
	
	if (!IsValid(InPreset) || InPageId == FAvaRundownPage::InvalidPageId || !InRundown)
	{
		return false;
	}

	const FAvaRundownPage& Page = InRundown->GetPage(InPageId);

	bool bModified = false;

	for (const FGuid& Id : InPropertyIds)
	{
		if (const TSharedPtr<const FRemoteControlEntity> Entity = InPreset->GetExposedEntity<const FRemoteControlEntity>(Id).Pin())
		{
			FAvaPlayableRemoteControlValue EntityValue;
			
			if (const EAvaPlayableRemoteControlResult Result = GetValueOfEntity(Entity, EntityValue.Value); Failed(Result))
			{
				UE_LOGF(LogAvaPlayableRemoteControl, Error, "Unable to get value of entity \"%ls\": %ls",
					*Entity->GetLabel().ToString(), *EnumToString(Result));
				continue;
			}
			
			const FAvaPlayableRemoteControlValue* StoredEntityValue = Page.GetRemoteControlEntityValue(Entity->GetId());

			if (StoredEntityValue && StoredEntityValue->IsSameValueAs(EntityValue))
			{
				continue;	// Skip if value is identical.
			}
			
			if (!InRundown->SetRemoteControlEntityValue(InPageId, Entity->GetId(), EntityValue))
			{
				UE_LOGF(LogAvaPlayableRemoteControl, Error, "Unable to set page entity value for: \"%ls\"", *Entity->GetLabel().ToString());
				continue;
			}

			bModified = true;
		}
	}
	return bModified;
}

TSharedPtr<FAvaRundownPageRCPropsNotifyHook> SAvaRundownPageRemoteControlProps::GetNotifyHook() const
{
	return NotifyHook;
}

FAvaRundownPageRCObject& SAvaRundownPageRemoteControlProps::FindOrAddPageRCObject(UObject* InObject)
{
	bool bIsAlreadyInSet;
	FAvaRundownPageRCObject& PageRCObject = PageRCObjects.FindOrAdd(FAvaRundownPageRCObject(InObject), &bIsAlreadyInSet);

	// Initialize if new element in set
	if (!bIsAlreadyInSet)
	{
		PageRCObject.Initialize(NotifyHook.Get());
	}

	return PageRCObject;
}

TSharedPtr<SWidget> SAvaRundownPageRemoteControlProps::GetContextMenuContent()
{
	TArray<TSharedPtr<FAvaRundownRCFieldItem>> SelectedItems = GetSelectedPropertyItems();
	if (SelectedItems.Num() > 0)
	{
		return ContextMenu->GeneratePageContextMenuWidget(RundownEditorWeak, *GetActivePage(), SharedThis(this));
	}
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE
