// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaRundownPageRCObject.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Misc/NotifyHook.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectKey.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"

class FAvaRundownEditor;
class FAvaRundownManagedInstance;
class FAvaRundownPagePropertyContextMenu;
class FAvaRundownRCDetailTreeNodeItem;
class FAvaRundownRCFieldItem;
class FName;
class FUICommandList;
class ITableRow;
class SAvaRundownPageDetails;
class SAvaRundownPageRemoteControlProps;
class STableViewBase;
class SWidget;
class UAvaRundown;
class URemoteControlPreset;
struct FAvaPlayableRemoteControlValue;
struct FAvaRundownPage;
struct FGuid;
struct FRemoteControlEntity;
template<typename ItemType> class STreeView;

namespace UE::AvaPlayback::Utils
{
	class FAsyncAssetLoader;
}

DECLARE_MULTICAST_DELEGATE_TwoParams(FAvaRundownRCPropertyHeaderRowExtensionDelegate, TSharedRef<SAvaRundownPageRemoteControlProps> Panel,
	TSharedRef<SHeaderRow>& HeaderRow)
DECLARE_DELEGATE_ThreeParams(FAvaRundownRCPropertyTableRowExtensionDelegate, TSharedRef<SAvaRundownPageRemoteControlProps> Panel,
	TSharedRef<const FAvaRundownRCFieldItem> ItemPtr, TSharedPtr<SWidget>& CurrentWidget)

/**
 * Note:
 *  Making a separate object for notify hook to break the SharedPtr circular dependency:
 *  PanelWidget ->(strong) ItemWidget -> (strong) Hook -> (weak) PanelWidget
 *  The ItemWidget has a weak ptr on the PanelWidget, but the Hook needs to be strong to match
 *  ownership with the PropertyRowGenerator that is using the Hook.
 */
class FAvaRundownPageRCPropsNotifyHook : public FNotifyHook
{
public:
	// Need to define a virtual destructor because FNotifyHook doesn't.
	virtual ~FAvaRundownPageRCPropsNotifyHook() = default;
};

/**
 * The page props implementation for remote control fields.
 */
class SAvaRundownPageRemoteControlProps : public SCompoundWidget
{
public:
	static const FName PropertyColumnName;
	static const FName ValueColumnName;

	static FAvaRundownRCPropertyHeaderRowExtensionDelegate& GetHeaderRowExtensionDelegate() { return HeaderRowExtensionDelegate; }
	static TArray<FAvaRundownRCPropertyTableRowExtensionDelegate>& GetTableRowExtensionDelegates(FName InExtensionName);

	SLATE_BEGIN_ARGS(SAvaRundownPageRemoteControlProps) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<SAvaRundownPageDetails>& InPageDetailPanel, const TSharedPtr<FAvaRundownEditor>& InRundownEditor);
	virtual ~SAvaRundownPageRemoteControlProps() override;

	/** Update the current page's remote control values from the defaults then refresh the widget. */
	void UpdateDefaultValuesAndRefresh(const TArray<int32>& InSelectedPageIds);

	/** Refreshes the content of this widget. */
	void Refresh(const TArray<int32>& InSelectedPageIds);

	/** Sets the visibility on the properties that are set by controllers */
	void SetShowControlledProperties(bool bInShowControlledProperties);

	//~ Begin SWidget
	virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

	const TArray<TSharedPtr<FAvaRundownRCFieldItem>> GetSelectedPropertyItems() const;
	
	TSharedPtr<FAvaRundownPageRCPropsNotifyHook> GetNotifyHook() const;

	FAvaRundownPageRCObject& FindOrAddPageRCObject(UObject* InObject);

	/**
	 * Utility to save the given Preset's Entities to the given page's property.
	 * @param InPreset Source Preset the entity values are read from.
	 * @param InEntityIds Set of entity ids to read from the given preset and save to the given page.
	 * @param InRundown Rundown.
	 * @param InPageId Page Id to save the value to.
	 */
	static bool SaveRemoteControlEntitiesToPage(const URemoteControlPreset* InPreset, const TSet<FGuid>& InEntityIds, UAvaRundown* InRundown, int32 InPageId);

private:
	static FAvaRundownRCPropertyHeaderRowExtensionDelegate HeaderRowExtensionDelegate;
	static TMap<FName, TArray<FAvaRundownRCPropertyTableRowExtensionDelegate>> TableRowExtensionDelegates;

	void OnRemoteControlEntitiesExposed(URemoteControlPreset* InPreset, const FGuid& InEntityId) { UpdateDefaultValuesAndRefresh({GetActivePageId()}); }
	void OnRemoteControlEntitiesUnexposed(URemoteControlPreset* InPreset, const FGuid& InEntityId) { UpdateDefaultValuesAndRefresh({GetActivePageId()}); }
	void OnRemoteControlEntitiesUpdated(URemoteControlPreset* InPreset, const TSet<FGuid>& InModifiedEntities) { UpdateDefaultValuesAndRefresh({GetActivePageId()}); }
	void OnRemoteControlExposedPropertiesModified(URemoteControlPreset* InPreset, const TSet<FGuid>& InModifiedProperties);
	void OnRemoteControlControllerModified(URemoteControlPreset* InPreset, const TSet<FGuid>& InModifiedControllerIds);

	void OnPostPropertyChanged(FProperty* InPropertyThatChanged);

	void RequestRefresh();

	void BindRemoteControlDelegates(URemoteControlPreset* InPreset);

	bool HasRemoteControlPreset(const URemoteControlPreset* InPreset) const;

	/** Returns the currently selected page if 1 page is currently selected, returns nullptr otherwise. */
	FAvaRundownPage* GetActivePage() const;

	/** Returns the currently selected page id if 1 page is currently selected, returns InvalidPageId otherwise. */
	int32 GetActivePageId() const { return ActivePageId; }

	/**
	 * Get Selected Page's entity value corresponding to the given entity (using entity Id to match).
	 * @return pointer to page's entity value, null if not found.
	 */
	const FAvaPlayableRemoteControlValue* GetSelectedPageEntityValue(const TSharedPtr<FRemoteControlEntity>& InRemoteControlEntity) const;

	/**
	 * Set (or add) Selected Page's entity value corresponding to the given entity (using entity Id to match).
	 * @return true if it succeeded, false otherwise.
	 */
	bool SetSelectedPageEntityValue(const TSharedPtr<FRemoteControlEntity>& InRemoteControlEntity, const FAvaPlayableRemoteControlValue& InValue) const;

	TSharedRef<ITableRow> OnGenerateControllerRow(TSharedPtr<FAvaRundownRCFieldItem> InItem, const TSharedRef<STableViewBase>& InOwnerTable);

	void OnGetEntityChildren(TSharedPtr<FAvaRundownRCFieldItem> InItem, TArray<TSharedPtr<FAvaRundownRCFieldItem>>& OutChildren);

	void OnExpansionChanged(TSharedPtr<FAvaRundownRCFieldItem> InItem, bool bInIsExpanded);

	void UpdateItemExpansionsRecursive(TConstArrayView<TSharedPtr<FAvaRundownRCFieldItem>> InItems);

	void RefreshTable(const TSet<FGuid>& InEntityIds = TSet<FGuid>());

	TSharedPtr<SWidget> GetContextMenuContent();

	TWeakPtr<FAvaRundownEditor> RundownEditorWeak;
	
	TArray<TSharedPtr<FAvaRundownManagedInstance>> ManagedInstances;

	/** The widget that lists the property rows. */
	TSharedPtr<STreeView<TSharedPtr<FAvaRundownRCFieldItem>>> FieldContainer;

	/** The data used to back the properties container list view. */
	TArray<TSharedPtr<FAvaRundownRCFieldItem>> FieldItems;

	int32 ActivePageId = -1;

	TSharedPtr<FUICommandList> CommandList;

	TSharedPtr<FAvaRundownPagePropertyContextMenu> ContextMenu;

	TSharedPtr<FAvaRundownPageRCPropsNotifyHook> NotifyHook;

	TSet<FAvaRundownPageRCObject> PageRCObjects;

	TSet<FString> ExpandedPaths;

	friend class FAvaRundownPageRCPropsNotifyHookImpl;

	bool bRefreshRequested = false;

	/** Whether the properties that are set by controllers are shown */
	bool bShowControlledProperties = false;

	TSharedPtr<UE::AvaPlayback::Utils::FAsyncAssetLoader> AsyncAssetLoader;
};