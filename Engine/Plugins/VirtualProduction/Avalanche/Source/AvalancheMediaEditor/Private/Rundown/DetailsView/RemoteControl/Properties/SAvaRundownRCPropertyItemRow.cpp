// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownRCPropertyItemRow.h"
#include "GameFramework/Actor.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Internationalization/Text.h"
#include "Item/AvaRundownRCFieldItem.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "RemoteControlEntity.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaRundownRCPropertyItemRow"

void SAvaRundownRCPropertyItemRow::Construct(const FArguments& InArgs, TSharedRef<SAvaRundownPageRemoteControlProps> InPropertyPanel,const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<const FAvaRundownRCFieldItem>& InRowItem)
{
	FieldItemWeak = InRowItem;
	PropertyPanelWeak = InPropertyPanel;
	NotifyHook = InPropertyPanel->GetNotifyHook();
	ValueContainer = nullptr;

	SMultiColumnTableRow<TSharedPtr<FAvaRundownRCFieldItem>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

TSharedRef<SWidget> SAvaRundownRCPropertyItemRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	TSharedPtr<const FAvaRundownRCFieldItem> ItemPtr = FieldItemWeak.Pin();

	if (ItemPtr.IsValid())
	{
		if (InColumnName == SAvaRundownPageRemoteControlProps::PropertyColumnName)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SExpanderArrow, SharedThis(this))
					.IndentAmount(12)
					.ShouldDrawWires(true)
				]
				+ SHorizontalBox::Slot()
				[
					CreateName()
				];
		}
		else if (InColumnName == SAvaRundownPageRemoteControlProps::ValueColumnName)
		{
			return SAssignNew(ValueContainer, SBox)
				.MinDesiredHeight(26.f)
				[
					CreateValue()
				];
		}
		else
		{
			TSharedPtr<SAvaRundownPageRemoteControlProps> PropertyPanel = PropertyPanelWeak.Pin();

			if (PropertyPanel.IsValid())
			{
				TSharedPtr<SWidget> Cell = nullptr;
				const TArray<FAvaRundownRCPropertyTableRowExtensionDelegate>& TableRowExtensionDelegates = PropertyPanel->GetTableRowExtensionDelegates(InColumnName);

				for (const FAvaRundownRCPropertyTableRowExtensionDelegate& TableRowExtensionDelegate : TableRowExtensionDelegates)
				{
					TableRowExtensionDelegate.ExecuteIfBound(PropertyPanel.ToSharedRef(), ItemPtr.ToSharedRef(), Cell);
				}

				if (Cell.IsValid())
				{
					return Cell.ToSharedRef();
				}
			}
		}
	}

	return SNullWidget::NullWidget;
}

void SAvaRundownRCPropertyItemRow::UpdateValue()
{
	if (ValueContainer.IsValid())
	{
		ValueContainer->SetContent(CreateValue());
	}
}

TSharedRef<SWidget> SAvaRundownRCPropertyItemRow::CreateName()
{
	TSharedPtr<const FAvaRundownRCFieldItem> FieldItem = FieldItemWeak.Pin();
	if (!FieldItem.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	const FNodeWidgets& NodeWidgets = FieldItem->GetNodeWidgets();

	TSharedPtr<SWidget> NameWidget = NodeWidgets.NameWidget;
	if (!NameWidget.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	NameWidget->SetToolTipText(TAttribute<FText>::CreateSP(this, &SAvaRundownRCPropertyItemRow::GetPropertyTooltipText));

	return SNew(SBox)
		.HAlign(NodeWidgets.NameWidgetLayoutData.HorizontalAlignment)
		.VAlign(NodeWidgets.NameWidgetLayoutData.VerticalAlignment)
		[
			NameWidget.ToSharedRef()
		];
}

TSharedRef<SWidget> SAvaRundownRCPropertyItemRow::CreateValue()
{
	TSharedPtr<const FAvaRundownRCFieldItem> FieldItem = FieldItemWeak.Pin();
	if (!FieldItem.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	const FNodeWidgets& NodeWidgets = FieldItem->GetNodeWidgets();
	if (!NodeWidgets.ValueWidget.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	TSharedRef<SHorizontalBox> ValueRow = SNew(SHorizontalBox);

	ValueRow->AddSlot()
		.HAlign(NodeWidgets.ValueWidgetLayoutData.HorizontalAlignment)
		.VAlign(NodeWidgets.ValueWidgetLayoutData.VerticalAlignment)
		[
			NodeWidgets.ValueWidget.ToSharedRef()
		];

	if (FieldItem->IsEntityControlled())
	{
		ValueRow->SetEnabled(false);
		ValueRow->AddSlot()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(3.f, 0.f, 0.f, 0.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Controlled", "(Controlled)"))
			];
	}

	return ValueRow;
}

FText SAvaRundownRCPropertyItemRow::GetPropertyTooltipText() const
{
	TSharedPtr<const FAvaRundownRCFieldItem> ItemPtr = FieldItemWeak.Pin();

	FText OwnerText = LOCTEXT("InvalidOwnerText", "(Invalid)");
	FText SubobjectPathText = LOCTEXT("InvalidSubobjectPathText", "(Invalid)");

	const TSharedPtr<FRemoteControlEntity> Entity = ItemPtr ? ItemPtr->GetEntity() : nullptr;
	if (Entity.IsValid())
	{
		const FString BindingPath = Entity->GetLastBindingPath().ToString();

		FName OwnerName;
		if (UObject* Object = Entity->GetBoundObject())
		{
			if (AActor* OwnerActor = Object->GetTypedOuter<AActor>())
			{
				OwnerText = FText::FromString(OwnerActor->GetActorLabel());
				OwnerName = OwnerActor->GetFName();
			}
			else if (AActor* Actor = Cast<AActor>(Object))
			{
				OwnerText = FText::FromString(Actor->GetActorLabel());
				OwnerName = Object->GetFName();
			}
			else
			{
				OwnerText = FText::FromString(Object->GetName());
				OwnerName = Object->GetFName();
			}
		}
		else
		{
			static const FString PersistentLevelString = TEXT(":PersistentLevel.");
			const int32 PersistentLevelIndex = BindingPath.Find(PersistentLevelString);
			if (PersistentLevelIndex != INDEX_NONE)
			{
				OwnerText = FText::FromName(OwnerName);
				OwnerName = *BindingPath.RightChop(PersistentLevelIndex + PersistentLevelString.Len());
			}
		}

		const int32 OwnerNameIndex = BindingPath.Find(OwnerName.ToString() + TEXT("."));
		if (OwnerNameIndex != INDEX_NONE)
		{
			SubobjectPathText = FText::FromString(*BindingPath.RightChop(OwnerNameIndex + OwnerName.GetStringLength() + 1));
		}
	}

	return FText::Format(LOCTEXT("PropertyTooltipText", "Owner: {0}\nSubobjectPath: {1}"), OwnerText, SubobjectPathText);
}

#undef LOCTEXT_NAMESPACE
