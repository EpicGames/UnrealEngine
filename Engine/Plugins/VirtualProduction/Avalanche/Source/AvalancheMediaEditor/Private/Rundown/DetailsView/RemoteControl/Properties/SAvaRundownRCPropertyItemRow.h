// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyRowGenerator.h"
#include "SAvaRundownPageRemoteControlProps.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableRow.h"

class FAvaRundownRCFieldItem;
class FText;

class SAvaRundownRCPropertyItemRow : public SMultiColumnTableRow<TSharedPtr<FAvaRundownRCFieldItem>>
{
public:
	SLATE_BEGIN_ARGS(SAvaRundownRCPropertyItemRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<SAvaRundownPageRemoteControlProps> InPropertyPanel,
		const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedPtr<const FAvaRundownRCFieldItem>& InRowItem);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

	void UpdateValue();

protected:
	TSharedRef<SWidget> CreateName();

	TSharedRef<SWidget> CreateValue();

	FText GetPropertyTooltipText() const;

	TWeakPtr<const FAvaRundownRCFieldItem> FieldItemWeak;
	TWeakPtr<SAvaRundownPageRemoteControlProps> PropertyPanelWeak;
	TSharedPtr<FAvaRundownPageRCPropsNotifyHook> NotifyHook;
	TSharedPtr<SBox> ValueContainer;
};
