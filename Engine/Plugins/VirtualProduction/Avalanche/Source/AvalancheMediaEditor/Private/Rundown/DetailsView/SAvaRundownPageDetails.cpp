// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownPageDetails.h"

#include "AvaRundownPageDetailView.h"
#include "Async/Async.h"
#include "Framework/Application/SlateApplication.h"
#include "IAvaMediaModule.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "RemoteControl/Controllers/SAvaRundownRCControllerPanel.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownEditor.h"
#include "Rundown/AvaRundownEditorSettings.h"
#include "Rundown/AvaRundownManagedInstanceCache.h"
#include "Rundown/AvaRundownPage.h"
#include "Rundown/DetailsView/RemoteControl/Properties/SAvaRundownPageRemoteControlProps.h"
#include "Rundown/Pages/Slate/SAvaRundownInstancedPageList.h"
#include "Rundown/Pages/Slate/SAvaRundownPageList.h"
#include "Styling/SlateBrush.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaRundownPageDetails"

namespace UE::AvaMedia::RundownEditor::Private
{
	bool ShouldPageDetailsShowProperties()
	{
		const UAvaRundownEditorSettings* RundownEditorSettings = UAvaRundownEditorSettings::Get();
		return RundownEditorSettings && RundownEditorSettings->bPageDetailsShowProperties;
	}
}

void SAvaRundownPageDetails::Construct(const FArguments& InArgs, const TSharedPtr<FAvaRundownEditor>& InRundownEditor)
{
	RundownEditorWeak = InRundownEditor;
	ActivePageId = FAvaRundownPage::InvalidPageId;

	InRundownEditor->GetOnPageEvent().AddSP(this, &SAvaRundownPageDetails::OnPageEvent);
	IAvaMediaModule::Get().GetManagedInstanceCache().OnEntryInvalidated.AddSP(this, &SAvaRundownPageDetails::OnManagedInstanceCacheEntryInvalidated);

	UAvaRundown* const Rundown = InRundownEditor->GetRundown();
	if (IsValid(Rundown))
	{
		Rundown->GetOnPagesChanged().AddSP(this, &SAvaRundownPageDetails::OnPagesChanged);
		Rundown->GetOnPageListChanged().AddSP(this, &SAvaRundownPageDetails::OnPageListChanged);
	}

	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;

	PageDetailView = EditModule.CreateDetailView(DetailsViewArgs);
	PageDetailView->OnFinishedChangingProperties().AddSP(this, &SAvaRundownPageDetails::OnFinishedChangingPageDetailProperties);

	ChildSlot
	[
		SNew(SScrollBox)
		+ SScrollBox::Slot()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(10.f, 10.f, 10.f, 0.f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.MaxWidth(75.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PageId", "Page Id"))
					.MinDesiredWidth(75.f)
				]
				+ SHorizontalBox::Slot()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.Padding(5.f, 0.f, 0.f, 0.f)
				.MaxWidth(70.f)
				[
					SNew(SEditableTextBox)
					.HintText(LOCTEXT("PageIdHint", "Page Id"))
					.OnTextCommitted(this, &SAvaRundownPageDetails::OnPageIdCommitted)
					.Text(this, &SAvaRundownPageDetails::GetPageId)
					.IsEnabled(this, &SAvaRundownPageDetails::HasSelectedPage)
				]
				+ SHorizontalBox::Slot()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.Padding(5.f, 0.f, 0.f, 0.f)
				.AutoWidth()
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText(LOCTEXT("DuplicatePageTooltip", "DuplicatePage"))
					.OnClicked(this, &SAvaRundownPageDetails::DuplicateSelectedPage)
					.IsEnabled(this, &SAvaRundownPageDetails::HasSelectedPage)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("GenericCommands.Duplicate"))
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(10.f, 3.f, 10.f, 0.f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.MaxWidth(75.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PageName", "Page Name"))
					.MinDesiredWidth(75.f)
				]
				+ SHorizontalBox::Slot()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.Padding(5.f, 0.f, 0.f, 0.f)
				[
					SNew(SEditableTextBox)
					.HintText(LOCTEXT("PageNameHint", "Page Name"))
					.OnTextChanged(this, &SAvaRundownPageDetails::OnPageNameChanged)
					.Text(this, &SAvaRundownPageDetails::GetPageDescription)
					.IsEnabled(this, &SAvaRundownPageDetails::HasSelectedPage)
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(10.f, 10.f, 10.f, 0.f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Info"))
				]
				+ SHorizontalBox::Slot()
				.Padding(10, 0, 0, 0)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PerformanceTip_Desc", "Close Page Details during in-editor playback to improve performance.\n(Does not affect remote playback or playback in another process)"))
					.Font(FAppStyle::GetFontStyle("NormalFontItalic"))
				]
			]
			// Controllers
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.f, 10.f, 0.f, 0.f))
			[
				SAssignNew(RCControllerPanel, SAvaRundownRCControllerPanel, InRundownEditor)
			]
			// Exposed Properties
			+ SVerticalBox::Slot()
			.Padding(FMargin(0.f, 10.f, 0.f, 0.f))
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(5.f, 0.f, 0.f, 0.f)
				.AutoWidth()
				[
					SNew(SButton)
					.ContentPadding(0)
					.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
					.OnClicked(this, &SAvaRundownPageDetails::ToggleExposedPropertiesVisibility)
					.ToolTipText(LOCTEXT("VisibilityButtonToolTip", "Toggle Exposed Properties Visibility"))
					.Content()
					[
						SNew(SImage)
						.Image(this, &SAvaRundownPageDetails::GetExposedPropertiesVisibilityBrush)
					]
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(5.f, 0.f, 0.f, 0.f)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Properties", "Properties"))
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(RemoteControlProps, SAvaRundownPageRemoteControlProps, SharedThis(this), InRundownEditor)
			]
			// Page Data View
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				PageDetailView.ToSharedRef()
			]
		]
	];
}

SAvaRundownPageDetails::~SAvaRundownPageDetails()
{
	if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		RundownEditor->GetOnPageEvent().RemoveAll(this);
		UAvaRundown* const Rundown = RundownEditor->GetRundown();
		if (IsValid(Rundown))
		{
			Rundown->GetOnPagesChanged().RemoveAll(this);
			Rundown->GetOnPageListChanged().RemoveAll(this);
		}
	}

	if (IAvaMediaModule::IsModuleLoaded() && IAvaMediaModule::Get().IsManagedInstanceCacheAvailable())
	{
		IAvaMediaModule::Get().GetManagedInstanceCache().OnEntryInvalidated.RemoveAll(this);
	}
}

void SAvaRundownPageDetails::OnPageEvent(const TArray<int32>& InSelectedPageIds, UE::AvaRundown::EPageEvent InPageEvent)
{
	bool bRefreshPanels = false;
	if (InPageEvent == UE::AvaRundown::EPageEvent::SelectionChanged || InPageEvent == UE::AvaRundown::EPageEvent::ReimportRequest)
	{
		const int32 PreviousActivePageId = ActivePageId;
		ActivePageId = InSelectedPageIds.IsEmpty() ? FAvaRundownPage::InvalidPageId : InSelectedPageIds[0];

		// Only refresh the panels if the page id changed or if a reimport request (forced refresh).
		bRefreshPanels = ActivePageId != PreviousActivePageId || InPageEvent == UE::AvaRundown::EPageEvent::ReimportRequest;
	}

	if (bRefreshPanels)
	{
		RemoteControlProps->Refresh(InSelectedPageIds);
		RCControllerPanel->Refresh(InSelectedPageIds);
		RefreshPageDetailView();
	}
}

void SAvaRundownPageDetails::RefreshPageDetailView()
{
	const FRundownContext RundownContext = GetRundownContext();
	if (RundownContext.IsValid())
	{
		const FAvaRundownPage& ActivePage = RundownContext.GetRundown()->GetPage(ActivePageId);
		if (ActivePage.IsValidPage())
		{
			if (!CurrentPageDetailViewData.IsValid())
			{
				CurrentPageDetailViewData.Reset(NewObject<UAvaRundownPageDetailView>());
			}
			CurrentPageDetailViewData->Commands = ActivePage.GetInstancedCommands();
			PageDetailView->SetObject(CurrentPageDetailViewData.Get());
			CurrentPageDetailViewPageId = ActivePageId;
			return;
		}
	}

	const TArray<UObject*> SelectedObjects;
	PageDetailView->SetObjects(SelectedObjects);
}

void SAvaRundownPageDetails::OnManagedInstanceCacheEntryInvalidated(const FSoftObjectPath& InAssetPath)
{
	if (!bRefreshSelectedPageQueued)
	{
		const FRundownContext RundownContext = GetRundownContext();
		const FAvaRundownPage& SelectedPage = GetSelectedPage(RundownContext);
		if (SelectedPage.IsValidPage())
		{
			if (SelectedPage.GetAssetPath(RundownContext.GetRundown()) == InAssetPath)
			{
				// Queue a refresh on next tick.
				// We don't want to refresh immediately to avoid issues with
				// cascading events within the managed instance cache.
				QueueUpdateAndRefreshSelectedPage();
			}
		}
	}
}

void SAvaRundownPageDetails::OnFinishedChangingPageDetailProperties(const FPropertyChangedEvent& InChangedProperty)
{
	// Ansme: is this part of the transaction?
	const FRundownContext RundownContext = GetRundownContext();
	if (RundownContext.IsValid() && CurrentPageDetailViewPageId != FAvaRundownPage::InvalidPageId && CurrentPageDetailViewData.IsValid())
	{
		FAvaRundownPage& Page = RundownContext.GetRundown()->GetPage(CurrentPageDetailViewPageId);
		if (Page.IsValidPage())
		{
			RundownContext.GetRundown()->Modify();
			Page.SetInstancedCommands(CurrentPageDetailViewData->Commands);
			RundownContext.GetRundown()->GetOnPagesChanged().Broadcast(RundownContext.GetRundown(), Page, EAvaRundownPageChanges::Commands);
			if (RCControllerPanel)
			{
				// Commands are also part of the page summary.
				RCControllerPanel->UpdatePageSummary(/*bInForceUpdate*/ true);
			}
		}
	}
}

FReply SAvaRundownPageDetails::ToggleExposedPropertiesVisibility()
{
	if (UAvaRundownEditorSettings* RundownEditorSettings = UAvaRundownEditorSettings::GetMutable())
	{
		RundownEditorSettings->bPageDetailsShowProperties = !RundownEditorSettings->bPageDetailsShowProperties;
		RundownEditorSettings->SaveConfig();
		RemoteControlProps->SetShowControlledProperties(RundownEditorSettings->bPageDetailsShowProperties);
	}
	return FReply::Handled();
}

const FSlateBrush* SAvaRundownPageDetails::GetExposedPropertiesVisibilityBrush() const
{
	if (UE::AvaMedia::RundownEditor::Private::ShouldPageDetailsShowProperties())
	{
		return FAppStyle::GetBrush(TEXT("Level.VisibleHighlightIcon16x"));
	}
	else
	{
		return FAppStyle::GetBrush(TEXT("Level.NotVisibleHighlightIcon16x"));
	}
}

bool SAvaRundownPageDetails::FRundownContext::IsValid() const
{
	return RundownEditor && RundownEditor->IsRundownValid(); 
}

UAvaRundown* SAvaRundownPageDetails::FRundownContext::GetRundown() const
{
	return IsValid() ? RundownEditor->GetRundown() : nullptr; 
}

SAvaRundownPageDetails::FRundownContext SAvaRundownPageDetails::GetRundownContext() const
{
	FRundownContext RundownContext;
	RundownContext.RundownEditor = RundownEditorWeak.Pin();
	return RundownContext;
}

const FAvaRundownPage& SAvaRundownPageDetails::GetSelectedPage(const FRundownContext& InRundownContext) const
{
	if (HasSelectedPage() && InRundownContext.IsValid())
	{
		return InRundownContext.GetRundown()->GetPage(ActivePageId);
	}

	return FAvaRundownPage::NullPage;
}

FAvaRundownPage& SAvaRundownPageDetails::GetMutableSelectedPage(const FRundownContext& InRundownContext) const
{
	if (HasSelectedPage() && InRundownContext.IsValid())
	{
		return InRundownContext.GetRundown()->GetPage(ActivePageId);
	}

	return FAvaRundownPage::NullPage;
}

void SAvaRundownPageDetails::QueueRefreshSelectedPage()
{
	if (bRefreshSelectedPageQueued)
	{
		return;
	}
	bRefreshSelectedPageQueued = true;
	
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSPLambda(this, [this](float)
	{
		const FRundownContext RundownContext = GetRundownContext();
		const FAvaRundownPage& SelectedPage = GetSelectedPage(RundownContext);
		if (SelectedPage.IsValidPage())
		{
			RemoteControlProps->Refresh({SelectedPage.GetPageId()});
			RCControllerPanel->Refresh({SelectedPage.GetPageId()});
			RefreshPageDetailView();
		}
		bRefreshSelectedPageQueued = false;
		return false;
	}));
}

void SAvaRundownPageDetails::QueueUpdateAndRefreshSelectedPage()
{
	if (bUpdateAndRefreshSelectedPageQueued)
	{
		return;
	}
	bUpdateAndRefreshSelectedPageQueued = true;

	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSPLambda(this, [this](float)
	{
		const FRundownContext RundownContext = GetRundownContext();
		const FAvaRundownPage& SelectedPage = GetSelectedPage(RundownContext);
		if (SelectedPage.IsValidPage())
		{
			RemoteControlProps->UpdateDefaultValuesAndRefresh({SelectedPage.GetPageId()});
			RCControllerPanel->Refresh({SelectedPage.GetPageId()});
			RefreshPageDetailView();
		}
		bUpdateAndRefreshSelectedPageQueued = true;
		return false;
	}));
}

bool SAvaRundownPageDetails::HasSelectedPage() const
{
	if (ActivePageId == FAvaRundownPage::InvalidPageId)
	{
		return false;
	}

	return RundownEditorWeak.IsValid();
}

FText SAvaRundownPageDetails::GetPageId() const
{
	const FRundownContext RundownContext = GetRundownContext();
	const FAvaRundownPage& SelectedPage = GetSelectedPage(RundownContext);

	if (SelectedPage.IsValidPage())
	{
		return FText::AsNumber(SelectedPage.GetPageId(), &UE::AvaRundown::FEditorMetrics::PageIdFormattingOptions);
	}

	return FText::GetEmpty();
}

void SAvaRundownPageDetails::OnPageIdCommitted(const FText& InNewText, ETextCommit::Type InCommitType)
{
	switch (InCommitType)
	{
		case ETextCommit::OnEnter:
		case ETextCommit::OnUserMovedFocus:
			break;

		case ETextCommit::Default:
		case ETextCommit::OnCleared:
		default:
			return;
	}

	if (!InNewText.IsNumeric())
	{
		return;
	}

	const FRundownContext RundownContext = GetRundownContext();

	if (!RundownContext.IsValid())
	{
		return;
	}

	const FAvaRundownPage& SelectedPage = GetSelectedPage(RundownContext);

	if (!SelectedPage.IsValidPage()) // Not FAvaRundownPage::NullPage
	{
		return;
	}

	const int32 NewId = FCString::Atoi(*InNewText.ToString());

	if (NewId == SelectedPage.GetPageId())
	{
		return;
	}

	if (RundownContext.GetRundown()->RenumberPageId(SelectedPage.GetPageId(), NewId))
	{
		const TSharedPtr<SAvaRundownInstancedPageList> PageList = RundownContext.RundownEditor->GetActiveListWidget();

		if (PageList.IsValid())
		{
			PageList->SelectPage(NewId);
		}
	}
}

FText SAvaRundownPageDetails::GetPageDescription() const
{
	const FRundownContext RundownContext = GetRundownContext();
	const FAvaRundownPage& SelectedPage = GetSelectedPage(RundownContext);

	if (SelectedPage.IsValidPage())
	{
		return SelectedPage.GetPageDescription();
	}

	return FText::GetEmpty();
}

void SAvaRundownPageDetails::OnPageNameChanged(const FText& InNewText)
{
	const FRundownContext RundownContext = GetRundownContext();
	FAvaRundownPage& SelectedPage = GetMutableSelectedPage(RundownContext);

	if (SelectedPage.IsValidPage()) // Not FAvaRundownPage::NullPage
	{
		SelectedPage.SetPageFriendlyName(InNewText);
		
		if (UAvaRundown* Rundown = RundownContext.GetRundown())
		{
			Rundown->GetOnPagesChanged().Broadcast(Rundown, SelectedPage, EAvaRundownPageChanges::FriendlyName);
		}
	}
}

FReply SAvaRundownPageDetails::DuplicateSelectedPage()
{
	const FRundownContext RundownContext = GetRundownContext();
	if (!RundownContext.IsValid())
	{
		return FReply::Unhandled();
	}
	
	const FAvaRundownPage& SelectedPage = GetSelectedPage(RundownContext);

	if (!SelectedPage.IsValidPage()) // Not FAvaRundownPage::NullPage
	{
		return FReply::Unhandled();
	}

	const TSharedPtr<SAvaRundownInstancedPageList> PageList = RundownContext.RundownEditor->GetActiveListWidget();

	if (!PageList.IsValid())
	{
		return FReply::Unhandled();
	}

	const TArray<int32> SelectedPages = PageList->GetSelectedPageIds();
	PageList->SelectPage(SelectedPage.GetPageId());
	PageList->DuplicateSelectedPages();
	PageList->SelectPages(SelectedPages);

	return FReply::Handled();
}

void SAvaRundownPageDetails::OnPagesChanged(const UAvaRundown* InRundown, const FAvaRundownPage& InPage, const EAvaRundownPageChanges InChanges)
{
	// Refreshing the page while the mouse is captured will result in losing the capture
	// and ending any drag event that is actively changing the value.
	if (!FSlateApplication::Get().GetMouseCaptureWindow() && InPage.GetPageId() == ActivePageId)
	{
		// Queue a refresh on next tick to avoid issues with cascading events.
		QueueRefreshSelectedPage();
	}
}

void SAvaRundownPageDetails::OnPageListChanged(const FAvaRundownPageListChangeParams& InParams)
{
	// If the current page is removed, fire off a selection changed immediately.
	if (InParams.AffectedPages.Contains(ActivePageId) && EnumHasAnyFlags(InParams.ChangeType, EAvaRundownPageListChange::RemovedPages))
	{
		OnPageEvent({}, UE::AvaRundown::EPageEvent::SelectionChanged);
	}
}

#undef LOCTEXT_NAMESPACE
