// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaRundownPageDetailView.h"
#include "Containers/Array.h"
#include "Rundown/AvaRundownEditorDefines.h"
#include "Rundown/AvaRundownPage.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/SCompoundWidget.h"

class FAvaRundownEditor;
class FReply;
class FText;
class SAvaRundownPageRemoteControlProps;
class SAvaRundownRCControllerPanel;
class UAvaRundown;
enum class EAvaRundownPageChanges : uint8;
struct FAvaRundownPage;
struct FAvaRundownPageListChangeParams;
struct FSlateBrush;
struct FSoftObjectPath;

class SAvaRundownPageDetails : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaRundownPageDetails) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedPtr<FAvaRundownEditor>& InRundownEditor);

	virtual ~SAvaRundownPageDetails() override;

	void OnPageEvent(const TArray<int32>& InSelectedPageIds, UE::AvaRundown::EPageEvent InPageEvent);
	void OnManagedInstanceCacheEntryInvalidated(const FSoftObjectPath& InAssetPath);

	void RefreshSelectedPage()
	{
		QueueRefreshSelectedPage();
	}

protected:
	void OnFinishedChangingPageDetailProperties(const FPropertyChangedEvent& InChangedProperty);

	FReply ToggleExposedPropertiesVisibility();

	const FSlateBrush* GetExposedPropertiesVisibilityBrush() const;

	/** Encapsulate how the rundown is accessed for this widget to reduce duplicated code. */
	struct FRundownContext
	{
		bool IsValid() const;
		UAvaRundown* GetRundown() const;

		TSharedPtr<FAvaRundownEditor> RundownEditor;
	};
	FRundownContext GetRundownContext() const;
	
	const FAvaRundownPage& GetSelectedPage(const FRundownContext& InRundownContext) const;
	FAvaRundownPage& GetMutableSelectedPage(const FRundownContext& InRundownContext) const;

	void QueueRefreshSelectedPage();
	void QueueUpdateAndRefreshSelectedPage();
	void RefreshPageDetailView();

	bool HasSelectedPage() const;

	FText GetPageId() const;

	/** Only update page id on commit. */
	void OnPageIdCommitted(const FText& InNewText, ETextCommit::Type InCommitType);

	FText GetPageDescription() const;

	/** Update page name live. */
	void OnPageNameChanged(const FText& InNewText);

	FReply DuplicateSelectedPage();

	void OnPagesChanged(const UAvaRundown* InRundown, const FAvaRundownPage& InPage, const EAvaRundownPageChanges InChanges);
	void OnPageListChanged(const FAvaRundownPageListChangeParams& InParams);
	
private:
	TWeakPtr<FAvaRundownEditor> RundownEditorWeak;

	TSharedPtr<SAvaRundownPageRemoteControlProps> RemoteControlProps;
	TSharedPtr<SAvaRundownRCControllerPanel> RCControllerPanel;

	/** Detail view of the page detail model. */
	TSharedPtr<IDetailsView> PageDetailView;

	/** Page view data of the current page (being edited). */
	TStrongObjectPtr<UAvaRundownPageDetailView> CurrentPageDetailViewData;

	/** This is separate from ActivePageId because of order of update events. ActivePageId update first. */ 
	int32 CurrentPageDetailViewPageId = FAvaRundownPage::InvalidPageId;

	bool bRefreshSelectedPageQueued = false;
	bool bUpdateAndRefreshSelectedPageQueued = false;

	int32 ActivePageId = FAvaRundownPage::InvalidPageId;
};
