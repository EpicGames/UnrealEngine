// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"

/** Type of the Page list */
enum class EAvaRundownSearchListType : uint8
{
	None      = 0x00, // None
	Template  = 0x01, // Template pages type
	Instanced = 0x10 // Instanced pages type
};
ENUM_CLASS_FLAGS(EAvaRundownSearchListType)
