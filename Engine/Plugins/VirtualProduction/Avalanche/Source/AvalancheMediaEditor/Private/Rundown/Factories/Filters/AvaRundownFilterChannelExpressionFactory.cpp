// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownFilterChannelExpressionFactory.h"

#include "Misc/TextFilterUtils.h"
#include "Rundown/AvaRundownPage.h"
#include "Rundown/Factories/Filters/AvaRundownFactoriesUtils.h"

namespace UE::AvaMediaEditor::Expression::Channel::Private
{
	static const FName KeyName = FName(TEXT("CHANNEL"));
}

FName FAvaRundownFilterChannelExpressionFactory::GetFilterIdentifier() const
{
	using namespace UE::AvaMediaEditor::Expression::Channel::Private;
	return KeyName;
}

bool FAvaRundownFilterChannelExpressionFactory::FilterExpression(const FAvaRundownPage& InItem, const FAvaRundownTextFilterArgs& InArgs) const
{
	return InArgs.ValueToCheck.CompareName(InItem.GetChannelName(), InArgs.ComparisonMode);
}

bool FAvaRundownFilterChannelExpressionFactory::SupportsComparisonOperation(ETextFilterComparisonOperation InComparisonOperation, EAvaRundownSearchListType InRundownSearchListType) const
{
	const bool bIsEqualityOperation = InComparisonOperation == ETextFilterComparisonOperation::Equal 
		|| InComparisonOperation == ETextFilterComparisonOperation::NotEqual;

	return bIsEqualityOperation 
	   && InRundownSearchListType == EAvaRundownSearchListType::Instanced;
}
