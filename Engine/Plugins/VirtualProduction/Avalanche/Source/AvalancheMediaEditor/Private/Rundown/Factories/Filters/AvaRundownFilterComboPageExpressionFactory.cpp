// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownFilterComboPageExpressionFactory.h"

#include "Misc/TextFilterUtils.h"
#include "Rundown/AvaRundownPage.h"

namespace UE::AvaMediaEditor::Expression::ComboPage::Private
{
	static const FName KeyName = FName(TEXT("ISCOMBOPAGE"));
}

FName FAvaRundownFilterComboPageExpressionFactory::GetFilterIdentifier() const
{
	using namespace UE::AvaMediaEditor::Expression::ComboPage::Private;
	return KeyName;
}

bool FAvaRundownFilterComboPageExpressionFactory::FilterExpression(const FAvaRundownPage& InItem, const FAvaRundownTextFilterArgs& InArgs) const
{
	if (InArgs.ItemRundown)
	{
		const FAvaRundownPage& TemplatePage = InItem.ResolveTemplate(InArgs.ItemRundown);
		const bool bIsPageComboTemplate = TemplatePage.IsComboTemplate();
		const bool bDisplayComboTemplate = InArgs.ValueToCheck.CompareName(NAME_TRUE, InArgs.ComparisonMode);
		return bDisplayComboTemplate ? bIsPageComboTemplate : !bIsPageComboTemplate;
	}
	// if there is something wrong just display everything
	return true;
}

bool FAvaRundownFilterComboPageExpressionFactory::SupportsComparisonOperation(ETextFilterComparisonOperation InComparisonOperation, EAvaRundownSearchListType InPlaylistSearchListType) const
{
	return InComparisonOperation == ETextFilterComparisonOperation::Equal || InComparisonOperation == ETextFilterComparisonOperation::NotEqual;
}