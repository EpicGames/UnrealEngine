// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownFilterIdExpressionFactory.h"

#include "Misc/TextFilterUtils.h"
#include "Rundown/AvaRundownPage.h"

namespace UE::AvaMediaEditor::Expression::Id::Private
{
	static const FName KeyName = FName(TEXT("ID"));
}

FName FAvaRundownFilterIdExpressionFactory::GetFilterIdentifier() const
{
	using namespace UE::AvaMediaEditor::Expression::Id::Private;
	return KeyName;
}

bool FAvaRundownFilterIdExpressionFactory::FilterExpression(const FAvaRundownPage& InItem, const FAvaRundownTextFilterArgs& InArgs) const
{
	const FTextFilterString FilterString = FString::FromInt(InItem.GetPageId());
	if (FilterString.CanCompareNumeric(InArgs.ValueToCheck))
	{
		return FilterString.CompareNumeric(InArgs.ValueToCheck, InArgs.ComparisonOperation);
	}
	return false;
}

bool FAvaRundownFilterIdExpressionFactory::SupportsComparisonOperation(ETextFilterComparisonOperation InComparisonOperation, EAvaRundownSearchListType InRundownSearchListType) const
{
	return true;
}
