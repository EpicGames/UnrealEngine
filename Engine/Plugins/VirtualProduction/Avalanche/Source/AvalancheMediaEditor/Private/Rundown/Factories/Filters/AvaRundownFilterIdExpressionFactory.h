// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/Factories/Filters/IAvaRundownFilterExpressionFactory.h"

class FAvaRundownFilterIdExpressionFactory : public IAvaRundownFilterExpressionFactory
{
public:
	//~ Begin IAvaFilterExpressionFactory interface
	virtual FName GetFilterIdentifier() const override;
	virtual bool FilterExpression(const FAvaRundownPage& InItem, const FAvaRundownTextFilterArgs& InArgs) const override;
	virtual bool SupportsComparisonOperation(ETextFilterComparisonOperation InComparisonOperation, EAvaRundownSearchListType InRundownSearchListType) const override;
	//~ End IAvaFilterExpressionFactory interface
};
