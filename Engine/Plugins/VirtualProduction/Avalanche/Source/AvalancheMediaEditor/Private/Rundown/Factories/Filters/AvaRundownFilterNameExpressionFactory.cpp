// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownFilterNameExpressionFactory.h"

#include "Misc/TextFilterUtils.h"
#include "Rundown/AvaRundownPage.h"

namespace UE::AvaMediaEditor::Expression::Name::Private
{
	static const FName KeyName = FName(TEXT("NAME"));
}

FName FAvaRundownFilterNameExpressionFactory::GetFilterIdentifier() const
{
	using namespace UE::AvaMediaEditor::Expression::Name::Private;
	return KeyName;
}

bool FAvaRundownFilterNameExpressionFactory::FilterExpression(const FAvaRundownPage& InItem, const FAvaRundownTextFilterArgs& InArgs) const
{
	const FName Name = FName(InItem.GetPageDescription().ToString());
	return InArgs.ValueToCheck.CompareName(Name, InArgs.ComparisonMode);
}

bool FAvaRundownFilterNameExpressionFactory::SupportsComparisonOperation(ETextFilterComparisonOperation InComparisonOperation, EAvaRundownSearchListType InRundownSearchListType) const
{
	return InComparisonOperation == ETextFilterComparisonOperation::Equal || InComparisonOperation == ETextFilterComparisonOperation::NotEqual;
}
