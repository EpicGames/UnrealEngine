// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownFilterPathExpressionFactory.h"

#include "Misc/TextFilterUtils.h"
#include "Rundown/AvaRundownPage.h"

namespace UE::AvaMediaEditor::Expression::Path::Private
{
	static const FName KeyName = FName(TEXT("ASSET"));
}

FName FAvaRundownFilterPathExpressionFactory::GetFilterIdentifier() const
{
	using namespace UE::AvaMediaEditor::Expression::Path::Private;
	return KeyName;
}

bool FAvaRundownFilterPathExpressionFactory::FilterExpression(const FAvaRundownPage& InItem, const FAvaRundownTextFilterArgs& InArgs) const
{
	if (!InArgs.ItemRundown)
	{
		return false;
	}
	const FName AssetPath = FName(InItem.GetAssetPath(InArgs.ItemRundown).GetAssetPathString());
	return InArgs.ValueToCheck.CompareName(AssetPath, InArgs.ComparisonMode);
}

bool FAvaRundownFilterPathExpressionFactory::SupportsComparisonOperation(ETextFilterComparisonOperation InComparisonOperation, EAvaRundownSearchListType InRundownSearchListType) const
{
	return InComparisonOperation == ETextFilterComparisonOperation::Equal || InComparisonOperation == ETextFilterComparisonOperation::NotEqual;
}
