// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownFilterStatusExpressionFactory.h"

#include "Misc/TextFilterUtils.h"
#include "Rundown/AvaRundownPage.h"

namespace UE::AvaMediaEditor::Expression::Status::Private
{
	static const FName KeyName = FName(TEXT("STATUS"));
}

FName FAvaRundownFilterStatusExpressionFactory::GetFilterIdentifier() const
{
	using namespace UE::AvaMediaEditor::Expression::Status::Private;
	return KeyName;
}

bool FAvaRundownFilterStatusExpressionFactory::FilterExpression(const FAvaRundownPage& InItem, const FAvaRundownTextFilterArgs& InArgs) const
{
	if (!InArgs.ItemRundown)
	{
		return false;
	}
	const TArray<FAvaRundownChannelPageStatus> Statuses = InItem.GetPageContextualStatuses(InArgs.ItemRundown);
	for (const FAvaRundownChannelPageStatus& Status : Statuses)
	{
		const FName StatusName = StaticEnum<EAvaRundownPageStatus>()->GetNameByValue(static_cast<int32>(Status.Status));
		if (InArgs.ValueToCheck.CompareName(StatusName, InArgs.ComparisonMode))
		{
			return true;
		}
	}
	return false;
}

bool FAvaRundownFilterStatusExpressionFactory::SupportsComparisonOperation(ETextFilterComparisonOperation InComparisonOperation, EAvaRundownSearchListType InRundownSearchListType) const
{
	return InComparisonOperation == ETextFilterComparisonOperation::Equal || InComparisonOperation == ETextFilterComparisonOperation::NotEqual;
}
