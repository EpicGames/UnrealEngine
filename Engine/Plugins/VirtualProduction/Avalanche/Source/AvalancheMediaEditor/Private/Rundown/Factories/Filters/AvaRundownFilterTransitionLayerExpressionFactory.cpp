// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownFilterTransitionLayerExpressionFactory.h"

#include "Misc/TextFilterUtils.h"
#include "Rundown/AvaRundownPage.h"

namespace UE::AvaMediaEditor::Expression::Layer::Private
{
	static const FName KeyName = FName(TEXT("TRANSITIONLAYER"));
}

FName FAvaRundownFilterTransitionLayerExpressionFactory::GetFilterIdentifier() const
{
	using namespace UE::AvaMediaEditor::Expression::Layer::Private;
	return KeyName;
}

bool FAvaRundownFilterTransitionLayerExpressionFactory::FilterExpression(const FAvaRundownPage& InItem, const FAvaRundownTextFilterArgs& InArgs) const
{
	if (!InArgs.ItemRundown)
	{
		return false;
	}
	return InArgs.ValueToCheck.CompareFString(InItem.GetTransitionLayer(InArgs.ItemRundown).ToString().ToUpper(), InArgs.ComparisonMode);
}

bool FAvaRundownFilterTransitionLayerExpressionFactory::SupportsComparisonOperation(ETextFilterComparisonOperation InComparisonOperation, EAvaRundownSearchListType InRundownSearchListType) const
{
	return InComparisonOperation == ETextFilterComparisonOperation::Equal || InComparisonOperation == ETextFilterComparisonOperation::NotEqual;
}
