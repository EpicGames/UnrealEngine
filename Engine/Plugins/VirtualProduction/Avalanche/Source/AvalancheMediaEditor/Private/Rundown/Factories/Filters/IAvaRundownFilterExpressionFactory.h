// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/TextFilterUtils.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/NameTypes.h"

class UAvaRundown;
enum class EAvaRundownSearchListType : uint8;
struct FAvaRundownPage;

/**
 * Holds the arguments required for the expression evaluation.
 */
struct FAvaRundownTextFilterArgs
{
	/** Item Rundown */
	const UAvaRundown* ItemRundown;

	/** Second Value */
	FTextFilterString ValueToCheck;

	/** Comparison Operation requested (Equal/Greater/etc...) still needed for certain Filter */
	ETextFilterComparisonOperation ComparisonOperation;

	/** Comparison Mode requested see ETextFilterTextComparisonMode for more information */
	ETextFilterTextComparisonMode ComparisonMode;
};

class IAvaRundownFilterExpressionFactory : public TSharedFromThis<IAvaRundownFilterExpressionFactory>
{
public:
	virtual ~IAvaRundownFilterExpressionFactory() {}

	/** Get the filter identifier for this factory */
	virtual FName GetFilterIdentifier() const = 0;

	/**
 	 * Create and return an Instance of the AvaRundownFilterExpressionFactory Requested
 	 * @tparam InRundownFilterExpressionFactoryType The type of the factory to instantiate
 	 * @param InArgs Additional Args for constructor of the AvaRundownFilterExpressionFactory class if needed
 	 * @return The new rundown filter factory
 	 */
	template <
		typename InRundownFilterExpressionFactoryType,
		typename... InArgsType
		UE_REQUIRES(TIsDerivedFrom<InRundownFilterExpressionFactoryType, IAvaRundownFilterExpressionFactory>::Value)
	>
	static TSharedRef<InRundownFilterExpressionFactoryType> MakeInstance(InArgsType&&... InArgs)
	{
		return MakeShared<InRundownFilterExpressionFactoryType>(Forward<InArgsType>(InArgs)...);
	}

	/**
	 * Evaluate the expression with the Value given
	 * @param InItem Current Item being checked
	 * @param InArgs The argument containing the data to evaluate the expression see FAvaTextFilterArgs for more information
	 * @return True if the expression evaluate to True, False otherwise
	 */
	virtual bool FilterExpression(const FAvaRundownPage& InItem, const FAvaRundownTextFilterArgs& InArgs) const = 0;

	/**
	 * Whether the factory support a given Comparison Operation
	 * @param InComparisonOperation Comparison Operation to check
	 * @param InRundownSearchListType Type of the Search List either Template or Instanced
	 * @return True if Factory support passed Comparison Operation, False otherwise
	 */
	virtual bool SupportsComparisonOperation(ETextFilterComparisonOperation InComparisonOperation, EAvaRundownSearchListType InRundownSearchListType) const = 0;
};
