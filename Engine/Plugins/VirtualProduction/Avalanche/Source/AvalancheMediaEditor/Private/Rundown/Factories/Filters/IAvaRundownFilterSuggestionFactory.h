// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Rundown/AvaRundownDefines.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTypeTraits.h"

class UAvaRundown;
struct FAvaRundownPage;
struct FAssetSearchBoxSuggestion;
enum class EAvaRundownSearchListType : uint8;

/**
 * Holds the arguments required for the suggestion factory to get the suggestion.
 */
struct FAvaRundownFilterSuggestionPayload
{
	/** Holds all the suggestion that will be shown */
	TArray<FAssetSearchBoxSuggestion>& PossibleSuggestions;

	/** The current string value written in the searchbox */
	const FString FilterValue;

	/** Current item to check */
	int32 ItemPageId = UE::AvaMedia::Rundown::InvalidPageId;

	/** Current item rundown */
	const UAvaRundown* Rundown = nullptr;

	/** Used to speed up the check on whether a suggestion is already added */
	TSet<FString>& FilterCache;
};

class IAvaRundownFilterSuggestionFactory : public TSharedFromThis<IAvaRundownFilterSuggestionFactory>
{
public:
	virtual ~IAvaRundownFilterSuggestionFactory() {}

	/**
	  * Create and return an Instance of the AvaRundownFilterSuggestionFactory Requested
	  * @tparam InRundownSuggestionFactoryType The type of the factory to instantiate
	  * @param InArgs Additional Args for constructor of the AvaRundownFilterSuggestionFactory class if needed
	  * @return The new suggestion factory
	  */
	template <
		typename InRundownSuggestionFactoryType,
		typename... InArgsType
		UE_REQUIRES(TIsDerivedFrom<InRundownSuggestionFactoryType, IAvaRundownFilterSuggestionFactory>::Value)
	>
	static TSharedRef<InRundownSuggestionFactoryType> MakeInstance(InArgsType&&... InArgs)
	{
		return MakeShared<InRundownSuggestionFactoryType>(Forward<InArgsType>(InArgs)...);
	}

	/** Get the suggestion identifier for this factory */
	virtual FName GetSuggestionIdentifier() const = 0;

	/** True if the suggestion doesn't need to use the item to add its suggestions */
	virtual bool IsSimpleSuggestion() const = 0;
	/**
	 * Add suggestions entry to be shown
	 * @param InPayload Arguments to be passed to the factory to get the suggestion to show
	 */
	virtual void AddSuggestion(const TSharedRef<FAvaRundownFilterSuggestionPayload>& InPayload) = 0;

	/**
	 * Whether the factory support a given type of suggestion
	 * @param InSuggestionType Suggestion type to check
	 * @return True if Factory support passed Suggestion Type, False otherwise
	 */
	virtual bool SupportSuggestionType(EAvaRundownSearchListType InSuggestionType) const = 0;
};
