// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownFilterChannelSuggestionFactory.h"

#include "Rundown/AvaRundown.h"
#include "Rundown/Factories/Filters/AvaRundownFactoriesUtils.h"
#include "SAssetSearchBox.h"

#define LOCTEXT_NAMESPACE "AvaRundownFilterChannelSuggestionFactory"

namespace UE::AvaMediaEditor::Suggestion::Channel::Private
{
	static const FName KeyName = FName(TEXT("CHANNEL"));
	static const FText CategoryLabel = LOCTEXT("ChannelCategoryLabel", "Ava-Rundown-Channel");
}

FName FAvaRundownFilterChannelSuggestionFactory::GetSuggestionIdentifier() const
{
	using namespace UE::AvaMediaEditor::Suggestion::Channel::Private;
	return KeyName;
}

void FAvaRundownFilterChannelSuggestionFactory::AddSuggestion(const TSharedRef<FAvaRundownFilterSuggestionPayload>& InPayload)
{
	using namespace UE::AvaMediaEditor::Suggestion::Channel::Private;

	const FAvaRundownPage& PageItem = UAvaRundown::GetPageSafe(InPayload->Rundown, InPayload->ItemPageId);
	if (PageItem.IsValidPage())
	{
		const FString ChannelName = PageItem.GetChannelName().ToString();

		FString ChannelNameSuggestion = FString::Printf(TEXT("Channel=%s"), *ChannelName);
		const bool bIsFilterValueValid = InPayload->FilterValue.IsEmpty() || ChannelNameSuggestion.Contains(InPayload->FilterValue);

		if (bIsFilterValueValid && !InPayload->FilterCache.Contains(ChannelNameSuggestion))
		{
			InPayload->FilterCache.Add(ChannelNameSuggestion);
			InPayload->PossibleSuggestions.Add(FAssetSearchBoxSuggestion{MoveTemp(ChannelNameSuggestion), FText::FromString(ChannelName), CategoryLabel});
		}
	}
}

bool FAvaRundownFilterChannelSuggestionFactory::SupportSuggestionType(EAvaRundownSearchListType InSuggestionType) const
{
	return InSuggestionType == EAvaRundownSearchListType::Instanced;
}

#undef LOCTEXT_NAMESPACE
