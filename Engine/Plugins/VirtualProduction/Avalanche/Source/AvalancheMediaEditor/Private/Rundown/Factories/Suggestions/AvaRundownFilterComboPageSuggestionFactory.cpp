// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownFilterComboPageSuggestionFactory.h"

#include "Rundown/Factories/Filters/AvaRundownFactoriesUtils.h"
#include "SAssetSearchBox.h"

#define LOCTEXT_NAMESPACE "AvaRundownFilterComboPageSuggestionFactory"

namespace UE::AvaMediaEditor::Suggestion::ComboPage::Private
{
	static const FName KeyName = FName(TEXT("ISCOMBOPAGE"));
	static const FText CategoryLabel = LOCTEXT("IsComboPageCategoryLabel", "Ava-Rundown-IsComboPage");
	static const FString TrueSuggestion = TEXT("IsComboPage=True");
	static const FText TrueDisplayName = LOCTEXT("ComboPageSuggestion_True","True");
	static const FString FalseSuggestion = TEXT("IsComboPage=False");
	static const FText FalseDisplayName = LOCTEXT("ComboPageSuggestion_False","False");
}

FName FAvaRundownFilterComboPageSuggestionFactory::GetSuggestionIdentifier() const
{
	using namespace UE::AvaMediaEditor::Suggestion::ComboPage::Private;
	return KeyName;
}

void FAvaRundownFilterComboPageSuggestionFactory::AddSuggestion(const TSharedRef<FAvaRundownFilterSuggestionPayload>& InPayload)
{
	using namespace UE::AvaMediaEditor::Suggestion::ComboPage::Private;

	const bool bIsFilterValueValidTrue = InPayload->FilterValue.IsEmpty() || TrueSuggestion.Contains(InPayload->FilterValue);
	if (bIsFilterValueValidTrue && !InPayload->FilterCache.Contains(TrueSuggestion))
	{
		InPayload->FilterCache.Add(TrueSuggestion);
		InPayload->PossibleSuggestions.Add(FAssetSearchBoxSuggestion{TrueSuggestion, TrueDisplayName, CategoryLabel});
	}
	const bool bIsFilterValueValidFalse = InPayload->FilterValue.IsEmpty() || FalseSuggestion.Contains(InPayload->FilterValue);
	if (bIsFilterValueValidFalse && !InPayload->FilterCache.Contains(FalseSuggestion))
	{
		InPayload->FilterCache.Add(FalseSuggestion);
		InPayload->PossibleSuggestions.Add(FAssetSearchBoxSuggestion{FalseSuggestion, FalseDisplayName, CategoryLabel});
	}
}

bool FAvaRundownFilterComboPageSuggestionFactory::SupportSuggestionType(EAvaRundownSearchListType InSuggestionType) const
{
	return InSuggestionType != EAvaRundownSearchListType::None;
}

#undef LOCTEXT_NAMESPACE
