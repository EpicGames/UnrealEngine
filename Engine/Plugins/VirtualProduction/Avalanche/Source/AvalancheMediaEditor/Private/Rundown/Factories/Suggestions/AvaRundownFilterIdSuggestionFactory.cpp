// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownFilterIdSuggestionFactory.h"

#include "Rundown/Factories/Filters/AvaRundownFactoriesUtils.h"
#include "SAssetSearchBox.h"

#define LOCTEXT_NAMESPACE "AvaRundownFilterIdSuggestionFactory"

namespace UE::AvaMediaEditor::Suggestion::Id::Private
{
	static const FName KeyName = FName(TEXT("ID"));
	static const FText CategoryLabel = LOCTEXT("IdCategoryLabel", "Ava-Rundown-Id");
	static const FString Suggestion = TEXT("Id");
	static const FText DisplayName = LOCTEXT("IdSuggestion","Id");
}

FName FAvaRundownFilterIdSuggestionFactory::GetSuggestionIdentifier() const
{
	using namespace UE::AvaMediaEditor::Suggestion::Id::Private;
	return KeyName;
}

void FAvaRundownFilterIdSuggestionFactory::AddSuggestion(const TSharedRef<FAvaRundownFilterSuggestionPayload>& InPayload)
{
	using namespace UE::AvaMediaEditor::Suggestion::Id::Private;

	const bool bIsFilterValueValid = InPayload->FilterValue.IsEmpty() || Suggestion.Contains(InPayload->FilterValue);
	if (bIsFilterValueValid && !InPayload->FilterCache.Contains(Suggestion))
	{
		InPayload->FilterCache.Add(Suggestion);
		InPayload->PossibleSuggestions.Add(FAssetSearchBoxSuggestion{Suggestion, DisplayName, CategoryLabel});
	}
}

bool FAvaRundownFilterIdSuggestionFactory::SupportSuggestionType(EAvaRundownSearchListType InSuggestionType) const
{
	return InSuggestionType != EAvaRundownSearchListType::None;
}

#undef LOCTEXT_NAMESPACE
