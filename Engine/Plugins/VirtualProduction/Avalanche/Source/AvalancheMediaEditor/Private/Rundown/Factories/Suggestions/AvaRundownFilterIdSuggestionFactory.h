// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/Factories/Filters/IAvaRundownFilterSuggestionFactory.h"

class FAvaRundownFilterIdSuggestionFactory : public IAvaRundownFilterSuggestionFactory
{
public:
	//~ Begin IAvaFilterSuggestionFactory interface
	virtual FName GetSuggestionIdentifier() const override;
	virtual bool IsSimpleSuggestion() const override { return true; }
	virtual void AddSuggestion(const TSharedRef<FAvaRundownFilterSuggestionPayload>& InPayload) override;
	virtual bool SupportSuggestionType(EAvaRundownSearchListType InSuggestionType) const override;
	//~ End IAvaFilterSuggestionFactory interface
};
