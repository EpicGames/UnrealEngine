// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownFilterNameSuggestionFactory.h"

#include "Rundown/AvaRundown.h"
#include "Rundown/Factories/Filters/AvaRundownFactoriesUtils.h"
#include "SAssetSearchBox.h"

#define LOCTEXT_NAMESPACE "AvaRundownFilterNameSuggestionFactory"

namespace UE::AvaMediaEditor::Suggestion::Name::Private
{
	static const FName KeyName = FName(TEXT("NAME"));
	static const FText CategoryLabel = LOCTEXT("NameCategoryLabel", "Ava-Rundown-Name");
}

FName FAvaRundownFilterNameSuggestionFactory::GetSuggestionIdentifier() const
{
	using namespace UE::AvaMediaEditor::Suggestion::Name::Private;
	return KeyName;
}

void FAvaRundownFilterNameSuggestionFactory::AddSuggestion(const TSharedRef<FAvaRundownFilterSuggestionPayload>& InPayload)
{
	using namespace UE::AvaMediaEditor::Suggestion::Name::Private;

	const FAvaRundownPage& PageItem = UAvaRundown::GetPageSafe(InPayload->Rundown, InPayload->ItemPageId);
	if (PageItem.IsValidPage())
	{
		const FText PageText = PageItem.GetPageDescription();

		FString PageName = TEXT("\"");
		PageName +=	PageItem.GetPageDescription().ToString();
		PageName.Append(TEXT("\""));

		FString NameSuggestion = FString::Printf(TEXT("Name=%s"), *PageName);
		const bool bIsFilterValueValid = InPayload->FilterValue.IsEmpty() || NameSuggestion.Contains(InPayload->FilterValue);

		if (bIsFilterValueValid && !InPayload->FilterCache.Contains(NameSuggestion))
		{
			InPayload->FilterCache.Add(NameSuggestion);
			InPayload->PossibleSuggestions.Add(FAssetSearchBoxSuggestion{MoveTemp(NameSuggestion), PageText, CategoryLabel});
		}
	}
}

bool FAvaRundownFilterNameSuggestionFactory::SupportSuggestionType(EAvaRundownSearchListType InSuggestionType) const
{
	return InSuggestionType != EAvaRundownSearchListType::None;
}

#undef LOCTEXT_NAMESPACE
