// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownFilterPathSuggestionFactory.h"

#include "Rundown/AvaRundown.h"
#include "Rundown/Factories/Filters/AvaRundownFactoriesUtils.h"
#include "SAssetSearchBox.h"

#define LOCTEXT_NAMESPACE "AvaRundownFilterPathSuggestionFactory"

namespace UE::AvaMediaEditor::Suggestion::Path::Private
{
	static const FName KeyName = FName(TEXT("ASSET"));
	static const FText CategoryLabel = LOCTEXT("PathCategoryLabel", "Ava-Rundown-Asset");
}

FName FAvaRundownFilterPathSuggestionFactory::GetSuggestionIdentifier() const
{
	using namespace UE::AvaMediaEditor::Suggestion::Path::Private;
	return KeyName;
}

void FAvaRundownFilterPathSuggestionFactory::AddSuggestion(const TSharedRef<FAvaRundownFilterSuggestionPayload>& InPayload)
{
	using namespace UE::AvaMediaEditor::Suggestion::Path::Private;

	const FAvaRundownPage& PageItem = UAvaRundown::GetPageSafe(InPayload->Rundown, InPayload->ItemPageId);
	if (PageItem.IsValidPage())
	{
		const FString AssetName = PageItem.GetAssetPath(InPayload->Rundown).GetAssetName();

		FString AssetNameSuggestion = FString::Printf(TEXT("Asset=%s"), *AssetName);
		const bool bIsFilterValueValid = InPayload->FilterValue.IsEmpty() || AssetNameSuggestion.Contains(InPayload->FilterValue);

		if (bIsFilterValueValid && !InPayload->FilterCache.Contains(AssetNameSuggestion))
		{
			InPayload->FilterCache.Add(AssetNameSuggestion);
			InPayload->PossibleSuggestions.Add(FAssetSearchBoxSuggestion{MoveTemp(AssetNameSuggestion), FText::FromString(AssetName), CategoryLabel});
		}
	}
}

bool FAvaRundownFilterPathSuggestionFactory::SupportSuggestionType(EAvaRundownSearchListType InSuggestionType) const
{
	return InSuggestionType != EAvaRundownSearchListType::None;
}

#undef LOCTEXT_NAMESPACE
