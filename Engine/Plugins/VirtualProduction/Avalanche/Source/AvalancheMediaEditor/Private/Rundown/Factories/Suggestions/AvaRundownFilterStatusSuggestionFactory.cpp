// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownFilterStatusSuggestionFactory.h"

#include "Rundown/AvaRundown.h"
#include "Rundown/Factories/Filters/AvaRundownFactoriesUtils.h"
#include "SAssetSearchBox.h"

#define LOCTEXT_NAMESPACE "AvaRundownFilterStatusSuggestionFactory"

namespace UE::AvaMediaEditor::Suggestion::Status::Private
{
	static const FName KeyName = FName(TEXT("STATUS"));
	static const FText CategoryLabel = LOCTEXT("StatusCategoryLabel", "Ava-Rundown-Status");
}

FName FAvaRundownFilterStatusSuggestionFactory::GetSuggestionIdentifier() const
{
	using namespace UE::AvaMediaEditor::Suggestion::Status::Private;
	return KeyName;
}

void FAvaRundownFilterStatusSuggestionFactory::AddSuggestion(const TSharedRef<FAvaRundownFilterSuggestionPayload>& InPayload)
{
	using namespace UE::AvaMediaEditor::Suggestion::Status::Private;

	const FAvaRundownPage& PageItem = UAvaRundown::GetPageSafe(InPayload->Rundown, InPayload->ItemPageId);
	if (PageItem.IsValidPage())
	{
		TArray<FAvaRundownChannelPageStatus> StatusPages = PageItem.GetPageContextualStatuses(InPayload->Rundown);
		for (FAvaRundownChannelPageStatus Status : StatusPages)
		{
			const FString StatusName = StaticEnum<EAvaRundownPageStatus>()->GetNameStringByValue(static_cast<int32>(Status.Status));

			FString StatusNameSuggestion = FString::Printf(TEXT("Status=%s"), *StatusName);
			const bool bIsFilterValueValid = InPayload->FilterValue.IsEmpty() || StatusNameSuggestion.Contains(InPayload->FilterValue);

			if (bIsFilterValueValid && !InPayload->FilterCache.Contains(StatusNameSuggestion))
			{
				InPayload->FilterCache.Add(StatusNameSuggestion);
				InPayload->PossibleSuggestions.Add(FAssetSearchBoxSuggestion{MoveTemp(StatusNameSuggestion), FText::FromString(StatusName), CategoryLabel});
			}
		}
	}
}

bool FAvaRundownFilterStatusSuggestionFactory::SupportSuggestionType(EAvaRundownSearchListType InSuggestionType) const
{
	return InSuggestionType != EAvaRundownSearchListType::None;
}

#undef LOCTEXT_NAMESPACE
