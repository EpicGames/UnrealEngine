// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownFilterTransitionLayerSuggestionFactory.h"

#include "Rundown/AvaRundown.h"
#include "Rundown/Factories/Filters/AvaRundownFactoriesUtils.h"
#include "SAssetSearchBox.h"

#define LOCTEXT_NAMESPACE "AvaRundownFilterLayerSuggestionFactory"

namespace UE::AvaMediaEditor::Suggestion::Layer::Private
{
	static const FName KeyName = FName(TEXT("TRANSITIONLAYER"));
	static const FText CategoryLabel = LOCTEXT("TransitionLayerCategoryLabel", "Ava-Rundown-Transition-Layer");
}

FName FAvaRundownFilterTransitionLayerSuggestionFactory::GetSuggestionIdentifier() const
{
	using namespace UE::AvaMediaEditor::Suggestion::Layer::Private;
	return KeyName;
}

void FAvaRundownFilterTransitionLayerSuggestionFactory::AddSuggestion(const TSharedRef<FAvaRundownFilterSuggestionPayload>& InPayload)
{
	using namespace UE::AvaMediaEditor::Suggestion::Layer::Private;

	const FAvaRundownPage& PageItem = UAvaRundown::GetPageSafe(InPayload->Rundown, InPayload->ItemPageId);
	if (PageItem.IsValidPage())
	{
		const FString PageLayer = PageItem.GetTransitionLayer(InPayload->Rundown).ToString();

		FString TransitionLayerSuggestion = FString::Printf(TEXT("TransitionLayer=%s"), *PageLayer);
		const bool bIsFilterValueValid = InPayload->FilterValue.IsEmpty() || TransitionLayerSuggestion.Contains(InPayload->FilterValue);

		if (bIsFilterValueValid && !InPayload->FilterCache.Contains(TransitionLayerSuggestion))
		{
			InPayload->FilterCache.Add(TransitionLayerSuggestion);
			InPayload->PossibleSuggestions.Add(FAssetSearchBoxSuggestion{MoveTemp(TransitionLayerSuggestion), FText::FromString(PageLayer), CategoryLabel});
		}
	}
}

bool FAvaRundownFilterTransitionLayerSuggestionFactory::SupportSuggestionType(EAvaRundownSearchListType InSuggestionType) const
{
	return InSuggestionType != EAvaRundownSearchListType::None;
}

#undef LOCTEXT_NAMESPACE
