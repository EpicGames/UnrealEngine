// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageFilterExpressionContext.h"

#include "IAvaMediaEditorModule.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownEditorDefines.h"
#include "Rundown/AvaRundownPage.h"
#include "Rundown/Factories/Filters/IAvaRundownFilterExpressionFactory.h"

void FAvaRundownPageFilterExpressionContext::SetItem(const FAvaRundownPage& InItem,  const UAvaRundown* InRundown, EAvaRundownSearchListType InRundownSearchListType)
{
	ItemPageId = InItem.GetPageId();
	ItemRundown = InRundown;
	RundownSearchListType = InRundownSearchListType;

	FString NameInfo = InItem.GetPageName();
	ItemName = *NameInfo;

	const FText IdInfo = FText::AsNumber(InItem.GetPageId(), &UE::AvaRundown::FEditorMetrics::PageIdFormattingOptions);
	ItemId = *IdInfo.ToString();

	const FString PathInfo = InItem.GetAssetPath(InRundown).GetAssetPathString();
	ItemPath = *PathInfo;

	const TArray<FAvaRundownChannelPageStatus> Statuses = InItem.GetPageProgramStatuses(InRundown);
	for (const FAvaRundownChannelPageStatus& Status : Statuses)
	{
		const FName StatusName = StaticEnum<EAvaRundownPageStatus>()->GetNameByValue(static_cast<int32>(Status.Status));
		ItemStatuses.Add(StatusName);
	}
}

void FAvaRundownPageFilterExpressionContext::ClearItem()
{
	ItemPageId = FAvaRundownPage::InvalidPageId;
	ItemRundown = nullptr;
	ItemName = NAME_None;
	ItemId = NAME_None;
	ItemPath = NAME_None;
	ItemStatuses.Reset();
}

bool FAvaRundownPageFilterExpressionContext::TestBasicStringExpression(const FTextFilterString& InValue
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (InValue.CompareName(ItemName, InTextComparisonMode))
	{
		return true;
	}
	if (InValue.CompareName(ItemId, InTextComparisonMode))
	{
		return true;
	}
	if (InValue.CompareName(ItemPath, ETextFilterTextComparisonMode::Partial))
	{
		return true;
	}
	for (const FName& Status : ItemStatuses)
	{
		if (InValue.CompareName(Status, InTextComparisonMode))
		{
			return true;
		}
	}
	return false;
}

bool FAvaRundownPageFilterExpressionContext::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	const IAvaMediaEditorModule& AvaMediaEditorModule = IAvaMediaEditorModule::Get();

	const FAvaRundownPage& PageItem = UAvaRundown::GetPageSafe(ItemRundown, ItemPageId);

	if (PageItem.IsValidPage() && AvaMediaEditorModule.CanFilterSupportComparisonOperation(InKey, InComparisonOperation, RundownSearchListType))
	{
		FAvaRundownTextFilterArgs FilterArgs;
		FilterArgs.ValueToCheck = InValue;
		FilterArgs.ItemRundown = ItemRundown;
		FilterArgs.ComparisonMode = InTextComparisonMode;
		FilterArgs.ComparisonOperation = InComparisonOperation;

		return AvaMediaEditorModule.FilterExpression(InKey, PageItem, FilterArgs);
	}
	return false;
}
