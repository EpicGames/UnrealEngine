// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Rundown/AvaRundownDefines.h"
#include "Rundown/Factories/Filters/AvaRundownFactoriesUtils.h"

class FName;
class FText;
class UAvaRundown;
struct FAvaRundownPage;

/** Expression context to test the given asset data against the current text filter */
class FAvaRundownPageFilterExpressionContext : public ITextFilterExpressionContext
{
public:
	FAvaRundownPageFilterExpressionContext()
		: ItemPageId(UE::AvaMedia::Rundown::InvalidPageId)
		, ItemRundown(nullptr)
		, RundownSearchListType(EAvaRundownSearchListType::None)
	{}

	void SetItem(const FAvaRundownPage& InItem, const UAvaRundown* InRundown, EAvaRundownSearchListType InRundownSearchListType);

	void ClearItem();

	virtual bool TestBasicStringExpression(const FTextFilterString& InValue
		, const ETextFilterTextComparisonMode InTextComparisonMode) const override;

	virtual bool TestComplexExpression(const FName& InKey
		, const FTextFilterString& InValue
		, const ETextFilterComparisonOperation InComparisonOperation
		, const ETextFilterTextComparisonMode InTextComparisonMode) const override;

private:
	/** Pointer to the asset we're currently filtering */
	int32 ItemPageId;

	const UAvaRundown* ItemRundown;

	EAvaRundownSearchListType RundownSearchListType;

	FName ItemName;

	FName ItemId;

	FName ItemPath;

	TArray<FName> ItemStatuses;
};
