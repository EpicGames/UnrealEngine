// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageTextFilter.h"

#include "AvaRundownPageFilterExpressionContext.h"
#include "Rundown/Factories/Filters/AvaRundownFactoriesUtils.h"

FAvaRundownPageTextFilter::FAvaRundownPageTextFilter()
	: TextFilterExpressionContext(MakeShared<FAvaRundownPageFilterExpressionContext>())
	, TextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::Complex)
{
}

bool FAvaRundownPageTextFilter::PassesFilter(const FAvaRundownPage& InItem) const
{
	const bool bMatched = TextFilterExpressionEvaluator.TestTextFilter(*TextFilterExpressionContext);
	TextFilterExpressionContext->ClearItem();
	return bMatched;
}

FText FAvaRundownPageTextFilter::GetFilterText() const
{
	return TextFilterExpressionEvaluator.GetFilterText();
}

void FAvaRundownPageTextFilter::SetFilterText(const FText& InFilterText)
{
	if (TextFilterExpressionEvaluator.SetFilterText(InFilterText))
	{
		OnChanged().Broadcast();
	}
}

void FAvaRundownPageTextFilter::SetItem(const FAvaRundownPage& InItem, const UAvaRundown* InRundown, EAvaRundownSearchListType InRundownSearchListType) const
{
	TextFilterExpressionContext->SetItem(InItem, InRundown, InRundownSearchListType);
}
