// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/IFilter.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Rundown/AvaRundownEditorDefines.h"
#include "Templates/SharedPointer.h"

class FAvaRundownPageFilterExpressionContext;
class FText;
class UAvaRundown;
enum class EAvaRundownSearchListType : uint8;

class FAvaRundownPageTextFilter
	: public IFilter<const FAvaRundownPage&>
	, public TSharedFromThis<FAvaRundownPageTextFilter>
{
public:
	FAvaRundownPageTextFilter();

	//~ Begin IFilter
	virtual FChangedEvent& OnChanged() override { return ChangedEvent; }
	virtual bool PassesFilter(const FAvaRundownPage& InItem) const override;
	//~ End IFilter

	FText GetFilterText() const;

	void SetFilterText(const FText& InFilterText);

	void SetItem(const FAvaRundownPage& InItem, const UAvaRundown* InRundown, EAvaRundownSearchListType InRundownSearchListType) const;

private:
	/** Transient context data, used when calling PassesFilter. Kept around to minimize re-allocations between multiple calls to PassesFilter */
	TSharedRef<FAvaRundownPageFilterExpressionContext> TextFilterExpressionContext;

	FTextFilterExpressionEvaluator TextFilterExpressionEvaluator;

	FChangedEvent ChangedEvent;
};
