// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageContextMenu.h"

#include "Framework/Commands/GenericCommands.h"
#include "IAvaMediaEditorModule.h"
#include "Rundown/AvaRundownCommands.h"
#include "Rundown/AvaRundownEditor.h"
#include "Rundown/Pages/AvaRundownPageContext.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "AvaRundownPageContextMenu"

TSharedRef<SWidget> FAvaRundownPageContextMenu::GeneratePageContextMenuWidget(const TWeakPtr<FAvaRundownEditor>& InRundownEditorWeak, const FAvaRundownPageListReference& InPageListReference, const TSharedPtr<FUICommandList>& InCommandList)
{
	UToolMenus* const ToolMenus = UToolMenus::Get();
	check(ToolMenus);

	const FName PageContextMenuName = IAvaMediaEditorModule::GetRundownPageMenuName();

	if (!ToolMenus->IsMenuRegistered(PageContextMenuName))
	{
		UToolMenu* const ContextMenu = ToolMenus->RegisterMenu(PageContextMenuName, NAME_None, EMultiBoxType::Menu);
		check(ContextMenu);

		ContextMenu->AddDynamicSection("PopulateContextMenu", FNewToolMenuDelegate::CreateStatic(
			[](UToolMenu* InMenu)
			{
				if (!InMenu)
				{
					return;
				}
				if (UAvaRundownPageContext* PageContext = InMenu->FindContext<UAvaRundownPageContext>())
				{
					if (TSharedPtr<FAvaRundownPageContextMenu> SourceContextMenu = PageContext->ContextMenuWeak.Pin())
					{
						SourceContextMenu->PopulatePageContextMenu(*InMenu, *PageContext);
					}
				}
			}));
	}

	UAvaRundownPageContext* const ContextObject = NewObject<UAvaRundownPageContext>();
	check(ContextObject);
	ContextObject->ContextMenuWeak    = SharedThis(this);
	ContextObject->RundownEditorWeak = InRundownEditorWeak;
	ContextObject->PageListReference  = InPageListReference;

	TSharedPtr<FExtender> Extender;

	// Compatibility with IAvaMediaEditorModule Rundown Menu Extensibility Manager
	if (InPageListReference.Type == EAvaRundownPageListType::Template && InCommandList.IsValid())
	{
		TSharedPtr<FExtensibilityManager> MenuExtensibility = IAvaMediaEditorModule::Get().GetRundownMenuExtensibilityManager();

		TSharedPtr<FAvaRundownEditor> RundownEditor = InRundownEditorWeak.Pin();

		if (MenuExtensibility.IsValid() && RundownEditor.IsValid())
		{
			if (const TArray<UObject*>* EditingObjects = RundownEditor->GetObjectsCurrentlyBeingEdited())
			{
				Extender = MenuExtensibility->GetAllExtenders(InCommandList.ToSharedRef(), *EditingObjects);
			}
		}
	}

	FToolMenuContext Context(InCommandList, Extender, ContextObject);
	return ToolMenus->GenerateWidget(PageContextMenuName, Context);
}

void FAvaRundownPageContextMenu::PopulatePageContextMenu(UToolMenu& InMenu, UAvaRundownPageContext& InContext)
{
	TSharedPtr<FExtensibilityManager> MenuExtensibility;

	const FGenericCommands& GenericCommands = FGenericCommands::Get();

	const FAvaRundownCommands& RundownCommands = FAvaRundownCommands::Get();

	const FAvaRundownPageListReference& PageListReference = InContext.GetPageListReference();

	// Page List Operations
	{
		FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("PageListOperations"), LOCTEXT("PlayListOperationsHeader", "Page List Actions"));

		if (PageListReference.Type == EAvaRundownPageListType::Template)
		{
			Section.AddMenuEntry(RundownCommands.AddTemplate);
			Section.AddMenuEntry(RundownCommands.CreatePageInstanceFromTemplate);
			Section.AddMenuEntry(RundownCommands.CreateComboTemplate);
		}

		Section.AddMenuEntry(RundownCommands.RemovePage);
		Section.AddMenuEntry(RundownCommands.RenumberPage);
		Section.AddMenuEntry(GenericCommands.Rename);
		Section.AddMenuEntry(GenericCommands.Cut);
		Section.AddMenuEntry(GenericCommands.Copy);
		Section.AddMenuEntry(GenericCommands.Paste);
		Section.AddMenuEntry(GenericCommands.Duplicate);
		Section.AddMenuEntry(RundownCommands.ReimportPage);
		Section.AddMenuEntry(RundownCommands.EditPageSource);

		if (PageListReference.Type == EAvaRundownPageListType::Template || PageListReference.Type == EAvaRundownPageListType::Instance)
		{
			Section.AddSubMenu(TEXT("ExportPages"),
				LOCTEXT("ExportPagesSubMenu", "Export Pages ..."),
				LOCTEXT("ExportPagesSubMenuTooltip", "Export selected pages to a separate resource."),
				FNewMenuDelegate::CreateLambda([&RundownCommands](FMenuBuilder& InMenuBuilder)
				{
					// When exporting instanced pages, corresponding templates follow along.
					InMenuBuilder.AddMenuEntry(RundownCommands.ExportPagesToRundown);
					InMenuBuilder.AddMenuEntry(RundownCommands.ExportPagesToJson);
					InMenuBuilder.AddMenuEntry(RundownCommands.ExportPagesToXml);
				}),
				false,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.SaveAll")
			);
		}

		if (PageListReference.Type == EAvaRundownPageListType::Instance)
		{
			Section.AddSubMenu(TEXT("ResetPages"),
				LOCTEXT("ResetPagesSubMenu", "Reset Pages ..."),
				LOCTEXT("ResetPagesSubMenuTooltip", "Reset selected pages to default values."),
				FNewMenuDelegate::CreateLambda([&RundownCommands](FMenuBuilder& InMenuBuilder)
					{
						InMenuBuilder.AddMenuEntry(RundownCommands.ResetValuesToDefaults,
							TEXT("InstanceResetPages"),
							LOCTEXT("InstanceResetPages", "Reset to Default"),
							LOCTEXT("InstanceResetPagesTooltip", "Reset all selected pages to default values."),
							FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault")));

						InMenuBuilder.AddMenuEntry(RundownCommands.ResetValuesToTemplate,
							TEXT("TemplateResetPages"),
							LOCTEXT("TemplateResetPages", "Reset to Template"),
							LOCTEXT("TemplateResetPagesTooltip", "Reset all selected pages to template values."),
							FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault")));
					}),
				false,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault"))
			);
		}
		else if (PageListReference.Type == EAvaRundownPageListType::Template)
		{
			Section.AddMenuEntry(RundownCommands.ResetValuesToDefaults,
				LOCTEXT("ResetPages", "Reset to Default"),
				LOCTEXT("ResetPagesTooltip", "Reset all selected template pages to default values."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault"))
			);
		}
	}

	// Show Control Actions
	if (PageListReference.Type != EAvaRundownPageListType::Template)
	{
		FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("ShowControlOperations"), LOCTEXT("ShowControlOperationsHeader", "Show Control Actions"));

		Section.AddMenuEntry(RundownCommands.Play);
		Section.AddMenuEntry(RundownCommands.UpdateValues);
		Section.AddMenuEntry(RundownCommands.Continue);
		Section.AddMenuEntry(RundownCommands.Stop);
		Section.AddMenuEntry(RundownCommands.ForceStop);
		Section.AddMenuEntry(RundownCommands.PlayNext);
		Section.AddMenuEntry(RundownCommands.LoadAllPages);
		Section.AddMenuEntry(RundownCommands.LoadNextPages);
		Section.AddMenuEntry(RundownCommands.LoadSelectedPages);
	}

	// Preview Actions
	{
		FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("PreviewOperations"), LOCTEXT("PreviewOperationsHeader", "Preview Actions"));

		Section.AddMenuEntry(RundownCommands.PreviewPlay);
		Section.AddMenuEntry(RundownCommands.PreviewFrame);
		Section.AddMenuEntry(RundownCommands.PreviewContinue);
		Section.AddMenuEntry(RundownCommands.PreviewStop);
		Section.AddMenuEntry(RundownCommands.PreviewForceStop);
		Section.AddMenuEntry(RundownCommands.PreviewPlayNext);
		Section.AddMenuEntry(RundownCommands.TakeToProgram);
	}
}

#undef LOCTEXT_NAMESPACE
