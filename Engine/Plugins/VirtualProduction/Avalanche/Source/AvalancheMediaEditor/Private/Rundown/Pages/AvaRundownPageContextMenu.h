// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FAvaRundownEditor;
class FUICommandList;
class SWidget;
class UAvaRundownPageContext;
class UToolMenu;
struct FAvaRundownPageListReference;

class FAvaRundownPageContextMenu : public TSharedFromThis<FAvaRundownPageContextMenu>
{
public:
	TSharedRef<SWidget> GeneratePageContextMenuWidget(const TWeakPtr<FAvaRundownEditor>& InRundownEditorWeak, const FAvaRundownPageListReference& InPageListReference, const TSharedPtr<FUICommandList>& InCommandList);

private:
	void PopulatePageContextMenu(UToolMenu& InMenu, UAvaRundownPageContext& InContext);
};
