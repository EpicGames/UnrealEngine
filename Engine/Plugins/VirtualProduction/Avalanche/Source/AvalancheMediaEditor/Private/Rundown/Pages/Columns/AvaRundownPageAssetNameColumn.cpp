// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageAssetNameColumn.h"

#include "Rundown/Pages/PageViews/AvaRundownTemplatePageViewImpl.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AvaRundownPageAssetNameColumn"

FText FAvaRundownPageAssetNameColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("PageId_AssetName", "Asset");
}

FText FAvaRundownPageAssetNameColumn::GetColumnToolTipText() const
{
	return LOCTEXT("PageId_ToolTip", "Name and Id of the template used to create this page.");
}

SHeaderRow::FColumn::FArguments FAvaRundownPageAssetNameColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnToolTipText())
		.FillWidth(0.1f)
		.ShouldGenerateWidget(true)
		.VAlignCell(EVerticalAlignment::VAlign_Center)
		;
}

TSharedRef<SWidget> FAvaRundownPageAssetNameColumn::ConstructRowWidget(const FAvaRundownPageViewRef& InPageView
	, const TSharedPtr<SAvaRundownPageViewRow>& InRow)
{
	const TSharedRef<FAvaRundownPageViewImpl> InstancedPageView = StaticCastSharedRef<FAvaRundownPageViewImpl>(InPageView);
	const UAvaRundown* Rundown = InPageView->GetRundown();

	return SNew(SBox)
		.Padding(UE::AvaRundown::FEditorMetrics::ColumnLeftOffset, 0.f, 0.f, 0.f)
		[
			SNew(STextBlock)
			.Text(InstancedPageView, &FAvaRundownPageViewImpl::GetObjectNames, Rundown)
		];
}

#undef LOCTEXT_NAMESPACE
