// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageAssetSelectorColumn.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "Engine/World.h"
#include "Framework/Application/SlateApplication.h"
#include "PropertyCustomizationHelpers.h"
#include "Rundown/Pages/PageViews/IAvaRundownPageView.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AvaRundownPageAssetSelectorColumn"

FText FAvaRundownPageAssetSelectorColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("AssetSelectorColumn_Name", "Motion Design Asset");
}

FText FAvaRundownPageAssetSelectorColumn::GetColumnToolTipText() const
{
	return LOCTEXT("AssetSelectorColumn_ToolTip", "Selects a given Motion Design Asset for the Page");
}

SHeaderRow::FColumn::FArguments FAvaRundownPageAssetSelectorColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnToolTipText())
		.FillWidth(0.25f)
		.ShouldGenerateWidget(true)
		.VAlignCell(EVerticalAlignment::VAlign_Center)
	;
}

namespace UE::AvaPageAssetSelectorColumn::Private
{
	TSharedRef<SWidget> GetAssetPicker(const FAvaRundownPageViewRef& InPageView)
	{
		const UAvaRundown* Rundown = InPageView->GetRundown();

		constexpr bool bAllowClear = true;
		TArray<const UClass*> AllowedClasses;
		AllowedClasses.Add(UWorld::StaticClass());

		const FAssetData AssetData = IAssetRegistry::Get()->GetAssetByObjectPath(InPageView->GetObjectPath(Rundown));
	
		return PropertyCustomizationHelpers::MakeAssetPickerWithMenu(
			AssetData,
			bAllowClear,
			AllowedClasses,
			PropertyCustomizationHelpers::GetNewAssetFactoriesForClasses(AllowedClasses),
			FOnShouldFilterAsset(),
			FOnAssetSelected::CreateSP(InPageView, &IAvaRundownPageView::OnObjectChanged),
			FSimpleDelegate::CreateLambda([]{FSlateApplication::Get().DismissAllMenus();}));
	}
}

TSharedRef<SWidget> FAvaRundownPageAssetSelectorColumn::ConstructRowWidget(const FAvaRundownPageViewRef& InPageView,
                                                                    const TSharedPtr<SAvaRundownPageViewRow>& InRow)
{
	const UAvaRundown* Rundown = InPageView->GetRundown();

	const FAvaRundownPageViewWeak PageViewWeak = InPageView;

	// The row widget is not reconstructed when page view changes.
	// So we need to create all the possible widgets and have them switch visibility.
	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			// Combo templates don't have an asset selector, use a text bock with asset names instead.
			SNew(STextBlock)
			.Visibility_Static(&FAvaRundownPageAssetSelectorColumn::GetAssetNameVisibility, PageViewWeak)
			.Text(InPageView, &IAvaRundownPageView::GetObjectNames, Rundown)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		]
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SNew(SComboButton)
				.Visibility_Static(&FAvaRundownPageAssetSelectorColumn::GetAssetSelectorVisibility, PageViewWeak)
				.OnGetMenuContent_Lambda([InPageView]{ return UE::AvaPageAssetSelectorColumn::Private::GetAssetPicker(InPageView);})
				.ContentPadding(FMargin(2.0f, 2.0f))
				.ButtonContent()
				[
					SNew(STextBlock)
					.Text(InPageView, &IAvaRundownPageView::GetObjectName, Rundown)
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				]
		];
}

EVisibility FAvaRundownPageAssetSelectorColumn::GetAssetSelectorVisibility(const FAvaRundownPageViewWeak InPageViewWeak)
{
	const TSharedPtr<IAvaRundownPageView> PageView = InPageViewWeak.Pin();
	return PageView->IsComboTemplate() ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility FAvaRundownPageAssetSelectorColumn::GetAssetNameVisibility(const FAvaRundownPageViewWeak InPageViewWeak)
{
	const TSharedPtr<IAvaRundownPageView> PageView = InPageViewWeak.Pin();
	return !PageView->IsComboTemplate() ? EVisibility::Collapsed : EVisibility::Visible;
}

#undef LOCTEXT_NAMESPACE
