// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/Pages/Columns/IAvaRundownPageViewColumn.h"

class FAvaRundownPageAssetSelectorColumn : public IAvaRundownPageViewColumn
{
public:
	UE_AVA_INHERITS(FAvaRundownPageAssetSelectorColumn, IAvaRundownPageViewColumn);

	//~ Begin IAvaRundownPageViewColumn
	virtual FText GetColumnDisplayNameText() const override;
	virtual FText GetColumnToolTipText() const override;
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual TSharedRef<SWidget> ConstructRowWidget(const FAvaRundownPageViewRef& InPageView, const TSharedPtr<SAvaRundownPageViewRow>& InRow) override;
	//~ End IAvaRundownPageViewColumn

	static EVisibility GetAssetSelectorVisibility(const FAvaRundownPageViewWeak InPageViewWeak);
	static EVisibility GetAssetNameVisibility(const FAvaRundownPageViewWeak InPageViewWeak);
};
