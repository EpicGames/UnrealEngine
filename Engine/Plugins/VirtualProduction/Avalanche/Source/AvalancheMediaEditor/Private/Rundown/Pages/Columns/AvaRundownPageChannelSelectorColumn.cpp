// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageChannelSelectorColumn.h"

#include "Slate/SAvaRundownPageChannelSelector.h"

#define LOCTEXT_NAMESPACE "AvaRundownPageChannelSelectorColumn"

FText FAvaRundownPageChannelSelectorColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("ChannelSelectorColumn_ChannelName", "Channel");
}

FText FAvaRundownPageChannelSelectorColumn::GetColumnToolTipText() const
{
	return LOCTEXT("ChannelSelectorColumn_ToolTip", "Selects a Broadcast Channel for the Page");
}

SHeaderRow::FColumn::FArguments FAvaRundownPageChannelSelectorColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnToolTipText())
		.FillWidth(0.25f)
		.ShouldGenerateWidget(true)
		.VAlignCell(EVerticalAlignment::VAlign_Center)
	;
}

TSharedRef<SWidget> FAvaRundownPageChannelSelectorColumn::ConstructRowWidget(const FAvaRundownPageViewRef& InPageView, const TSharedPtr<SAvaRundownPageViewRow>& InRow)
{
	return SNew(SBox)
		.Padding(UE::AvaRundown::FEditorMetrics::ColumnLeftOffset, 0.f, 0.f, 0.f)
		[
			SNew(SAvaRundownPageChannelSelector, InPageView, InRow)
		];
}

#undef LOCTEXT_NAMESPACE
