// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageEnabledColumn.h"

#include "Rundown/Pages/PageViews/AvaRundownInstancedPageViewImpl.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "AvaRundownPageEnabledColumn"

FText FAvaRundownPageEnabledColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("EnabledColumn_Name", "Enabled");
}

FText FAvaRundownPageEnabledColumn::GetColumnToolTipText() const
{
	return LOCTEXT("EnabledColumn_ToolTip", "Determines whether this Page should be considered for Rundown");
}

SHeaderRow::FColumn::FArguments FAvaRundownPageEnabledColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnToolTipText())
		.FixedWidth(24.0f)
		.VAlignCell(EVerticalAlignment::VAlign_Center)
		.HAlignCell(EHorizontalAlignment::HAlign_Center)
		.ShouldGenerateWidget(true)
		[
			SNew(SCheckBox)
			.IsChecked(true)
			.Visibility(EVisibility::Hidden)
		];
}

TSharedRef<SWidget> FAvaRundownPageEnabledColumn::ConstructRowWidget(const FAvaRundownPageViewRef& InPageView, const TSharedPtr<SAvaRundownPageViewRow>& InRow)
{
	const TSharedRef<FAvaRundownInstancedPageViewImpl> InstancedPageView = StaticCastSharedRef<FAvaRundownInstancedPageViewImpl>(InPageView);

	return SNew(SBox)
		.Padding(UE::AvaRundown::FEditorMetrics::ColumnLeftOffset, 0.f, 0.f, 0.f)
		[
			SNew(SCheckBox)
			.IsChecked(InstancedPageView, &FAvaRundownInstancedPageViewImpl::IsEnabled)
			.OnCheckStateChanged(InstancedPageView, &FAvaRundownInstancedPageViewImpl::SetEnabled)
		];
}

#undef LOCTEXT_NAMESPACE
