// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageIdColumn.h"

#include "Slate/SAvaRundownPageId.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "AvaRundownPageIdColumn"

FText FAvaRundownPageIdColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("PageId_Name", "Page #");
}

FText FAvaRundownPageIdColumn::GetColumnToolTipText() const
{
	return LOCTEXT("PageId_ToolTip", "Page Id");
}

SHeaderRow::FColumn::FArguments FAvaRundownPageIdColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnToolTipText())
		.FixedWidth(50.f)
		.ShouldGenerateWidget(true)
		.VAlignCell(EVerticalAlignment::VAlign_Center)
	;
}

TSharedRef<SWidget> FAvaRundownPageIdColumn::ConstructRowWidget(const FAvaRundownPageViewRef& InPageView
	, const TSharedPtr<SAvaRundownPageViewRow>& InRow)
{
	return SNew(SBox)
		.Padding(UE::AvaRundown::FEditorMetrics::ColumnLeftOffset, 0.f, 0.f, 0.f)
		[
			SNew(SAvaRundownPageId, InPageView)
		];
}

#undef LOCTEXT_NAMESPACE
