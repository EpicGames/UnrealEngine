// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageNameColumn.h"

#include "Slate/SAvaRundownPageName.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "AvaRundownPageNameColumn"

FText FAvaRundownPageNameColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("PageName_Name", "Description");
}

FText FAvaRundownPageNameColumn::GetColumnToolTipText() const
{
	return LOCTEXT("PageName_ToolTip", "Description of the given Page. Can be null or repeated");
}

SHeaderRow::FColumn::FArguments FAvaRundownPageNameColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnToolTipText())
		.FillWidth(0.25f)
		.ShouldGenerateWidget(true)
		.VAlignCell(EVerticalAlignment::VAlign_Center)
	;
}

TSharedRef<SWidget> FAvaRundownPageNameColumn::ConstructRowWidget(const FAvaRundownPageViewRef& InPageView
	, const TSharedPtr<SAvaRundownPageViewRow>& InRow)
{
	return SNew(SBox)
		.Padding(UE::AvaRundown::FEditorMetrics::ColumnLeftOffset, 0.f, 0.f, 0.f)
		[
			SNew(SAvaRundownPageName, InPageView, InRow)
		];
}

#undef LOCTEXT_NAMESPACE
