// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageStatusColumn.h"

#include "AvaMediaEditorStyle.h"
#include "Broadcast/AvaBroadcast.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownPage.h"
#include "Rundown/Pages/PageViews/AvaRundownInstancedPageViewImpl.h"
#include "Rundown/Pages/PageViews/IAvaRundownPageView.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "AvaRundownPageStatusColumn"

FText FAvaRundownPageStatusColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("PageId_Status", "Status");
}

FText FAvaRundownPageStatusColumn::GetColumnToolTipText() const
{
	return LOCTEXT("PageId_ToolTip", "Page Status");
}

SHeaderRow::FColumn::FArguments FAvaRundownPageStatusColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnToolTipText())
		.FixedWidth(94.f)
		.ShouldGenerateWidget(true)
		.VAlignCell(EVerticalAlignment::VAlign_Center)
	;
}

TSharedRef<SWidget> FAvaRundownPageStatusColumn::ConstructRowWidget(const FAvaRundownPageViewRef& InPageView
	, const TSharedPtr<SAvaRundownPageViewRow>& InRow)
{
	const TSharedRef<FAvaRundownInstancedPageViewImpl> InstancedPageView = StaticCastSharedRef<FAvaRundownInstancedPageViewImpl>(
		StaticCastSharedRef<FAvaRundownPageViewImpl>(InPageView)
		);

	const TWeakPtr<FAvaRundownInstancedPageViewImpl> InstancedPageViewWeak = InstancedPageView;
	
	TSharedRef<SHorizontalBox> ButtonList = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(UE::AvaRundown::FEditorMetrics::ColumnLeftOffset, 0.f, 0.f, 0.f)
		[
			SNew(SButton)
			.ButtonStyle(FAvaMediaEditorStyle::Get(), "AvaMediaEditor.BorderlessButton")
			.OnClicked(InPageView, &IAvaRundownPageView::OnPreviewButtonClicked)
			.IsEnabled(InPageView, &IAvaRundownPageView::CanPreview)
			.ToolTipText(InPageView, &IAvaRundownPageView::GetPreviewInTooltip)
			[
				SNew(SBorder)
				.Padding(FMargin(0.f, 0.f))
				.BorderImage_Static(&FAvaRundownPageStatusColumn::GetPreviewBorderBackgroundImage, InstancedPageViewWeak)
				.BorderBackgroundColor(FLinearColor(1.0f, 0.0f, 0.0f, 1.0f))
				[
					SNew(SImage)
					.ColorAndOpacity_Static(&FAvaRundownPageStatusColumn::GetIsPreviewingButtonColor, InstancedPageViewWeak)
					.Image(FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.MediaPreviewing"))
				]
			]
		];
	
	ButtonList->AddSlot()
		.Padding(UE::AvaRundown::FEditorMetrics::ColumnLeftOffset, 0.f, 0.f, 0.f)
		[
			SNew(SButton)
			.ButtonStyle(FAvaMediaEditorStyle::Get(), "AvaMediaEditor.BorderlessButton")
			.OnClicked(InstancedPageView, &FAvaRundownInstancedPageViewImpl::OnAssetStatusButtonClicked)
			.IsEnabled(InstancedPageView, &FAvaRundownInstancedPageViewImpl::CanChangeAssetStatus)
			.ToolTipText_Static(&FAvaRundownPageStatusColumn::GetAssetStatusButtonTooltip, InstancedPageViewWeak)
			[
				SNew(SImage)
				.ColorAndOpacity_Static(&FAvaRundownPageStatusColumn::GetAssetStatusButtonColor, InstancedPageViewWeak)
				.Image(FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.MediaAssetStatus"))
			]
		];

	ButtonList->AddSlot()
		.Padding(UE::AvaRundown::FEditorMetrics::ColumnLeftOffset, 0.f, 0.f, 0.f)
		[
			SNew(SBorder)
			.Padding(FMargin(5.f, 4.f))
			.BorderImage_Static(&FAvaRundownPageStatusColumn::GetProgramBorderBackgroundImage, InstancedPageViewWeak)
			.BorderBackgroundColor(FLinearColor(1.0f, 0.0f, 0.0f, 1.0f))
			.ToolTipText_Static(&FAvaRundownPageStatusColumn::GetTakeInTooltip, InstancedPageViewWeak)
			[
				SNew(SImage)
				.ColorAndOpacity_Static(&FAvaRundownPageStatusColumn::GetIsPlayingButtonColor, InstancedPageViewWeak)
				.Image(FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.MediaPlaying"))
			]
		];

	return ButtonList;
}

FSlateColor FAvaRundownPageStatusColumn::GetIsPreviewingButtonColor(const TWeakPtr<FAvaRundownInstancedPageViewImpl> InPageViewWeak)
{
	static const FSlateColor Error(FStyleColors::AccentRed.GetSpecifiedColor());
	static const FSlateColor Syncing(FStyleColors::AccentOrange.GetSpecifiedColor());
	static const FSlateColor Loading(FStyleColors::AccentBlue.GetSpecifiedColor());
	static const FSlateColor NotPreviewing(FStyleColors::AccentGray.GetSpecifiedColor());
	static const FSlateColor Previewing(FStyleColors::AccentGreen.GetSpecifiedColor());
	
	if (const FAvaRundownPageViewPtr PageView = InPageViewWeak.Pin())
	{
		UAvaRundown* Previewlist = PageView->GetRundown();

		if (IsValid(Previewlist))
		{
			FAvaRundownPage& Page = Previewlist->GetPage(PageView->GetPageId());

			if (Page.IsValidPage())
			{
				const TArray<FAvaRundownChannelPageStatus> Statuses = Page.GetPagePreviewStatuses(Previewlist);

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Unknown,
					EAvaRundownPageStatus::Error,
					EAvaRundownPageStatus::Missing}))
				{
					return Error;
				}

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Syncing}))
				{
					return Syncing;
				}

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Loading}))
				{
					return Loading;
				}

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Previewing}))
				{
					return Previewing;
				}

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Available,
					EAvaRundownPageStatus::Loaded,
					EAvaRundownPageStatus::Playing}))
				{
					return NotPreviewing;
				}
			}
		}
	}

	return Error;
}

const FSlateBrush* FAvaRundownPageStatusColumn::GetPreviewBorderBackgroundImage(const TWeakPtr<FAvaRundownInstancedPageViewImpl> InPageViewWeak)
{
	bool bIsPlayHead = false;
	
	if (const FAvaRundownPageViewPtr PageView = InPageViewWeak.Pin())
	{
		const UAvaRundown* Rundown = PageView->GetRundown();

		if (IsValid(Rundown))
		{
			if (const FAvaRundownPageListPlaybackContextCollection* ContextCollection = Rundown->GetPageListPlaybackContextCollection())
			{
				if (const TSharedPtr<FAvaRundownPageListPlaybackContext> Context = ContextCollection->GetContext(/*bInIsPreview*/ true, Rundown->GetDefaultPreviewChannelName()))
				{
					bIsPlayHead = Context->PlayHeadPageId == PageView->GetPageId();
				}
			}
		}
	}
	
	if (bIsPlayHead)
	{
		return FAppStyle::GetBrush("Border");
	}
	return FAppStyle::GetBrush("NoBorder");
}

FSlateColor FAvaRundownPageStatusColumn::GetIsPlayingButtonColor(const TWeakPtr<FAvaRundownInstancedPageViewImpl> InPageViewWeak)
{
	static const FSlateColor Error(FStyleColors::AccentRed.GetSpecifiedColor());
	static const FSlateColor NotPlaying(FStyleColors::AccentGray.GetSpecifiedColor());
	static const FSlateColor Playing(FStyleColors::AccentGreen.GetSpecifiedColor());

	if (const FAvaRundownPageViewPtr PageView = InPageViewWeak.Pin())
	{
		UAvaRundown* Rundown = PageView->GetRundown();

		if (IsValid(Rundown))
		{
			FAvaRundownPage& Page = Rundown->GetPage(PageView->GetPageId());

			if (Page.IsValidPage())
			{
				const TArray<FAvaRundownChannelPageStatus> Statuses = Page.GetPageProgramStatuses(Rundown);

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Playing}))
				{
					return Playing;
				}

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Unknown,
					EAvaRundownPageStatus::Offline,
					EAvaRundownPageStatus::Missing,
					EAvaRundownPageStatus::Error}))
				{
					return Error;
				}
			}
		}
	}

	return NotPlaying;
}

const FSlateBrush* FAvaRundownPageStatusColumn::GetProgramBorderBackgroundImage(const TWeakPtr<FAvaRundownInstancedPageViewImpl> InPageViewWeak)
{
	bool bIsPlayHead = false;
	
	if (const FAvaRundownPageViewPtr PageView = InPageViewWeak.Pin())
	{
		const UAvaRundown* Rundown = PageView->GetRundown();

		if (IsValid(Rundown))
		{
			if (const FAvaRundownPageListPlaybackContextCollection* ContextCollection = Rundown->GetPageListPlaybackContextCollection())
			{
				if (const TSharedPtr<FAvaRundownPageListPlaybackContext> Context = ContextCollection->GetContext(/*bInIsPreview*/ false, NAME_None))
				{
					bIsPlayHead = Context->PlayHeadPageId == PageView->GetPageId();
				}
			}
		}
	}
	
	if (bIsPlayHead)
	{
		return FAppStyle::GetBrush("Border");
	}
	return FAppStyle::GetBrush("NoBorder");
}

FText FAvaRundownPageStatusColumn::GetTakeInTooltip(const TWeakPtr<FAvaRundownInstancedPageViewImpl> InPageViewWeak)
{
	// static const FText BaseTooltip = LOCTEXT("TakeIn", "Take In\n\n- Click: Play\n- Control+Click: Continue");
	static const FText BaseTooltip = LOCTEXT("Program_Status", "Program Status: ");
	static const FText PlayedTooltip = LOCTEXT("Program_Playing", "Playing");
	static const FText NotPlayedTooltip = LOCTEXT("Program_Stopped", "Stopped");
	static const FText ErrorTooltip = LOCTEXT("Program_CantPlay", "**Cannot Take In**");
	static const FText NoOutputs = LOCTEXT("Program_NoOutputs", "No Outputs Selected");
	static const FText NewLines = LOCTEXT("Program_NewLines", "\n\n");

	if (const FAvaRundownPageViewPtr PageView = InPageViewWeak.Pin())
	{
		UAvaRundown* Rundown = PageView->GetRundown();

		if (IsValid(Rundown))
		{
			TArray<FText> Texts;
			Texts.Add(BaseTooltip);
			bool bAddedErrorTooltip = false;

			// Checks whether it can play based on situation, not status
			FString FailureReason;
			if (!Rundown->CanPlayPage(PageView->GetPageId(), /*bInPreview*/ false, NAME_None, &FailureReason))
			{
				Texts.Add(ErrorTooltip);
				Texts.Add(FText::Format(LOCTEXT("Program_CantPlayReason", "Reason: {0}"), FText::FromString(FailureReason)));
				bAddedErrorTooltip = true;
			}

			const FAvaRundownPage& Page = Rundown->GetPage(PageView->GetPageId());

			// Check actual status
			if (Page.IsValidPage())
			{
				const FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetChannel(Page.GetChannelName());

				if (Channel.GetMediaOutputs().IsEmpty())
				{
					if (!bAddedErrorTooltip)
					{
						Texts.Add(ErrorTooltip);
						bAddedErrorTooltip = true;
					}

					Texts.Add(NoOutputs);
				}
				else
				{
					Texts.Add(Rundown->IsPagePlaying(Page) ? PlayedTooltip : NotPlayedTooltip);
				}
			}

			return FText::Join(NewLines, Texts);
		}
	}

	return BaseTooltip;
}

FSlateColor FAvaRundownPageStatusColumn::GetAssetStatusButtonColor(const TWeakPtr<FAvaRundownInstancedPageViewImpl> InPageViewWeak)
{
	static const FSlateColor Error(FStyleColors::AccentRed.GetSpecifiedColor());
	static const FSlateColor Loading(FStyleColors::AccentBlue.GetSpecifiedColor());
	static const FSlateColor Available(FStyleColors::AccentYellow.GetSpecifiedColor());
	static const FSlateColor Loaded(FStyleColors::AccentGreen.GetSpecifiedColor());
	static const FSlateColor Playing(FStyleColors::AccentOrange.GetSpecifiedColor());

	if (const TSharedPtr<FAvaRundownInstancedPageViewImpl> PageView = InPageViewWeak.Pin())
	{
		UAvaRundown* Rundown = PageView->GetRundown();

		if (IsValid(Rundown))
		{
			FAvaRundownPage& Page = Rundown->GetPage(PageView->GetPageId());

			if (Page.IsValidPage())
			{
				const TArray<FAvaRundownChannelPageStatus> Statuses = Page.GetPageProgramStatuses(Rundown);

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Unknown,
					EAvaRundownPageStatus::Error,
					EAvaRundownPageStatus::Missing}))
				{
					return Error;
				}

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Loading}))
				{
					return Loading;
				}

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Previewing,
					EAvaRundownPageStatus::Playing}))
				{
					return Playing;
				}

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Loaded}))
				{
					return Loaded;
				}

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Available}))
				{
					return Available;
				}
			}
		}
	}

	return Error;
}

FText FAvaRundownPageStatusColumn::GetAssetStatusButtonTooltip(const TWeakPtr<FAvaRundownInstancedPageViewImpl> InPageViewWeak)
{
	if (const TSharedPtr<FAvaRundownInstancedPageViewImpl> PageView = InPageViewWeak.Pin())
	{
		UAvaRundown* Rundown = PageView->GetRundown();

		if (IsValid(Rundown))
		{
			FAvaRundownPage& Page = Rundown->GetPage(PageView->GetPageId());

			if (Page.IsValidPage())
			{
				const UEnum* PageStatusEnum = StaticEnum<EAvaRundownPageStatus>();
				const UEnum* ChannelTypeEnum = StaticEnum<EAvaBroadcastChannelType>();
				const TArray<FAvaRundownChannelPageStatus> Statuses = Page.GetPageStatuses(Rundown);
				TArray<FText> StatusTexts;
				StatusTexts.Reserve(Statuses.Num());

				for (const FAvaRundownChannelPageStatus& Status : Statuses)
				{
					FText StatusTypeText = FText::Format(LOCTEXT("StatusType", "{0}: {1}"),
						ChannelTypeEnum->GetDisplayNameTextByValue(static_cast<int64>(Status.Type)),
						PageStatusEnum->GetDisplayNameTextByValue(static_cast<int64>(Status.Status)));
					
					StatusTexts.Add(MoveTemp(StatusTypeText));
				}

				if (!StatusTexts.IsEmpty())
				{
					return FText::Format(
						LOCTEXT("StatusInfo", "Status: {0}."),
						FText::Join(LOCTEXT("Separator", ", "), StatusTexts)
					);
				}
			}
		}
	}

	return LOCTEXT("NoStatues", "Status Information Unavailable.");
}

#undef LOCTEXT_NAMESPACE
