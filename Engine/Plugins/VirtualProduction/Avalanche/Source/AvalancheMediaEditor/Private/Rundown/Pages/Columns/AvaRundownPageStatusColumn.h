// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/Pages/Columns/IAvaRundownPageViewColumn.h"

class FAvaRundownInstancedPageViewImpl;

class FAvaRundownPageStatusColumn : public IAvaRundownPageViewColumn
{
public:
	UE_AVA_INHERITS(FAvaRundownPageStatusColumn, IAvaRundownPageViewColumn);

	//~ Begin IAvaRundownPageViewColumn
	virtual FText GetColumnDisplayNameText() const override;
	virtual FText GetColumnToolTipText() const override;
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual TSharedRef<SWidget> ConstructRowWidget(const FAvaRundownPageViewRef& InPageView, const TSharedPtr<SAvaRundownPageViewRow>& InRow) override;
	//~ End IAvaRundownPageViewColumn

	static FSlateColor GetIsPreviewingButtonColor(const TWeakPtr<FAvaRundownInstancedPageViewImpl> InPageViewWeak);

	static const FSlateBrush* GetPreviewBorderBackgroundImage(const TWeakPtr<FAvaRundownInstancedPageViewImpl> InPageViewWeak);
	
	static FSlateColor GetIsPlayingButtonColor(const TWeakPtr<FAvaRundownInstancedPageViewImpl> InPageViewWeak);

	static const FSlateBrush* GetProgramBorderBackgroundImage(const TWeakPtr<FAvaRundownInstancedPageViewImpl> InPageViewWeak);

	static FText GetTakeInTooltip(const TWeakPtr<FAvaRundownInstancedPageViewImpl> InPageViewWeak);

	static FSlateColor GetAssetStatusButtonColor(const TWeakPtr<FAvaRundownInstancedPageViewImpl> InPageViewWeak);
	static FText GetAssetStatusButtonTooltip(const TWeakPtr<FAvaRundownInstancedPageViewImpl> InPageViewWeak);
};
