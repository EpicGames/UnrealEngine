// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/Pages/Columns/IAvaRundownPageViewColumn.h"

class FAvaRundownPageTemplateNameColumn : public IAvaRundownPageViewColumn
{
public:
	UE_AVA_INHERITS(FAvaRundownPageTemplateNameColumn, IAvaRundownPageViewColumn);

	//~ Begin IAvaRundownPageViewColumn
	virtual FText GetColumnDisplayNameText() const override;
	virtual FText GetColumnToolTipText() const override;
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual TSharedRef<SWidget> ConstructRowWidget(const FAvaRundownPageViewRef& InPageView, const TSharedPtr<SAvaRundownPageViewRow>& InRow) override;
	//~ End IAvaRundownPageViewColumn
};
