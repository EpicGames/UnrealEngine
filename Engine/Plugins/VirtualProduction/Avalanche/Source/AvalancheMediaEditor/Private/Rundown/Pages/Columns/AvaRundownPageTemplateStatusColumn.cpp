// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageTemplateStatusColumn.h"

#include "AvaMediaEditorStyle.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownPage.h"
#include "Rundown/Pages/PageViews/AvaRundownPageViewImpl.h"
#include "Rundown/Pages/PageViews/AvaRundownTemplatePageViewImpl.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "AvaRundownPageTemplateStatusColumn"

FText FAvaRundownPageTemplateStatusColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("PageId_Status", "Status");
}

FText FAvaRundownPageTemplateStatusColumn::GetColumnToolTipText() const
{
	return LOCTEXT("PageId_ToolTip", "Page Status");
}

SHeaderRow::FColumn::FArguments FAvaRundownPageTemplateStatusColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnToolTipText())
		.FixedWidth(94.f)
		.ShouldGenerateWidget(true)
		.VAlignCell(EVerticalAlignment::VAlign_Center)
	;
}

TSharedRef<SWidget> FAvaRundownPageTemplateStatusColumn::ConstructRowWidget(const FAvaRundownPageViewRef& InPageView
	, const TSharedPtr<SAvaRundownPageViewRow>& InRow)
{
	const TSharedRef<FAvaRundownTemplatePageViewImpl> TemplatePageView = StaticCastSharedRef<FAvaRundownTemplatePageViewImpl>(
		StaticCastSharedRef<FAvaRundownPageViewImpl>(InPageView)
	);

	const TWeakPtr<FAvaRundownTemplatePageViewImpl> TemplatePageViewWeak = TemplatePageView;
	
	TSharedRef<SHorizontalBox> ButtonList = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(UE::AvaRundown::FEditorMetrics::ColumnLeftOffset, 0.f, 0.f, 0.f)
		[
			SNew(SButton)
			.ButtonStyle(FAvaMediaEditorStyle::Get(), "AvaMediaEditor.BorderlessButton")
			.OnClicked(TemplatePageView, &FAvaRundownTemplatePageViewImpl::OnPreviewButtonClicked)
			.IsEnabled(TemplatePageView, &FAvaRundownTemplatePageViewImpl::CanPreview)
			.ToolTipText(InPageView, &IAvaRundownPageView::GetPreviewInTooltip)
			[
				SNew(SImage)
				.ColorAndOpacity_Static(&FAvaRundownPageTemplateStatusColumn::GetIsPreviewingButtonColor, TemplatePageViewWeak)
				.Image(FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.MediaPreviewing"))
			]
		];

	ButtonList->AddSlot()
		.Padding(UE::AvaRundown::FEditorMetrics::ColumnLeftOffset, 0.f, 0.f, 0.f)
		[
			SNew(SButton)
			.ButtonStyle(FAvaMediaEditorStyle::Get(), "AvaMediaEditor.BorderlessButton")
			.OnClicked(TemplatePageView, &FAvaRundownTemplatePageViewImpl::OnAssetStatusButtonClicked)
			.IsEnabled(TemplatePageView, &FAvaRundownTemplatePageViewImpl::CanChangeAssetStatus)
			.ToolTipText_Static(&FAvaRundownPageTemplateStatusColumn::GetAssetStatusButtonTooltip, TemplatePageViewWeak)
			[
				SNew(SImage)
				.ColorAndOpacity_Static(&FAvaRundownPageTemplateStatusColumn::GetAssetStatusButtonColor, TemplatePageViewWeak)
				.Image(FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.MediaAssetStatus"))
			]
		];

	ButtonList->AddSlot()
		.Padding(UE::AvaRundown::FEditorMetrics::ColumnLeftOffset, 0.f, 0.f, 0.f)
		[
			SNew(SButton)
			.ButtonStyle(FAvaMediaEditorStyle::Get(), "AvaMediaEditor.BorderlessButton")
			.OnClicked(TemplatePageView, &FAvaRundownTemplatePageViewImpl::OnSyncStatusButtonClicked)
			.IsEnabled(TemplatePageView, &FAvaRundownTemplatePageViewImpl::CanChangeSyncStatus)
			.ToolTipText_Static(&FAvaRundownPageTemplateStatusColumn::GetSyncStatusButtonTooltip, TemplatePageViewWeak)
			[
				SNew(SImage)
				.ColorAndOpacity_Static(&FAvaRundownPageTemplateStatusColumn::GetSyncStatusButtonColor, TemplatePageViewWeak)
				.Image(FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.MediaSyncStatus"))
			]
		];

	return ButtonList;
}

FSlateColor FAvaRundownPageTemplateStatusColumn::GetIsPreviewingButtonColor(const TWeakPtr<FAvaRundownTemplatePageViewImpl> InPageViewWeak)
{
	static const FSlateColor Error(FStyleColors::AccentRed.GetSpecifiedColor());
	static const FSlateColor Syncing(FStyleColors::AccentOrange.GetSpecifiedColor());
	static const FSlateColor Loading(FStyleColors::AccentBlue.GetSpecifiedColor());
	static const FSlateColor NotPreviewing(FStyleColors::AccentGray.GetSpecifiedColor());
	static const FSlateColor Previewing(FStyleColors::AccentGreen.GetSpecifiedColor());

	if (const TSharedPtr<FAvaRundownTemplatePageViewImpl> PageView = InPageViewWeak.Pin())
	{
		const UAvaRundown* Rundown = PageView->GetRundown();

		if (IsValid(Rundown))
		{
			const FAvaRundownPage& Page = Rundown->GetPage(PageView->GetPageId());

			if (Page.IsValidPage())
			{
				const TArray<FAvaRundownChannelPageStatus> Statuses = Page.GetPagePreviewStatuses(Rundown);

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Unknown, 
					EAvaRundownPageStatus::Error, 
					EAvaRundownPageStatus::Missing}))
				{
					return Error;
				}

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Syncing}))
				{
					return Syncing;
				}

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Loading}))
				{
					return Loading;
				}

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Previewing}))
				{
					return Previewing;
				}

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Available,
					EAvaRundownPageStatus::Loaded,
					EAvaRundownPageStatus::Playing}))
				{
					return NotPreviewing;
				}
			}
		}
	}

	return Error;
}

FSlateColor FAvaRundownPageTemplateStatusColumn::GetAssetStatusButtonColor(const TWeakPtr<FAvaRundownTemplatePageViewImpl> InPageViewWeak)
{
	static const FSlateColor Error(FStyleColors::AccentRed.GetSpecifiedColor());
	static const FSlateColor Loading(FStyleColors::AccentBlue.GetSpecifiedColor());
	static const FSlateColor Available(FStyleColors::AccentYellow.GetSpecifiedColor());
	static const FSlateColor Loaded(FStyleColors::AccentGreen.GetSpecifiedColor());

	if (const TSharedPtr<FAvaRundownTemplatePageViewImpl> PageView = InPageViewWeak.Pin())
	{
		UAvaRundown* Rundown = PageView->GetRundown();

		if (IsValid(Rundown))
		{
			const FAvaRundownPage& Page = Rundown->GetPage(PageView->GetPageId());

			if (Page.IsValidPage())
			{
				const TArray<FAvaRundownChannelPageStatus> Statuses = Page.GetPagePreviewStatuses(Rundown);

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Unknown, 
					EAvaRundownPageStatus::Error, 
					EAvaRundownPageStatus::Missing}))
				{
					return Error;
				}

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Loading}))
				{
					return Loading;
				}

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Loaded}))
				{
					return Loaded;
				}

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Available}))
				{
					return Available;
				}
			}
		}
	}

	return Error;
}

FText FAvaRundownPageTemplateStatusColumn::GetAssetStatusButtonTooltip(const TWeakPtr<FAvaRundownTemplatePageViewImpl> InPageViewWeak)
{
	if (const TSharedPtr<FAvaRundownTemplatePageViewImpl> PageView = InPageViewWeak.Pin())
	{
		const UAvaRundown* Rundown = PageView->GetRundown();

		if (IsValid(Rundown))
		{
			const FAvaRundownPage& Page = Rundown->GetPage(PageView->GetPageId());

			if (Page.IsValidPage())
			{
				const UEnum* StatusEnum = StaticEnum<EAvaRundownPageStatus>();
				const TArray<FAvaRundownChannelPageStatus> Statuses = Page.GetPagePreviewStatuses(Rundown);
				TArray<FText> StatusTexts;

				for (const FAvaRundownChannelPageStatus& Status : Statuses)
				{
					StatusTexts.Add(StatusEnum->GetDisplayNameTextByValue(static_cast<int64>(Status.Status)));
				}

				if (!StatusTexts.IsEmpty())
				{
					return FText::Format(
						LOCTEXT("StatusInfo", "Status: {0}."),
						FText::Join(LOCTEXT("Separator", ", "), StatusTexts)
					);
				}
			}
		}
	}

	return LOCTEXT("NoStatues", "Status Information Unavailable.");
}

FText FAvaRundownPageTemplateStatusColumn::GetSyncStatusButtonTooltip(const TWeakPtr<FAvaRundownTemplatePageViewImpl> InPageViewWeak)
{
	static FText Error = LOCTEXT("Error", "Storm Sync Error");
	static FText NeedsSync = LOCTEXT("NeedsSync", "Storm Sync Required");
	static FText Syncing = LOCTEXT("Syncing", "Storm Syncing");
	static FText Synced = LOCTEXT("Synced", "Storm Synced");

	if (const TSharedPtr<FAvaRundownTemplatePageViewImpl> PageView = InPageViewWeak.Pin())
	{
		const UAvaRundown* Rundown = PageView->GetRundown();

		if (IsValid(Rundown))
		{
			const FAvaRundownPage& Page = Rundown->GetPage(PageView->GetPageId());

			if (Page.IsValidPage())
			{
				const TArray<FAvaRundownChannelPageStatus> Statuses = Page.GetPagePreviewStatuses(Rundown);

				for (const FAvaRundownChannelPageStatus& Status : Statuses)
				{
					if (Status.bNeedsSync)
					{
						return NeedsSync;
					}
				}

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Syncing}))
				{
					return Syncing;
				}

				return Synced;
			}
		}
	}

	return Error;
}

FSlateColor FAvaRundownPageTemplateStatusColumn::GetSyncStatusButtonColor(
	const TWeakPtr<FAvaRundownTemplatePageViewImpl> InPageViewWeak)
{
	static const FSlateColor Error(FStyleColors::AccentRed.GetSpecifiedColor());
	static const FSlateColor NeedsSync(FStyleColors::AccentYellow.GetSpecifiedColor());
	static const FSlateColor Syncing(FStyleColors::AccentBlue.GetSpecifiedColor());
	static const FSlateColor Synced(FStyleColors::AccentGreen.GetSpecifiedColor());

	if (const TSharedPtr<FAvaRundownTemplatePageViewImpl> PageView = InPageViewWeak.Pin())
	{
		const UAvaRundown* Rundown = PageView->GetRundown();

		if (IsValid(Rundown))
		{
			const FAvaRundownPage& Page = Rundown->GetPage(PageView->GetPageId());

			if (Page.IsValidPage())
			{
				const TArray<FAvaRundownChannelPageStatus> Statuses = Page.GetPagePreviewStatuses(Rundown);

				for (const FAvaRundownChannelPageStatus& Status : Statuses)
				{
					if (Status.bNeedsSync)
					{
						return NeedsSync;
					}
				}

				if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Syncing}))
				{
					return Syncing;
				}

				return Synced;
			}
		}
	}

	return Error;
}

#undef LOCTEXT_NAMESPACE
