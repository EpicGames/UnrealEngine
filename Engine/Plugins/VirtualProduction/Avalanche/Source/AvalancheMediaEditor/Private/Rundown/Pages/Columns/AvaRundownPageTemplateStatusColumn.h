// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/Pages/Columns/IAvaRundownPageViewColumn.h"

class FAvaRundownTemplatePageViewImpl;

class FAvaRundownPageTemplateStatusColumn : public IAvaRundownPageViewColumn
{
public:
	UE_AVA_INHERITS(FAvaRundownPageTemplateStatusColumn, IAvaRundownPageViewColumn);

	//~ Begin IAvaRundownPageViewColumn
	virtual FText GetColumnDisplayNameText() const override;
	virtual FText GetColumnToolTipText() const override;
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual TSharedRef<SWidget> ConstructRowWidget(const FAvaRundownPageViewRef& InPageView, const TSharedPtr<SAvaRundownPageViewRow>& InRow) override;
	//~ End IAvaRundownPageViewColumn

	static FSlateColor GetIsPreviewingButtonColor(const TWeakPtr<FAvaRundownTemplatePageViewImpl> InPageViewWeak);

	static FSlateColor GetAssetStatusButtonColor(const TWeakPtr<FAvaRundownTemplatePageViewImpl> InPageViewWeak);
	static FText GetAssetStatusButtonTooltip(const TWeakPtr<FAvaRundownTemplatePageViewImpl> InPageViewWeak);

	static FText GetSyncStatusButtonTooltip(const TWeakPtr<FAvaRundownTemplatePageViewImpl> InPageViewWeak);
	static FSlateColor GetSyncStatusButtonColor(const TWeakPtr<FAvaRundownTemplatePageViewImpl> InPageViewWeak);
};
