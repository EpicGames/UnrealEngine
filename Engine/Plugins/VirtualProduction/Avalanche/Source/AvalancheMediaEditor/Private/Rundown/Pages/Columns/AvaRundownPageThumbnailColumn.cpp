// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageThumbnailColumn.h"

#include "Slate/SAvaRundownPageThumbnail.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "AvaRundownPageThumbnailColumn"

FText FAvaRundownPageThumbnailColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("PageThumbnail_Name", "Thumbnail");
}

FText FAvaRundownPageThumbnailColumn::GetColumnToolTipText() const
{
	return LOCTEXT("PageThumbnail_Tooltip", "Preview thumbnail of the page, if available");
}

SHeaderRow::FColumn::FArguments FAvaRundownPageThumbnailColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnToolTipText())
		.FixedWidth(70.f)
		.ShouldGenerateWidget(true)
		.VAlignCell(EVerticalAlignment::VAlign_Center);
}

TSharedRef<SWidget> FAvaRundownPageThumbnailColumn::ConstructRowWidget(const FAvaRundownPageViewRef& InPageView
	, const TSharedPtr<SAvaRundownPageViewRow>& InRow)
{
	return SNew(SBox)
		.Padding(UE::AvaRundown::FEditorMetrics::ColumnLeftOffset, 0.f, 0.f, 0.f)
		[
			SNew(SAvaRundownPageThumbnail, InPageView, InRow)
			.ThumbnailWidgetSize(64)
		];
}

#undef LOCTEXT_NAMESPACE
