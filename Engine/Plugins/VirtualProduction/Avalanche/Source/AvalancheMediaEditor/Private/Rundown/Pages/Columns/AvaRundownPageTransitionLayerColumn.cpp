// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageTransitionLayerColumn.h"

#include "AvaMediaEditorStyle.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/Pages/PageViews/IAvaRundownPageView.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AvaRundownPageTransitionLayerColumn"

FText FAvaRundownPageTransitionLayerColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("TransitionLayerColumn_LayerName", "Layer");
}

FText FAvaRundownPageTransitionLayerColumn::GetColumnToolTipText() const
{
	return LOCTEXT("TransitionLayerColumn_ToolTip", "Transition layer name for the page");
}

SHeaderRow::FColumn::FArguments FAvaRundownPageTransitionLayerColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnToolTipText())
		.FillWidth(0.25f)
		.ShouldGenerateWidget(true)
		.VAlignCell(EVerticalAlignment::VAlign_Center)
	;
}

namespace UE::AvaMediaEditor::RundownPageTransitionLayerColumn::Private
{
	static EVisibility GetTransitionModeVisibility(FAvaRundownPageViewWeak InPageViewWeak)
	{
		EVisibility Visibility = EVisibility::Hidden;	//... to preserve alignment.
		if (const FAvaRundownPageViewPtr PageView = InPageViewWeak.Pin())
		{
			const UAvaRundown* Rundown = PageView->GetRundown();
			if (IsValid(Rundown))
			{
				const FAvaRundownPage& Page = Rundown->GetPage(PageView->GetPageId());
				if (Page.IsValidPage())
				{
					// Current logic: show the reuse icon if any of the sub-templates has the reuse mode.
					const int32 NumTemplates = Page.GetNumTemplates(Rundown);
					for (int32 TemplateIndex = 0; TemplateIndex < NumTemplates; ++TemplateIndex)
					{
						if (Page.GetTransitionMode(Rundown, TemplateIndex) == EAvaTransitionInstancingMode::Reuse)
						{
							Visibility = EVisibility::Visible;
							break;
						}
					}
				}
			}
		}
		return Visibility;
	}
}

TSharedRef<SWidget> FAvaRundownPageTransitionLayerColumn::ConstructRowWidget(const FAvaRundownPageViewRef& InPageView, const TSharedPtr<SAvaRundownPageViewRow>& InRow)
{
	using namespace UE::AvaMediaEditor::RundownPageTransitionLayerColumn::Private;
	const FAvaRundownPageViewWeak PageViewWeak = InPageView;
	
	TSharedRef<SHorizontalBox> RowWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(UE::AvaRundown::FEditorMetrics::ColumnLeftOffset, 0.f, 0.f, 0.f)
		[
			SNew(SImage)
			.Image(FAvaMediaEditorStyle::Get().GetBrush("AvaMediaEditor.TransitionModeReuse"))
			.ToolTipText(LOCTEXT("TransitionModeReuseTooltip", "Transition Mode: Reuse - Consecutive pages on the same layer will reuse the same instance."))
			.Visibility_Static(&GetTransitionModeVisibility, PageViewWeak)
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
		+ SHorizontalBox::Slot()
		.Padding(UE::AvaRundown::FEditorMetrics::ColumnLeftOffset, 0.f, 0.f, 0.f)
		[
			SNew(STextBlock)
			.Text(InPageView, &IAvaRundownPageView::GetPageTransitionLayerNameText)
		];
	return RowWidget;
}

#undef LOCTEXT_NAMESPACE
