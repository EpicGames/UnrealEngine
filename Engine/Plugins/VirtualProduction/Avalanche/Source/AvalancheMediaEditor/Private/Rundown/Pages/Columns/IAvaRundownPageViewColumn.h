// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "Rundown/AvaRundownEditorDefines.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Views/SHeaderRow.h"

class FName;
class FText;
class SAvaRundownPageViewRow;
class SWidget;

class IAvaRundownPageViewColumn : public IAvaTypeCastable, public TSharedFromThis<IAvaRundownPageViewColumn>
{
public:
	UE_AVA_INHERITS(IAvaRundownPageViewColumn, IAvaTypeCastable);

	FName GetColumnId() const { return GetTypeId().ToName(); }

	virtual FText GetColumnDisplayNameText() const = 0;
	virtual FText GetColumnToolTipText() const = 0;

	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() = 0;
	
	virtual TSharedRef<SWidget> ConstructRowWidget(const FAvaRundownPageViewRef& InPageView, const TSharedPtr<SAvaRundownPageViewRow>& InRow) = 0;
};
