// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownPageChannelSelector.h"
#include "Broadcast/AvaBroadcast.h"
#include "Internationalization/Text.h"
#include "Rundown/Pages/PageViews/IAvaRundownInstancedPageView.h"
#include "Rundown/Pages/PageViews/AvaRundownInstancedPageViewImpl.h"
#include "Rundown/Pages/PageViews/AvaRundownPageViewImpl.h"
#include "Rundown/Pages/Slate/SAvaRundownPageViewRow.h"
#include "UObject/NameTypes.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

void SAvaRundownPageChannelSelector::Construct(const FArguments& InArgs, const FAvaRundownPageViewRef& InPageView, const TSharedPtr<SAvaRundownPageViewRow>& InRow)
{
	UpdateChannelNames();

	const TSharedRef<IAvaRundownInstancedPageView> InstancedPageView = StaticCastSharedRef<IAvaRundownInstancedPageView>(
		StaticCastSharedRef<FAvaRundownInstancedPageViewImpl>(
			StaticCastSharedRef<FAvaRundownPageViewImpl>(InPageView)
		)
	);
	
	PageViewWeak = InstancedPageView;
	PageViewRowWeak = InRow;
	
	ChildSlot
	[
		SAssignNew(ChannelCombo, SComboBox<FName>)
		.InitiallySelectedItem(InstancedPageView->GetChannelName())
		.OptionsSource(&ChannelNames)
		.OnGenerateWidget(this, &SAvaRundownPageChannelSelector::GenerateChannelWidget)
		.OnSelectionChanged(this, &SAvaRundownPageChannelSelector::OnChannelSelectionChanged)
		.OnComboBoxOpening(this, &SAvaRundownPageChannelSelector::OnComboBoxOpening)
		[
			SNew(STextBlock)
			.Text(this, &SAvaRundownPageChannelSelector::GetCurrentChannelName)
		]
	];
}

TSharedRef<SWidget> SAvaRundownPageChannelSelector::GenerateChannelWidget(FName InChannelName)
{
	return SNew(STextBlock)
		.Text(FText::FromName(InChannelName));
}

void SAvaRundownPageChannelSelector::OnChannelSelectionChanged(SComboBox<FName>::NullableOptionType InProposedSelection, ESelectInfo::Type InSelectInfo)
{
	if (const FAvaRundownInstancedPageViewPtr PageView = PageViewWeak.Pin())
	{
		PageView->SetChannel(InProposedSelection);
	}
}

FText SAvaRundownPageChannelSelector::GetCurrentChannelName() const
{
	if (const FAvaRundownInstancedPageViewPtr PageView = PageViewWeak.Pin())
	{
		return FText::FromName(PageView->GetChannelName());
	}
	return FText::GetEmpty();
}

void SAvaRundownPageChannelSelector::OnComboBoxOpening()
{
	UpdateChannelNames();
	
	const FAvaRundownInstancedPageViewPtr PageView = PageViewWeak.Pin();

	if (!PageView.IsValid())
	{
		return;
	}
	
	check(ChannelCombo.IsValid());
	ChannelCombo->SetSelectedItem(PageView->GetChannelName());
}

void SAvaRundownPageChannelSelector::UpdateChannelNames()
{
	const UAvaBroadcast& AvaBroadcast = UAvaBroadcast::Get();
	ChannelNames.Reset(AvaBroadcast.GetChannelNameCount());
	
	for (int32 ChannelIndex = 0; ChannelIndex < AvaBroadcast.GetChannelNameCount(); ++ChannelIndex)
	{
		const FName ChannelName = AvaBroadcast.GetChannelName(ChannelIndex); 
		if (AvaBroadcast.GetChannelType(ChannelName) == EAvaBroadcastChannelType::Program)
		{
			ChannelNames.Add(ChannelName);
		}
	}
}
