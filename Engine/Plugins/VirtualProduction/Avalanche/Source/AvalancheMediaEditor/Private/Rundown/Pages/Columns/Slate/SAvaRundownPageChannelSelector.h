// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Rundown/AvaRundownEditorDefines.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class IAvaRundownInstancedPageView;
class SAvaRundownPageViewRow;
class FName;
class FText;
class SWidget;

class SAvaRundownPageChannelSelector : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaRundownPageChannelSelector){}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const FAvaRundownPageViewRef& InPageView, const TSharedPtr<SAvaRundownPageViewRow>& InRow);

	TSharedRef<SWidget> GenerateChannelWidget(FName InChannelName);
	
	void OnChannelSelectionChanged(SComboBox<FName>::NullableOptionType InProposedSelection, ESelectInfo::Type InSelectInfo);
	
	FText GetCurrentChannelName() const;
	
	void OnComboBoxOpening();
	
protected:
	void UpdateChannelNames();

	TWeakPtr<IAvaRundownInstancedPageView> PageViewWeak;
	
	TWeakPtr<SAvaRundownPageViewRow> PageViewRowWeak;
	
	TSharedPtr<SComboBox<FName>> ChannelCombo;

	TArray<FName> ChannelNames;

	bool bInEditingMode = false;
};
