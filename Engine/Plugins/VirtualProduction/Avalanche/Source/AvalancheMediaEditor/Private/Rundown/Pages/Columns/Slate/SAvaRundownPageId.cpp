// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownPageId.h"
#include "Internationalization/Text.h"
#include "Rundown/AvaRundown.h"
#include "ScopedTransaction.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaRundownPageId"

void SAvaRundownPageId::Construct(const FArguments& InArgs, const FAvaRundownPageViewPtr& InPageView)
{
	PageViewWeak = InPageView;

	InPageView->GetOnRenumber().AddSP(this, &SAvaRundownPageId::OnRenumberAction);
	
	ChildSlot
	[
		SAssignNew(InlineTextBlock, SInlineEditableTextBlock)
		.Text(InPageView.Get(), &IAvaRundownPageView::GetPageIdText)
		.OnTextCommitted(this, &SAvaRundownPageId::OnTextCommitted)
		.OnVerifyTextChanged(this, &SAvaRundownPageId::OnVerifyTextChanged)
		.OnEnterEditingMode(this, &SAvaRundownPageId::OnEnterEditingMode)
		.OnExitEditingMode(this, &SAvaRundownPageId::OnExitEditingMode)
	];
}

SAvaRundownPageId::~SAvaRundownPageId()
{
	if (PageViewWeak.IsValid())
	{
		PageViewWeak.Pin()->GetOnRenumber().RemoveAll(this);
	}
}

void SAvaRundownPageId::OnRenumberAction(EAvaRundownPageActionState InRenumberAction)
{
	if (InRenumberAction == EAvaRundownPageActionState::Requested)
	{
		check(InlineTextBlock.IsValid());
		InlineTextBlock->EnterEditingMode();
	}
}

void SAvaRundownPageId::OnTextCommitted(const FText& InText, ETextCommit::Type InCommitInfo)
{
	const FAvaRundownPageViewPtr PageView = PageViewWeak.Pin();

	if (!PageView)
	{
		return;
	}
	
	switch (InCommitInfo)
	{
	case ETextCommit::OnEnter:
		//falls through
				
	case ETextCommit::OnUserMovedFocus:
		RenumberPageId(InText, PageView);
		PageView->GetOnRenumber().Broadcast(EAvaRundownPageActionState::Completed);
		break;

	case ETextCommit::Default:
		//falls through
				
	case ETextCommit::OnCleared:
		//falls through
			
	default:
		PageView->GetOnRenumber().Broadcast(EAvaRundownPageActionState::Cancelled);
		break;
	}
}

bool SAvaRundownPageId::OnVerifyTextChanged(const FText& InText, FText& OutErrorMessage)
{
	const FString String = InText.ToString();
	
	if (!String.IsNumeric())
	{
		OutErrorMessage = LOCTEXT("NonNumericError", "Text is not Numeric");
		return false;
	}

	UAvaRundown* const Rundown = PageViewWeak.IsValid()
		? PageViewWeak.Pin()->GetRundown()
		: nullptr;

	if (!Rundown)
	{
		OutErrorMessage = LOCTEXT("NoRundown", "No Rundown found for Item");
		return false;
	}
	
	const int32 Id = FCString::Atoi(*String);

	if (Rundown->GetPage(Id).IsValidPage())
	{
		OutErrorMessage = LOCTEXT("ExistingPageWithId", "A Page already exists with input Id");
		return false;
	}
	
	return true;
}

void SAvaRundownPageId::OnEnterEditingMode()
{
	bInEditingMode = true;
}

void SAvaRundownPageId::OnExitEditingMode()
{
	bInEditingMode = false;
}

void SAvaRundownPageId::RenumberPageId(const FText& InText, const FAvaRundownPageViewPtr& InPageView)
{
	const FString String = InText.ToString();
	check(String.IsNumeric());

	UAvaRundown* const Rundown = InPageView->GetRundown();
	check(Rundown);
	
	const int32 Id = FCString::Atoi(*String);	

	FScopedTransaction Transaction(LOCTEXT("RenumberPageId", "Renumber Page Id"));
	Rundown->Modify();
	
	const bool bResult = Rundown->RenumberPageId(InPageView->GetPageId(), Id);

	if (!bResult)
	{
		Transaction.Cancel();
	}
}

#undef LOCTEXT_NAMESPACE