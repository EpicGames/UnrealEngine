// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/AvaRundownEditorDefines.h"
#include "Rundown/Pages/PageViews/IAvaRundownPageView.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/SCompoundWidget.h"

class SInlineEditableTextBlock;
class FText;

class SAvaRundownPageId : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaRundownPageId){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FAvaRundownPageViewPtr& InPageView);

	virtual ~SAvaRundownPageId() override;

	void OnRenumberAction(EAvaRundownPageActionState InRenumberAction);
	
	void OnTextCommitted(const FText& InText, ETextCommit::Type InCommitInfo);
	
	bool OnVerifyTextChanged(const FText& InText, FText& OutErrorMessage);

	void OnEnterEditingMode();
	
	void OnExitEditingMode();

	void RenumberPageId(const FText& InText, const FAvaRundownPageViewPtr& InPageView);
	
protected:
	TWeakPtr<IAvaRundownPageView> PageViewWeak;
	
	TSharedPtr<SInlineEditableTextBlock> InlineTextBlock;

	bool bInEditingMode = false;
};
