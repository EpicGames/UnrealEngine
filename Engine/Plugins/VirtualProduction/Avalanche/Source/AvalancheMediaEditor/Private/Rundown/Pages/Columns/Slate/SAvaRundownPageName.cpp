// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownPageName.h"
#include "Framework/Application/SlateApplication.h"
#include "Internationalization/Text.h"
#include "Rundown/Pages/PageViews/IAvaRundownPageView.h"
#include "Rundown/Pages/Slate/SAvaRundownPageViewRow.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

void SAvaRundownPageName::Construct(const FArguments& InArgs, const FAvaRundownPageViewPtr& InPageView, const TSharedPtr<SAvaRundownPageViewRow>& InRow)
{
	PageViewWeak = InPageView;
	PageViewRowWeak = InRow;

	InPageView->GetOnRename().AddSP(this, &SAvaRundownPageName::OnRenameAction);
	
	ChildSlot
	[
		SAssignNew(InlineTextBlock, SInlineEditableTextBlock)
		.Text(InPageView.Get(), &IAvaRundownPageView::GetPageDescription)
		.OnTextCommitted(this, &SAvaRundownPageName::OnTextCommitted)
		.OnVerifyTextChanged(this, &SAvaRundownPageName::OnVerifyTextChanged)
		.OnEnterEditingMode(this, &SAvaRundownPageName::OnEnterEditingMode)
		.OnExitEditingMode(this, &SAvaRundownPageName::OnExitEditingMode)
		.IsSelected(InRow.Get(), &SAvaRundownPageViewRow::IsSelectedExclusively)
		.IsReadOnly(this, &SAvaRundownPageName::IsReadOnly)
	];
}

SAvaRundownPageName::~SAvaRundownPageName()
{
	if (PageViewWeak.IsValid())
	{
		PageViewWeak.Pin()->GetOnRename().RemoveAll(this);
	}
}

void SAvaRundownPageName::OnRenameAction(EAvaRundownPageActionState InRenameAction)
{
	if (InRenameAction == EAvaRundownPageActionState::Requested)
	{
		check(InlineTextBlock.IsValid());
		bIsReadOnly = false;	// Only allow entering edit mode from rename action.
		InlineTextBlock->EnterEditingMode();
	}
}

void SAvaRundownPageName::OnTextCommitted(const FText& InText, ETextCommit::Type InCommitInfo)
{
	if (const FAvaRundownPageViewPtr PageView = PageViewWeak.Pin())
	{
		switch (InCommitInfo)
		{
		case ETextCommit::OnEnter:
			//falls through
				
		case ETextCommit::OnUserMovedFocus:
			RenamePage(InText, PageView);
			PageView->GetOnRename().Broadcast(EAvaRundownPageActionState::Completed);
			break;

		case ETextCommit::Default:
			//falls through
				
		case ETextCommit::OnCleared:
			//falls through
			
		default:
			PageView->GetOnRename().Broadcast(EAvaRundownPageActionState::Cancelled);
			break;
		}
	}
}

bool SAvaRundownPageName::OnVerifyTextChanged(const FText& InText, FText& OutErrorMessage)
{
	return true;
}

void SAvaRundownPageName::OnEnterEditingMode()
{
	bInEditingMode = true;
}

void SAvaRundownPageName::OnExitEditingMode()
{
	bInEditingMode = false;
	bIsReadOnly = true;
}

void SAvaRundownPageName::RenamePage(const FText& InText, const FAvaRundownPageViewPtr& InPageView)
{
	check(InPageView.IsValid());

	const bool bRenameSuccessful = InPageView->RenameFriendlyName(InText);
	if (bRenameSuccessful)
	{
		FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Mouse);
	}
}

bool SAvaRundownPageName::IsReadOnly() const
{
	return bIsReadOnly;
}