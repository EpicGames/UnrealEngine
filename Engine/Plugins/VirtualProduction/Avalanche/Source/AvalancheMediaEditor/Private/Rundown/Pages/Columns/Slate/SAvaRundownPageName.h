// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/AvaRundownEditorDefines.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/SCompoundWidget.h"

class FText;
class IAvaRundownPageView;
class SAvaRundownPageViewRow;
class SInlineEditableTextBlock;
enum class EAvaRundownPageActionState : uint8;

class SAvaRundownPageName : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaRundownPageName){}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const FAvaRundownPageViewPtr& InPageView, const TSharedPtr<SAvaRundownPageViewRow>& InRow);
	virtual ~SAvaRundownPageName() override;

	void OnRenameAction(EAvaRundownPageActionState InRenameAction);
	
	void OnTextCommitted(const FText& InText, ETextCommit::Type InCommitInfo);
	bool OnVerifyTextChanged(const FText& InText, FText& OutErrorMessage);

	void OnEnterEditingMode();
	void OnExitEditingMode();

	void RenamePage(const FText& InText, const FAvaRundownPageViewPtr& InPageView);

	bool IsReadOnly() const;
	
protected:
	TWeakPtr<IAvaRundownPageView> PageViewWeak;
	
	TWeakPtr<SAvaRundownPageViewRow> PageViewRowWeak;
	
	TSharedPtr<SInlineEditableTextBlock> InlineTextBlock;

	bool bInEditingMode = false;
	bool bIsReadOnly = true;
};
