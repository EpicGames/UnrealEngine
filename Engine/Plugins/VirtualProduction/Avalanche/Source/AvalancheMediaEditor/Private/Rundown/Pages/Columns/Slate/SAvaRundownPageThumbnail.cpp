// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownPageThumbnail.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "AssetThumbnail.h"
#include "Framework/Application/SlateApplication.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/Pages/PageViews/IAvaRundownPageView.h"
#include "Rundown/Pages/Slate/SAvaRundownPageViewRow.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Widgets/Layout/SScaleBox.h"

void SAvaRundownPageThumbnail::Construct(const FArguments& InArgs, const FAvaRundownPageViewPtr& InPageView, const TSharedPtr<SAvaRundownPageViewRow>& InRow)
{
	PageViewWeak = InPageView;
	PageViewRowWeak = InRow;
	ThumbnailWidgetSize = InArgs._ThumbnailWidgetSize;

	InitThumbnailWidget();

	ChildSlot
	[
		SNew(SBorder)
		.Padding(FMargin(0.0f, 0.0f, 5.0f, 5.0f))
		.BorderImage(FAppStyle::Get().GetBrush("ContentBrowser.AssetTileItem.DropShadow"))
		[
			SNew(SBox)
			.Padding(0.0f)
			.WidthOverride(ThumbnailWidgetSize)
			.HeightOverride(ThumbnailWidgetSize)
			[
				SNew(SScaleBox)
				.Stretch(EStretch::ScaleToFit)
				.StretchDirection(EStretchDirection::Both)
				[
					SNew(SBorder)
					.Padding(0.0f)
					.BorderImage(FAppStyle::Get().GetBrush("ContentBrowser.AssetTileItem.ThumbnailAreaBackground"))
					[
						ThumbnailWidget.ToSharedRef()
					]
				]
			]
		]
	];
}

void SAvaRundownPageThumbnail::OnAssetUpdate(const UAvaRundown* InRundown, const FAvaRundownPage& InPage, EAvaRundownPageChanges InPageChangeType)
{
	if (!EnumHasAnyFlags(InPageChangeType, EAvaRundownPageChanges::Blueprint|EAvaRundownPageChanges::Commands) || !InRundown)
	{
		return;
	}

	if (const TSharedPtr<IAvaRundownPageView> PageView = PageViewWeak.Pin())
	{
		int32 PageIdToCheck = PageView->GetPageId();
		if (!PageView->IsTemplate())
		{
			PageIdToCheck = InRundown->GetPage(PageView->GetPageId()).GetTemplateId();
		}

		if (AssetThumbnail.IsValid() && InPage.IsValidPage() && InPage.IsTemplate() && PageIdToCheck == InPage.GetPageId())
		{
			FSoftObjectPath AssetPath = InPage.GetAssetPath(InRundown);
			
			if (AssetPath.IsNull() && InPage.HasCommands(InRundown))
			{
				AssetPath = FSoftObjectPath(InRundown);	// Using rundown itself as asset thumbnail.
			}

			const FAssetData Data = IAssetRegistry::Get()->GetAssetByObjectPath(AssetPath);

			AssetThumbnail->SetAsset(Data);
			AssetThumbnail->RefreshThumbnail();
			// Just to ensure that the viewport won't turn black on an asset which wasn't loaded yet
			bWasAssetLoaded = false;
		}
	}
}

void SAvaRundownPageThumbnail::InitThumbnailWidget()
{
	if (const TSharedPtr<IAvaRundownPageView> PageView =  PageViewWeak.Pin())
	{
		if (UAvaRundown* Rundown = PageView->GetRundown())
		{
			Rundown->GetOnPagesChanged().AddSP(this, &SAvaRundownPageThumbnail::OnAssetUpdate);

			FAvaRundownPage& Page = Rundown->GetPage(PageView->GetPageId());
			if (!PageView->IsTemplate())
			{
				Page = Rundown->GetPage(Page.GetTemplateId());
			}
			
			FSoftObjectPath AssetPath = Page.GetAssetPath(Rundown);
			if (AssetPath.IsNull() && Page.HasCommands(Rundown))
			{
				AssetPath = FSoftObjectPath(Rundown);	// Using rundown itself as asset thumbnail. avoid the "None".
			}

			const FAssetData Data = IAssetRegistry::Get()->GetAssetByObjectPath(AssetPath);


			AssetThumbnail = MakeShared<FAssetThumbnail>(Data, ThumbnailWidgetSize * 2, ThumbnailWidgetSize * 2, UThumbnailManager::Get().GetSharedThumbnailPool());
			AssetThumbnail->SetRealTime(false);

			ThumbnailWidget = AssetThumbnail->MakeThumbnailWidget();

			Tooltip = FSlateApplication::Get().MakeToolTip(NSLOCTEXT("SAvaRundownPageThumbnail", "SAvaRundownPageThumbnail_Tooltip", "Tooltip"));
			Tooltip->SetContentWidget(SNew(SBox)
					.Padding(0.0f)
					.WidthOverride(ThumbnailWidgetSize * 2)
					.HeightOverride(ThumbnailWidgetSize * 2)
					[
						SNew(SScaleBox)
						.Stretch(EStretch::ScaleToFit)
						.StretchDirection(EStretchDirection::Both)
						[
							AssetThumbnail->MakeThumbnailWidget()
						]
					]);

			ThumbnailWidget->SetToolTip(Tooltip);
		}
	}

	if (!ThumbnailWidget.IsValid())
	{
		ThumbnailWidget = SNullWidget::NullWidget;
	}
}

void SAvaRundownPageThumbnail::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (!bWasAssetLoaded && AssetThumbnail->GetAsset())
	{
		bWasAssetLoaded = true;
		AssetThumbnail->RefreshThumbnail();
	}
}

SAvaRundownPageThumbnail::~SAvaRundownPageThumbnail()
{
	if (const TSharedPtr<IAvaRundownPageView> PageView =  PageViewWeak.Pin())
	{
		if (UAvaRundown* Rundown = PageView->GetRundown())
		{
			Rundown->GetOnPagesChanged().RemoveAll(this);
		}
	}
}
