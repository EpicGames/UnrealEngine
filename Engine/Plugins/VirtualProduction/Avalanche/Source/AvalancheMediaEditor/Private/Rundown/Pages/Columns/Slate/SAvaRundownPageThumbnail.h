// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/AvaRundownEditorDefines.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FAssetThumbnail;
class IToolTip;
class SAvaRundownPageViewRow;
class UAvaRundown;
enum class EAvaRundownPageChanges : uint8;

class SAvaRundownPageThumbnail : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SAvaRundownPageThumbnail)
		: _ThumbnailWidgetSize(64)
	{}

	SLATE_ARGUMENT(int32, ThumbnailWidgetSize)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const FAvaRundownPageViewPtr& InPageView, const TSharedPtr<SAvaRundownPageViewRow>& InRow);
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	~SAvaRundownPageThumbnail() override;

private:
	void OnAssetUpdate(const UAvaRundown* InRundown, const FAvaRundownPage& InPage, EAvaRundownPageChanges InPageChangeType);

	void InitThumbnailWidget();

private:
	TWeakPtr<IAvaRundownPageView> PageViewWeak;

	TWeakPtr<SAvaRundownPageViewRow> PageViewRowWeak;

	TSharedPtr<FAssetThumbnail> AssetThumbnail;

	TSharedPtr<SWidget> ThumbnailWidget;

	TSharedPtr<IToolTip> Tooltip;

	bool bWasAssetLoaded = false;

	int32 ThumbnailWidgetSize = 0;
};
