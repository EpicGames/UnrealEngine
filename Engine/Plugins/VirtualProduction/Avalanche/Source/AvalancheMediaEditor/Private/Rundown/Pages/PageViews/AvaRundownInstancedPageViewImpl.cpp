// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownInstancedPageViewImpl.h"

#include "Broadcast/AvaBroadcast.h"
#include "Framework/Application/SlateApplication.h"
#include "Input/Reply.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownEditorDefines.h"
#include "Rundown/AvaRundownPage.h"

#define LOCTEXT_NAMESPACE "AvaRundownInstancedPageViewImpl"

FAvaRundownInstancedPageViewImpl::FAvaRundownInstancedPageViewImpl(int32 InPageId, UAvaRundown* InRundown, const TSharedPtr<SAvaRundownPageList>& InPageList)
	: FAvaRundownPageViewImpl(InPageId, InRundown, InPageList)
{
}

FReply FAvaRundownInstancedPageViewImpl::OnPlayButtonClicked()
{
	UAvaRundown* Rundown = GetRundown();

	if (IsValid(Rundown))
	{
		const FAvaRundownPage& Page = GetPage();

		if (Page.IsValidPage())
		{
			const TArray<FAvaRundownChannelPageStatus> Statuses = Page.GetPageProgramStatuses(Rundown);
			FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();

			const bool bContinue = KeyState.IsControlDown() || KeyState.IsCommandDown();

			if (bContinue)
			{
				Rundown->ContinuePage(Page.GetPageId(), false);
			}
			else if (FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Playing}))
			{
				Rundown->PlayPage(Page.GetPageId(), EAvaRundownPagePlayType::PlayFromStart, EAvaRundownPagePlaySource::Editor);
			}
			else
			{
				Rundown->PlayPage(Page.GetPageId(), EAvaRundownPagePlayType::PlayFromStart, EAvaRundownPagePlaySource::Editor);
			}

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

bool FAvaRundownInstancedPageViewImpl::CanPlay() const
{
	UAvaRundown* Rundown = GetRundown();

	if (IsValid(Rundown))
	{
		const FAvaRundownPage& Page = GetPage();

		if (Page.IsValidPage())
		{
			const FAvaBroadcastOutputChannel& Channel = UAvaBroadcast::Get().GetCurrentProfile().GetChannel(Page.GetChannelName());

			if (Channel.GetMediaOutputs().Num() == 0)
			{
				return false;
			}

			const TArray<FAvaRundownChannelPageStatus> Statuses = Page.GetPageProgramStatuses(Rundown);

			const bool bHasError = FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Error, EAvaRundownPageStatus::Loading,
				EAvaRundownPageStatus::Missing, EAvaRundownPageStatus::Syncing, EAvaRundownPageStatus::Unknown});

			if (bHasError)
			{
				return false;
			}

			const bool bCanPlay = FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Available, EAvaRundownPageStatus::Loaded,
				EAvaRundownPageStatus::Playing, EAvaRundownPageStatus::NeedsSync});

			if (bCanPlay)
			{
				return true;
			}
		}
	}

	return false;
}

ECheckBoxState FAvaRundownInstancedPageViewImpl::IsEnabled() const
{
	const FAvaRundownPage& Page = GetPage();
	if (Page.IsValidPage() && Page.IsEnabled())
	{
		return  ECheckBoxState::Checked;
	}
	return ECheckBoxState::Unchecked;
}

void FAvaRundownInstancedPageViewImpl::SetEnabled(ECheckBoxState InState)
{
	if (!IsPageSelected())
	{
		SetPageSelection(EAvaRundownPageViewSelectionChangeType::ReplaceSelection);
	}

	PerformWorkOnPages(LOCTEXT("SetEnabled", "Set Enabled"),
		[InState](FAvaRundownPage& InPage)->bool
		{
			InPage.SetEnabled(InState == ECheckBoxState::Checked);
			return true;
		});
}

FName FAvaRundownInstancedPageViewImpl::GetChannelName() const
{
	const FAvaRundownPage& Page = GetPage();
	return Page.IsValidPage() ? Page.GetChannelName() : FName();
}

bool FAvaRundownInstancedPageViewImpl::SetChannel(FName InChannel)
{
	if (!IsPageSelected())
	{
		SetPageSelection(EAvaRundownPageViewSelectionChangeType::ReplaceSelection);
	}

	return PerformWorkOnPages(LOCTEXT("SetChannel", "Set Channel"),
		[this, InChannel](FAvaRundownPage& InPage)->bool
		{
			InPage.SetChannelName(InChannel);
			GetRundown()->GetOnPagesChanged().Broadcast(GetRundown(), InPage, EAvaRundownPageChanges::Channel);
			return true;
		});
}

const FAvaRundownPage& FAvaRundownInstancedPageViewImpl::GetTemplate() const
{
	UAvaRundown* Rundown = RundownWeak.Get();

	if (IsValid(Rundown))
	{
		const FAvaRundownPage& Page = Rundown->GetPage(PageId);

		if (Page.IsValidPage())
		{
			if (Page.IsTemplate())
			{
				return Page;
			}

			const FAvaRundownPage& Template = Rundown->GetPage(Page.GetTemplateId());

			if (Template.IsValidPage())
			{
				return Template;
			}
		}
	}

	return FAvaRundownPage::NullPage;
}

FText FAvaRundownInstancedPageViewImpl::GetTemplateDescription() const
{
	const FAvaRundownPage& Template = GetTemplate();

	if (!Template.IsValidPage())
	{
		return FText::GetEmpty();
	}

	return FText::Format(
		LOCTEXT("TemplateFormat", "{0}: {1}"),
		FText::AsNumber(Template.GetPageId(), &UE::AvaRundown::FEditorMetrics::PageIdFormattingOptions),
		Template.GetPageDescription()
	);
}

bool FAvaRundownInstancedPageViewImpl::IsTemplate() const 
{ 
	return false;
}

bool FAvaRundownInstancedPageViewImpl::IsComboTemplate() const 
{ 
	return false;
}

#undef LOCTEXT_NAMESPACE
