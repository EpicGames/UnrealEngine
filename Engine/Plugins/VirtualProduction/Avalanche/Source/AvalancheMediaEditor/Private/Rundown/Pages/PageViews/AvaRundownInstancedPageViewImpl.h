// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaRundownPageViewImpl.h"
#include "Rundown/Pages/PageViews/IAvaRundownInstancedPageView.h"

class FAvaRundownInstancedPageViewImpl : public FAvaRundownPageViewImpl, public IAvaRundownInstancedPageView
{
public:
	UE_AVA_INHERITS(FAvaRundownInstancedPageViewImpl, FAvaRundownPageViewImpl, IAvaRundownInstancedPageView);

	FAvaRundownInstancedPageViewImpl(int32 InPageId, UAvaRundown* InRundown, const TSharedPtr<SAvaRundownPageList>& InPageList);

	virtual bool IsTemplate() const override;
	virtual bool IsComboTemplate() const override;

	virtual FReply OnPlayButtonClicked() override;
	virtual bool CanPlay() const override;

	virtual ECheckBoxState IsEnabled() const override;
	virtual void SetEnabled(ECheckBoxState InState) override;

	virtual FName GetChannelName() const override;
	virtual bool SetChannel(FName InChannel) override;

	virtual const FAvaRundownPage& GetTemplate() const override;
	virtual FText GetTemplateDescription() const override;
};
