// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageViewImpl.h"

#include "AssetRegistry/AssetData.h"
#include "Framework/Application/SlateApplication.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownEditorDefines.h"
#include "Rundown/AvaRundownPage.h"
#include "Rundown/AvaRundownPageCommand.h"
#include "Rundown/Pages/Slate/SAvaRundownPageList.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "AvaRundownPageViewImpl"

namespace UE::AvaMediaEditor::RundownPageViewImpl
{
	static FString ListSeparator(TEXT(", "));

	void AppendWithSeparator(FString& InDestinationString, const FString& InStringToAdd, const FString& InSeparator = ListSeparator)
	{
		if (!InDestinationString.IsEmpty())
		{
			InDestinationString += InSeparator;
		}
		InDestinationString += InStringToAdd;
	}
}

FAvaRundownPageViewImpl::FAvaRundownPageViewImpl(int32 InPageId, UAvaRundown* InRundown, const TSharedPtr<SAvaRundownPageList>& InPageList)
	: PageId(InPageId)
	, RundownWeak(InRundown)
	, PageListWeak(InPageList)
{
}

UAvaRundown* FAvaRundownPageViewImpl::GetRundown() const
{
	return RundownWeak.Get();
}

int32 FAvaRundownPageViewImpl::GetPageId() const
{
	const FAvaRundownPage& Page = GetPage();
	return Page.IsValidPage()
		? Page.GetPageId()
		: FAvaRundownPage::InvalidPageId;
}

FText FAvaRundownPageViewImpl::GetPageIdText() const
{
	const int32 Id = GetPageId();
	return Id != FAvaRundownPage::InvalidPageId
		? FText::AsNumber(Id, &UE::AvaRundown::FEditorMetrics::PageIdFormattingOptions)
		: LOCTEXT("InvalidIdText", "(invalid)");
}

FText FAvaRundownPageViewImpl::GetPageNameText() const
{
	const FAvaRundownPage& Page = GetPage();

	return Page.IsValidPage()
		? FText::FromString(Page.GetPageName())
		: FText();
}

FText FAvaRundownPageViewImpl::GetPageTransitionLayerNameText() const
{
	using namespace UE::AvaMediaEditor::RundownPageViewImpl;

	if (const UAvaRundown* Rundown = RundownWeak.Get())
	{
		const FAvaRundownPage& Page = GetPage();
		if (Page.IsValidPage())
		{
			FString TransitionLayers;

			// Collect Transition Layers from asset template
			if (Page.HasTransitionLogic(Rundown))
			{
				const int32 NumTemplates = Page.GetNumTemplates(Rundown);
				for (int32 TemplateIndex = 0; TemplateIndex < NumTemplates; ++TemplateIndex)
				{
					AppendWithSeparator(TransitionLayers, Page.GetTransitionLayer(Rundown, TemplateIndex).ToString());
				}
			}

			// Collect Transition Layers from commands.
			Page.ForEachInstancedCommands([&TransitionLayers](const FAvaRundownPageCommand& InCommand, const FAvaRundownPage&)
			{
				const FString CommandLayersString = InCommand.GetTransitionLayerString(ListSeparator);
				if (!CommandLayersString.IsEmpty())
				{
					AppendWithSeparator(TransitionLayers, CommandLayersString);
				}
			}, Rundown, /*bInDirectOnly*/ false); // Traverse templates

			return !TransitionLayers.IsEmpty() ? FText::FromString(TransitionLayers) : LOCTEXT("PageTransitionLayerText_NA", "N/A");
		}
	}

	// Either invalid rundown or invalid page.
	return LOCTEXT("PageTransitionLayerText_Invalid", "(invalid)");
}

FText FAvaRundownPageViewImpl::GetPageSummary() const
{
	const FAvaRundownPage& Page = GetPage();

	return Page.IsValidPage()
		? Page.GetPageSummary()
		: FText();
}

FText FAvaRundownPageViewImpl::GetPageDescription() const
{
	const FAvaRundownPage& Page = GetPage();

	return Page.IsValidPage()
		? Page.GetPageDescription()
		: FText();
}

bool FAvaRundownPageViewImpl::HasObjectPath(const UAvaRundown* InRundown) const
{
	const FAvaRundownPage& Page = GetPage();
	return Page.IsValidPage() && !Page.IsComboTemplate();
}

FSoftObjectPath FAvaRundownPageViewImpl::GetObjectPath(const UAvaRundown* InRundown) const
{
	const FAvaRundownPage& Page = GetPage();
	if (Page.IsValidPage())
	{
		return Page.GetAssetPath(InRundown);
	}

	return FSoftObjectPath();
}

FText FAvaRundownPageViewImpl::GetObjectName(const UAvaRundown* InRundown) const
{
	const FAvaRundownPage& Page = GetPage();
	if (Page.IsValidPage())
	{
		if (Page.ResolveTemplate(InRundown).IsComboTemplate())
		{
			// Since combo templates don't have an asset selector, use same placeholder.
			return LOCTEXT("AssetName_ComboPage_NA", "N/A");
		}
		return FText::FromString(Page.GetAssetPath(InRundown).GetAssetName());
	}
	return FText::GetEmpty();
}

FText FAvaRundownPageViewImpl::GetObjectNames(const UAvaRundown* InRundown) const
{
	using namespace UE::AvaMediaEditor::RundownPageViewImpl;

	const FAvaRundownPage& Page = GetPage();
	if (Page.IsValidPage())
	{
		FString AssetNames;
		const int32 NumTemplates = Page.GetNumTemplates(InRundown);
		for (int32 TemplateIndex = 0; TemplateIndex < NumTemplates; ++TemplateIndex)
		{
			const FSoftObjectPath AssetPath = Page.GetAssetPath(InRundown);
			if (!AssetPath.IsNull())
			{
				AppendWithSeparator(AssetNames, AssetPath.GetAssetName());
			}
		}

		if (!AssetNames.IsEmpty())
		{
			return FText::FromString(AssetNames);
		}
		
		if (Page.HasCommands(InRundown))
		{
			return LOCTEXT("AssetName_CommandPage", "Commands");
		}
	}
	return FText::GetEmpty();
}

void FAvaRundownPageViewImpl::OnObjectChanged(const FAssetData& InAssetData)
{
	if (!IsPageSelected())
	{
		SetPageSelection(EAvaRundownPageViewSelectionChangeType::ReplaceSelection);
	}

	PerformWorkOnPages(LOCTEXT("UpdateAsset", "Update Motion Design Asset"),
		[this, &InAssetData](FAvaRundownPage& InPage)->bool
		{
			if (!InPage.UpdateAsset(InAssetData.GetSoftObjectPath())) 
			{
 				return false;
			}
			GetRundown()->GetOnPagesChanged().Broadcast(GetRundown(), InPage, EAvaRundownPageChanges::Blueprint);
			return true;
		});
}

bool FAvaRundownPageViewImpl::HasCommands(const UAvaRundown* InRundown) const
{
	if (const FAvaRundownPage& Page = GetPage(); Page.IsValidPage())
	{
		return Page.HasCommands(InRundown);
	}
	return false;
}

bool FAvaRundownPageViewImpl::Rename(const FText& InNewName)
{
	UAvaRundown* const Rundown = RundownWeak.Get();
	if (!Rundown)
	{
		return false;
	}

	FAvaRundownPage& Page  = Rundown->GetPage(PageId);
	if (Page.IsValidPage())
	{
		FScopedTransaction Transaction(LOCTEXT("RenamePage", "Rename Page"));
		Rundown->Modify();
		
		Page.Rename(InNewName.ToString());
		Rundown->GetOnPagesChanged().Broadcast(Rundown, Page, EAvaRundownPageChanges::Name);
		return true;
	}
	return false;
}

bool FAvaRundownPageViewImpl::RenameFriendlyName(const FText& InNewName)
{
	UAvaRundown* const Rundown = RundownWeak.Get();
	if (!Rundown)
	{
		return false;
	}

	FAvaRundownPage& Page  = Rundown->GetPage(PageId);
	if (Page.IsValidPage())
	{
		FScopedTransaction Transaction(LOCTEXT("RenamePage", "Rename Page"));
		Rundown->Modify();
		
		Page.RenameFriendlyName(InNewName.ToString());
		Rundown->GetOnPagesChanged().Broadcast(Rundown, Page, EAvaRundownPageChanges::FriendlyName);
		return true;
	}
	return false;
}

FReply FAvaRundownPageViewImpl::OnAssetStatusButtonClicked()
{
	return FReply::Handled();
}

bool FAvaRundownPageViewImpl::CanChangeAssetStatus() const
{
	UAvaRundown* Rundown = RundownWeak.Get();

	if (IsValid(Rundown))
	{
		const FAvaRundownPage& Page = Rundown->GetPage(PageId);

		if (Page.IsValidPage())
		{
			const TArray<FAvaRundownChannelPageStatus> Statuses = Page.GetPageContextualStatuses(Rundown);

			return !FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Error, EAvaRundownPageStatus::Loaded, EAvaRundownPageStatus::Missing,
				EAvaRundownPageStatus::Playing, EAvaRundownPageStatus::Previewing, EAvaRundownPageStatus::Syncing, EAvaRundownPageStatus::Unknown});
		}
	}

	return false;
}

FReply FAvaRundownPageViewImpl::OnPreviewButtonClicked()
{
	UAvaRundown* Rundown = GetRundown();

	if (IsValid(Rundown))
	{
		const FAvaRundownPage& Page = GetPage();

		if (Page.IsValidPage())
		{
			const TArray<FAvaRundownChannelPageStatus> Statuses = Page.GetPagePreviewStatuses(Rundown);
			const bool bIsPreviewing = FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Previewing});
			const int32 ThisPageId = Page.GetPageId();

			// Alt = restart
			// Control = continue
			// Shift = frame
			const bool bFromFrame = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
			const bool bContinue = FSlateApplication::Get().GetModifierKeys().IsControlDown()
				|| FSlateApplication::Get().GetModifierKeys().IsCommandDown();

			const EAvaRundownPagePlayType PreviewType = bFromFrame ? EAvaRundownPagePlayType::PreviewFromFrame : EAvaRundownPagePlayType::PreviewFromStart;

			if (bIsPreviewing && bContinue)
			{
				Rundown->ContinuePage(ThisPageId, true);
			}
			else
			{
				Rundown->PlayPage(ThisPageId, PreviewType, EAvaRundownPagePlaySource::Editor);
			}

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

bool FAvaRundownPageViewImpl::CanPreview() const
{
	UAvaRundown* Rundown = GetRundown();

	if (IsValid(Rundown))
	{
		const FAvaRundownPage& Page = GetPage();

		if (Page.IsValidPage())
		{
			const TArray<FAvaRundownChannelPageStatus> Statuses = Page.GetPagePreviewStatuses(Rundown);
			const bool bIsPreviewing = FAvaRundownPage::StatusesContainsStatus(Statuses, {EAvaRundownPageStatus::Previewing});
			const int32 ThisPageId = Page.GetPageId();

			// Alt = restart
			// Control = continue
			// Shift = frame
			const bool bFromFrame = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
			const bool bContinue = FSlateApplication::Get().GetModifierKeys().IsControlDown()
				|| FSlateApplication::Get().GetModifierKeys().IsCommandDown();
			const bool bRestartPreview = FSlateApplication::Get().GetModifierKeys().IsAltDown();

			if (bIsPreviewing)
			{
				if (bContinue)
				{
					return Rundown->CanContinuePage(ThisPageId, true);
				}
				else
				{
					return Rundown->CanStopPage(ThisPageId, EAvaRundownPageStopOptions::Default, true);

					// Unable to test if we can play as well.
				}
			}
			else
			{
				return Rundown->CanPlayPage(ThisPageId, true);
			}
		}
	}

	return false;
}

FText FAvaRundownPageViewImpl::GetPreviewInTooltip() const
{
	static const FText BaseTooltip = LOCTEXT("Preview_BaseTooltip", "Preview\n\n- Click: Preview from start\n- +Shift: Use Preview Frame\n- +Control: Continue");
	static const FText StatusTooltip = LOCTEXT("Preview_Status", "Preview Status: ");
	static const FText PlayedTooltip = LOCTEXT("Preview_Playing", "Playing");
	static const FText NotPlayedTooltip = LOCTEXT("Preview_Stopped", "Stopped");
	static const FText ErrorTooltip = LOCTEXT("Preview_CantPlay", "**Cannot Preview**");
	static const FText NoOutputs = LOCTEXT("Preview_NoOutputs", "No Outputs Selected");
	static const FText NewLines = LOCTEXT("Preview_NewLines", "\n\n");
	static const FText ExtraTooltip = LOCTEXT("Preview_ExtraTooltip", "Click: Preview from start\n- +Shift: Use Preview Frame\n- +Control: Continue");
	
	if (const UAvaRundown* Rundown = GetRundown(); IsValid(Rundown))
	{
		TArray<FText> Texts;
		Texts.Add(StatusTooltip);

		// Checks whether it can play based on situation, not status
		FString FailureReason;
		if (!Rundown->CanPlayPage(PageId, /*bInPreview*/ true, Rundown->GetDefaultPreviewChannelName(), &FailureReason))
		{
			Texts.Add(ErrorTooltip);
			Texts.Add(FText::Format(LOCTEXT("Preview_CantPlayReason", "Reason: {0}"), FText::FromString(FailureReason)));
		}
		
		// Check actual status
		Texts.Add(Rundown->IsPagePreviewing(PageId) ? PlayedTooltip : NotPlayedTooltip);
		Texts.Add(ExtraTooltip);

		return FText::Join(NewLines, Texts);
	}
	return BaseTooltip;
}

bool FAvaRundownPageViewImpl::IsPageSelected() const
{
	const TSharedPtr<SAvaRundownPageList> PageList = PageListWeak.Pin();
	return PageList.IsValid() ? PageList->GetSelectedPageIds().Contains(PageId) : false;;
}

bool FAvaRundownPageViewImpl::SetPageSelection(EAvaRundownPageViewSelectionChangeType InSelectionChangeType)
{
	TSharedPtr<SAvaRundownPageList> PageList = PageListWeak.Pin();
	if (!PageList.IsValid())
	{
		return false;
	}

	switch (InSelectionChangeType)
	{
		case EAvaRundownPageViewSelectionChangeType::Deselect:
			if (PageList->GetSelectedPageIds().Contains(PageId))
			{
				PageList->DeselectPage(PageId);
			}

			return true;

		case EAvaRundownPageViewSelectionChangeType::AddToSelection:
			if (!PageList->GetSelectedPageIds().Contains(PageId))
			{
				PageList->SelectPage(PageId, false);
			}

			return true;

		case EAvaRundownPageViewSelectionChangeType::ReplaceSelection:
			PageList->DeselectPages();
			PageList->SelectPage(PageId, false);
			return true;

		default:
			// Not possible
			return false;
	}
}

bool FAvaRundownPageViewImpl::PerformWorkOnPages(const FText& InTransactionSessionName, TFunction<bool(FAvaRundownPage&)>&& InWork)
{
	UAvaRundown* const Rundown = RundownWeak.Get();
	if (!Rundown)
	{
		return false;
	}

	FAvaRundownPage& UnderlyingPage  = Rundown->GetPage(PageId);
	if (!UnderlyingPage.IsValidPage())
	{
		return false;
	}

	TSet<FAvaRundownPage*> PagesToPerformWork;
	PagesToPerformWork.Add(&UnderlyingPage);

	if (TSharedPtr<SAvaRundownPageList> PageList = PageListWeak.Pin())
	{
		TConstArrayView<int32> SelectedPageIds = PageList->GetSelectedPageIds();

		if (SelectedPageIds.Contains(UnderlyingPage.GetPageId()))
		{
			for (int32 SelectedPageId : SelectedPageIds)
			{
				FAvaRundownPage& CurrentPage = Rundown->GetPage(SelectedPageId);

				if (CurrentPage.IsValidPage())
				{
					PagesToPerformWork.Add(&CurrentPage);
				}
			}
		}
	}
	
	if (!PagesToPerformWork.IsEmpty())
	{
		FScopedTransaction Transaction(InTransactionSessionName);
		Rundown->Modify();

		int32 WorkDoneCount = 0;
		
		for (FAvaRundownPage* const PageToPerformWork : PagesToPerformWork)
		{
			if (PageToPerformWork && InWork(*PageToPerformWork))
			{
				++WorkDoneCount;
			}
		}

		if (WorkDoneCount == 0)
		{
			Transaction.Cancel();
		}
		return WorkDoneCount > 0;
	}

	return false;
}

const FAvaRundownPage& FAvaRundownPageViewImpl::GetPage() const
{
	if (RundownWeak.IsValid())
	{
		return RundownWeak->GetPage(PageId);
	}
	return FAvaRundownPage::NullPage;
}

#undef LOCTEXT_NAMESPACE
