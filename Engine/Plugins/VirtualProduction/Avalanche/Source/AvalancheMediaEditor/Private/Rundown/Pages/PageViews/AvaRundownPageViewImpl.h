// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/Pages/PageViews/IAvaRundownPageView.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FReply;
class SAvaRundownPageList;
class UAvaRundown;
struct FAssetData;
struct FAvaRundownPage;

class FAvaRundownPageViewImpl : public IAvaRundownPageView
{
public:
	UE_AVA_INHERITS(FAvaRundownPageViewImpl, IAvaRundownPageView);

	FAvaRundownPageViewImpl(int32 InPageId, UAvaRundown* InRundown, const TSharedPtr<SAvaRundownPageList>& InPageList);

	// Fast-path for in-place UI refresh. 
	void RefreshPageId(int32 InPageId)
	{
		PageId = InPageId;
	}

	virtual UAvaRundown* GetRundown() const override;
	
	virtual int32 GetPageId() const override;
	virtual FText GetPageIdText() const override;
	virtual FText GetPageNameText() const override;
	virtual FText GetPageTransitionLayerNameText() const override;
	virtual FText GetPageSummary() const override;
	virtual FText GetPageDescription() const override;

	virtual bool HasObjectPath(const UAvaRundown* InRundown) const override;
	virtual FSoftObjectPath GetObjectPath(const UAvaRundown* InRundown) const override;
	virtual FText GetObjectName(const UAvaRundown* InRundown) const override;
	virtual FText GetObjectNames(const UAvaRundown* InRundown) const override;
	virtual void OnObjectChanged(const FAssetData& InAssetData) override;

	virtual bool HasCommands(const UAvaRundown* InRundown) const override;

	virtual bool Rename(const FText& InNewName) override;
	virtual bool RenameFriendlyName(const FText& InNewName) override;
	virtual FOnPageAction& GetOnRename() override { return OnRename; }
	virtual FOnPageAction& GetOnRenumber() override { return OnRenumber; }

	virtual FReply OnAssetStatusButtonClicked() override;
	virtual bool CanChangeAssetStatus() const override;
	
	virtual FReply OnPreviewButtonClicked() override;
	virtual bool CanPreview() const override;
	virtual FText GetPreviewInTooltip() const override;

	virtual bool IsPageSelected() const override;
	virtual bool SetPageSelection(EAvaRundownPageViewSelectionChangeType InSelectionChangeType) override;

protected:
	/**
	 * Performs the given work on the underlying Page of this Page View
	 * and, if this page is selected, to do this work on the rest of the selected pages too
	 * @param InTransactionSessionName the session name to use for the transaction
	 * @param InWork the function to execute on each page. needs to return true for the work to be considered done
	 * @return whether any work was performed and a transaction was completed 
	 */
	bool PerformWorkOnPages(const FText& InTransactionSessionName, TFunction<bool(FAvaRundownPage&)>&& InWork);
	
	const FAvaRundownPage& GetPage() const;

protected:
	/** The Id used to search the Actual Page from the Rundown */
	int32 PageId;
	
	TWeakObjectPtr<UAvaRundown> RundownWeak;

	TWeakPtr<SAvaRundownPageList> PageListWeak;
	
	FOnPageAction OnRename;
	
	FOnPageAction OnRenumber;

};
