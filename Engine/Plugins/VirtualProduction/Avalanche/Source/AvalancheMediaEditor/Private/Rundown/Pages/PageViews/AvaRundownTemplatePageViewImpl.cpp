// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownTemplatePageViewImpl.h"

#include "Input/Reply.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownPage.h"

FAvaRundownTemplatePageViewImpl::FAvaRundownTemplatePageViewImpl(int32 InPageId, UAvaRundown* InRundown, const TSharedPtr<SAvaRundownPageList>& InPageList)
		: FAvaRundownPageViewImpl(InPageId, InRundown, InPageList)
{
}

UAvaRundown* FAvaRundownTemplatePageViewImpl::GetRundown() const
{ 
	return FAvaRundownPageViewImpl::GetRundown(); 
}

FReply FAvaRundownTemplatePageViewImpl::OnPlayButtonClicked()
{
	return FReply::Handled();
}

bool FAvaRundownTemplatePageViewImpl::CanPlay() const
{
	return false;
}

FReply FAvaRundownTemplatePageViewImpl::OnSyncStatusButtonClicked()
{
	if (!CanChangeSyncStatus())
	{
		return FReply::Unhandled();
	}

	UAvaRundown* Rundown = GetRundown();

	if (IsValid(Rundown))
	{
		const FAvaRundownPage& Page = GetPage();

		if (Page.IsValidPage())
		{
			//TODO: Actual sync

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

bool FAvaRundownTemplatePageViewImpl::CanChangeSyncStatus() const
{
	UAvaRundown* Rundown = GetRundown();

	if (IsValid(Rundown))
	{
		const FAvaRundownPage& Page = GetPage();

		if (Page.IsValidPage())
		{
			TArray<FAvaRundownChannelPageStatus> Statuses = Page.GetPagePreviewStatuses(Rundown);
			bool bNeedsSync = false;

			for (const FAvaRundownChannelPageStatus& Status : Statuses)
			{
				if (Status.bNeedsSync)
				{
					bNeedsSync = true;
					break;
				}
			}

			return bNeedsSync;
		}
	}

	return false;
}

bool FAvaRundownTemplatePageViewImpl::IsTemplate() const 
{ 
	return true;
}

bool FAvaRundownTemplatePageViewImpl::IsComboTemplate() const 
{ 
	const FAvaRundownPage& Page = GetPage();
	return Page.IsValidPage() && Page.IsComboTemplate();
}
