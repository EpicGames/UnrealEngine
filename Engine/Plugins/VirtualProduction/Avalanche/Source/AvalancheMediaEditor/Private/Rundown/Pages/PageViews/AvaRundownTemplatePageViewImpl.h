// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaRundownPageViewImpl.h"
#include "Rundown/Pages/PageViews/IAvaRundownTemplatePageView.h"

class FReply;
struct FAssetData;

class FAvaRundownTemplatePageViewImpl : public FAvaRundownPageViewImpl, public IAvaRundownTemplatePageView
{
public:
	UE_AVA_INHERITS(FAvaRundownTemplatePageViewImpl, FAvaRundownPageViewImpl, IAvaRundownTemplatePageView);

	FAvaRundownTemplatePageViewImpl(int32 InPageId, UAvaRundown* InRundown, const TSharedPtr<SAvaRundownPageList>& InPageList);

	virtual UAvaRundown* GetRundown() const override;

	virtual bool IsTemplate() const override;
	virtual bool IsComboTemplate() const override;
	virtual FReply OnPlayButtonClicked() override;
	virtual bool CanPlay() const override;

	FReply OnSyncStatusButtonClicked();
	bool CanChangeSyncStatus() const;
};
