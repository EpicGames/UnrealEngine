// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"

enum class ECheckBoxState : uint8;
struct FAvaRundownPage;

class IAvaRundownInstancedPageView
{
public:
	UE_AVA_TYPE(IAvaRundownInstancedPageView);

	virtual ~IAvaRundownInstancedPageView() = default;

	virtual ECheckBoxState IsEnabled() const = 0;
	virtual void SetEnabled(ECheckBoxState InState) = 0;

	virtual FName GetChannelName() const = 0;
	virtual bool SetChannel(FName InChannel) = 0;

	virtual const FAvaRundownPage& GetTemplate() const = 0;
	virtual FText GetTemplateDescription() const = 0;
};
