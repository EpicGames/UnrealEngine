// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"

class FReply;
class FText;
class UAvaRundown;
struct FAssetData;
struct FSoftObjectPath;

enum class EAvaRundownPageActionState : uint8
{
	None,
	Requested,
	Completed,
	Cancelled,
};

enum class EAvaRundownPageViewSelectionChangeType : uint8
{
	Deselect,
	ReplaceSelection,
	AddToSelection
};

class IAvaRundownPageView : public IAvaTypeCastable, public TSharedFromThis<IAvaRundownPageView>
{
public:
	UE_AVA_INHERITS(IAvaRundownPageView, IAvaTypeCastable);

	virtual UAvaRundown* GetRundown() const = 0;
	
	virtual int32 GetPageId() const = 0;
	virtual FText GetPageIdText() const = 0;
	virtual FText GetPageNameText() const = 0;
	virtual FText GetPageTransitionLayerNameText() const = 0;

	virtual FText GetPageSummary() const = 0;
	virtual FText GetPageDescription() const = 0;

	virtual bool IsTemplate() const = 0;
	
	/** Returns true if the page is a template, and that template is a combo. */
	virtual bool IsComboTemplate() const = 0;

	virtual bool HasObjectPath(const UAvaRundown* InRundown) const = 0;
	virtual FSoftObjectPath GetObjectPath(const UAvaRundown* InRundown) const = 0;
	/** Returns asset name if single asset template. */
	virtual FText GetObjectName(const UAvaRundown* InRundown) const = 0;
	/** Returns list of all assets: single, combo or commands. */
	virtual FText GetObjectNames(const UAvaRundown* InRundown) const = 0;
	virtual void OnObjectChanged(const FAssetData& InAssetData) = 0;

	/** Returns true if the page has commands, either directly or from it's templates. */
	virtual bool HasCommands(const UAvaRundown* InRundown) const = 0;

	virtual bool Rename(const FText& InNewName) = 0;
	virtual bool RenameFriendlyName(const FText& InNewName) = 0;
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPageAction, EAvaRundownPageActionState);
	virtual FOnPageAction& GetOnRename() = 0;
	virtual FOnPageAction& GetOnRenumber() = 0;

	virtual FReply OnAssetStatusButtonClicked() = 0;
	virtual bool CanChangeAssetStatus() const = 0;

	virtual FReply OnPreviewButtonClicked() = 0;
	virtual bool CanPreview() const = 0;
	virtual FText GetPreviewInTooltip() const = 0;

	virtual FReply OnPlayButtonClicked() = 0;
	virtual bool CanPlay() const = 0;

	virtual bool IsPageSelected() const = 0;
	virtual bool SetPageSelection(EAvaRundownPageViewSelectionChangeType InSelectionChangeType) = 0;
};
