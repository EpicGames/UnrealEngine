// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownInstancedPageList.h"

#include "Misc/PathViews.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownCommands.h"
#include "Rundown/AvaRundownEditor.h"
#include "Rundown/AvaRundownEditorSettings.h"
#include "Rundown/AvaRundownEditorUtils.h"
#include "Rundown/AvaRundownPage.h"
#include "Rundown/AvaRundownPlaybackUtils.h"
#include "Rundown/AvaRundownSerializationUtils.h"
#include "Rundown/Pages/Columns/AvaRundownPageAssetNameColumn.h"
#include "Rundown/Pages/Columns/AvaRundownPageChannelSelectorColumn.h"
#include "Rundown/Pages/Columns/AvaRundownPageEnabledColumn.h"
#include "Rundown/Pages/Columns/AvaRundownPageIdColumn.h"
#include "Rundown/Pages/Columns/AvaRundownPageNameColumn.h"
#include "Rundown/Pages/Columns/AvaRundownPageStatusColumn.h"
#include "Rundown/Pages/Columns/AvaRundownPageTemplateNameColumn.h"
#include "Rundown/Pages/PageViews/AvaRundownInstancedPageViewImpl.h"
#include "Rundown/TabFactories/AvaRundownSubListDocumentTabFactory.h"
#include "SAvaRundownPageList.h"
#include "ScopedTransaction.h"
#include "Styling/StyleColors.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SAvaRundownInstancedPageList"

void SAvaRundownInstancedPageList::PrivateRegisterAttributes(struct FSlateAttributeDescriptor::FInitializer&)
{

}

void SAvaRundownInstancedPageList::Construct(const FArguments& InArgs, TSharedPtr<FAvaRundownEditor> InRundownEditor, const FAvaRundownPageListReference& InPageListReference)
{
	SAvaRundownPageList::Construct(SAvaRundownPageList::FArguments(), InRundownEditor, InPageListReference);

	UAvaRundown* const Rundown = InRundownEditor->GetRundown();
	check(Rundown);

	check(InPageListReference.Type == EAvaRundownPageListType::Instance || Rundown->IsValidSubList(PageListReference));

	Rundown->GetOnPageListChanged().AddSP(this, &SAvaRundownInstancedPageList::OnPageListChanged);
	Rundown->GetOnActiveListChanged().AddSP(this, &SAvaRundownInstancedPageList::OnActiveListChanged);

	if (Rundown->IsValidSubList(PageListReference))
	{
		SearchBar->InsertSlot(0)
			.AutoWidth()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(5.f, 0.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Name", "Name:"))
			];

		SearchBar->InsertSlot(1)
			.AutoWidth()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(5.f, 0.f)
			[
				SNew(SEditableTextBox)
				.HintText(LOCTEXT("PageViewName", "Page View Name"))
				.OnTextCommitted(this, &SAvaRundownInstancedPageList::OnPageViewNameCommitted)
				.Text(this, &SAvaRundownInstancedPageList::GetPageViewName)
				.MinDesiredWidth(150.f)
			];

		SearchBar->InsertSlot(2)
			.AutoWidth()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(5.f, 0.f)
			[
				SNew(SButton)
				.IsFocusable(false)
				.ToolTipText(LOCTEXT("PageViewDeleteToolTip", "Delete Current Page View"))
				.OnClicked(this, &SAvaRundownInstancedPageList::OnDeletePageView)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}

	SearchBar->AddSlot()
		.AutoWidth()
		.VAlign(EVerticalAlignment::VAlign_Center)
		.Padding(5.f, 0.f)
		[
			SNew(SButton)
			.ToolTipText(LOCTEXT("MakeActiveTooltip", "Page List used for Show Controls (Take In, etc.)"))
			.OnClicked(this, &SAvaRundownInstancedPageList::MakeActive)
			.IsEnabled(this, &SAvaRundownInstancedPageList::CanMakeActive)
			.Visibility_Lambda([this]()
			{
				if (TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
				{
					UAvaRundown* const Rundown = RundownEditor->GetRundown();
			
					if (IsValid(Rundown))
					{
						return Rundown->GetSubLists().IsEmpty()? EVisibility::Collapsed : EVisibility::Visible;
					}
				}
				return EVisibility::Collapsed;
			})
			.ButtonColorAndOpacity(this, &SAvaRundownInstancedPageList::GetMakeActiveButtonColor)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ActivePage", "Show-Control Pages"))
			]
		];

	Refresh();
}

SAvaRundownInstancedPageList::~SAvaRundownInstancedPageList()
{
	if (UAvaRundown* const Rundown = GetValidRundown())
	{
		Rundown->GetOnPageListChanged().RemoveAll(this);
		Rundown->GetOnActiveListChanged().RemoveAll(this);
	}
}

void SAvaRundownInstancedPageList::Refresh()
{
	if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		UAvaRundown* const Rundown = RundownEditor->GetRundown();
		check(Rundown);

		if (PageListReference.Type == EAvaRundownPageListType::Instance)
		{
			const TArray<FAvaRundownPage>& Pages = Rundown->GetInstancedPages().Pages;
			const int32 VisiblePageCount = VisiblePageIds.IsEmpty() ? Pages.Num() : VisiblePageIds.Num();
			
			if (PageViews.Num() != VisiblePageCount)
			{
				PageViews.Reset(Pages.Num());

				for (const FAvaRundownPage& Page : Pages)
				{
					if (IsPageVisible(Page))
					{
						PageViews.Emplace(MakeShared<FAvaRundownInstancedPageViewImpl>(Page.GetPageId(), Rundown, SharedThis(this)));
					}
				}
			}
			else
			{
				// Number of page didn't change, just refresh ids.
				int32 PageViewIndex = 0;
				for (const FAvaRundownPage& Page : Pages)
				{
					if (IsPageVisible(Page))
					{
						if (FAvaRundownPageViewImpl* PageView = PageViews[PageViewIndex]->CastTo<FAvaRundownPageViewImpl>())
						{
							PageView->RefreshPageId(Page.GetPageId());
						}
						++PageViewIndex;
					}
				}
			}
		}
		else if (Rundown->IsValidSubList(PageListReference))
		{
			const FAvaRundownSubList& SubList = Rundown->GetSubList(PageListReference);
			const int32 VisiblePageCount = VisiblePageIds.IsEmpty() ? SubList.PageIds.Num() : VisiblePageIds.Num();

			if (PageViews.Num() != VisiblePageCount)
			{
				PageViews.Reset(SubList.PageIds.Num());
			
				for (const int32 PageId : SubList.PageIds)
				{
					if (IsPageVisible(PageId))
					{
						PageViews.Emplace(MakeShared<FAvaRundownInstancedPageViewImpl>(PageId, Rundown, SharedThis(this)));
					}
				}
			}
			else
			{
				// Number of page didn't change, just refresh ids.
				int32 PageViewIndex = 0;
				for (const int32 PageId : SubList.PageIds)
				{
					if (IsPageVisible(PageId))
					{
						if (FAvaRundownPageViewImpl* PageView = PageViews[PageViewIndex]->CastTo<FAvaRundownPageViewImpl>())
						{
							PageView->RefreshPageId(PageId);
						}
						++PageViewIndex;
					}
				}
			}
		}

		PageListView->RequestListRefresh();
	}
}

void SAvaRundownInstancedPageList::CreateColumns()
{
	HeaderRow = SNew(SHeaderRow)
		.Visibility(EVisibility::Visible)
		.CanSelectGeneratedColumn(true);

	Columns.Empty();
	HeaderRow->ClearColumns();

	//TODO: Extensibility?
	TArray<TSharedPtr<IAvaRundownPageViewColumn>> FoundColumns;
	FoundColumns.Add(MakeShared<FAvaRundownPageEnabledColumn>());
	FoundColumns.Add(MakeShared<FAvaRundownPageIdColumn>());
	FoundColumns.Add(MakeShared<FAvaRundownPageTemplateNameColumn>());
	FoundColumns.Add(MakeShared<FAvaRundownPageNameColumn>());
	FoundColumns.Add(MakeShared<FAvaRundownPageAssetNameColumn>());
	FoundColumns.Add(MakeShared<FAvaRundownPageChannelSelectorColumn>());
	FoundColumns.Add(MakeShared<FAvaRundownPageStatusColumn>());

	for (const TSharedPtr<IAvaRundownPageViewColumn>& Column : FoundColumns)
	{
		const FName ColumnId = Column->GetColumnId();
		Columns.Add(ColumnId, Column);
		HeaderRow->AddColumn(Column->ConstructHeaderRowColumn());
		HeaderRow->SetShowGeneratedColumn(ColumnId, false);
	}
}

TSharedPtr<SWidget> SAvaRundownInstancedPageList::OnContextMenuOpening()
{
	if (TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		return GetPageListContextMenu();
	}

	return SNullWidget::NullWidget;
}

void SAvaRundownInstancedPageList::BindCommands()
{
	SAvaRundownPageList::BindCommands();

	//Rundown Commands
	{
		const FAvaRundownCommands& RundownCommands = FAvaRundownCommands::Get();

		CommandList->MapAction(RundownCommands.RemovePage,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::RemoveSelectedPages),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanRemoveSelectedPages));

		CommandList->MapAction(RundownCommands.RenumberPage,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::RenumberSelectedPages),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanRenumberSelectedPages));

		CommandList->MapAction(RundownCommands.ReimportPage,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::ReimportSelectedPage),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanReimportSelectedPage));

		CommandList->MapAction(RundownCommands.EditPageSource,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::EditSelectedPageSource),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanEditSelectedPageSource));

		CommandList->MapAction(RundownCommands.ExportPagesToRundown,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::ExportSelectedPagesToRundown),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanExportSelectedPagesToRundown));

		CommandList->MapAction(RundownCommands.ExportPagesToJson,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::ExportSelectedPagesToExternalFile, TEXT("json")),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanExportSelectedPagesToExternalFile, TEXT("json")));

		CommandList->MapAction(RundownCommands.ExportPagesToXml,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::ExportSelectedPagesToExternalFile, TEXT("xml")),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanExportSelectedPagesToExternalFile, TEXT("xml")));

		CommandList->MapAction(RundownCommands.ResetValuesToDefaults,
			FExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::ResetPagesToDefaults, false),
			FCanExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::CanResetPagesToDefaults, false));

		CommandList->MapAction(RundownCommands.ResetValuesToTemplate,
			FExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::ResetPagesToDefaults, true),
			FCanExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::CanResetPagesToDefaults, true));

		CommandList->MapAction(RundownCommands.Play,
			FExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::PlaySelectedPage),
			FCanExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::CanPlaySelectedPage));

		CommandList->MapAction(RundownCommands.LoadAllPages,
			FExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::LoadAllPages),
			FCanExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::CanLoadAllPages));

		CommandList->MapAction(RundownCommands.LoadNextPages,
			FExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::LoadNextPages),
			FCanExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::CanLoadNextPages));

		CommandList->MapAction(RundownCommands.LoadSelectedPages,
			FExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::LoadSelectedPages),
			FCanExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::CanLoadSelectedPages));

		CommandList->MapAction(RundownCommands.UpdateValues,
			FExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::UpdateValuesOnSelectedPage),
			FCanExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::CanUpdateValuesOnSelectedPage));

		CommandList->MapAction(RundownCommands.Stop,
			FExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::StopSelectedPage, false),
			FCanExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::CanStopSelectedPage, false));

		CommandList->MapAction(RundownCommands.ForceStop,
			FExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::StopSelectedPage, true),
			FCanExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::CanStopSelectedPage, true));

		CommandList->MapAction(RundownCommands.Continue,
			FExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::ContinueSelectedPage),
			FCanExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::CanContinueSelectedPage));

		CommandList->MapAction(RundownCommands.PlayNext,
			FExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::PlayNextPageNoReturn),
			FCanExecuteAction::CreateSP(this, &SAvaRundownInstancedPageList::CanPlayNextPage));

		CommandList->MapAction(RundownCommands.PreviewFrame,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::PreviewPlaySelectedPage, true),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanPreviewPlaySelectedPage));

		CommandList->MapAction(RundownCommands.PreviewPlay,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::PreviewPlaySelectedPage, false),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanPreviewPlaySelectedPage));

		CommandList->MapAction(RundownCommands.PreviewStop,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::PreviewStopSelectedPage, false),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanPreviewStopSelectedPage, false));

		CommandList->MapAction(RundownCommands.PreviewForceStop,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::PreviewStopSelectedPage, true),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanPreviewStopSelectedPage, true));

		CommandList->MapAction(RundownCommands.PreviewContinue,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::PreviewContinueSelectedPage),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanPreviewContinueSelectedPage));

		CommandList->MapAction(RundownCommands.PreviewPlayNext,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::PreviewPlayNextPage),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanPreviewPlayNextPage));

		CommandList->MapAction(RundownCommands.TakeToProgram,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::TakeToProgram),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanTakeToProgram));
	}
}

bool SAvaRundownInstancedPageList::HandleDropAssets(const TArray<FSoftObjectPath>& InAvaAssets, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem)
{
	UAvaRundown* Rundown = GetValidRundown();

	if (!Rundown)
	{
		return false;
	}
	
	if (PageListReference.Type == EAvaRundownPageListType::View && !Rundown->IsValidSubList(PageListReference))
	{
		return false;
	}

	FAvaRundownPageInsertPosition InsertAt = MakeInsertPosition(InDropZone, InItem);
	
	TArray<FSoftObjectPath> NewAvaAssets = InAvaAssets;
	TArray<int32> NewPageIds;

	// If we are adding above, they should be reversed, so the last is added first
	// and the next to last added above that, etc.
	if (InsertAt.IsAddAbove())
	{
		Algo::Reverse(NewAvaAssets);
	}

	bool bHasValidAssets = false;
	Rundown->Modify();

	for (const FSoftObjectPath& AvaAsset : NewAvaAssets)
	{
		if (AvaAsset.IsNull())
		{
			continue;
		}

		// Create Template
		const int32 NewTemplateId = Rundown->AddTemplate();

		if (NewTemplateId == FAvaRundownPage::InvalidPageId)
		{
			continue;
		}

		Rundown->GetPage(NewTemplateId).UpdateAsset(AvaAsset);

		int32 NewInstanceId = FAvaRundownPage::InvalidPageId;

		if (PageListReference.Type == EAvaRundownPageListType::Instance && InsertAt.IsValid())
		{
			NewInstanceId = Rundown->AddPageFromTemplate(NewTemplateId, FAvaRundownPageIdGeneratorParams::FromInsertPosition(InsertAt), InsertAt);
		}
		else
		{
			NewInstanceId = Rundown->AddPageFromTemplate(NewTemplateId);
		}

		if (NewInstanceId == FAvaRundownPage::InvalidPageId)
		{
			continue;
		}

		if (PageListReference.Type == EAvaRundownPageListType::View)
		{
			Rundown->AddPageToSubList(PageListReference, NewInstanceId, InsertAt);
		}

		InsertAt.ConditionalUpdateAdjacentId(NewInstanceId);
		NewPageIds.Add(NewInstanceId);
		bHasValidAssets = true;
	}

	if (bHasValidAssets)
	{
		Refresh();
		DeselectPages();
		SelectPages(NewPageIds);
	}

	return bHasValidAssets;
}

bool SAvaRundownInstancedPageList::HandleDropRundowns(const TArray<FSoftObjectPath>& InRundownPaths, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem)
{
	UAvaRundown* Rundown = GetValidRundown();

	if (!Rundown)
	{
		return false;
	}

	FAvaRundownPageInsertPosition InsertPosition = MakeInsertPosition(InDropZone, InItem);
	
	TArray<FSoftObjectPath> NewRundownPaths = InRundownPaths;
	TArray<int32> NewPageIds;
	
	// If we are adding above, they should be reversed, so the last is added first
	// and the next to last added above that, etc.
	if (!InsertPosition.bAddBelow)
	{
		Algo::Reverse(NewRundownPaths);
	}
	
	Rundown->Modify();

	for (const FSoftObjectPath& RundownPath : NewRundownPaths)
	{
		if (RundownPath.IsNull())
		{
			continue;
		}

		if (const UAvaRundown* SourceRundown = Cast<UAvaRundown>(RundownPath.TryLoad()))
		{
			TArray<int32> ImportedPageIds = UE::AvaRundownEditor::Utils::ImportInstancedPagesFromRundown(Rundown, SourceRundown, InsertPosition);
			if (!ImportedPageIds.IsEmpty())
			{
				NewPageIds.Append(ImportedPageIds);
				InsertPosition.ConditionalUpdateAdjacentId(NewPageIds.Last());	// todo: last or first, depends on AddBelow.
			}
		}
	}

	if (!NewPageIds.IsEmpty())
	{
		Refresh();
		DeselectPages();
		SelectPages(NewPageIds);
	}

	return !NewPageIds.IsEmpty();
}

bool SAvaRundownInstancedPageList::HandleDropPageIds(const FAvaRundownPageListReference& InPageListReference, const TArray<int32>& InPageIds, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem)
{
	// Nothing to do.
	if (InPageIds.IsEmpty())
	{
		return true;
	}
	
	UAvaRundown* Rundown = GetValidRundown();

	if (!Rundown)
	{
		return false;
	}

	const bool bFromTemplates = InPageListReference.Type == EAvaRundownPageListType::Template;
	const bool bFromMainList = InPageListReference.Type == EAvaRundownPageListType::Instance;
	const bool bFromSubList = Rundown->IsValidSubList(InPageListReference);

	if (!bFromTemplates && !bFromMainList && !bFromSubList)
	{
		return false;
	}

	const bool bToMainList = PageListReference.Type == EAvaRundownPageListType::Instance;
	const bool bToSubList = Rundown->IsValidSubList(PageListReference);

	if (!bToMainList && !bToSubList)
	{
		return false;
	}

	FScopedTransaction DropTransaction(LOCTEXT("DropTransaction", "Drop Pages"));
	Rundown->Modify();
	bool bDidSomething = false;

	if (bToMainList)
	{
		if (bFromTemplates)
		{
			bDidSomething = HandleDropPageIdsOnMainListFromTemplates(InPageIds, InDropZone, InItem);
		}
		else if (bFromMainList)
		{
			bDidSomething = HandleDropPageIdsOnMainListFromMainList(InPageIds, InDropZone, InItem);
		}
		// Cannot drop from sub list to main list
	}
	else if (bToSubList)
	{
		if (bFromTemplates)
		{
			bDidSomething = HandleDropPageIdsOnSubListFromTemplates(InPageIds, InDropZone, InItem);
		}
		else if (bFromMainList)
		{
			bDidSomething = HandleDropPageIdsOnSubListFromMainList(InPageIds, InDropZone, InItem);
		}
		else if (bFromSubList)
		{
			bDidSomething = HandleDropPageIdsOnSubListFromSubList(InPageListReference, InPageIds, InDropZone, InItem);
		}
	}

	if (!bDidSomething)
	{
		// We didn't do anything.
		DropTransaction.Cancel();
	}

	return bDidSomething;
}

bool SAvaRundownInstancedPageList::HandleDropPageIdsOnMainListFromTemplates(const TArray<int32>& InPageIds, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem)
{
	if (PageListReference.Type != EAvaRundownPageListType::Instance)
	{
		return false;
	}

	// Nothing to do.
	if (InPageIds.IsEmpty())
	{
		return true;
	}

	UAvaRundown* Rundown = GetValidRundown();

	if (!Rundown || !Rundown->CanAddPage())
	{
		return false;
	}

	FAvaRundownPageInsertPosition InsertAt = MakeInsertPosition(InDropZone, InItem);	

	if (InsertAt.IsValid())
	{
		const int32 DroppedOnPageIndex = Rundown->GetInstancedPages().GetPageIndex(InsertAt.AdjacentId);

		if (DroppedOnPageIndex == INDEX_NONE)
		{
			InsertAt.AdjacentId = FAvaRundownPage::InvalidPageId;
		}
	}
	
	TArray<int32> ActualPageIds = InPageIds;
	TArray<int32> NewPageIds;
	NewPageIds.Reserve(InPageIds.Num());

	if (InsertAt.IsAddAbove())
	{
		Algo::Reverse(ActualPageIds);
	}

	for (const int32 PageId : ActualPageIds)
	{
		int32 NewInstanceId = Rundown->AddPageFromTemplate(PageId, FAvaRundownPageIdGeneratorParams::FromInsertPosition(InsertAt), InsertAt);

		if (NewInstanceId != FAvaRundownPage::InvalidPageId)
		{
			NewPageIds.Add(NewInstanceId);
			InsertAt.ConditionalUpdateAdjacentId(NewInstanceId);
		}
	}

	if (NewPageIds.IsEmpty())
	{
		return false;
	}

	SelectPages(NewPageIds, true);

	return true;
}

bool SAvaRundownInstancedPageList::HandleDropExternalFiles(const TArray<FString>& InFiles, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem)
{
	UAvaRundown* Rundown = GetValidRundown();

	if (!Rundown)
	{
		return false;
	}
	
	FAvaRundownPageInsertPosition InsertAt = MakeInsertPosition(InDropZone, InItem);
	
	TArray<FString> NewFiles = InFiles;
	TArray<int32> NewPageIds;

	// If we are adding above, they should be reversed, so the last is added first
	// and the next to last added above that, etc.
	if (InsertAt.IsAddAbove())
	{
		Algo::Reverse(NewFiles);
	}
	
	Rundown->Modify();
	
	for (const FString& File : InFiles)
	{
		if (FPathViews::GetExtension(File).Equals(TEXT("json"), ESearchCase::IgnoreCase))
		{
			using namespace UE::AvaRundownEditor::Utils;
			using namespace UE::AvaMedia::RundownSerializationUtils;
			const TStrongObjectPtr<UAvaRundown> TmpRundown(NewObject<UAvaRundown>());
			FText ErrorMessage;
			
			if (LoadRundownFromJson(TmpRundown.Get(), *File, ErrorMessage))
			{
				if (!TmpRundown->GetInstancedPages().Pages.IsEmpty())
				{
					TArray<int32> ImportedPageIds = ImportInstancedPagesFromRundown(GetRundown(), TmpRundown.Get(), InsertAt);
					if (!ImportedPageIds.IsEmpty())
					{
						NewPageIds.Append(ImportedPageIds);
						InsertAt.ConditionalUpdateAdjacentId(NewPageIds.Last());	// todo: last or first, depends on AddBelow.
					}
					else
					{
						UE_LOGF(LogAvaRundown, Error, "Failed to merge %ls in current rundown.", *File);	
					}
				}
				else
				{
					UE_LOGF(LogAvaRundown, Warning, "Merging %ls has no instanced pages to merge.", *File);
				}
			}
			else
			{
				UE_LOGF(LogAvaRundown, Error, "Failed to load rundown from file \"%ls\". Reason: %ls.", *File, *ErrorMessage.ToString());
			}
		}
		else
		{
			UE_LOGF(LogAvaRundown, Error, "%ls is not a supported file type. Only rundowns in \"json\" format are supported.", *File);
		}
	}
	
	if (!NewPageIds.IsEmpty())
	{
		Refresh();
		DeselectPages();
		SelectPages(NewPageIds);
	}

	return !NewPageIds.IsEmpty();
}

bool SAvaRundownInstancedPageList::HandleDropPageIdsOnMainListFromMainList(const TArray<int32>& InPageIds, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem)
{
	if (PageListReference.Type != EAvaRundownPageListType::Instance)
	{
		return false;
	}

	// Nothing to do.
	if (InPageIds.IsEmpty())
	{
		return true;
	}

	UAvaRundown* Rundown = GetValidRundown();

	if (!Rundown || !Rundown->CanChangePageOrder())
	{
		return false;
	}

	const FAvaRundownPageCollection& InstancedPageCollection = Rundown->GetInstancedPages();
	int32 DroppedOnPageIndex = INDEX_NONE;

	if (InItem.IsValid() && InstancedPageCollection.PageIndices.Contains(InItem->GetPageId()))
	{
		DroppedOnPageIndex = InstancedPageCollection.PageIndices[InItem->GetPageId()];

		// Nothing to do.
		if (InPageIds.Num() == 1 && DroppedOnPageIndex == InPageIds[0])
		{
			return true;
		}
	}

	TArray<int32> MovedPageIdIndices;
	TArray<int32> NewPageOrder;
	TArray<int32> NewSelectedIds;

	MovedPageIdIndices.Reserve(InPageIds.Num());
	NewPageOrder.Reserve(PageViews.Num());
	NewSelectedIds.Reserve(InPageIds.Num());

	// This has the byproduct of removing invalid ids.
	for (int32 PageId : InPageIds)
	{
		if (const int32* PageIndexPtr = InstancedPageCollection.PageIndices.Find(PageId))
		{
			MovedPageIdIndices.Add(*PageIndexPtr);
			NewSelectedIds.Add(PageId);
		}
	}

	// Nothing to do.
	if (MovedPageIdIndices.IsEmpty())
	{
		return true;
	}

	for (int32 PageViewIndex = 0; PageViewIndex < PageViews.Num(); ++PageViewIndex)
	{
		const bool bHasMovedThisPage = MovedPageIdIndices.Contains(PageViewIndex);

		if (PageViewIndex == DroppedOnPageIndex)
		{
			const bool bAddBefore = InDropZone == EItemDropZone::AboveItem;

			if (bAddBefore)
			{
				NewPageOrder.Append(MovedPageIdIndices);
			}

			// If we moved the page we dropped onto, it will already be in the list
			if (!bHasMovedThisPage)
			{
				NewPageOrder.Add(PageViewIndex);
			}

			if (!bAddBefore)
			{
				NewPageOrder.Append(MovedPageIdIndices);
			}

			continue;
		}

		if (bHasMovedThisPage)
		{
			continue;
		}

		NewPageOrder.Add(PageViewIndex);
	}

	Rundown->ChangePageOrder(UAvaRundown::InstancePageList, NewPageOrder);
	SelectPages(NewSelectedIds, true);

	return true;
}

bool SAvaRundownInstancedPageList::HandleDropPageIdsOnSubListFromTemplates(const TArray<int32>& InPageIds, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem)
{
	if (PageListReference.Type != EAvaRundownPageListType::View)
	{
		return false;
	}

	// Nothing to do.
	if (InPageIds.IsEmpty())
	{
		return true;
	}

	UAvaRundown* Rundown = GetValidRundown();

	if (!Rundown || !Rundown->CanAddPage() || !Rundown->IsValidSubList(PageListReference))
	{
		return false;
	}

	FAvaRundownPageInsertPosition InsertAt = MakeInsertPosition(InDropZone, InItem);
	
	if (InsertAt.IsValid())
	{
		// Make sure the sublist has the dropped on item. (It should)
		const FAvaRundownSubList& ToSubList = Rundown->GetSubList(PageListReference);
		if (!ToSubList.PageIds.Contains(InsertAt.AdjacentId))
		{
			InsertAt.AdjacentId = FAvaRundownPage::InvalidPageId;
		}
	}
	
	TArray<int32> ActualPageIds = InPageIds;
	TArray<int32> NewPageIds;
	NewPageIds.Reserve(InPageIds.Num());

	if (InsertAt.IsAddAbove())
	{
		Algo::Reverse(ActualPageIds);
	}

	for (const int32 PageId : ActualPageIds)
	{
		const int32 NewInstanceId = Rundown->AddPageFromTemplate(PageId, FAvaRundownPageIdGeneratorParams::FromInsertPosition(InsertAt), InsertAt);

		if (NewInstanceId != FAvaRundownPage::InvalidPageId)
		{
			NewPageIds.Add(NewInstanceId);
		}
	}

	if (NewPageIds.IsEmpty())
	{
		return false;
	}

	for (const int32 PageId : NewPageIds)
	{
		if (Rundown->AddPageToSubList(PageListReference, PageId, InsertAt))
		{
			InsertAt.ConditionalUpdateAdjacentId(PageId);
		}
	}

	SelectPages(NewPageIds, true);

	return true;
}

bool SAvaRundownInstancedPageList::HandleDropPageIdsOnSubListFromMainList(const TArray<int32>& InPageIds, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem)
{
	if (PageListReference.Type != EAvaRundownPageListType::View)
	{
		return false;
	}

	// Nothing to do.
	if (InPageIds.IsEmpty())
	{
		return true;
	}

	UAvaRundown* Rundown = GetValidRundown();

	if (!Rundown || !Rundown->CanAddPage() || !Rundown->IsValidSubList(PageListReference))
	{
		return false;
	}

	FAvaRundownSubList& ToSubList = Rundown->GetSubList(PageListReference);

	FAvaRundownPageInsertPosition InsertAt = MakeInsertPosition(InDropZone, InItem);
	
	int32 DroppedOnPageIndex = INDEX_NONE;

	if (InsertAt.IsValid())
	{
		DroppedOnPageIndex = ToSubList.PageIds.IndexOfByKey(InsertAt.AdjacentId);
		if (DroppedOnPageIndex == INDEX_NONE)
		{
			InsertAt.AdjacentId = FAvaRundownPage::InvalidPageId;
		}
	}

	TArray<int32> ActualPageIds;
	ActualPageIds.Reserve(InPageIds.Num());

	// Do not add pages already in this sub list
	for (int32 PageId : InPageIds)
	{
		if (!ToSubList.PageIds.Contains(PageId))
		{
			ActualPageIds.Add(PageId);
		}
	}

	if (InsertAt.IsAddAbove())
	{
		Algo::Reverse(ActualPageIds);
	}

	if (InsertAt.IsValid() && InPageIds.Contains(InsertAt.AdjacentId))
	{
		// If we have a valid index and we're adding above, iterate up until we find one we're not removing
		if (DroppedOnPageIndex != INDEX_NONE && InsertAt.IsAddAbove())
		{
			for (DroppedOnPageIndex = DroppedOnPageIndex - 1; DroppedOnPageIndex > INDEX_NONE; --DroppedOnPageIndex)
			{
				if (InPageIds.Contains(ToSubList.PageIds[DroppedOnPageIndex]) == false)
				{
					break;
				}
			}
		}

		// If we have no valid index, iterate from the end of the array until we find out that we aren't removing
		if (DroppedOnPageIndex == INDEX_NONE)
		{
			for (DroppedOnPageIndex = ToSubList.PageIds.Num() - 1; DroppedOnPageIndex > INDEX_NONE; --DroppedOnPageIndex)
			{
				if (InPageIds.Contains(ToSubList.PageIds[DroppedOnPageIndex]) == false)
				{
					break;
				}
			}
		}
		else if (InsertAt.IsAddBelow())
		{
			for (DroppedOnPageIndex = DroppedOnPageIndex + 1; DroppedOnPageIndex < ToSubList.PageIds.Num(); ++DroppedOnPageIndex)
			{
				if (InPageIds.Contains(ToSubList.PageIds[DroppedOnPageIndex]) == false)
				{
					break;
				}
			}
		}

		// If we still have no valid index, just forget about dropping on a page
		if (DroppedOnPageIndex == INDEX_NONE || !ToSubList.PageIds.IsValidIndex(DroppedOnPageIndex))
		{
			InsertAt.AdjacentId = FAvaRundownPage::InvalidPageId;
		}
		// Make sure our id is correct.
		else
		{
			InsertAt.AdjacentId = ToSubList.PageIds[DroppedOnPageIndex];
		}
	}

	for (const int32 PageId : ActualPageIds)
	{
		Rundown->AddPageToSubList(PageListReference, PageId, InsertAt);
		InsertAt.ConditionalUpdateAdjacentId(PageId);
	}

	SelectPages(ActualPageIds, true);

	return true;
}

bool SAvaRundownInstancedPageList::HandleDropPageIdsOnSubListFromSubList(const FAvaRundownPageListReference& InFromList, const TArray<int32>& InPageIds, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem)
{
	if (PageListReference.Type != EAvaRundownPageListType::View)
	{
		return false;
	}

	UAvaRundown* Rundown = GetValidRundown();
	if (!Rundown || !Rundown->IsValidSubList(PageListReference) || !Rundown->IsValidSubList(InFromList))
	{
		return false;
	}

	if (PageListReference == InFromList)
	{
		if (!Rundown->CanChangePageOrder())
		{
			return false;
		}
	}
	else
	{
		if (!Rundown->CanAddPage())
		{
			return false;
		}
	}

	FAvaRundownSubList& ToSubList = Rundown->GetSubList(PageListReference);

	FAvaRundownPageInsertPosition InsertAt = MakeInsertPosition(InDropZone, InItem);
	
	int32 DroppedOnPageIndex = INDEX_NONE;

	if (InsertAt.IsValid())
	{
		DroppedOnPageIndex = ToSubList.PageIds.IndexOfByKey(InsertAt.AdjacentId);

		if (DroppedOnPageIndex == INDEX_NONE)
		{
			InsertAt.AdjacentId = FAvaRundownPage::InvalidPageId;
		}
	}

	TArray<int32> ActualPageIds;
	ActualPageIds.Reserve(InPageIds.Num());

	// Nothing to do
	if (PageListReference == InFromList)
	{
		if (ToSubList.PageIds.Num() <= 1)
		{
			return true;
		}

		// Remove and re-add all pages
		ActualPageIds = InPageIds;
	}
	else
	{
		// Do not add pages already in this sub list
		for (int32 PageId : InPageIds)
		{
			if (!ToSubList.PageIds.Contains(PageId))
			{
				ActualPageIds.Add(PageId);
			}
		}
	}

	// Nothing to do
	if (ActualPageIds.IsEmpty())
	{
		return true;
	}

	if (InsertAt.IsAddAbove())
	{
		Algo::Reverse(ActualPageIds);
	}

	if (InsertAt.IsValid() && InPageIds.Contains(InsertAt.AdjacentId))
	{
		// If we have a valid index and we're adding above, iterate up until we find one we're not removing
		if (DroppedOnPageIndex != INDEX_NONE && InsertAt.IsAddAbove())
		{
			for (DroppedOnPageIndex = DroppedOnPageIndex - 1; DroppedOnPageIndex > INDEX_NONE; --DroppedOnPageIndex)
			{
				if (InPageIds.Contains(ToSubList.PageIds[DroppedOnPageIndex]) == false)
				{
					break;
				}
			}
		}

		// If we have no valid index, iterate from the end of the array until we find out that we aren't removing
		if (DroppedOnPageIndex == INDEX_NONE)
		{
			for (DroppedOnPageIndex = ToSubList.PageIds.Num() - 1; DroppedOnPageIndex > INDEX_NONE; --DroppedOnPageIndex)
			{
				if (InPageIds.Contains(ToSubList.PageIds[DroppedOnPageIndex]) == false)
				{
					break;
				}
			}
		}
		else if (InsertAt.IsAddBelow())
		{
			for (DroppedOnPageIndex = DroppedOnPageIndex + 1; DroppedOnPageIndex < ToSubList.PageIds.Num(); ++DroppedOnPageIndex)
			{
				if (InPageIds.Contains(ToSubList.PageIds[DroppedOnPageIndex]) == false)
				{
					break;
				}
			}
		}

		// If we still have no valid index, just forget about dropping on a page
		if (DroppedOnPageIndex == INDEX_NONE || !ToSubList.PageIds.IsValidIndex(DroppedOnPageIndex))
		{
			InsertAt.AdjacentId = FAvaRundownPage::InvalidPageId;
		}
		// Make sure our id is correct.
		else
		{
			InsertAt.AdjacentId = ToSubList.PageIds[DroppedOnPageIndex];
		}
	}

	Rundown->RemovePagesFromSubList(InFromList, InPageIds);

	for (const int32 PageId : ActualPageIds)
	{
		Rundown->AddPageToSubList(PageListReference, PageId, InsertAt);
		InsertAt.ConditionalUpdateAdjacentId(PageId);
	}

	SelectPages(ActualPageIds, true);

	return true;
}

void SAvaRundownInstancedPageList::OnTabActivated(TSharedRef<SDockTab> InDockTab, ETabActivationCause InActivationCause)
{
	MakeActive();
}

void SAvaRundownInstancedPageList::PlaySelectedPage() const
{
	if (UAvaRundown* Rundown = GetRundown())
	{
		const TArray<int32> PageIds = GetPagesToTakeIn();
		Rundown->PlayPages(PageIds, EAvaRundownPagePlayType::PlayFromStart, EAvaRundownPagePlaySource::Editor);
	}
}

bool SAvaRundownInstancedPageList::CanPlaySelectedPage() const
{
	const TArray<int32> PageIds = GetPagesToTakeIn();
	return !PageIds.IsEmpty();
}

bool SAvaRundownInstancedPageList::CanLoadAllPages() const
{
	return CanLoadPages(GetAllPageIds());
}

void SAvaRundownInstancedPageList::LoadAllPages()
{
	LoadPages(GetAllPageIds());
}

bool SAvaRundownInstancedPageList::CanLoadNextPages() const
{
	const int32 NextPageId = GetNextPageIdToLoad();
	return CanLoadPages(MakeArrayView(&NextPageId, 1));
}

void SAvaRundownInstancedPageList::LoadNextPages()
{
	const int32 NextPageId = GetNextPageIdToLoad();
	LoadPages(MakeArrayView(&NextPageId, 1));
}

bool SAvaRundownInstancedPageList::CanLoadSelectedPages() const
{
	return CanLoadPages(SelectedPageIds);
}

void SAvaRundownInstancedPageList::LoadSelectedPages()
{
	LoadPages(SelectedPageIds);
}

bool SAvaRundownInstancedPageList::CanLoadPages(TConstArrayView<int32> InPageIds) const
{
	const UAvaRundown* const Rundown = GetRundown();
	if (!Rundown)
	{
		return false;
	}

	constexpr bool bIsPreview = false;

	for (int32 PageId : InPageIds)
	{
		if (Rundown->CanPlayPage(PageId, bIsPreview))
		{
			return true;
		}
	}
	return false;
}

void SAvaRundownInstancedPageList::LoadPages(TConstArrayView<int32> InPageIds)
{
	UAvaRundown* const Rundown = GetRundown();
	if (!Rundown)
	{
		return;
	}

	constexpr bool bIsPreview = false;
	const FName PreviewChannel = NAME_None;

	IAvaRundownPageLoadingManager& PageLoadingManager = Rundown->GetPageLoadingManager();
	for (int32 PageId : InPageIds)
	{
		if (Rundown->CanPlayPage(PageId, bIsPreview))
		{
			PageLoadingManager.RequestLoadPage(PageId, bIsPreview, PreviewChannel);
		}
	}
}

bool SAvaRundownInstancedPageList::CanUpdateValuesOnSelectedPage() const
{
	const TArray<int32> PageIds = GetPagesToUpdate();
	return !PageIds.IsEmpty();
}

void SAvaRundownInstancedPageList::UpdateValuesOnSelectedPage()
{
	if (const UAvaRundown* Rundown = GetRundown())
	{
		const TArray<int32> PageIds = GetPagesToUpdate();
		for (const int32 PageId : PageIds)
		{
			Rundown->PushRuntimeRemoteControlValues(PageId, false);
		}
	}
}

void SAvaRundownInstancedPageList::ContinueSelectedPage() const
{
	if (UAvaRundown* Rundown = GetRundown())
	{
		const TArray<int32> PageIds = GetPagesToContinue();
		for (const int32 PageId : PageIds)
		{
			Rundown->ContinuePage(PageId, false);
		}
	}
}

bool SAvaRundownInstancedPageList::CanContinueSelectedPage() const
{
	const TArray<int32> PageIds = GetPagesToContinue();
	return !PageIds.IsEmpty();
}

void SAvaRundownInstancedPageList::StopSelectedPage(bool bInForce) const
{
	if (UAvaRundown* Rundown = GetRundown())
	{
		const EAvaRundownPageStopOptions StopOptions = bInForce ? EAvaRundownPageStopOptions::ForceNoTransition : EAvaRundownPageStopOptions::Default;
		const TArray<int32> PageIds = GetPagesToTakeOut(bInForce);
		Rundown->StopPages(PageIds, StopOptions, false);
	}
}

bool SAvaRundownInstancedPageList::CanStopSelectedPage(bool bInForce) const
{
	const TArray<int32> PageIds = GetPagesToTakeOut(bInForce);
	return !PageIds.IsEmpty();
}

TArray<int32> SAvaRundownInstancedPageList::PlayNextPage() const
{
	TArray<int32> NextPageIds;
	if (UAvaRundown* Rundown = GetRundown())
	{
		const int32 PageIdToTakeNext = GetPageIdToTakeNext();
		if (IsPageIdValid(PageIdToTakeNext))
		{
			Rundown->PlayPage(PageIdToTakeNext, EAvaRundownPagePlayType::PlayFromStart, EAvaRundownPagePlaySource::Editor);
			NextPageIds.Add(PageIdToTakeNext);
		}
	}
	return NextPageIds;
}

bool SAvaRundownInstancedPageList::CanPlayNextPage() const
{
	const int32 PageIdToTakeNext = GetPageIdToTakeNext();
	return IsPageIdValid(PageIdToTakeNext);
}

void SAvaRundownInstancedPageList::OnPageListChanged(const FAvaRundownPageListChangeParams& InParams)
{
	if (PageListReference != InParams.PageListReference)
	{
		return;
	}

	if (PageListReference.Type == EAvaRundownPageListType::View)
	{
		if (const UAvaRundown* Rundown = GetValidRundown())
		{
			// If the sublist has been removed, close the tab.
			if (!Rundown->IsValidSubList(PageListReference))
			{
				RequestCloseTab();
				return;
			}
		}
	}

	RefreshPagesVisibility();
	Refresh();
}

void SAvaRundownInstancedPageList::OnActiveListChanged()
{
	// If the sublist has been removed, close the tab.
	if (PageListReference.Type == EAvaRundownPageListType::View)
	{
		if (const UAvaRundown* Rundown = GetValidRundown())
		{
			if (!Rundown->IsValidSubList(PageListReference))
			{
				RequestCloseTab();
			}
		}
	}
}

void SAvaRundownInstancedPageList::RequestCloseTab()
{
	if (const TSharedPtr<FAvaRundownEditor> RundownEditor = GetRundownEditor())
	{
		const FName DocumentTabId = FAvaRundownSubListDocumentTabFactory::GetTabId(PageListReference);
		RundownEditor->RequestCloseDocumentTab(DocumentTabId);
		RundownEditor->UnregisterDocumentTabFactory(DocumentTabId);
	}
}

FReply SAvaRundownInstancedPageList::MakeActive()
{
	if (CanMakeActive())
	{
		GetRundown()->SetActivePageList(PageListReference);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

bool SAvaRundownInstancedPageList::CanMakeActive() const
{
	if (const UAvaRundown* const Rundown = GetValidRundown())
	{
		if (Rundown->IsPlaying())
		{
			return false;
		}
		return (Rundown->GetActivePageListReference() != PageListReference);
	}
	return false;
}

FSlateColor SAvaRundownInstancedPageList::GetMakeActiveButtonColor() const
{
	static const FSlateColor Inactive(FStyleColors::AccentGray.GetSpecifiedColor());
	static const FSlateColor Active(FStyleColors::PrimaryHover.GetSpecifiedColor());

	if (const UAvaRundown* const Rundown = GetValidRundown())
	{
		if (Rundown->GetActivePageListReference() == PageListReference)
		{
			return Active;
		}
	}

	return Inactive;
}

FText SAvaRundownInstancedPageList::GetPageViewName() const
{
	if (const UAvaRundown* const Rundown = GetValidRundown())
	{
		const FAvaRundownSubList& SubList = Rundown->GetSubList(PageListReference);		
		if (SubList.IsValid())
		{
			return SubList.Name;
		}
	}
	return FText::GetEmpty();
}

void SAvaRundownInstancedPageList::OnPageViewNameCommitted(const FText& InNewText, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::Default || InCommitType == ETextCommit::OnCleared)
	{
		return;
	}

	if (UAvaRundown* const Rundown = GetValidRundown())
	{
		if (Rundown->IsValidSubList(PageListReference))
		{
			FScopedTransaction Transaction(LOCTEXT("RenamePageView", "Rename Page View"));
			Rundown->Modify();
			Rundown->RenameSubList(PageListReference, InNewText);
		}
	}
}

FReply SAvaRundownInstancedPageList::OnDeletePageView()
{
	if (UAvaRundown* const Rundown = GetValidRundown())
	{
		if (Rundown->IsValidSubList(PageListReference))
		{
			FScopedTransaction Transaction(LOCTEXT("DeletePageView", "Delete Page View"));
			Rundown->Modify();
			if (Rundown->RemoveSubList(PageListReference))
			{
				return FReply::Handled();
			}

			Transaction.Cancel();
		}
	}
	return FReply::Unhandled();
}

TArray<int32> SAvaRundownInstancedPageList::AddPastedPages(const TArray<FAvaRundownPage>& InPages)
{
	if (UAvaRundown* Rundown = GetValidRundown())
	{
		using namespace UE::AvaRundownEditor::Utils;
		FImportTemplateMap ImportedTemplateIds;
		return ImportInstancedPages(Rundown, PageListReference, InPages, {}, ImportedTemplateIds);
	}
	return {};
}

TArray<int32> SAvaRundownInstancedPageList::FilterPlayingPages(FFilterPageFunctionRef InFilterPageFunction) const
{
	if (const UAvaRundown* Rundown = GetRundown())
	{
		return InFilterPageFunction(Rundown, Rundown->GetPlayingPageIds());
	}
	return {};
}

TArray<int32> SAvaRundownInstancedPageList::FilterSelectedOrPlayingPages(FFilterPageFunctionRef InFilterPageFunction, const bool bInAllowFallback) const
{
	if (const UAvaRundown* Rundown = GetRundown())
	{
		if (!SelectedPageIds.IsEmpty())
		{
			const TArray<int32> SelectedPages = InFilterPageFunction(Rundown, SelectedPageIds);
			if (!bInAllowFallback || !SelectedPages.IsEmpty())
			{
				return SelectedPages;
			}
		}
		return InFilterPageFunction(Rundown, Rundown->GetPlayingPageIds());
	}
	return {};
}

TArray<int32> SAvaRundownInstancedPageList::FilterPageSetForProgram(FFilterPageFunctionRef InFilterPageFunction, const EAvaRundownPageSet InPageSet) const
{
	switch (InPageSet)
	{
	case EAvaRundownPageSet::SelectedOrPlayingStrict:
		return FilterSelectedOrPlayingPages(InFilterPageFunction, /*bAllowFallback*/ false);
	case EAvaRundownPageSet::SelectedOrPlaying:
		return FilterSelectedOrPlayingPages(InFilterPageFunction, /*bAllowFallback*/ true);
	case EAvaRundownPageSet::Selected:
		return FilterSelectedPages(InFilterPageFunction);
	case EAvaRundownPageSet::Playing:
		return FilterPlayingPages(InFilterPageFunction);
	default:
		return FilterSelectedOrPlayingPages(InFilterPageFunction, /*bAllowFallback*/ true);
	}
}

TArray<int32> SAvaRundownInstancedPageList::GetAllPageIds() const
{
	TArray<int32> PageIds;
	PageIds.Reserve(PageViews.Num());
	for (const FAvaRundownPageViewPtr& PageView : PageViews)
	{
		if (PageView.IsValid())
		{
			PageIds.Add(PageView->GetPageId());
		}
	}
	return PageIds;
}

TArray<int32> SAvaRundownInstancedPageList::GetPagesToTakeIn() const
{
	auto KeepPagesToPlay = [](const UAvaRundown* InRundown, const TArray<int32>& InPageIds)
	{
		TArray<int32> OutPageIds;
		OutPageIds.Reserve(InPageIds.Num());
		for (int32 PageId : InPageIds)
		{
			if (InRundown->CanPlayPage(PageId, false))
			{
				OutPageIds.Add(PageId);
			}
		}
		return OutPageIds;
	};
	return FilterSelectedPages(KeepPagesToPlay);
}

TArray<int32> SAvaRundownInstancedPageList::GetPagesToTakeOut(bool bInForce) const
{
	const EAvaRundownPageStopOptions StopOptions = bInForce ? EAvaRundownPageStopOptions::ForceNoTransition : EAvaRundownPageStopOptions::Default;
	auto KeepPagesToStop = [StopOptions](const UAvaRundown* InRundown, const TArray<int32>& InPageIds)
	{
		TArray<int32> OutPageIds;
		OutPageIds.Reserve(InPageIds.Num());
		for (int32 PageId : InPageIds)
		{
			if (InRundown->CanStopPage(PageId, StopOptions, false))
			{
				OutPageIds.Add(PageId);
			}
		}
		return OutPageIds;
	};

	const UAvaRundownEditorSettings* RundownEditorSettings = UAvaRundownEditorSettings::Get();
	const EAvaRundownPageSet PageSet = RundownEditorSettings ? RundownEditorSettings->TakeOutActionPageSet : EAvaRundownPageSet::SelectedOrPlaying;
	return FilterPageSetForProgram(KeepPagesToStop, PageSet);
}

TArray<int32> SAvaRundownInstancedPageList::GetPagesToContinue() const
{
	auto KeepPagesToContinue = [](const UAvaRundown* InRundown, const TArray<int32>& InPageIds)
	{
		TArray<int32> OutPageIds;
		OutPageIds.Reserve(InPageIds.Num());
		for (int32 PageId : InPageIds)
		{
			if (InRundown->CanContinuePage(PageId, false))
			{
				OutPageIds.Add(PageId);
			}
		}
		return OutPageIds;
	};

	const UAvaRundownEditorSettings* RundownEditorSettings = UAvaRundownEditorSettings::Get();
	const EAvaRundownPageSet PageSet = RundownEditorSettings ? RundownEditorSettings->ContinueActionPageSet : EAvaRundownPageSet::SelectedOrPlaying;
	return FilterPageSetForProgram(KeepPagesToContinue, PageSet);
}

TArray<int32> SAvaRundownInstancedPageList::GetPagesToUpdate() const
{
	auto KeepPagesToUpdate = [](const UAvaRundown* InRundown, const TArray<int32>& InPageIds)
	{
		TArray<int32> OutPageIds;
		OutPageIds.Reserve(InPageIds.Num());
		for (int32 PageId : InPageIds)
		{
			if (InRundown->IsPagePlaying(PageId))
			{
				OutPageIds.Add(PageId);
			}
		}
		return OutPageIds;
	};

	const UAvaRundownEditorSettings* RundownEditorSettings = UAvaRundownEditorSettings::Get();
	const EAvaRundownPageSet PageSet = RundownEditorSettings ? RundownEditorSettings->UpdateValuesActionPageSet : EAvaRundownPageSet::SelectedOrPlaying;
	return FilterPageSetForProgram(KeepPagesToUpdate, PageSet);
}

int32 SAvaRundownInstancedPageList::GetNextPageIdToLoad() const
{
	const UAvaRundown* const Rundown = GetRundown();
	if (!Rundown)
	{
		return FAvaRundownPage::InvalidPageId;
	}

	// Try to use the program context if available. If not playing program, use preview loading
	const FAvaRundownPageListPlaybackContextCollection* const ContextCollection = Rundown->GetPageListPlaybackContextCollection();
	const bool bUseProgramContext = ContextCollection && ContextCollection->GetContext(/*bPreview*/false, /*PreviewChannel*/NAME_None).IsValid();

	return bUseProgramContext ? GetPageIdToTakeNext() : GetPageIdToPreviewNext();
}

int32 SAvaRundownInstancedPageList::GetPageIdToTakeNext() const
{
	return FAvaRundownPlaybackUtils::GetPageIdToPlayNext(GetRundown(), GetPageListReference(), /*bInPreview*/ false, NAME_None);
}

void SAvaRundownInstancedPageList::ResetPagesToDefaults(bool bInResetToTemplate)
{
	if (UAvaRundown* Rundown = GetRundown())
	{
		FScopedTransaction Transaction(LOCTEXT("ResetPagesTransaction", "Reset Pages"));
		Rundown->Modify();

		for (int32 PageId : SelectedPageIds)
		{
			const EAvaPlayableRemoteControlChanges Changes =
				Rundown->ResetRemoteControlValues(PageId, bInResetToTemplate, /*bInIsDefault=*/false);

			if (EnumHasAnyFlags(Changes, EAvaPlayableRemoteControlChanges::ControllerValues))
			{
				UE::AvaRundownEditor::Utils::ForceUpdatePageSummary(PageId, Rundown);
			}
		}
	}
}

bool SAvaRundownInstancedPageList::CanResetPagesToDefaults(bool bInResetToTemplate) const
{
	bool bContainsDifferentValues = false;

	if (UAvaRundown* Rundown = GetRundown())
	{
		for (const int32 PageId : SelectedPageIds)
		{
			const FAvaRundownPage& Page = Rundown->GetPage(PageId);

			if (!Page.IsValidPage() || !Page.IsEnabled() || Page.IsTemplate())
			{
				return false;
			}

			FAvaPlayableRemoteControlValues DefaultValues;
			if (Page.GetDefaultRemoteControlValues(Rundown, bInResetToTemplate, DefaultValues))
			{
				const FAvaPlayableRemoteControlValues& PageValues = Page.GetRemoteControlValues();
				if (!(PageValues.HasSameEntityValues(DefaultValues) && PageValues.HasSameControllerValues(DefaultValues)))
				{
					bContainsDifferentValues = true;
				}
			}
		}
	}

	return bContainsDifferentValues;
}

#undef LOCTEXT_NAMESPACE
