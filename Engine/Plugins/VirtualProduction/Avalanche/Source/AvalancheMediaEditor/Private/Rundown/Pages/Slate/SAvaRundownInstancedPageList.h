// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/AvaRundownEditorDefines.h"
#include "SAvaRundownPageList.h"

class SDockTab;
enum ETabActivationCause : uint8;

class SAvaRundownInstancedPageList : public SAvaRundownPageList
{
	SLATE_DECLARE_WIDGET(SAvaRundownInstancedPageList, SAvaRundownPageList)

public:
	SLATE_BEGIN_ARGS(SAvaRundownInstancedPageList) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TSharedPtr<FAvaRundownEditor> InRundownEditor, const FAvaRundownPageListReference& PageListReference);
	virtual ~SAvaRundownInstancedPageList() override;

	//~ Begin SAvaRundownPageList
	virtual void Refresh() override;
	virtual void CreateColumns() override;
	virtual TSharedPtr<SWidget> OnContextMenuOpening() override;
	virtual void BindCommands() override;
	virtual bool HandleDropAssets(const TArray<FSoftObjectPath>& InAvaAssets, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem) override;
	virtual bool HandleDropRundowns(const TArray<FSoftObjectPath>& InRundownPaths, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem) override;
	virtual bool HandleDropPageIds(const FAvaRundownPageListReference& InPageListReference, const TArray<int32>& InPageIds, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem) override;
	virtual bool HandleDropExternalFiles(const TArray<FString>& InFiles, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem) override;
	//~ End SAvaRundownPageList

	bool HandleDropPageIdsOnMainListFromTemplates(const TArray<int32>& InPageIds, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem);
	bool HandleDropPageIdsOnMainListFromMainList(const TArray<int32>& InPageIds, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem);
	bool HandleDropPageIdsOnSubListFromTemplates(const TArray<int32>& InPageIds, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem);
	bool HandleDropPageIdsOnSubListFromMainList(const TArray<int32>& InPageIds, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem);
	bool HandleDropPageIdsOnSubListFromSubList(const FAvaRundownPageListReference& InFromList, const TArray<int32>& InPageIds, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem);

	void OnTabActivated(TSharedRef<SDockTab> InDockTab, ETabActivationCause InActivationCause);

	/** Plays the currently selected page. */
	void PlaySelectedPage() const;
	bool CanPlaySelectedPage() const;

	/** Returns true if there's at least 1 page in this page list that can be loaded */
	bool CanLoadAllPages() const;
	/** Requests load all the pages in this page list that can play */
	void LoadAllPages();

	/** Returns true if there's at least 1 page in from the next pages that can be loaded */
	bool CanLoadNextPages() const;
	/** Requests load the next pages that can play */
	void LoadNextPages();

	/** Returns true if there's at least 1 page from the list of selected pages that can be loaded */
	bool CanLoadSelectedPages() const;
	/** Requests load the selected pages that can play */
	void LoadSelectedPages();

	/** Returns true if there's at least 1 page from the given ids that can be loaded */
	bool CanLoadPages(TConstArrayView<int32> InPageIds) const;
	/** Requests load the given page ids that can play */
	void LoadPages(TConstArrayView<int32> InPageIds);

	bool CanUpdateValuesOnSelectedPage() const;
	void UpdateValuesOnSelectedPage();
	
	void ContinueSelectedPage() const;
	bool CanContinueSelectedPage() const;

	void StopSelectedPage(bool bInForce) const;
	bool CanStopSelectedPage(bool bInForce) const;

	TArray<int32> PlayNextPage() const;
	bool CanPlayNextPage() const;

	TArray<int32> GetPageIdsToTakeNext() const
	{
		const int32 NextPageId = GetPageIdToTakeNext();
		if (IsPageIdValid(NextPageId))
		{
			return { NextPageId };
		}
		return {};
	}
	
private:
	void OnPageListChanged(const FAvaRundownPageListChangeParams& InParams);
	void OnActiveListChanged();

	TArray<int32> FilterPlayingPages(FFilterPageFunctionRef InFilterPageFunction) const;
	TArray<int32> FilterSelectedOrPlayingPages(FFilterPageFunctionRef InFilterPageFunction, const bool bInAllowFallback) const;
	TArray<int32> FilterPageSetForProgram(FFilterPageFunctionRef InFilterPageFunction, const EAvaRundownPageSet InPageSet) const;

	/** Retrieves all the page ids available in this page list */
	TArray<int32> GetAllPageIds() const;
	TArray<int32> GetPagesToTakeIn() const;
	TArray<int32> GetPagesToTakeOut(bool bInForce) const;
	TArray<int32> GetPagesToContinue() const;
	TArray<int32> GetPagesToUpdate() const;
	int32 GetNextPageIdToLoad() const;
	int32 GetPageIdToTakeNext() const;

protected:
	void RequestCloseTab();
	
	FReply MakeActive();
	bool CanMakeActive() const;
	FSlateColor GetMakeActiveButtonColor() const;

	FText GetPageViewName() const;
	void OnPageViewNameCommitted(const FText& InNewText, ETextCommit::Type InCommitType);
	FReply OnDeletePageView();

	void PlayNextPageNoReturn() const { PlayNextPage(); }

	virtual TArray<int32> AddPastedPages(const TArray<FAvaRundownPage>& InPages) override;

	void ResetPagesToDefaults(bool bInResetToTemplate);
	bool CanResetPagesToDefaults(bool bInResetToTemplate) const;
};
