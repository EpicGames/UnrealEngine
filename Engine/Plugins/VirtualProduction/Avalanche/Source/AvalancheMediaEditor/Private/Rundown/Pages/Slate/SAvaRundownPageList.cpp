// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownPageList.h"

#include "AvaAssetTags.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Editor/EditorWidgets/Public/SAssetSearchBox.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IAvaMediaEditorModule.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownEditor.h"
#include "Rundown/AvaRundownEditorSettings.h"
#include "Rundown/AvaRundownEditorUtils.h"
#include "Rundown/AvaRundownPage.h"
#include "Rundown/AvaRundownPlaybackUtils.h"
#include "Rundown/AvaRundownSerializationUtils.h"
#include "Rundown/Factories/Filters/AvaRundownFactoriesUtils.h"
#include "Rundown/Factories/Filters/IAvaRundownFilterSuggestionFactory.h"
#include "Rundown/Filters/AvaRundownPageTextFilter.h"
#include "Rundown/Pages/AvaRundownPageContext.h"
#include "Rundown/Pages/AvaRundownPageContextMenu.h"
#include "Rundown/Pages/PageViews/AvaRundownPageViewImpl.h"
#include "Rundown/Pages/Slate/SAvaRundownPageViewRow.h"
#include "SAvaRundownRenumberPages.h"
#include "ScopedTransaction.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SAvaRundownPageList"

namespace UE::AvaRundown::Private
{
	static const FString PageClipboardPrefix = TEXT("MotionDesignPages");
	static const FString PageEntriesName = TEXT("Pages");

	void ExtractAssetSearchFilterTerms(const FText& SearchText, FString* OutFilterKey, FString* OutFilterValue, int32* OutSuggestionInsertionIndex)
	{
		const FString SearchString = SearchText.ToString();

		if (OutFilterKey)
		{
			OutFilterKey->Reset();
		}
		if (OutFilterValue)
		{
			OutFilterValue->Reset();
		}
		if (OutSuggestionInsertionIndex)
		{
			*OutSuggestionInsertionIndex = SearchString.Len();
		}

		// Build the search filter terms so that we can inspect the tokens
		FTextFilterExpressionEvaluator LocalFilter(ETextFilterExpressionEvaluatorMode::Complex);
		LocalFilter.SetFilterText(SearchText);
	
		// Inspect the tokens to see what the last part of the search term was
		// If it was a key->value pair then we'll use that to control what kinds of results we show
		// For anything else we just use the text from the last token as our filter term to allow incremental auto-complete
		const TArray<FExpressionToken>& FilterTokens = LocalFilter.GetFilterExpressionTokens();
		if (FilterTokens.Num() > 0)
		{
			const FExpressionToken& LastToken = FilterTokens.Last();
			// If the last token is a text token, then consider it as a value and walk back to see if we also have a key
			if (LastToken.Node.Cast<TextFilterExpressionParser::FTextToken>())
			{
				if (OutFilterValue)
				{
					*OutFilterValue = LastToken.Context.GetString();
				}
				if (OutSuggestionInsertionIndex)
				{
					*OutSuggestionInsertionIndex = FMath::Min(*OutSuggestionInsertionIndex, LastToken.Context.GetCharacterIndex());
				}
				if (FilterTokens.IsValidIndex(FilterTokens.Num() - 2))
				{
					const FExpressionToken& ComparisonToken = FilterTokens[FilterTokens.Num() - 2];
					if (ComparisonToken.Node.Cast<TextFilterExpressionParser::FEqual>())
					{
						if (FilterTokens.IsValidIndex(FilterTokens.Num() - 3))
						{
							const FExpressionToken& KeyToken = FilterTokens[FilterTokens.Num() - 3];
							if (KeyToken.Node.Cast<TextFilterExpressionParser::FTextToken>())
							{
								if (OutFilterKey)
								{
									*OutFilterKey = KeyToken.Context.GetString();
								}
								if (OutSuggestionInsertionIndex)
								{
									*OutSuggestionInsertionIndex = FMath::Min(*OutSuggestionInsertionIndex, KeyToken.Context.GetCharacterIndex());
								}
							}
						}
					}
				}
			}
			// If the last token is a comparison operator, then walk back and see if we have a key
			else if (LastToken.Node.Cast<TextFilterExpressionParser::FEqual>())
			{
				if (FilterTokens.IsValidIndex(FilterTokens.Num() - 2))
				{
					const FExpressionToken& KeyToken = FilterTokens[FilterTokens.Num() - 2];
					if (KeyToken.Node.Cast<TextFilterExpressionParser::FTextToken>())
					{
						if (OutFilterKey)
						{
							*OutFilterKey = KeyToken.Context.GetString();
						}
						if (OutSuggestionInsertionIndex)
						{
							*OutSuggestionInsertionIndex = FMath::Min(*OutSuggestionInsertionIndex, KeyToken.Context.GetCharacterIndex());
						}
					}
				}
			}
		}
	}

	const FAvaRundownPageCollection& GetPageCollection(const UAvaRundown* InRundown, EAvaRundownPageListType InPageListType)
	{
		check(InRundown);
		return InPageListType == EAvaRundownPageListType::Template ? InRundown->GetTemplatePages() : InRundown->GetInstancedPages();
	}
	
	EAvaRundownSearchListType GetPageSearchListType(EAvaRundownPageListType InPageListType)
	{
		switch(InPageListType)
		{
		case EAvaRundownPageListType::Instance:
		case EAvaRundownPageListType::View:
			return EAvaRundownSearchListType::Instanced;
			
		case EAvaRundownPageListType::Template:
			return EAvaRundownSearchListType::Template;
			
		default:
			return EAvaRundownSearchListType::None;
		}
	}
}

void SAvaRundownPageList::PrivateRegisterAttributes(struct FSlateAttributeDescriptor::FInitializer&)
{

}

void SAvaRundownPageList::Construct(const FArguments& InArgs, TSharedPtr<FAvaRundownEditor> InRundownEditor, const FAvaRundownPageListReference& InPageListReference)
{
	RundownEditorWeak = InRundownEditor;
	check(InRundownEditor.IsValid());
	
	OnActiveTabChangedDelegateHandle = FGlobalTabmanager::Get()->OnActiveTabChanged_Subscribe(FOnActiveTabChanged::FDelegate::CreateSP(this, &SAvaRundownPageList::OnActiveTabChanged));
	
	PageContextMenu = MakeShared<FAvaRundownPageContextMenu>();

	PageListReference = InPageListReference;
	CommandList = MakeShared<FUICommandList>();
	BindCommands();

	PageTextFilter = MakeShared<FAvaRundownPageTextFilter>();
	PageTextFilter->OnChanged().AddSP(this, &SAvaRundownPageList::OnPageTextFilterChanged);
	RefreshPagesVisibility();
	
	const UAvaRundown* const Rundown = InRundownEditor->GetRundown();
	check(Rundown);

	InRundownEditor->GetOnPageEvent().AddSP(this, &SAvaRundownPageList::OnPageEvent);

	CreateColumns();

	ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.f)
			[
				SAssignNew(SearchBar, SHorizontalBox)
				// Search Box
				+ SHorizontalBox::Slot()
				.Padding(4.f, 2.f)
				[
					SAssignNew(AssetSearchBoxPtr, SAssetSearchBox)
					.HintText(LOCTEXT("FilterSearch", "Search..."))
					.ToolTipText(LOCTEXT("FilterSearchHint", "Type here to search..."))
					.ShowSearchHistory(true)
					.OnTextChanged(this, &SAvaRundownPageList::OnSearchTextChanged)
					.OnTextCommitted(this, &SAvaRundownPageList::OnSearchTextCommitted)
					.OnAssetSearchBoxSuggestionFilter(this, &SAvaRundownPageList::OnSearchBoxSuggestionFilter)
					.OnAssetSearchBoxSuggestionChosen(this, &SAvaRundownPageList::OnAssetSearchBoxSuggestionChosen)
					.DelayChangeNotificationsWhileTyping(true)
					.Visibility(EVisibility::Visible)
					.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("SAvaRundownPageListSearchBox")))
				]
			]
			// Pages Collection
			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(0.f)
				[
					SAssignNew(PageListView, SListView<FAvaRundownPageViewPtr>)
					.HeaderRow(HeaderRow)
					.ListItemsSource(&PageViews)
					.HandleSpacebarSelection(true)
					.SelectionMode(ESelectionMode::Multi)
					.ClearSelectionOnClick(true)
					.OnGenerateRow(this, &SAvaRundownPageList::GeneratePageTableRow)
					.OnSelectionChanged(this, &SAvaRundownPageList::OnPageSelected)
					.OnContextMenuOpening(this, &SAvaRundownPageList::OnContextMenuOpening)
				]
			]
		];

	Refresh();
}

SAvaRundownPageList::~SAvaRundownPageList()
{
	FGlobalTabmanager::Get()->OnActiveTabChanged_Unsubscribe(OnActiveTabChangedDelegateHandle);
	OnActiveTabChangedDelegateHandle.Reset();

	if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		RundownEditor->GetOnPageEvent().RemoveAll(this);
	}
}

void SAvaRundownPageList::OnPageEvent(const TArray<int32>& InSelectedPageIds, UE::AvaRundown::EPageEvent InPageEvent)
{
	using namespace  UE::AvaRundown;

	switch (InPageEvent)
	{
		case EPageEvent::SelectionRequest:
			OnPageSelectionRequested(InSelectedPageIds);
			break;

		case EPageEvent::RenameRequest:
			OnPageItemActionRequested(InSelectedPageIds, &IAvaRundownPageView::GetOnRename);
			break;

		case EPageEvent::RenumberRequest:
			OnPageItemActionRequested(InSelectedPageIds, &IAvaRundownPageView::GetOnRenumber);
			break;
		default:
			break; // skip everything else
	}
}

void SAvaRundownPageList::OnPageItemActionRequested(const TArray<int32>& InPageIds, IAvaRundownPageView::FOnPageAction& (IAvaRundownPageView::* InFunc)())
{
	const TSet<int32> PageIdSet(InPageIds);
	for (const FAvaRundownPageViewPtr& PageView : PageViews)
	{
		if (PageIdSet.Contains(PageView->GetPageId()))
		{
			(*PageView.*InFunc)().Broadcast(EAvaRundownPageActionState::Requested);
		}
	}
}

void SAvaRundownPageList::OnPageSelectionRequested(const TArray<int32>& InPageIds)
{
	TArray<FAvaRundownPageViewPtr> DesiredSelectedItems;
	DesiredSelectedItems.Empty(InPageIds.Num());
	for (int32 PageId : InPageIds)
	{
		DesiredSelectedItems.Add(GetPageViewPtr(PageId));
	}
	PageListView->ClearSelection();
	PageListView->SetItemSelection(DesiredSelectedItems, true);
}

TSharedRef<ITableRow> SAvaRundownPageList::GeneratePageTableRow(FAvaRundownPageViewPtr InPageView, const TSharedRef<STableViewBase>& InOwnerTable)
{
	check(InPageView.IsValid());
	return SNew(SAvaRundownPageViewRow, InPageView, SharedThis(this));
}

void SAvaRundownPageList::OnPageSelected(FAvaRundownPageViewPtr InPageView, ESelectInfo::Type InSelectInfo)
{
	if (PageListView.IsValid())
	{
		TArray<FAvaRundownPageViewPtr> SelectedItems;
		PageListView->GetSelectedItems(SelectedItems);
		SelectedPageIds.Empty(SelectedItems.Num());

		for (const FAvaRundownPageViewPtr& PageView : SelectedItems)
		{
			if (PageView.IsValid())
			{
				const int32 SelectedPage = PageView->GetPageId();
				SelectedPageIds.Add(SelectedPage);
			}
		}

		TSharedPtr<FAvaRundownEditor> RundownEditor = GetRundownEditor();

		if (RundownEditor.IsValid())
		{
			RundownEditor->GetOnPageEvent().Broadcast(SelectedPageIds, UE::AvaRundown::EPageEvent::SelectionChanged);
		}
	}
}

TSharedPtr<IAvaRundownPageViewColumn> SAvaRundownPageList::FindColumn(FName InColumnName) const
{
	if (const TSharedPtr<IAvaRundownPageViewColumn>* const FoundColumn = Columns.Find(InColumnName))
	{
		return *FoundColumn;
	}
	return nullptr;
}

FAvaRundownPageViewPtr SAvaRundownPageList::GetPageViewPtr(int32 InPageId) const
{
	for (const FAvaRundownPageViewPtr& PageView : PageViews)
	{
		if (PageView->GetPageId() == InPageId)
		{
			return PageView;
		}
	}
	return nullptr;
}

void SAvaRundownPageList::SelectPage(int32 InPageId, bool bInScrollIntoView)
{
	if (PageListView.IsValid())
	{
		for (const FAvaRundownPageViewPtr& PageView : PageViews)
		{
			if (PageView->GetPageId() == InPageId)
			{
				PageListView->SetItemSelection(PageView, true, ESelectInfo::Direct);

				if (bInScrollIntoView)
				{
					PageListView->RequestScrollIntoView(PageView);
				}

				return;
			}
		}
	}
}

void SAvaRundownPageList::SelectPages(const TArray<int32>& InPageIds, bool bInScrollIntoView)
{
	if (PageListView.IsValid())
	{
		for (int32 PageId : InPageIds)
		{
			SelectPage(PageId, bInScrollIntoView);
		}
	}
}

void SAvaRundownPageList::DeselectPage(int32 InPageId)
{
	if (PageListView.IsValid())
	{
		for (const FAvaRundownPageViewPtr& PageView : PageViews)
		{
			if (PageView->GetPageId() == InPageId)
			{
				PageListView->SetItemSelection(PageView, false, ESelectInfo::Direct);
				return;
			}
		}
	}
}

void SAvaRundownPageList::DeselectPages()
{
	if (PageListView.IsValid())
	{
		PageListView->ClearSelection();
	}
}

TArray<int32> SAvaRundownPageList::GetPlayingPageIds() const
{
	if (const UAvaRundown* Rundown = GetRundown())
	{
		return Rundown->GetPlayingPageIds();
	}
	return TArray<int32>();
}

TSharedPtr<FAvaRundownEditor> SAvaRundownPageList::GetRundownEditor() const
{
	return RundownEditorWeak.Pin();
}

UAvaRundown* SAvaRundownPageList::GetRundown() const
{
	const TSharedPtr<FAvaRundownEditor> RundownEditor = GetRundownEditor();
	return RundownEditor.IsValid() ? RundownEditor->GetRundown() : nullptr;
}

UAvaRundown* SAvaRundownPageList::GetValidRundown() const
{
	UAvaRundown* Rundown = GetRundown();
	return IsValid(Rundown) ? Rundown : nullptr;
}

void SAvaRundownPageList::BindCommands()
{
	//Generic Commands
	{
		const FGenericCommands& GenericCommands = FGenericCommands::Get();

		CommandList->MapAction(GenericCommands.Rename,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::RenameSelectedPage),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanRenameSelectedPage));

		CommandList->MapAction(GenericCommands.Delete,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::RemoveSelectedPages),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanRemoveSelectedPages));

		CommandList->MapAction(GenericCommands.Cut,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::CutSelectedPages),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanCutSelectedPages));

		CommandList->MapAction(GenericCommands.Copy,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::CopySelectedPages),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanCopySelectedPages));

		CommandList->MapAction(GenericCommands.Paste,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::Paste),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanPaste));

		CommandList->MapAction(GenericCommands.Duplicate,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::DuplicateSelectedPages),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanDuplicateSelectedPages));
	}
}

int32 SAvaRundownPageList::GetFirstSelectedPageId() const
{
	return SelectedPageIds.Num() > 0 ? SelectedPageIds[0] : FAvaRundownPage::InvalidPageId;
}

TSharedRef<SWidget> SAvaRundownPageList::GetPageListContextMenu()
{
	return PageContextMenu->GeneratePageContextMenuWidget(RundownEditorWeak, PageListReference, CommandList);
}

bool SAvaRundownPageList::CanAddPage() const
{
	if (const UAvaRundown* Rundown = GetRundown(); IsValid(Rundown))
	{
		return Rundown->CanAddPage();
	}
	return false;
}

bool SAvaRundownPageList::CanAddTemplate() const
{
	return CanAddPage();
}

bool SAvaRundownPageList::CanCreateInstance() const
{
	return CanAddPage();
}

bool SAvaRundownPageList::CanCopySelectedPages() const
{
	return !SelectedPageIds.IsEmpty() && IsValid(GetRundown());
}

void SAvaRundownPageList::CopySelectedPages()
{
	if (!CanCopySelectedPages())
	{
		return;
	}

	UAvaRundown* Rundown = GetRundown();
	if (!IsValid(Rundown))
	{
		UE_LOGF(LogAvaRundown, Error, "Can't copy from invalid rundown.");
		return;
	}

	FString SerializedString = UE::AvaRundownEditor::Utils::SerializePagesToJson(Rundown, SelectedPageIds);
	
	// Add Prefix to quickly identify whether current clipboard is from Motion Design Pages or not
	SerializedString = *FString::Printf(TEXT("%s%s"), *UE::AvaRundown::Private::PageClipboardPrefix, *SerializedString);

	FPlatformApplicationMisc::ClipboardCopy(*SerializedString);
}

bool SAvaRundownPageList::CanCutSelectedPages() const
{
	return CanCopySelectedPages() && CanRemoveSelectedPages();
}

void SAvaRundownPageList::CutSelectedPages()
{
	if (CanCutSelectedPages())
	{
		CopySelectedPages();
		RemoveSelectedPages();
	}
}

bool SAvaRundownPageList::CanPaste() const
{
	if (!CanAddPage())
	{
		return false;
	}

	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);
	return PastedText.StartsWith(UE::AvaRundown::Private::PageClipboardPrefix);
}

void SAvaRundownPageList::Paste()
{
	if (!CanPaste())
	{
		return;
	}

	UAvaRundown* Rundown = GetRundown();
	if (!IsValid(Rundown))
	{
		return;
	}	

	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);
	PastedText.RightChopInline(UE::AvaRundown::Private::PageClipboardPrefix.Len());

	const TArray<FAvaRundownPage> Pages = UE::AvaRundownEditor::Utils::DeserializePagesFromJson(PastedText);
	
	if (Pages.Num() > 0)
	{
		FScopedTransaction Transaction(LOCTEXT("AddPages", "Add Pages"));
		Rundown->Modify();

		const TArray<int32> PageIds = AddPastedPages(Pages);

		if (PageIds.IsEmpty())
		{
			Transaction.Cancel();
		}
		else
		{
			DeselectPages();
			SelectPages(PageIds);
		}
	}
}

bool SAvaRundownPageList::CanDuplicateSelectedPages() const
{
	return CanCopySelectedPages() && CanAddPage();
}

void SAvaRundownPageList::DuplicateSelectedPages()
{
	if (CanDuplicateSelectedPages())
	{
		CopySelectedPages();
		Paste();
	}
}

bool SAvaRundownPageList::CanRemoveSelectedPages() const
{
	if (!SelectedPageIds.IsEmpty())
	{
		if (const UAvaRundown* Rundown = GetRundown(); IsValid(Rundown))
		{
			return Rundown->CanRemovePages(SelectedPageIds);
		}
	}
	return false;
}

void SAvaRundownPageList::RemoveSelectedPages()
{
	if (CanRemoveSelectedPages())
	{
		if (UAvaRundown* Rundown = GetRundown(); IsValid(Rundown))
		{
			FScopedTransaction Transaction(LOCTEXT("RemoveSelectedPages", "Remove Selected Pages"));
			Rundown->Modify();

			int32 RemovedCount = 0;

			if (PageListReference.Type != EAvaRundownPageListType::View)
			{
				RemovedCount = Rundown->RemovePages(SelectedPageIds);
			}
			else if (Rundown->IsValidSubList(PageListReference))
			{
				RemovedCount = Rundown->RemovePagesFromSubList(PageListReference, SelectedPageIds);
			}

			if (RemovedCount == 0)
			{
				Transaction.Cancel();
			}
		}
	}
}

bool SAvaRundownPageList::CanRenameSelectedPage() const
{
	if (SelectedPageIds.Num() == 1)
	{
		if (const UAvaRundown* Rundown = GetRundown(); IsValid(Rundown))
		{
			return Rundown->GetPage(SelectedPageIds[0]).IsValidPage();
		}
	}
	return false;
}

void SAvaRundownPageList::RenameSelectedPage()
{
	if (CanRenameSelectedPage())
	{
		OnPageEvent(SelectedPageIds, UE::AvaRundown::EPageEvent::RenameRequest);
	}
}

bool SAvaRundownPageList::CanRenumberSelectedPages() const
{
	const int32 SelectedPageCount = SelectedPageIds.Num();
	if (SelectedPageCount == 0)
	{
		return false;
	}

	const UAvaRundown* const Rundown = GetRundown();
	if (!IsValid(Rundown))
	{
		return false;
	}

	for (const int32 PageId : SelectedPageIds)
	{
		if (Rundown->CanRenumberPageId(PageId))
		{
			return true;
		}
	}

	return false;
}

void SAvaRundownPageList::RenumberSelectedPages()
{
	const int32 SelectedCount = SelectedPageIds.Num();
	if (SelectedCount == 0)
	{
		return;
	}
	
	if (SelectedCount == 1)
	{
		// Begin re-numbering inline
		OnPageEvent(SelectedPageIds, UE::AvaRundown::EPageEvent::RenumberRequest);
	}
	else
	{
		// Bring up dialog for re-numbering multiple selection
		const TSharedPtr<SAvaRundownRenumberPages> RenumberPagesDialog = SNew(SAvaRundownRenumberPages)
			.OnAccept_Lambda([this](const int32 InBaseNumber, const int32 InIncrement)
				{
					if (UAvaRundown* const Rundown = GetRundown())
					{
						Rundown->RenumberPageIds(SelectedPageIds, FAvaRundownPageIdGeneratorParams(InBaseNumber, InIncrement));
					}
				});
		FSlateApplication::Get().AddModalWindow(RenumberPagesDialog.ToSharedRef(), SharedThis(this));
	}
}

bool SAvaRundownPageList::CanReimportSelectedPage() const
{
	return SelectedPageIds.Num() > 0 && GetRundownEditor().IsValid();
}

void SAvaRundownPageList::ReimportSelectedPage() const
{
	if (CanReimportSelectedPage())
	{
		if (const TSharedPtr<FAvaRundownEditor> RundownEditor = GetRundownEditor())
		{
			if (UAvaRundown* Rundown = RundownEditor->GetRundown(); IsValid(Rundown))
			{
				// Enforce invalidation of the Motion Design Managed Instance Cache for the selected page(s).
				Rundown->InvalidateManagedInstanceCacheForPages(SelectedPageIds);
				Rundown->UpdateAssetForPages(SelectedPageIds, true);
				
				using namespace UE::AvaRundownEditor::Utils;
				if (UpdateDefaultRemoteControlValues(Rundown, SelectedPageIds) != EAvaPlayableRemoteControlChanges::None)
				{
					RundownEditor->MarkAsModified();
				}
			}

			RundownEditor->GetOnPageEvent().Broadcast(SelectedPageIds, UE::AvaRundown::EPageEvent::ReimportRequest);
		}
	}
}

bool SAvaRundownPageList::CanEditSelectedPageSource() const
{
	if (SelectedPageIds.Num() == 1)
	{
		if (UAvaRundown* Rundown = GetRundown(); IsValid(Rundown))
		{
			const FAvaRundownPage& Page = Rundown->GetPage(SelectedPageIds[0]);
			return Page.IsValidPage();
		}
	}
	return false;
}

void SAvaRundownPageList::EditSelectedPageSource()
{
	if (SelectedPageIds.Num() == 1)
	{
		if (UAvaRundown* Rundown = GetRundown(); IsValid(Rundown))
		{
			const FAvaRundownPage& Page = Rundown->GetPage(SelectedPageIds[0]);
			if (Page.IsValidPage())
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Page.GetAssetPath(Rundown));
			}
		}
	}
}

bool SAvaRundownPageList::CanExportSelectedPagesToRundown()
{
	return SelectedPageIds.Num() > 0;
}

void SAvaRundownPageList::ExportSelectedPagesToRundown()
{
	const UAvaRundown* SourceRundown = GetRundown();
	if (!SourceRundown || SelectedPageIds.IsEmpty())
	{
		return;
	}

	using namespace UE::AvaRundownEditor::Utils;
	if (const TStrongObjectPtr<UAvaRundown> NewRundown = ExportPagesToRundown(SourceRundown, SelectedPageIds))
	{	
		const UPackage* const RundownPackage = SourceRundown->GetPackage();
		const FString DefaultPackagePath = FPackageName::GetLongPackagePath(RundownPackage->GetName());
		const FString DefaultAssetName = SourceRundown->GetName() + TEXT("_Exported");		
		const FString SaveObjectPath = GetSaveAssetAsPath(DefaultPackagePath, DefaultAssetName);

		if (!SaveObjectPath.IsEmpty())
		{
			const FString PackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
			const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);
			const FString AssetName = FPackageName::GetLongPackageAssetName(PackageName);
			SaveDuplicateRundown(NewRundown.Get(), AssetName, PackagePath);
		}
	}
}

bool SAvaRundownPageList::CanExportSelectedPagesToExternalFile(const TCHAR* InType)
{
	return SelectedPageIds.Num() > 0;
}

void SAvaRundownPageList::ExportSelectedPagesToExternalFile(const TCHAR* InType)
{
	const UAvaRundown* SourceRundown = GetRundown();
	if (!SourceRundown || SelectedPageIds.IsEmpty())
	{
		return;
	}

	using namespace UE::AvaRundownEditor::Utils;
	using namespace UE::AvaMedia::RundownSerializationUtils;
	if (const TStrongObjectPtr<UAvaRundown> NewRundown = ExportPagesToRundown(SourceRundown, SelectedPageIds))
	{
		if (FCString::Stricmp(InType, TEXT("json")) == 0)
		{
			const FString ExportFilename = GetExportFilepath(SourceRundown, TEXT("json file"), TEXT("json"));
			if (!ExportFilename.IsEmpty())
			{
				FText ErrorMessage;
				if (!SaveRundownToJson(NewRundown.Get(), *ExportFilename, ErrorMessage))
				{
					UE_LOGF(LogAvaRundown, Error,
						"Failed to export rundown \"%ls\" to file \"%ls\". Reason: %ls",
						*NewRundown->GetFullName(), *ExportFilename, *ErrorMessage.ToString());
				}
			}
		}
		else if (FCString::Stricmp(InType, TEXT("xml")) == 0)
		{
			const FString ExportFilename = GetExportFilepath(SourceRundown, TEXT("xml file"), TEXT("xml"));
			if (!ExportFilename.IsEmpty())
			{
				SaveRundownToXml(NewRundown.Get(), *ExportFilename, EXmlSerializationEncoding::Utf8);
			}
		}
		else
		{
			UE_LOGF(LogAvaRundown, Error, "Export Pages to external file doesn't support type \"%ls\".", InType);
		}
	}
}

bool SAvaRundownPageList::CanPreviewPlaySelectedPage() const
{
	const TArray<int32> PageIds = GetPagesToPreviewIn();
	return !PageIds.IsEmpty();
}

void SAvaRundownPageList::PreviewPlaySelectedPage(bool bInToMark) const
{
	if (UAvaRundown* Rundown = GetRundown())
	{
		const TArray<int32> PageIds = GetPagesToPreviewIn();
		Rundown->PlayPages(PageIds, bInToMark ? EAvaRundownPagePlayType::PreviewFromFrame : EAvaRundownPagePlayType::PreviewFromStart, EAvaRundownPagePlaySource::Editor);
	}
}

bool SAvaRundownPageList::CanPreviewStopSelectedPage(bool bInForce) const
{
	const TArray<int32> PageIds = GetPagesToPreviewOut(bInForce);
	return !PageIds.IsEmpty();
}

void SAvaRundownPageList::PreviewStopSelectedPage(bool bInForce) const
{
	if (UAvaRundown* Rundown = GetRundown())
	{
		const EAvaRundownPageStopOptions StopOptions = bInForce ? EAvaRundownPageStopOptions::ForceNoTransition : EAvaRundownPageStopOptions::Default;
		const TArray<int32> PageIds = GetPagesToPreviewOut(bInForce);
		Rundown->StopPages(PageIds, StopOptions, true);
	}
}

bool SAvaRundownPageList::CanPreviewContinueSelectedPage() const
{
	const TArray<int32> PageIds = GetPagesToPreviewContinue();
	return !PageIds.IsEmpty();
}

void SAvaRundownPageList::PreviewContinueSelectedPage() const
{
	if (UAvaRundown* Rundown = GetRundown())
	{
		const TArray<int32> PageIds = GetPagesToPreviewContinue();
		for (const int32 PageId : PageIds)
		{
			Rundown->ContinuePage(PageId, true);
		}
	}
}

bool SAvaRundownPageList::CanPreviewPlayNextPage() const
{
	const int32 PageIdToPreviewNext = GetPageIdToPreviewNext();
	return IsPageIdValid(PageIdToPreviewNext);
}

void SAvaRundownPageList::PreviewPlayNextPage()
{
	if (UAvaRundown* Rundown = GetRundown())
	{
		const int32 PageIdToPreviewNext = GetPageIdToPreviewNext();
		if (IsPageIdValid(PageIdToPreviewNext))
		{
			Rundown->PlayPage(PageIdToPreviewNext, EAvaRundownPagePlayType::PreviewFromStart, EAvaRundownPagePlaySource::Editor);
			DeselectPages();
			SelectPages({PageIdToPreviewNext});
		}
	}
}

bool SAvaRundownPageList::CanTakeToProgram() const
{
	const TArray<int32> PageIds = GetPagesToTakeToProgram();
	return !PageIds.IsEmpty();
}

void SAvaRundownPageList::TakeToProgram() const
{
	if (UAvaRundown* Rundown = GetRundown(); IsValid(Rundown))
	{
		const TArray<int32> PageIds = GetPagesToTakeToProgram();
		Rundown->PlayPages(PageIds, EAvaRundownPagePlayType::PlayFromStart, EAvaRundownPagePlaySource::Editor);
	}
}

namespace UE::AvaPageList::Private
{
	static bool IsAvaAsset(const FAssetData& InAssetData, bool bInLogInfo)
	{
		static const FName& SceneAssetTagName = UE::Ava::AssetTags::MotionDesignScene;
		static const FString& AssetTagValueEnabled = UE::Ava::AssetTags::Values::Enabled;

		const UClass* AssetClass = InAssetData.GetClass(EResolveClass::Yes);
		if (!IsValid(AssetClass))
		{
			return false;
		}
		
		const EMotionDesignAssetType AssetType = FAvaSoftAssetPath::GetAssetTypeFromClass(AssetClass, true);
		if (AssetType == EMotionDesignAssetType::Unknown)
		{
			return false;
		}
		
		// If the asset is a level, we check if it has the Motion Design scene tag.
		if (AssetType == EMotionDesignAssetType::World)
		{
			const FAssetTagValueRef SceneTag = InAssetData.TagsAndValues.FindTag(SceneAssetTagName);
			
			if (!SceneTag.IsSet())
			{
				if (bInLogInfo)
				{
					UE_LOGF(LogAvaRundown, Display,
						"Level Asset \"%ls\" is not an Motion Design Scene (Asset Tag not found).",
						*InAssetData.GetSoftObjectPath().ToString());
				}
				return false;
			}
			
			if (!SceneTag.Equals(AssetTagValueEnabled))
			{
				if (bInLogInfo)
				{
					UE_LOGF(LogAvaRundown, Display,
						"Level Asset \"%ls\" is an Motion Design Scene but not enabled.",
						*InAssetData.GetSoftObjectPath().ToString());
				}
				return false;						
			}
		}
		return true;
	}
}

bool SAvaRundownPageList::IsAssetDropSupported(const FAssetData& InAsset, const FSoftObjectPath& InDestinationRundownPath)
{
	using namespace UE::AvaPageList::Private;
	if (IsAvaAsset(InAsset, false))
	{
		return true;
	}
	if (InAsset.IsInstanceOf<UAvaRundown>(EResolveClass::Yes))
	{
		// Don't drop a rundown in itself.
		if (InAsset.GetSoftObjectPath() != InDestinationRundownPath)
		{
			return true;
		}
	}
	return false;
}

TArray<FSoftObjectPath> SAvaRundownPageList::FilterAvaAssetPaths(const TArray<FAssetData>& InAssets)
{
	using namespace UE::AvaPageList::Private;
	TArray<FSoftObjectPath> AvaAssets;
	AvaAssets.Reserve(InAssets.Num());
	
	for (const FAssetData& AssetData : InAssets)
	{
		if (IsAvaAsset(AssetData, true))
		{
			AvaAssets.Add(AssetData.GetSoftObjectPath());
		}
	}
	return AvaAssets;
}

TArray<FSoftObjectPath> SAvaRundownPageList::FilterRundownPaths(const TArray<FAssetData>& InAssets, const FSoftObjectPath& InDestinationRundownPath)
{
	TArray<FSoftObjectPath> Rundowns;
	Rundowns.Reserve(InAssets.Num());

	for (const FAssetData& AssetData : InAssets)
	{
		if (AssetData.IsInstanceOf<UAvaRundown>(EResolveClass::Yes))
		{
			FSoftObjectPath RundownPath = AssetData.GetSoftObjectPath();
			// Don't drop a rundown in itself.
			if (RundownPath != InDestinationRundownPath)
			{
				Rundowns.Add(MoveTemp(RundownPath));
			}
		}
	}
	
	return Rundowns;
}

FAvaRundownPageInsertPosition SAvaRundownPageList::MakeInsertPosition(EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem)
{
	FAvaRundownPageInsertPosition InsertPosition;
	InsertPosition.AdjacentId = InItem.IsValid() ? InItem->GetPageId() : FAvaRundownPage::InvalidPageId;
	InsertPosition.bAddBelow = InDropZone != EItemDropZone::AboveItem;
	return InsertPosition;
}

bool SAvaRundownPageList::CanHandleDragObjects(const FDragDropEvent& InDragDropEvent) const
{
	if (const TSharedPtr<FAssetDragDropOp> AssetDragDropOp = InDragDropEvent.GetOperationAs<FAssetDragDropOp>())
	{
		const FSoftObjectPath DestinationRundownPath(GetRundown());
		for (const FAssetData& AssetData : AssetDragDropOp->GetAssets())
		{
			if (IsAssetDropSupported(AssetData, DestinationRundownPath))
			{
				return true;
			}
		}
		return false;
	}

	if (const TSharedPtr<FAvaRundownPageViewRowDragDropOp> PageDragDropOp = InDragDropEvent.GetOperationAs<FAvaRundownPageViewRowDragDropOp>())
	{
		return true;
	}

	if (const TSharedPtr<FExternalDragOperation> ExternalDragDropOp = InDragDropEvent.GetOperationAs<FExternalDragOperation>())
	{
		if (ExternalDragDropOp->HasFiles())
		{
			for (const FString& File : ExternalDragDropOp->GetFiles())
			{
				if (UE::AvaRundownEditor::Utils::CanLoadRundownFromFile(*File))
				{
					return true;
				}
			}
		}
	}
	
	return false;
}

bool SAvaRundownPageList::HandleDropEvent(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem)
{
	if (const TSharedPtr<FAssetDragDropOp> AssetDragDropOp = InDragDropEvent.GetOperationAs<FAssetDragDropOp>())
	{
		const TArray<FSoftObjectPath> AvaAssets = FilterAvaAssetPaths(AssetDragDropOp->GetAssets());
		const TArray<FSoftObjectPath> Rundowns = FilterRundownPaths(AssetDragDropOp->GetAssets(), FSoftObjectPath(GetRundown()));

		FScopedTransaction DropTransaction(LOCTEXT("AssetDropTransaction", "Drop Motion Design Assets onto Page List"));

		bool bIsHandled = false;
		if (!Rundowns.IsEmpty())
		{
			bIsHandled |= HandleDropRundowns(Rundowns, InDropZone, InItem);
		}
		if (!AvaAssets.IsEmpty())
		{
			bIsHandled |= HandleDropAssets(AvaAssets, InDropZone, InItem);
		}
		return bIsHandled;
	}

	if (const TSharedPtr<FAvaRundownPageViewRowDragDropOp> PageDragDropOp = InDragDropEvent.GetOperationAs<FAvaRundownPageViewRowDragDropOp>())
	{
		if (const TSharedPtr<SAvaRundownPageList> FromPageList = PageDragDropOp->GetPageList())
		{
			return HandleDropPageIds(FromPageList->GetPageListReference(), PageDragDropOp->GetDraggedIds(), InDropZone, InItem);
		}
		return false;
	}

	if (const TSharedPtr<FExternalDragOperation> ExternalDragDropOp = InDragDropEvent.GetOperationAs<FExternalDragOperation>())
	{
		FScopedTransaction DropTransaction(LOCTEXT("ExternalDropTransaction", "Drop External Asset onto Page List"));

		return ExternalDragDropOp->HasFiles() && HandleDropExternalFiles(ExternalDragDropOp->GetFiles(), InDropZone, InItem);
	}

	return false;
}

FReply SAvaRundownPageList::OnDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return FReply::Handled();
}

FReply SAvaRundownPageList::OnDragOver(const FGeometry& InMyGeometry, const FDragDropEvent& InDragDropEvent)
{
	if (CanHandleDragObjects(InDragDropEvent))
	{
		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

FReply SAvaRundownPageList::OnDrop(const FGeometry& InMyGeometry, const FDragDropEvent& InDragDropEvent)
{
	if (HandleDropEvent(InDragDropEvent, EItemDropZone::OntoItem, nullptr))
	{
		return FReply::Handled();
	}
		
	return FReply::Unhandled();
}

FReply SAvaRundownPageList::OnKeyDown(const FGeometry& InMyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SAvaRundownPageList::RefreshPagesVisibility()
{
	VisiblePageIds.Reset();

	const UAvaRundown* Rundown = GetValidRundown();
	if (!Rundown)
	{
		return;
	}

	const EAvaRundownSearchListType PageSearchListType = UE::AvaRundown::Private::GetPageSearchListType(PageListReference.Type);

	auto FilterPage = [this, Rundown, PageSearchListType](const FAvaRundownPage& InPage)
	{
		if (PageTextFilter)
		{
			PageTextFilter->SetItem(InPage, Rundown, PageSearchListType);
			if (PageTextFilter->PassesFilter(InPage))
			{
				VisiblePageIds.Add(InPage.GetPageId());
			}
		}
		else
		{
			VisiblePageIds.Add(InPage.GetPageId());
		}
	};

	if (Rundown->IsValidSubList(PageListReference))
	{
		const FAvaRundownSubList& SubList = Rundown->GetSubList(PageListReference);
		for (const int32 PageId : SubList.PageIds)
		{
			const FAvaRundownPage& Page = Rundown->GetPage(PageId);
			if (Page.IsValidPage())
			{
				FilterPage(Page);
			}
		}
	}
	else
	{
		const FAvaRundownPageCollection& Collection = UE::AvaRundown::Private::GetPageCollection(Rundown, PageListReference.Type);
		for (const FAvaRundownPage& Page : Collection.Pages)
		{
			FilterPage(Page);
		}
	}
}

FText SAvaRundownPageList::OnAssetSearchBoxSuggestionChosen(const FText& InSearchText, const FString& InSuggestion)
{
	int32 SuggestionInsertionIndex = 0;
	UE::AvaRundown::Private::ExtractAssetSearchFilterTerms(InSearchText, nullptr, nullptr, &SuggestionInsertionIndex);

	FString SearchString = InSearchText.ToString();
	SearchString.RemoveAt(SuggestionInsertionIndex, SearchString.Len() - SuggestionInsertionIndex, EAllowShrinking::No);
	SearchString.Append(InSuggestion);
	return FText::FromString(SearchString);
}

void SAvaRundownPageList::OnSearchTextChanged(const FText& FilterText)
{
	if (PageTextFilter)
	{
		PageTextFilter->SetFilterText(FilterText);
	}
}

void SAvaRundownPageList::OnSearchTextCommitted(const FText& FilterText, ETextCommit::Type CommitType)
{
	if (PageTextFilter)
	{
		PageTextFilter->SetFilterText(FilterText);
	}
}

void SAvaRundownPageList::OnSearchBoxSuggestionFilter(const FText& InSearchText, TArray<FAssetSearchBoxSuggestion>& OutPossibleSuggestions, FText& OutSuggestionHighlightText)
{
	// We don't bind the suggestion list, so this list should be empty as we populate it here based on the search term
	check(OutPossibleSuggestions.IsEmpty());

	FString FilterKey;
	FString FilterValue;
	UE::AvaRundown::Private::ExtractAssetSearchFilterTerms(InSearchText, &FilterKey, &FilterValue, nullptr);

	const IAvaMediaEditorModule& AvaMediaEditorModule = IAvaMediaEditorModule::Get();
	const EAvaRundownSearchListType PageListType = UE::AvaRundown::Private::GetPageSearchListType(PageListReference.Type);

	TSet<FString> FilterCache;
	const TSharedRef<FAvaRundownFilterSuggestionPayload> SimplePayload =
				MakeShared<FAvaRundownFilterSuggestionPayload>(FAvaRundownFilterSuggestionPayload{ OutPossibleSuggestions, FilterValue, FAvaRundownPage::InvalidPageId, nullptr, FilterCache });
	for (const TSharedPtr<IAvaRundownFilterSuggestionFactory>& SimpleSuggestion : AvaMediaEditorModule.GetSimpleSuggestions(PageListType))
	{
		SimpleSuggestion->AddSuggestion(SimplePayload);
	}

	if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		if (const UAvaRundown* Rundown = RundownEditor->GetRundown())
		{
			const TSharedRef<FAvaRundownFilterSuggestionPayload> ComplexPayload =
				MakeShared<FAvaRundownFilterSuggestionPayload>(FAvaRundownFilterSuggestionPayload{ OutPossibleSuggestions, FilterValue, FAvaRundownPage::InvalidPageId, Rundown, FilterCache });

			if (Rundown->IsValidSubList(PageListReference))
			{
				for (const int32 PageId : Rundown->GetSubList(PageListReference).PageIds)
				{
					for (const TSharedPtr<IAvaRundownFilterSuggestionFactory>& ComplexSuggestion : AvaMediaEditorModule.GetComplexSuggestions(PageListType))
					{
						ComplexPayload->ItemPageId = PageId;
						ComplexSuggestion->AddSuggestion(ComplexPayload);
					}
				}
			}
			else
			{
				using namespace UE::AvaRundown::Private;
				for (const FAvaRundownPage& Page : GetPageCollection(Rundown, PageListReference.Type).Pages)
				{
					for (const TSharedPtr<IAvaRundownFilterSuggestionFactory>& ComplexSuggestion : AvaMediaEditorModule.GetComplexSuggestions(PageListType))
					{
						ComplexPayload->ItemPageId = Page.GetPageId();
						ComplexSuggestion->AddSuggestion(ComplexPayload);
					}
				}
			}
		}
	}
	OutSuggestionHighlightText = FText::FromString(FilterValue);
}

void SAvaRundownPageList::OnPageTextFilterChanged()
{
	RefreshPagesVisibility();
	Refresh();
}

void SAvaRundownPageList::OnActiveTabChanged(TSharedPtr<SDockTab> InPreviouslyActive, TSharedPtr<SDockTab> InNewlyActivated)
{
	if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		if (InNewlyActivated && RundownEditor->GetAssociatedTabManager() == InNewlyActivated->GetTabManagerPtr())
		{
			const TSharedPtr<SWidget> Content = InNewlyActivated->GetContent();
			if (Content.Get() == this)
			{
				// The set of selected pages of this page list becomes the current set edited. Relevant for page detail.
				RundownEditor->GetOnPageEvent().Broadcast(SelectedPageIds, UE::AvaRundown::EPageEvent::SelectionChanged);
			}
		}
	}
}

TArray<int32> SAvaRundownPageList::FilterSelectedPages(FFilterPageFunctionRef InFilterPageFunction) const
{
	if (const UAvaRundown* Rundown = GetRundown())
	{
		if (!SelectedPageIds.IsEmpty())
		{
			return InFilterPageFunction(Rundown, SelectedPageIds);
		}
	}
	return {};
}
	
TArray<int32> SAvaRundownPageList::FilterPreviewingPages(FFilterPageFunctionRef InFilterPageFunction) const
{
	if (const UAvaRundown* Rundown = GetRundown())
	{
		return InFilterPageFunction(Rundown, Rundown->GetPreviewingPageIds());
	}
	return {};
}
	
TArray<int32> SAvaRundownPageList::FilterSelectedOrPreviewingPages(FFilterPageFunctionRef InFilterPageFunction, const bool bInAllowFallback) const
{
	if (const UAvaRundown* Rundown = GetRundown())
	{
		if (!SelectedPageIds.IsEmpty())
		{
			const TArray<int32> SelectedPages = InFilterPageFunction(Rundown, SelectedPageIds);
			if (!bInAllowFallback || !SelectedPages.IsEmpty())
			{
				return SelectedPages;
			}
		}
		return InFilterPageFunction(Rundown, Rundown->GetPreviewingPageIds());
	}
	return {};
}
	
TArray<int32> SAvaRundownPageList::FilterPageSetForPreview(FFilterPageFunctionRef InFilterPageFunction, const EAvaRundownPageSet InPageSet) const
{
	switch (InPageSet)
	{
	case EAvaRundownPageSet::SelectedOrPlayingStrict:
		return FilterSelectedOrPreviewingPages(InFilterPageFunction, /*bAllowFallback*/ false);
	case EAvaRundownPageSet::SelectedOrPlaying:
		return FilterSelectedOrPreviewingPages(InFilterPageFunction, /*bAllowFallback*/ true);
	case EAvaRundownPageSet::Selected:
		return FilterSelectedPages(InFilterPageFunction);
	case EAvaRundownPageSet::Playing:
		return FilterPreviewingPages(InFilterPageFunction);
	default:
		return FilterSelectedOrPreviewingPages(InFilterPageFunction, /*bAllowFallback*/ true);
	}
}

TArray<int32> SAvaRundownPageList::GetPagesToPreviewIn() const
{
	auto KeepPagesToPreviewIn = [](const UAvaRundown* InRundown, const TArray<int32>& InPageIds)
	{
		TArray<int32> OutPageIds;
		OutPageIds.Reserve(InPageIds.Num());
		for (int32 PageId : InPageIds)
		{
			if (InRundown->CanPlayPage(PageId, true))
			{
				OutPageIds.Add(PageId);
			}
		}
		return OutPageIds;
	};
	return FilterSelectedPages(KeepPagesToPreviewIn);
}

TArray<int32> SAvaRundownPageList::GetPagesToPreviewOut(bool bInForce) const
{
	EAvaRundownPageStopOptions StopOptions = bInForce ? EAvaRundownPageStopOptions::ForceNoTransition : EAvaRundownPageStopOptions::Default;
	auto KeepPagesToPreviewStop = [StopOptions](const UAvaRundown* InRundown, const TArray<int32>& InPageIds)
	{
		TArray<int32> OutPageIds;
		OutPageIds.Reserve(InPageIds.Num());
		for (int32 PageId : InPageIds)
		{
			if (InRundown->CanStopPage(PageId, StopOptions, true))
			{
				OutPageIds.Add(PageId);
			}
		}
		return OutPageIds;
	};

	const UAvaRundownEditorSettings* RundownEditorSettings = UAvaRundownEditorSettings::Get();
	const EAvaRundownPageSet PageSet = RundownEditorSettings ? RundownEditorSettings->PreviewOutActionPageSet : EAvaRundownPageSet::SelectedOrPlaying;
	return FilterPageSetForPreview(KeepPagesToPreviewStop, PageSet);
}

TArray<int32> SAvaRundownPageList::GetPagesToPreviewContinue() const
{
	auto KeepPagesToPreviewContinue = [](const UAvaRundown* InRundown, const TArray<int32>& InPageIds)
	{
		TArray<int32> OutPageIds;
		OutPageIds.Reserve(InPageIds.Num());
		for (int32 PageId : InPageIds)
		{
			if (InRundown->CanContinuePage(PageId, true))
			{
				OutPageIds.Add(PageId);
			}
		}
		return OutPageIds;
	};

	const UAvaRundownEditorSettings* RundownEditorSettings = UAvaRundownEditorSettings::Get();
	const EAvaRundownPageSet PageSet = RundownEditorSettings ? RundownEditorSettings->PreviewContinueActionPageSet : EAvaRundownPageSet::SelectedOrPlaying;
	return FilterPageSetForPreview(KeepPagesToPreviewContinue, PageSet);
}

TArray<int32> SAvaRundownPageList::GetPagesToTakeToProgram() const
{
	// Remark: taking to program will take all previewing pages that are not already playing
	// regardless of what is selected.
	return FAvaRundownPlaybackUtils::GetPagesToTakeToProgram(GetRundown(), {});
}

int32 SAvaRundownPageList::GetPageIdToPreviewNext() const
{
	return FAvaRundownPlaybackUtils::GetPageIdToPlayNext(
		GetRundown(), GetPageListReference(), /*bInPreview*/ true, UAvaRundown::GetDefaultPreviewChannelName());
}

#undef LOCTEXT_NAMESPACE
