// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownEditorDefines.h"
#include "Rundown/Pages/PageViews/IAvaRundownPageView.h"
#include "Widgets/SCompoundWidget.h"

class FAvaRundownEditor;
class FAvaRundownPageContextMenu;
class FAvaRundownPageTextFilter;
class FUICommandList;
class IAvaRundownPageViewColumn;
class ITableRow;
class SAssetSearchBox;
class SDockTab;
class SHeaderRow;
class SHorizontalBox;
class STableViewBase;
enum class EItemDropZone;
struct FAssetSearchBoxSuggestion;
struct FAvaRundownPage;
template<typename ItemType> class SListView;

class SAvaRundownPageList : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SAvaRundownPageList, SCompoundWidget)

public:
	SLATE_BEGIN_ARGS(SAvaRundownPageList) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TSharedPtr<FAvaRundownEditor> InRundownEditor, const FAvaRundownPageListReference& InPageListReference);
	virtual ~SAvaRundownPageList() override;

	const FAvaRundownPageListReference& GetPageListReference() const { return PageListReference; }

	void OnPageEvent(const TArray<int32>& InSelectedPageIds, UE::AvaRundown::EPageEvent InPageEvent);

	void OnPageItemActionRequested(const TArray<int32>& InPageIds, IAvaRundownPageView::FOnPageAction&(IAvaRundownPageView::*InFunc)());

	void OnPageSelectionRequested(const TArray<int32>& InPageIds);

	TSharedRef<ITableRow> GeneratePageTableRow(FAvaRundownPageViewPtr InPageView, const TSharedRef<STableViewBase>& InOwnerTable);

	void OnPageSelected(FAvaRundownPageViewPtr InPageView, ESelectInfo::Type InSelectInfo);

	TSharedPtr<SListView<FAvaRundownPageViewPtr>> GetPageListView() { return PageListView; }

	TSharedPtr<IAvaRundownPageViewColumn> FindColumn(FName InColumnName) const;

	FAvaRundownPageViewPtr GetPageViewPtr(int32 InPageId) const;

	void SelectPage(int32 InPageId, bool bInScrollIntoView = true);
	void SelectPages(const TArray<int32>& InPageIds, bool bInScrollIntoView = true);
	void DeselectPage(int32 InPageId);
	void DeselectPages();

	const TArray<int32>& GetSelectedPageIds() const { return SelectedPageIds; }
	TArray<int32> GetPlayingPageIds() const;
	const TArray<FAvaRundownPageViewPtr>& GetPageViews() const { return PageViews; }

	TSharedPtr<FAvaRundownEditor> GetRundownEditor() const;

	UAvaRundown* GetRundown() const;
	UAvaRundown* GetValidRundown() const;

	virtual void BindCommands();

	virtual void Refresh() = 0;

	virtual void CreateColumns() = 0;

	virtual TSharedPtr<SWidget> OnContextMenuOpening() = 0;
	
	int32 GetFirstSelectedPageId() const;

	TSharedRef<SWidget> GetPageListContextMenu();

	bool CanCopySelectedPages() const;
	void CopySelectedPages();

	bool CanCutSelectedPages() const;
	void CutSelectedPages();

	bool CanPaste() const;
	void Paste();

	bool CanDuplicateSelectedPages() const;
	void DuplicateSelectedPages();

	bool CanAddPage() const;
	bool CanAddTemplate() const;
	bool CanCreateInstance() const;

	bool CanRemoveSelectedPages() const;
	void RemoveSelectedPages();

	bool CanRenameSelectedPage() const;
	void RenameSelectedPage();

	bool CanRenumberSelectedPages() const;
	void RenumberSelectedPages();

	bool CanReimportSelectedPage() const;
	void ReimportSelectedPage() const;

	bool CanEditSelectedPageSource() const;
	void EditSelectedPageSource();

	bool CanExportSelectedPagesToRundown();
	void ExportSelectedPagesToRundown();

	bool CanExportSelectedPagesToExternalFile(const TCHAR* InType);
	void ExportSelectedPagesToExternalFile(const TCHAR* InType);

	bool CanPreviewPlaySelectedPage() const;
	void PreviewPlaySelectedPage(bool bInToMark) const;

	bool CanPreviewStopSelectedPage(bool bInForce) const;
	void PreviewStopSelectedPage(bool bInForce) const;

	bool CanPreviewContinueSelectedPage() const;
	void PreviewContinueSelectedPage() const;

	bool CanPreviewPlayNextPage() const;
	void PreviewPlayNextPage();

	bool CanTakeToProgram() const;
	void TakeToProgram() const;

	/** For the given asset, check if it supported for drop operation. */
	static bool IsAssetDropSupported(const FAssetData& InAsset, const FSoftObjectPath& InDestinationRundownPath);
	
	/** From the given array of asset data, filter only ava assets. */
	static TArray<FSoftObjectPath> FilterAvaAssetPaths(const TArray<FAssetData>& InAssets);

	/** From the given array of asset data, keep only the rundowns. */
	static TArray<FSoftObjectPath> FilterRundownPaths(const TArray<FAssetData>& InAssets, const FSoftObjectPath& InDestinationRundownPath);

	/** Helper to convert drop parameters to insert position. */
	static FAvaRundownPageInsertPosition MakeInsertPosition(EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem);

	virtual bool CanHandleDragObjects(const FDragDropEvent& InDragDropEvent) const;
	virtual bool HandleDropEvent(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem);
	
	virtual bool HandleDropAssets(const TArray<FSoftObjectPath>& InAvaAssets, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem) = 0;
	virtual bool HandleDropRundowns(const TArray<FSoftObjectPath>& InRundownPaths, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem) = 0;
	virtual bool HandleDropPageIds(const FAvaRundownPageListReference& PageListReference, const TArray<int32>& InPageIds, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem) = 0;
	virtual bool HandleDropExternalFiles(const TArray<FString>& InFiles, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem) = 0;

	//~ Begin SWidget
	virtual FReply OnDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnDragOver(const FGeometry& InMyGeometry, const FDragDropEvent& InDragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& InMyGeometry, const FDragDropEvent& InDragDropEvent) override;
	virtual FReply OnKeyDown(const FGeometry& InMyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget

protected: 
	static bool IsPageIdValid(int32 InPageId)
	{
		return InPageId != FAvaRundownPage::InvalidPageId;
	}

	using FFilterPageFunctionRef = TFunctionRef<TArray<int32>(const UAvaRundown* InRundown, const TArray<int32>& InPageIds)>;
	
	TArray<int32> FilterSelectedPages(FFilterPageFunctionRef InFilterPageFunction) const;
	TArray<int32> FilterPreviewingPages(FFilterPageFunctionRef InFilterPageFunction) const;
	TArray<int32> FilterSelectedOrPreviewingPages(FFilterPageFunctionRef InFilterPageFunction, const bool bInAllowFallback) const;
	TArray<int32> FilterPageSetForPreview(FFilterPageFunctionRef InFilterPageFunction, const EAvaRundownPageSet InPageSet) const;
	
	TArray<int32> GetPagesToPreviewIn() const;
	TArray<int32> GetPagesToPreviewOut(bool bInForce) const;
	TArray<int32> GetPagesToPreviewContinue() const;
	TArray<int32> GetPagesToTakeToProgram() const;
	int32 GetPageIdToPreviewNext() const;

	bool IsPageVisible(const FAvaRundownPage& InPage) const
	{
		return VisiblePageIds.Contains(InPage.GetPageId());
	}

	bool IsPageVisible(int32 InPageId) const
	{
		return VisiblePageIds.Contains(InPageId);
	}

	/** Applies PageTextFilter to current page list and caches the result. */
	void RefreshPagesVisibility();
	
private:
	FText OnAssetSearchBoxSuggestionChosen(const FText& InSearchText, const FString& InSuggestion);

	void OnSearchTextChanged(const FText& FilterText);

	void OnSearchTextCommitted(const FText& FilterText, ETextCommit::Type CommitType);

	void OnSearchBoxSuggestionFilter(const FText& InSearchText, TArray<FAssetSearchBoxSuggestion>& OutPossibleSuggestions, FText& OutSuggestionHighlightText);

	void OnPageTextFilterChanged();

	void OnActiveTabChanged(TSharedPtr<SDockTab> InPreviouslyActive, TSharedPtr<SDockTab> InNewlyActivated);

protected:
	TWeakPtr<FAvaRundownEditor> RundownEditorWeak;

	TSharedPtr<FAvaRundownPageContextMenu> PageContextMenu;

	FAvaRundownPageListReference PageListReference;

	TSharedPtr<SHorizontalBox> SearchBar;

	TSharedPtr<SHeaderRow> HeaderRow;

	TMap<FName, TSharedPtr<IAvaRundownPageViewColumn>> Columns;

	TSharedPtr<SListView<FAvaRundownPageViewPtr>> PageListView;

	TArray<FAvaRundownPageViewPtr> PageViews;

	TArray<int32> SelectedPageIds;

	TSharedPtr<FUICommandList> CommandList;

	TSharedPtr<FAvaRundownPageTextFilter> PageTextFilter;

	/** Cached result of PageTextFilter applied to current page list. */
	TSet<int32> VisiblePageIds;
	
	/**
	 * For template and instance lists, it will return the list of page ids created.
	 * For a sub list it will return the list of page ids that were added to the page view.
	 */
	virtual TArray<int32> AddPastedPages(const TArray<FAvaRundownPage>& InPages) = 0;

private:
	TSharedPtr<SAssetSearchBox> AssetSearchBoxPtr;
	
	/** Delegate handle bound to the FGlobalTabmanager::OnActiveTabChanged delegate */
	FDelegateHandle OnActiveTabChangedDelegateHandle;
};
