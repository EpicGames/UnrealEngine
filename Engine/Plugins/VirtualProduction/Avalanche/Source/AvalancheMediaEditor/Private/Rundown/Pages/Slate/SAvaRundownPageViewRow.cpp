// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rundown/Pages/Slate/SAvaRundownPageViewRow.h"
#include "AvaMediaEditorStyle.h"
#include "Input/DragAndDrop.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "Layout/Geometry.h"
#include "Misc/Attribute.h"
#include "Rundown/Pages/Columns/IAvaRundownPageViewColumn.h"
#include "Rundown/Pages/PageViews/IAvaRundownPageView.h"
#include "SAvaRundownPageList.h"
#include "ScopedTransaction.h"
#include "Styling/SlateBrush.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "UObject/NameTypes.h"

#define LOCTEXT_NAMESPACE "SAvaRundownPageViewRow"

namespace UE::AvaMediaEditor::RowUtils
{
	FName ActivePageBrushName = FName(TEXT("TableView.ActivePageBrush"));
	FName ActivePageHoveredBrushName = FName(TEXT("TableView.Hovered.ActivePageBrush"));
	FName ActivePageSelectedBrushName = FName(TEXT("TableView.Selected.ActivePageBrush"));
	FName DisabledPageBrushName = FName(TEXT("TableView.DisabledPageBrush"));
	FName DisabledPageHoveredBrushName = FName(TEXT("TableView.Hovered.DisabledPageBrush"));
	FName DisabledPageSelectedBrushName = FName(TEXT("TableView.Selected.DisabledPageBrush"));
}

void SAvaRundownPageViewRow::Construct(const FArguments& InArgs, FAvaRundownPageViewPtr InPageView, const TSharedRef<SAvaRundownPageList>& InPageList)
{
	PageViewWeak = InPageView;
	PageListWeak = InPageList;

	TSharedPtr<SListView<FAvaRundownPageViewPtr>> PageListView = InPageList->GetPageListView();
	check(PageListView.IsValid());
	
	SMultiColumnTableRow::Construct(
		FSuperRowType::FArguments()
		.OnDragDetected(this, &SAvaRundownPageViewRow::OnPageViewDragDetected)
		.OnCanAcceptDrop(this, &SAvaRundownPageViewRow::OnPageViewCanAcceptDrop)
		.OnAcceptDrop(this, &SAvaRundownPageViewRow::OnPageViewAcceptDrop)
		, PageListView.ToSharedRef()
	);
	SetBorderImage(TAttribute<const FSlateBrush*>(this, &SAvaRundownPageViewRow::GetBorder));
}

TSharedRef<SWidget> SAvaRundownPageViewRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	if (const FAvaRundownPageViewPtr PageView = PageViewWeak.Pin())
	{
		if (const TSharedPtr<IAvaRundownPageViewColumn> Column = PageListWeak.Pin()->FindColumn(InColumnName))
		{
			const TSharedRef<SWidget> RowWidget = Column->ConstructRowWidget(PageView.ToSharedRef(), SharedThis(this));
			RowWidget->SetOnMouseDoubleClick(FPointerEventHandler::CreateLambda([this](const FGeometry&, const FPointerEvent&)
			{
				return PageViewWeak.IsValid()? PageViewWeak.Pin()->OnPreviewButtonClicked() : FReply::Handled();
			}));
			return RowWidget;
		}
	}
	return SNullWidget::NullWidget;
}

const FSlateBrush* SAvaRundownPageViewRow::GetBorder() const
{
	if (const TSharedPtr<IAvaRundownPageView> PageView = PageViewWeak.Pin())
	{
		const UAvaRundown* const Rundown = PageView->GetRundown();
		if (Rundown && !PageView->IsTemplate())
		{
			const FAvaRundownPage& Page = Rundown->GetPage(PageView->GetPageId());
			const TArray<FAvaRundownChannelPageStatus> Statuses = Page.GetPageContextualStatuses(Rundown);

			if (FAvaRundownPage::StatusesContainsStatus(Statuses, { EAvaRundownPageStatus::Loaded, EAvaRundownPageStatus::Playing }))
			{
				return GetRowColorBrush(EAvaRundownRowState::Active);
			}

			if (FAvaRundownPage::StatusesContainsStatus(Statuses, { EAvaRundownPageStatus::Error, EAvaRundownPageStatus::Missing }))
			{
				return GetRowColorBrush(EAvaRundownRowState::Disabled);
			}
		}
	}

	return SMultiColumnTableRow::GetBorder();
}

FReply SAvaRundownPageViewRow::OnPageViewDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InPointerEvent)
{
	TSharedPtr<SAvaRundownPageList> PageList = PageListWeak.Pin();

	if (!PageList.IsValid() || PageList->GetSelectedPageIds().IsEmpty())
	{
		return FReply::Unhandled();
	}

	return FReply::Handled().BeginDragDrop(FAvaRundownPageViewRowDragDropOp::New(PageList));
}

TOptional<EItemDropZone> SAvaRundownPageViewRow::OnPageViewCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FAvaRundownPageViewPtr InItem)
{
	if (TSharedPtr<SAvaRundownPageList> PageList = PageListWeak.Pin())
	{
		if (PageList->CanHandleDragObjects(InDragDropEvent))
		{
			return InDropZone;
		}
	}

	return TOptional<EItemDropZone>();
}

FReply SAvaRundownPageViewRow::OnPageViewAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FAvaRundownPageViewPtr InItem)
{
	if (const TSharedPtr<SAvaRundownPageList> PageList = PageListWeak.Pin())
	{
		if (PageList->HandleDropEvent(InDragDropEvent, InDropZone, InItem))
		{
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

const FSlateBrush* SAvaRundownPageViewRow::GetRowColorBrush(EAvaRundownRowState InRowState) const
{
	using namespace UE::AvaMediaEditor::RowUtils;
	FName NormalStateBrushName;
	FName HoveredStateBrushName;
	FName SelectedStateBrushName;

	if (InRowState == EAvaRundownRowState::Active)
	{
		NormalStateBrushName = ActivePageBrushName;
		HoveredStateBrushName = ActivePageHoveredBrushName;
		SelectedStateBrushName = ActivePageSelectedBrushName;
	}
	else
	{
		NormalStateBrushName = DisabledPageBrushName;
		HoveredStateBrushName = DisabledPageHoveredBrushName;
		SelectedStateBrushName = DisabledPageSelectedBrushName;
	}

	if (IsHovered())
	{
		return FAvaMediaEditorStyle::Get().GetBrush(HoveredStateBrushName);
	}

	if (IsSelected())
	{
		return FAvaMediaEditorStyle::Get().GetBrush(SelectedStateBrushName);
	}

	return FAvaMediaEditorStyle::Get().GetBrush(NormalStateBrushName);
}

TSharedPtr<SWidget> FAvaRundownPageViewRowDragDropOp::GetDefaultDecorator() const
{
	TArray<FText> IdTexts;

	for (int32 DraggedId : DraggedIds)
	{
		IdTexts.Add(FText::AsNumber(DraggedId, &UE::AvaRundown::FEditorMetrics::PageIdFormattingOptions));
	}

	return SNew(SBorder)
		.Padding(5.f)
		.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
		.Content()
		[
			SNew(STextBlock)
			.Text(FText::Join(LOCTEXT("Comma", ", "), IdTexts))
		];
}

TSharedRef<FAvaRundownPageViewRowDragDropOp> FAvaRundownPageViewRowDragDropOp::New(const TSharedPtr<SAvaRundownPageList>& InPageList)
{
	check(InPageList.IsValid());

	TSharedRef<FAvaRundownPageViewRowDragDropOp> Operation = MakeShared<FAvaRundownPageViewRowDragDropOp>();
	Operation->PageListWeak = InPageList;
	Operation->DraggedIds = InPageList->GetSelectedPageIds(); // Cache this just in case.
	Operation->Construct();
	return Operation;
}

#undef LOCTEXT_NAMESPACE
