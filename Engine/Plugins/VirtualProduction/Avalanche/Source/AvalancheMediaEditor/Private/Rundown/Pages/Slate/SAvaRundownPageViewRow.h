// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Input/DragAndDrop.h"
#include "Rundown/AvaRundownEditorDefines.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "UObject/NameTypes.h"
#include "Misc/Optional.h"

class FDragDropEvent;
class FReply;
class IAvaRundownPageView;
class SAvaRundownPageList;
class SWidget;
struct FGeometry;
struct FPointerEvent;
struct FSlateBrush;

enum class EAvaRundownRowState
{
	Active,
	Disabled
};

class FAvaRundownPageViewRowDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FAvaRundownPageViewRowDragDropOp, FDragDropOperation)

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;

	static TSharedRef<FAvaRundownPageViewRowDragDropOp> New(const TSharedPtr<SAvaRundownPageList>& InPageList);

	TSharedPtr<SAvaRundownPageList> GetPageList() const { return PageListWeak.Pin(); }
	const TArray<int32>& GetDraggedIds() const { return DraggedIds; }

	FName Color;

protected:
	TWeakPtr<SAvaRundownPageList> PageListWeak;
	TArray<int32> DraggedIds;
};

class SAvaRundownPageViewRow : public SMultiColumnTableRow<FAvaRundownPageViewPtr>
{
public:	
	SLATE_BEGIN_ARGS(SAvaRundownPageViewRow){}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, FAvaRundownPageViewPtr InPageView, const TSharedRef<SAvaRundownPageList>& InPageList);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

	virtual const FSlateBrush* GetBorder() const override;

protected:
	FReply OnPageViewDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InPointerEvent);
	TOptional<EItemDropZone> OnPageViewCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FAvaRundownPageViewPtr InItem);
	FReply OnPageViewAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FAvaRundownPageViewPtr InItem);

	TWeakPtr<IAvaRundownPageView> PageViewWeak;
	TWeakPtr<SAvaRundownPageList> PageListWeak;

private:
	const FSlateBrush* GetRowColorBrush(EAvaRundownRowState InRowState) const;
};
