// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownReadPage.h"

#include "Framework/Application/SlateApplication.h"
#include "Rundown/AvaRundownEditor.h"
#include "SAvaRundownInstancedPageList.h"
#include "SAvaRundownPageList.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "SAvaRundownReadPage"

class SAvaRundownReadPageEditableTextBox : public SEditableTextBox
{
public:
	bool IsKeyRelevant(const FKeyEvent& InKeyEvent) const
	{
		static const TSet<FKey> NumPadKeys { EKeys::NumPadZero, EKeys::NumPadOne, EKeys::NumPadTwo
			, EKeys::NumPadThree, EKeys::NumPadFour, EKeys::NumPadFive, EKeys::NumPadSix, EKeys::NumPadSeven
			, EKeys::NumPadEight, EKeys::NumPadNine };

		const bool bTextNeedsFocus = EditableText.IsValid() && !EditableText->HasKeyboardFocus();

		return bTextNeedsFocus && NumPadKeys.Contains(InKeyEvent.GetKey());
	}

	bool ProcessRundownKeyDown(const FKeyEvent& InKeyEvent)
	{
		// Check if Editable Text needs focus, and that the Key Event is a NumPad Key event
		if (IsKeyRelevant(InKeyEvent))
		{
			SetText(FText::GetEmpty());
			TSharedRef<SWidget> TextWidget = EditableText.ToSharedRef();
			FSlateApplication::Get().SetKeyboardFocus(TextWidget);
			TextWidget->OnKeyDown(TextWidget->GetTickSpaceGeometry(), InKeyEvent);
			return true;
		}
		return false;
	}
};

void SAvaRundownReadPage::Construct(const FArguments& InArgs, TSharedPtr<FAvaRundownEditor> InRundownEditor)
{
	RundownEditorWeak = InRundownEditor;
	InRundownEditor->GetOnPageEvent().AddSP(this, &SAvaRundownReadPage::OnPageEvent);

	const int32 PageId = InRundownEditor->GetFirstSelectedPageOnActiveSubListWidget();
	const FText PageIdText = PageId != FAvaRundownPage::InvalidPageId
		? FText::AsNumber(PageId, &UE::AvaRundown::FEditorMetrics::PageIdFormattingOptions)
		: FText::GetEmpty();

	// Small overestimate of 5 digit width.
	// Only relevant when text is empty (as numbers themselves are formatted for min 5 digits)
	constexpr float ReadPageWidth = 40.f;

	ChildSlot
	.Padding(2.f, 0.f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			SAssignNew(ReadPageText, SAvaRundownReadPageEditableTextBox)
			.Text(PageIdText)
			.OnVerifyTextChanged(this, &SAvaRundownReadPage::OnVerifyTextChanged)
			.OnTextCommitted(this, &SAvaRundownReadPage::OnTextCommitted)
			.MinDesiredWidth(ReadPageWidth)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.Text(LOCTEXT("ReadPageButton", "Read Page"))
			.OnClicked(this, &SAvaRundownReadPage::OnReadPageClicked)
		]
	];
}

SAvaRundownReadPage::~SAvaRundownReadPage()
{
	if (RundownEditorWeak.IsValid())
	{
		RundownEditorWeak.Pin()->GetOnPageEvent().RemoveAll(this);
	}
}

void SAvaRundownReadPage::OnPageEvent(const TArray<int32>& InSelectedPageIds, UE::AvaRundown::EPageEvent InPageEvent)
{
	if (!InSelectedPageIds.IsEmpty())
	{
		ReadPageText->SetText(FText::AsNumber(InSelectedPageIds[0]
			, &UE::AvaRundown::FEditorMetrics::PageIdFormattingOptions));
	}
	else
	{
		ReadPageText->SetText(FText::GetEmpty());
	}
}

bool SAvaRundownReadPage::IsKeyRelevant(const FKeyEvent& InKeyEvent) const
{
	if (ReadPageText.IsValid())
	{
		return ReadPageText->IsKeyRelevant(InKeyEvent);
	}
	return false;
}

bool SAvaRundownReadPage::ProcessRundownKeyDown(const FKeyEvent& InKeyEvent)
{
	if (ReadPageText.IsValid())
	{
		return ReadPageText->ProcessRundownKeyDown(InKeyEvent);
	}
	return false;
}

bool SAvaRundownReadPage::OnVerifyTextChanged(const FText& InText, FText& OutErrorMessage) const
{
	const FString String = InText.ToString();

	if (String.IsEmpty())
	{
		//Allow for the Text to be Empty (i.e. no pages are selected)
		return true;
	}
	
	if (!String.IsNumeric())
	{
		OutErrorMessage = LOCTEXT("NonNumericError", "Text is not Numeric");
		return false;
	}

	UAvaRundown* const Rundown = RundownEditorWeak.IsValid()
		? RundownEditorWeak.Pin()->GetRundown()
		: nullptr;

	if (!Rundown)
	{
		OutErrorMessage = LOCTEXT("NoRundown", "No Rundown found for Item");
		return false;
	}
	
	const int32 Id = FCString::Atoi(*String);

	if (!Rundown->GetPage(Id).IsValidPage())
	{
		OutErrorMessage = LOCTEXT("NoPageFound", "No Page with given Page Id exists");
		return false;
	}
	
	return true;
}

void SAvaRundownReadPage::OnTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin();
	check(RundownEditor.IsValid());

	TSharedPtr<SAvaRundownInstancedPageList> PageList = RundownEditor->GetActiveListWidget();

	if (PageList.IsValid())
	{
		if (InText.IsNumeric())
		{
			const int32 Id = FCString::Atoi(*InText.ToString());

			FModifierKeysState KeyState = FSlateApplication::Get().GetModifierKeys();
			const bool bAddtoSelection = KeyState.IsControlDown() || KeyState.IsCommandDown() || KeyState.IsAltDown();

			if (!bAddtoSelection)
			{
				PageList->DeselectPages();
			}

			PageList->SelectPage(Id);
		}
		else if (InText.IsEmpty())
		{
			PageList->DeselectPages();
		}
	}
}

FReply SAvaRundownReadPage::OnReadPageClicked()
{
	OnTextCommitted(ReadPageText->GetText(), ETextCommit::Type::OnEnter);
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
