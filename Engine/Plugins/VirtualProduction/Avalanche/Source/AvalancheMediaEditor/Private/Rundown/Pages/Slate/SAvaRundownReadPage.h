// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/AvaRundownEditorDefines.h"
#include "Widgets/SCompoundWidget.h"

class FAvaRundownEditor;
class SAvaRundownReadPageEditableTextBox;

class SAvaRundownReadPage : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SAvaRundownReadPage) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FAvaRundownEditor> InRundownEditor);

	virtual ~SAvaRundownReadPage() override;

	void OnPageEvent(const TArray<int32>& InSelectedPageIds, UE::AvaRundown::EPageEvent InPageEvent);

	bool IsKeyRelevant(const FKeyEvent& InKeyEvent) const;

	bool ProcessRundownKeyDown(const FKeyEvent& InKeyEvent);

	bool OnVerifyTextChanged(const FText& InText, FText& OutErrorMessage) const;

	void OnTextCommitted(const FText& InText, ETextCommit::Type InCommitType);

	FReply OnReadPageClicked();

private:
	TWeakPtr<FAvaRundownEditor> RundownEditorWeak;

	TSharedPtr<SAvaRundownReadPageEditableTextBox> ReadPageText;
};
