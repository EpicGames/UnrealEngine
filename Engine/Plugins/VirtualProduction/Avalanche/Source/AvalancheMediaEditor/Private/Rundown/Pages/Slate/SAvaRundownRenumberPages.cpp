// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownRenumberPages.h"
#include "Framework/Application/SlateApplication.h"
#include "Input/Reply.h"
#include "SPrimaryButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaRundownRenumberPages"

void SAvaRundownRenumberPages::Construct(const FArguments& InArgs)
{
	OnAccept = InArgs._OnAccept;

	SWindow::Construct(SWindow::FArguments()
		.Title(LOCTEXT("DialogTitle", "Renumber Pages"))
		.AutoCenter(EAutoCenter::PrimaryWorkArea)
		.SizingRule(ESizingRule::Autosized)
		.SupportsMinimize(false)
		.SupportsMaximize(false));
	
	SetContent(
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10.f)
		[
			ConstructBaseNumberWidget()
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10.f, 0.f, 10.f, 10.f)
		[
			ConstructIncrementWidget()
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10.f, 5.f, 10.f, 10.f)
		.HAlign(HAlign_Right)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(FMargin(5.f, 0.f))
			+ SUniformGridPanel::Slot(0, 0)
			[
				SNew(SPrimaryButton)
				.Text(LOCTEXT("OkButtonText", "OK"))
				.OnClicked(this, &SAvaRundownRenumberPages::HandleAcceptClick)
			]
			+ SUniformGridPanel::Slot(1, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("CancelButtonText", "Cancel"))
				.OnClicked(this, &SAvaRundownRenumberPages::HandleCancelClick)
			]
		]
	);
}

TSharedRef<SWidget> SAvaRundownRenumberPages::ConstructBaseNumberWidget()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(0.5f)
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BaseNumberLabel", "Base Number:"))
		]
		+ SHorizontalBox::Slot()
		.FillWidth(0.5f)
		.Padding(10.f, 0.f, 0.f, 0.f)
		[
			SNew(SNumericEntryBox<int32>)
			.AllowSpin(true)
			.MinValue(0)
			.MinSliderValue(0)
			.MaxSliderValue(MaxSliderValue)
			.Value(this, &SAvaRundownRenumberPages::GetBaseNumber)
			.OnValueChanged(this, &SAvaRundownRenumberPages::HandleBaseNumberChanged)
		];
}

TSharedRef<SWidget> SAvaRundownRenumberPages::ConstructIncrementWidget()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(0.5f)
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("IncrementLabel", "Increment:"))
		]
		+ SHorizontalBox::Slot()
		.FillWidth(0.5f)
		.Padding(10.f, 0.f, 0.f, 0.f)
		[
			SNew(SNumericEntryBox<int32>)
			.AllowSpin(true)
			.MinValue(1)
			.MinSliderValue(1)
			.MaxSliderValue(MaxSliderValue)
			.Value(this, &SAvaRundownRenumberPages::GetIncrement)
			.OnValueChanged(this, &SAvaRundownRenumberPages::HandleIncrementChanged)
		];
}

TOptional<int32> SAvaRundownRenumberPages::GetBaseNumber() const
{
	return BaseNumber;
}

void SAvaRundownRenumberPages::HandleBaseNumberChanged(const int32 InNewValue)
{
	BaseNumber = InNewValue;
}

TOptional<int32> SAvaRundownRenumberPages::GetIncrement() const
{
	return Increment;
}

void SAvaRundownRenumberPages::HandleIncrementChanged(const int32 InNewValue)
{
	Increment = InNewValue;
}

FReply SAvaRundownRenumberPages::HandleAcceptClick()
{
	OnAccept.ExecuteIfBound(BaseNumber, Increment);

	RequestDestroyWindow();

	return FReply::Handled();
}

FReply SAvaRundownRenumberPages::HandleCancelClick()
{
	RequestDestroyWindow();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
