// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "Widgets/SWindow.h"

class FReply;

class SAvaRundownRenumberPages : public SWindow
{
public:
	DECLARE_DELEGATE_TwoParams(FOnRundownRenumberPagesResult, const int32 /*BaseNumber*/, const int32 /*Increment*/)

	SLATE_BEGIN_ARGS(SAvaRundownRenumberPages)
	{}
		SLATE_EVENT(FOnRundownRenumberPagesResult, OnAccept)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:
	static constexpr int32 MaxSliderValue = 1000;

	TSharedRef<SWidget> ConstructBaseNumberWidget();
	TSharedRef<SWidget> ConstructIncrementWidget();

	TOptional<int32> GetBaseNumber() const;
	void HandleBaseNumberChanged(const int32 InNewValue);

	TOptional<int32> GetIncrement() const;
	void HandleIncrementChanged(const int32 InNewValue);

	FReply HandleAcceptClick();
	FReply HandleCancelClick();

	FOnRundownRenumberPagesResult OnAccept;

	int32 BaseNumber = 0;
	int32 Increment = 1;
};
