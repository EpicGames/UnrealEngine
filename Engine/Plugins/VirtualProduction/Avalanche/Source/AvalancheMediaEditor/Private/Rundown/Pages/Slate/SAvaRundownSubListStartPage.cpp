// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownSubListStartPage.h"

#include "Rundown/AvaRundownEditor.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaRundownSubListStartPage"

void SAvaRundownSubListStartPage::Construct(const FArguments& InArgs, TSharedPtr<FAvaRundownEditor> InRundownEditor)
{
	RundownEditorWeak = InRundownEditor;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(10.f, 10.f)
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		[
			SNew(SButton)
			.ToolTipText(LOCTEXT("AddSubListTooltip", "Add Page View"))
			.OnClicked(this, &SAvaRundownSubListStartPage::OnCreateSubListClicked)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AddSubList", "Add Page View"))
			]
		]
		+ SVerticalBox::Slot()
		.Padding(10.f, 10.f)
		.HAlign(EHorizontalAlignment::HAlign_Center)
		.VAlign(EVerticalAlignment::VAlign_Top)
		.AutoHeight()
		[
			SNew(SButton)
			.ToolTipText(LOCTEXT("ShowAllSubListTooltip", "Show All Page Views"))
			.OnClicked(this, &SAvaRundownSubListStartPage::OnShowAllSubListsClicked)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ShowAllSubList", "Show All Page Views"))
			]
		]
	];
}

FReply SAvaRundownSubListStartPage::OnCreateSubListClicked()
{
	if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{		
		if (UAvaRundown* Rundown = RundownEditor->GetRundown())
		{
			FScopedTransaction Transaction(LOCTEXT("AddPageView", "Add PageView"));
			Rundown->Modify();
			const FAvaRundownPageListReference CreatedSubListReference = Rundown->AddSubList();
			Rundown->SetActivePageList(CreatedSubListReference);
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

FReply SAvaRundownSubListStartPage::OnShowAllSubListsClicked()
{
	if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		RundownEditor->RefreshSubListTabs();
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
