// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FAvaRundownEditor;

class SAvaRundownSubListStartPage : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaRundownSubListStartPage) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FAvaRundownEditor> InRundownEditor);
	virtual ~SAvaRundownSubListStartPage() override {}

protected:
	TWeakPtr<FAvaRundownEditor> RundownEditorWeak;

	FReply OnCreateSubListClicked();
	FReply OnShowAllSubListsClicked();
};
