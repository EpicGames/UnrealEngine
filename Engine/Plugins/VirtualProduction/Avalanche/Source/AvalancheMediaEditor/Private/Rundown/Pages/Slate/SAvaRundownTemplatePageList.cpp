// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownTemplatePageList.h"

#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownCommands.h"
#include "Rundown/AvaRundownEditor.h"
#include "Rundown/AvaRundownEditorUtils.h"
#include "Rundown/AvaRundownManagedInstanceCache.h"
#include "Rundown/AvaRundownPage.h"
#include "Rundown/Pages/Columns/AvaRundownPageAssetSelectorColumn.h"
#include "Rundown/Pages/Columns/AvaRundownPageIdColumn.h"
#include "Rundown/Pages/Columns/AvaRundownPageNameColumn.h"
#include "Rundown/Pages/Columns/AvaRundownPageTemplateStatusColumn.h"
#include "Rundown/Pages/Columns/AvaRundownPageThumbnailColumn.h"
#include "Rundown/Pages/Columns/AvaRundownPageTransitionLayerColumn.h"
#include "Rundown/Pages/PageViews/AvaRundownTemplatePageViewImpl.h"
#include "ScopedTransaction.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SAvaRundownTemplatePageList"

void SAvaRundownTemplatePageList::PrivateRegisterAttributes(struct FSlateAttributeDescriptor::FInitializer&)
{

}

void SAvaRundownTemplatePageList::Construct(const FArguments& InArgs, TSharedPtr<FAvaRundownEditor> InRundownEditor)
{
	SAvaRundownPageList::Construct(SAvaRundownPageList::FArguments(), InRundownEditor, UAvaRundown::TemplatePageList);

	RundownEditorWeak = InRundownEditor;
	check(InRundownEditor.IsValid());

	UAvaRundown* const Rundown = InRundownEditor->GetRundown();
	check(Rundown);

	Rundown->GetOnPageListChanged().AddSP(this, &SAvaRundownTemplatePageList::OnPageListChanged);

	Refresh();
}

SAvaRundownTemplatePageList::~SAvaRundownTemplatePageList()
{
	if (UAvaRundown* const Rundown = GetValidRundown())
	{
		Rundown->GetOnPageListChanged().RemoveAll(this);
	}
}

void SAvaRundownTemplatePageList::Refresh()
{
	if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		UAvaRundown* const Rundown = RundownEditor->GetRundown();
		if (!Rundown)
		{
			return;
		}

		const TArray<FAvaRundownPage>& Pages = Rundown->GetTemplatePages().Pages;
		const int32 VisiblePageCount = VisiblePageIds.IsEmpty() ? Pages.Num() : VisiblePageIds.Num();
			
		if (PageViews.Num() != VisiblePageCount)
		{
			PageViews.Reset(Pages.Num());

			for (const FAvaRundownPage& Page : Pages)
			{
				if (IsPageVisible(Page))
				{
					PageViews.Emplace(MakeShared<FAvaRundownTemplatePageViewImpl>(Page.GetPageId(), Rundown, SharedThis(this)));
				}
			}
		}
		else
		{
			// Number of page didn't change, just refresh ids.
			int32 PageViewIndex = 0;
			for (const FAvaRundownPage& Page : Pages)
			{
				if (IsPageVisible(Page))
				{
					if (FAvaRundownPageViewImpl* PageView = PageViews[PageViewIndex]->CastTo<FAvaRundownPageViewImpl>())
					{
						PageView->RefreshPageId(Page.GetPageId());
					}
					++PageViewIndex;
				}
			}
		}

		PageListView->RequestListRefresh();
	}
}

void SAvaRundownTemplatePageList::CreateColumns()
{
	HeaderRow = SNew(SHeaderRow)
		.Visibility(EVisibility::Visible)
		.CanSelectGeneratedColumn(true);

	Columns.Empty();
	HeaderRow->ClearColumns();

	//TODO: Extensibility?
	TArray<TSharedPtr<IAvaRundownPageViewColumn>> FoundColumns;
	FoundColumns.Add(MakeShared<FAvaRundownPageThumbnailColumn>());
	FoundColumns.Add(MakeShared<FAvaRundownPageIdColumn>());
	FoundColumns.Add(MakeShared<FAvaRundownPageNameColumn>());
	FoundColumns.Add(MakeShared<FAvaRundownPageAssetSelectorColumn>());
	FoundColumns.Add(MakeShared<FAvaRundownPageTransitionLayerColumn>());
	FoundColumns.Add(MakeShared<FAvaRundownPageTemplateStatusColumn>());

	for (const TSharedPtr<IAvaRundownPageViewColumn>& Column : FoundColumns)
	{
		const FName ColumnId = Column->GetColumnId();
		Columns.Add(ColumnId, Column);
		HeaderRow->AddColumn(Column->ConstructHeaderRowColumn());
		HeaderRow->SetShowGeneratedColumn(ColumnId, false);
	}
}

TSharedPtr<SWidget> SAvaRundownTemplatePageList::OnContextMenuOpening()
{
	if (TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		return GetPageListContextMenu();
	}

	return SNullWidget::NullWidget;
}

void SAvaRundownTemplatePageList::BindCommands()
{
	SAvaRundownPageList::BindCommands();

	//Rundown Commands
	{
		const FAvaRundownCommands& RundownCommands = FAvaRundownCommands::Get();

		CommandList->MapAction(RundownCommands.AddTemplate,
			FExecuteAction::CreateSP(this, &SAvaRundownTemplatePageList::AddTemplate),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanAddTemplate));

		CommandList->MapAction(RundownCommands.CreatePageInstanceFromTemplate,
			FExecuteAction::CreateSP(this, &SAvaRundownTemplatePageList::CreateInstance),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanCreateInstance));

		CommandList->MapAction(RundownCommands.CreateComboTemplate,
			FExecuteAction::CreateSP(this, &SAvaRundownTemplatePageList::CreateComboTemplate),
			FCanExecuteAction::CreateSP(this, &SAvaRundownTemplatePageList::CanCreateComboTemplate));

		CommandList->MapAction(RundownCommands.RemovePage,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::RemoveSelectedPages),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanRemoveSelectedPages));

		CommandList->MapAction(RundownCommands.RenumberPage,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::RenumberSelectedPages),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanRenumberSelectedPages));

		CommandList->MapAction(RundownCommands.ReimportPage,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::ReimportSelectedPage),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanReimportSelectedPage));

		CommandList->MapAction(RundownCommands.EditPageSource,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::EditSelectedPageSource),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanEditSelectedPageSource));

		CommandList->MapAction(RundownCommands.ResetValuesToDefaults,
			FExecuteAction::CreateSP(this, &SAvaRundownTemplatePageList::ResetPagesToDefaults),
			FCanExecuteAction::CreateSP(this, &SAvaRundownTemplatePageList::CanResetPagesToDefaults));

		CommandList->MapAction(RundownCommands.PreviewFrame,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::PreviewPlaySelectedPage, true),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanPreviewPlaySelectedPage));

		CommandList->MapAction(RundownCommands.PreviewPlay,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::PreviewPlaySelectedPage, false),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanPreviewPlaySelectedPage));

		CommandList->MapAction(RundownCommands.PreviewStop,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::PreviewStopSelectedPage, false),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanPreviewStopSelectedPage, false));

		CommandList->MapAction(RundownCommands.PreviewForceStop,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::PreviewStopSelectedPage, true),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanPreviewStopSelectedPage, true));

		CommandList->MapAction(RundownCommands.PreviewContinue,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::PreviewContinueSelectedPage),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanPreviewContinueSelectedPage));

		CommandList->MapAction(RundownCommands.PreviewPlayNext,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::PreviewPlayNextPage),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanPreviewPlayNextPage));

		CommandList->MapAction(RundownCommands.TakeToProgram,
			FExecuteAction::CreateSP(this, &SAvaRundownPageList::TakeToProgram),
			FCanExecuteAction::CreateSP(this, &SAvaRundownPageList::CanTakeToProgram));
	}
}

bool SAvaRundownTemplatePageList::HandleDropAssets(const TArray<FSoftObjectPath>& InAvaAssets, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem)
{
	UAvaRundown* Rundown = GetValidRundown();
	if (!Rundown)
	{
		return false;
	}

	FAvaRundownPageInsertPosition InsertAt = MakeInsertPosition(InDropZone, InItem);
	TArray<FSoftObjectPath> NewAvaAssets = InAvaAssets;
	TArray<int32> NewTemplateIds;

	// If we are adding above, they should be reversed, so the last is added first
	// and the next to last added above that, etc.
	if (InsertAt.IsAddAbove())
	{
		Algo::Reverse(NewAvaAssets);
	}

	bool bHasValidAssets = false;
	Rundown->Modify();

	for (const FSoftObjectPath& AvaAsset : NewAvaAssets)
	{
		if (AvaAsset.IsNull())
		{
			continue;
		}

		int32 NewTemplateId = Rundown->AddTemplate(FAvaRundownPageIdGeneratorParams::FromInsertPosition(InsertAt));
		
		if (NewTemplateId == FAvaRundownPage::InvalidPageId)
		{
			continue;
		}

		Rundown->GetPage(NewTemplateId).UpdateAsset(AvaAsset);
		InsertAt.ConditionalUpdateAdjacentId(NewTemplateId);
		NewTemplateIds.Add(NewTemplateId);

		bHasValidAssets = true;
	}

	if (bHasValidAssets)
	{
		Refresh();
		DeselectPages();
		SelectPages(NewTemplateIds);
	}

	return bHasValidAssets;
}

bool SAvaRundownTemplatePageList::HandleDropRundowns(const TArray<FSoftObjectPath>& InRundownPaths, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem)
{
	// Not supported directly.
	// The templates will import automatically when the rundown pages are imported.
	return false;	
}

bool SAvaRundownTemplatePageList::HandleDropPageIds(const FAvaRundownPageListReference& InPageListReference, const TArray<int32>& InPageIds, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem)
{
	// Can only drop templates onto the templates list.
	if (InPageListReference.Type != EAvaRundownPageListType::Template)
	{
		return false;
	}

	// Nothing to do.
	if (InPageIds.IsEmpty())
	{
		return true;
	}

	UAvaRundown* Rundown = GetValidRundown();
	if (!Rundown || !Rundown->CanChangePageOrder())
	{
		return false;
	}
	
	const FAvaRundownPageCollection& TemplatePageCollection = Rundown->GetTemplatePages();
	int32 DroppedOnPageIndex = INDEX_NONE;

	if (InItem.IsValid() && TemplatePageCollection.PageIndices.Contains(InItem->GetPageId()))
	{
		DroppedOnPageIndex = TemplatePageCollection.PageIndices[InItem->GetPageId()];

		// Nothing to do.
		if (InPageIds.Num() == 1 && DroppedOnPageIndex == InPageIds[0])
		{
			return true;
		}
	}

	TArray<int32> MovedPageIdIndices;
	TArray<int32> NewPageOrder;
	TArray<int32> NewSelectedIds;

	MovedPageIdIndices.Reserve(InPageIds.Num());
	NewPageOrder.Reserve(PageViews.Num());
	NewSelectedIds.Reserve(InPageIds.Num());

	// This has the byproduct of removing invalid ids.
	for (int32 PageId : InPageIds)
	{
		if (const int32* PageIndexPtr = TemplatePageCollection.PageIndices.Find(PageId))
		{
			MovedPageIdIndices.Add(*PageIndexPtr);
			NewSelectedIds.Add(PageId);
		}
	}

	// Nothing to do.
	if (MovedPageIdIndices.IsEmpty())
	{
		return true;
	}

	for (int32 PageViewIndex = 0; PageViewIndex < PageViews.Num(); ++PageViewIndex)
	{
		const bool bHasMovedThisPage = MovedPageIdIndices.Contains(PageViewIndex);

		if (PageViewIndex == DroppedOnPageIndex)
		{
			const bool bAddBefore = InDropZone == EItemDropZone::AboveItem;

			if (bAddBefore)
			{
				NewPageOrder.Append(MovedPageIdIndices);
			}

			// If we moved the page we dropped onto, it will already be in the list
			if (!bHasMovedThisPage)
			{
				NewPageOrder.Add(PageViewIndex);
			}

			if (!bAddBefore)
			{
				NewPageOrder.Append(MovedPageIdIndices);
			}

			continue;
		}

		if (bHasMovedThisPage)
		{
			continue;
		}

		NewPageOrder.Add(PageViewIndex);
	}

	Rundown->ChangePageOrder(PageListReference, NewPageOrder);
	SelectPages(NewSelectedIds, true);

	return true;
}

bool SAvaRundownTemplatePageList::HandleDropExternalFiles(const TArray<FString>& InFiles, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem)
{
	// Not supported directly.
	// The templates will import automatically when the rundown pages are imported.
	return false;
}

void SAvaRundownTemplatePageList::AddTemplate()
{
	if (UAvaRundown* Rundown = GetValidRundown())
	{
		int32 LastIndex = FAvaRundownPage::InvalidPageId;

		for (const int32 SelectedIndex : SelectedPageIds)
		{
			LastIndex = FMath::Max(LastIndex, SelectedIndex);
		}

		const int32 NewTemplateId = Rundown->AddTemplate(FAvaRundownPageIdGeneratorParams(LastIndex));

		if (NewTemplateId != FAvaRundownPage::InvalidPageId)
		{
			DeselectPages();
			SelectPage(NewTemplateId);
		}
	}
}

void SAvaRundownTemplatePageList::CreateInstance()
{
	if (SelectedPageIds.IsEmpty())
	{
		return;
	}

	if (UAvaRundown* Rundown = GetValidRundown())
	{
		const TArray<int32> NewPageIds = Rundown->AddPagesFromTemplates(SelectedPageIds);

		if (!NewPageIds.IsEmpty())
		{
			DeselectPages();
			SelectPages(NewPageIds);
		}
	}
}

class FAvaRundownTemplatePageListErrorContext : public FOutputDevice
{
public:
	const TCHAR* ContextName;

	FAvaRundownTemplatePageListErrorContext(const TCHAR* InContextName)
		: FOutputDevice()
		, ContextName(InContextName)
	{
	}

	virtual void Serialize(const TCHAR* InText, ELogVerbosity::Type InVerbosity, const class FName& InCategory) override
	{
		UE_LOGF(LogAvaRundown, Error, "%ls: %ls", ContextName, InText);
	}
};


void SAvaRundownTemplatePageList::CreateComboTemplate()
{
	UAvaRundown* Rundown = GetValidRundown();
	
	if (SelectedPageIds.IsEmpty() || !Rundown)
	{
		return;
	}

	FAvaRundownTemplatePageListErrorContext ErrorContext(TEXT("CreateComboTemplate"));
	const TArray<int32> TemplateIds = Rundown->ValidateTemplateIdsForComboTemplate(SelectedPageIds, ErrorContext);

	if (TemplateIds.Num() > 1)
	{
		Rundown->AddComboTemplate(TemplateIds);
	}
	else
	{
		ErrorContext.Log(TEXT("Need at least 2 suitable templates to create a combo template."));
	}
}

bool SAvaRundownTemplatePageList::CanCreateComboTemplate()
{
	UAvaRundown* Rundown = GetValidRundown();
	
	if (SelectedPageIds.Num() < 2 || !Rundown)
	{
		return false;
	}

	TSet<FAvaTagId> LayerIds;

	for (const int32 SelectedPageId : SelectedPageIds)
	{
		const FAvaRundownPage& Page = Rundown->GetPage(SelectedPageId);
		if (Page.IsValidPage()
			&& Page.IsTemplate()
			&& !Page.IsComboTemplate()
			&& Page.HasTransitionLogic(Rundown)
			&& Page.GetTransitionLayer(Rundown).IsValid()
			&& !LayerIds.Contains(Page.GetTransitionLayer(Rundown).TagId))
		{
			LayerIds.Add(Page.GetTransitionLayer(Rundown).TagId);
		}
	}
	// Need more than one template to create a combo template.
	return LayerIds.Num() > 1;
}

TArray<int32> SAvaRundownTemplatePageList::AddPastedPages(const TArray<FAvaRundownPage>& InPages)
{
	if (UAvaRundown* Rundown = GetValidRundown())
	{
		using namespace UE::AvaRundownEditor::Utils;
		FImportTemplateMap ImportedTemplateIds;
		return ImportTemplatePages(Rundown, InPages, ImportedTemplateIds);
	}
	return {};
}

void SAvaRundownTemplatePageList::OnPageListChanged(const FAvaRundownPageListChangeParams& InParams)
{
	if (PageListReference != InParams.PageListReference)
	{
		return;
	}

	RefreshPagesVisibility();
	Refresh();
}

void SAvaRundownTemplatePageList::ResetPagesToDefaults()
{
	UAvaRundown* Rundown = GetValidRundown();
	
	if (SelectedPageIds.IsEmpty() || !Rundown)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ResetPagesTransaction", "Reset Pages"));
	Rundown->Modify();
	
	for (const int32 SelectedPageId : SelectedPageIds)
	{
		const FAvaRundownPage& Page = Rundown->GetPage(SelectedPageId);

		if (!Page.IsValidPage() || !Page.IsEnabled() || !Page.IsTemplate())
		{
			continue;
		}
		
		Rundown->ResetRemoteControlValues(SelectedPageId, /*bInUseTemplateValues=*/false, /*bInIsDefault=*/false);
	}
}

bool SAvaRundownTemplatePageList::CanResetPagesToDefaults() const
{
	const UAvaRundown* const Rundown = GetValidRundown();

	if (SelectedPageIds.IsEmpty() || !Rundown)
	{
		return false;
	}

	bool bContainsDifferentValues = false;

	for (const int32 SelectedPageId : SelectedPageIds)
	{
		const FAvaRundownPage& Page = Rundown->GetPage(SelectedPageId);

		if (!Page.IsValidPage() || !Page.IsEnabled() || !Page.IsTemplate())
		{
			return false;
		}

		FAvaPlayableRemoteControlValues DefaultValues;
		if (Page.GetDefaultRemoteControlValues(Rundown, /*bInUseTemplateValues=*/false, DefaultValues))
		{
			const FAvaPlayableRemoteControlValues& PageValues = Page.GetRemoteControlValues();
			if (!(PageValues.HasSameEntityValues(DefaultValues) && PageValues.HasSameControllerValues(DefaultValues)))
			{
				bContainsDifferentValues = true;
			}
		}
	}

	return bContainsDifferentValues;
}

#undef LOCTEXT_NAMESPACE
