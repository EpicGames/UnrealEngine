// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownEditorDefines.h"
#include "SAvaRundownPageList.h"

class FAvaRundownEditor;
class IAvaRundownPageViewColumn;
class ITableRow;
class SHeaderRow;
class SSearchBox;
class STableViewBase;
struct FAvaRundownPage;
template <typename ItemType> class SListView;

class SAvaRundownTemplatePageList : public SAvaRundownPageList
{
	SLATE_DECLARE_WIDGET(SAvaRundownTemplatePageList, SAvaRundownPageList)

public:
	SLATE_BEGIN_ARGS(SAvaRundownTemplatePageList) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TSharedPtr<FAvaRundownEditor> InRundownEditor);
	virtual ~SAvaRundownTemplatePageList() override;

	//~ Begin SAvaRundownPageList
	virtual void Refresh() override;
	virtual void CreateColumns() override;
	virtual TSharedPtr<SWidget> OnContextMenuOpening() override;
	virtual void BindCommands() override;
	virtual bool HandleDropAssets(const TArray<FSoftObjectPath>& InAvaAssets, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem) override;
	virtual bool HandleDropRundowns(const TArray<FSoftObjectPath>& InRundownPaths, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem) override;
	virtual bool HandleDropPageIds(const FAvaRundownPageListReference& InPageListReference, const TArray<int32>& InPageIds, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem) override;
	virtual bool HandleDropExternalFiles(const TArray<FString>& InFiles, EItemDropZone InDropZone, const FAvaRundownPageViewPtr& InItem) override;
	//~ End SAvaRundownPageList

	void AddTemplate();

	void CreateInstance();
	void CreateComboTemplate();
	bool CanCreateComboTemplate();

protected:
	virtual TArray<int32> AddPastedPages(const TArray<FAvaRundownPage>& InPages) override;

	void ResetPagesToDefaults();
	bool CanResetPagesToDefaults() const;

private:
	void OnPageListChanged(const FAvaRundownPageListChangeParams& InParams);
};
