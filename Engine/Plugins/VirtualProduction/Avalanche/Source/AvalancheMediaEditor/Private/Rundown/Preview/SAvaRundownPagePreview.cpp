// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownPagePreview.h"

#include "AvaMediaEditorStyle.h"
#include "AvaMediaSettings.h"
#include "Broadcast/OutputDevices/Slate/SAvaBroadcastCaptureImage.h"
#include "Brushes/SlateImageBrush.h"
#include "Engine/RendererSettings.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IAvaMediaEditorModule.h"
#include "Internationalization/Text.h"
#include "ISettingsModule.h"
#include "Layout/Visibility.h"
#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownCommands.h"
#include "Rundown/AvaRundownEditor.h"
#include "Rundown/AvaRundownEditorSettings.h"
#include "Rundown/Preview/SAvaRundownPreviewChannelSelector.h"
#include "Styling/SlateBrush.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "SAvaRundownPagePreview"

/**
 * Wrapper to encapsulate a FSlateImageBrush that can be updated with a render target.
 * If the render target becomes null, the brush is updated to be a solid color.
 */
struct SAvaRundownPagePreview::FPreviewBrush
{
	FSlateImageBrush Brush;
	FVector2D RenderTargetSize;

	FPreviewBrush(const FVector2D& InRenderTargetSize)
		: Brush(NAME_None, InRenderTargetSize, FLinearColor::Black)
		, RenderTargetSize(InRenderTargetSize)
	{

	}

	void Update(UTextureRenderTarget2D* InRenderTarget)
	{
		if (InRenderTarget)
		{
			RenderTargetSize = FVector2D(InRenderTarget->SizeX, InRenderTarget->SizeY);

			//If Brush is invalid, or the Brush's Texture Target doesn't match the new Render Target, reset the Brush.
			if (Brush.GetResourceObject() != InRenderTarget)
			{
				Brush = FSlateImageBrush(InRenderTarget, RenderTargetSize);
			}
			//If Sizes mismatch, just resizes rather than recreating the Brush with same underlying Resource
			else if (Brush.GetImageSize() != RenderTargetSize)
			{
				Brush.SetImageSize(RenderTargetSize);
			}
		}
		else if (Brush.GetResourceObject() != nullptr || Brush.GetImageSize() != RenderTargetSize)
		{
			// Preserve the size of the preview brush so the checkerboard remains the same.
			Brush = FSlateImageBrush(NAME_None, RenderTargetSize, FLinearColor::Black);
		}
	}
};

void SAvaRundownPagePreview::Construct(const FArguments& InArgs, const TSharedPtr<FAvaRundownEditor>& InRundownEditor)
{
	RundownEditorWeak = InRundownEditor;

	const FIntPoint DefaultRenderTargetSize = UAvaMediaSettings::Get().PreviewDefaultResolution;
	PreviewBrush = MakeUnique<FPreviewBrush>(FVector2d(DefaultRenderTargetSize.X, DefaultRenderTargetSize.Y));

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			CreatePagePreviewToolBar(InRundownEditor->GetToolkitCommands())
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFit)
			.StretchDirection(EStretchDirection::Both)
			[
				SNew( SOverlay )
				+SOverlay::Slot()
				[
					SNew( SImage )
					.Image( FAvaMediaEditorStyle::Get().GetBrush( "AvaMediaEditor.Checkerboard" ) )
					.Visibility(this, &SAvaRundownPagePreview::GetCheckerboardVisibility)
				]
				+SOverlay::Slot()
				[
					SNew(SAvaBroadcastCaptureImage)
					.ImageArgs(SImage::FArguments()
						.Image(this, &SAvaRundownPagePreview::GetPreviewBrush))
					.ShouldInvertAlpha(true)
					.EnableGammaCorrection(false)
					.EnableBlending(this, &SAvaRundownPagePreview::IsBlendingEnabled)
				]
			]
		]
	];
}

SAvaRundownPagePreview::SAvaRundownPagePreview() = default;
SAvaRundownPagePreview::~SAvaRundownPagePreview() = default;

TSharedRef<SWidget> SAvaRundownPagePreview::CreatePagePreviewToolBar(const TSharedRef<FUICommandList>& InCommandList)
{
	FSlimHorizontalToolBarBuilder ToolBarBuilder(InCommandList, FMultiBoxCustomization::None);
	ToolBarBuilder.SetStyle(&FAvaMediaEditorStyle::Get(), "AvaMediaEditor.ToolBar");

	const FAvaRundownCommands& RundownCommands = FAvaRundownCommands::Get();

	ToolBarBuilder.BeginSection(TEXT("ShowControl"));
	{
		ToolBarBuilder.AddToolBarButton(RundownCommands.PreviewFrame);

		ToolBarBuilder.BeginStyleOverride("AvaMediaEditor.ToolBarGreenButtonOverride");
		{
			ToolBarBuilder.AddToolBarButton(RundownCommands.PreviewPlay);
		}
		ToolBarBuilder.EndStyleOverride();

		ToolBarBuilder.AddToolBarButton(RundownCommands.PreviewContinue);
		ToolBarBuilder.AddToolBarButton(RundownCommands.PreviewStop);
		ToolBarBuilder.AddToolBarButton(RundownCommands.PreviewPlayNext);

		ToolBarBuilder.BeginStyleOverride("AvaMediaEditor.ToolBarRedButtonOverride");
		{
			ToolBarBuilder.AddToolBarButton(RundownCommands.TakeToProgram);
		}
		ToolBarBuilder.EndStyleOverride();

		if (TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
		{
			ToolBarBuilder.AddWidget(RundownEditor->MakeReadPageWidget());	
		}
	}
	ToolBarBuilder.EndSection();

	ToolBarBuilder.BeginSection("Settings");
	{
		ToolBarBuilder.AddWidget(SNew(SSpacer), NAME_None, false, HAlign_Right);

		ToolBarBuilder.AddToolBarButton(
				FUIAction(
					FExecuteAction::CreateSP(this, &SAvaRundownPagePreview::HandleCheckerboardActionExecute),
					FCanExecuteAction::CreateLambda([]	{ return true;}))
				, NAME_None
				, LOCTEXT("AlphaPreview", "Alpha Preview")
				, LOCTEXT("AlphaPreview_ToolTip", "Toggle alpha preview (checker board).")
				, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Checkerboard")
			);

		ToolBarBuilder.BeginStyleOverride("AvaMediaEditor.CalloutToolbar");
		{
			ToolBarBuilder.AddComboButton(
				FUIAction(),
				FOnGetContent::CreateSP(this, &SAvaRundownPagePreview::OnGenerateSettingsMenu),
				LOCTEXT("SettingsMenu", "Preview Settings"),
				FText::GetEmpty(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Settings")
			);
		}
		ToolBarBuilder.EndStyleOverride();
	}
	ToolBarBuilder.EndSection();	

	return ToolBarBuilder.MakeWidget();
}

TSharedRef<SWidget> SAvaRundownPagePreview::OnGenerateSettingsMenu()
{
	TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin();
	TSharedPtr<FUICommandList> ToolkitCommands = RundownEditor.IsValid() ? RundownEditor->GetToolkitCommands() : TSharedPtr<FUICommandList>();
	
	FMenuBuilder MenuBuilder(true, ToolkitCommands);
	MenuBuilder.BeginSection("ResolutionSection", LOCTEXT("ResolutionSectionHeader", "Resolution Options"));
	{
		// Add a bunch of standard resolutions.
		AddResolutionMenuEntry(MenuBuilder, LOCTEXT("Res_3840x2160", "3840 x 2160"), FIntPoint(3840,2160));
		AddResolutionMenuEntry(MenuBuilder, LOCTEXT("Res_1920x1080", "1920 x 1080"), FIntPoint(1920,1080));
		AddResolutionMenuEntry(MenuBuilder, LOCTEXT("Res_960x540", "960 x 540"), FIntPoint(960,540));
		AddResolutionMenuEntry(MenuBuilder, LOCTEXT("Res_480x270", "480 x 270"), FIntPoint(480,270));
	}
	MenuBuilder.EndSection();

	MenuBuilder.AddMenuSeparator();

	MenuBuilder.AddWidget(SNew(SAvaRundownPreviewChannelSelector), LOCTEXT("PreviewChannelSelectorLabel", "Channel"));
	
	MenuBuilder.AddMenuSeparator();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("Settings", "Settings"),
		LOCTEXT("Settings_Tooltip", "Opens the Motion Design Media Settings."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SAvaRundownPagePreview::HandleSettingsActionExecute)
		),
		NAME_None,
		EUserInterfaceActionType::Button);
	
	return MenuBuilder.MakeWidget();
}

UTextureRenderTarget2D* SAvaRundownPagePreview::GetPreviewRenderTarget() const
{
	const TSharedPtr<FAvaRundownEditor> Editor = RundownEditorWeak.Pin();
	if (Editor.IsValid() && Editor->IsRundownValid())
	{
		return Editor->GetRundown()->GetPreviewRenderTarget();
	}
	return nullptr;
}

const FSlateBrush* SAvaRundownPagePreview::GetPreviewBrush() const
{
	// Update the brush here to ensure the cached image attribute is updated with the correct render target.
	PreviewBrush->Update(GetPreviewRenderTarget());
	return &PreviewBrush->Brush;
}

bool SAvaRundownPagePreview::IsBlendingEnabled() const
{
	const UAvaRundownEditorSettings* Settings = UAvaRundownEditorSettings::Get();
	return Settings ? Settings->bPreviewCheckerBoard : false;
}

EVisibility SAvaRundownPagePreview::GetCheckerboardVisibility() const
{	
	if (const UAvaRundownEditorSettings* Settings = UAvaRundownEditorSettings::Get())
	{
		return Settings->bPreviewCheckerBoard ? EVisibility::Visible : EVisibility::Hidden;
	}
	return EVisibility::Hidden;
}

void SAvaRundownPagePreview::SetPreviewResolution(FIntPoint InResolution) const
{
	// Update settings
	UAvaMediaSettings& AvaMediaSettings = UAvaMediaSettings::GetMutable();
	AvaMediaSettings.PreviewDefaultResolution = InResolution;
	AvaMediaSettings.SaveConfig();

	// Resize render target
	UTextureRenderTarget2D* RenderTarget = GetPreviewRenderTarget();
	if (RenderTarget)
	{
		RenderTarget->ResizeTarget(InResolution.X, InResolution.Y);
	}
	else
	{
		// The brush needs a resize.
		PreviewBrush->RenderTargetSize = FVector2D(InResolution.X, InResolution.Y);
	}
	
	// Refresh the brush.
	PreviewBrush->Update(RenderTarget);
}

bool SAvaRundownPagePreview::IsPreviewResolution(FIntPoint InResolution) const
{
	return UAvaMediaSettings::Get().PreviewDefaultResolution == InResolution;
}

void SAvaRundownPagePreview::AddResolutionMenuEntry(FMenuBuilder& InOutMenuBuilder, const FText& Label, const FIntPoint& InResolution)
{
	InOutMenuBuilder.AddMenuEntry(
			Label,
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SAvaRundownPagePreview::SetPreviewResolution, InResolution),
				FCanExecuteAction::CreateLambda([]() { return true; }),
				FIsActionChecked::CreateSP(this, &SAvaRundownPagePreview::IsPreviewResolution, InResolution)
			),
			NAME_None, EUserInterfaceActionType::RadioButton);
}

void SAvaRundownPagePreview::HandleCheckerboardActionExecute() const
{
	UAvaRundownEditorSettings* RundownEditorSettings = UAvaRundownEditorSettings::GetMutable();
	if (!RundownEditorSettings)
	{
		UE_LOGF(LogAvaRundown, Error, "Failed to retrieve Rundown Editor Settings.");
		return;
	}
	
	bool& bShowCheckerBoard = RundownEditorSettings->bPreviewCheckerBoard;
	bShowCheckerBoard = !bShowCheckerBoard;
	RundownEditorSettings->SaveConfig();	// We want this to be persistent.
	if (bShowCheckerBoard)
	{
		const IConsoleVariable* PropagateAlphaCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PostProcessing.PropagateAlpha"));
		if (PropagateAlphaCVar && PropagateAlphaCVar->GetBool() != true)
		{
			const FText NotificationText = LOCTEXT("AlphaSupport",
				"An output requested alpha but the required project setting is not enabled!\n"
				"Warning: update can add renderer performance costs.\n"
				"Go to Project Settings > Rendering > Default Settings > 'Alpha Output' to enable it in DefaultEngine.ini.");

			FNotificationInfo Info(NotificationText);
			Info.ExpireDuration = 5.0f;	// The message is long, need more time to read it.
			FSlateNotificationManager::Get().AddNotification(Info);

			// Also output to logs.
			UE_LOGF(LogAvaRundown, Warning, "%ls", *NotificationText.ToString());
		}
	}
}

void SAvaRundownPagePreview::HandleSettingsActionExecute() const
{
	// Bring up editor of UAvaMediaEditorSettings.
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Project", "Motion Design", "Playback & Broadcast");
}

#undef LOCTEXT_NAMESPACE
