// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Widgets/SCompoundWidget.h"

class FAvaRundownEditor;
class FMenuBuilder;
class FText;
class FUICommandList;
class SWidget;
class UTextureRenderTarget2D;
struct EVisibility;
struct FSlateBrush;

class SAvaRundownPagePreview : public SCompoundWidget
{
public:	
	SLATE_BEGIN_ARGS(SAvaRundownPagePreview) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedPtr<FAvaRundownEditor>& InRundownEditor);
	SAvaRundownPagePreview();
	virtual ~SAvaRundownPagePreview() override;

	TSharedRef<SWidget> CreatePagePreviewToolBar(const TSharedRef<FUICommandList>& InCommandList);
	TSharedRef<SWidget> OnGenerateSettingsMenu();

protected:
	const FSlateBrush* GetPreviewBrush() const;
	bool IsBlendingEnabled() const;
	EVisibility GetCheckerboardVisibility() const;
	void SetPreviewResolution(FIntPoint InResolution) const;
	bool IsPreviewResolution(FIntPoint InResolution) const;
	
	UTextureRenderTarget2D* GetPreviewRenderTarget() const;
	
	void AddResolutionMenuEntry(FMenuBuilder& InOutMenuBuilder, const FText& Label, const FIntPoint& InResolution);
	
	void HandleCheckerboardActionExecute() const;
	void HandleSettingsActionExecute() const;

protected:
	TWeakPtr<FAvaRundownEditor> RundownEditorWeak;

	struct FPreviewBrush;
	TUniquePtr<FPreviewBrush> PreviewBrush;
};
