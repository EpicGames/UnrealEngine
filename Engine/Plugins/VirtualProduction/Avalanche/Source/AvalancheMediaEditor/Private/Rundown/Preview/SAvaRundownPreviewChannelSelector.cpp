// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownPreviewChannelSelector.h"

#include "AvaMediaSettings.h"
#include "Broadcast/AvaBroadcast.h"

void SAvaRundownPreviewChannelSelector::Construct(const FArguments& InArgs)
{
	UpdateChannelNames();
	
	ChildSlot
	[
		SAssignNew(ChannelCombo, SComboBox<FName>)
		.InitiallySelectedItem(GetPreviewChannelNameFromSettings())
		.OptionsSource(&ChannelNames)
		.OnGenerateWidget(this, &SAvaRundownPreviewChannelSelector::GenerateChannelWidget)
		.OnSelectionChanged(this, &SAvaRundownPreviewChannelSelector::OnChannelSelectionChanged)
		.OnComboBoxOpening(this, &SAvaRundownPreviewChannelSelector::OnComboBoxOpening)
		[
			SNew(STextBlock)
			.Text(this, &SAvaRundownPreviewChannelSelector::GetCurrentChannelName)
		]
	];
}

TSharedRef<SWidget> SAvaRundownPreviewChannelSelector::GenerateChannelWidget(FName InChannelName)
{
	return SNew(STextBlock)
		.Text(FText::FromName(InChannelName));
}

void SAvaRundownPreviewChannelSelector::OnChannelSelectionChanged(SComboBox<FName>::NullableOptionType InProposedSelection, ESelectInfo::Type InSelectInfo)
{
	UAvaMediaSettings& AvaMediaSettings = UAvaMediaSettings::GetMutable();
	AvaMediaSettings.PreviewChannelName = !InProposedSelection.IsNone() ? InProposedSelection.ToString() : FString();
	AvaMediaSettings.SaveConfig();
}

FText SAvaRundownPreviewChannelSelector::GetCurrentChannelName() const
{
	return FText::FromName(GetPreviewChannelNameFromSettings());
}

FName SAvaRundownPreviewChannelSelector::GetPreviewChannelNameFromSettings()
{
	const UAvaMediaSettings& Settings = UAvaMediaSettings::Get();
	return !Settings.PreviewChannelName.IsEmpty() ? FName(Settings.PreviewChannelName) : NAME_None; 
}

void SAvaRundownPreviewChannelSelector::OnComboBoxOpening()
{
	UpdateChannelNames();
	
	check(ChannelCombo.IsValid());
	ChannelCombo->SetSelectedItem(GetPreviewChannelNameFromSettings());
}

void SAvaRundownPreviewChannelSelector::UpdateChannelNames()
{
	const UAvaBroadcast& AvaBroadcast = UAvaBroadcast::Get();
	ChannelNames.Reset(AvaBroadcast.GetChannelNameCount() + 1);
	ChannelNames.Add(NAME_None);	// Add none to allow user to deselect.
	
	for (int32 ChannelIndex = 0; ChannelIndex < AvaBroadcast.GetChannelNameCount(); ++ChannelIndex)
	{
		const FName ChannelName = AvaBroadcast.GetChannelName(ChannelIndex);
		if (AvaBroadcast.GetChannelType(ChannelName) == EAvaBroadcastChannelType::Preview)
		{
			ChannelNames.Add(ChannelName);
		}
	}
}
