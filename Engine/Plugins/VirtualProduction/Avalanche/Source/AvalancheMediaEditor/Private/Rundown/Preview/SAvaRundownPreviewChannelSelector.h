// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"

class SAvaRundownPreviewChannelSelector : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaRundownPreviewChannelSelector){}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

	TSharedRef<SWidget> GenerateChannelWidget(FName InChannelName);
	
	void OnChannelSelectionChanged(SComboBox<FName>::NullableOptionType InProposedSelection, ESelectInfo::Type InSelectInfo);
	
	FText GetCurrentChannelName() const;

	static FName GetPreviewChannelNameFromSettings();

	void OnComboBoxOpening();

protected:
	void UpdateChannelNames();
	
	TSharedPtr<SComboBox<FName>> ChannelCombo;

	TArray<FName> ChannelNames;
};
