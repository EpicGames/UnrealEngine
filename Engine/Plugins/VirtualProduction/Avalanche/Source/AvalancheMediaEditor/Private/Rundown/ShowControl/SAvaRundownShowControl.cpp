// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRundownShowControl.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/Text.h"
#include "Playback/AvaPlaybackGraph.h"
#include "Rundown/AvaRundownCommands.h"
#include "Rundown/AvaRundownEditor.h"
#include "Rundown/Pages/Slate/SAvaRundownInstancedPageList.h"
#include "Rundown/Pages/Slate/SAvaRundownPageList.h"
#include "Rundown/TabFactories/AvaRundownSubListDocumentTabFactory.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "SAvaRundownShowControl"

namespace UE::AvaRundown::Private
{
/** All the available targets for page loading */
enum class EPageLoadTarget : uint8
{
	All,
	Next,
	Selected,
};
/** Global page load target, defaults to 'All' and affects all rundown editors */
static EPageLoadTarget GPageLoadTarget = EPageLoadTarget::All;

const TSharedPtr<FUICommandInfo>& GetLoadCommandInfo(EPageLoadTarget InTarget)
{
	const FAvaRundownCommands& RundownCommands = FAvaRundownCommands::Get();
	switch (InTarget)
	{
	case EPageLoadTarget::All: 
	default:
		return RundownCommands.LoadAllPages;

	case EPageLoadTarget::Next: 
		return RundownCommands.LoadNextPages;

	case EPageLoadTarget::Selected:
		return RundownCommands.LoadSelectedPages;
	}
};

FButtonArgs MakeLoadPageButtonArgs(const TSharedRef<FUICommandList>& InCommandList)
{
	FButtonArgs ButtonArgs;
	ButtonArgs.UserInterfaceActionType = EUserInterfaceActionType::Button;
	ButtonArgs.LabelOverride = TAttribute<FText>::CreateLambda([]()
		{
			return GetLoadCommandInfo(GPageLoadTarget)->GetLabel();
		});
	ButtonArgs.ToolTipOverride = TAttribute<FText>::CreateLambda([]()
		{
			return GetLoadCommandInfo(GPageLoadTarget)->GetDescription();
		});
	ButtonArgs.IconOverride = TAttribute<FSlateIcon>::CreateLambda([]()
		{
			return GetLoadCommandInfo(GPageLoadTarget)->GetIcon();
		});
	ButtonArgs.Action.ExecuteAction.BindLambda([CommandListWeak = InCommandList->AsWeak()]()
		{
			if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
			{
				CommandList->ExecuteAction(GetLoadCommandInfo(GPageLoadTarget).ToSharedRef());
			}
		});
	ButtonArgs.Action.CanExecuteAction.BindLambda([CommandListWeak = InCommandList->AsWeak()]()->bool
		{
			if (TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
			{
				return CommandList->CanExecuteAction(GetLoadCommandInfo(GPageLoadTarget).ToSharedRef());
			}
			return false;
		});
	return ButtonArgs;
}

FOnGetContent MakeLoadPageOptionsMenu()
{
	return FOnGetContent::CreateLambda([]()->TSharedRef<SWidget>
	{
		const bool bInShouldCloseWindowAfterMenuSelection = false;
		FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, nullptr);

		auto MakeMenuEntry = [&MenuBuilder](EPageLoadTarget InTarget)
			{
				const TSharedPtr<FUICommandInfo>& CommandInfo = GetLoadCommandInfo(InTarget);
				FUIAction UIAction;
				UIAction.ExecuteAction.BindLambda([InTarget]()
					{
						GPageLoadTarget = InTarget;
					});
				UIAction.GetActionCheckState.BindLambda([InTarget]()->ECheckBoxState
					{
						return GPageLoadTarget == InTarget ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					});
				MenuBuilder.AddMenuEntry(CommandInfo->GetLabel()
					, CommandInfo->GetDescription()
					, FSlateIcon() // NOTE : The page load commands all have the same icon. When this is not the case, this should change to CommandInfo->GetIcon()
					, UIAction
					, NAME_None
					, EUserInterfaceActionType::RadioButton
					, NAME_None
					, CommandInfo->GetInputText());
			};

		MakeMenuEntry(EPageLoadTarget::All);
		MakeMenuEntry(EPageLoadTarget::Next);
		MakeMenuEntry(EPageLoadTarget::Selected);
		return MenuBuilder.MakeWidget();
	});
}

} // UE::AvaRundown::Private

void SAvaRundownShowControl::Construct(const FArguments& InArgs, const TSharedPtr<FAvaRundownEditor>& InRundownEditor)
{
	RundownEditorWeak = InRundownEditor;
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			BuildShowControlToolBar(InRundownEditor->GetToolkitCommands())
		]
	];
}

TSharedRef<SWidget> SAvaRundownShowControl::BuildShowControlToolBar(const TSharedRef<FUICommandList>& InCommandList)
{
	FSlimHorizontalToolBarBuilder ToolBarBuilder(InCommandList, FMultiBoxCustomization::None);
	ToolBarBuilder.SetStyle(&FAvaMediaEditorStyle::Get(), "AvaMediaEditor.ToolBar");

	const FAvaRundownCommands& RundownCommands = FAvaRundownCommands::Get();

	ToolBarBuilder.BeginSection(TEXT("ShowControl"));
	{
		ToolBarBuilder.BeginStyleOverride("AvaMediaEditor.ToolBarRedButtonOverride");
		{
			ToolBarBuilder.AddToolBarButton(RundownCommands.Play);
		}
		ToolBarBuilder.EndStyleOverride();

		ToolBarBuilder.AddToolBarButton(RundownCommands.Continue);
		ToolBarBuilder.AddToolBarButton(RundownCommands.Stop);

		ToolBarBuilder.BeginStyleOverride("AvaMediaEditor.ToolBarRedButtonOverride");
		{
			ToolBarBuilder.AddToolBarButton(RundownCommands.PlayNext);
		}
		ToolBarBuilder.EndStyleOverride();

		// Page Load + Page Load Options
		ToolBarBuilder.AddSeparator();
		ToolBarBuilder.AddToolBarButton(UE::AvaRundown::Private::MakeLoadPageButtonArgs(InCommandList));
		ToolBarBuilder.AddComboButton(FUIAction()
			, UE::AvaRundown::Private::MakeLoadPageOptionsMenu()
			, TAttribute<FText>()
			, LOCTEXT("PageLoadOptionsTooltip", "Page Load Options")
			, FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewportToolBar.OptionsDropdown")
			, /*bSimpleComboBox*/ true
		);
		ToolBarBuilder.AddSeparator();
		ToolBarBuilder.AddWidget(CreateActiveListWidget());
		ToolBarBuilder.AddSeparator();
		ToolBarBuilder.AddWidget(CreateNextPageWidget());
	}
	ToolBarBuilder.EndSection();

	return ToolBarBuilder.MakeWidget();
}

TSharedRef<SWidget> SAvaRundownShowControl::CreateActiveListWidget()
{
	return
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(EVerticalAlignment::VAlign_Center)
		.Padding(5.f, 0.f)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ActiveView", "Active View:"))
		]
		+ SHorizontalBox::Slot()
		.VAlign(EVerticalAlignment::VAlign_Center)
		.Padding(0.f, 0.f, 5.f, 0.f)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(this, &SAvaRundownShowControl::GetActiveListName)
		];
}

FText SAvaRundownShowControl::GetActiveListName() const
{
	if (const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		const UAvaRundown* Rundown = RundownEditor->GetRundown();

		if (IsValid(Rundown))
		{
			const FAvaRundownPageListReference& ActiveList = Rundown->GetActivePageListReference();

			// Can't happen. Here for completeness.
			if (ActiveList.Type == EAvaRundownPageListType::Template)
			{
				static const FText Templates(LOCTEXT("Templates", "Templates"));
				return Templates;
			}

			if (ActiveList.Type == EAvaRundownPageListType::Instance)
			{
				static const FText AllPages(LOCTEXT("AllPages", "All Pages"));
				return AllPages;
			}

			return FAvaRundownSubListDocumentTabFactory::GetTabLabel(ActiveList, Rundown);
		}
	}

	return FText::GetEmpty();
}

TSharedRef<SWidget> SAvaRundownShowControl::CreateNextPageWidget()
{
	return
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(EVerticalAlignment::VAlign_Center)
		.Padding(5.f, 0.f)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("NextUp", "Next Up:"))
		]
		+ SHorizontalBox::Slot()
		.VAlign(EVerticalAlignment::VAlign_Center)
		.Padding(0.f, 0.f, 5.f, 0.f)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(this, &SAvaRundownShowControl::GetNextPageName)
		];
}

FText SAvaRundownShowControl::GetNextPageName() const
{
	static const FText NoPage(LOCTEXT("None", "-"));

	if (TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin())
	{
		if (const UAvaRundown* Rundown = RundownEditor->GetRundown(); IsValid(Rundown))
		{
			if (Rundown->GetInstancedPages().Pages.IsEmpty())
			{
				return NoPage;
			}

			const TSharedPtr<SAvaRundownInstancedPageList> ActiveSubListWidget = RundownEditor->GetActiveListWidget();

			if (!ActiveSubListWidget.IsValid() || ActiveSubListWidget->GetPlayingPageIds().IsEmpty())
			{
				return NoPage;
			}

			TArray<int32> NextUps = ActiveSubListWidget->GetPageIdsToTakeNext();

			FText NextUpMessage = FText::GetEmpty();
			FFormatOrderedArguments FormatOrderedArguments;
			
			for (const int32 NextUp : NextUps)
			{
				if (const int32* PageIndex = Rundown->GetInstancedPages().PageIndices.Find(NextUp))
				{
					FormatOrderedArguments.Add(FText::Format(
							LOCTEXT("NextUpFormat", "{0}: {1}"),
							FText::AsNumber(NextUp, &UE::AvaRundown::FEditorMetrics::PageIdFormattingOptions),
							FText::FromString(Rundown->GetInstancedPages().Pages[*PageIndex].GetPageName())
						));
				}
			}

			if (FormatOrderedArguments.Num() > 0)
			{
				return FText::Join(LOCTEXT("NextUpDelimiter", "\n"), FormatOrderedArguments);
			}
		}
	}

	return NoPage;
}

#undef LOCTEXT_NAMESPACE
