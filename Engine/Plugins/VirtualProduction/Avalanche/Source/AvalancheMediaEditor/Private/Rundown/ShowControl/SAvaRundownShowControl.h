// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FAvaRundownEditor;
class FUICommandList;
class FText;
class SWidget;

class SAvaRundownShowControl : public SCompoundWidget
{
public:	
	SLATE_BEGIN_ARGS(SAvaRundownShowControl) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedPtr<FAvaRundownEditor>& InRundownEditor);

	TSharedRef<SWidget> BuildShowControlToolBar(const TSharedRef<FUICommandList>& InCommandList);

protected:	
	TWeakPtr<FAvaRundownEditor> RundownEditorWeak;

	TSharedRef<SWidget> CreateActiveListWidget();
	FText GetActiveListName() const;

	TSharedRef<SWidget> CreateNextPageWidget();
	FText GetNextPageName() const;
};
