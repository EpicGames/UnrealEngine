// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownChannelLayerStatusListTabFactory.h"

#include "AvaMediaEditorStyle.h"
#include "Rundown/ChannelLayerStatus/SAvaRundownChannelLayerStatusList.h"

const FName FAvaRundownChannelLayerStatusListTabFactory::TabID(TEXT("MotionDesignChannelLayerStatusList"));

#define LOCTEXT_NAMESPACE "AvaRundownChannelLayerStatusListTabFactory"

FAvaRundownChannelLayerStatusListTabFactory::FAvaRundownChannelLayerStatusListTabFactory(const TSharedPtr<FAvaRundownEditor>& InRundownEditor)
	: FAvaRundownTabFactory(TabID, InRundownEditor)
{
	TabLabel = LOCTEXT("ChannelLayerStatus_TabLabel", "Layer Status");
	TabIcon = FSlateIcon(FAvaMediaEditorStyle::Get().GetStyleSetName(), TEXT("AvaMediaEditor.BroadcastIcon"));

	bIsSingleton = true;
	bShouldAutosize = true;

	ViewMenuDescription = LOCTEXT("ChannelLayerStatus_ViewMenu_Desc", "Displays the Status of all Channel Layers in Broadcast");
	ViewMenuTooltip = LOCTEXT("ChannelLayerStatus_ViewMenu_ToolTip", "Displays the Status of all Channel Layers in Broadcast");
}

TSharedRef<SWidget> FAvaRundownChannelLayerStatusListTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SAvaRundownChannelLayerStatusList, RundownEditorWeak.Pin());
}

#undef LOCTEXT_NAMESPACE
