// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownChannelStatusListTabFactory.h"

#include "AvaMediaEditorStyle.h"
#include "Rundown/ChannelStatus/SAvaRundownChannelStatusList.h"

const FName FAvaRundownChannelStatusListTabFactory::TabID(TEXT("MotionDesignChannelStatusList"));

#define LOCTEXT_NAMESPACE "AvaRundownChannelStatusListTabFactory"

FAvaRundownChannelStatusListTabFactory::FAvaRundownChannelStatusListTabFactory(const TSharedPtr<FAvaRundownEditor>& InRundownEditor)
	: FAvaRundownTabFactory(TabID, InRundownEditor)
{
	TabLabel = LOCTEXT("ChannelStatus_TabLabel", "Channel Status");
	TabIcon = FSlateIcon(FAvaMediaEditorStyle::Get().GetStyleSetName(), TEXT("AvaMediaEditor.BroadcastIcon"));

	bIsSingleton = true;
	bShouldAutosize = true;

	ViewMenuDescription = LOCTEXT("ChannelStatus_ViewMenu_Desc", "Displays the Status of all Channels in Broadcast");
	ViewMenuTooltip = LOCTEXT("ChannelStatus_ViewMenu_ToolTip", "Displays the Status of all Channels in Broadcast");
}

TSharedRef<SWidget> FAvaRundownChannelStatusListTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SAvaRundownChannelStatusList);
}

#undef LOCTEXT_NAMESPACE
