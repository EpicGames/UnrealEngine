// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownInstancedPageListTabFactory.h"

#include "Rundown/AvaRundownEditor.h"
#include "Rundown/Pages/Slate/SAvaRundownInstancedPageList.h"
#include "Widgets/Docking/SDockTab.h"

const FName FAvaRundownInstancedPageListTabFactory::TabID(TEXT("MotionDesignInstancedRundownPageList"));

#define LOCTEXT_NAMESPACE "AvaRundownInstancedPageListTabFactory"

FAvaRundownInstancedPageListTabFactory::FAvaRundownInstancedPageListTabFactory(const TSharedPtr<FAvaRundownEditor>& InRundownEditor)
	: FAvaRundownTabFactory(TabID, InRundownEditor)
{
	TabLabel = LOCTEXT("RundownPageList_TabLabel", "Pages");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlacementBrowser.Icons.All");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("RundownPageList_ViewMenu_Desc", "Pages");
	ViewMenuTooltip = LOCTEXT("RundownPageList_ViewMenu_ToolTip", "Pages");
}

TSharedRef<SWidget> FAvaRundownInstancedPageListTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& InInfo) const
{
	return SNew(SAvaRundownInstancedPageList, RundownEditorWeak.Pin(), UAvaRundown::InstancePageList);
}

TSharedRef<SDockTab> FAvaRundownInstancedPageListTabFactory::OnSpawnTab(const FSpawnTabArgs& InSpawnArgs,
	TWeakPtr<FTabManager> InWeakTabManager) const
{
	TSharedRef<SDockTab> NewTab = FAvaRundownTabFactory::OnSpawnTab(InSpawnArgs, InWeakTabManager);
	TSharedRef<SWidget> Content = NewTab->GetContent();

	if (Content->GetWidgetClass().GetWidgetType() == SBorder::StaticWidgetClass().GetWidgetType()
		&& Content->GetTypeAsString() == "SBorder") // Subclasses of SBorder that do not propertly declare themselves will have the same widget class.
	{
		Content = StaticCastSharedRef<SBorder>(Content)->GetContent();
	}

	TSharedRef<SAvaRundownInstancedPageList> PageList = StaticCastSharedRef<SAvaRundownInstancedPageList>(Content);
	NewTab->SetOnTabActivated(SDockTab::FOnTabActivatedCallback::CreateSP(PageList, &SAvaRundownInstancedPageList::OnTabActivated));

	return NewTab;
}

#undef LOCTEXT_NAMESPACE
