// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaRundownTabFactory.h"

class FAvaRundownInstancedPageListTabFactory : public FAvaRundownTabFactory
{
public:
	static const FName TabID;
	
	FAvaRundownInstancedPageListTabFactory(const TSharedPtr<FAvaRundownEditor>& InRundownEditor);
	
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& InInfo) const override;

protected:
	virtual TSharedRef<SDockTab> OnSpawnTab(const FSpawnTabArgs& InSpawnArgs, TWeakPtr<FTabManager> InWeakTabManager) const override;
};
