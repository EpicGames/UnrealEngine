// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageDetailsTabFactory.h"

#include "Rundown/AvaRundownEditor.h"
#include "Rundown/DetailsView/SAvaRundownPageDetails.h"

const FName FAvaRundownPageDetailsTabFactory::TabID(TEXT("MotionDesignRundownPageDetails"));

#define LOCTEXT_NAMESPACE "AvaPageDetailsTabFactory"

FAvaRundownPageDetailsTabFactory::FAvaRundownPageDetailsTabFactory(const TSharedPtr<FAvaRundownEditor>& InRundownEditor)
	: FAvaRundownTabFactory(TabID, InRundownEditor)
{
	TabLabel = LOCTEXT("RundownPageDetails_TabLabel", "Page Details");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("RundownPageDetails_ViewMenu_Desc", "Page Details");
	ViewMenuTooltip = LOCTEXT("RundownPageDetails_ViewMenu_ToolTip", "Page Details");
}

TSharedRef<SWidget> FAvaRundownPageDetailsTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SAvaRundownPageDetails, RundownEditorWeak.Pin());
}

#undef LOCTEXT_NAMESPACE
