// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownPageViewerTabFactory.h"

#include "Rundown/AvaRundownEditor.h"
#include "Rundown/Preview/SAvaRundownPagePreview.h"

const FName FAvaRundownPageViewerTabFactory::TabID(TEXT("MotionDesignRundownPageViewer"));

#define LOCTEXT_NAMESPACE "AvaRundownPageViewerTabFactory"

FAvaRundownPageViewerTabFactory::FAvaRundownPageViewerTabFactory(const TSharedPtr<FAvaRundownEditor>& InRundownEditor)
	: FAvaRundownTabFactory(TabID, InRundownEditor)
{
	TabLabel = LOCTEXT("RundownPageViewer_TabLabel", "Page Preview");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.VirtualProduction");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("RundownPageViewer_ViewMenu_Desc", "Previews the selected page");
	ViewMenuTooltip = LOCTEXT("RundownPageViewer_ViewMenu_ToolTip", "Previews the selected page");
}

TSharedRef<SWidget> FAvaRundownPageViewerTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin();
	
	return SNew(SAvaRundownPagePreview, RundownEditor);
}

#undef LOCTEXT_NAMESPACE
