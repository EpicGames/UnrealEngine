// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownShowControlTabFactory.h"

#include "Rundown/AvaRundownEditor.h"
#include "Rundown/ShowControl/SAvaRundownShowControl.h"

const FName FAvaRundownShowControlTabFactory::TabID(TEXT("MotionDesignRundownShowControl"));

#define LOCTEXT_NAMESPACE "AvaRundownShowControlTabFactory"

FAvaRundownShowControlTabFactory::FAvaRundownShowControlTabFactory(const TSharedPtr<FAvaRundownEditor>& InRundownEditor)
	: FAvaRundownTabFactory(TabID, InRundownEditor)
{
	TabLabel = LOCTEXT("RundownShowControl_TabLabel", "Show Control");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Settings");

	bIsSingleton = true;
	bShouldAutosize = false;

	ViewMenuDescription = LOCTEXT("RundownShowControl_ViewMenu_Desc", "Show Control");
	ViewMenuTooltip = LOCTEXT("RundownShowControl_ViewMenu_ToolTip", "Show Control");
}

TSharedRef<SWidget> FAvaRundownShowControlTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SAvaRundownShowControl, RundownEditorWeak.Pin());
}

#undef LOCTEXT_NAMESPACE
