// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaRundownTabFactory.h"

class FAvaRundownShowControlTabFactory : public FAvaRundownTabFactory
{
public:
	static const FName TabID;
	
	FAvaRundownShowControlTabFactory(const TSharedPtr<FAvaRundownEditor>& InRundownEditor);
	
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
};
