// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownSubListDocumentTabFactory.h"

#include "Rundown/AvaRundown.h"
#include "Rundown/AvaRundownEditor.h"
#include "Rundown/Pages/Slate/SAvaRundownInstancedPageList.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkflowOrientedApp/ApplicationMode.h"

#define LOCTEXT_NAMESPACE "AvaRundownSubListDocumentTabFactory"

const FName FAvaRundownSubListDocumentTabFactory::FactoryId = "AvaSubListTabFactory";
const FString FAvaRundownSubListDocumentTabFactory::BaseTabName(TEXT("AvaSubListDocument"));

FName FAvaRundownSubListDocumentTabFactory::GetTabId(const FAvaRundownPageListReference& InSubListReference)
{
	return FName(BaseTabName + "_" + InSubListReference.SubListId.ToString());
}

FText FAvaRundownSubListDocumentTabFactory::GetTabLabel(const FAvaRundownPageListReference& InSubListReference, const UAvaRundown* InRundown)
{
	FText SubListIdAsText;
	
	if (IsValid(InRundown))
	{
		const FAvaRundownSubList& SubList = InRundown->GetSubList(InSubListReference);
		if (SubList.IsValid())
		{
			if (!SubList.Name.IsEmpty())
			{
				return SubList.Name;
			}
			
			// If the sublist has no display name, we will use it's index in the sublists.
			SubListIdAsText = FText::AsNumber(InRundown->GetSubListIndex(SubList) + 1);
		}
	}

	// If the rundown is not available for some reason, use the id.
	if (SubListIdAsText.IsEmpty())
	{
		SubListIdAsText = FText::FromString(InSubListReference.SubListId.ToString());
	}
	
	return FText::Format(LOCTEXT("RundownSubListDocument_TabLabel", "Page View {0}"), SubListIdAsText);
}

FText FAvaRundownSubListDocumentTabFactory::GetTabDescription(const FAvaRundownPageListReference& InSubListReference, const UAvaRundown* InRundown)
{
	const FText SubListId = FText::FromString(InSubListReference.SubListId.ToString());
	const FText SubListLabel = GetTabLabel(InSubListReference, InRundown);
	return FText::Format(LOCTEXT("RundownSubListDocument_ViewMenu_Desc", "{0} Id: {1}"), SubListLabel, SubListId);
}

FText FAvaRundownSubListDocumentTabFactory::GetTabTooltip(const FAvaRundownPageListReference& InSubListReference, const UAvaRundown* InRundown)
{
	const FText SubListId = FText::FromString(InSubListReference.SubListId.ToString());
	const FText SubListLabel = GetTabLabel(InSubListReference, InRundown);
	return FText::Format(LOCTEXT("RundownSubListDocument_ViewMenu_ToolTip", "{0} Id: {1}"), SubListLabel, SubListId);
}

FAvaRundownSubListDocumentTabFactory::FAvaRundownSubListDocumentTabFactory(const FAvaRundownPageListReference& InSubListReference, const TSharedPtr<FAvaRundownEditor>& InRundownEditor)
	: FDocumentTabFactory(GetTabId(InSubListReference), InRundownEditor)
	, RundownEditorWeak(InRundownEditor)
{
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlacementBrowser.Icons.All");
	SubListReference = InSubListReference;
	TabIdentifier = GetTabId(InSubListReference);
	
	UAvaRundown* Rundown = InRundownEditor ? InRundownEditor->GetRundown() : nullptr;
	TabLabel = GetTabLabel(InSubListReference, Rundown);
	ViewMenuDescription = GetTabDescription(InSubListReference, Rundown);
	ViewMenuTooltip = GetTabTooltip(InSubListReference, Rundown);

	if (Rundown)
	{
		// Allow propagation of tab label changes.
		Rundown->GetOnPageListChanged().AddRaw(this, &FAvaRundownSubListDocumentTabFactory::OnPageListChanged);
	}
}

FAvaRundownSubListDocumentTabFactory::~FAvaRundownSubListDocumentTabFactory()
{
	const TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin();
	if (UAvaRundown* Rundown = RundownEditor ? RundownEditor->GetRundown() : nullptr)
	{
		Rundown->GetOnPageListChanged().RemoveAll(this);
	}
}

TSharedRef<SWidget> FAvaRundownSubListDocumentTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& InInfo) const
{
	if (!SubListReference.SubListId.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SAvaRundownInstancedPageList, RundownEditorWeak.Pin(), SubListReference);
}

FTabSpawnerEntry& FAvaRundownSubListDocumentTabFactory::RegisterTabSpawner(TSharedRef<FTabManager> InTabManager, const FApplicationMode* InCurrentApplicationMode) const
{
	const FText PageViewsGroupName = LOCTEXT("SubMenuLabel", "Page Views");
	TSharedPtr<FWorkspaceItem> GroupItem;

	// Find or Add the PageView Group Item.
	if (InCurrentApplicationMode)
	{
		if (const TSharedPtr<FWorkspaceItem> WorkspaceMenu = InCurrentApplicationMode->GetWorkspaceMenuCategory())
		{
			for (const TSharedRef<FWorkspaceItem>& Item : WorkspaceMenu->GetChildItems())
			{
				if (Item->GetDisplayName().ToString() == PageViewsGroupName.ToString())
				{
					GroupItem = Item;
					break;
				}
			}
		
			if (!GroupItem.IsValid())
			{
				GroupItem = WorkspaceMenu->AddGroup(PageViewsGroupName, LOCTEXT("SubMenuTooltip", "Motion Design Rundown Page Views"), TabIcon);
			}
		}
	}
	
	//Note: We don't provide the application mode to avoid setting the group.
	FTabSpawnerEntry& SpawnerEntry = FDocumentTabFactory::RegisterTabSpawner(InTabManager, GroupItem.IsValid() ? nullptr : InCurrentApplicationMode);

	if (GroupItem.IsValid())
	{
		SpawnerEntry.SetGroup(GroupItem.ToSharedRef());
	}

	// Bind the spawner entry label to the current document factory label.
	SpawnerEntry.SetDisplayNameAttribute(TAttribute<FText>( this, &FAvaRundownSubListDocumentTabFactory::GetTabTitle));
	
	return SpawnerEntry;
}

TSharedRef<SDockTab> FAvaRundownSubListDocumentTabFactory::OnSpawnTab(const FSpawnTabArgs& InSpawnArgs, TWeakPtr<FTabManager> InTabManagerWeak) const
{
	// Intercept the spawned tab to bind our handlers.
	TSharedRef<SDockTab> SpawnedTab = FDocumentTabFactory::OnSpawnTab(InSpawnArgs, InTabManagerWeak);

	if (SpawnedTab != SNullWidget::NullWidget)
	{
		const TSharedRef<SAvaRundownInstancedPageList> PageList = StaticCastSharedRef<SAvaRundownInstancedPageList>(SpawnedTab->GetContent());
		SpawnedTab->SetOnTabActivated(SDockTab::FOnTabActivatedCallback::CreateSP(PageList, &SAvaRundownInstancedPageList::OnTabActivated));
	}

	// Bind the tab label to the current document factory label.
	SpawnedTab->SetLabel(TAttribute<FText>( this, &FAvaRundownSubListDocumentTabFactory::GetTabTitle));

	return SpawnedTab;
}

FText FAvaRundownSubListDocumentTabFactory::GetTabTitle() const
{
	return TabLabel;
}

void FAvaRundownSubListDocumentTabFactory::OnPageListChanged(const FAvaRundownPageListChangeParams& InParams)
{
	if (SubListReference != InParams.PageListReference)
	{
		return;
	}

	if (EnumHasAnyFlags(InParams.ChangeType, EAvaRundownPageListChange::SubListRenamed))
	{
		// This will automatically propagate to the SpawnerEntry (Windows menu) and the Spawned Tab Label.
		TabLabel = GetTabLabel(InParams.PageListReference, InParams.Rundown);
	}
}

#undef LOCTEXT_NAMESPACE
