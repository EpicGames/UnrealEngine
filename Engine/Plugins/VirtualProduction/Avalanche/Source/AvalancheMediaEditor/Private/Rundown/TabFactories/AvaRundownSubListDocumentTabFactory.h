// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "AvaMediaDefines.h"

class FAvaRundownEditor;
class UAvaRundown;
struct FAvaRundownPageListChangeParams;

/** Base class for all Tab Factories in Ava SubListDocument Editor */
class FAvaRundownSubListDocumentTabFactory : public FDocumentTabFactory
{
public:
	static const FName FactoryId;
	static const FString BaseTabName;

	static FName GetTabId(const FAvaRundownPageListReference& InSubListReference);
	static FText GetTabLabel(const FAvaRundownPageListReference& InSubListReference, const UAvaRundown* InRundown);
	static FText GetTabDescription(const FAvaRundownPageListReference& InSubListReference, const UAvaRundown* InRundown);
	static FText GetTabTooltip(const FAvaRundownPageListReference& InSubListReference, const UAvaRundown* InRundown);

	FAvaRundownSubListDocumentTabFactory(const FAvaRundownPageListReference& InSubListReference, const TSharedPtr<FAvaRundownEditor>& InSubListDocumentEditor);

	virtual ~FAvaRundownSubListDocumentTabFactory() override;

	//~ Begin FWorkflowTabFactory
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& InInfo) const override;
	virtual FTabSpawnerEntry& RegisterTabSpawner(TSharedRef<FTabManager> InTabManager, const FApplicationMode* InCurrentApplicationMode) const override;
protected:
	virtual TSharedRef<SDockTab> OnSpawnTab(const FSpawnTabArgs& InSpawnArgs, TWeakPtr<FTabManager> InTabManagerWeak) const override;
	//~ End FWorkflowTabFactory

protected:
	FText GetTabTitle() const;

	void OnPageListChanged(const FAvaRundownPageListChangeParams& InParams);
	
protected:
	TWeakPtr<FAvaRundownEditor> RundownEditorWeak;
	FAvaRundownPageListReference SubListReference;
};

