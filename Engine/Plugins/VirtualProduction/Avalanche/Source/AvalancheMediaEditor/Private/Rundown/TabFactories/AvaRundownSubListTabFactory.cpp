// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownSubListTabFactory.h"

#include "Rundown/AvaRundownEditor.h"
#include "Rundown/Pages/Slate/SAvaRundownSubListStartPage.h"

#define LOCTEXT_NAMESPACE "AvaRundownSubListTabFactory"

const FName FAvaRundownSubListTabFactory::TabID(TEXT("AvaRundownSubList"));

FAvaRundownSubListTabFactory::FAvaRundownSubListTabFactory(const TSharedPtr<FAvaRundownEditor>& InRundownEditor)
	: FAvaRundownTabFactory(TabID, InRundownEditor)
{
	RundownEditorWeak = InRundownEditor;

	TabLabel = LOCTEXT("RundownSubList_TabLabel", "Page View Start");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlacementBrowser.Icons.All");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("RundownSubList_ViewMenu_Desc", "Page View Start");
	ViewMenuTooltip = LOCTEXT("RundownSubList_ViewMenu_ToolTip", "Page View Start");
}

TSharedRef<SWidget> FAvaRundownSubListTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& InInfo) const
{
	TSharedPtr<FAvaRundownEditor> RundownEditor = RundownEditorWeak.Pin();

	return SNew(SAvaRundownSubListStartPage, RundownEditor);
}

#undef LOCTEXT_NAMESPACE
