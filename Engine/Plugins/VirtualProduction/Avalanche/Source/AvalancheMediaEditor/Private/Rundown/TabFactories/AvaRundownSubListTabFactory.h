// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaRundownTabFactory.h"

class FAvaRundownEditor;

//Base class for all Tab Factories in Ava SubList Editor
class FAvaRundownSubListTabFactory : public FAvaRundownTabFactory
{
public:
	static const FName TabID;

	FAvaRundownSubListTabFactory(const TSharedPtr<FAvaRundownEditor>& InRundownEditor);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& InInfo) const override;
};

