// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownTabFactory.h"

#include "Rundown/AvaRundownEditor.h"

FAvaRundownTabFactory::FAvaRundownTabFactory(const FName& InTabID, const TSharedPtr<FAvaRundownEditor>& InRundownEditor)
	: FWorkflowTabFactory(InTabID, InRundownEditor)
	, RundownEditorWeak(InRundownEditor)
{
}
