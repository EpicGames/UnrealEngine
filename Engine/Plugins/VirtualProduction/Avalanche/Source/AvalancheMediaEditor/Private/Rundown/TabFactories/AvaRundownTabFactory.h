// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class FAvaRundownEditor;

//Base class for all Tab Factories in Ava Rundown Editor
class FAvaRundownTabFactory : public FWorkflowTabFactory
{
public:

	FAvaRundownTabFactory(const FName& InTabID, const TSharedPtr<FAvaRundownEditor>& InRundownEditor);

protected:

	TWeakPtr<FAvaRundownEditor> RundownEditorWeak;
};

