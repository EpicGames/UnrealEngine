// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRundownTemplatePageListTabFactory.h"

#include "Rundown/AvaRundownEditor.h"
#include "Rundown/Pages/Slate/SAvaRundownTemplatePageList.h"

const FName FAvaRundownTemplatePageListTabFactory::TabID(TEXT("MotionDesignTemplateRundownPageList"));

#define LOCTEXT_NAMESPACE "AvaRundownTemplatePageListTabFactory"

FAvaRundownTemplatePageListTabFactory::FAvaRundownTemplatePageListTabFactory(const TSharedPtr<FAvaRundownEditor>& InRundownEditor)
	: FAvaRundownTabFactory(TabID, InRundownEditor)
{
	TabLabel = LOCTEXT("RundownPageList_TabLabel", "Templates");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlacementBrowser.Icons.All");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("RundownPageList_ViewMenu_Desc", "Templates");
	ViewMenuTooltip = LOCTEXT("RundownPageList_ViewMenu_ToolTip", "Templates");
}

TSharedRef<SWidget> FAvaRundownTemplatePageListTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& InInfo) const
{
	return SNew(SAvaRundownTemplatePageList, RundownEditorWeak.Pin());
}

#undef LOCTEXT_NAMESPACE
