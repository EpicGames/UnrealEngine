// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaRundownTabFactory.h"

class FAvaRundownTemplatePageListTabFactory : public FAvaRundownTabFactory
{
public:
	static const FName TabID;
	
	FAvaRundownTemplatePageListTabFactory(const TSharedPtr<FAvaRundownEditor>& InRundownEditor);
	
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& InInfo) const override;
};
