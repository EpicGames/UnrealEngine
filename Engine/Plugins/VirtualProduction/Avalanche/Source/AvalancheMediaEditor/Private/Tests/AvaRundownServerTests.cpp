// Copyright Epic Games, Inc. All Rights Reserved.

#include "Broadcast/AvaBroadcast.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputRootItem.h"
#include "Broadcast/OutputDevices/AvaBroadcastOutputTreeItem.h"
#include "IAvaMediaModule.h"
#include "MediaOutput.h"
#include "MessageEndpointBuilder.h"
#include "Misc/AutomationTest.h"
#include "Misc/FileHelper.h"
#include "Rundown/AvaRundownMessages.h"
#include "Rundown/AvaRundownServerMediaOutputUtils.h"
#include "Rundown/IAvaRundownServer.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"

DEFINE_LOG_CATEGORY_STATIC(LogAvaRundownServerTests, Log, All);

namespace UE::AvaRundownServerTests
{
	class FRundownServerTestBase : public FAutomationTestBase
	{
	public:
		FRundownServerTestBase(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask)
		{
			// UE-185435 - Blackmagic lib generates an error if the driver is not installed.
			LocalSuppressedLogCategories.Add(TEXT("LogBlackmagicCore"));
		}

		//~ Begin FAutomationTestBase
		virtual bool ShouldCaptureLogCategory(const FName& Category) const override
		{
			if (LocalSuppressedLogCategories.Contains(Category))
			{
				return false;
			}
			return FAutomationTestBase::ShouldCaptureLogCategory(Category);
		}
		//~ End FAutomationTestBase

	protected:
		TSet<FName> LocalSuppressedLogCategories;
	};

	class FTestClient
	{
	public:
		TSharedPtr<FMessageEndpoint> MessageEndpoint;
		FAutomationTestBase* Test = nullptr;
		TSet<int32> ReceivedRequestIds;

		bool HasReceivedResponse(int32 InRequestId) const
		{
			return ReceivedRequestIds.Contains(InRequestId);
		}
		
		void Init(FAutomationTestBase* InTest)
		{
			Test = InTest;
			
			MessageEndpoint = FMessageEndpoint::Builder("AvaRundownTestClient")
				.Handling<FAvaRundownChannelImage>(this, &FTestClient::HandleRundownChannelImage)
				.Handling<FAvaRundownServerMsg>(this, &FTestClient::HandleServerMessage);
		}
		
		void HandleRundownChannelImage(const FAvaRundownChannelImage& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
		{
			ReceivedRequestIds.Add(InMessage.RequestId);
			
			UE_LOGF(LogAvaRundownServerTests, Log, "Received Rundown Channel Image successfully.");

			const FString SaveName = FString::Printf(TEXT("%s%s"), *FPaths::ProjectSavedDir(), TEXT("ChannelImageTest.jpeg"));
			UE_LOGF(LogAvaRundownServerTests, Display, "Saving Image to %ls", *SaveName);

			FFileHelper::SaveArrayToFile(InMessage.ImageData, *SaveName);
		}

		void HandleServerMessage(const FAvaRundownServerMsg& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
		{
			ReceivedRequestIds.Add(InMessage.RequestId);

			const ELogVerbosity::Type Verbosity = ParseLogVerbosityFromString(InMessage.Verbosity);
				
			if (Verbosity <= ELogVerbosity::Error && Test)
			{
				Test->AddError(InMessage.Text, 1);
			}
		}
	};

	TSharedPtr<IAvaRundownServer> GetOrCreateRundownServer()
	{
		TSharedPtr<IAvaRundownServer> RundownServer = IAvaMediaModule::Get().GetRundownServer();
		if (!RundownServer)
		{
			UE_LOGF(LogAvaRundownServerTests, Log, "Rundown Server not started. Starting one temporarily for the test.");

			// Start a temporary server. It will be deleted when the last latent command is finished.
			RundownServer = IAvaMediaModule::Get().MakeDetachedRundownServer(TEXT(""));
		}
		return RundownServer;
	}

	void BackupBroadcastConfig()
	{
		const UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
		const FString SaveFilepath = Broadcast.GetBroadcastSaveFilepath();
		const FString BackupFilepath = SaveFilepath + TEXT(".tests.backup");
		IFileManager::Get().Copy(*BackupFilepath, *SaveFilepath);
		UE_LOGF(LogAvaRundownServerTests, Log, "Broadcast config backed up.");
	}

	void RestoreBroadcastConfig()
	{
		UAvaBroadcast& Broadcast = UAvaBroadcast::Get();
		const FString SaveFilepath = Broadcast.GetBroadcastSaveFilepath();
		const FString BackupFilepath = SaveFilepath + TEXT(".tests.backup");
		if (!IFileManager::Get().FileExists(*BackupFilepath))
		{
			UE_LOGF(LogAvaRundownServerTests, Error, "Failed to restore broadcast config: backup file \"%ls\" doesn't exist.", *BackupFilepath);
			return;
		}

		IFileManager::Get().Copy(*SaveFilepath, *BackupFilepath);
		IFileManager::Get().Delete(*BackupFilepath);

		// Reload broadcast from backup file.
		Broadcast.StopBroadcast();
		Broadcast.LoadBroadcast();

		UE_LOGF(LogAvaRundownServerTests, Log, "Broadcast config restored.");
	}
}

DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER(FRundownServerWaitForResponse, int32, RequestId, TSharedPtr<UE::AvaRundownServerTests::FTestClient>, TestClient, TSharedPtr<IAvaRundownServer>, RundownServer);
bool FRundownServerWaitForResponse::Update()
{
	if (!TestClient.IsValid())
	{
		UE_LOGF(LogAvaRundownServerTests, Error, "Test Client is not valid.");
		return true;
	}
	
	return TestClient->HasReceivedResponse(RequestId);
}

DEFINE_LATENT_AUTOMATION_COMMAND(FRundownServerRestoreBroadcastConfig);
bool FRundownServerRestoreBroadcastConfig::Update()
{
	using namespace UE::AvaRundownServerTests;
	RestoreBroadcastConfig();
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRundownServerGetChannelImage, "MotionDesign.RundownServer.GetChannelImage", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter));
bool FRundownServerGetChannelImage::RunTest(const FString& Parameters)
{
	using namespace UE::AvaRundownServerTests;
	
	const TSharedPtr<IAvaRundownServer> RundownServer = GetOrCreateRundownServer();
	
	// Build a test client for this request.
	const TSharedPtr<FTestClient> TestClient = MakeShared<FTestClient>();
	TestClient->Init(this);

	// Send the request message.
	constexpr int32 RequestId = 1000;
	FAvaRundownGetChannelImage* Request = FMessageEndpoint::MakeMessage<FAvaRundownGetChannelImage>();
	Request->RequestId = RequestId;
	Request->ChannelName = UAvaBroadcast::Get().GetChannelName(0).ToString();
	
	TestClient->MessageEndpoint->Send(Request, RundownServer->GetMessageAddress());
	
	ADD_LATENT_AUTOMATION_COMMAND(FRundownServerWaitForResponse(RequestId, TestClient, RundownServer));
	return true;
}

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FRundownServerAddChannelDevice, UE::AvaRundownServerTests::FRundownServerTestBase, "MotionDesign.RundownServer.AddChannelDevice", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter));
bool FRundownServerAddChannelDevice::RunTest(const FString& Parameters)
{
	using namespace UE::AvaRundownServerTests;

	// Need to backup the broadcast config because we are about to make changes.
	BackupBroadcastConfig();
	
	const TSharedPtr<IAvaRundownServer> RundownServer = GetOrCreateRundownServer();

	// Build a test client for this request.
	const TSharedPtr<FTestClient> TestClient = MakeShared<FTestClient>();
	TestClient->Init(this);

	// Send the request message.
	FAvaRundownAddChannelDevice* Request = FMessageEndpoint::MakeMessage<FAvaRundownAddChannelDevice>();
	constexpr int32 RequestId = 1000;
	Request->RequestId = RequestId;
	Request->ChannelName = UAvaBroadcast::Get().GetChannelName(0).ToString();
	Request->MediaOutputName = TEXT("InvalidDevice");

	// Find an existing output so the request succeeds.
	const FAvaOutputTreeItemPtr OutputDevices = MakeShared<FAvaBroadcastOutputRootItem>();
	FAvaBroadcastOutputTreeItem::RefreshTree(OutputDevices, IAvaBroadcastOutputTreeItem::FRefreshChildrenParams());
	
	if (OutputDevices->GetChildren().Num() > 0)
	{
		FAvaOutputTreeItemPtr ExistingOutput = OutputDevices->GetChildren()[0];
		while (ExistingOutput.IsValid() && ExistingOutput->GetChildren().Num() > 0)
		{
			ExistingOutput = ExistingOutput->GetChildren()[0];
		}
		if (ExistingOutput.IsValid())
		{
			Request->MediaOutputName = ExistingOutput->GetDisplayName().ToString();
			UE_LOGF(LogAvaRundownServerTests, Log, "Selected media output for test: \"%ls\".", *Request->MediaOutputName);
		}
	}
	
	TestClient->MessageEndpoint->Send(Request, RundownServer->GetMessageAddress());
	
	ADD_LATENT_AUTOMATION_COMMAND(FRundownServerWaitForResponse(RequestId, TestClient, RundownServer));
	ADD_LATENT_AUTOMATION_COMMAND(FRundownServerRestoreBroadcastConfig());
	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRundownServerMediaOutputSerialization, "MotionDesign.RundownServer.MediaOutputSerialization", (EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter));
bool FRundownServerMediaOutputSerialization::RunTest(const FString& Parameters)
{
	for (const UClass* const Class : TObjectRange<UClass>())
	{
		// For now, we test all the media output classes, technically, this should work with all of them.
		// The result of this test depend on the plugins that are enabled. 
		// The Blackmagic plugin should be enabled at least.
		const bool bIsMediaOutputClass = Class->IsChildOf(UMediaOutput::StaticClass()) && Class != UMediaOutput::StaticClass();
		if (bIsMediaOutputClass)
		{
			const UMediaOutput* const MediaOutput = NewObject<UMediaOutput>(GetTransientPackage(), Class, NAME_None, RF_Transactional);
			FString MediaOutputJson = FAvaRundownServerMediaOutputUtils::SerializeMediaOutput(MediaOutput);

			const FString SaveName = FString::Printf(TEXT("%sSerializationTest_%s.json"), *FPaths::ProjectSavedDir(), *Class->GetName());
			UE_LOGF(LogAvaRundownServerTests, Display, "Serializing Media Output \"%ls\" to \"%ls\"", *Class->GetName(), *SaveName);

			TUniquePtr<FArchive> Archive(IFileManager::Get().CreateFileWriter(*SaveName));
			*Archive << MediaOutputJson;
			
			Archive->Close();
		}
	}
	return true;
}