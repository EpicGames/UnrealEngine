// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Templates/SharedPointer.h"

class IAvaRundownFilterSuggestionFactory;
class FExtensibilityManager;
class FName;
struct FAvaRundownPage;
struct FAvaRundownTextFilterArgs;
struct FSlateIcon;
enum class EAvaRundownSearchListType : uint8;
enum class ETextFilterComparisonOperation : uint8;

DECLARE_LOG_CATEGORY_EXTERN(LogAvaMediaEditor, Log, All);

class IAvaMediaEditorModule : public IModuleInterface
{
	static constexpr const TCHAR* ModuleName = TEXT("AvalancheMediaEditor");

public:
	static bool IsLoaded()
	{
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	static IAvaMediaEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IAvaMediaEditorModule>(ModuleName);
	}

	/** Returns the tool menu name used for Page Context Menu */
	static FName GetRundownPageMenuName()
	{
		return TEXT("AvaRundownPageContextMenu");
	}

	virtual FSlateIcon GetToolbarBroadcastButtonIcon() const = 0;

	/** Returns the toolbar extensibility manager for the Broadcast Editor */
	virtual TSharedPtr<FExtensibilityManager> GetBroadcastToolBarExtensibilityManager() = 0;

	/** Returns the toolbar extensibility manager for the Playback Editor */
	virtual TSharedPtr<FExtensibilityManager> GetPlaybackToolBarExtensibilityManager() = 0;

	/** Returns the toolbar extensibility manager for the Playback Editor */
	virtual TSharedPtr<FExtensibilityManager> GetRundownToolBarExtensibilityManager() = 0;

	/**
	 * Returns the context menu extensibility manager for the Rundown Editor's Template Pages
	 * @remark prefer extending with the UToolMenu named after IAvaMediaEditorModule::GetRundownPageMenuName, and using UAvaRundownPageContext to retrieve context information
	 */
	virtual TSharedPtr<FExtensibilityManager> GetRundownMenuExtensibilityManager() = 0;

	/**
 	* Check if current rundown filter expression factory support the comparison operation
 	* @param InFilterKey Filter key to get the rundown filter expression factory needed
 	* @param InOperation Operation to check if supported
 	* @param InRundownSearchListType Type of the Search List either Template or Instanced
 	* @return True if operation is supported, False otherwise
 	*/
	virtual bool CanFilterSupportComparisonOperation(const FName& InFilterKey, ETextFilterComparisonOperation InOperation, EAvaRundownSearchListType InRundownSearchListType) const = 0;

	/**
 	* Evaluate the expression and return the result
 	* @param InFilterKey Filter Key to get the Factory
 	* @param InItem Item that is currently checked
 	* @param InArgs Args to evaluate the expression see FAvaRundownTextFilterArgs for more information
 	* @return True if the expression evaluated to True, False otherwise
 	*/
	virtual bool FilterExpression(const FName& InFilterKey, const FAvaRundownPage& InItem, const FAvaRundownTextFilterArgs& InArgs) const = 0;

	/**
 	* Get all simple suggestions with the given type (Template/Instanced/All)
 	* @param InSuggestionType Type of suggestion to get, see EAvaRundownSearchListType for more information
 	* @return An Array containing all suggestions of the given type
 	*/
	virtual TArray<TSharedPtr<IAvaRundownFilterSuggestionFactory>> GetSimpleSuggestions(EAvaRundownSearchListType InSuggestionType) const = 0;

	/**
 	* Get all complex suggestions with the given type (Template/Instanced/All)
 	* @param InSuggestionType Type of suggestion to get, see EAvaRundownSearchListType for more information
 	* @return An Array containing all suggestions of the given type
 	*/
	virtual TArray<TSharedPtr<IAvaRundownFilterSuggestionFactory>> GetComplexSuggestions(EAvaRundownSearchListType InSuggestionType) const = 0;
};
