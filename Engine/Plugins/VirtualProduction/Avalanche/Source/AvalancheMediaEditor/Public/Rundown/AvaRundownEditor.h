// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaRundownEditorDefines.h"
#include "Rundown/AvaRundown.h"
#include "TickableEditorObject.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"

class FAvaRundownEditorInputProcessor;
class FAvaRundownManagedInstance;
class SAvaRundownInstancedPageList;
class SAvaRundownPageList;
class SAvaRundownReadPage;
enum class EAvaRundownSearchListType : uint8;

class FAvaRundownEditor : public FWorkflowCentricApplication
{
public:
	FAvaRundownEditor();

	virtual ~FAvaRundownEditor() override;

	void InitRundownEditor(const EToolkitMode::Type InMode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UAvaRundown* InRundown);

	//~ Begin IToolkit
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	//~ End IToolkit

	//~ Begin FAssetEditorToolkit
	virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;
	//~ End FAssetEditorToolkit
	
	TSharedRef<SWidget> MakeReadPageWidget();

	void ExtendToolBar(TSharedPtr<FExtender> InExtender);
	void FillPageToolBar(FToolBarBuilder& OutToolBarBuilder);

	bool IsRundownValid() const { return AvaRundown.IsValid(); }
	AVALANCHEMEDIAEDITOR_API UAvaRundown* GetRundown() const;

	/** Begin transaction buffer, but don't mark as dirty yet. */
	void BeginModify();
	
	/** Mark the rundown as dirty. */
	void MarkAsModified();

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPageEvent, const TArray<int32>&, UE::AvaRundown::EPageEvent);
	FOnPageEvent& GetOnPageEvent() { return OnPageEvent; }

	bool IsKeyRelevant(const FKeyEvent& InKeyEvent) const;
	bool HandleKeyDownEvent(const FKeyEvent& InKeyEvent);

	TSharedPtr<SAvaRundownPageList> GetTemplateListWidget() const;
	TSharedPtr<SAvaRundownPageList> GetInstanceListWidget() const;
	TSharedPtr<SAvaRundownPageList> GetListWidget(const FAvaRundownPageListReference& InPageListReference) const;
	TSharedPtr<SAvaRundownPageList> GetListWidget(const FName& InTabId) const;

	/** Returns the currently active Instanced Page List. */
	TSharedPtr<SAvaRundownInstancedPageList> GetActiveListWidget() const;
	
	/** Returns the currently focused Page List, including templates, instances or views. */
	TSharedPtr<SAvaRundownPageList> GetFocusedListWidget() const;

	AVALANCHEMEDIAEDITOR_API int32 GetFirstSelectedPageOnActiveSubListWidget() const;
	AVALANCHEMEDIAEDITOR_API TConstArrayView<int32> GetSelectedPagesOnActiveSubListWidget() const;
	AVALANCHEMEDIAEDITOR_API TConstArrayView<int32> GetSelectedPagesOnFocusedWidget() const;

	bool CanAddTemplate() const;
	void AddTemplate();

	bool CanPlaySelectedPage() const;
	void PlaySelectedPage();

	bool CanUpdateValuesOnSelectedPage() const;
	void UpdateValuesOnSelectedPage();
	
	bool CanStopSelectedPage(bool bInForce) const;
	void StopSelectedPage(bool bInForce);

	bool CanContinueSelectedPage() const;
	void ContinueSelectedPage();

	bool CanPlayNextPage() const;
	void PlayNextPage();

	/** Returns true if there's at least 1 page in the active page list that can be loaded */
	bool CanLoadAllPages() const;
	/** Requests load all the pages in this page list that can play */
	void LoadAllPages();

	/** Returns true if there's at least 1 page in from the next pages in active page list that can be loaded */
	bool CanLoadNextPages() const;
	/** Requests load the next pages that can play */
	void LoadNextPages();

	/** Returns true if there's at least 1 page from the list of selected pages in active page list that can be loaded */
	bool CanLoadSelectedPages() const;
	/** Requests load the selected pages that can play */
	void LoadSelectedPages();

	bool CanPreviewPlaySelectedPage() const;
	void PreviewPlaySelectedPage(bool bInToMark);

	bool CanPreviewContinueSelectedPage() const;
	void PreviewContinueSelectedPage();

	bool CanPreviewStopSelectedPage(bool bInForce) const;
	void PreviewStopSelectedPage(bool bInForce);

	bool CanPreviewPlayNextPage() const;
	void PreviewPlayNextPage();

	bool CanTakeToProgram() const;
	void TakeToProgram() const;

	bool CanCreateInstancesFromSelectedTemplates() const;
	void CreateInstancesFromSelectedTemplates();

	bool CanRemoveSelectedPages() const;
	void RemoveSelectedPages();

	void RefreshSubListTabs();

	bool RequestCloseDocumentTab(const FName& InDocumentTabId);

	void UnregisterDocumentTabFactory(const FName& InDocumentTabId);

protected:
	void RegisterApplicationModes();

	void CreateRundownCommands();

	FText GetCurrentProfileName() const;

	TSharedRef<SWidget> MakeProfileComboButton();

	void StartAutoPlayCommand(const TArray<FString>& InArgs);
	void StopAutoPlayCommand(const TArray<FString>& InArgs);
	void CancelAutoPlay();
	
	void LoadPageCommand(const TArray<FString>& InArgs);
	void UnloadPageCommand(const TArray<FString>& InArgs);
	void PlayPageCommand(const TArray<FString>& InArgs, EAvaRundownPagePlayType InPlayType);
	void ContinuePageCommand(const TArray<FString>& InArgs, bool bInPreview);
	void PlayNextPageCommand(const TArray<FString>& InArgs, bool bInPreview);
	void StopPageCommand(const TArray<FString>& InArgs, bool bInPreview, bool bInForce);
	void StopLayersCommand(const TArray<FString>& InArgs, bool bInPreview, bool bInForce);
	void TakeToProgramCommand(const TArray<FString>& InArgs);
	void StopPageTransitionCommand(const TArray<FString>& InArgs);
	void StartChannelCommand(const TArray<FString>& InArgs);
	void StopChannelCommand(const TArray<FString>& InArgs);
	
	TArray<int32> ArgumentsToPageIds(const TArray<FString>& InArgs, bool bInPreview, FString& OutErrors) const;

	using FMacroCommandFunction = TFunction<void(const TArray<FString>& InArgs)>;
	using FBindableMacroCommands = TMap<FName, FMacroCommandFunction>;
	const FBindableMacroCommands& GetBindableMacroCommands();

protected:
	/**
	 *	Tickable object that keeps track of the play time of a page and will automatically
	 *	play the next page once the specified play time has elapsed. The measure of play time
	 *	begins when the page has started playing, the loading time is ignored.
	 *	This is tracked from the page's status.
	 */
	struct FAutoPlayTicker : public FTickableEditorObject
	{
		static constexpr double DefaultPlayInterval = 10;

		TWeakPtr<FAvaRundownEditor> RundownEditorWeak;
		double PlayInterval = DefaultPlayInterval;
		double NextPageStartTime = 0;
		bool bIsPagePlaying = false;
		bool bIsCancelled = false;

		FAutoPlayTicker(TWeakPtr<FAvaRundownEditor> InRundownEditorWeak, double InPlayInterval);

		//~ Begin FTickableEditorObject
		virtual void Tick(float DeltaTime) override;
		virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
		virtual TStatId GetStatId() const override;
		//~ End FTickableEditorObject
	};
	TUniquePtr<FAutoPlayTicker> AutoPlayTicker;

	TSharedPtr<SDockTab> CreateSubListTab(const FAvaRundownPageListReference& InSubListReference);

	void RefreshSubListTab(const FAvaRundownPageListReference& InSubListReference, bool bInSetActive = false);

	void OnPageListChanged(const FAvaRundownPageListChangeParams& InParams);
	void OnActiveListChanged();
	void HandleOnPagePlayerAdded(UAvaRundown* InRundown, UAvaRundownPagePlayer* InPagePlayer);
	void OnCanClosePlaybackContext(const UAvaRundown* InRundown, bool& bOutResult) const;

	TWeakObjectPtr<UAvaRundown> AvaRundown;

	FOnPageEvent OnPageEvent;

private:
	TSharedPtr<SAvaRundownReadPage> ReadPageWidget;

	TSharedPtr<FAvaRundownEditorInputProcessor> InputProcessor;

	/**
	 * All rundown editor instances share the same command handlers.
	 * The commands get dispatched only to the active editor.
	 */
	class FSharedConsoleCommands;
	TSharedPtr<FSharedConsoleCommands> SharedConsoleCommands;

	FBindableMacroCommands BindableMacroCommands;
	
	/** Optional user specified action for stopping pages. */
	TOptional<bool> bStopPagesOnCloseOverride;
};
