// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

#include "AvaRundownEditorDefines.generated.h"

class IAvaRundownInstancedPageView;
class IAvaRundownPageView;
class IAvaRundownTemplatePageView;
struct FAvaRundownPage;

typedef TWeakPtr<IAvaRundownPageView> FAvaRundownPageViewWeak;
typedef TSharedPtr<IAvaRundownPageView> FAvaRundownPageViewPtr;
typedef TSharedRef<IAvaRundownPageView> FAvaRundownPageViewRef;
typedef TWeakPtr<IAvaRundownTemplatePageView> FAvaRundownTemplatePageViewWeak;
typedef TSharedPtr<IAvaRundownTemplatePageView> FAvaRundownTemplatePageViewPtr;
typedef TSharedRef<IAvaRundownTemplatePageView> FAvaRundownTemplatePageViewRef;
typedef TWeakPtr<IAvaRundownInstancedPageView> FAvaRundownInstancedPageViewWeak;
typedef TSharedPtr<IAvaRundownInstancedPageView> FAvaRundownInstancedPageViewPtr;
typedef TSharedRef<IAvaRundownInstancedPageView> FAvaRundownInstancedPageViewRef;


/** Defines the page set that an action applies to. */
UENUM()
enum class EAvaRundownPageSet : uint8
{
	/**
	 * The action applies to either the selected pages or playing pages if no pages are selected.
	 * If the selection is not empty and has no applicable pages, then the action is disabled.
	 */
	SelectedOrPlayingStrict,
	
	/**
	 * The action applies to applicable selected pages or playing pages if no applicable pages are selected.
	 * The selection may have pages, but if not applicable to the action, playing pages are used instead.
	 */
	SelectedOrPlaying,
	
	/**
	 * The action applies to selected pages only.
	 * For actions requiring the page to be playing, this effectively becomes "selected and playing".
	 */
	Selected,
	
	/** The action applies to playing pages only regardless of selection. */
	Playing
};

namespace UE::AvaRundown
{
	struct AVALANCHEMEDIAEDITOR_API FEditorMetrics
	{
		static constexpr float ColumnLeftOffset = 5.f;
		static const FNumberFormattingOptions PageIdFormattingOptions;
	};
	
	enum class EPageEvent : uint8
	{
		/** Called after the Selection has Changed */ 
		SelectionChanged,

		/** Called to sync the Selection of current Selected Items in the Caller */
		SelectionRequest,

		/** Called to Request a Rename of the Selected Items */
		RenameRequest,

		/** Called to Request Renumbering Page Id of Selected Items */
		RenumberRequest,

		/** Called to Request Reimporting Assets of Selected Items */
		ReimportRequest,
	};
}
