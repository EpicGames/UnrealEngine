// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMediaDefines.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "AvaRundownPageContext.generated.h"

class FAvaRundownEditor;
class FAvaRundownPageContextMenu;

UCLASS(MinimalAPI)
class UAvaRundownPageContext : public UObject
{
	GENERATED_BODY()

	friend FAvaRundownPageContextMenu;

public:
	TSharedPtr<FAvaRundownEditor> GetRundownEditor() const
	{
		return RundownEditorWeak.Pin();
	}

	const FAvaRundownPageListReference& GetPageListReference() const
	{
		return PageListReference;
	}

private:
	TWeakPtr<FAvaRundownPageContextMenu> ContextMenuWeak;

	TWeakPtr<FAvaRundownEditor> RundownEditorWeak;

	FAvaRundownPageListReference PageListReference;
};
