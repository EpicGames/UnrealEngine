// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/AvaRundownPage.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "AvaRundownPageControllerContext.generated.h"

class FAvaRundownEditor;
class FAvaRundownPageControllerContextMenu;
class SAvaRundownRCControllerPanel;

UCLASS(MinimalAPI)
class UAvaRundownPageControllerContext : public UObject
{
	GENERATED_BODY()

public:
	void InitContext(const TWeakPtr<FAvaRundownEditor>& InRundownEditorWeak
		, const int32 InRundownPageId
		, const TWeakPtr<SAvaRundownRCControllerPanel>& InControllerListWidgetWeak)
	{
		RundownEditorWeak = InRundownEditorWeak;
		RundownPageId = InRundownPageId;
		ControllerListWidgetWeak = InControllerListWidgetWeak;
	}

	TSharedPtr<FAvaRundownEditor> GetRundownEditor() const
	{
		return RundownEditorWeak.Pin();
	}

	int32 GetRundownPageId() const
	{
		return RundownPageId;
	}

	TSharedPtr<SAvaRundownRCControllerPanel> GetControllerListWidget() const
	{
		return ControllerListWidgetWeak.Pin();
	}

private:
	TWeakPtr<FAvaRundownEditor> RundownEditorWeak;

	int32 RundownPageId = FAvaRundownPage::InvalidPageId;

	TWeakPtr<SAvaRundownRCControllerPanel> ControllerListWidgetWeak;
};
