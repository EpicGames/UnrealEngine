// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rundown/AvaRundownPage.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "AvaRundownPagePropertyContext.generated.h"

class FAvaRundownEditor;
class FAvaRundownPagePropertyContextMenu;
class SAvaRundownPageRemoteControlProps;

UCLASS(MinimalAPI)
class UAvaRundownPagePropertyContext : public UObject
{
	GENERATED_BODY()

public:
	void InitContext(const TWeakPtr<FAvaRundownEditor>& InRundownEditorWeak
		, const int32 InRundownPageId
		, const TWeakPtr<SAvaRundownPageRemoteControlProps>& InPropertyListWidgetWeak)
	{
		RundownEditorWeak = InRundownEditorWeak;
		RundownPageId = InRundownPageId;
		PropertyListWidgetWeak = InPropertyListWidgetWeak;
	}

	TSharedPtr<FAvaRundownEditor> GetRundownEditor() const
	{
		return RundownEditorWeak.Pin();
	}

	int32 GetRundownPageId() const
	{
		return RundownPageId;
	}

	TSharedPtr<SAvaRundownPageRemoteControlProps> GetPropertyListWidget() const
	{
		return PropertyListWidgetWeak.Pin();
	}

private:
	TWeakPtr<FAvaRundownEditor> RundownEditorWeak;

	int32 RundownPageId = FAvaRundownPage::InvalidPageId;

	TWeakPtr<SAvaRundownPageRemoteControlProps> PropertyListWidgetWeak;
};
