// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheModifiers : ModuleRules
{
	public AvalancheModifiers(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"ActorModifier",
				"ActorModifierCore",
				"ActorModifierLayout",
				"Avalanche",
				"AvalancheSequence",
				"Core",
				"CoreUObject",
				"DynamicMesh",
				"Engine",
				"GeometryCore"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Avalanche",
				"AvalancheSceneTree",
				"AvalancheShapes",
				"ClonerEffectorMeshBuilder",
				"DynamicMaterial",
				"GeometryAlgorithms",
				"GeometryFramework",
				"GeometryMask",
				"GeometryScriptingCore",
				"MeshModelingToolsExp",
				"ModelingComponents",
				"ModelingOperators",
				"ProceduralMeshComponent",
				"Text3D"
			}
		);

		if (Target.Type == TargetRules.TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"AssetRegistry",
				"AssetTools",
				"AvalancheOutliner",
				"SlateCore",
				"UnrealEd"
			});
		}
	}
}
