// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Transform.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Tests/Framework/AvaTestUtils.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

// General Motion Design unit test log
DECLARE_LOG_CATEGORY_EXTERN(LogAvaModifiersTest, Log, All);

class AAvaShapeActor;
class AActor;
class AAvaTestDynamicMeshActor;
class AAvaTestStaticMeshActor;
class UWorld;

struct FAvaModifierTestUtils
{
public:
	FAvaModifierTestUtils(TSharedPtr<FAvaTestUtils> InTestUtils);

	AAvaShapeActor* SpawnShapeActor();
	AAvaTestDynamicMeshActor* SpawnTestDynamicMeshActor(FTransform InTransform = FTransform(FVector(0, 0, 0)));
	AAvaTestStaticMeshActor* SpawnTestStaticMeshActor();
	TArray<AAvaTestDynamicMeshActor*> SpawnTestDynamicMeshActors(int32 InNumberOfActors,
																 AAvaTestDynamicMeshActor* InParentActor = nullptr);

	FName GetModifierName(TSubclassOf<UActorModifierCoreBase> InModifierClass);
	UActorModifierCoreStack* GenerateModifierStackForActor(AActor* InModifiedActor);
	FActorModifierCoreStackInsertOp GenerateInsertOp(FName InModifierName);

	void LogMissingModifiers(const TSet<FName>& InMissingModifiers);

private:
	TSharedPtr<FAvaTestUtils> TestUtils;
};
