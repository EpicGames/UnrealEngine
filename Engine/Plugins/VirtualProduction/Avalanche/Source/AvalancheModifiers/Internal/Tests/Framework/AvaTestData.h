// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "UObject/NameTypes.h"

class FAvaTestData
{
public:
	FAvaTestData();
	TSet<FName> ExpectedConversionModifiers;
	TSet<FName> ExpectedGeometryModifiers;
	TSet<FName> ExpectedLayoutModifiersDynamicMesh;
	TSet<FName> ExpectedLayoutModifiersStaticMesh;
	TSet<FName> ExpectedRenderingModifiers;
};
