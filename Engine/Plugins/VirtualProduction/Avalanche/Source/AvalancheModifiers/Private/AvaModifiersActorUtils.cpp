// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaModifiersActorUtils.h"

#include "AvaActorUtils.h"
#include "AvaSceneTree.h"
#include "Components/PrimitiveComponent.h"
#include "Extensions/ActorModifierSceneTreeUpdateExtension.h"
#include "GameFramework/Actor.h"
#include "IAvaSceneInterface.h"
#include "Utilities/ActorModifierActorUtils.h"

#if WITH_EDITOR
#include "AvaOutlinerSubsystem.h"
#include "AvaOutlinerUtils.h"
#include "IAvaOutliner.h"
#endif

AActor* FAvaModifiersActorUtils::FindActorFromReferenceContainer(AActor* const InActor, const EActorModifierReferenceContainer InReferenceContainer, const bool bInIgnoreHiddenActors)
{
	if (InReferenceContainer == EActorModifierReferenceContainer::Other || !IsValid(InActor))
	{
		return nullptr;
	}

	// Note: Using Typed Outer World, instead of GetWorld(), since the typed outer World could be a streamed in world
	// and GetWorld() only returns the main world.
	UWorld* const World = InActor->GetTypedOuter<UWorld>();
	if (!IsValid(World))
	{
		return nullptr;
	}

	AActor* const ParentActor = InActor->GetAttachParentActor();

	bool bIsOutlinerAttachedActors = false;
	TArray<AActor*> AttachedActors;

#if WITH_EDITOR
	UAvaOutlinerSubsystem* const OutlinerSubsystem = World->GetSubsystem<UAvaOutlinerSubsystem>();
	if (IsValid(OutlinerSubsystem))
	{
		TSharedPtr<IAvaOutliner> AvaOutliner = OutlinerSubsystem->GetOutliner();
		if (AvaOutliner.IsValid())
		{
			bIsOutlinerAttachedActors = true;
			if (IsValid(ParentActor))
			{
				AttachedActors = FAvaOutlinerUtils::EditorOutlinerChildActors(AvaOutliner, ParentActor);
			}
			else // no valid parent, use world actors instead
			{
				TArray<FAvaOutlinerItemPtr> OutlinerRootChildren = AvaOutliner->GetTreeRoot()->GetChildren();
				const bool bValidRootActors = FAvaOutlinerUtils::EditorOutlinerItemsToActors(OutlinerRootChildren, AttachedActors);
			}
		}
	}
#endif

	if (!bIsOutlinerAttachedActors)
	{
		if (const IAvaSceneInterface* SceneInterface = FAvaActorUtils::GetSceneInterfaceFromActor(InActor))
		{
			const FAvaSceneTree& SceneTree = SceneInterface->GetSceneTree();

			if (!IsValid(ParentActor))
			{
				for (const int32 ChildId : SceneTree.GetRootNode().GetChildrenIndices())
				{
					const FAvaSceneItem* Item = SceneTree.GetItemAtIndex(ChildId);
					AttachedActors.Add(Item->Resolve<AActor>(World));
				}
			}
			else
			{
				AttachedActors = SceneTree.GetChildActors(ParentActor);
			}
		}
	}

	if (AttachedActors.Num() == 0)
	{
		return nullptr;
	}

	if (bInIgnoreHiddenActors)
	{
		for (int32 ChildIndex = AttachedActors.Num() - 1; ChildIndex >= 0; --ChildIndex)
		{
			if (AttachedActors[ChildIndex]->IsHidden()
#if WITH_EDITOR
				|| AttachedActors[ChildIndex]->IsTemporarilyHiddenInEditor()
#endif
				)
			{
				AttachedActors.RemoveAt(ChildIndex);
			}
		}
	}

	int32 FromActorIndex = -1;
	if (!AttachedActors.Find(InActor, FromActorIndex))
	{
		return nullptr;
	}

	switch (InReferenceContainer)
	{
		case EActorModifierReferenceContainer::Previous:
		{
			const int32 PreviousActorIndex = FromActorIndex - 1;
			if (!AttachedActors.IsValidIndex(PreviousActorIndex))
			{
				return ParentActor;
			}
			return AttachedActors[PreviousActorIndex];
		}
		case EActorModifierReferenceContainer::Next:
		{
			const int32 NextActorIndex = FromActorIndex + 1;
			if (!AttachedActors.IsValidIndex(NextActorIndex))
			{
				return nullptr;
			}
			return AttachedActors[NextActorIndex];
		}
		case EActorModifierReferenceContainer::First:
		{
			if (!AttachedActors.IsValidIndex(0))
			{
				return nullptr;
			}
			return AttachedActors[0] != InActor ? AttachedActors[0] : nullptr;
		}
		case EActorModifierReferenceContainer::Last:
		{
			AActor* const LastChildActor = AttachedActors.Last();
			if (!IsValid(LastChildActor))
			{
				return nullptr;
			}
			return LastChildActor != InActor ? LastChildActor : nullptr;
		}
		default: ;
	}

	return nullptr;
}

TArray<AActor*> FAvaModifiersActorUtils::GetReferenceActors(const FActorModifierSceneTreeActor* InTrackedActor)
{
	TArray<AActor*> ReferenceActors;

	if (!InTrackedActor)
	{
		return ReferenceActors;
	}

	AActor* LocalActor = InTrackedActor->GetLocalActor();
	if (!IsValid(LocalActor))
	{
		return ReferenceActors;
	}

	if (InTrackedActor->ReferenceContainer == EActorModifierReferenceContainer::Other)
	{
		if (AActor* ReferenceActor = InTrackedActor->ReferenceActorWeak.Get())
		{
			ReferenceActors.Add(ReferenceActor);
		}

		return ReferenceActors;
	}

	AActor* ContextActor = LocalActor;
	while (AActor* NewReferenceActor = FAvaModifiersActorUtils::FindActorFromReferenceContainer(ContextActor, InTrackedActor->ReferenceContainer, false))
	{
		// Take only siblings
		if (NewReferenceActor->GetAttachParentActor() != LocalActor->GetAttachParentActor())
		{
			break;
		}

		ReferenceActors.Add(NewReferenceActor);
		ContextActor = NewReferenceActor;

		if (InTrackedActor->bSkipHiddenActors
				&& (NewReferenceActor->IsHidden()
#if WITH_EDITOR
				|| NewReferenceActor->IsTemporarilyHiddenInEditor()
#endif
				))
		{
			continue;
		}

		break;
	}

	return ReferenceActors;
}