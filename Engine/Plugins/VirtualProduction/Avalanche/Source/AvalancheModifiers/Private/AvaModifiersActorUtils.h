// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/Box.h"
#include "Math/OrientedBox.h"
#include "Math/TransformNonVectorized.h"
#include "UObject/WeakObjectPtrTemplates.h"

enum class EActorModifierReferenceContainer : uint8;
enum class EActorModifierAlignment : uint8;
class AActor;
struct FActorModifierSceneTreeActor;

// All operations that can be reused or shared in modifiers should go here
struct FAvaModifiersActorUtils
{
	/** Begin Outliner */
	static AActor* FindActorFromReferenceContainer(AActor* const InActor, const EActorModifierReferenceContainer InReferenceContainer, const bool bInIgnoreHiddenActors);
	static TArray<AActor*> GetReferenceActors(const FActorModifierSceneTreeActor* InTrackedActor);
	/** End Outliner */
};