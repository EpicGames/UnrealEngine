// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaModifiersModule.h"

#include "AvaModifiersLog.h"
#include "AvaModifiersSceneTreeResolver.h"
#include "Engine/Level.h"
#include "Extensions/ActorModifierSceneTreeUpdateExtension.h"
#include "GameFramework/Actor.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogAvaModifier);

IMPLEMENT_MODULE(FAvalancheModifiersModule, AvalancheModifiers)

void FAvalancheModifiersModule::StartupModule()
{
	FActorModifierSceneTreeUpdateExtension::OnGetSceneTreeResolver().BindRaw(this, &FAvalancheModifiersModule::GetSceneTreeResolver);
}

void FAvalancheModifiersModule::ShutdownModule()
{
	FActorModifierSceneTreeUpdateExtension::OnGetSceneTreeResolver().Unbind();
}

TSharedPtr<IActorModifierSceneTreeCustomResolver> FAvalancheModifiersModule::GetSceneTreeResolver(ULevel* InLevel)
{
	if (IsValid(InLevel))
	{
		return MakeShared<FAvaModifiersSceneTreeResolver>(InLevel);
	}

	return nullptr;
}
