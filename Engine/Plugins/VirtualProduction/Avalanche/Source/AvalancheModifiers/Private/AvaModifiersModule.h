// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointerFwd.h"

class IActorModifierSceneTreeCustomResolver;
class ULevel;

class FAvalancheModifiersModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

private:
	TSharedPtr<IActorModifierSceneTreeCustomResolver> GetSceneTreeResolver(ULevel* InLevel);
};