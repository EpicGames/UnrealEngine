// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaModifiersParametricMaterial.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"

FAvaModifiersParametricMaterial::FAvaModifiersParametricMaterial()
{
	DefaultMaterial  = nullptr;
	InstanceMaterial = nullptr;
	
	MaskColor = FLinearColor::Red;

	static UMaterial* LoadedMaterial = LoadResource();
	DefaultMaterial = LoadedMaterial;
}

UMaterial* FAvaModifiersParametricMaterial::GetDefaultMaterial() const
{
	return DefaultMaterial;
}

UMaterialInstanceDynamic* FAvaModifiersParametricMaterial::GetMaterial() const
{
	return InstanceMaterial;
}

void FAvaModifiersParametricMaterial::ApplyChanges(UObject* Outer)
{
	EnsureCurrentMaterial(Outer);
	ApplyParams();
}

void FAvaModifiersParametricMaterial::ApplyParams() const
{
	if (!InstanceMaterial)
	{
		return;
	}

	InstanceMaterial->SetVectorParameterValue(TEXT("MaskColor"), MaskColor);
	InstanceMaterial->SetScalarParameterValue(TEXT("GlobalOpacity"), MaskColor.A);
}

void FAvaModifiersParametricMaterial::CreateAndApply(UObject* Outer)
{
	InstanceMaterial = UMaterialInstanceDynamic::Create(GetDefaultMaterial(),Outer);
	ApplyParams();
}

void FAvaModifiersParametricMaterial::EnsureCurrentMaterial(UObject* Outer)
{
	if (GetMaterial() == nullptr)
	{
		CreateAndApply(Outer);
	}
}

UMaterial* FAvaModifiersParametricMaterial::LoadResource() const
{
	// get material asset
	static const FString AssetPath = TEXT("/Script/Engine.Material'/Avalanche/ToolboxResources/M_Toolbox_Mask.M_Toolbox_Mask'");
	
	UMaterial* LoadedMaterial = FindObject<UMaterial>(nullptr, *AssetPath);
	if (!LoadedMaterial)
	{
		LoadedMaterial = LoadObject<UMaterial>(nullptr, *AssetPath);
	}

	return LoadedMaterial;
}
