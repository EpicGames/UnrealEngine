// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaModifiersPreviewPlane.h"

#include "Components/SceneComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "GameFramework/Actor.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceDynamic.h"

FAvaModifierPreviewPlane::FAvaModifierPreviewPlane()
{
	static UStaticMesh* PreviewMesh = LoadPreviewResource();
	PreviewStaticMesh = PreviewMesh;
}

void FAvaModifierPreviewPlane::Create(USceneComponent* InActorComponent)
{
	if (PreviewComponent
		|| !PreviewStaticMesh
		|| !InActorComponent
		|| !InActorComponent->GetOwner())
	{
		return;
	}

	AActor* Actor = InActorComponent->GetOwner();

	PreviewComponent = NewObject<UStaticMeshComponent>(Actor);
	PreviewComponent->OnComponentCreated();
	PreviewComponent->SetupAttachment(InActorComponent);
	PreviewComponent->RegisterComponent();

	PreviewComponent->SetHiddenInGame(true);
	PreviewComponent->SetBoundsScale(UE_KINDA_SMALL_NUMBER);
	PreviewComponent->Bounds = FBox(ForceInitToZero);

#if WITH_EDITOR
	PreviewComponent->SetIsVisualizationComponent(true);
#endif

	PreviewDynMaterial.MaskColor = FLinearColor(0, 1, 0, 0.2f);
	PreviewDynMaterial.ApplyChanges();
	PreviewComponent->SetMaterial(0, PreviewDynMaterial.GetMaterial());
}

void FAvaModifierPreviewPlane::Destroy()
{
	if (!PreviewComponent)
	{
		return;
	}

	PreviewComponent->DestroyComponent();
	PreviewComponent = nullptr;
}

void FAvaModifierPreviewPlane::Show() const
{
	if (!PreviewComponent)
	{
		return;
	}

	PreviewComponent->SetStaticMesh(PreviewStaticMesh);
	PreviewComponent->SetVisibility(true, false);
}

void FAvaModifierPreviewPlane::Hide() const
{
	if (!PreviewComponent)
	{
		return;
	}

	PreviewComponent->SetVisibility(false, false);
	PreviewComponent->SetStaticMesh(nullptr);
}

void FAvaModifierPreviewPlane::Update(const FTransform& InRelativeTransform) const
{
	if (!PreviewComponent)
	{
		return;
	}

	PreviewComponent->SetMaterial(0, PreviewDynMaterial.GetMaterial());
	PreviewComponent->SetWorldScale3D(InRelativeTransform.GetScale3D());
	PreviewComponent->SetRelativeLocation(InRelativeTransform.GetLocation());
	PreviewComponent->SetRelativeRotation(InRelativeTransform.GetRotation());
}

UStaticMesh* FAvaModifierPreviewPlane::LoadPreviewResource() const
{
	// get material asset
	static const FString AssetPath = TEXT("/Script/Engine.StaticMesh'/Avalanche/EditorResources/SM_TwoSidedPlane.SM_TwoSidedPlane'");

	UStaticMesh* LoadedMesh = FindObject<UStaticMesh>(nullptr, *AssetPath);
	if (!LoadedMesh)
	{
		LoadedMesh = LoadObject<UStaticMesh>(nullptr, *AssetPath);
	}

	return LoadedMesh;
}
