// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaActorUtils.h"
#include "AvaSceneSubsystem.h"
#include "AvaSceneTree.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Extensions/ActorModifierSceneTreeUpdateExtension.h"
#include "GameFramework/Actor.h"
#include "IAvaSceneInterface.h"

#if WITH_EDITOR
#include "AvaOutlinerSubsystem.h"
#include "AvaOutlinerUtils.h"
#include "IAvaOutliner.h"
#endif

/** This allows modifier to preserve hierarchy of the Motion Design outliner */
class FAvaModifiersSceneTreeResolver : public IActorModifierSceneTreeCustomResolver
{
public:
	explicit FAvaModifiersSceneTreeResolver(ULevel* InLevel)
		: LevelWeak(InLevel)
	{}

	virtual void Activate() override
	{
#if WITH_EDITOR
		const ULevel* Level = LevelWeak.Get();

		if (!Level)
		{
			return;
		}

		const UWorld* World = Level->GetWorld();

		if (!World)
		{
			return;
		}

		if (UAvaOutlinerSubsystem* const OutlinerSubsystem = World->GetSubsystem<UAvaOutlinerSubsystem>())
		{
			UAvaOutlinerSubsystem::FActorHierarchyChanged& ActorHierarchyChanged = OutlinerSubsystem->OnActorHierarchyChanged();

			ActorHierarchyChanged.RemoveAll(this);
			ActorHierarchyChanged.AddSP(this, &FAvaModifiersSceneTreeResolver::OnOutlinerHierarchyChanged);

			if (const TSharedPtr<IAvaOutliner> Outliner = OutlinerSubsystem->GetOutliner())
			{
				IAvaOutliner::FOnOutlinerLoaded& OnOutlinerLoaded = Outliner->GetOnOutlinerLoaded();
				OnOutlinerLoaded.RemoveAll(this);
				OnOutlinerLoaded.AddSP(this, &FAvaModifiersSceneTreeResolver::OnOutlinerLoaded);
			}
		}
#endif
	}

	virtual void Deactivate() override
	{
#if WITH_EDITOR
		const ULevel* Level = LevelWeak.Get();
		
		if (!Level)
		{
			return;
		}

		const UWorld* World = Level->GetWorld();

		if (!World)
		{
			return;
		}

		if (UAvaOutlinerSubsystem* const OutlinerSubsystem = World->GetSubsystem<UAvaOutlinerSubsystem>())
		{
			OutlinerSubsystem->OnActorHierarchyChanged().RemoveAll(this);

			if (const TSharedPtr<IAvaOutliner> Outliner = OutlinerSubsystem->GetOutliner())
			{
				Outliner->GetOnOutlinerLoaded().RemoveAll(this);
			}
		}
#endif
	}

	virtual bool GetDirectChildrenActor(AActor* InActor, TArray<AActor*>& OutActors) const override
	{
		if (const IAvaSceneInterface* SceneInterface = FAvaActorUtils::GetSceneInterfaceFromActor(InActor))
		{
			const FAvaSceneTree& SceneTree = SceneInterface->GetSceneTree();
			OutActors = SceneTree.GetChildActors(InActor);
			return true;
		}
		return false;
	}

	virtual bool GetRootActors(ULevel* InLevel, TArray<AActor*>& OutActors) const override
	{
		if (!IsValid(InLevel))
		{
			return false;	
		}
	
		bool bIsOutlinerAttachedActors = false;
	
#if WITH_EDITOR
		if (const UWorld* const World = InLevel->GetTypedOuter<UWorld>())
		{
			if (const UAvaOutlinerSubsystem* const OutlinerSubsystem = World->GetSubsystem<UAvaOutlinerSubsystem>())
			{
				if (const TSharedPtr<IAvaOutliner> AvaOutliner = OutlinerSubsystem->GetOutliner())
				{
					OutActors = FAvaOutlinerUtils::EditorOutlinerChildActors(AvaOutliner, nullptr);
					bIsOutlinerAttachedActors = true;
				}
			}
		}
#endif

		if (!bIsOutlinerAttachedActors)
		{
			if (const UWorld* const World = InLevel->GetWorld())
			{
				if (const UAvaSceneSubsystem* SceneSubsystem = World->GetSubsystem<UAvaSceneSubsystem>())
				{
					if (const IAvaSceneInterface* SceneInterface = SceneSubsystem->GetSceneInterface(InLevel))
					{
						const FAvaSceneTree& SceneTree = SceneInterface->GetSceneTree();
						OutActors = SceneTree.GetRootActors(InLevel);
						bIsOutlinerAttachedActors = true;
					}
				}
			}
		}

		return bIsOutlinerAttachedActors;
	}

	virtual FOnActorHierarchyChanged::RegistrationType& OnActorHierarchyChanged() override
	{
		return OnHierarchyChangedDelegate;
	}

private:
#if WITH_EDITOR
	void OnOutlinerLoaded()
	{
		OnHierarchyChangedDelegate.Broadcast(/** Actor */nullptr);
	}

	void OnOutlinerHierarchyChanged(AActor* InActor, const AActor* InParent, EAvaOutlinerHierarchyChangeType InChange)
	{
		OnHierarchyChangedDelegate.Broadcast(InActor);
	}
#endif

	FOnActorHierarchyChanged OnHierarchyChangedDelegate;
	TWeakObjectPtr<ULevel> LevelWeak;
};