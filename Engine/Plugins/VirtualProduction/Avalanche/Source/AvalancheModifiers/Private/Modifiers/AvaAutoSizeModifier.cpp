// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaAutoSizeModifier.h"

#include "Async/Async.h"
#include "AvaModifiersActorUtils.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/MeshTransforms.h"
#include "AvaShapeActor.h"
#include "DynamicMeshes/AvaShape2DDynMeshBase.h"
#include "Shared/ActorModifierTransformShared.h"
#include "Utilities/ActorModifierActorUtils.h"

#define LOCTEXT_NAMESPACE "AvaAutoSizeModifier"

// todo : Add function to UE::ActorModifier::ActorUtils
namespace UE::ActorModifier::Private
{
	FBox GetActorLocalBounds(AActor* InActor, bool bInIncludeChildren, bool bInSkipHidden)
	{
		TSet<TWeakObjectPtr<AActor>> AttachedModifyActors {InActor};

		if (bInIncludeChildren)
		{
			TArray<AActor*> AttachedActors;
			InActor->GetAttachedActors(AttachedActors, /** Reset */false, /** Recurse */true);
			Algo::Transform(AttachedActors, AttachedModifyActors, [](AActor* InAttachedActor)
			{
				return InAttachedActor;
			});
		}

		constexpr bool bTransformBox = false;
		return UE::ActorModifier::ActorUtils::GetActorsBounds(AttachedModifyActors, InActor->GetActorTransform(), bInSkipHidden, bTransformBox);
	}
}

bool UAvaAutoSizeModifier::IsModifierDirtyable() const
{
	AActor* ActorModified = GetModifiedActor();
	AActor* const TrackedActor = ReferenceActor.ReferenceActorWeak.Get();

	if (!IsValid(TrackedActor) || !IsValid(ActorModified))
	{
		return Super::IsModifierDirtyable();
	}

	constexpr bool bSkipHidden = true;
	const FBox ReferenceActorLocalBounds = UE::ActorModifier::Private::GetActorLocalBounds(TrackedActor, bIncludeChildren, bSkipHidden);
	const FBox ModifiedActorBounds = UE::ActorModifier::Private::GetActorLocalBounds(ActorModified, /** Recurse */false, /** SkipHidden */false);

	// Compute the ratio between reference and modified actor bounds size and compare it to cached version (without X axis, only YZ are considered for Width/Height)
	const FVector RatioBounds = FVector(0, 1, 1) * (ReferenceActorLocalBounds.GetSize() / FVector::Max(ModifiedActorBounds.GetSize(), FVector(UE_SMALL_NUMBER)));
	if (RatioBounds.Equals(CachedFollowLocation, 0.01))
	{
		return Super::IsModifierDirtyable();
	}

	return true;
}

void UAvaAutoSizeModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.AllowTick(true);
	InMetadata.SetName(TEXT("AutoSize"));
	InMetadata.SetCategory(TEXT("Geometry"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Auto Size"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "The modified actor will be resized to act as a background for a reference actor"));
#endif

	InMetadata.SetCompatibilityRule([](const AActor* InActor)->bool
	{
		bool bSupported = false;
		if (InActor)
		{
			if (const UDynamicMeshComponent* DynMeshComponent = InActor->FindComponentByClass<UDynamicMeshComponent>())
			{
				DynMeshComponent->ProcessMesh([&bSupported](const FDynamicMesh3& ProcessMesh)
				{
					bSupported = ProcessMesh.VertexCount() > 0 && static_cast<FBox>(ProcessMesh.GetBounds(true)).GetSize().X == 0;
				});
			}
		}
		return bSupported;
	});
}

void UAvaAutoSizeModifier::OnModifiedActorTransformed()
{
	// Do nothing when moved
}

void UAvaAutoSizeModifier::SavePreState()
{
	if (UAvaShape2DDynMeshBase* const Shape2DDynMesh = ShapeDynMesh2DWeak.Get())
	{
		// Skip saving pre-state for Motion Design 2D Shapes as the mesh isn't dealt with directly in Apply.
		// @see UAvaAutoSizeModifier::RestorePreState for more details
	}
	else
	{
		Super::SavePreState();
	}
}

void UAvaAutoSizeModifier::RestorePreState()
{
	if (UAvaShape2DDynMeshBase* const Shape2DDynMesh = ShapeDynMesh2DWeak.Get())
	{
		// Don't restore the mesh when dealing with Motion Design Shapes.
		// Apply for 2D shapes only modifies the Size2D of the shape, not the mesh directly.
		// Size2D isn't saved/restore either as this modifier doesn't rely on the Size2D previous state. It only restores back the original size when disabled.
	}
	else
	{
		Super::RestorePreState();
	}
}

void UAvaAutoSizeModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

	AddExtension<FActorModifierTransformUpdateExtension>(this);
	AddExtension<FActorModifierRenderStateUpdateExtension>(this);
	AddExtension<FActorModifierSceneTreeUpdateExtension>(this);

	if (FActorModifierSceneTreeUpdateExtension* SceneExtension = GetExtension<FActorModifierSceneTreeUpdateExtension>())
	{
		SceneExtension->TrackSceneTree(0, &ReferenceActor);
	}

	bDeprecatedPropertiesMigrated = true;
}

void UAvaAutoSizeModifier::OnModifierEnabled(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierEnabled(InReason);

	// Save actor layout state
	if (UActorModifierTransformShared* LayoutShared = GetShared<UActorModifierTransformShared>(true))
	{
		LayoutShared->SaveActorState(this, GetModifiedActor());
	}

	if (InReason == EActorModifierCoreEnableReason::User)
	{
		if (const AAvaShapeActor* const ReferenceAsShapeActor = Cast<AAvaShapeActor>(GetModifiedActor()))
		{
			if (UAvaShape2DDynMeshBase* const Shape2DDynMesh = Cast<UAvaShape2DDynMeshBase>(ReferenceAsShapeActor->GetDynamicMesh()))
			{
				ShapeDynMesh2DWeak = Shape2DDynMesh;
				PreModifierShapeDynMesh2DSize = Shape2DDynMesh->GetSize2D();
			}
		}
	}
}

void UAvaAutoSizeModifier::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierDisabled(InReason);

	// Restore actor layout state
	if (UActorModifierTransformShared* LayoutShared = GetShared<UActorModifierTransformShared>(false))
	{
		LayoutShared->RestoreActorState(this, GetModifiedActor());
	}

	if (UAvaShape2DDynMeshBase* const Shape2DDynMesh = ShapeDynMesh2DWeak.Get())
	{
		Shape2DDynMesh->SetSize2D(PreModifierShapeDynMesh2DSize);
	}
}

void UAvaAutoSizeModifier::Apply()
{
	AActor* const CurrentlyModifiedActor = GetModifiedActor();

	AActor* TrackedActor = ReferenceActor.ReferenceActorWeak.Get();
	if (!IsValid(TrackedActor))
	{
		Next();
		return;
	}

	UDynamicMeshComponent* const DynMeshComponent = GetMeshComponent();
	if (!IsValid(DynMeshComponent))
	{
		Fail(LOCTEXT("InvalidDynamicMeshComponent", "Invalid dynamic mesh component on modified actor"));
		return;
	}

	if (FActorModifierRenderStateUpdateExtension* RenderExtension = GetExtension<FActorModifierRenderStateUpdateExtension>())
	{
		RenderExtension->SetTrackedActorVisibility(TrackedActor, bIncludeChildren);
	}

	constexpr bool bSkipHidden = true;
	CachedReferenceBounds = UE::ActorModifier::Private::GetActorLocalBounds(TrackedActor, bIncludeChildren, bSkipHidden);

	const FBox ModifiedActorBounds = UE::ActorModifier::Private::GetActorLocalBounds(CurrentlyModifiedActor, /** Recurse */false, /** SkipHidden */false);

	// Reuse this unused property for now since headers cannot be modified in a hotfix release
	// Store the ratio between reference and modified actor bounds size (without X axis, only YZ are considered for Width/Height)
	// todo : rename this property to CachedActorSizeRatio
	CachedFollowLocation = FVector(0, 1, 1) * (CachedReferenceBounds.GetSize() / FVector::Max(ModifiedActorBounds.GetSize(), FVector(UE_SMALL_NUMBER)));

	FVector ModifiedActorBoundsOrigin;
	FVector ModifiedActorBoundsExtent;
	ModifiedActorBounds.GetCenterAndExtents(ModifiedActorBoundsOrigin, ModifiedActorBoundsExtent);

	FVector ReferenceBoundsExtent = CachedReferenceBounds.GetExtent();

	// Add padding only if there is content inside otherwise don't
	if (!CachedReferenceBounds.GetExtent().IsNearlyZero())
	{
		ReferenceBoundsExtent += FVector(0, PaddingHorizontal, PaddingVertical);
	}

	// Avoid division by 0
	if (ModifiedActorBoundsExtent.IsNearlyZero())
	{
		ModifiedActorBoundsExtent += FVector(UE_KINDA_SMALL_NUMBER);
	}

	const float ScaleRatioY = ReferenceBoundsExtent.Y/ModifiedActorBoundsExtent.Y;
	const float ScaleRatioZ = ReferenceBoundsExtent.Z/ModifiedActorBoundsExtent.Z;
	const FVector ScaleRatio = FVector(1.0f, ScaleRatioY, ScaleRatioZ);

	// check if we can use Shape2DDynMesh SetSize2D for this modified mesh.
	// this allows us to keep properly scaled corner bevels and slants
	if (UAvaShape2DDynMeshBase* const Shape2DDynMesh = ShapeDynMesh2DWeak.Get())
	{
		FVector2D DesiredSize2D = FVector2D::Max(
			FVector2D(ReferenceBoundsExtent.Y, ReferenceBoundsExtent.Z)*2.0f,
			UAvaShape2DDynMeshBase::MinSize2D
		);

		switch (FitMode)
		{
			case EAvaAutoSizeFitMode::WidthAndHeight:
				// nothing to do
				break;
			case EAvaAutoSizeFitMode::WidthOnly:
				{
					DesiredSize2D = FVector2D(DesiredSize2D.X, Shape2DDynMesh->GetSize2D().Y);
				}
				break;
			case EAvaAutoSizeFitMode::HeightOnly:
				{
					DesiredSize2D = FVector2D(Shape2DDynMesh->GetSize2D().X, DesiredSize2D.Y);
				}
				break;
			default: ;
		}

		// Not liking this as this will update the shape and trigger a stack update,
		// we might create a loop where the modifier dirties the shape and the shape dirties modifiers
		if (!Shape2DDynMesh->GetSize2D().Equals(DesiredSize2D, 0.01))
		{
			Shape2DDynMesh->SetSize2D(DesiredSize2D);
		}
	}
	else // for all other dynamic meshes, just scale all vertices
	{
		FVector NewScaleValue = CurrentlyModifiedActor->GetActorScale3D() * ScaleRatio;

		switch (FitMode)
		{
			case EAvaAutoSizeFitMode::WidthAndHeight:
				break;
			case EAvaAutoSizeFitMode::WidthOnly:
				NewScaleValue.Z = 1.0f;
				break;
			case EAvaAutoSizeFitMode::HeightOnly:
				NewScaleValue.Y = 1.0f;
				break;
			default: ;
		}

		DynMeshComponent->GetDynamicMesh()->EditMesh([&NewScaleValue](FDynamicMesh3& EditMesh)
		{
			MeshTransforms::Scale(EditMesh, NewScaleValue, FVector::Zero(), true);
		}
		, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	}

	Next();
}

void UAvaAutoSizeModifier::PostLoad()
{
	if (!bDeprecatedPropertiesMigrated
		&& ReferenceActor.ReferenceContainer == EActorModifierReferenceContainer::Other
		&& ReferenceActor.ReferenceActorWeak == nullptr)
	{
		ReferenceActor.ReferenceContainer = ReferenceContainer_DEPRECATED;
		ReferenceActor.ReferenceActorWeak = ReferenceActorWeak_DEPRECATED;
		ReferenceActor.bSkipHiddenActors = bIgnoreHiddenActors_DEPRECATED;
		bDeprecatedPropertiesMigrated = true;
	}

	const FVector2D Padding2D(Padding.Left + Padding.Right, Padding.Top + Padding.Bottom);
	if (!Padding2D.IsNearlyZero()
		&& PaddingHorizontal == 0
		&& PaddingVertical == 0)
	{
		Padding = FMargin(0);
		PaddingHorizontal = Padding2D.X;
		PaddingVertical = Padding2D.Y;
	}

	Super::PostLoad();
}

#if WITH_EDITOR
void UAvaAutoSizeModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	static const FName ReferenceActorPropertyName = GET_MEMBER_NAME_CHECKED(UAvaAutoSizeModifier, ReferenceActor);
	static const FName PaddingHorizontalPropertyName = GET_MEMBER_NAME_CHECKED(UAvaAutoSizeModifier, PaddingHorizontal);
	static const FName PaddingVerticalPropertyName = GET_MEMBER_NAME_CHECKED(UAvaAutoSizeModifier, PaddingVertical);
	static const FName IncludeChildrenName = GET_MEMBER_NAME_CHECKED(UAvaAutoSizeModifier, bIncludeChildren);
	static const FName FitModePropertyName = GET_MEMBER_NAME_CHECKED(UAvaAutoSizeModifier, FitMode);

	if (MemberName == ReferenceActorPropertyName)
	{
		OnReferenceActorChanged();
	}
	else if (MemberName == PaddingHorizontalPropertyName
		|| MemberName == PaddingVerticalPropertyName
		|| MemberName == FitModePropertyName
		|| MemberName == IncludeChildrenName)
	{
		MarkModifierDirty();
	}
}
#endif

void UAvaAutoSizeModifier::SetReferenceActor(const FActorModifierSceneTreeActor& InReferenceActor)
{
	if (ReferenceActor == InReferenceActor)
	{
		return;
	}

	ReferenceActor = InReferenceActor;
	OnReferenceActorChanged();
}

void UAvaAutoSizeModifier::SetPaddingHorizontal(double InPadding)
{
	if (FMath::IsNearlyEqual(InPadding, PaddingHorizontal))
	{
		return;
	}

	PaddingHorizontal = InPadding;
	MarkModifierDirty();
}

void UAvaAutoSizeModifier::SetPaddingVertical(double InPadding)
{
	if (FMath::IsNearlyEqual(InPadding, PaddingVertical))
	{
		return;
	}

	PaddingVertical = InPadding;
	MarkModifierDirty();
}

void UAvaAutoSizeModifier::SetFitMode(const EAvaAutoSizeFitMode InFitMode)
{
	if (FitMode != InFitMode)
	{
		FitMode = InFitMode;

		MarkModifierDirty();
	}
}

void UAvaAutoSizeModifier::SetIncludeChildren(bool bInIncludeChildren)
{
	if (bIncludeChildren == bInIncludeChildren)
	{
		return;
	}

	bIncludeChildren = bInIncludeChildren;
	MarkModifierDirty();
}

void UAvaAutoSizeModifier::OnTransformUpdated(AActor* InActor, bool bInParentMoved)
{
	if (bInParentMoved)
	{
		return;
	}

	OnRenderStateUpdated(InActor, nullptr);
}

void UAvaAutoSizeModifier::OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent)
{
	const AActor* ActorModified = GetModifiedActor();
	if (!ActorModified)
	{
		return;
	}

	const AActor* TrackedActor = ReferenceActor.ReferenceActorWeak.Get();
	if (!TrackedActor)
	{
		return;
	}

	const bool bIsReferenceActor = InActor == TrackedActor;
	const bool bIsReferenceActorChildren = InActor->IsAttachedTo(TrackedActor);

	if (bIsReferenceActor
		|| (bIncludeChildren && bIsReferenceActorChildren))
	{
		const bool bIsModifierDirtyable = IsModifierDirtyable();

		// Only update if bounds changed
		if (bIsModifierDirtyable)
		{
			MarkModifierDirty();
		}
	}
}

void UAvaAutoSizeModifier::OnActorVisibilityChanged(AActor* InActor)
{
	OnRenderStateUpdated(InActor, nullptr);
}

void UAvaAutoSizeModifier::OnSceneTreeTrackedActorChanged(int32 InIdx, AActor* InPreviousActor, AActor* InNewActor)
{
	if (InNewActor == GetModifiedActor())
	{
		OnReferenceActorChanged();
		return;
	}

	// Untrack reference actor
	if (FActorModifierTransformUpdateExtension* TransformExtension = GetExtension<FActorModifierTransformUpdateExtension>())
	{
		TransformExtension->UntrackActor(InPreviousActor);
		TransformExtension->TrackActor(InNewActor, true);
	}

	MarkModifierDirty();
}

void UAvaAutoSizeModifier::OnSceneTreeTrackedActorChildrenChanged(int32 InIdx,
	const TSet<TWeakObjectPtr<AActor>>& InPreviousChildrenActors,
	const TSet<TWeakObjectPtr<AActor>>& InNewChildrenActors)
{
	if (bIncludeChildren)
	{
		if (IsModifierDirtyable())
		{
			MarkModifierDirty();
		}
	}
}

void UAvaAutoSizeModifier::OnReferenceActorChanged()
{
	if (ReferenceActor.ReferenceActorWeak.Get() == GetModifiedActor())
	{
		ReferenceActor.ReferenceActorWeak = nullptr;
	}

	if (const FActorModifierSceneTreeUpdateExtension* SceneExtension = GetExtension<FActorModifierSceneTreeUpdateExtension>())
	{
		SceneExtension->CheckTrackedActorUpdate(0);
	}
}

#undef LOCTEXT_NAMESPACE
