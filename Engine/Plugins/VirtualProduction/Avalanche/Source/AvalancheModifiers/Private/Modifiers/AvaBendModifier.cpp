// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaBendModifier.h"

#include "Async/Async.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMeshEditor.h"
#include "Selections/MeshConnectedComponents.h"
#include "SpaceDeformerOps/BendMeshOp.h"

#define LOCTEXT_NAMESPACE "AvaBendModifier"

void UAvaBendModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("Bend"));
	InMetadata.SetCategory(TEXT("Geometry"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Bend"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Bend the current geometry shape with a transition between two sides"));
#endif
	InMetadata.AddDependency(TEXT("Subdivide"));
}

void UAvaBendModifier::Apply()
{
	UDynamicMeshComponent* const DynMeshComp = GetMeshComponent();
	if (!IsValid(DynMeshComp))
	{
		Fail(LOCTEXT("InvalidDynamicMeshComponent", "Invalid dynamic mesh component on modified actor"));
		return;
	}

	if (FMath::IsNearlyZero(Angle))
	{
		Next();
		return;
	}

	const FTransform BendTransform(BendRotation, BendPosition);
	if (!BendTransform.IsValid())
	{
		Next();
		return;
	}
	
	using namespace UE::Geometry;
			
	constexpr float LowerExtent = 10.0f;
	const FFrame3d BendFrame = FFrame3d(BendTransform);

	DynMeshComp->GetDynamicMesh()->EditMesh([this, BendFrame, LowerExtent](FDynamicMesh3& EditMesh) 
	{
		FMeshConnectedComponents Components(&EditMesh);
		Components.FindConnectedTriangles();

		TArray<TSharedPtr<FDynamicMesh3>> Submeshes;
		for (const FMeshConnectedComponents::FComponent& MeshComponent : Components.Components)
		{
			TSharedRef<FDynamicMesh3> Submesh = MakeShared<FDynamicMesh3>();
			Submesh->EnableMatchingAttributes(EditMesh);

			FDynamicMeshEditor Editor(&Submesh.Get());

			FMeshIndexMappings IndexMap;
			FDynamicMeshEditResult Result;
			Editor.AppendTriangles(&EditMesh, MeshComponent.Indices, IndexMap, Result);

			if (Submesh->TriangleCount() > 0)
			{
				Submeshes.Add(MoveTemp(Submesh));
			}
		}

		for (const TSharedPtr<FDynamicMesh3>& Submesh : Submeshes)
		{
			const FBox MeshBounds = static_cast<FBox>(Submesh->GetBounds(true));
			const float BendExtent = (MeshBounds.GetSize().Z / 2.0) * Extent;

			FBendMeshOp BendOperation;
			BendOperation.OriginalMesh = Submesh;
			BendOperation.GizmoFrame = BendFrame;
			BendOperation.LowerBoundsInterval = (bSymmetricExtents) ? -BendExtent : -LowerExtent;
			BendOperation.UpperBoundsInterval = BendExtent;
			BendOperation.BendDegrees = Angle;
			BendOperation.bLockBottom = !bBidirectional;
			BendOperation.CalculateResult(nullptr);
			*Submesh = MoveTemp(*BendOperation.ExtractResult().Release());
		}

		if (!Submeshes.IsEmpty())
		{
			FDynamicMesh3 MergeMesh;
			MergeMesh.EnableMatchingAttributes(EditMesh);
			FDynamicMeshEditor Editor(&MergeMesh);
			for (const TSharedPtr<FDynamicMesh3>& Submesh : Submeshes)
			{
				FMeshIndexMappings IndexMap;
				Editor.AppendMesh(Submesh.Get(), IndexMap);
			}

			EditMesh = MoveTemp(MergeMesh);
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	Next();
}

#if WITH_EDITOR
void UAvaBendModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();
	
	static const FName BendPositionName = GET_MEMBER_NAME_CHECKED(UAvaBendModifier, BendPosition);
	static const FName BendRotationName = GET_MEMBER_NAME_CHECKED(UAvaBendModifier, BendRotation);
	static const FName AngleName = GET_MEMBER_NAME_CHECKED(UAvaBendModifier, Angle);	
	static const FName ExtentName = GET_MEMBER_NAME_CHECKED(UAvaBendModifier, Extent);
	static const FName SymmetricExtentsName = GET_MEMBER_NAME_CHECKED(UAvaBendModifier, bSymmetricExtents);
	static const FName BidirectionalName = GET_MEMBER_NAME_CHECKED(UAvaBendModifier, bBidirectional);

	if (MemberName == BendPositionName ||
		MemberName == BendRotationName)
	{
		OnBendTransformChanged();
	}
	else if (MemberName == AngleName ||
		MemberName == ExtentName ||
		MemberName == SymmetricExtentsName ||
		MemberName == BidirectionalName)
	{
		OnBendOptionChanged();
	}
}
#endif

void UAvaBendModifier::SetAngle(float InAngle)
{
	if (Angle == InAngle)
	{
		return;
	}

	Angle = InAngle;
	OnBendOptionChanged();
}

void UAvaBendModifier::SetExtent(float InExtent)
{
	if (Extent == InExtent)
	{
		return;
	}

	Extent = InExtent;
	OnBendOptionChanged();
}

void UAvaBendModifier::SetBendPosition(const FVector& InBendPosition)
{
	if (BendPosition == InBendPosition)
	{
		return;
	}

	BendPosition = InBendPosition;
	OnBendTransformChanged();
}

void UAvaBendModifier::SetBendRotation(const FRotator& InBendRotation)
{
	if (BendRotation == InBendRotation)
	{
		return;
	}

	BendRotation = InBendRotation;
	OnBendTransformChanged();
}

void UAvaBendModifier::SetSymmetricExtents(bool bInSymmetricExtents)
{
	if (bSymmetricExtents == bInSymmetricExtents)
	{
		return;
	}

	bSymmetricExtents = bInSymmetricExtents;
	OnBendOptionChanged();
}

void UAvaBendModifier::SetBidirectional(bool bInBidirectional)
{
	if (bBidirectional == bInBidirectional)
	{
		return;
	}

	bBidirectional = bInBidirectional;
	OnBendOptionChanged();
}

void UAvaBendModifier::OnBendTransformChanged()
{
	MarkModifierDirty();
}

void UAvaBendModifier::OnBendOptionChanged()
{
	MarkModifierDirty();
}

#undef LOCTEXT_NAMESPACE
