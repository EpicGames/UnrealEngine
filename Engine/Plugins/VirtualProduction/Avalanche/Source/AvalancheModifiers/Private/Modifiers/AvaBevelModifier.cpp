// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaBevelModifier.h"

#include "Async/Async.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "GroupTopology.h"
#include "Operations/MeshBevel.h"
#include "Parameterization/DynamicMeshUVEditor.h"

#define LOCTEXT_NAMESPACE "AvaBevelModifier"

void UAvaBevelModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("Bevel"));
	InMetadata.SetCategory(TEXT("Geometry"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Bevel"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Create chamfered or rounded corners on geometry that smooth edges and corners"));
#endif
}

void UAvaBevelModifier::Apply()
{
	UDynamicMeshComponent* const DynMeshComp = GetMeshComponent();

	if (!IsValid(DynMeshComp))
	{
		Fail(LOCTEXT("InvalidDynamicMeshComponent", "Invalid dynamic mesh component on modified actor"));
		return;
	}

	if (Inset <= 0.f || DynMeshComp->GetDynamicMesh()->GetTriangleCount() == 0)
	{
		Next();
		return;
	}

	using namespace UE::Geometry;

	DynMeshComp->GetDynamicMesh()->EditMesh([this, DynMeshComp](FDynamicMesh3& EditMesh)
	{
		// Weld edges
		FMergeCoincidentMeshEdges Welder(&EditMesh);
		Welder.Apply();

		// Apply bevel operator for multiple iterations
		const FGroupTopology Topology(&EditMesh, true);
		FMeshBevel Bevel;
		Bevel.InsetDistance = Inset;
		Bevel.NumSubdivisions = Iterations;
		Bevel.RoundWeight = Iterations > 0 ? Roundness : 0;
		Bevel.InitializeFromGroupTopology(EditMesh, Topology);
		Bevel.Apply(EditMesh, nullptr);

		// Get polygroup layer for back side
		FDynamicMeshPolygroupAttribute* const BevelPolygroup = FindOrCreatePolygroupLayer(EditMesh, UAvaBevelModifier::BevelPolygroupLayerName, &Bevel.NewTriangles);

		// TODO : Fix UVs temp, UV modifier needed instead of this
		FRotator BoxRotation = DynMeshComp->GetComponentTransform().Rotator();
		FBox MeshBounds = static_cast<FBox>(EditMesh.GetBounds(true));
		const FTransform PlaneTransform(BoxRotation, MeshBounds.GetCenter(), FVector::OneVector);
		FDynamicMeshUVOverlay* UVOverlay = EditMesh.Attributes()->GetUVLayer(0);
		FDynamicMeshUVEditor UVEditor(&EditMesh, UVOverlay);
		const FFrame3d ProjectionFrame(PlaneTransform);
		FUVEditResult Result;
		UVEditor.SetTriangleUVsFromBoxProjection(Bevel.NewTriangles, [](const FVector3d& Pos)
		{
			return Pos;
		}
		, ProjectionFrame, MeshBounds.GetSize(), 3, &Result);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	Next();
}

#if WITH_EDITOR
void UAvaBevelModifier::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UAvaBevelModifier, Inset))
	{
		OnInsetChanged();
	}
	else if (MemberName == GET_MEMBER_NAME_CHECKED(UAvaBevelModifier, Iterations))
	{
		OnIterationsChanged();
	}
	else if (MemberName == GET_MEMBER_NAME_CHECKED(UAvaBevelModifier, Roundness))
	{
		OnRoundnessChanged();
	}
}
#endif

void UAvaBevelModifier::SetInset(float InInset)
{
	InInset = FMath::Clamp(InInset, UAvaBevelModifier::MinInset, GetMaxInsetDistance());

	if (FMath::IsNearlyEqual(Inset, InInset))
	{
		return;
	}

	Inset = InInset;
	OnInsetChanged();
}

void UAvaBevelModifier::SetIterations(int32 InIterations)
{
	InIterations = FMath::Clamp(InIterations, UAvaBevelModifier::MinIterations, UAvaBevelModifier::MaxIterations);

	if (Iterations == InIterations)
	{
		return;
	}

	Iterations = InIterations;
	OnIterationsChanged();
}

void UAvaBevelModifier::SetRoundness(float InRoundness)
{
	InRoundness = FMath::Clamp(InRoundness, UAvaBevelModifier::MinRoundness, UAvaBevelModifier::MaxRoundness);

	if (FMath::IsNearlyEqual(Roundness, InRoundness))
	{
		return;
	}

	Roundness = InRoundness;
	OnRoundnessChanged();
}

void UAvaBevelModifier::OnInsetChanged()
{
	Inset = FMath::Min(Inset, GetMaxInsetDistance());
	MarkModifierDirty();
}

void UAvaBevelModifier::OnIterationsChanged()
{
	MarkModifierDirty();
}

void UAvaBevelModifier::OnRoundnessChanged()
{
	MarkModifierDirty();
}

float UAvaBevelModifier::GetMaxInsetDistance() const
{
	if (!PreModifierCachedMesh.IsSet())
	{
		return 0.f;
	}

	const FBox Bounds = static_cast<FBox>(PreModifierCachedMesh.GetValue().GetBounds(true));
	const FVector Size3d = Bounds.GetSize();
	return FMath::Max(0, FMath::Min3(Size3d.X / 2, Size3d.Y / 2, Size3d.Z / 2) - 0.001f);
}

#undef LOCTEXT_NAMESPACE
