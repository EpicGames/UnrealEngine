// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaBooleanModifier.h"

#include "Async/Async.h"
#include "Components/DynamicMeshComponent.h"
#include "Containers/Ticker.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "GeometryScript/MeshBasicEditFunctions.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Operations/MeshBoolean.h"
#include "Operations/MeshSelfUnion.h"
#include "Operations/OffsetMeshRegion.h"
#include "Profilers/AvaBooleanModifierProfiler.h"
#include "Shared/AvaBooleanModifierShared.h"
#include "Shared/ActorModifierVisibilityShared.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"

#define LOCTEXT_NAMESPACE "AvaBooleanModifier"

void UAvaBooleanModifier::OnMaskVisibilityChange(const UWorld* World, bool bMaskActorVisible) const
{
	if (GetWorld() == World)
	{
		if (Mode != EAvaBooleanMode::None)
		{
			if (UDynamicMeshComponent* DynMeshComp = GetMeshComponent())
			{
				DynMeshComp->SetVisibility(bMaskActorVisible);
			}
		}
	}
}

void UAvaBooleanModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetProfilerClass<FAvaBooleanModifierProfiler>();
	InMetadata.SetName(TEXT("Boolean"));
	InMetadata.SetCategory(TEXT("Geometry"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Boolean"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Subtracts or intersects a geometry shape with another one when they collide"));
#endif
}

void UAvaBooleanModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

	if (FActorModifierTransformUpdateExtension* TransformExtension = AddExtension<FActorModifierTransformUpdateExtension>(this))
	{
		TransformExtension->TrackActor(GetModifiedActor(), true);
	}

	UpdateMaskDelegates();
}

void UAvaBooleanModifier::SavePreState()
{
	Super::SavePreState();

	SaveOriginalMaterials();
}

void UAvaBooleanModifier::RestorePreState()
{
	Super::RestorePreState();

	RestoreOriginalMaterials();
}

void UAvaBooleanModifier::Apply()
{
	if (!IsMeshValid())
	{
		Fail(LOCTEXT("InvalidDynamicMeshComponent", "Invalid dynamic mesh component on modified actor"));
		return;
	}

	const bool bIsEmptyTriangleCount = PreModifierCachedMesh->TriangleCount() == 0;
	const bool bHasNoAttributes = !PreModifierCachedMesh->HasAttributes();
	if (bIsEmptyTriangleCount || bHasNoAttributes)
	{
		Fail(LOCTEXT("InvalidMeshData", "Invalid triangle count or attributes"));
		return;
	}

	// Update masking material here too to avoid flickering between mask and original materials due to async task
	UpdateMaskingMaterials();

	ApplyInternal();
	UpdateMaskingMaterials();

	Next();
}

void UAvaBooleanModifier::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierDisabled(InReason);

	UpdateMaskVisibility();
	UpdateMaskingMaterials();
	// unbind delegates
	UpdateMaskDelegates();
	// Track modifier
	if (UAvaBooleanModifierShared* Shared = GetShared<UAvaBooleanModifierShared>(false))
	{
		Shared->UntrackModifierChannel(this);
	}
	// update colliding shapes
	if (Mode != EAvaBooleanMode::None)
	{
		for (TWeakObjectPtr<UAvaBooleanModifier> CollidingModifier : CollidingModifiers)
		{
			if (CollidingModifier.IsValid())
			{
				CollidingModifier->MarkModifierDirty();
			}
		}
	}
}

void UAvaBooleanModifier::OnModifierEnabled(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierEnabled(InReason);

	UpdateMaskVisibility();
	UpdateMaskingMaterials();
	// bind delegates
	UpdateMaskDelegates();
	// Track modifier
	if (UAvaBooleanModifierShared* Shared = GetShared<UAvaBooleanModifierShared>(true))
	{
		Shared->TrackModifierChannel(this);
	}
	// update colliding shapes
	if (Mode != EAvaBooleanMode::None)
	{
		for (TWeakObjectPtr<UAvaBooleanModifier> CollidingModifier : CollidingModifiers)
		{
			if (CollidingModifier.IsValid())
			{
				CollidingModifier->MarkModifierDirty();
			}
		}
	}
}

void UAvaBooleanModifier::OnModifierRemoved(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierRemoved(InReason);

	UAvaShapeDynamicMeshBase::OnMaskVisibility.RemoveAll(this);
}

void UAvaBooleanModifier::OnTransformUpdated(AActor* InActor, bool bInParentMoved)
{
	if (!InActor || InActor != GetModifiedActor())
	{
		return;
	}

	// Update if we have moved the actual actor with the modifier
	if (!bInParentMoved)
	{
		MarkModifierDirty();
		return;
	}

	// Delay to let the transform update propagate to attachment tree
	UActorModifierCoreSubsystem::GetModifierTicker().AddTicker(FTickerDelegate::CreateWeakLambda(this, [this, bInParentMoved, InActor](float InDelta)
	{
		// Check if colliding modifiers will be affected by this move
		for (const TWeakObjectPtr<UAvaBooleanModifier>& CollidingModifierWeak : CollidingModifiers)
		{
			const UAvaBooleanModifier* CollidingModifier = CollidingModifierWeak.Get();
			if (!CollidingModifier)
			{
				continue;
			}

			const AActor* CollidingActor = CollidingModifier->GetModifiedActor();
			if (!CollidingActor)
			{
				continue;
			}

			const FTransform CurrentRelativeTransform = InActor->GetActorTransform().GetRelativeTransform(CollidingActor->GetActorTransform());
			const FTransform LastRelativeTransform = LastTransform.GetRelativeTransform(CollidingModifier->LastTransform);

			// Update if parent has moved and caused a difference between this modifier and colliding modifiers
			if (bInParentMoved
				&& !CurrentRelativeTransform.Equals(LastRelativeTransform, 0.01))
			{
				MarkModifierDirty();
				return false;
			}
		}

		if (UAvaBooleanModifierShared* Shared = GetShared<UAvaBooleanModifierShared>(false))
		{
			// Update if new colliding modifiers are found after move
			const TSet<TWeakObjectPtr<UAvaBooleanModifier>> OutCollidingModifiersWeak = Shared->GetIntersectingModifiers(this);
			if (OutCollidingModifiersWeak.Num() != CollidingModifiers.Num())
			{
				MarkModifierDirty();
				return false;
			}

			// Update if previous colliding modifiers are not colliding anymore after move
			const TSet<TWeakObjectPtr<UAvaBooleanModifier>> NotCollidingModifiersWeak = CollidingModifiers.Difference(OutCollidingModifiersWeak);
			if (!NotCollidingModifiersWeak.IsEmpty())
			{
				MarkModifierDirty();
				return false;
			}
		}

		return false;
	}));
}

#if WITH_EDITOR
void UAvaBooleanModifier::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	static const FName ModeName = GET_MEMBER_NAME_CHECKED(UAvaBooleanModifier, Mode);
	static const FName ChannelName = GET_MEMBER_NAME_CHECKED(UAvaBooleanModifier, Channel);

	if (MemberName == ModeName)
	{
		OnModeChanged();
	}
	else if (MemberName == ChannelName)
	{
		OnChannelChanged();
	}
}
#endif

void UAvaBooleanModifier::SetMode(EAvaBooleanMode InMode)
{
	if (Mode == InMode)
	{
		return;
	}

	Mode = InMode;
	OnModeChanged();
}

void UAvaBooleanModifier::SetChannel(uint8 InChannel)
{
	if (Channel == InChannel)
	{
		return;
	}

	Channel = InChannel;
	OnChannelChanged();
}

void UAvaBooleanModifier::CreateMaskDepth() const
{
	// Only apply Depth on the mask shape
	if (Mode == EAvaBooleanMode::None)
	{
		return;
	}

	using namespace UE::Geometry;

	static const FVector ExtrudeDepth(-FVector::XAxisVector * UAvaBooleanModifier::MinDepth);

	// cannot apply boolean with tool mesh if no depth
	GetMeshComponent()->EditMesh([this](FDynamicMesh3& EditMesh)
	{
		FOffsetMeshRegion Extruder(&EditMesh);
		for (int32 Tid : EditMesh.TriangleIndicesItr())
		{
			Extruder.Triangles.Add(Tid);
		}
		Extruder.OffsetPositionFunc = [this](const FVector3d& Position, const FVector3d& VertexVector, int VertexID)
		{
			return Position + ExtrudeDepth;
		};
		Extruder.bIsPositiveOffset = true;
		Extruder.UVScaleFactor = 0.01f;
		Extruder.bOffsetFullComponentsAsSolids = false;
		Extruder.Apply();

		// move mesh back from half depth
		MeshTransforms::Translate(EditMesh, -ExtrudeDepth/2);
	});
}

void UAvaBooleanModifier::ApplyInternal()
{
	UAvaBooleanModifierShared* Shared = GetShared<UAvaBooleanModifierShared>(false);
	if (!Shared)
	{
		Fail(LOCTEXT("InvalidSharedObject", "Invalid boolean modifier shared object"));
		return;
	}

	const bool bIsMasking = Mode != EAvaBooleanMode::None;

	// Ensure mask has a depth before testing intersections
	if (bIsMasking)
	{
		CreateMaskDepth();
	}

	// find other colliding shapes
	TSet<TWeakObjectPtr<UAvaBooleanModifier>> OutCollidingModifiersWeak = Shared->GetIntersectingModifiers(this, &ChannelInfo);

	if (!OutCollidingModifiersWeak.IsEmpty())
	{
		if (bIsMasking)
		{
			for (const TWeakObjectPtr<UAvaBooleanModifier>& CollidingModifierWeak : OutCollidingModifiersWeak)
			{
				UAvaBooleanModifier* CollidingModifier = CollidingModifierWeak.Get();
				if (!CollidingModifier)
				{
					continue;
				}

				// Mark other dirty to restore and reapply this mask
				CollidingModifier->MarkModifierDirty();
			}
		}
		else
		{
			using namespace UE::Geometry;

			TMap<EAvaBooleanMode, FDynamicMesh3> ModeTools;

			// Build the mode tools
			for (const TWeakObjectPtr<UAvaBooleanModifier>& CollidingModifierWeak : OutCollidingModifiersWeak)
			{
				const UAvaBooleanModifier* CollidingModifier = CollidingModifierWeak.Get();
				if (!CollidingModifier)
				{
					continue;
				}

				const EAvaBooleanMode ToolMode = CollidingModifier->GetMode();
				if (ToolMode == EAvaBooleanMode::None)
				{
					continue;
				}

				const UDynamicMeshComponent* ToolMeshComponent = CollidingModifier->GetMeshComponent();
				if (!ToolMeshComponent)
				{
					continue;
				}

				FDynamicMesh3& ModeTool = ModeTools.FindOrAdd(ToolMode);

				ToolMeshComponent->ProcessMesh([ToolMeshComponent, &ModeTool](const FDynamicMesh3& InToolMesh)
				{
					FDynamicMesh3 ToolMesh = InToolMesh;
					MeshTransforms::ApplyTransform(ToolMesh, ToolMeshComponent->GetComponentTransform());

					FGeometryScriptAppendMeshOptions AppendOptions;
					AppendOptions.CombineMode = EGeometryScriptCombineAttributesMode::EnableAllMatching;
					FMeshIndexMappings TmpMappings;
					AppendOptions.UpdateAttributesForCombineMode(ModeTool, ToolMesh);

					FDynamicMeshEditor Editor(&ModeTool);
					Editor.AppendMesh(&ToolMesh, TmpMappings);
				});
			}

			// Remove tools intersection
			for (TPair<EAvaBooleanMode, FDynamicMesh3>& ModeToolPair : ModeTools)
			{
				FMeshSelfUnion Union(&ModeToolPair.Value);
				Union.WindingThreshold = 0.5f;
				Union.bTrimFlaps = true;
				Union.bSimplifyAlongNewEdges = true;
				Union.SimplificationAngleTolerance = 0.01;
				Union.Compute();
			}

			FDynamicMesh3 OutputMesh;
			GetMeshComponent()->ProcessMesh([&OutputMesh](const FDynamicMesh3& InEditMesh)
			{
				OutputMesh = InEditMesh;
			});

			// Apply mode tools
			for (const TPair<EAvaBooleanMode, FDynamicMesh3>& ModeToolPair : ModeTools)
			{
				UAvaBooleanModifier::ApplyTool(ModeToolPair.Key, ModeToolPair.Value, this, OutputMesh);
			}

			GetMeshComponent()->EditMesh([&OutputMesh](FDynamicMesh3& InEditMesh)
			{
				InEditMesh = MoveTemp(OutputMesh);
			});
		}
	}

	// update non colliding shapes
	TSet<TWeakObjectPtr<UAvaBooleanModifier>> NotCollidingModifiersWeak = CollidingModifiers.Difference(OutCollidingModifiersWeak);
	for (const TWeakObjectPtr<UAvaBooleanModifier>& NotCollidingModifierWeak : NotCollidingModifiersWeak)
	{
		UAvaBooleanModifier* NotCollidingModifier = NotCollidingModifierWeak.Get();
		if (!NotCollidingModifier)
		{
			continue;
		}

		// Only update if other is not a mask
		if (NotCollidingModifier->GetMode() == EAvaBooleanMode::None)
		{
			NotCollidingModifier->MarkModifierDirty();
		}
	}

	CollidingModifiers = OutCollidingModifiersWeak;

	if (const AActor* ActorModified = GetModifiedActor())
	{
		LastTransform = ActorModified->GetActorTransform();
	}
}

void UAvaBooleanModifier::OnModeChanged()
{
	UpdateMaskVisibility();
	UpdateMaskingMaterials();
	UpdateMaskDelegates();
	OnMaskingOptionsChanged();
}

void UAvaBooleanModifier::OnChannelChanged()
{
	if (UAvaBooleanModifierShared* Shared = GetShared<UAvaBooleanModifierShared>(true))
	{
		Shared->UpdateModifierChannel(this);
	}

	OnMaskingOptionsChanged();
}

void UAvaBooleanModifier::OnMaskingOptionsChanged()
{
	MarkModifierDirty();
}

void UAvaBooleanModifier::SaveOriginalMaterials()
{
	if (const UDynamicMeshComponent* DynMeshComp = GetMeshComponent())
	{
		const int32 MaterialCount = DynMeshComp->GetNumMaterials();

		TArray<TObjectPtr<UMaterialInterface>> Materials;
		for (int32 MatIdx = 0; MatIdx < MaterialCount; MatIdx++)
		{
			const UMaterialInterface* Mat = DynMeshComp->GetMaterial(MatIdx);
			if (!Mat || Mat != ParametricMaskMaterial.GetMaterial())
			{
				Materials.Add(DynMeshComp->GetMaterial(MatIdx));
			}
		}

		if (Materials.Num() == MaterialCount)
		{
			OriginalMaterials = Materials;
		}
	}
}

void UAvaBooleanModifier::RestoreOriginalMaterials()
{
	if (UDynamicMeshComponent* DynMeshComp = GetMeshComponent())
	{
		const int32 MaterialCount = DynMeshComp->GetNumMaterials();

		for (int32 MatIdx = 0; MatIdx < MaterialCount; MatIdx++)
		{
			if (OriginalMaterials.IsValidIndex(MatIdx) && DynMeshComp->GetMaterial(MatIdx) == ParametricMaskMaterial.GetMaterial())
			{
				DynMeshComp->SetMaterial(MatIdx, OriginalMaterials[MatIdx].Get());
			}
		}

		OriginalMaterials.Empty();
	}
}

void UAvaBooleanModifier::UpdateMaskingMaterials()
{
	RestoreOriginalMaterials();

	const bool bIsMasking = (Mode != EAvaBooleanMode::None) && IsModifierEnabled();

	if (bIsMasking)
	{
		FLinearColor MaskColor;
		switch(Mode)
		{
			case EAvaBooleanMode::Intersect:
				MaskColor = FLinearColor::Blue;
			break;
			case EAvaBooleanMode::Subtract:
				MaskColor = FLinearColor::Red;
			break;
			case EAvaBooleanMode::Union:
				MaskColor = FLinearColor::Green;
			break;
			default:
			return;
		}

		MaskColor.A = 0.03f;

		ParametricMaskMaterial.MaskColor = MaskColor;
		ParametricMaskMaterial.ApplyChanges();

		if (UDynamicMeshComponent* DynMeshComp = GetMeshComponent())
		{
			const int32 MaterialCount = DynMeshComp->GetNumMaterials();

			// Save before switching
			SaveOriginalMaterials();

			// Set Mask material
			for (int32 MatIdx = 0; MatIdx < MaterialCount; MatIdx++)
			{
				DynMeshComp->SetMaterial(MatIdx, ParametricMaskMaterial.GetMaterial());
			}
		}
	}
}

void UAvaBooleanModifier::UpdateMaskDelegates()
{
	const bool bIsMasking = (Mode != EAvaBooleanMode::None) && IsModifierEnabled();

	if (bIsMasking)
	{
		UAvaShapeDynamicMeshBase::OnMaskVisibility.AddUObject(this, &UAvaBooleanModifier::OnMaskVisibilityChange);
		UAvaShapeDynamicMeshBase::OnMaskEnabled.Broadcast(GetModifiedActor());
	}
	else
	{
		UAvaShapeDynamicMeshBase::OnMaskVisibility.RemoveAll(this);
		UAvaShapeDynamicMeshBase::OnMaskDisabled.Broadcast(GetModifiedActor());
	}
}

void UAvaBooleanModifier::UpdateMaskVisibility()
{
	if (AActor* ActorModified = GetModifiedActor())
	{
		if (UActorModifierVisibilityShared* VisibilityShared = GetShared<UActorModifierVisibilityShared>(true))
		{
			if (Mode != EAvaBooleanMode::None)
			{
				// Save state
				if (IsModifierEnabled())
				{
					VisibilityShared->SaveActorState(this, ActorModified);
					VisibilityShared->SetActorVisibility(this, ActorModified, true, false, EActorModifierVisibilityActor::Game);
					return;
				}
			}

			// Restore state
			if (VisibilityShared->IsActorStateSaved(this, ActorModified))
			{
				VisibilityShared->RestoreActorState(this, ActorModified);
			}
		}
	}
}

void UAvaBooleanModifier::ApplyTool(EAvaBooleanMode InMode, const UE::Geometry::FDynamicMesh3& InModeTool, const UAvaBooleanModifier* InTarget, UE::Geometry::FDynamicMesh3& OutMesh)
{
	if (!IsValid(InTarget)
		|| !InTarget->GetMeshComponent()
		|| InTarget->GetMode() != EAvaBooleanMode::None)
	{
		return;
	}

	using namespace UE::Geometry;

	UDynamicMeshComponent* TargetDynMesh = InTarget->GetMeshComponent();

	TargetDynMesh->ProcessMesh([TargetDynMesh, InMode, &InModeTool, &OutMesh](const FDynamicMesh3& InSourceMesh)
	{
		FTransformSRT3d SourceTransform(TargetDynMesh->GetComponentTransform());

		FMeshBoolean::EBooleanOp Operation;
		switch (InMode)
		{
			case EAvaBooleanMode::Intersect:
			{
				Operation = FMeshBoolean::EBooleanOp::Intersect;
			}
			break;

			case EAvaBooleanMode::Subtract:
			{
				Operation = FMeshBoolean::EBooleanOp::Difference;
			}
			break;

			case EAvaBooleanMode::Union:
			{
				Operation = FMeshBoolean::EBooleanOp::Union;
			}
			break;

			default: return;
		}

		FMeshBoolean MeshBoolean(
		&OutMesh
			, SourceTransform
			, &InModeTool
			, FTransform::Identity
			, &OutMesh
			, Operation
		);
		MeshBoolean.bPutResultInInputSpace = true;
		MeshBoolean.bSimplifyAlongNewEdges = true;
		MeshBoolean.bWeldSharedEdges = true;
		MeshBoolean.bCollapseDegenerateEdgesOnCut = true;
		MeshBoolean.bPreserveTriangleGroups = true;
		MeshBoolean.bTrackAllNewEdges = false;
		MeshBoolean.Compute();

		if (OutMesh.TriangleCount() > 0)
		{
			// Boolean result is in the space of TargetTransform, so invert that
			MeshTransforms::ApplyTransformInverse(OutMesh, SourceTransform, true);
		}
	});
}

#undef LOCTEXT_NAMESPACE
