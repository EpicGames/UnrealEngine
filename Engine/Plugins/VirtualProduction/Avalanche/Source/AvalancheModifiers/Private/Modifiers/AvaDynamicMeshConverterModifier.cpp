// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaDynamicMeshConverterModifier.h"

#include "Async/Async.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Engine/StaticMesh.h"
#include "Extensions/ActorModifierSceneTreeUpdateExtension.h"
#include "GeometryScript/MeshAssetFunctions.h"
#include "GeometryScript/MeshBasicEditFunctions.h"

#if WITH_EDITOR
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "Dialogs/DlgPickAssetPath.h"
#endif

#define LOCTEXT_NAMESPACE "AvaDynamicMeshConverterModifier"

FAvaDynamicMeshConverterModifierComponentState::FAvaDynamicMeshConverterModifierComponentState(UPrimitiveComponent* InPrimitiveComponent)
	: Component(InPrimitiveComponent)
{
	if (Component.IsValid())
	{
		bComponentVisible = InPrimitiveComponent->GetVisibleFlag();
		bComponentHiddenInGame = InPrimitiveComponent->bHiddenInGame;

		for (int32 Index = 0; Index < InPrimitiveComponent->GetNumMaterials(); Index++)
		{
			ComponentMaterialsWeak.Add(InPrimitiveComponent->GetMaterial(Index));
		}
	}
}

void FAvaDynamicMeshConverterModifierComponentState::UpdateRelativeTransform(const FTransform& InParentTransform)
{
	if (const UPrimitiveComponent* PrimitiveComponent = Component.Get())
	{
		ActorRelativeTransform = PrimitiveComponent->GetComponentTransform().GetRelativeTransform(InParentTransform);
	}
}

void UAvaDynamicMeshConverterModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("DynamicMeshConverter"));
	InMetadata.SetCategory(TEXT("Conversion"));
	InMetadata.AllowTick(true);
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Dynamic Mesh Converter"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Converts various actor mesh types into a single dynamic mesh, this is a heavy operation"));
#endif
	InMetadata.SetCompatibilityRule([](const AActor* InActor)->bool
	{
		return InActor && !InActor->FindComponentByClass<UDynamicMeshComponent>();
	});
}

void UAvaDynamicMeshConverterModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

	AddDynamicMeshComponent();

	AddExtension<FActorModifierRenderStateUpdateExtension>(this);

	if (FActorModifierSceneTreeUpdateExtension* SceneExtension = AddExtension<FActorModifierSceneTreeUpdateExtension>(this))
	{
		TrackedActor.ReferenceContainer = EActorModifierReferenceContainer::Other;
		TrackedActor.ReferenceActorWeak = SourceActorWeak.Get();
		TrackedActor.bSkipHiddenActors = false;
		SceneExtension->TrackSceneTree(0, &TrackedActor);
	}
}

void UAvaDynamicMeshConverterModifier::OnModifierEnabled(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierEnabled(InReason);

	if (bComponentCreated)
	{
		if (UDynamicMeshComponent* PrimitiveComponent = GetMeshComponent())
		{
			PrimitiveComponent->SetVisibleFlag(true);
			PrimitiveComponent->SetHiddenInGame(false);
		}
	}
}

void UAvaDynamicMeshConverterModifier::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierDisabled(InReason);

	if (bComponentCreated)
	{
		if (UDynamicMeshComponent* PrimitiveComponent = GetMeshComponent())
		{
			PrimitiveComponent->SetVisibleFlag(false);
			PrimitiveComponent->SetHiddenInGame(true);
		}
	}
}

void UAvaDynamicMeshConverterModifier::RestorePreState()
{
	UAvaGeometryBaseModifier::RestorePreState();

	for (FAvaDynamicMeshConverterModifierComponentState& ConvertedComponent : ConvertedComponents)
	{
		if (UPrimitiveComponent* PrimitiveComponent = ConvertedComponent.Component.Get())
		{
			PrimitiveComponent->SetHiddenInGame(ConvertedComponent.bComponentHiddenInGame);
			PrimitiveComponent->SetVisibility(ConvertedComponent.bComponentVisible);
		}
	}
}

void UAvaDynamicMeshConverterModifier::OnModifierRemoved(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierRemoved(InReason);

	RemoveDynamicMeshComponent();
}

bool UAvaDynamicMeshConverterModifier::IsModifierDirtyable() const
{
	const double CurrentTime = FPlatformTime::Seconds();

	if (UpdateInterval > 0
		&& CurrentTime - LastTransformUpdateTime > UpdateInterval)
	{
		const_cast<UAvaDynamicMeshConverterModifier*>(this)->LastTransformUpdateTime = CurrentTime;

		for (const FAvaDynamicMeshConverterModifierComponentState& ConvertedComponent : ConvertedComponents)
		{
			UPrimitiveComponent* PrimitiveComponent = ConvertedComponent.Component.Get();
			if (!PrimitiveComponent)
			{
				continue;
			}

			const AActor* ChildActor = PrimitiveComponent->GetOwner();
			if (!ChildActor)
			{
				continue;
			}

			const UDynamicMeshComponent* DynamicMeshComponent = GetMeshComponent();
			if (!DynamicMeshComponent)
			{
				continue;
			}

			// Check Transform
			const FTransform ExpectedTransform = PrimitiveComponent->GetComponentTransform().GetRelativeTransform(DynamicMeshComponent->GetComponentTransform());
			if (!ConvertedComponent.ActorRelativeTransform.Equals(ExpectedTransform, 0.01))
			{
				return true;
			}

			// Check Materials
			FAvaDynamicMeshConverterModifierComponentState NewState(PrimitiveComponent);
			for (int32 Index = 0; Index < ConvertedComponent.ComponentMaterialsWeak.Num(); Index++)
			{
				if (!NewState.ComponentMaterialsWeak.IsValidIndex(Index)
					|| ConvertedComponent.ComponentMaterialsWeak[Index] != NewState.ComponentMaterialsWeak[Index])
				{
					return true;
				}
			}
		}
	}

	return Super::IsModifierDirtyable();
}

void UAvaDynamicMeshConverterModifier::OnSceneTreeTrackedActorChildrenChanged(int32 InIdx, const TSet<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TSet<TWeakObjectPtr<AActor>>& InNewChildrenActors)
{
	if (bIncludeAttachedActors)
	{
		MarkModifierDirty();
	}
}

void UAvaDynamicMeshConverterModifier::Apply()
{
	if (!IsMeshValid())
	{
		Fail(LOCTEXT("InvalidDynamicMeshComponent", "Invalid dynamic mesh component on modified actor"));
		return;
	}

	UDynamicMeshComponent* DynMeshComponent = GetMeshComponent();

	if (!IsValid(DynMeshComponent))
	{
		Fail(LOCTEXT("InvalidDynamicMeshComponent", "Invalid dynamic mesh component on modified actor"));
		return;
	}

	TArray<TWeakObjectPtr<UMaterialInterface>> MaterialsWeak;
	if (!ConvertComponents(MaterialsWeak))
	{
		Fail(LOCTEXT("ConversionFailed", "Conversion to dynamic mesh failed"));
		return;
	}

	for (int32 MatIndex = 0; MatIndex < MaterialsWeak.Num(); MatIndex++)
	{
		UMaterialInterface* Material = MaterialsWeak[MatIndex].Get();
		DynMeshComponent->SetMaterial(MatIndex, Material);
	}

	// Hide converted component
	if (bHideConvertedMesh)
	{
		for (const FAvaDynamicMeshConverterModifierComponentState& ConvertedComponent : ConvertedComponents)
		{
			if (UPrimitiveComponent* PrimitiveComponent = ConvertedComponent.Component.Get())
			{
				PrimitiveComponent->SetVisibility(false);
				PrimitiveComponent->SetHiddenInGame(true);
			}
		}
	}

	Next();
}

#if WITH_EDITOR
void UAvaDynamicMeshConverterModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	static const FName SourceActorName = GET_MEMBER_NAME_CHECKED(UAvaDynamicMeshConverterModifier, SourceActorWeak);

	if (MemberName == SourceActorName)
	{
		OnSourceActorChanged();
	}
}

void UAvaDynamicMeshConverterModifier::ConvertToStaticMeshAsset()
{
	using namespace UE::Geometry;

	UDynamicMeshComponent* DynMeshComponent = GetMeshComponent();
	const AActor* OwningActor = GetModifiedActor();

	if (!OwningActor || !DynMeshComponent)
	{
		return;
	}

	// generate name for asset
	const FString NewNameSuggestion = TEXT("SM_MotionDesign_") + OwningActor->GetActorNameOrLabel();
	FString PackageName = FString(TEXT("/Game/Meshes/")) + NewNameSuggestion;
	FString AssetName;

	const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(PackageName, TEXT(""), PackageName, AssetName);

	const TSharedPtr<SDlgPickAssetPath> PickAssetPathWidget =
		SNew(SDlgPickAssetPath)
		.Title(LOCTEXT("ConvertToStaticMeshPickName", "Choose New StaticMesh Location"))
		.DefaultAssetPath(FText::FromString(PackageName));

	if (PickAssetPathWidget->ShowModal() != EAppReturnType::Ok)
	{
		return;
	}

	// get input name provided by user
	FString UserPackageName = PickAssetPathWidget->GetFullAssetPath().ToString();
	FName MeshName(*FPackageName::GetLongPackageAssetName(UserPackageName));

	// is input name valid ?
	if (MeshName == NAME_None)
	{
		// Use default if invalid
		UserPackageName = PackageName;
		MeshName = *AssetName;
	}

	const FDynamicMesh3* MeshIn = DynMeshComponent->GetMesh();

	// empty mesh do not export
	if (!MeshIn || MeshIn->TriangleCount() == 0)
	{
		return;
	}

	// find/create package
	UPackage* Package = CreatePackage(*UserPackageName);
	check(Package);

	// Create StaticMesh object
	UStaticMesh* DestinationMesh = NewObject<UStaticMesh>(Package, MeshName, RF_Public | RF_Standalone);
	UDynamicMesh* SourceMesh = DynMeshComponent->GetDynamicMesh();

	// export options
	FGeometryScriptCopyMeshToAssetOptions AssetOptions;
	AssetOptions.bReplaceMaterials = false;
	AssetOptions.bEnableRecomputeNormals = false;
	AssetOptions.bEnableRecomputeTangents = false;
	AssetOptions.bEnableRemoveDegenerates = true;

	// LOD options
	FGeometryScriptMeshWriteLOD TargetLOD;
	TargetLOD.LODIndex = 0;

	EGeometryScriptOutcomePins OutResult;

	UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshToStaticMesh(SourceMesh, DestinationMesh, AssetOptions, TargetLOD, OutResult);
	DestinationMesh->GetBodySetup()->AggGeom = DynMeshComponent->GetBodySetup()->AggGeom;

	if (OutResult == EGeometryScriptOutcomePins::Success)
	{
		// Notify asset registry of new asset
		FAssetRegistryModule::AssetCreated(DestinationMesh);
	}
}
#endif

void UAvaDynamicMeshConverterModifier::SetSourceActorWeak(const TWeakObjectPtr<AActor>& InActor)
{
	if (InActor.Get() == SourceActorWeak.Get())
	{
		return;
	}

	SourceActorWeak = InActor;
	OnSourceActorChanged();
}

void UAvaDynamicMeshConverterModifier::SetComponentTypes(const TSet<EAvaDynamicMeshConverterModifierType>& InTypes)
{
	EAvaDynamicMeshConverterModifierType NewComponentType = EAvaDynamicMeshConverterModifierType::None;

	for (const EAvaDynamicMeshConverterModifierType Type : InTypes)
	{
		EnumAddFlags(NewComponentType, Type);
	}

	SetComponentType(static_cast<int32>(NewComponentType));
}

TSet<EAvaDynamicMeshConverterModifierType> UAvaDynamicMeshConverterModifier::GetComponentTypes() const
{
	TSet<EAvaDynamicMeshConverterModifierType> ComponentTypes
	{
		EAvaDynamicMeshConverterModifierType::StaticMeshComponent,
		EAvaDynamicMeshConverterModifierType::DynamicMeshComponent,
		EAvaDynamicMeshConverterModifierType::SkeletalMeshComponent,
		EAvaDynamicMeshConverterModifierType::BrushComponent,
		EAvaDynamicMeshConverterModifierType::ProceduralMeshComponent
	};

	for (TSet<EAvaDynamicMeshConverterModifierType>::TIterator It(ComponentTypes); It; ++It)
	{
		if (!HasFlag(*It))
		{
			It.RemoveCurrent();
		}
	}

	return ComponentTypes;
}

void UAvaDynamicMeshConverterModifier::SetComponentType(int32 InComponentType)
{
	if (ComponentType == InComponentType)
	{
		return;
	}

	ComponentType = InComponentType;
	MarkModifierDirty();
}

void UAvaDynamicMeshConverterModifier::SetFilterActorMode(EAvaDynamicMeshConverterModifierFilter InFilter)
{
	if (FilterActorMode == InFilter)
	{
		return;
	}

	FilterActorMode = InFilter;
	MarkModifierDirty();
}

void UAvaDynamicMeshConverterModifier::SetFilterActorClasses(const TSet<TSubclassOf<AActor>>& InClasses)
{
	if (FilterActorClasses.Includes(InClasses) && FilterActorClasses.Num() == InClasses.Num())
	{
		return;
	}

	FilterActorClasses = InClasses;
	MarkModifierDirty();
}

void UAvaDynamicMeshConverterModifier::SetIncludeAttachedActors(bool bInInclude)
{
	if (bIncludeAttachedActors == bInInclude)
	{
		return;
	}

	bIncludeAttachedActors = bInInclude;
	MarkModifierDirty();
}

void UAvaDynamicMeshConverterModifier::SetHideConvertedMesh(bool bInHide)
{
	if (bHideConvertedMesh == bInHide)
	{
		return;
	}

	bHideConvertedMesh = bInHide;
	MarkModifierDirty();
}

void UAvaDynamicMeshConverterModifier::SetUpdateInterval(float InInterval)
{
	InInterval = FMath::Max(0.f, InInterval);
	if (FMath::IsNearlyEqual(UpdateInterval, InInterval))
	{
		return;
	}

	UpdateInterval = InInterval;
	MarkModifierDirty();
}

void UAvaDynamicMeshConverterModifier::OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent)
{
	if (!IsValid(InActor) || !IsValid(InComponent))
	{
		return;
	}

	const UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(InComponent);
	if (!PrimitiveComponent)
	{
		return;
	}

	const UDynamicMeshComponent* DynamicMeshComponent = GetMeshComponent();
	if (PrimitiveComponent == DynamicMeshComponent)
	{
		return;
	}

	const AActor* SourceActor = SourceActorWeak.Get();
	if (!SourceActor)
	{
		return;
	}

	const bool bIsSourceActor = InActor == SourceActor;
	const bool bIsAttachedToSourceActor = bIncludeAttachedActors && InActor->IsAttachedTo(SourceActor);
	if (!bIsSourceActor && !bIsAttachedToSourceActor)
	{
		return;
	}

	MarkModifierDirty();
}

void UAvaDynamicMeshConverterModifier::OnSourceActorChanged()
{
	AActor* SourceActor = SourceActorWeak.Get();
	const AActor* ActorModified = GetModifiedActor();

	if (!SourceActor || !ActorModified)
	{
		return;
	}

	bHideConvertedMesh = SourceActor == ActorModified || SourceActor->IsAttachedTo(ActorModified);

	if (const FActorModifierSceneTreeUpdateExtension* SceneExtension = GetExtension<FActorModifierSceneTreeUpdateExtension>())
	{
		TrackedActor.ReferenceActorWeak = SourceActor;
		SceneExtension->CheckTrackedActorUpdate(0);
	}
}

bool UAvaDynamicMeshConverterModifier::ConvertComponents(TArray<TWeakObjectPtr<UMaterialInterface>>& OutMaterialsWeak)
{
	if (!IsMeshValid() || !SourceActorWeak.IsValid())
	{
		return false;
	}

	TSet<FAvaDynamicMeshConverterModifierComponentState> NewConvertedComponents;
	NewConvertedComponents.Reserve(ConvertedComponents.Num());

	MeshBuilder.Reset();

	UDynamicMeshComponent* DynamicMeshComponent = GetMeshComponent();
	const FTransform SourceTransform = DynamicMeshComponent->GetComponentTransform();

	// Get relevant actors
	TArray<AActor*> FilteredActors;
	GetFilteredActors(FilteredActors);

	auto FindOrAddComponentState = [this, &SourceTransform, &NewConvertedComponents](UPrimitiveComponent* InComponent)
	{
		FAvaDynamicMeshConverterModifierComponentState NewState(InComponent);
		if (FAvaDynamicMeshConverterModifierComponentState* OldState = ConvertedComponents.Find(NewState))
		{
			OldState->UpdateRelativeTransform(SourceTransform);
			NewConvertedComponents.Emplace(*OldState);
		}
		else
		{
			NewState.UpdateRelativeTransform(SourceTransform);
			NewConvertedComponents.Emplace(NewState);
		}
	};

	ECEMeshBuilderComponentType ComponentTypes = ECEMeshBuilderComponentType::None;

	if (HasFlag(EAvaDynamicMeshConverterModifierType::StaticMeshComponent))
	{
		EnumAddFlags(ComponentTypes, ECEMeshBuilderComponentType::StaticMeshComponent);
	}

	if (HasFlag(EAvaDynamicMeshConverterModifierType::DynamicMeshComponent))
	{
		EnumAddFlags(ComponentTypes, ECEMeshBuilderComponentType::DynamicMeshComponent);
	}

	if (HasFlag(EAvaDynamicMeshConverterModifierType::SkeletalMeshComponent))
	{
		EnumAddFlags(ComponentTypes, ECEMeshBuilderComponentType::SkeletalMeshComponent);
	}

	if (HasFlag(EAvaDynamicMeshConverterModifierType::BrushComponent))
	{
		EnumAddFlags(ComponentTypes, ECEMeshBuilderComponentType::BrushComponent);
	}

	if (HasFlag(EAvaDynamicMeshConverterModifierType::ProceduralMeshComponent))
	{
		EnumAddFlags(ComponentTypes, ECEMeshBuilderComponentType::ProceduralMeshComponent);
	}

	FCEMeshBuilder::FCEMeshBuilderAppendParams AppendParams;
	AppendParams.ComponentTypes = ComponentTypes;
	AppendParams.ExcludeComponents.Add(GetMeshComponent());

	for (const AActor* FilteredActor : FilteredActors)
	{
		for (UPrimitiveComponent* PrimitiveComponent : MeshBuilder.AppendActor(FilteredActor, SourceTransform, AppendParams))
		{
			FindOrAddComponentState(PrimitiveComponent);
		}
	}

	ConvertedComponents = MoveTemp(NewConvertedComponents);

	FCEMeshBuilder::FCEMeshBuilderParams BuilderParams;
	BuilderParams.bNotifyMeshChangeEvents = true;
	return MeshBuilder.BuildDynamicMesh(DynamicMeshComponent->GetDynamicMesh(), OutMaterialsWeak, BuilderParams);
}

bool UAvaDynamicMeshConverterModifier::HasFlag(EAvaDynamicMeshConverterModifierType InFlag) const
{
	return EnumHasAnyFlags(static_cast<EAvaDynamicMeshConverterModifierType>(ComponentType), InFlag);
}

void UAvaDynamicMeshConverterModifier::AddDynamicMeshComponent()
{
	UDynamicMeshComponent* DynMeshComponent = GetMeshComponent();

	if (DynMeshComponent)
	{
		return;
	}

	AActor* ActorModified = GetModifiedActor();

	if (!IsValid(ActorModified))
	{
		return;
	}

#if WITH_EDITOR
	ActorModified->Modify();
	Modify();
#endif

	const UClass* const NewComponentClass = UDynamicMeshComponent::StaticClass();

	// Construct the new component and attach as needed
	DynMeshComponent = NewObject<UDynamicMeshComponent>(ActorModified
		, NewComponentClass
		, MakeUniqueObjectName(ActorModified, NewComponentClass, TEXT("DynamicMeshComponent"))
		, RF_Transactional);

	// Add to SerializedComponents array so it gets saved
	ActorModified->AddInstanceComponent(DynMeshComponent);
	DynMeshComponent->OnComponentCreated();
	DynMeshComponent->RegisterComponent();

	if (USceneComponent* RootComponent = ActorModified->GetRootComponent())
	{
		static const FAttachmentTransformRules AttachRules(EAttachmentRule::SnapToTarget, EAttachmentRule::SnapToTarget, EAttachmentRule::SnapToTarget, false);
		DynMeshComponent->AttachToComponent(RootComponent, AttachRules);
	}
	else
	{
		ActorModified->SetRootComponent(DynMeshComponent);
	}

	DynMeshComponent->SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
	DynMeshComponent->SetGenerateOverlapEvents(true);

#if WITH_EDITOR
	// Rerun construction scripts
	ActorModified->RerunConstructionScripts();
#endif

	bComponentCreated = true;
}

void UAvaDynamicMeshConverterModifier::RemoveDynamicMeshComponent()
{
	UDynamicMeshComponent* DynMeshComponent = GetMeshComponent();

	if (!DynMeshComponent)
	{
		return;
	}

	// Did we create the component or was it already there
	if (!bComponentCreated)
	{
		return;
	}

	AActor* ActorModified = GetModifiedActor();

	if (!IsValid(ActorModified))
	{
		return;
	}

#if WITH_EDITOR
	ActorModified->Modify();
	Modify();
#endif

	const FDetachmentTransformRules DetachRules(EDetachmentRule::KeepWorld, false);
	DynMeshComponent->DetachFromComponent(DetachRules);

	ActorModified->RemoveInstanceComponent(DynMeshComponent);
	DynMeshComponent->DestroyComponent(false);

	bComponentCreated = false;
}

void UAvaDynamicMeshConverterModifier::GetFilteredActors(TArray<AActor*>& OutActors) const
{
	if (AActor* OriginActor = SourceActorWeak.Get())
	{
		OutActors.Add(OriginActor);
		if (bIncludeAttachedActors)
		{
			OriginActor->GetAttachedActors(OutActors, false, true);
		}
		// Filter actor class
		if (FilterActorMode != EAvaDynamicMeshConverterModifierFilter::None)
		{
			for (int32 Idx = OutActors.Num() - 1; Idx >= 0; Idx--)
			{
				const AActor* CurrentActor = OutActors[Idx];
				if (!IsValid(CurrentActor))
				{
					continue;
				}
				// Include this actor if it's in the filter class
				if (FilterActorMode == EAvaDynamicMeshConverterModifierFilter::Include)
				{
					if (!FilterActorClasses.Contains(CurrentActor->GetClass()))
					{
						OutActors.RemoveAt(Idx);
					}
				}
				else // Exclude this actor if it's in the filter class
				{
					if (FilterActorClasses.Contains(CurrentActor->GetClass()))
					{
						OutActors.RemoveAt(Idx);
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
