// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaExtrudeModifier.h"

#include "Async/Async.h"
#include "AvaShapeActor.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMeshEditor.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "GeometryScript/MeshModelingFunctions.h"
#include "Operations/MeshSelfUnion.h"
#include "Operations/OffsetMeshRegion.h"
#include "UDynamicMesh.h"

#define LOCTEXT_NAMESPACE "AvaExtrudeModifier"

struct FAvaExtrudeModifierVersion
{
	enum Type : int32
	{
		PreVersioning = 0,

		/** Added ExtrudeMode enum to choose among multiple extrude modes */
		ExtrudeMode,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	static constexpr FGuid GUID = FGuid(0x9271D8A5, 0xBF414602, 0xA20FC0A4, 0x9D829563);
};

FCustomVersionRegistration GRegisterAvaExtrudeModifierVersion(FAvaExtrudeModifierVersion::GUID, static_cast<int32>(FAvaExtrudeModifierVersion::LatestVersion), TEXT("AvaExtrudeModifierVersion"));

void UAvaExtrudeModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("Extrude"));
	InMetadata.SetCategory(TEXT("Geometry"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Extrude"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Duplicates and moves points and triangles following their normal direction"));
#endif

	InMetadata.DisallowBefore(TEXT("Outline"));
	InMetadata.SetCompatibilityRule([](const AActor* InActor)->bool
	{
		bool bSupported = false;
		if(InActor)
		{
			if (const UDynamicMeshComponent* DynMeshComponent = InActor->FindComponentByClass<UDynamicMeshComponent>())
			{
				DynMeshComponent->ProcessMesh([&bSupported](const FDynamicMesh3& ProcessMesh)
				{
					bSupported = ProcessMesh.VertexCount() > 0 && static_cast<FBox>(ProcessMesh.GetBounds(true)).GetSize().X == 0;
				});
			}
		}
		return bSupported;
	});
}

void UAvaExtrudeModifier::Apply()
{
	const float ExtrudeDistance = FMath::Abs(Depth);
	
	if (ExtrudeDistance <= 0)
	{
		Next();
		return;
	}

	UDynamicMeshComponent* const DynMeshComp = GetMeshComponent();

	if (!IsValid(DynMeshComp))
	{
		Fail(LOCTEXT("InvalidDynamicMeshComponent", "Invalid dynamic mesh component on modified actor"));
		return;
	}
	
	const float Inset = Depth < 0 ? -1 : 1;

	FGeometryScriptMeshExtrudeOptions Options;
	Options.ExtrudeDirection = GetExtrudeDirection() * Inset;
	Options.ExtrudeDistance = ExtrudeDistance;
	Options.bSolidsToShells = false;
	Options.UVScale = 1;
	
	using namespace UE::Geometry;
	DynMeshComp->GetDynamicMesh()->EditMesh([this, Options](FDynamicMesh3& EditMesh)
	{
		FDynamicMeshUVOverlay* const UVOverlay = EditMesh.Attributes()->PrimaryUV();
		FDynamicMeshMaterialAttribute* const MaterialAttr = EditMesh.Attributes()->GetMaterialID();
		
		if (EditMesh.TriangleCount() > 0)
		{
			// weld edges to avoid end caps when we extrude geometry
			FMergeCoincidentMeshEdges WeldOp(&EditMesh);
			WeldOp.Apply();
		}
		
		TSet<int32> OriginalTriangleIds;
		for (int32 TId : EditMesh.TriangleIndicesItr())
		{
			OriginalTriangleIds.Add(TId);
		}

		// extrude options
		const FVector3d ExtrudeVec = static_cast<double>(Options.ExtrudeDistance) * Options.ExtrudeDirection;
		FOffsetMeshRegion Extruder(&EditMesh);
		Extruder.Triangles = OriginalTriangleIds.Array();
		Extruder.OffsetPositionFunc = [ExtrudeVec](const FVector3d& Position, const FVector3d& VertexVector, int VertexID)
		{
			return Position - ExtrudeVec;
		};
		Extruder.bIsPositiveOffset = Options.ExtrudeDistance > 0;
		Extruder.UVScaleFactor = Options.UVScale;
		Extruder.bOffsetFullComponentsAsSolids = Options.bSolidsToShells;
		Extruder.Apply();

		// get new UV ids to scale them properly
		{
			TSet<int32> ExtrudeUVIds;
			TArray<int32> ExtrudeTIds;
			for (int32 TId : Extruder.AllModifiedAndNewTriangles)
			{
				// set correct triangle id based on extruded section
				if (!OriginalTriangleIds.Contains(TId))
				{
					MaterialAttr->SetValue(TId, 0);
					// retrieve new uv indexes to scale them later
					FIndex3i UVIndexes = UVOverlay->GetTriangle(TId);
					ExtrudeUVIds.Add(UVIndexes.A);
					ExtrudeUVIds.Add(UVIndexes.B);
					ExtrudeUVIds.Add(UVIndexes.C);
					// retrieve new triangle
					ExtrudeTIds.Add(TId);
				}
			}
			
			// Get polygroup layer for extrude side
			FDynamicMeshPolygroupAttribute* const ExtrudePolygroup = FindOrCreatePolygroupLayer(EditMesh, UAvaExtrudeModifier::ExtrudePolygroupLayerName, &ExtrudeTIds);
			
			// Apply uv transform if avalanche shape with options
			if (AAvaShapeActor* ShapeActor = Cast<AAvaShapeActor>(GetModifiedActor()))
			{
				if (UAvaShapeDynamicMeshBase* ShapeComponent = ShapeActor->GetDynamicMesh())
				{
					const FVector UVSize = GetMeshBounds().GetSize() / EditMesh.GetBounds().MaxDim();
					FAvaShapeMaterialUVParameters& UVParams = *ShapeComponent->GetInUseMaterialUVParams(UAvaShapeDynamicMeshBase::MESH_INDEX_PRIMARY);
					UE::AvaShapes::TransformMeshUVs(EditMesh, ExtrudeUVIds.Array(), UVParams, FVector2D(UVSize.Y, UVSize.Z), FVector2D(0.5, 0.5), 90.f);
				}
			}
		}

		// Offset mesh
		FVector3d MeshOffset = FVector3d::ZeroVector;
		switch (ExtrudeMode)
		{
			case EAvaExtrudeMode::Opposite:
			MeshOffset = ExtrudeVec;
			break;

		case EAvaExtrudeMode::Symmetrical:
			MeshOffset = ExtrudeVec * 0.5f;
			break;

		case EAvaExtrudeMode::Front:
			break;
		default: ;
		}

		// move mesh backwards if needed
		if (ExtrudeMode != EAvaExtrudeMode::Front)
		{
			// move mesh to correct location
			for (int32 VId : EditMesh.VertexIndicesItr())
			{
				FVector3d Loc = EditMesh.GetVertex(VId);
				Loc += MeshOffset;
				EditMesh.SetVertex(VId, Loc);
			}
		}

		// close back
		if (bCloseBack)
		{
			// mirror original mesh to create back
			FMeshIndexMappings TmpMappings;
			FDynamicMeshEditor Editor(&EditMesh);
			FDynamicMesh3 NewBack = PreModifierCachedMesh.GetValue();
			FTransform BackTransform(FRotator(0, 180, 0), MeshOffset, FVector(1, -1, 1));
			MeshTransforms::ApplyTransform(NewBack, BackTransform, true);
			Editor.AppendMesh(&NewBack, TmpMappings);

			// Get polygroup layer for back side
			TArray<int32> OutBackTriangles;
			TmpMappings.GetTriangleMap().GetForwardMap().GenerateValueArray(OutBackTriangles);
			FDynamicMeshPolygroupAttribute* const BackPolygroup = FindOrCreatePolygroupLayer(EditMesh, UAvaExtrudeModifier::BackPolygroupLayerName, &OutBackTriangles);
		}

		// Remove geometry self intersections
		{
			FMeshSelfUnion Union(&EditMesh);
			Union.bCollapseDegenerateEdgesOnCut = true;
			Union.bTrimFlaps = false;
			Union.Compute();
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown);

	Next();
}

void UAvaExtrudeModifier::SetDepth(float InDepth)
{
	InDepth = FMath::Max(0, InDepth);

	if (FMath::IsNearlyEqual(Depth, InDepth))
	{
		return;
	}

	Depth = InDepth;
	OnDepthChanged();
}

void UAvaExtrudeModifier::OnDepthChanged()
{
	MarkModifierDirty();
}

void UAvaExtrudeModifier::SetCloseBack(bool bInCloseBack)
{
	if (bCloseBack == bInCloseBack)
	{
		return;
	}

	bCloseBack = bInCloseBack;
	OnCloseBackChanged();
}

void UAvaExtrudeModifier::OnCloseBackChanged()
{
	MarkModifierDirty();
}

void UAvaExtrudeModifier::SetExtrudeMode(EAvaExtrudeMode InExtrudeMode)
{
	if (ExtrudeMode == InExtrudeMode)
	{
		return;
	}

	ExtrudeMode = InExtrudeMode;
	OnExtrudeModeChanged();
}

void UAvaExtrudeModifier::OnExtrudeModeChanged()
{
	MarkModifierDirty();
}

FVector UAvaExtrudeModifier::GetExtrudeDirection() const
{
	const FVector MeshSize = GetMeshBounds().GetSize();
	if (MeshSize.X <= UE_KINDA_SMALL_NUMBER)
	{
		return FVector::XAxisVector;
	}
	if (MeshSize.Y <= UE_KINDA_SMALL_NUMBER)
	{
		return FVector::YAxisVector;
	}
	if (MeshSize.Z <= UE_KINDA_SMALL_NUMBER)
	{
		return FVector::ZAxisVector;
	}
	return FVector::ZeroVector;
}

void UAvaExtrudeModifier::Serialize(FArchive& InArchive)
{
	InArchive.UsingCustomVersion(FAvaExtrudeModifierVersion::GUID);

	Super::Serialize(InArchive);

	const int32 Version = InArchive.CustomVer(FAvaExtrudeModifierVersion::GUID);

	if (Version < FAvaExtrudeModifierVersion::ExtrudeMode)
	{
		if (bMoveMeshOppositeDirection_DEPRECATED)
		{
			ExtrudeMode = EAvaExtrudeMode::Opposite;
		}
		else
		{
			ExtrudeMode = EAvaExtrudeMode::Front;
		}
	}
}

#if WITH_EDITOR
void UAvaExtrudeModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();
	
	static const FName DepthName = GET_MEMBER_NAME_CHECKED(UAvaExtrudeModifier, Depth);
	static const FName CloseBackName = GET_MEMBER_NAME_CHECKED(UAvaExtrudeModifier, bCloseBack);
	static const FName ExtrudeModeName = GET_MEMBER_NAME_CHECKED(UAvaExtrudeModifier, ExtrudeMode);
	
	if (MemberName == DepthName)
	{
		OnDepthChanged();
	}
	else if (MemberName == CloseBackName)
	{
		OnCloseBackChanged();
	}
	else if (MemberName == ExtrudeModeName)
	{
		OnExtrudeModeChanged();
	}
}
#endif

#undef LOCTEXT_NAMESPACE
