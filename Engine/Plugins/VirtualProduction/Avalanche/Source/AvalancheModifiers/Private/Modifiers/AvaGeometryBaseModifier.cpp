// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaGeometryBaseModifier.h"

#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Polygroups/PolygroupUtil.h"
#include "Profilers/AvaGeometryModifierProfiler.h"

#define LOCTEXT_NAMESPACE "AvaGeometryBaseModifier"

void UAvaGeometryBaseModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetProfilerClass<FAvaGeometryModifierProfiler>();
	InMetadata.SetCompatibilityRule([](const AActor* InActor)->bool
	{
		return InActor && InActor->FindComponentByClass<UDynamicMeshComponent>();
	});

	// Avoid geometry modifiers after translucent or layout modifiers
	InMetadata.AvoidAfterCategory(TEXT("Translucent"));
	InMetadata.AvoidAfterCategory(TEXT("Layout"));
	InMetadata.AvoidAfterCategory(TEXT("Rendering"));
}

bool UAvaGeometryBaseModifier::IsModifierReady() const
{
	if (!GetMeshComponent())
	{
		return false;
	}

	return true;
}

void UAvaGeometryBaseModifier::SavePreState()
{
	if (const UDynamicMeshComponent* const DynMeshComp = GetMeshComponent())
	{
		DynMeshComp->ProcessMesh([this](const FDynamicMesh3& ProcessMesh)
		{
			PreModifierCachedMesh = ProcessMesh;
			PreModifierCachedBounds = static_cast<FBox>(ProcessMesh.GetBounds(true));
		});
	}
}

void UAvaGeometryBaseModifier::RestorePreState()
{
	if (PreModifierCachedMesh.IsSet() && IsMeshValid())
	{
		if (UDynamicMeshComponent* const DynMeshComp = GetMeshComponent())
		{
			DynMeshComp->GetDynamicMesh()->SetMesh(PreModifierCachedMesh.GetValue());
		}
	}
}

void UAvaGeometryBaseModifier::OnModifierEnabled(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierEnabled(InReason);

	if (UDynamicMeshComponent* const DynMeshComp = GetMeshComponent())
	{
		DynMeshComp->OnMeshChanged.AddUObject(this, &UAvaGeometryBaseModifier::OnMeshChanged);
	}
}

void UAvaGeometryBaseModifier::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierDisabled(InReason);

	if (UDynamicMeshComponent* const DynMeshComp = GetMeshComponent())
	{
		DynMeshComp->OnMeshChanged.RemoveAll(this);
	}
}

void UAvaGeometryBaseModifier::OnMeshChanged()
{
	if (const UActorModifierCoreStack* RootStack = GetRootModifierStack())
	{
		if (RootStack->IsModifierIdle()
			&& IsModifierEnabled()
			&& RootStack->IsModifierInitialized()
			&& !RootStack->IsModifierExecutionLocked())
		{
			PreModifierCachedMesh.Reset();
			MarkModifierDirty();
		}
	}
}

UDynamicMeshComponent* UAvaGeometryBaseModifier::GetMeshComponent() const
{
	if (!MeshComponent.IsValid())
	{
		if (const AActor* ActorModified = GetModifiedActor())
		{
			if (UDynamicMeshComponent* FoundComponent = ActorModified->FindComponentByClass<UDynamicMeshComponent>())
			{
				UAvaGeometryBaseModifier* const MutableThis = const_cast<UAvaGeometryBaseModifier*>(this);
				MutableThis->MeshComponent = FoundComponent;
			}
		}
	}
	return MeshComponent.Get();
}

UDynamicMesh* UAvaGeometryBaseModifier::GetMeshObject() const
{
	if (UDynamicMeshComponent* DynamicMeshComponent = GetMeshComponent())
	{
		return DynamicMeshComponent->GetDynamicMesh();
	}

	return nullptr;
}

bool UAvaGeometryBaseModifier::IsMeshValid() const
{
	return IsValid(GetModifiedActor()) && IsValid(GetMeshComponent());
}

FBox UAvaGeometryBaseModifier::GetMeshBounds() const
{
	FBox MeshBounds;
	if (IsMeshValid())
	{
		GetMeshComponent()->ProcessMesh([&MeshBounds](const FDynamicMesh3& ProcessMesh)
		{
			MeshBounds = static_cast<FBox>(ProcessMesh.GetBounds(true));
		});
	}
	return MeshBounds;
}

UE::Geometry::FDynamicMeshPolygroupAttribute* UAvaGeometryBaseModifier::FindOrCreatePolygroupLayer(FDynamicMesh3& EditMesh, const FName& InLayerName, TArray<int32>* GroupTriangles)
{
	UE::Geometry::FDynamicMeshPolygroupAttribute* Polygroup = nullptr;
	int32 LayerIdx = UE::Geometry::FindPolygroupLayerIndexByName(EditMesh, InLayerName);
	if (LayerIdx == INDEX_NONE)
	{
		const int32 NewLayerCount = EditMesh.Attributes()->NumPolygroupLayers() + 1;
		EditMesh.Attributes()->SetNumPolygroupLayers(NewLayerCount);
		LayerIdx = NewLayerCount - 1;
		Polygroup = EditMesh.Attributes()->GetPolygroupLayer(LayerIdx);
		Polygroup->SetName(InLayerName);
	}
	else
	{
		Polygroup = EditMesh.Attributes()->GetPolygroupLayer(LayerIdx);
	}
	if (GroupTriangles)
	{
		for (const int32 TId : *GroupTriangles)
		{
			if (EditMesh.IsTriangle(TId))
			{
				Polygroup->SetValue(TId, LayerIdx);
			}
		}
	}
	return Polygroup;
}

#undef LOCTEXT_NAMESPACE
