// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaGlobalOpacityModifier.h"

#include "AvaModifiersLog.h"
#include "Components/DMMaterialValue.h"
#include "Components/MaterialValues/DMMaterialValueFloat1.h"
#include "Material/DynamicMaterialInstance.h"
#include "Materials/Material.h"
#include "Model/DynamicMaterialModel.h"
#include "Modifiers/Utilities/ActorModifierCoreUtilities.h"

#define LOCTEXT_NAMESPACE "AvaGlobalOpacityModifier"

UAvaGlobalOpacityModifier::UAvaGlobalOpacityModifier()
{
#if WITH_EDITOR
	bShowMaterialParameters = false;
#endif

	FAvaMaterialParameterMapScalar* ScalarParameter = MaterialParameters.FindScalarParameter(
		UDynamicMaterialModel::GlobalOpacityParameterName, 
		/* Create if needed */ true
	);
	
	if (ScalarParameter)
	{
		ScalarParameter->Value = GlobalOpacity;
	}
}

#if WITH_EDITOR
void UAvaGlobalOpacityModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	static const FName GlobalOpacityName = GET_MEMBER_NAME_CHECKED(UAvaGlobalOpacityModifier, GlobalOpacity);

	if (MemberName == GlobalOpacityName)
	{
		OnGlobalOpacityChanged();
	}
}
#endif

void UAvaGlobalOpacityModifier::SetGlobalOpacity(float InOpacity)
{
	if (FMath::IsNearlyEqual(GlobalOpacity, InOpacity))
	{
		return;
	}

	GlobalOpacity = InOpacity;
	OnGlobalOpacityChanged();
}

void UAvaGlobalOpacityModifier::OnGlobalOpacityChanged()
{
	GlobalOpacity = FMath::Clamp<float>(GlobalOpacity, UE_SMALL_NUMBER * 2, 1.f);

	FAvaMaterialParameterMapScalar* ScalarParameter = MaterialParameters.FindScalarParameter(
		UDynamicMaterialModel::GlobalOpacityParameterName,
		/* Create if needed */ true
	);

	if (ScalarParameter)
	{
		ScalarParameter->Value = GlobalOpacity;
		OnMaterialParametersChanged();
	}
	else
	{
		UE_LOGF(LogAvaModifier, Warning, "[%ls][%ls][%ls] Unable to find or add the global opacity material parameter"
			, *UE::ActorModifierCore::Utilities::GetActorNameSafe(GetModifiedActor())
			, *GetNameSafe(GetClass())
			, *GetModifierName().ToString());
	}
}

void UAvaGlobalOpacityModifier::OnActorMaterialAdded(UMaterialInstanceDynamic* InAdded)
{
	Super::OnActorMaterialAdded(InAdded);

	if (IsValid(InAdded))
	{
		if (const UMaterial* Material = InAdded->GetBaseMaterial())
		{
			if (Material->BlendMode == BLEND_Masked || Material->BlendMode == BLEND_Translucent)
			{
				++SupportedOpacityMaterialCount;
			}
		}
	}

#if WITH_EDITOR
	if (const UDynamicMaterialInstance* MDI = Cast<UDynamicMaterialInstance>(InAdded))
	{
		if (const UDynamicMaterialModel* Model = MDI->GetMaterialModel())
		{
			if (UDMMaterialValue* GlobalOpacityValue = Model->GetGlobalParameterValue(UDynamicMaterialModel::GlobalOpacityParameterName))
			{
				GlobalOpacityValue->GetOnUpdate().RemoveAll(this);
				GlobalOpacityValue->GetOnUpdate().AddUObject(this, &UAvaGlobalOpacityModifier::OnDynamicMaterialValueChanged);
			}
		}
	}
#endif
}

void UAvaGlobalOpacityModifier::OnActorMaterialRemoved(UMaterialInstanceDynamic* InRemoved)
{
	Super::OnActorMaterialRemoved(InRemoved);
	
	if (IsValid(InRemoved))
	{
		if (const UMaterial* Material = InRemoved->GetBaseMaterial())
		{
			if (Material->BlendMode == BLEND_Masked || Material->BlendMode == BLEND_Translucent)
			{
				--SupportedOpacityMaterialCount;
			}
		}
	}

#if WITH_EDITOR
	if (const UDynamicMaterialInstance* MDI = Cast<UDynamicMaterialInstance>(InRemoved))
	{
		if (const UDynamicMaterialModel* Model = MDI->GetMaterialModel())
		{
			if (UDMMaterialValue* GlobalOpacityValue = Model->GetGlobalParameterValue(UDynamicMaterialModel::GlobalOpacityParameterName))
			{
				GlobalOpacityValue->GetOnUpdate().RemoveAll(this);
			}
		}
	}
#endif
}

void UAvaGlobalOpacityModifier::OnDynamicMaterialValueChanged(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType)
{
	if (!EnumHasAnyFlags(InUpdateType, EDMUpdateType::Value))
	{
		return;
	}

	if (const UDMMaterialValueFloat1* FloatValue = Cast<UDMMaterialValueFloat1>(InComponent))
	{
		if (!FMath::IsNearlyEqual(FloatValue->GetValue(), GlobalOpacity))
		{
			MarkModifierDirty();
		}
	}
}

void UAvaGlobalOpacityModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("GlobalOpacity"));
	InMetadata.SetCategory(TEXT("Rendering"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Global Opacity"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Sets global opacity parameters on an actor with Material Designer Instances generated with the Material Designer"));
#endif
}

void UAvaGlobalOpacityModifier::Apply()
{
	if (SupportedOpacityMaterialCount <= 0)
	{
		Fail(LOCTEXT("NoOpacityMaterialFound", "No Supported Opacity Material Found"));
		return;
	}

	Super::Apply();
}

#undef LOCTEXT_NAMESPACE
