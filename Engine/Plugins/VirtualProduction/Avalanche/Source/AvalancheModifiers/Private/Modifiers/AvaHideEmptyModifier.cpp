// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaHideEmptyModifier.h"

#include "Shared/ActorModifierVisibilityShared.h"
#include "Text3DComponent.h"

#define LOCTEXT_NAMESPACE "AvaHideEmptyModifier"

void UAvaHideEmptyModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("HideEmpty"));
	InMetadata.SetCategory(TEXT("Rendering"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Hide Empty"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Hides a container when the text content is empty"));
#endif

	InMetadata.SetCompatibilityRule([](const AActor* InActor)->bool
	{
		return InActor && InActor->FindComponentByClass<UText3DComponent>();
	});
}

void UAvaHideEmptyModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

	if (InReason == EActorModifierCoreEnableReason::User)
	{
		if (AActor* ActorModified = GetModifiedActor())
		{
			ReferenceActor.ReferenceContainer = EActorModifierReferenceContainer::Other;
			ReferenceActor.ReferenceActorWeak = ActorModified;
			ReferenceActor.bSkipHiddenActors = false;

			TextComponent = ActorModified->FindComponentByClass<UText3DComponent>();
		}
	}

	OnContainerActorChanged();
}

void UAvaHideEmptyModifier::OnModifierEnabled(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierEnabled(InReason);

	if (UText3DComponent* Text3DComponent = TextComponent.Get())
	{
		Text3DComponent->OnTextGenerated().RemoveAll(this);
		Text3DComponent->OnTextGenerated().AddUObject(this, &UAvaHideEmptyModifier::OnTextChanged);

		OnContainerActorChanged();
	}
}

void UAvaHideEmptyModifier::Apply()
{
	const UText3DComponent* Text3DComponent = TextComponent.Get();
	if (!Text3DComponent)
	{
		Next();
		return;
	}

	AActor* ActorContainer = ReferenceActor.ReferenceActorWeak.Get();
	if (!ActorContainer || !ActorContainer->GetRootComponent())
	{
		Next();
		return;
	}

	UActorModifierVisibilityShared* VisibilityShared = GetShared<UActorModifierVisibilityShared>(true);
	if (!VisibilityShared)
	{
		Fail(LOCTEXT("InvalidSharedObject", "Invalid modifier shared object retrieved"));
		return;
	}

	if (Text3DComponent->GetText().IsEmpty())
	{
		const bool bNewVisibility = bInvertVisibility;
		VisibilityShared->SetActorVisibility(this, ActorContainer, !bNewVisibility, true);
	}
	else
	{
		const bool bNewVisibility = !bInvertVisibility;
		VisibilityShared->SetActorVisibility(this, ActorContainer, !bNewVisibility, true);
	}

	TSet<TWeakObjectPtr<AActor>> NewChildrenActorsWeak;
	{
		TArray<AActor*> AttachedActors;
		ActorContainer->GetAttachedActors(AttachedActors, false, true);
		Algo::Transform(AttachedActors, NewChildrenActorsWeak, [](AActor* InActor) { return TWeakObjectPtr<AActor>(InActor); });
	}

	VisibilityShared->RestoreActorsState(this, ChildrenActorsWeak.Difference(NewChildrenActorsWeak));
	ChildrenActorsWeak = NewChildrenActorsWeak;

	Next();
}

void UAvaHideEmptyModifier::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierDisabled(InReason);

	if (UText3DComponent* Text3DComponent = TextComponent.Get())
	{
		Text3DComponent->OnTextGenerated().RemoveAll(this);
	}
}

void UAvaHideEmptyModifier::OnModifiedActorTransformed()
{
	// Overwrite parent behaviour don't do anything when moved
}

#if WITH_EDITOR
void UAvaHideEmptyModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	static const FName ContainerActorName = GET_MEMBER_NAME_CHECKED(UAvaHideEmptyModifier, ContainerActorWeak);
	static const FName InvertVisibilityName = GET_MEMBER_NAME_CHECKED(UAvaHideEmptyModifier, bInvertVisibility);

	if (MemberName == ContainerActorName)
	{
		OnContainerActorChanged();
	}
	else if (MemberName == InvertVisibilityName)
	{
		OnInvertVisibilityChanged();
	}
}
#endif

void UAvaHideEmptyModifier::SetContainerActorWeak(TWeakObjectPtr<AActor> InContainer)
{
	if (ContainerActorWeak == InContainer)
	{
		return;
	}

	ContainerActorWeak = InContainer;
	OnContainerActorChanged();
}

void UAvaHideEmptyModifier::SetInvertVisibility(bool bInInvert)
{
	if (bInvertVisibility == bInInvert)
	{
		return;
	}

	bInvertVisibility = bInInvert;
	OnInvertVisibilityChanged();
}

void UAvaHideEmptyModifier::OnTextChanged()
{
	MarkModifierDirty();
}

void UAvaHideEmptyModifier::OnContainerActorChanged()
{
	ReferenceActor.ReferenceContainer = EActorModifierReferenceContainer::Other;
	ReferenceActor.ReferenceActorWeak = ContainerActorWeak;
	ReferenceActor.bSkipHiddenActors = false;

	if (const FActorModifierSceneTreeUpdateExtension* SceneExtension = GetExtension<FActorModifierSceneTreeUpdateExtension>())
	{
		SceneExtension->CheckTrackedActorUpdate(0);
	}
}

void UAvaHideEmptyModifier::OnInvertVisibilityChanged()
{
	MarkModifierDirty();
}

void UAvaHideEmptyModifier::OnSceneTreeTrackedActorChanged(int32 InIdx, AActor* InPreviousActor, AActor* InNewActor)
{
	Super::OnSceneTreeTrackedActorChanged(InIdx, InPreviousActor, InNewActor);

	if (UActorModifierVisibilityShared* VisibilityShared = GetShared<UActorModifierVisibilityShared>(false))
	{
		VisibilityShared->RestoreActorState(this, InPreviousActor);
	}

	MarkModifierDirty();
}

void UAvaHideEmptyModifier::OnSceneTreeTrackedActorDirectChildrenChanged(int32 InIdx,
	const TArray<TWeakObjectPtr<AActor>>& InPreviousChildrenActors,
	const TArray<TWeakObjectPtr<AActor>>& InNewChildrenActors)
{
	// Overwrite parent behaviour, don't do anything when children ordered changed
}

#undef LOCTEXT_NAMESPACE
