// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaMaterialParameterModifier.h"

#include "Components/PrimitiveComponent.h"
#include "Materials/MaterialInstanceDynamic.h"

#if WITH_EDITOR
#include "AvaOutlinerSubsystem.h"
#include "Engine/World.h"
#endif

#define LOCTEXT_NAMESPACE "AvaMaterialParameterModifier"

namespace UE::Avalanche::Modifiers::Private
{
	template<typename InArrayType>
	void MatchKeys(const TArray<InArrayType>& InSource, TArray<InArrayType>& InTarget)
	{
		// Cache source parameter names
		TSet<FName> SourceParameterNames;
		SourceParameterNames.Reserve(InSource.Num());

		for (const InArrayType& Parameter : InSource)
		{
			SourceParameterNames.Add(Parameter.Name);
		}

		using TParameterIterator = TArray<InArrayType>::TIterator;

		// Remove parameters no longer tracked
		for (TParameterIterator It = InTarget.CreateIterator(); It; ++It)
		{
			if (!SourceParameterNames.Contains(It->Name))
			{
				It.RemoveCurrent();
			}
		}

		// Cache target parameter names
		TSet<FName> TargetParameterNames;
		TargetParameterNames.Reserve(InTarget.Num());

		for (const InArrayType& Parameter : InTarget)
		{
			TargetParameterNames.Add(Parameter.Name);
		}

		using TConstParameterIterator = TArray<InArrayType>::TConstIterator;

		// Add newly tracked values
		for (TConstParameterIterator It = InSource.CreateConstIterator(); It; ++It)
		{
			if (!TargetParameterNames.Contains(It->Name))
			{
				InTarget.Add(*It);
			}
		}
	}
}

void FAvaMaterialParameterMap::MatchKeys(const FAvaMaterialParameterMap& InParameterMap)
{
	UE::Avalanche::Modifiers::Private::MatchKeys(InParameterMap.ScalarParameterStructs,  ScalarParameterStructs);
	UE::Avalanche::Modifiers::Private::MatchKeys(InParameterMap.VectorParameterStructs,  VectorParameterStructs);
	UE::Avalanche::Modifiers::Private::MatchKeys(InParameterMap.TextureParameterStructs, TextureParameterStructs);
}

void FAvaMaterialParameterMap::PushParametersTo(UMaterialInstanceDynamic* InMaterial)
{
	if (!IsValid(InMaterial))
	{
		return;
	}

	// Apply Scalar
	for (const FAvaMaterialParameterMapScalar& ScalarPair : ScalarParameterStructs)
	{
		InMaterial->SetScalarParameterValue(ScalarPair.Name, ScalarPair.Value);
	}

	// Apply Color
	for (const FAvaMaterialParameterMapVector& VectorPair : VectorParameterStructs)
	{
		InMaterial->SetVectorParameterValue(VectorPair.Name, VectorPair.Value);
	}

	// Apply Texture
	for (const FAvaMaterialParameterMapTexture& TexturePair : TextureParameterStructs)
	{
		InMaterial->SetTextureParameterValue(TexturePair.Name, TexturePair.Value);
	}
}

void FAvaMaterialParameterMap::PullParametersFrom(UMaterialInstanceDynamic* InMaterial)
{
	if (!IsValid(InMaterial))
	{
		return;
	}

	// Get Scalar
	for (FAvaMaterialParameterMapScalar& ScalarPair : ScalarParameterStructs)
	{
		ScalarPair.Value = InMaterial->K2_GetScalarParameterValue(ScalarPair.Name);
	}

	// Get Color
	for (FAvaMaterialParameterMapVector& VectorPair : VectorParameterStructs)
	{
		VectorPair.Value = InMaterial->K2_GetVectorParameterValue(VectorPair.Name);
	}

	// Get Texture
	for (FAvaMaterialParameterMapTexture& TexturePair : TextureParameterStructs)
	{
		TexturePair.Value = InMaterial->K2_GetTextureParameterValue(TexturePair.Name);
	}
}

const FAvaMaterialParameterMapScalar* FAvaMaterialParameterMap::FindScalarParameter(FName InParameterName) const
{
	for (const FAvaMaterialParameterMapScalar& Parameter : ScalarParameterStructs)
	{
		if (Parameter.Name == InParameterName)
		{
			return &Parameter;
		}
	}

	return nullptr;
}

FAvaMaterialParameterMapScalar* FAvaMaterialParameterMap::FindScalarParameter(FName InParameterName, bool bInCreateIfMissing)
{
	for (FAvaMaterialParameterMapScalar& Parameter : ScalarParameterStructs)
	{
		if (Parameter.Name == InParameterName)
		{
			return &Parameter;
		}
	}

	if (!bInCreateIfMissing)
	{
		return nullptr;
	}

	ScalarParameterStructs.Emplace(InParameterName, 0.f);

	return &ScalarParameterStructs.Last();
}

const FAvaMaterialParameterMapVector* FAvaMaterialParameterMap::FindVectorParameter(FName InParameterName) const
{
	for (const FAvaMaterialParameterMapVector& Parameter : VectorParameterStructs)
	{
		if (Parameter.Name == InParameterName)
		{
			return &Parameter;
		}
	}

	return nullptr;
}

FAvaMaterialParameterMapVector* FAvaMaterialParameterMap::FindVectorParameter(FName InParameterName, bool bInCreateIfMissing)
{
	for (FAvaMaterialParameterMapVector& Parameter : VectorParameterStructs)
	{
		if (Parameter.Name == InParameterName)
		{
			return &Parameter;
		}
	}

	if (!bInCreateIfMissing)
	{
		return nullptr;
	}

	VectorParameterStructs.Emplace(InParameterName, FLinearColor::Black);

	return &VectorParameterStructs.Last();
}

const FAvaMaterialParameterMapTexture* FAvaMaterialParameterMap::FindTextureParameter(FName InParameterName) const
{
	for (const FAvaMaterialParameterMapTexture& Parameter : TextureParameterStructs)
	{
		if (Parameter.Name == InParameterName)
		{
			return &Parameter;
		}
	}

	return nullptr;
}

FAvaMaterialParameterMapTexture* FAvaMaterialParameterMap::FindTextureParameter(FName InParameterName, bool bInCreateIfMissing)
{
	for (FAvaMaterialParameterMapTexture& Parameter : TextureParameterStructs)
	{
		if (Parameter.Name == InParameterName)
		{
			return &Parameter;
		}
	}

	if (!bInCreateIfMissing)
	{
		return nullptr;
	}

	TextureParameterStructs.Emplace(InParameterName, nullptr);

	return &TextureParameterStructs.Last();
}

#if WITH_EDITOR
void FAvaMaterialParameterMap::UpdateDeprecatedParameterProperties()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	for (const TPair<FName, float>& ScalarPair : ScalarParameters_DEPRECATED)
	{
		FAvaMaterialParameterMapScalar* ScalarParameter = FindScalarParameter(ScalarPair.Key, /* Create if doesn't exist */ false);

		// Only add old values if new ones don't already exist.
		if (!ScalarParameter)
		{
			ScalarParameter = FindScalarParameter(ScalarPair.Key, /* Create if doesn't exist */ true);
			
			if (ScalarParameter)
			{
				ScalarParameter->Value = ScalarPair.Value;
			}
		}
	}

	for (const TPair<FName, FLinearColor>& VectorPair : VectorParameters_DEPRECATED)
	{
		FAvaMaterialParameterMapVector* VectorParameter = FindVectorParameter(VectorPair.Key, /* Create if doesn't exist */ false);

		// Only add old values if new ones don't already exist.
		if (!VectorParameter)
		{
			VectorParameter = FindVectorParameter(VectorPair.Key, /* Create if doesn't exist */ true);

			if (VectorParameter)
			{
				VectorParameter->Value = VectorPair.Value;
			}
		}
	}

	for (const TPair<FName, TObjectPtr<UTexture>>& TexturePair : TextureParameters_DEPRECATED)
	{
		FAvaMaterialParameterMapTexture* TextureParameter = FindTextureParameter(TexturePair.Key, /* Create if doesn't exist */ false);

		// Only add old values if new ones don't already exist.
		if (!TextureParameter)
		{
			TextureParameter = FindTextureParameter(TexturePair.Key, /* Create if doesn't exist */ true);

			if (TextureParameter)
			{
				TextureParameter->Value = TexturePair.Value;
			}
		}
	}

	ScalarParameters_DEPRECATED.Empty();
	VectorParameters_DEPRECATED.Empty();
	TextureParameters_DEPRECATED.Empty();

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif

UAvaMaterialParameterModifier::UAvaMaterialParameterModifier()
{
	MaterialClass = UMaterialInstanceDynamic::StaticClass();
}

void UAvaMaterialParameterModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("MaterialParameter"));
	InMetadata.SetCategory(TEXT("Rendering"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Material Parameter"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Sets specified dynamic materials parameters on an actor and its children"));
#endif

	InMetadata.SetCompatibilityRule([this](const AActor* InActor)->bool
	{
		if (!IsValid(InActor))
		{
			return false;
		}

		// Check actor component has at least one dynamic instance material in its components or children
		const bool bResult = ForEachComponent<UPrimitiveComponent>([](UPrimitiveComponent* InComponent)
		{
			if (InComponent)
			{
				for (int32 Idx = 0; Idx < InComponent->GetNumMaterials(); Idx++)
				{
					const UMaterialInterface* Mat = InComponent->GetMaterial(Idx);
					if (!IsValid(Mat) || !Mat->IsA<UMaterialInstanceDynamic>())
					{
						continue;
					}

					return false;
				}
			}

			return true;
		}
		, EActorModifierCoreComponentType::All
		, EActorModifierCoreLookup::SelfAndAllChildren
		, InActor);

		return !bResult;
	});
}

void UAvaMaterialParameterModifier::OnModifierEnabled(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierEnabled(InReason);

#if WITH_EDITOR
	/** Bind to delegate to detect material changes */
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddUObject(this, &UAvaMaterialParameterModifier::OnActorPropertyChanged);
#endif
}

void UAvaMaterialParameterModifier::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierDisabled(InReason);

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
#endif
}

void UAvaMaterialParameterModifier::OnModifiedActorTransformed()
{
	// Overwrite parent class behaviour don't do anything when moved
}

void UAvaMaterialParameterModifier::RestorePreState()
{
	RestoreMaterialParameters();
}

void UAvaMaterialParameterModifier::SavePreState()
{
	Super::SavePreState();

	// Scan for materials in actors
    ScanActorMaterials();

	// Save original parameters
    SaveMaterialParameters();
}

void UAvaMaterialParameterModifier::Apply()
{
	// Set new parameter value to them
	for (TPair<TWeakObjectPtr<UMaterialInstanceDynamic>, FAvaMaterialParameterMap>& MaterialParameterPair : SavedMaterialParameters)
	{
		if (UMaterialInstanceDynamic* MID = MaterialParameterPair.Key.Get())
		{
			// Set State
			MaterialParameters.PushParametersTo(MID);
		}
	}

	Next();
}

#if WITH_EDITOR
void UAvaMaterialParameterModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	static const FName MaterialParametersName = GET_MEMBER_NAME_CHECKED(UAvaMaterialParameterModifier, MaterialParameters);
	static const FName UpdateChildrenName = GET_MEMBER_NAME_CHECKED(UAvaMaterialParameterModifier, bUpdateChildren);

	if (MemberName == MaterialParametersName)
	{
		OnMaterialParametersChanged();
	}
	else if (MemberName == UpdateChildrenName)
	{
		OnUpdateChildrenChanged();
	}
}

void UAvaMaterialParameterModifier::OnActorPropertyChanged(UObject* InObject, FPropertyChangedEvent& InChangeEvent)
{
	if (!InObject)
	{
		return;
	}

	if (const AActor* Actor = Cast<AActor>(InObject))
	{
		if (IsActorSupported(Actor))
		{
			MarkModifierDirty();
		}
	}
	else if (const UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(InObject))
	{
		if (!GetComponentDynamicMaterials(PrimitiveComponent).IsEmpty())
		{
			MarkModifierDirty();
		}
	}
}
#endif

bool UAvaMaterialParameterModifier::IsActorSupported(const AActor* InActor) const
{
	if (!InActor)
	{
		return false;
	}

	const AActor* ActorModified = GetModifiedActor();
	if (!ActorModified)
	{
		return false;
	}

	TArray<UPrimitiveComponent*> PrimitiveComponents;
	InActor->GetComponents(PrimitiveComponents, false);
	const bool bAttachedToModifiedActor = InActor->IsAttachedTo(ActorModified);
	for (const UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
	{
		for (UMaterialInstanceDynamic* MID : GetComponentDynamicMaterials(PrimitiveComponent))
		{
			if (bAttachedToModifiedActor)
			{
				return true;
			}
			else if (SavedMaterialParameters.Contains(MID))
			{
				return true;
			}
		}
	}

	return false;
}

TSet<UMaterialInstanceDynamic*> UAvaMaterialParameterModifier::GetComponentDynamicMaterials(const UPrimitiveComponent* InComponent) const
{
	TSet<UMaterialInstanceDynamic*> Materials;
	if (!InComponent)
	{
		return Materials;
	}

	for (int32 Idx = 0; Idx < InComponent->GetNumMaterials(); Idx++)
	{
		UMaterialInterface* Mat = InComponent->GetMaterial(Idx);
		if (!IsValid(Mat) || !Mat->IsA(MaterialClass))
		{
			continue;
		}

		if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(Mat))
		{
			Materials.Add(MID);
		}
	}

	return Materials;
}

#if WITH_EDITOR
void UAvaMaterialParameterModifier::PostLoad()
{
	Super::PostLoad();

	UpdateDeprecatedParameterProperties();
}

void UAvaMaterialParameterModifier::UpdateDeprecatedParameterProperties()
{
	MaterialParameters.UpdateDeprecatedParameterProperties();

	for (TPair<TWeakObjectPtr<UMaterialInstanceDynamic>, FAvaMaterialParameterMap>& SavedMaterialPair : SavedMaterialParameters)
	{
		SavedMaterialPair.Value.UpdateDeprecatedParameterProperties();
	}
}
#endif

void UAvaMaterialParameterModifier::SetMaterialParameters(const FAvaMaterialParameterMap& InParameterMap)
{
	MaterialParameters = InParameterMap;
	OnMaterialParametersChanged();
}

void UAvaMaterialParameterModifier::SetUpdateChildren(bool bInUpdateChildren)
{
	if (bUpdateChildren == bInUpdateChildren)
	{
		return;
	}

	bUpdateChildren = bInUpdateChildren;
	OnUpdateChildrenChanged();
}

void UAvaMaterialParameterModifier::OnSceneTreeTrackedActorChildrenChanged(int32 InIdx,
	const TSet<TWeakObjectPtr<AActor>>& InPreviousChildrenActors,
	const TSet<TWeakObjectPtr<AActor>>& InNewChildrenActors)
{
	Super::OnSceneTreeTrackedActorChildrenChanged(InIdx, InPreviousChildrenActors, InNewChildrenActors);

	if (!bUpdateChildren)
	{
		return;
	}

	MarkModifierDirty();
}

void UAvaMaterialParameterModifier::OnSceneTreeTrackedActorDirectChildrenChanged(int32 InIdx,
	const TArray<TWeakObjectPtr<AActor>>& InPreviousChildrenActors,
	const TArray<TWeakObjectPtr<AActor>>& InNewChildrenActors)
{
	// Overwrite parent class behaviour don't do anything when direct children changed
}

void UAvaMaterialParameterModifier::SaveMaterialParameters()
{
	// Save original values
	for (TPair<TWeakObjectPtr<UMaterialInstanceDynamic>, FAvaMaterialParameterMap>& MaterialParameterPair : SavedMaterialParameters)
	{
		if (UMaterialInstanceDynamic* MID = MaterialParameterPair.Key.Get())
		{
			MaterialParameterPair.Value.PullParametersFrom(MID);
		}
	}
}

void UAvaMaterialParameterModifier::RestoreMaterialParameters()
{
	// Set original saved values back
	for (TPair<TWeakObjectPtr<UMaterialInstanceDynamic>, FAvaMaterialParameterMap>& MaterialParameterPair : SavedMaterialParameters)
	{
		if (UMaterialInstanceDynamic* MID = MaterialParameterPair.Key.Get())
		{
			MaterialParameterPair.Value.PushParametersTo(MID);
		}
	}
}

void UAvaMaterialParameterModifier::ScanActorMaterials()
{
	// Used to compare with current one to remove materials not tracked anymore
	TSet<UMaterialInstanceDynamic*> PrevScannedMaterials;
	for (TPair<TWeakObjectPtr<UMaterialInstanceDynamic>, FAvaMaterialParameterMap>& MaterialParameterPair : SavedMaterialParameters)
	{
		PrevScannedMaterials.Add(MaterialParameterPair.Key.Get());
	}

	ForEachComponent<UPrimitiveComponent>([this, &PrevScannedMaterials](UPrimitiveComponent* InComponent)->bool
	{
#if WITH_EDITOR
		if (InComponent->IsVisualizationComponent())
		{
			return true;
		}
#endif

		for (UMaterialInstanceDynamic* MID : GetComponentDynamicMaterials(InComponent))
		{
			const bool bMaterialAdded = !SavedMaterialParameters.Contains(MID);
			FAvaMaterialParameterMap& ParameterMap = SavedMaterialParameters.FindOrAdd(MID);

			// Removed untracked keys
			ParameterMap.MatchKeys(MaterialParameters);

			if (bMaterialAdded)
			{
				// Save original values
                ParameterMap.PullParametersFrom(MID);
				OnActorMaterialAdded(MID);
			}

			PrevScannedMaterials.Remove(MID);
		}

		return true;
	}
	, EActorModifierCoreComponentType::All
	, bUpdateChildren ? EActorModifierCoreLookup::SelfAndAllChildren : EActorModifierCoreLookup::Self);

	// Remove materials not tracked anymore
	for (UMaterialInstanceDynamic* MID : PrevScannedMaterials)
	{
		SavedMaterialParameters.Remove(MID);
		OnActorMaterialRemoved(MID);
	}
}

void UAvaMaterialParameterModifier::OnMaterialParametersChanged()
{
	MarkModifierDirty();
}

void UAvaMaterialParameterModifier::OnUpdateChildrenChanged()
{
	MarkModifierDirty();
}

#undef LOCTEXT_NAMESPACE
