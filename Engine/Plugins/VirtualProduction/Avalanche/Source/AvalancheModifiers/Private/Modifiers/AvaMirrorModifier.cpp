// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaMirrorModifier.h"

#include "Async/Async.h"
#include "Components/DynamicMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Operations/MeshMirror.h"
#include "Operations/MeshPlaneCut.h"

#define LOCTEXT_NAMESPACE "AvaMirrorModifier"

void UAvaMirrorModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("Mirror"));
	InMetadata.SetCategory(TEXT("Geometry"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Mirror"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Duplicates and mirrors a geometry shape along a plane"));
#endif
}

void UAvaMirrorModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

#if WITH_EDITOR
	CreatePreviewComponent();
#endif
}

void UAvaMirrorModifier::Apply()
{
	const FTransform MirrorFrame(MirrorFrameRotation, MirrorFramePosition);
	
	if (!MirrorFrame.IsValid())
	{
		Next();
		return;
	}

	UDynamicMeshComponent* const DynMeshComp = GetMeshComponent();

	if (!IsValid(DynMeshComp))
	{
		Fail(LOCTEXT("InvalidDynamicMeshComponent", "Invalid dynamic mesh component on modified actor"));
		return;
	}
	
	using namespace UE::Geometry;
		
	constexpr double PlaneTolerance = FMathf::ZeroTolerance * 10.0;

	const FFrame3d UseCutFrame(MirrorFrame);
	const FVector3d MirrorPlaneOrigin = UseCutFrame.Origin;
	FVector3d UseNormal = UseCutFrame.Z();
	
	if (bApplyPlaneCut && bFlipCutSide)
	{
		UseNormal = -UseNormal;
	}
			
	DynMeshComp->GetDynamicMesh()->EditMesh([this, MirrorPlaneOrigin, UseNormal, PlaneTolerance](FDynamicMesh3& EditMesh)
	{
		if (bApplyPlaneCut)
		{
			FMeshPlaneCut Cutter(&EditMesh, MirrorPlaneOrigin, UseNormal);
			Cutter.PlaneTolerance = PlaneTolerance;
			Cutter.Cut();
		}

		FMeshMirror MirrorOperation(&EditMesh, MirrorPlaneOrigin, UseNormal);
		MirrorOperation.bWeldAlongPlane = bWeldAlongPlane;
		MirrorOperation.bAllowBowtieVertexCreation = false;
		MirrorOperation.PlaneTolerance = PlaneTolerance;

		MirrorOperation.MirrorAndAppend(nullptr);

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

#if WITH_EDITOR
	UpdatePreviewComponent();
#endif

	Next();
}

void UAvaMirrorModifier::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierDisabled(InReason);
	
#if WITH_EDITOR
	UpdatePreviewComponent();
#endif
}

void UAvaMirrorModifier::OnModifierRemoved(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierRemoved(InReason);
	
#if WITH_EDITOR
	DestroyPreviewComponent();
#endif
}

#if WITH_EDITOR
void UAvaMirrorModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();
	
	static const FName ShowMirrorFrameName = GET_MEMBER_NAME_CHECKED(UAvaMirrorModifier, bShowMirrorFrame);
	static const FName ApplyPlaneCutName = GET_MEMBER_NAME_CHECKED(UAvaMirrorModifier, bApplyPlaneCut);
	static const FName ApplyPlaneCutFlipCutSideName = GET_MEMBER_NAME_CHECKED(UAvaMirrorModifier, bFlipCutSide);
	static const FName WeldAlongPlaneName = GET_MEMBER_NAME_CHECKED(UAvaMirrorModifier, bWeldAlongPlane);
	static const FName MirrorFramePositionName = GET_MEMBER_NAME_CHECKED(UAvaMirrorModifier, MirrorFramePosition);
	static const FName MirrorFrameRotationName = GET_MEMBER_NAME_CHECKED(UAvaMirrorModifier, MirrorFrameRotation);

	if (MemberName == MirrorFramePositionName ||
		MemberName == MirrorFrameRotationName)
	{
		OnMirrorFrameChanged();
	}
	else if (MemberName == ApplyPlaneCutName ||
		MemberName == ApplyPlaneCutFlipCutSideName ||
		MemberName == WeldAlongPlaneName)
	{
		OnMirrorOptionChanged();
	}
	else if (MemberName == ShowMirrorFrameName)
	{
		OnShowMirrorFrameChanged();
	}
}
#endif

void UAvaMirrorModifier::SetMirrorFramePosition(const FVector& InMirrorFramePosition)
{
	if (MirrorFramePosition == InMirrorFramePosition)
	{
		return;
	}

	MirrorFramePosition = InMirrorFramePosition;
	OnMirrorFrameChanged();
}

void UAvaMirrorModifier::SetMirrorFrameRotation(const FRotator& InMirrorFrameRotation)
{
	if (MirrorFrameRotation == InMirrorFrameRotation)
	{
		return;
	}

	MirrorFrameRotation = InMirrorFrameRotation;
	OnMirrorFrameChanged();
}

void UAvaMirrorModifier::SetApplyPlaneCut(bool bInApplyPlaneCut)
{
	if (bApplyPlaneCut == bInApplyPlaneCut)
	{
		return;
	}

	bApplyPlaneCut = bInApplyPlaneCut;
	OnMirrorOptionChanged();
}

void UAvaMirrorModifier::SetFlipCutSide(bool bInFlipCutSide)
{
	if (bFlipCutSide == bInFlipCutSide)
	{
		return;
	}

	bFlipCutSide = bInFlipCutSide;
	OnMirrorOptionChanged();
}

void UAvaMirrorModifier::SetWeldAlongPlane(bool bInWeldAlongPlane)
{
	if (bWeldAlongPlane == bInWeldAlongPlane)
	{
		return;
	}

	bWeldAlongPlane = bInWeldAlongPlane;
	OnMirrorOptionChanged();
}

void UAvaMirrorModifier::OnMirrorFrameChanged()
{
#if WITH_EDITOR
	UpdatePreviewComponent();
#endif
	
	MarkModifierDirty();
}

void UAvaMirrorModifier::OnMirrorOptionChanged()
{
	MarkModifierDirty();	
}

#if WITH_EDITOR
void UAvaMirrorModifier::CreatePreviewComponent()
{
	if (UDynamicMeshComponent* const DynMeshComponent = GetMeshComponent())
	{
		PreviewPlane.Create(DynMeshComponent);
	}
}

void UAvaMirrorModifier::DestroyPreviewComponent()
{
	PreviewPlane.Destroy();
}

void UAvaMirrorModifier::UpdatePreviewComponent()
{
	PreviewPlane.Update(FTransform(
		MirrorFrameRotation,
		MirrorFramePosition,
		FVector(1.5, 1.5, 0.005)
	));

	if (bShowMirrorFrame && IsModifierEnabled())
	{
		PreviewPlane.Show();
	}
	else
	{
		PreviewPlane.Hide();
	}
}

void UAvaMirrorModifier::OnShowMirrorFrameChanged()
{
	UpdatePreviewComponent();
}
#endif

#undef LOCTEXT_NAMESPACE
