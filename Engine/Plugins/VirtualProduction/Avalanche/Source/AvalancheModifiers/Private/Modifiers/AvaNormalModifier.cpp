// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaNormalModifier.h"

#include "Async/Async.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/MeshNormals.h"
#include "Polygroups/PolygroupSet.h"

#define LOCTEXT_NAMESPACE "AvaNormalModifier"

void UAvaNormalModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("Normal"));
	InMetadata.SetCategory(TEXT("Geometry"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Normal"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Recompute the normals and split them based on different options"));
#endif
}

void UAvaNormalModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);
}

void UAvaNormalModifier::Apply()
{
	UDynamicMeshComponent* const DynMeshComp = GetMeshComponent();
	
	if (!IsValid(DynMeshComp))
	{
		Fail(LOCTEXT("InvalidDynamicMeshComponent", "Invalid dynamic mesh component on modified actor"));
		return;
	}

	using namespace UE::Geometry;
			
	DynMeshComp->GetDynamicMesh()->EditMesh([this](FDynamicMesh3& EditMesh)
	{
		FDynamicMeshNormalOverlay* NormalOverlay = EditMesh.Attributes()->PrimaryNormals();
		FMeshNormals MeshNormals(&EditMesh);
		const bool bRecomputeNormals = bAreaWeighted || bAngleWeighted || bInvert || SplitMethod != EAvaNormalModifierSplitMethod::None;

		if (bInvert)
		{
			EditMesh.ReverseOrientation(true);
		}
	
		switch (SplitMethod)
		{
			case EAvaNormalModifierSplitMethod::Triangle:
				FMeshNormals::InitializeMeshToPerTriangleNormals(&EditMesh);
			break;
			case EAvaNormalModifierSplitMethod::Vertex:
				FMeshNormals::InitializeOverlayToPerVertexNormals(NormalOverlay, false);
			break;
			case EAvaNormalModifierSplitMethod::PolyGroup:
				{
					// Gather groups in mesh
					FPolygroupSet MeshGroups(&EditMesh);
					const int32 Layer = GetPolyGroupLayerIdx();
					if (Layer != INDEX_NONE && Layer < EditMesh.Attributes()->NumPolygroupLayers())
					{
						const FDynamicMeshPolygroupAttribute* PolyGroupAttr = EditMesh.Attributes()->GetPolygroupLayer(Layer);
						MeshGroups = FPolygroupSet(&EditMesh, PolyGroupAttr);
					}
					// Share vertex between tris if we are from same group
					NormalOverlay->CreateFromPredicate([&MeshGroups, Layer](int VID, int TA, int TB)
					{
						if (MeshGroups.PolygroupAttrib)
						{
							return MeshGroups.GetTriangleGroup(TA) == MeshGroups.GetTriangleGroup(TB) && MeshGroups.GetTriangleGroup(TA) == Layer;
						}
						return MeshGroups.GetTriangleGroup(TA) == MeshGroups.GetTriangleGroup(TB);
					}, 0);
				}
			break;
			case EAvaNormalModifierSplitMethod::Threshold:
				{
					// Compute normals because we need them for threshold comparison
                    MeshNormals.ComputeTriangleNormals();
                    const TArray<FVector3d>& Normals = MeshNormals.GetNormals();
                    // Share vertex between tris if we do not surpass a threshold
                    float AngleThresholdRadians = FMathf::Cos(AngleThreshold * FMathf::DegToRad);
                    NormalOverlay->CreateFromPredicate([&Normals, &AngleThresholdRadians](int VID, int TA, int TB)
                    {
                    	return Normals[TA].Dot(Normals[TB]) > AngleThresholdRadians;
                    }, 0);
				}
			break;
			default:
			break;
		}
		
		if (bRecomputeNormals)
		{
			// Recompute normals with area or angle weight
			MeshNormals.RecomputeOverlayNormals(NormalOverlay, bAreaWeighted, bAngleWeighted);
			// Copy mesh normals to overlay
			MeshNormals.CopyToOverlay(NormalOverlay, false);
		}
	}, EDynamicMeshChangeType::AttributeEdit, EDynamicMeshAttributeChangeFlags::NormalsTangents);
	
	Next();
}

#if WITH_EDITOR
void UAvaNormalModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();
	
	static const FName AngleWeightedName = GET_MEMBER_NAME_CHECKED(UAvaNormalModifier, bAngleWeighted);
	static const FName AreaWeightedName = GET_MEMBER_NAME_CHECKED(UAvaNormalModifier, bAreaWeighted);
	static const FName InvertName = GET_MEMBER_NAME_CHECKED(UAvaNormalModifier, bInvert);
	static const FName SplitMethodName = GET_MEMBER_NAME_CHECKED(UAvaNormalModifier, SplitMethod);
	static const FName AngleThresholdName = GET_MEMBER_NAME_CHECKED(UAvaNormalModifier, AngleThreshold);
	static const FName PolyGroupLayerName = GET_MEMBER_NAME_CHECKED(UAvaNormalModifier, PolyGroupLayer);
	
	if (MemberName == AngleWeightedName)
	{
		OnAngleWeightedChanged();
	}
	else if (MemberName == AreaWeightedName)
	{
		OnAreaWeightedChanged();
	}
	else if (MemberName == InvertName)
	{
		OnInvertChanged();
	}
	else if (MemberName == SplitMethodName)
	{
		OnSplitMethodChanged();
	}
	else if (MemberName == AngleThresholdName)
	{
		OnAngleThresholdChanged();
	}
	else if (MemberName == PolyGroupLayerName)
	{
		OnPolyGroupLayerChanged();
	}
}
#endif

void UAvaNormalModifier::SetAngleWeighted(bool bInAngleWeighted)
{
	if (bAngleWeighted == bInAngleWeighted)
	{
		return;
	}

	bAngleWeighted = bInAngleWeighted;
	OnAngleWeightedChanged();
}

void UAvaNormalModifier::SetAreaWeighted(bool bInAreaWeighted)
{
	if (bAreaWeighted == bInAreaWeighted)
	{
		return;
	}

	bAreaWeighted = bInAreaWeighted;
	OnAreaWeightedChanged();
}

void UAvaNormalModifier::SetInvert(bool bInInvert)
{
	if (bInvert == bInInvert)
	{
		return;
	}

	bInvert = bInInvert;
	OnInvertChanged();
}

void UAvaNormalModifier::SetSplitMethod(EAvaNormalModifierSplitMethod InSplitMethod)
{
	if (SplitMethod == InSplitMethod)
	{
		return;
	}

	SplitMethod = InSplitMethod;
	OnSplitMethodChanged();
}

void UAvaNormalModifier::SetAngleThreshold(float InAngleThreshold)
{
	if (AngleThreshold == InAngleThreshold)
	{
		return;
	}

	if (InAngleThreshold < 0.f || InAngleThreshold > 180.f)
	{
		return;
	}

	AngleThreshold = InAngleThreshold;
	OnAngleThresholdChanged();
}

void UAvaNormalModifier::SetPolyGroupLayerIdx(int32 InPolyGroupLayer)
{
	const TArray<FString> Layers = GetPolyGroupLayers();

	if (!Layers.IsValidIndex(InPolyGroupLayer))
	{
		return;
	}

	FString PolyGroupLayerStr(Layers[InPolyGroupLayer]);
	
	if (PolyGroupLayer == PolyGroupLayerStr)
	{
		return;
	}

	PolyGroupLayer = PolyGroupLayerStr;
	OnPolyGroupLayerChanged();
}

int32 UAvaNormalModifier::GetPolyGroupLayerIdx() const
{
	if (PolyGroupLayer == TEXT("None"))
	{
		return INDEX_NONE;
	}
	const TArray<FString> Layers = GetPolyGroupLayers();
	return Layers.Find(PolyGroupLayer);
}

void UAvaNormalModifier::SetPolyGroupLayer(const FString& InPolyGroupLayer)
{
	if (PolyGroupLayer == InPolyGroupLayer)
	{
		return;
	}
	
	const TArray<FString> Layers = GetPolyGroupLayers();
	if (!Layers.Contains(InPolyGroupLayer))
	{
		return;
	}

	PolyGroupLayer = InPolyGroupLayer;
	OnPolyGroupLayerChanged();
}

void UAvaNormalModifier::OnAngleWeightedChanged()
{
	MarkModifierDirty();
}

void UAvaNormalModifier::OnAreaWeightedChanged()
{
	MarkModifierDirty();
}

void UAvaNormalModifier::OnInvertChanged()
{
	MarkModifierDirty();
}

void UAvaNormalModifier::OnSplitMethodChanged()
{
	MarkModifierDirty();
}

void UAvaNormalModifier::OnAngleThresholdChanged()
{
	MarkModifierDirty();
}

void UAvaNormalModifier::OnPolyGroupLayerChanged()
{
	MarkModifierDirty();
}

TArray<FString> UAvaNormalModifier::GetPolyGroupLayers() const
{
	TArray<FString> Layers;
	if (IsMeshValid())
	{
		if (FDynamicMesh3* Mesh = GetMeshComponent()->GetMesh())
		{
			if (const int32 NumPolyGroupLayers = Mesh->Attributes()->NumPolygroupLayers())
			{
				for (int32 Idx = 0; Idx < NumPolyGroupLayers; Idx++)
				{
					const UE::Geometry::FDynamicMeshPolygroupAttribute* PolyGroup = Mesh->Attributes()->GetPolygroupLayer(Idx);
					Layers.Add(FString::FromInt(Idx) + TEXT(". ") + PolyGroup->GetName().ToString());
				}
			}
		}
	}
	else
	{
		Layers.Add(TEXT("None"));
	}
	return Layers;
}

#undef LOCTEXT_NAMESPACE
