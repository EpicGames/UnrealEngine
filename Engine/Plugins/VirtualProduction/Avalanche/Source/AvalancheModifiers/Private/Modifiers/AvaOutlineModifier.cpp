// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaOutlineModifier.h"

#include "Async/Async.h"
#include "AvaShapeActor.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "Operations/InsetMeshRegion.h"
#include "Operations/MeshSelfUnion.h"
#include "Parameterization/DynamicMeshUVEditor.h"

#define LOCTEXT_NAMESPACE "AvaOutlineModifier"

void UAvaOutlineModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("Outline"));
	InMetadata.SetCategory(TEXT("Geometry"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Outline"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Creates a new region inside or outside of a geometry shape"));
#endif

	InMetadata.DisallowAfter(TEXT("Extrude"));
	InMetadata.SetCompatibilityRule([](const AActor* InActor)->bool
	{
		bool bSupported = false;
		if(InActor)
		{
			if (const UDynamicMeshComponent* DynMeshComponent = InActor->FindComponentByClass<UDynamicMeshComponent>())
			{
				DynMeshComponent->ProcessMesh([&bSupported](const FDynamicMesh3& ProcessMesh)
				{
					bSupported = ProcessMesh.VertexCount() > 0 && static_cast<FBox>(ProcessMesh.GetBounds(true)).GetSize().X == 0;
				});
			}
		}
		return bSupported;
	});
}

void UAvaOutlineModifier::Apply()
{
	if (Distance == 0.f)
	{
		Next();
		return;
	}
	
	const bool bIsEmptyTriangleCount = PreModifierCachedMesh->TriangleCount() == 0;
	const bool bHasNoAttributes = !PreModifierCachedMesh->HasAttributes();
	
	if (bIsEmptyTriangleCount || bHasNoAttributes)
	{
		Fail(LOCTEXT("InvalidMeshData", "Invalid triangle count or attributes"));
		return;
	}
	
	// clamp distance inset
	if (Mode == EAvaOutlineMode::Inset && Distance > GetMaxInsetDistance())
	{
		Distance = GetMaxInsetDistance();
	}
	
	using namespace UE::Geometry;
		
	GetMeshComponent()->EditMesh([this](FDynamicMesh3& EditMesh)
	{
		FDynamicMeshNormalOverlay* const NormalOverlay = EditMesh.Attributes()->PrimaryNormals();
		FDynamicMeshUVOverlay* const UVOverlay = EditMesh.Attributes()->PrimaryUV();
		const FVector Size3D = GetMeshBounds().GetSize();

		if (EditMesh.TriangleCount() > 0)
		{
			// weld edges to avoid loose triangles inside
			FMergeCoincidentMeshEdges WeldOp(&EditMesh);
			WeldOp.Apply();
		}
		
		// setup inset region by selecting triangles facing -X
		FInsetMeshRegion InsetOp(&EditMesh);
		for (int32 TId : EditMesh.TriangleIndicesItr())
		{
			FVector3d TriNormal = EditMesh.GetTriNormal(TId);
			if (TriNormal.Equals(-FVector::XAxisVector))
			{
				InsetOp.Triangles.Add(TId);
			}
		}
		InsetOp.AreaCorrection = 0.f;
		InsetOp.InsetDistance = (Mode == EAvaOutlineMode::Outset) ? -Distance : Distance;
		InsetOp.bSolveRegionInteriors = false;
		InsetOp.Softness = 1.f;
		InsetOp.bReproject = false;
		InsetOp.UVScaleFactor = (Size3D.Y/Size3D.Z) / Size3D.Y;
		InsetOp.Apply();

		// fix normals and reverse triangles
		for (int32 TId : InsetOp.AllModifiedTriangles)
		{
			if (!InsetOp.Triangles.Contains(TId))
			{
				if (Mode == EAvaOutlineMode::Outset)
				{
					EditMesh.ReverseTriOrientation(TId);
					const FIndex3i VerticesIdx = EditMesh.GetTriangle(TId);
					const int32 NIdA = NormalOverlay->GetElementIDAtVertex(TId, VerticesIdx.A);
					const int32 NIdB = NormalOverlay->GetElementIDAtVertex(TId, VerticesIdx.B);
					const int32 NIdC = NormalOverlay->GetElementIDAtVertex(TId, VerticesIdx.C);
					const FVector3f NormalA = -1 * NormalOverlay->GetElement(NIdA);
					const FVector3f NormalB = -1 * NormalOverlay->GetElement(NIdA);
					const FVector3f NormalC = -1 * NormalOverlay->GetElement(NIdA);
					NormalOverlay->SetElement(NIdA, NormalA);
					NormalOverlay->SetElement(NIdB, NormalB);
					NormalOverlay->SetElement(NIdC, NormalC);
				}
			}
			else if(bRemoveInside)
			{
				EditMesh.RemoveTriangle(TId);
			}
		}

		// Remove original triangle if we remove inside
		if (bRemoveInside)
		{
			for (const int32 TId : InsetOp.Triangles)
			{
				InsetOp.AllModifiedTriangles.Remove(TId);
			}
		}
		 
		// fix uv by planar projection
		const FTransform PlaneTransform(FRotator(0, 90, 90), FVector(0), FVector(Size3D.Y, Size3D.Z, 0));
		FDynamicMeshUVEditor UVEditor(&EditMesh, UVOverlay);
		const FFrame3d ProjectionFrame(PlaneTransform);
		const FVector Scale = PlaneTransform.GetScale3D();
		const FVector2d Dimensions(Scale.X, Scale.Y);
		FUVEditResult Result;
		UVEditor.SetTriangleUVsFromPlanarProjection(InsetOp.AllModifiedTriangles, [](const FVector3d& Pos) {
			return Pos;
		}, ProjectionFrame, Dimensions, &Result);

		if (AAvaShapeActor* ShapeActor = Cast<AAvaShapeActor>(GetModifiedActor()))
		{
			if (UAvaShapeDynamicMeshBase* ShapeGenerator = ShapeActor->GetDynamicMesh())
			{
				FAvaShapeMaterialUVParameters& UVParams = *ShapeGenerator->GetInUseMaterialUVParams(UAvaShapeDynamicMeshBase::MESH_INDEX_PRIMARY);
				UE::AvaShapes::TransformMeshUVs(EditMesh, Result.NewUVElements, UVParams, FVector2D(Size3D.Y,Size3D.Z), FVector2D(0.5, 0.5), 0.f);
			}
		}
	});
	
	Next();
}

#if WITH_EDITOR
void UAvaOutlineModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();
	
	static const FName ModeName = GET_MEMBER_NAME_CHECKED(UAvaOutlineModifier, Mode);
	static const FName DistanceName = GET_MEMBER_NAME_CHECKED(UAvaOutlineModifier, Distance);
	static const FName RemoveInsideName = GET_MEMBER_NAME_CHECKED(UAvaOutlineModifier, bRemoveInside);

	if (MemberName == ModeName)
	{
		OnModeChanged();
	}
	else if (MemberName == DistanceName)
	{
		OnDistanceChanged();
	}
	else if (MemberName == RemoveInsideName)
	{
		OnRemoveInsideChanged();
	}
}
#endif

void UAvaOutlineModifier::SetMode(EAvaOutlineMode InMode)
{
	if (Mode == InMode)
	{
		return;
	}

	Mode = InMode;
	OnModeChanged();
}

void UAvaOutlineModifier::SetDistance(float InDistance)
{
	if (Distance == InDistance)
	{
		return;
	}
	
	if (InDistance < 0)
	{
		return;
	}

	Distance = InDistance;
	OnDistanceChanged();
}

void UAvaOutlineModifier::SetRemoveInside(bool bInRemoveInside)
{
	if (bRemoveInside == bInRemoveInside)
	{
		return;
	}

	bRemoveInside = bInRemoveInside;
	OnRemoveInsideChanged();
}

float UAvaOutlineModifier::GetMaxInsetDistance() const
{
	const FVector Size = GetMeshBounds().GetSize();
	return (FMath::Min(Size.Y, Size.Z) / 2) - 0.1f;
}

void UAvaOutlineModifier::OnModeChanged()
{
	MarkModifierDirty();
}

void UAvaOutlineModifier::OnDistanceChanged()
{
	if (Mode == EAvaOutlineMode::Inset && Distance > GetMaxInsetDistance())
	{
		Distance = GetMaxInsetDistance();
	}
	MarkModifierDirty();
}

void UAvaOutlineModifier::OnRemoveInsideChanged()
{
	MarkModifierDirty();
}

#undef LOCTEXT_NAMESPACE
