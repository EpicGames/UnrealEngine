// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaPatternModifier.h"

#include "Async/Async.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMeshEditor.h"
#include "Tools/AvaPatternModifierCircleTool.h"
#include "Tools/AvaPatternModifierGridTool.h"
#include "Tools/AvaPatternModifierLineTool.h"

#define LOCTEXT_NAMESPACE "AvaPatternModifier"

struct FAvaPatternModifierVersion
{
	enum Type : int32
	{
		PreVersioning = 0,
		/** Moved properties within modifier */
		MigrateProperties,
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	static constexpr FGuid GUID = FGuid(0x9271D8A4, 0xBF414601, 0xA20FC0A3, 0x9D829565);
};

FCustomVersionRegistration GRegisterAvaPatternModifierVersion(FAvaPatternModifierVersion::GUID, static_cast<int32>(FAvaPatternModifierVersion::LatestVersion), TEXT("AvaPatternModifierVersion"));

void UAvaPatternModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("Pattern"));
	InMetadata.SetCategory(TEXT("Geometry"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Pattern"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Repeats a geometry multiple times following a specific layout pattern"));
#endif
}

void UAvaPatternModifier::Apply()
{
	if (!ActiveTool)
	{
		Fail(LOCTEXT("InvalidActiveTool", "Active tool is invalid or not set"));
		return;
	}

	using namespace UE::Geometry;

	UDynamicMeshComponent* TargetMeshComponent = GetMeshComponent();

	if (!IsValid(TargetMeshComponent))
	{
		Fail(LOCTEXT("InvalidDynamicMeshComponent", "Invalid dynamic mesh component on modified actor"));
		return;
	}

	// Get the original bounds before clearing the mesh
	OriginalMeshBounds = GetMeshBounds();

	const TArray<FTransform> Transforms = ActiveTool->GetTransformInstances(OriginalMeshBounds);
	if (Transforms.IsEmpty())
	{
		Fail(LOCTEXT("InvalidInstanceCount", "Requested instance count is not supported"));
		return;
	}

	const uint32 TriangleCount = TargetMeshComponent->GetDynamicMesh()->GetTriangleCount() * Transforms.Num();
	constexpr uint32 MaxTriangleCount = 10000000;
	if (TriangleCount > MaxTriangleCount)
	{
		Fail(FText::Format(LOCTEXT("HighInstanceCount", "Requested instance count is too high:\n{0} instances totalling in {1} triangles"), FText::AsNumber(Transforms.Num()), FText::AsNumber(TriangleCount)));
		return;
	}

	TargetMeshComponent->GetDynamicMesh()->EditMesh([this, &Transforms](FDynamicMesh3& AppendToMesh)
	{
		// copy original mesh once into tmp mesh
		FDynamicMesh3 TmpMesh = AppendToMesh;

		// clear all since we have a copy
		for (const int32 TId : AppendToMesh.TriangleIndicesItr())
		{
			AppendToMesh.RemoveTriangle(TId);
		}

		FMeshIndexMappings TmpMappings;
		FDynamicMeshEditor Editor(&AppendToMesh);
		for (const FTransform& TransformInstance : Transforms)
		{
			MeshTransforms::ApplyTransform(TmpMesh, TransformInstance, /** ReverseIfNeeded */true);
			Editor.AppendMesh(&TmpMesh, TmpMappings);
			TmpMappings.Reset();
			MeshTransforms::ApplyTransformInverse(TmpMesh, TransformInstance, /** ReverseIfNeeded */true);
		}

		const FVector CenterAxis = ActiveTool->GetCenterAlignmentAxis();

		// center this mesh
		if (!CenterAxis.IsZero())
		{
			const FBox BoundingBox = static_cast<FBox>(AppendToMesh.GetBounds(true));
			if (BoundingBox.IsValid)
			{
				const FVector BoundingCenter = BoundingBox.GetCenter();
				// center only on specified axis
				const FVector Translate = BoundingCenter * CenterAxis;
				MeshTransforms::Translate(AppendToMesh, -Translate);
			}
		}
	}
	, EDynamicMeshChangeType::GeneralEdit
	, EDynamicMeshAttributeChangeFlags::Unknown);

	Next();
}

UAvaPatternModifierTool* UAvaPatternModifier::FindOrAddTool(TSubclassOf<UAvaPatternModifierTool> InToolClass)
{
	if (!InToolClass.Get())
	{
		return nullptr;
	}

	if (const TObjectPtr<UAvaPatternModifierTool>* Tool = Tools.FindByPredicate([InToolClass](const TObjectPtr<UAvaPatternModifierTool>& InTool)
	{
		return InTool && InTool->GetClass() == InToolClass.Get();
	}))
	{
		return *Tool;
	}

	if (const UAvaPatternModifierTool* DefaultObject = InToolClass.GetDefaultObject())
	{
		if (UAvaPatternModifierTool* NewTool = NewObject<UAvaPatternModifierTool>(this, InToolClass.Get(), DefaultObject->GetToolName()))
		{
			Tools.Add(NewTool);
			return NewTool;
		}
	}

	return nullptr;
}

void UAvaPatternModifier::Serialize(FArchive& InArchive)
{
	InArchive.UsingCustomVersion(FAvaPatternModifierVersion::GUID);

	Super::Serialize(InArchive);

	const int32 Version = InArchive.CustomVer(FAvaPatternModifierVersion::GUID);

	if (Version < FAvaPatternModifierVersion::LatestVersion)
	{
		MigrateVersion(Version);
	}
}

#if WITH_EDITOR
void UAvaPatternModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	static const FName ActiveToolClassName = GET_MEMBER_NAME_CHECKED(UAvaPatternModifier, ActiveToolClass);

	if (MemberName.IsEqual(ActiveToolClassName))
	{
		OnActiveToolClassChanged();
	}
}
#endif

UAvaPatternModifier::UAvaPatternModifier()
{
	ActiveToolClass = UAvaPatternModifierLineTool::StaticClass();
}

void UAvaPatternModifier::OnModifierEnabled(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierEnabled(InReason);
	OnActiveToolClassChanged();
}

void UAvaPatternModifier::SetActiveToolClass(const TSubclassOf<UAvaPatternModifierTool>& InClass)
{
	if (!InClass.Get() || ActiveToolClass == InClass)
	{
		return;
	}

	ActiveToolClass = InClass;
	OnActiveToolClassChanged();
}

void UAvaPatternModifier::OnActiveToolClassChanged()
{
	ActiveTool = FindOrAddTool(ActiveToolClass);
	MarkModifierDirty();
}

void UAvaPatternModifier::MigrateVersion(int32 InCurrentVersion)
{
	if (InCurrentVersion >= FAvaPatternModifierVersion::LatestVersion)
	{
		return;
	}

	if (InCurrentVersion < FAvaPatternModifierVersion::MigrateProperties)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS

		if (UAvaPatternModifierLineTool* LineTool = FindOrAddTool<UAvaPatternModifierLineTool>())
		{
			LineTool->LineAxis = LineLayoutOptions.Axis;
			LineTool->LineCount = LineLayoutOptions.RepeatCount;
			LineTool->LineSpacing = LineLayoutOptions.Spacing;
			LineTool->bLineAccumulateTransform = LineLayoutOptions.bAccumulateTransform;
			LineTool->LineRotation = LineLayoutOptions.Rotation;
			LineTool->LineScale = LineLayoutOptions.Scale;

			if (LineLayoutOptions.bCentered)
			{
				LineTool->LineAlignment = EAvaPatternModifierLineAlignment::Center;
			}
			else if (LineLayoutOptions.bAxisInverted)
			{
				LineTool->LineAlignment = EAvaPatternModifierLineAlignment::End;
			}
			else
			{
				LineTool->LineAlignment = EAvaPatternModifierLineAlignment::Start;
			}
		}

		if (UAvaPatternModifierGridTool* GridTool = FindOrAddTool<UAvaPatternModifierGridTool>())
		{
			GridTool->GridPlane = GridLayoutOptions.Plane;
			GridTool->GridCountX = GridLayoutOptions.RepeatCount.X;
			GridTool->GridCountY = GridLayoutOptions.RepeatCount.Y;
			GridTool->GridSpacingX = GridLayoutOptions.Spacing.X;
			GridTool->GridSpacingY = GridLayoutOptions.Spacing.Y;
			GridTool->bGridAccumulateTransform = GridLayoutOptions.bAccumulateTransform;
			GridTool->GridRotation = GridLayoutOptions.Rotation;
			GridTool->GridScale = GridLayoutOptions.Scale;

			if (GridLayoutOptions.bCentered)
			{
				GridTool->GridAlignment = EAvaPatternModifierGridAlignment::Center;
			}
			else if (GridLayoutOptions.AxisInverted.bX && GridLayoutOptions.AxisInverted.bY)
			{
				GridTool->GridAlignment = EAvaPatternModifierGridAlignment::TopRight;
			}
			else if (GridLayoutOptions.AxisInverted.bX && !GridLayoutOptions.AxisInverted.bY)
			{
				GridTool->GridAlignment = EAvaPatternModifierGridAlignment::BottomRight;
			}
			else if (!GridLayoutOptions.AxisInverted.bX && GridLayoutOptions.AxisInverted.bY)
			{
				GridTool->GridAlignment = EAvaPatternModifierGridAlignment::TopLeft;
			}
			else if (!GridLayoutOptions.AxisInverted.bX && !GridLayoutOptions.AxisInverted.bY)
			{
				GridTool->GridAlignment = EAvaPatternModifierGridAlignment::BottomLeft;
			}
		}

		if (UAvaPatternModifierCircleTool* CircleTool = FindOrAddTool<UAvaPatternModifierCircleTool>())
		{
			CircleTool->CirclePlane = CircleLayoutOptions.Plane;
			CircleTool->CircleRadius = CircleLayoutOptions.Radius;
			CircleTool->CircleStartAngle = CircleLayoutOptions.StartAngle;
			CircleTool->CircleFullAngle = CircleLayoutOptions.FullAngle;
			CircleTool->CircleCount = CircleLayoutOptions.RepeatCount;
			CircleTool->bCircleAccumulateTransform = CircleLayoutOptions.bAccumulateTransform;
			CircleTool->CircleRotation = CircleLayoutOptions.Rotation;
			CircleTool->CircleScale = CircleLayoutOptions.Scale;
		}

		if (Layout == EAvaPatternModifierLayout::Line)
		{
			ActiveToolClass = UAvaPatternModifierLineTool::StaticClass();
		}
		else if (Layout == EAvaPatternModifierLayout::Grid)
		{
			ActiveToolClass = UAvaPatternModifierGridTool::StaticClass();
		}
		else if (Layout == EAvaPatternModifierLayout::Circle)
		{
			ActiveToolClass = UAvaPatternModifierCircleTool::StaticClass();
		}

		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

#undef LOCTEXT_NAMESPACE
