// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaPlaneCutModifier.h"

#include "ConstrainedDelaunay2.h"
#include "Components/DynamicMeshComponent.h"
#include "CuttingOps/PlaneCutOp.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "Operations/MeshPlaneCut.h"

#define LOCTEXT_NAMESPACE "AvaPlaneCutModifier"

FVector UAvaPlaneCutModifier::GetPlaneLocation() const
{
	const AActor* ActorModified = GetModifiedActor();
	if (!PreModifierCachedBounds.IsSet() || !ActorModified)
	{
		return FVector::ZeroVector;
	}

	// Restrict plane cut to actual bounds from center of bounds to extent
	const FBox& PreModifierBounds = PreModifierCachedBounds.GetValue();
	const FVector PreModifierCenter = PreModifierBounds.GetCenter();
	const FVector PreModifierExtent = PreModifierBounds.GetExtent();
	
	const FVector PlaneNormal = PlaneRotation.RotateVector(FVector::ZAxisVector);
	const FVector PlaneLocation = PreModifierCenter + PlaneNormal * PlaneOrigin;
	
	return ClampVector(PlaneLocation, PreModifierCenter - PreModifierExtent, PreModifierCenter + PreModifierExtent);
}

void UAvaPlaneCutModifier::OnPlaneRotationChanged()
{
#if WITH_EDITOR
	UpdatePreviewComponent();
#endif
	MarkModifierDirty();
}

void UAvaPlaneCutModifier::OnFillHolesChanged()
{
	MarkModifierDirty();
}

void UAvaPlaneCutModifier::OnInvertCutChanged()
{
	MarkModifierDirty();
}

void UAvaPlaneCutModifier::OnPlaneOriginChanged()
{
#if WITH_EDITOR
	UpdatePreviewComponent();
#endif
	MarkModifierDirty();
}

#if WITH_EDITOR
void UAvaPlaneCutModifier::OnUsePreviewChanged()
{
	if (!PreModifierCachedBounds.IsSet())
	{
		MarkModifierDirty();
	}
	else
	{
		UpdatePreviewComponent();
	}
}

void UAvaPlaneCutModifier::CreatePreviewComponent()
{
	if (UDynamicMeshComponent* const DynMeshComponent = GetMeshComponent())
	{
		PreviewPlane.Create(DynMeshComponent);
	}
}

void UAvaPlaneCutModifier::DestroyPreviewComponent()
{
	PreviewPlane.Destroy();
}

void UAvaPlaneCutModifier::UpdatePreviewComponent()
{
	const AActor* ActorModified = GetModifiedActor();
	if (!PreModifierCachedBounds.IsSet() || !ActorModified)
	{
		return;
	}

	const FVector PlaneLocation = GetPlaneLocation();
	
	// Divide bounds by 100 since it's the mesh size and multiply by actor scale
	const FBox& PreModifierBounds = PreModifierCachedBounds.GetValue();
	const FVector PlaneMeshSize = ActorModified->GetActorScale3D() * (PreModifierBounds.TransformBy(FTransform(PlaneRotation)).GetSize() / 100.f);

	PreviewPlane.Update(FTransform(
		PlaneRotation,
		PlaneLocation + PlaneRotation.RotateVector(FVector::UpVector) * 1,
		FVector::Max(PlaneMeshSize, FVector(0.1f, 0.1f, 0.1f)))
	);
	
	if (bUsePreview && IsModifierEnabled())
	{
		PreviewPlane.Show();
	}
	else
	{
		PreviewPlane.Hide();
	}
}

void UAvaPlaneCutModifier::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();
	
	static const FName PlaneCutOriginName = GET_MEMBER_NAME_CHECKED(UAvaPlaneCutModifier, PlaneOrigin);
	static const FName PlaneRotationName = GET_MEMBER_NAME_CHECKED(UAvaPlaneCutModifier, PlaneRotation);
	static const FName FillHolesName = GET_MEMBER_NAME_CHECKED(UAvaPlaneCutModifier, bFillHoles);
	static const FName InvertCutName = GET_MEMBER_NAME_CHECKED(UAvaPlaneCutModifier, bInvertCut);
	static const FName UsePreviewName = GET_MEMBER_NAME_CHECKED(UAvaPlaneCutModifier, bUsePreview);
	
	if (MemberName == PlaneCutOriginName)
	{
		OnPlaneOriginChanged();
	}
	else if (MemberName == PlaneRotationName)
	{
		OnPlaneRotationChanged();
	}
	else if (MemberName == FillHolesName)
	{
		OnFillHolesChanged();
	}
	else if (MemberName == InvertCutName)
	{
		OnInvertCutChanged();
	}
	else if (MemberName == UsePreviewName)
	{
		OnUsePreviewChanged();
	}
}
#endif

void UAvaPlaneCutModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("PlaneCut"));
	InMetadata.SetCategory(TEXT("Geometry"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Plane Cut"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Cuts a shape based on a 2D plane"));
#endif
}

void UAvaPlaneCutModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

#if WITH_EDITOR
	CreatePreviewComponent();
#endif
}

void UAvaPlaneCutModifier::Apply()
{
	UDynamicMeshComponent* const DynMeshComp = GetMeshComponent();
	if (!IsValid(DynMeshComp))
	{
		Fail(LOCTEXT("InvalidDynamicMeshComponent", "Invalid dynamic mesh component on modified actor"));
		return;
	}

	using namespace UE::Geometry;
		
	DynMeshComp->EditMesh([this](FDynamicMesh3& InEditMesh)
	{
		if (InEditMesh.TriangleCount() > 0)
		{
			// Weld in order to fill holes properly otherwise can't determine open boundaries properly
			FMergeCoincidentMeshEdges WeldOp(&InEditMesh);
			WeldOp.Apply();
		}

		const FVector PlaneLocation = GetPlaneLocation();
		const FVector PlaneNormal = PlaneRotation.RotateVector(FVector::ZAxisVector);
			
		FMeshPlaneCut Cut(&InEditMesh, PlaneLocation, PlaneNormal * (bInvertCut ? -1 : 1));
		const float MaxDim = InEditMesh.GetBounds().MaxDim();
		Cut.UVScaleFactor = MaxDim > 0 ? 1.f / MaxDim : 1.f;
		Cut.Cut();

		if (bFillHoles)
		{
			Cut.HoleFill(ConstrainedDelaunayTriangulate<double>, true);
		}
	});

#if WITH_EDITOR
	UpdatePreviewComponent();
#endif
		
	Next();
}

void UAvaPlaneCutModifier::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierDisabled(InReason);

#if WITH_EDITOR
	UpdatePreviewComponent();
#endif
}

void UAvaPlaneCutModifier::OnModifierRemoved(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierRemoved(InReason);

#if WITH_EDITOR
	DestroyPreviewComponent();
#endif
}

void UAvaPlaneCutModifier::SetPlaneOrigin(float InOrigin)
{
	if (InOrigin == PlaneOrigin)
	{
		return;
	}

	PlaneOrigin = InOrigin;
	OnPlaneOriginChanged();
}

void UAvaPlaneCutModifier::SetPlaneRotation(const FRotator& InRotation)
{
	if (InRotation == PlaneRotation)
	{
		return;
	}

	PlaneRotation = InRotation;
	OnPlaneRotationChanged();
}

void UAvaPlaneCutModifier::SetInvertCut(bool bInInvertCut)
{
	if (bInInvertCut == bInvertCut)
	{
		return;
	}

	bInvertCut = bInInvertCut;
	OnInvertCutChanged();
}

void UAvaPlaneCutModifier::SetFillHoles(bool bInFillHoles)
{
	if (bInFillHoles == bFillHoles)
	{
		return;
	}

	bFillHoles = bInFillHoles;
	OnFillHolesChanged();
}

#undef LOCTEXT_NAMESPACE
