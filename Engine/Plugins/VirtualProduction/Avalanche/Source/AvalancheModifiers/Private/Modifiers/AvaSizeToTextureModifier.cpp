// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaSizeToTextureModifier.h"

#include "AvaShapeActor.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMeshes/AvaShape2DDynMeshBase.h"
#include "Engine/Texture.h"

#define LOCTEXT_NAMESPACE "AvaTextureSizeModifier"

#if WITH_EDITOR
const TAvaPropertyChangeDispatcher<UAvaSizeToTextureModifier> UAvaSizeToTextureModifier::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UAvaSizeToTextureModifier, Texture), &UAvaSizeToTextureModifier::OnTextureOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaSizeToTextureModifier, Rule), &UAvaSizeToTextureModifier::OnTextureOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaSizeToTextureModifier, FixedHeight), &UAvaSizeToTextureModifier::OnTextureOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaSizeToTextureModifier, FixedWidth), &UAvaSizeToTextureModifier::OnTextureOptionsChanged },
};

void UAvaSizeToTextureModifier::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UAvaSizeToTextureModifier::SetTexture(UTexture* InTexture)
{
	if (InTexture == Texture)
	{
		return;
	}

	Texture = InTexture;
	OnTextureOptionsChanged();
}

void UAvaSizeToTextureModifier::SetRule(EAvaSizeToTextureRule InRule)
{
	if (Rule == InRule)
	{
		return;
	}

	Rule = InRule;
	OnTextureOptionsChanged();
}

void UAvaSizeToTextureModifier::SetFixedHeight(float InFixedHeight)
{
	InFixedHeight = FMath::Max(0, InFixedHeight);
	if (FMath::IsNearlyEqual(FixedHeight, InFixedHeight))
	{
		return;
	}

	FixedHeight = InFixedHeight;
	OnTextureOptionsChanged();
}

void UAvaSizeToTextureModifier::SetFixedWidth(float InFixedWidth)
{
	InFixedWidth = FMath::Max(0, InFixedWidth);
	if (FMath::IsNearlyEqual(FixedWidth, InFixedWidth))
	{
		return;
	}

	FixedWidth = InFixedWidth;
	OnTextureOptionsChanged();
}

void UAvaSizeToTextureModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("SizeToTexture"));
	InMetadata.SetCategory(TEXT("Geometry"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Size To Texture"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "The modified actor will be resized to match a texture size based on the provided rule"));
#endif

	InMetadata.SetCompatibilityRule([](const AActor* InActor)->bool
	{
		bool bSupported = false;
		if (InActor)
		{
			if (const UDynamicMeshComponent* DynMeshComponent = InActor->FindComponentByClass<UDynamicMeshComponent>())
			{
				DynMeshComponent->ProcessMesh([&bSupported](const FDynamicMesh3& InProcessMesh)
				{
					// Only flat meshes are supported => no depth => (x == 0)
					bSupported = InProcessMesh.VertexCount() > 0 && FMath::IsNearlyZero((InProcessMesh.GetBounds(true).Width()));
				});
			}
		}
		return bSupported;
	});
}

void UAvaSizeToTextureModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

	AddExtension<FActorModifierRenderStateUpdateExtension>(this);
}

void UAvaSizeToTextureModifier::OnModifierEnabled(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierEnabled(InReason);

	if (InReason == EActorModifierCoreEnableReason::User)
	{
		if (const AAvaShapeActor* const ShapeActor = Cast<AAvaShapeActor>(GetModifiedActor()))
		{
			if (UAvaShape2DDynMeshBase* const Shape2D = Cast<UAvaShape2DDynMeshBase>(ShapeActor->GetDynamicMesh()))
			{
				Shape2DWeak = Shape2D;
				PreModifierShape2DSize = Shape2D->GetSize2D();
			}
		}
	}
}

void UAvaSizeToTextureModifier::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierDisabled(InReason);

	if (UAvaShape2DDynMeshBase* const Shape2D = Shape2DWeak.Get())
	{
		Shape2D->SetSize2D(PreModifierShape2DSize);
	}
}

void UAvaSizeToTextureModifier::Apply()
{
	if (!IsValid(Texture))
	{
		Next();
		return;
	}

	const float Width = Texture->GetSurfaceWidth();
	const float Height = Texture->GetSurfaceHeight();

	if (FMath::IsNearlyZero(Width) || FMath::IsNearlyZero(Height))
	{
		Fail(LOCTEXT("InvalidTextureSize", "Invalid Texture Size"));
		return;
	}

	if (Rule == EAvaSizeToTextureRule::FixedHeight && FMath::IsNearlyZero(FixedHeight))
	{
		Fail(LOCTEXT("InvalidFixedHeight", "Invalid Fixed Height"));
		return;
	}

	if (Rule == EAvaSizeToTextureRule::FixedWidth && FMath::IsNearlyZero(FixedWidth))
	{
		Fail(LOCTEXT("InvalidFixedWidth", "Invalid Fixed Width"));
		return;
	}

	UDynamicMeshComponent* const DynMeshComponent = GetMeshComponent();
	if (!IsValid(DynMeshComponent))
	{
		Fail(LOCTEXT("InvalidDynamicMeshComponent", "Invalid dynamic mesh component on modified actor"));
		return;
	}
	
	FVector2D ShapeSize;
	FVector2D ShapeScale;
	GetShapeSizeAndScale(ShapeSize, ShapeScale);

	if (ShapeSize.IsNearlyZero() || ShapeScale.IsNearlyZero())
	{
		Fail(LOCTEXT("InvalidShapeSizeOrScale", "Invalid Shape Size or Scale"));
		return;
	}

	// Scaled shape size
	ShapeSize = ShapeSize * ShapeScale;

	const float WidthHeightRatio = Width / Height;
	const float HeightWidthRatio = Height / Width;
	
	// Check if size is already correct and compute new size
	FVector2D NewSize = FVector2D::ZeroVector;
	if (Rule == EAvaSizeToTextureRule::FixedHeight)
	{
		if (FMath::IsNearlyEqual(ShapeSize.Y, FixedHeight) && FMath::IsNearlyEqual(ShapeSize.X, FixedHeight * WidthHeightRatio))
		{
			Next();
			return;
		}

		NewSize = FVector2D(FixedHeight * WidthHeightRatio, FixedHeight);
	}
	else if (Rule == EAvaSizeToTextureRule::FixedWidth)
	{
		if (FMath::IsNearlyEqual(ShapeSize.X, FixedWidth) && FMath::IsNearlyEqual(ShapeSize.Y, FixedWidth * HeightWidthRatio))
		{
			Next();
			return;
		}
		
		NewSize = FVector2D(FixedWidth, FixedWidth * HeightWidthRatio);
	}
	else if (Rule == EAvaSizeToTextureRule::AdaptiveWidth)
	{
		if (FMath::IsNearlyEqual(ShapeSize.X, ShapeSize.Y * WidthHeightRatio))
		{
			Next();
			return;
		}

		NewSize = FVector2D(ShapeSize.Y * WidthHeightRatio, ShapeSize.Y);
	}
	else if (Rule == EAvaSizeToTextureRule::AdaptiveHeight)
	{
		if (FMath::IsNearlyEqual(ShapeSize.Y, ShapeSize.X * HeightWidthRatio))
		{
			Next();
			return;
		}

		NewSize = FVector2D(ShapeSize.X, ShapeSize.X * HeightWidthRatio);
	}

	if (NewSize.IsNearlyZero())
	{
		Fail(LOCTEXT("InvalidNewSize", "Invalid New Size for Shape"));
		return;
	}
	
	CachedScale = ShapeScale;
	CachedSize = NewSize;

	if (UAvaShape2DDynMeshBase* const Shape2D = Shape2DWeak.Get())
	{
		const FVector2D NewUnscaledShapeSize = CachedSize / ShapeScale;
		if (!Shape2D->GetSize2D().Equals(NewUnscaledShapeSize, 0.01))
		{
			Shape2D->SetSize2D(NewUnscaledShapeSize);

			// Refresh UV after changing size, these calls are async so size and UV update will be batched together
			Shape2D->SetMaterialUVParams(UAvaShape2DDynMeshBase::MESH_INDEX_PRIMARY, Shape2D->GetMaterialUVParams(UAvaShape2DDynMeshBase::MESH_INDEX_PRIMARY));
		}
	}
	else
	{
		const FVector NewUnscaledShapeSize = FVector(1, CachedSize.X / ShapeSize.X, CachedSize.Y / ShapeSize.Y);
		DynMeshComponent->GetDynamicMesh()->EditMesh([this, &NewUnscaledShapeSize](FDynamicMesh3& InEditMesh)
		{
			MeshTransforms::Scale(InEditMesh, NewUnscaledShapeSize, FVector::Zero(), /** ReverseOrientation */true);
		}
		, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
	}

	Next();
}

void UAvaSizeToTextureModifier::OnModifiedActorTransformed()
{
	Super::OnModifiedActorTransformed();

	CheckSizeOrScaleChanged();
}

void UAvaSizeToTextureModifier::SavePreState()
{
	if (UAvaShape2DDynMeshBase* const Shape2D = Shape2DWeak.Get())
	{
		// Skip saving pre-state for Motion Design 2D Shapes as the mesh isn't dealt with directly in Apply.
		// @see UAvaSizeToTextureModifier::RestorePreState for more details
	}
	else
	{
		Super::SavePreState();
	}
}

void UAvaSizeToTextureModifier::RestorePreState()
{
	if (UAvaShape2DDynMeshBase* const Shape2D = Shape2DWeak.Get())
	{
		// Don't restore the mesh when dealing with Motion Design Shapes.
		// Apply for 2D shapes only modifies the Size2D of the shape, not the mesh directly.
		// Size2D isn't saved/restore either as this modifier doesn't rely on the Size2D previous state. It only restores back the original size when disabled.
	}
	else
	{
		Super::RestorePreState();
	}
}

void UAvaSizeToTextureModifier::OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent)
{
	if (InComponent == GetMeshComponent())
	{
		CheckSizeOrScaleChanged();
	}
}

void UAvaSizeToTextureModifier::OnActorVisibilityChanged(AActor* InActor)
{
	// Do nothing
}

void UAvaSizeToTextureModifier::OnTextureOptionsChanged()
{
	MarkModifierDirty();
}

void UAvaSizeToTextureModifier::GetShapeSizeAndScale(FVector2D& OutShapeSize, FVector2D& OutShapeScale) const
{
	OutShapeSize = FVector2D(0, 0);
	OutShapeScale = FVector2D(0, 0);

	if (const UDynamicMeshComponent* const DynMeshComponent = GetMeshComponent())
	{
		DynMeshComponent->ProcessMesh([&OutShapeSize](const FDynamicMesh3& InProcessMesh)
		{
			const UE::Geometry::FAxisAlignedBox3d AlignedBox = InProcessMesh.GetBounds(true);
			OutShapeSize = FVector2D(AlignedBox.Height(), AlignedBox.Depth());
		});

		OutShapeScale = FVector2D(DynMeshComponent->GetComponentScale().Y, DynMeshComponent->GetComponentScale().Z);
	}
}

void UAvaSizeToTextureModifier::CheckSizeOrScaleChanged()
{
	FVector2D ShapeSize;
	FVector2D ShapeScale;
	GetShapeSizeAndScale(ShapeSize, ShapeScale);

	ShapeSize = ShapeSize * ShapeScale;

	if (!CachedSize.Equals(ShapeSize))
	{
		MarkModifierDirty();	
	}
}

#undef LOCTEXT_NAMESPACE
