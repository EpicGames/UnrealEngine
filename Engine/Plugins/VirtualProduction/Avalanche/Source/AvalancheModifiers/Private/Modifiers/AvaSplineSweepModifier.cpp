// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaSplineSweepModifier.h"

#include "MeshBoundaryLoops.h"
#include "Components/DynamicMeshComponent.h"
#include "Components/SplineComponent.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "GeometryScript/MeshPrimitiveFunctions.h"
#include "GeometryScript/PolyPathFunctions.h"
#include "Operations/InsetMeshRegion.h"

#define LOCTEXT_NAMESPACE "AvaSplineSweepModifier"

#if WITH_EDITOR
const TAvaPropertyChangeDispatcher<UAvaSplineSweepModifier> UAvaSplineSweepModifier::PropertyChangeDispatcher =
{
	{ GET_MEMBER_NAME_CHECKED(UAvaSplineSweepModifier, SampleMode), &UAvaSplineSweepModifier::OnSplineOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaSplineSweepModifier, SampleDistance), &UAvaSplineSweepModifier::OnSplineOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaSplineSweepModifier, Steps), &UAvaSplineSweepModifier::OnSplineOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaSplineSweepModifier, ProgressOffset), &UAvaSplineSweepModifier::OnSplineOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaSplineSweepModifier, ProgressStart), &UAvaSplineSweepModifier::OnSplineOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaSplineSweepModifier, ProgressEnd), &UAvaSplineSweepModifier::OnSplineOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaSplineSweepModifier, ScaleStart), &UAvaSplineSweepModifier::OnModifierOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaSplineSweepModifier, ScaleEnd), &UAvaSplineSweepModifier::OnModifierOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaSplineSweepModifier, bCapped), &UAvaSplineSweepModifier::OnModifierOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaSplineSweepModifier, bLooped), &UAvaSplineSweepModifier::OnModifierOptionsChanged },
	{ GET_MEMBER_NAME_CHECKED(UAvaSplineSweepModifier, SplineActorWeak), &UAvaSplineSweepModifier::OnSplineActorWeakChanged },
};

void UAvaSplineSweepModifier::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	PropertyChangeDispatcher.OnPropertyChanged(this, InPropertyChangedEvent);
}
#endif

void UAvaSplineSweepModifier::SetSteps(int32 InSteps)
{
	InSteps = FMath::Clamp(InSteps, 3, MaxSampleCount);
	if (Steps == InSteps)
	{
		return;
	}

	Steps = InSteps;
	OnSplineOptionsChanged();
}

void UAvaSplineSweepModifier::SetProgressOffset(float InOffset)
{
	if (FMath::IsNearlyEqual(ProgressOffset, InOffset))
	{
		return;
	}

	ProgressOffset = InOffset;
	OnSplineOptionsChanged();
}

void UAvaSplineSweepModifier::SetProgressStart(float InStart)
{
	if (FMath::IsNearlyEqual(ProgressStart, InStart))
	{
		return;
	}

	if (InStart < 0.f || InStart > 1.f)
	{
		return;
	}

	ProgressStart = InStart;
	OnSplineOptionsChanged();
}

void UAvaSplineSweepModifier::SetProgressEnd(float InEnd)
{
	if (FMath::IsNearlyEqual(ProgressEnd, InEnd))
	{
		return;
	}

	if (InEnd < 0.f || InEnd > 1.f)
	{
		return;
	}

	ProgressEnd = InEnd;
	OnSplineOptionsChanged();
}

void UAvaSplineSweepModifier::SetScaleStart(float InScaleStart)
{
	if (FMath::IsNearlyEqual(ScaleStart, InScaleStart))
	{
		return;
	}

	if (InScaleStart < 0.f)
	{
		return;
	}

	ScaleStart = InScaleStart;
	MarkModifierDirty();
}

void UAvaSplineSweepModifier::SetScaleEnd(float InScaleEnd)
{
	if (FMath::IsNearlyEqual(ScaleEnd, InScaleEnd))
	{
		return;
	}

	if (InScaleEnd < 0.f)
	{
		return;
	}

	ScaleEnd = InScaleEnd;
	MarkModifierDirty();
}

void UAvaSplineSweepModifier::SetCapped(bool bInCapped)
{
	if (bCapped == bInCapped)
	{
		return;
	}

	bCapped = bInCapped;
	MarkModifierDirty();
}

void UAvaSplineSweepModifier::SetLooped(bool bInLooped)
{
	if (bLooped == bInLooped)
	{
		return;
	}

	bLooped = bInLooped;
	MarkModifierDirty();
}

void UAvaSplineSweepModifier::SetSplineActorWeak(TWeakObjectPtr<AActor> InSplineActorWeak)
{
	if (SplineActorWeak == InSplineActorWeak)
	{
		return;
	}

	SplineActorWeak = InSplineActorWeak;
	OnSplineActorWeakChanged();
}

void UAvaSplineSweepModifier::SetSampleMode(EAvaSplineSweepSampleMode InMode)
{
	if (SampleMode == InMode)
	{
		return;
	}

	SampleMode = InMode;
	OnSplineOptionsChanged();
}

void UAvaSplineSweepModifier::SetSampleDistance(float InDistance)
{
	if (FMath::IsNearlyEqual(SampleDistance, InDistance))
	{
		return;
	}

	if (SampleDistance < 1.f)
	{
		return;
	}

	SampleDistance = InDistance;
	OnSplineOptionsChanged();
}

void UAvaSplineSweepModifier::OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent)
{
	if (InActor == SplineActorWeak.Get())
	{
		OnSplineOptionsChanged();
	}
}

void UAvaSplineSweepModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("SplineSweep"));
	InMetadata.SetCategory(TEXT("Geometry"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Spline Sweep"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Samples a spline path and creates a 3D geometry based on a 2D shape slice"));
#endif

	InMetadata.SetCompatibilityRule([](const AActor* InActor)->bool
	{
		bool bSupported = false;

		if (InActor)
		{
			if (const UDynamicMeshComponent* DynMeshComponent = InActor->FindComponentByClass<UDynamicMeshComponent>())
			{
				DynMeshComponent->ProcessMesh([&bSupported](const FDynamicMesh3& InProcessMesh)
				{
					bSupported = InProcessMesh.VertexCount() > 0 && FMath::IsNearlyEqual(static_cast<FBox>(InProcessMesh.GetBounds(true)).GetSize().X, 0);
				});
			}
		}

		return bSupported;
	});
}

void UAvaSplineSweepModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

	AddExtension<FActorModifierRenderStateUpdateExtension>(this);
}

void UAvaSplineSweepModifier::Apply()
{
	const AActor* ActorModified = GetModifiedActor();
	if (!ActorModified)
	{
		return;
	}

	const USplineComponent* SplineComponent = SplineComponentWeak.Get();
	UDynamicMesh* TargetMesh = GetMeshObject();

	if (!SplineComponent || !TargetMesh)
	{
		Next();
		return;
	}

	const bool bInvalidProgress = ProgressStart > ProgressEnd;
	const bool bOutOfRange = !SplineComponent->IsClosedLoop()
		&& (ProgressStart + ProgressOffset > 1.f || ProgressEnd + ProgressOffset < 0.f);

	if (bOutOfRange || bInvalidProgress)
	{
		// Clear mesh
		TargetMesh->EditMesh([](FDynamicMesh3& InMesh)
		{
			for (const int32 TId : InMesh.TriangleIndicesItr())
			{
				InMesh.RemoveTriangle(TId);
			}
		});

		Next();
		return;
	}

	if (Steps != SplineSamples.Num())
	{
		SampleSpline();
	}

	// Samples needed should be >= 3 to avoid any issue during AppendSweepPolygon operation
	if (SplineSamples.Num() < 3)
	{
		Next();
		return;
	}

	TArray<FVector2D> PolygonVertices;
	TargetMesh->EditMesh([&PolygonVertices](FDynamicMesh3& InMesh)
	{
		// Weld edges
		if (InMesh.TriangleCount() > 0)
		{
			UE::Geometry::FMergeCoincidentMeshEdges WeldOp(&InMesh);
			WeldOp.Apply();
		}

		// Add border vertices
		for (const int32 VId : InMesh.VertexIndicesItr())
		{
			if (InMesh.IsBoundaryVertex(VId))
			{
				const FVector3d Vertex = InMesh.GetVertex(VId);
				PolygonVertices.AddUnique(FVector2D(Vertex.Y, Vertex.Z));
			}
		}

		// Clear mesh
		for (const int32 TId : InMesh.TriangleIndicesItr())
		{
			InMesh.RemoveTriangle(TId);
		}
	});

	if (PolygonVertices.Num() < 3)
	{
		Next();
		return;
	}

	// Calculate the centroid of the vertices
    FVector2D Centroid(0, 0);
    for (const FVector2D& Vertex : PolygonVertices)
    {
        Centroid += Vertex;
    }
    Centroid /= PolygonVertices.Num();

    // Sort the array counter clock wise
    PolygonVertices.Sort([Centroid](const FVector2D& A, const FVector2D& B)
    {
	    const FVector2D VectorA = A - Centroid;
	    const FVector2D VectorB = B - Centroid;

	    const float AngleA = FMath::Atan2(VectorA.Y, VectorA.X);
	    const float AngleB = FMath::Atan2(VectorB.Y, VectorB.X);

		return AngleA < AngleB;
    });

	static constexpr FGeometryScriptPrimitiveOptions PrimitiveOptions;
	const bool bLoopSweep = (SplineComponent->IsClosedLoop() || bLooped) && ProgressStart == 0.f && ProgressEnd == 1.f;
	UGeometryScriptLibrary_MeshPrimitiveFunctions::AppendSweepPolygon(TargetMesh, PrimitiveOptions, FTransform::Identity, PolygonVertices, SplineSamples, bLoopSweep, bCapped, ScaleStart, ScaleEnd);

	Next();
}

void UAvaSplineSweepModifier::OnModifierOptionsChanged()
{
	MarkModifierDirty();
}

void UAvaSplineSweepModifier::OnSplineActorWeakChanged()
{
	const AActor* SplineActor = SplineActorWeak.Get();

	if (!SplineActor)
	{
		return;
	}

	USplineComponent* SplineComponent = SplineActor->FindComponentByClass<USplineComponent>();

	// Don't allow actors without spline component
	if (!SplineComponent)
	{
		SplineActorWeak.Reset();
	}

	// Don't update if we already track this component
	if (SplineComponentWeak == SplineComponent)
	{
		return;
	}

	SplineComponentWeak = SplineComponent;
	OnSplineOptionsChanged();
}

void UAvaSplineSweepModifier::OnSplineOptionsChanged()
{
	if (!SampleSpline())
	{
		return;
	}

	MarkModifierDirty();
}

bool UAvaSplineSweepModifier::SampleSpline()
{
	// We sample spline here and not in the modifier apply to avoid unnecessary computations
	const USplineComponent* SplineComponent = SplineComponentWeak.Get();

	if (!SplineComponent)
	{
		return false;
	}

	const float SplineLength = SplineComponent->GetSplineLength();
	const float RangeOffset = ProgressOffset * SplineLength;
	const int32 SplineSampleCount = SampleMode == EAvaSplineSweepSampleMode::CustomDistance
		? SplineLength/SampleDistance * Steps
		: Steps;

	TArray<double> SplineFrameTimes;
	FGeometryScriptSplineSamplingOptions SampleOptions;
	SampleOptions.NumSamples = FMath::Min(MaxSampleCount, SplineSampleCount);
	SampleOptions.SampleSpacing = EGeometryScriptSampleSpacing::UniformDistance;
	SampleOptions.RangeMethod = EGeometryScriptEvaluateSplineRange::DistanceRange;
	SampleOptions.RangeStart = RangeOffset + ProgressStart * SplineLength;
	SampleOptions.RangeEnd = RangeOffset + ProgressEnd * SplineLength;

	return UGeometryScriptLibrary_PolyPathFunctions::SampleSplineToTransforms(SplineComponent, SplineSamples, SplineFrameTimes, SampleOptions, FTransform::Identity, false);
}

#undef LOCTEXT_NAMESPACE
