// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaSubdivideModifier.h"

#include "Async/Async.h"
#include "Components/DynamicMeshComponent.h"
#include "GeometryScript/GeometryScriptSelectionTypes.h"
#include "Operations/PNTriangles.h"
#include "Operations/SelectiveTessellate.h"
#include "Operations/UniformTessellate.h"

#define LOCTEXT_NAMESPACE "AvaSubdivideModifier"

void UAvaSubdivideModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("Subdivide"));
	InMetadata.SetCategory(TEXT("Geometry"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Subdivide"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Adds resolution to the geometry shape by dividing faces or edges into smaller units"));
#endif
}

void UAvaSubdivideModifier::Apply()
{
	UDynamicMeshComponent* const DynMeshComp = GetMeshComponent();
	if (!IsValid(DynMeshComp))
	{
		Fail(LOCTEXT("InvalidDynamicMeshComponent", "Invalid dynamic mesh component on modified actor"));
		return;
	}
	
	using namespace UE::Geometry;

	bool bSuccess = true;
			
	switch (Type)
	{
		case EAvaSubdivisionType::Selective:
		{
			DynMeshComp->GetDynamicMesh()->EditMesh([this, &bSuccess](FDynamicMesh3& EditMesh) 
			{	
				FDynamicMesh3 TessellatedMesh;
				FSelectiveTessellate SelectiveTessellateOp(&EditMesh, &TessellatedMesh);
				const FGeometryScriptMeshSelection MeshSelection;
				
				TArray<int32> Triangles;
				MeshSelection.ConvertToMeshIndexArray(EditMesh, Triangles, EGeometryScriptIndexType::Triangle);

				TUniquePtr<FTessellationPattern> Pattern;
				if (Triangles.Num() > 0)
				{
					Pattern = FSelectiveTessellate::CreateConcentricRingsTessellationPattern(&EditMesh, Cuts, Triangles);
				}
				else
				{
					Pattern = FSelectiveTessellate::CreateConcentricRingsTessellationPattern(&EditMesh, Cuts);
				}
	
				SelectiveTessellateOp.SetPattern(Pattern.Get());
				SelectiveTessellateOp.bUseParallel = true; // enable multithreading

				bSuccess = SelectiveTessellateOp.Validate() == EOperationValidationResult::Ok && SelectiveTessellateOp.Compute();

				if (bSuccess)
				{
					EditMesh = MoveTemp(TessellatedMesh);
				}
			}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
		}
		break;
		
		case EAvaSubdivisionType::Uniform:
		DynMeshComp->GetDynamicMesh()->EditMesh([this, &bSuccess](FDynamicMesh3& EditMesh) 
		{
			FUniformTessellate TessellateOperator(&EditMesh);
			TessellateOperator.TessellationNum = Cuts;

			bSuccess = TessellateOperator.Validate() == EOperationValidationResult::Ok && TessellateOperator.Compute();
		}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
		break;
		
		case EAvaSubdivisionType::PN:
		{
			DynMeshComp->GetDynamicMesh()->EditMesh([this, &bSuccess](FDynamicMesh3& EditMesh) 
			{
				FPNTriangles PNTessellateOperator(&EditMesh);
				PNTessellateOperator.TessellationLevel = Cuts;
				PNTessellateOperator.bRecalculateNormals = bRecomputeNormals;

				bSuccess = PNTessellateOperator.Validate() == EOperationValidationResult::Ok && PNTessellateOperator.Compute();
			}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);
		}
		break;
		
		default:;
	}

	if (bSuccess)
	{
		Next();
	}
	else
	{
		Fail(LOCTEXT("TesselationOperationFail", "Tesselation operation failed"));
	}
}

#if WITH_EDITOR
void UAvaSubdivideModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();
	const bool bAnyPropertyChanged = MemberName.IsNone();

	static const FName CutsName = GET_MEMBER_NAME_CHECKED(UAvaSubdivideModifier, Cuts);
	static const FName RecomputeNormalsName = GET_MEMBER_NAME_CHECKED(UAvaSubdivideModifier, bRecomputeNormals);
	static const FName TypeName = GET_MEMBER_NAME_CHECKED(UAvaSubdivideModifier, Type);

	if (bAnyPropertyChanged || MemberName == CutsName)
	{
		SetCuts(Cuts);
	}

	if (bAnyPropertyChanged ||
		MemberName == CutsName ||
		MemberName == RecomputeNormalsName ||
		MemberName == TypeName)
	{
		OnOptionsChanged();
	}
}
#endif

void UAvaSubdivideModifier::SetCuts(int32 InCuts)
{
	InCuts = FMath::Clamp(InCuts, UAvaSubdivideModifier::MinSubdivideCuts, UAvaSubdivideModifier::MaxSubdivideCuts);
	if (Cuts == InCuts)
	{
		return;
	}

	Cuts = InCuts;
	OnOptionsChanged();
}

void UAvaSubdivideModifier::SetRecomputeNormals(bool bInRecomputeNormals)
{
	if (bRecomputeNormals == bInRecomputeNormals)
	{
		return;
	}

	bRecomputeNormals = bInRecomputeNormals;
	OnOptionsChanged();
}

void UAvaSubdivideModifier::SetType(EAvaSubdivisionType InType)
{
	if (Type == InType)
	{
		return;
	}

	Type = InType;
	OnOptionsChanged();
}

void UAvaSubdivideModifier::OnOptionsChanged()
{
	MarkModifierDirty();
}

#undef LOCTEXT_NAMESPACE
