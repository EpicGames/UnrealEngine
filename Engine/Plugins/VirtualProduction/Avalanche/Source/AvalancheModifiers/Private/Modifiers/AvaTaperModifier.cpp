// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaTaperModifier.h"

#include "Async/Async.h"
#include "Components/DynamicMeshComponent.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/AvaSubdivideModifier.h"
#include "Operations/OffsetMeshRegion.h"
#include "Tools/AvaTaperTool.h"

#define LOCTEXT_NAMESPACE "AvaTaperModifier"

void UAvaTaperModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("Taper"));
	InMetadata.SetCategory(TEXT("Geometry"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Taper"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Moves the vertices location of a geometry shape using a curve"));
#endif

	InMetadata.AddDependency(TEXT("Subdivide"));
}

void UAvaTaperModifier::Apply()
{
	UDynamicMeshComponent* const DynMeshComp = GetMeshComponent();
	if (!IsValid(DynMeshComp))
	{
		Fail(LOCTEXT("InvalidDynamicMeshComponent", "Invalid dynamic mesh component on modified actor"));
		return;
	}
	
	// no need to taper here!
	if (Amount <= 0)
	{
		Next();
		return;
	}

	using namespace UE::Geometry;

	constexpr float MinDepth = 0.001f;
	
	const FBox MeshBounds = GetMeshBounds();

	if (MeshBounds.GetSize().X < MinDepth)
	{
		// cannot apply taper to mesh if there's no depth
		DynMeshComp->EditMesh([MinDepth](FDynamicMesh3& EditMesh)
		{
			FOffsetMeshRegion Extruder(&EditMesh);
			for (int32 Tid : EditMesh.TriangleIndicesItr())
			{
				Extruder.Triangles.Add(Tid);
			}

			const FVector ExtrudeDepth = (-FVector::XAxisVector * MinDepth);

			Extruder.OffsetPositionFunc = [&ExtrudeDepth](const FVector3d& Position, const FVector3d& VertexVector, int VertexID)
			{
				return Position + ExtrudeDepth;
			};

			Extruder.bIsPositiveOffset = true;
			Extruder.UVScaleFactor = 0.01f;
			Extruder.bOffsetFullComponentsAsSolids = false;
			Extruder.Apply();				
		});
	}

	// in case the tool doesn't exist yet
	CreateTaperTool();

	// retrieve the dynamic mesh to be modified
	if (FDynamicMesh3* const SharedMesh = DynMeshComp->GetMesh())
	{
		FAvaTaperSettings TaperSettings;
		TaperSettings.Amount = Amount;
		TaperSettings.Extent = GetRequiredExtent();
		TaperSettings.Offset = GetRequiredOffset();
		TaperSettings.InterpolationType = InterpolationType;
		TaperSettings.ZAxisResolution = GetSubdividersCuts();

		// setup the taper with by passing the mesh and the current settings
		if (TaperTool->Setup(SharedMesh, TaperSettings))
		{
			TUniquePtr<FDynamicMesh3> TaperedMesh = MakeUnique<FDynamicMesh3>();

			// applies the taper, returns true when succeeding
			const bool bTaperSuccess = TaperTool->ApplyTaper(TaperedMesh);

			if (bTaperSuccess && TaperedMesh)
			{
				DynMeshComp->GetDynamicMesh()->EditMesh([&TaperedMesh](FDynamicMesh3& EditMesh)
				{
					EditMesh = MoveTemp(*TaperedMesh);
				}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown);
			}
			
			Next();
		}
	}
}

void UAvaTaperModifier::CreateTaperTool()
{
	if (!TaperTool)
	{
		TaperTool = NewObject<UAvaTaperTool>(this);
	}
}

#if WITH_EDITOR
void UAvaTaperModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();
	
	static const FName AmountName = GET_MEMBER_NAME_CHECKED(UAvaTaperModifier, Amount);
	static const FName InterpolationTypeName = GET_MEMBER_NAME_CHECKED(UAvaTaperModifier, InterpolationType);
	static const FName OffsetName = GET_MEMBER_NAME_CHECKED(UAvaTaperModifier, Offset);
	static const FName ReferenceFrameName = GET_MEMBER_NAME_CHECKED(UAvaTaperModifier, ReferenceFrame);
	static const FName LowerExtentName = GET_MEMBER_NAME_CHECKED(UAvaTaperModifier, LowerExtent);
	static const FName UpperExtentName = GET_MEMBER_NAME_CHECKED(UAvaTaperModifier, UpperExtent);
	static const FName ExtentName = GET_MEMBER_NAME_CHECKED(UAvaTaperModifier, Extent);

	if (MemberName == AmountName ||
		MemberName == InterpolationTypeName ||
		MemberName == OffsetName ||
		MemberName == ReferenceFrameName ||
		MemberName == LowerExtentName ||
		MemberName == UpperExtentName ||
		MemberName == ExtentName)
	{
		OnParameterChanged();
	}
}
#endif

void UAvaTaperModifier::SetAmount(float InAmount)
{
	if (Amount == InAmount)
	{
		return;
	}

	Amount = FMath::Clamp(InAmount, 0.0f, 1.0f);
	OnParameterChanged();
}

void UAvaTaperModifier::SetUpperExtent(float InUpperExtent)
{
	if (UpperExtent == InUpperExtent)
	{
		return;
	}

	UpperExtent = FMath::Clamp(InUpperExtent, 0.0f, 1.0f);
	OnParameterChanged();
}

void UAvaTaperModifier::SetLowerExtent(float InLowerExtent)
{
	if (LowerExtent == InLowerExtent)
	{
		return;
	}

	LowerExtent = FMath::Clamp(InLowerExtent, 0.0f, 1.0f);
	OnParameterChanged();
}

void UAvaTaperModifier::SetExtent(EAvaTaperExtent InExtent)
{
	if (Extent == InExtent)
	{
		return;
	}

	Extent = InExtent;
	OnParameterChanged();
}

void UAvaTaperModifier::SetInterpolationType(EAvaTaperInterpolationType InInterpolationType)
{
	if (InterpolationType == InInterpolationType)
	{
		return;
	}

	InterpolationType = InInterpolationType;
	OnParameterChanged();
}

void UAvaTaperModifier::SetReferenceFrame(EAvaTaperReferenceFrame InReferenceFrame)
{
	if (ReferenceFrame == InReferenceFrame)
	{
		return;
	}

	ReferenceFrame = InReferenceFrame;
	OnParameterChanged();
}

void UAvaTaperModifier::SetResolution(int32 InResolution)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	if (Resolution == InResolution)
	{
		return;
	}

	Resolution = FMath::Clamp(InResolution, UAvaTaperModifier::MinTaperLatticeResolution, UAvaTaperModifier::MaxTaperLatticeResolution);
	OnParameterChanged();

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UAvaTaperModifier::SetOffset(FVector2D InOffset)
{
	if (InOffset == Offset)
	{
		return;
	}

	Offset = InOffset;
	OnParameterChanged();
}

void UAvaTaperModifier::OnParameterChanged()
{
	MarkModifierDirty();
}

FVector2D UAvaTaperModifier::GetRequiredOffset() const
{
	if (ReferenceFrame == EAvaTaperReferenceFrame::Custom)
	{
		return Offset;
	}
	else
	{
		return FVector2D::ZeroVector;
	}
}

FVector2D UAvaTaperModifier::GetRequiredExtent() const
{
	switch (Extent)
	{
		case EAvaTaperExtent::WholeShape:
			return FVector2D(1.0, 1.0);
			
		case EAvaTaperExtent::Custom:
			return FVector2D(LowerExtent, UpperExtent) / 100.0;

		default:
			return FVector2D(1.0, 1.0);
	}	
}


int32 UAvaTaperModifier::GetSubdividersCuts() const
{
	int32 TotalSubdivisions = 0;
	
	if (const UActorModifierCoreStack* Stack = GetModifierStack())
	{
		FActorModifierCoreStackSearchOp SearchOp;
		SearchOp.bSkipStack = true;
		SearchOp.Position = EActorModifierCoreStackPosition::Before;
		SearchOp.PositionContext = const_cast<UAvaTaperModifier*>(this);
		const TArray<UActorModifierCoreBase*> Modifiers = Stack->FindModifiers(UAvaSubdivideModifier::StaticClass(), SearchOp);

		for (const UActorModifierCoreBase* const Modifier : Modifiers)
		{
			if (const UAvaSubdivideModifier* SubdivideModifier = Cast<UAvaSubdivideModifier>(Modifier))
			{
				TotalSubdivisions += SubdivideModifier->GetCuts();
			}
		}
	}

	return TotalSubdivisions;
}

#undef LOCTEXT_NAMESPACE
