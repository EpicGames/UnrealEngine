// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaTranslucentPriorityModifier.h"

#include "Camera/CameraActor.h"
#include "Camera/CameraComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/RendererSettings.h"
#include "EngineUtils.h"
#include "Framework/AvaGameInstance.h"
#include "GameFramework/Actor.h"
#include "Modifiers/ActorModifierCoreBase.h"
#include "Modifiers/ActorModifierRenderStateDirtyEvent.h"
#include "Shared/AvaTranslucentPriorityModifierShared.h"

#if WITH_EDITOR
#include "AvaOutlinerSubsystem.h"
#include "AvaOutlinerUtils.h"
#include "IAvaOutliner.h"
#endif

#define LOCTEXT_NAMESPACE "AvaTranslucentPriorityModifier"

void UAvaTranslucentPriorityModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("TranslucentSortPriority"));
	InMetadata.SetCategory(TEXT("Rendering"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Translucent Sort Priority"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Set the translucency sort priority of a primitive component based on different modes"));
#endif
}

void UAvaTranslucentPriorityModifier::OnModifierAdded(EActorModifierCoreEnableReason InReason)
{
	Super::OnModifierAdded(InReason);

	if (InReason == EActorModifierCoreEnableReason::User)
	{
		CameraActorWeak = GetDefaultCameraActor();
		Mode = EAvaTranslucentPriorityModifierMode::AutoOutlinerTreeInverted;
	}

	if (UAvaTranslucentPriorityModifierShared* SharedObject = GetShared<UAvaTranslucentPriorityModifierShared>(true))
	{
		SortPriorityOffset = SharedObject->GetSortPriorityOffset();
		SortPriorityStep = SharedObject->GetSortPriorityStep();
		SharedObject->OnLevelGlobalsChangedDelegate.AddUObject(this, &UAvaTranslucentPriorityModifier::OnGlobalSortPriorityOffsetChanged);
	}
}

void UAvaTranslucentPriorityModifier::OnModifierRemoved(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierRemoved(InReason);

	if (UAvaTranslucentPriorityModifierShared* SharedObject = GetShared<UAvaTranslucentPriorityModifierShared>(false))
	{
		SharedObject->OnLevelGlobalsChangedDelegate.RemoveAll(this);
	}
}

void UAvaTranslucentPriorityModifier::SavePreState()
{
	Super::SavePreState();

	constexpr bool bCreate = true;
	if (UAvaTranslucentPriorityModifierShared* SharedObject = GetShared<UAvaTranslucentPriorityModifierShared>(bCreate))
	{
		AActor* ActorModified = GetModifiedActor();

		ChildrenActorsWeak.Reset();
		PrimitiveComponentsWeak.Reset();
		ForEachComponent<UPrimitiveComponent>([this, &SharedObject, &ActorModified](UPrimitiveComponent* InComponent)->bool
			{
				// Component is already linked to a modifier
				if (const UAvaTranslucentPriorityModifier* ModifierContext = SharedObject->FindModifierContext(InComponent))
				{
					const AActor* ModifierActor = ModifierContext->GetModifiedActor();

					// Skip if a translucent priority modifier is attached on the component owner since it has priority over us
					// or if the modifier handles children and is attached to the current owning actor
					if (ModifierContext != this
						&& (ModifierActor == InComponent->GetOwner()
						|| (ModifierContext->bIncludeChildren && ModifierActor->IsAttachedTo(ActorModified))))
					{
						return true;
					}
				}

				PrimitiveComponentsWeak.Add(InComponent);
				ChildrenActorsWeak.Add(InComponent->GetOwner());

				return true;
			}
			, EActorModifierCoreComponentType::All
			, bIncludeChildren ? EActorModifierCoreLookup::SelfAndAllChildren : EActorModifierCoreLookup::Self);

		SharedObject->SetComponentsState(this, PrimitiveComponentsWeak);
	}

	if (FActorModifierTransformUpdateExtension* TransformExtension = GetExtension<FActorModifierTransformUpdateExtension>())
	{
		TransformExtension->TrackActors(ChildrenActorsWeak, true);
	}
}

void UAvaTranslucentPriorityModifier::RestorePreState()
{
	Super::RestorePreState();

	if (FActorModifierTransformUpdateExtension* TransformExtension = GetExtension<FActorModifierTransformUpdateExtension>())
	{
		TransformExtension->UntrackActors(ChildrenActorsWeak);
	}

	if (UAvaTranslucentPriorityModifierShared* SharedObject = GetShared<UAvaTranslucentPriorityModifierShared>(false))
	{
		SharedObject->RestoreComponentsState(this, PrimitiveComponentsWeak, false);
	}
}

void UAvaTranslucentPriorityModifier::Apply()
{
	const URendererSettings* RendererSettings = GetDefault<URendererSettings>();

	if (RendererSettings && RendererSettings->bOrderedIndependentTransparencyEnable)
	{
		Fail(LOCTEXT("InvalidRendererSettingsOIT", "Incompatible with ordered independent transparency project setting"));
		return;
	}

	const UAvaTranslucentPriorityModifierShared* SharedObject = GetShared<UAvaTranslucentPriorityModifierShared>(false);

	if (!SharedObject)
	{
		Fail(LOCTEXT("InvalidModifierSharedObject", "The modifier shared object is invalid"));
		return;
	}

	// Used cached component state if present
	if (CachedSortedComponentStates.IsEmpty())
	{
		CachedSortedComponentStates = SharedObject->GetSortedComponentStates(this);
	}

	// Get global sort offset and step for this level
	const int32 GlobalOffset = SharedObject->GetSortPriorityOffset();
	const int32 GlobalStep = SharedObject->GetSortPriorityStep();

	LastSortPriorities.Empty(CachedSortedComponentStates.Num());

	if (Mode == EAvaTranslucentPriorityModifierMode::Manual)
	{
		int32 TranslucentSortPriority = GlobalOffset + SortPriority;

		// Sets all components with the same priority
		for (const FAvaTranslucentPriorityModifierComponentState& SortedComponentState : CachedSortedComponentStates)
		{
			if (SortedComponentState.Modifier == this)
			{
				if (UPrimitiveComponent* const Component = SortedComponentState.PrimitiveComponent)
				{
					LastSortPriorities.Add(Component, TranslucentSortPriority);
					
					UE::ActorModifierCore::FRenderStateDirtyReasonScope Scope(UE::ActorModifierCore::ERenderStateDirtyReason::TranslucencyPriority);
					Component->SetTranslucentSortPriority(TranslucentSortPriority);

					TranslucentSortPriority += GlobalStep;
				}
			}
		}
	}
	else
	{
		int32 TranslucentSortPriority = GlobalOffset;

		// Increment sort priority for each component that this modifier handles
		AActor* PreviousComponentOwner = nullptr;
		for (const FAvaTranslucentPriorityModifierComponentState& SortedComponentState : CachedSortedComponentStates)
		{
			UPrimitiveComponent* const Component = SortedComponentState.PrimitiveComponent;
			if (!Component || !Component->GetOwner() || !SortedComponentState.Modifier)
			{
				continue;
			}

			const bool bHasIteratedOnce = PreviousComponentOwner != nullptr;
			const bool bShouldIncreaseStep = Mode == EAvaTranslucentPriorityModifierMode::AutoCameraDistance || PreviousComponentOwner != Component->GetOwner();

			if (bHasIteratedOnce && bShouldIncreaseStep)
			{
				TranslucentSortPriority += GlobalStep;
			}

			PreviousComponentOwner = Component->GetOwner();

			// This modifier handles this component
			if (SortedComponentState.Modifier == this)
			{
				LastSortPriorities.Add(Component, TranslucentSortPriority);

				UE::ActorModifierCore::FRenderStateDirtyReasonScope Scope(UE::ActorModifierCore::ERenderStateDirtyReason::TranslucencyPriority);
				Component->SetTranslucentSortPriority(TranslucentSortPriority);
			}
			// Another modifier handles this component
			else if (Component->TranslucencySortPriority != TranslucentSortPriority)
			{
				// Cache to avoid doing the same query for the same result
				if (SortedComponentState.Modifier->CachedSortedComponentStates.IsEmpty())
				{
					SortedComponentState.Modifier->CachedSortedComponentStates = CachedSortedComponentStates;
				}
				SortedComponentState.Modifier->MarkModifierDirty();
			}
		}
	}

	CachedSortedComponentStates.Empty();

	Next();
}

void UAvaTranslucentPriorityModifier::OnModifierDisabled(EActorModifierCoreDisableReason InReason)
{
	Super::OnModifierDisabled(InReason);

	if (UAvaTranslucentPriorityModifierShared* SharedObject = GetShared<UAvaTranslucentPriorityModifierShared>(false))
	{
		SharedObject->RestoreComponentsState(this, true);
	}

	if (FActorModifierTransformUpdateExtension* TransformExtension = GetExtension<FActorModifierTransformUpdateExtension>())
	{
		TransformExtension->UntrackActors(ChildrenActorsWeak);
	}
}

void UAvaTranslucentPriorityModifier::OnModifiedActorTransformed()
{
	if (Mode == EAvaTranslucentPriorityModifierMode::AutoCameraDistance)
	{
		MarkModifierDirty();
	}
}

void UAvaTranslucentPriorityModifier::PostLoad()
{
	Super::PostLoad();

	// Transfer previous sort priorities and component to shared object
	if (!PreviousSortPriorities.IsEmpty())
	{
		if (UAvaTranslucentPriorityModifierShared* SharedObject = GetShared<UAvaTranslucentPriorityModifierShared>(true))
		{
			for (const TPair<TWeakObjectPtr<UPrimitiveComponent>, int32>& PreviousSortPriority : PreviousSortPriorities)
			{
				if (UPrimitiveComponent* PrimitiveComponent = PreviousSortPriority.Key.Get())
				{
					PrimitiveComponent->TranslucencySortPriority = PreviousSortPriority.Value;
					SharedObject->SaveComponentState(this, PrimitiveComponent, true);
				}
			}

			PreviousSortPriorities.Empty();
		}
	}
}

#if WITH_EDITOR
void UAvaTranslucentPriorityModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UAvaTranslucentPriorityModifier, Mode))
	{
		OnModeChanged();
	}
	else if (MemberName == GET_MEMBER_NAME_CHECKED(UAvaTranslucentPriorityModifier, CameraActorWeak))
	{
		OnCameraActorChanged();
	}
	else if (MemberName == GET_MEMBER_NAME_CHECKED(UAvaTranslucentPriorityModifier, SortPriority))
	{
		OnSortPriorityChanged();
	}
	else if (MemberName == GET_MEMBER_NAME_CHECKED(UAvaTranslucentPriorityModifier, SortPriorityOffset)
		|| MemberName == GET_MEMBER_NAME_CHECKED(UAvaTranslucentPriorityModifier, SortPriorityStep))
	{
		OnSortPriorityLevelGlobalsChanged();
	}
	else if (MemberName == GET_MEMBER_NAME_CHECKED(UAvaTranslucentPriorityModifier, bIncludeChildren))
	{
		OnIncludeChildrenChanged();
	}
}
#endif

void UAvaTranslucentPriorityModifier::SetMode(EAvaTranslucentPriorityModifierMode InMode)
{
	if (Mode == InMode)
	{
		return;
	}

	Mode = InMode;
	OnModeChanged();
}

void UAvaTranslucentPriorityModifier::SetCameraActor(ACameraActor* InCameraActor)
{
	SetCameraActorWeak(InCameraActor);
}

void UAvaTranslucentPriorityModifier::SetCameraActorWeak(const TWeakObjectPtr<ACameraActor>& InCameraActor)
{
	if (CameraActorWeak == InCameraActor)
	{
		return;
	}

	if (Mode != EAvaTranslucentPriorityModifierMode::AutoCameraDistance)
	{
		return;
	}

	CameraActorWeak = InCameraActor;
	OnCameraActorChanged();
}

void UAvaTranslucentPriorityModifier::SetSortPriority(int32 InSortPriority)
{
	if (SortPriority == InSortPriority)
	{
		return;
	}

	if (Mode != EAvaTranslucentPriorityModifierMode::Manual)
	{
		return;
	}

	SortPriority = InSortPriority;
	OnSortPriorityChanged();
}

void UAvaTranslucentPriorityModifier::SetSortPriorityOffset(int32 InOffset)
{
	if (SortPriorityOffset == InOffset)
	{
		return;
	}

	SortPriorityOffset = InOffset;
	OnSortPriorityLevelGlobalsChanged();
}

void UAvaTranslucentPriorityModifier::SetSortPriorityStep(int32 InStep)
{
	if (SortPriorityStep == InStep)
	{
		return;
	}

	SortPriorityStep = InStep;
	OnSortPriorityLevelGlobalsChanged();
}

void UAvaTranslucentPriorityModifier::SetIncludeChildren(bool bInIncludeChildren)
{
	if (bIncludeChildren == bInIncludeChildren)
	{
		return;
	}

	bIncludeChildren = bInIncludeChildren;
	OnIncludeChildrenChanged();
}

void UAvaTranslucentPriorityModifier::OnSceneTreeTrackedActorChildrenChanged(int32 InIdx,
	const TSet<TWeakObjectPtr<AActor>>& InPreviousChildrenActors,
	const TSet<TWeakObjectPtr<AActor>>& InNewChildrenActors)
{
	if (bIncludeChildren)
	{
		MarkModifierDirty();
	}
}

void UAvaTranslucentPriorityModifier::OnSceneTreeTrackedActorDirectChildrenChanged(int32 InIdx,
	const TArray<TWeakObjectPtr<AActor>>& InPreviousChildrenActors,
	const TArray<TWeakObjectPtr<AActor>>& InNewChildrenActors)
{
	// Overwrite parent, don't do anything when children order changed
}

void UAvaTranslucentPriorityModifier::OnSceneTreeTrackedActorRearranged(int32 InIdx, AActor* InRearrangedActor)
{
	Super::OnSceneTreeTrackedActorRearranged(InIdx, InRearrangedActor);

	if (Mode == EAvaTranslucentPriorityModifierMode::AutoOutlinerTree || Mode == EAvaTranslucentPriorityModifierMode::AutoOutlinerTreeInverted)
	{
		MarkModifierDirty();
	}
}

UE::ActorModifierCore::ERenderStateDirtyReason UAvaTranslucentPriorityModifier::GetRenderStateDirtyEventTypes() const
{
	return UE::ActorModifierCore::ERenderStateDirtyReason::TranslucencyPriority
		| UE::ActorModifierCore::ERenderStateDirtyReason::SceneTree;
}

void UAvaTranslucentPriorityModifier::OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent)
{
	Super::OnRenderStateUpdated(InActor, InComponent);

	const AActor* const ActorModified = GetModifiedActor();
	UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(InComponent);

	if (!IsValid(ActorModified)
		|| !IsValid(InActor)
		|| !IsValid(PrimitiveComponent))
	{
		return;
	}

	if (!bIncludeChildren && InActor != ActorModified)
	{
		return;
	}

	if (bIncludeChildren && InActor != ActorModified && !InActor->IsAttachedTo(ActorModified))
	{
		return;
	}

	if (bIncludeChildren && !ChildrenActorsWeak.Contains(InActor))
	{
		const UAvaTranslucentPriorityModifierShared* SharedObject = GetShared<UAvaTranslucentPriorityModifierShared>(false);
		if (!SharedObject)
		{
			return;
		}

		// If the component is already handled by another modifier, return early
		const UAvaTranslucentPriorityModifier* OtherModifier = SharedObject->FindModifierContext(PrimitiveComponent);
		if (OtherModifier && OtherModifier != this)
		{
			return;
		}
	}

	// In case the sort priority hasn't changed do not update
	if (const int32* LastSortPriority = LastSortPriorities.Find(PrimitiveComponent))
	{
		if (PrimitiveComponent->TranslucencySortPriority == *LastSortPriority)
		{
			return;
		}
	}

	MarkModifierDirty();
}

void UAvaTranslucentPriorityModifier::OnTransformUpdated(AActor* InActor, bool bInParentMoved)
{
	Super::OnTransformUpdated(InActor, bInParentMoved);

	const AActor* const ActorModified = GetModifiedActor();

	// Only AutoCameraDistance mode relies on actor location to compute sort priority
	if (!IsValid(ActorModified)
		|| Mode != EAvaTranslucentPriorityModifierMode::AutoCameraDistance
		|| (!bIncludeChildren && InActor != ActorModified)
		|| (bIncludeChildren && !ChildrenActorsWeak.Contains(InActor)))
	{
		return;
	}

	MarkModifierDirty();
}

void UAvaTranslucentPriorityModifier::OnModeChanged()
{
	MarkModifierDirty();
}

void UAvaTranslucentPriorityModifier::OnCameraActorChanged()
{
	if (Mode == EAvaTranslucentPriorityModifierMode::AutoCameraDistance)
	{
		MarkModifierDirty();
	}
}

void UAvaTranslucentPriorityModifier::OnSortPriorityChanged()
{
	if (Mode == EAvaTranslucentPriorityModifierMode::Manual)
	{
		MarkModifierDirty();
	}
}

void UAvaTranslucentPriorityModifier::OnSortPriorityLevelGlobalsChanged() const
{
	if (UAvaTranslucentPriorityModifierShared* SharedObject = GetShared<UAvaTranslucentPriorityModifierShared>(false))
	{
		SharedObject->SetSortPriorityOffset(SortPriorityOffset);
		SharedObject->SetSortPriorityStep(SortPriorityStep);
	}
}

void UAvaTranslucentPriorityModifier::OnGlobalSortPriorityOffsetChanged()
{
	if (const UAvaTranslucentPriorityModifierShared* SharedObject = GetShared<UAvaTranslucentPriorityModifierShared>(false))
	{
		SortPriorityOffset = SharedObject->GetSortPriorityOffset();
		SortPriorityStep = SharedObject->GetSortPriorityStep();
		MarkModifierDirty();
	}
}

void UAvaTranslucentPriorityModifier::OnIncludeChildrenChanged()
{
	MarkModifierDirty();
}

ACameraActor* UAvaTranslucentPriorityModifier::GetDefaultCameraActor() const
{
	const AActor* const ActorModified = GetModifiedActor();
	const UWorld* const World = IsValid(ActorModified) ? ActorModified->GetWorld() : nullptr;

	if (IsValid(World))
	{
		for (ACameraActor* const WorldCameraActor : TActorRange<ACameraActor>(World))
		{
			if (!IsValid(WorldCameraActor))
			{
				continue;
			}

			return WorldCameraActor;
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
