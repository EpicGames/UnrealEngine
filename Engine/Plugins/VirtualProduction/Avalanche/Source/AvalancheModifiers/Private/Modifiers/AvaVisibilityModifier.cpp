// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/AvaVisibilityModifier.h"

#include "AvaModifiersActorUtils.h"
#include "GameFramework/Actor.h"
#include "Misc/ITransaction.h"
#include "Shared/ActorModifierVisibilityShared.h"

#define LOCTEXT_NAMESPACE "AvaVisibilityModifier"

void UAvaVisibilityModifier::Apply()
{
	const FActorModifierSceneTreeUpdateExtension* SceneExtension = GetExtension<FActorModifierSceneTreeUpdateExtension>();
	if (!SceneExtension)
	{
		Fail(LOCTEXT("InvalidSceneExtension", "Scene extension could not be found"));
		return;
	}

	constexpr bool bCreate = true;
	UActorModifierVisibilityShared* VisibilityShared = GetShared<UActorModifierVisibilityShared>(bCreate);
	if (!VisibilityShared)
	{
		Fail(LOCTEXT("InvalidSharedObject", "Invalid modifier shared object retrieved"));
		return;
	}

	AActor* const ModifyActor = GetModifiedActor();
	const TArray<TWeakObjectPtr<AActor>> AttachedActors = SceneExtension->GetDirectChildrenActor(ModifyActor);

#if WITH_EDITOR
	const bool bHiddenInEditor = ModifyActor->IsTemporarilyHiddenInEditor(/** IncludeParent */false);
#else
	const bool bHiddenInEditor = false;
#endif

	// Top most modifier in tree has priority over this one if it is hiding current one
	bool bIsNestedVisibilityModifier = false;
	const TArray<UAvaVisibilityModifier*> VisibilityModifiers = GetModifiersAbove(ModifyActor);
	if (!VisibilityModifiers.IsEmpty())
	{
		// This actor is not visible and hidden by another visibility modifier do not proceed
		for (const UAvaVisibilityModifier* VisibilityModifier : VisibilityModifiers)
		{
			if (VisibilityModifier && VisibilityModifier->IsChildActorHidden(ModifyActor))
			{
				bIsNestedVisibilityModifier = true;
				break;
			}
		}
	}
	// We are the top root modifier, if this actor is hidden, do not handle children actors
	else if (bSkipWhenHidden && (ModifyActor->IsHidden() || bHiddenInEditor))
	{
		Next();
		return;
	}

	TSet<TWeakObjectPtr<AActor>> NewChildrenActorsWeak;
	for (int32 ChildIndex = 0; ChildIndex < AttachedActors.Num(); ++ChildIndex)
	{
		AActor* AttachedActor = AttachedActors[ChildIndex].Get();

		if (!AttachedActor)
		{
			continue;
		}

		// No need to handle nested children actor, only direct children, visibility will propagate
		if (AttachedActor->GetAttachParentActor() != ModifyActor)
		{
			continue;
		}

		bool bHideActor = false;

		if (!bTreatAsRange)
		{
			bHideActor = ChildIndex == Index;
		}
		else
		{
			bHideActor = ChildIndex <= Index;
		}

		bHideActor = bInvertVisibility ? bHideActor : !bHideActor;
		DirectChildrenActorsWeak.Add(AttachedActor, bHideActor);

		TArray<AActor*> AttachedChildActors {AttachedActor};
		AttachedActor->GetAttachedActors(AttachedChildActors, false, true);
		for (AActor* AttachedChildActor : AttachedChildActors)
		{
			// If we are not hiding, then the top nearest modifier in the tree takes precedence
			if (!bHideActor)
			{
				UAvaVisibilityModifier* VisibilityModifier = GetFirstModifierAbove(AttachedChildActor);

				if (VisibilityModifier && VisibilityModifier != this)
				{
					VisibilityModifier->MarkModifierDirty();
					continue;
				}
			}

			if (!bIsNestedVisibilityModifier)
			{
				VisibilityShared->SetActorVisibility(this, AttachedChildActor, bHideActor, false);
			}

			NewChildrenActorsWeak.Add(AttachedActor);
		}
	}

	if (!bIsNestedVisibilityModifier)
	{
		// Untrack previous actors that are not attached anymore
		VisibilityShared->RestoreActorsState(this, ChildrenActorsWeak.Difference(NewChildrenActorsWeak));
	}

	ChildrenActorsWeak = NewChildrenActorsWeak;

	FActorModifierRenderStateUpdateExtension* RenderStateExtension = GetExtension<FActorModifierRenderStateUpdateExtension>();

	// Reset tracked actors state to detect change in visibility later
	for (const TWeakObjectPtr<AActor>& ChildrenActorWeak : ChildrenActorsWeak)
	{
		if (AActor* ChildrenActor = ChildrenActorWeak.Get())
		{
			RenderStateExtension->UntrackActorVisibility(ChildrenActor);
			RenderStateExtension->TrackActorVisibility(ChildrenActor);
		}
	}

	Next();
}

#if WITH_EDITOR
void UAvaVisibilityModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();

	static const FName IndexPropertyName = GET_MEMBER_NAME_CHECKED(UAvaVisibilityModifier, Index);
	static const FName InvertVisibilityPropertyName = GET_MEMBER_NAME_CHECKED(UAvaVisibilityModifier, bInvertVisibility);
	static const FName TreatAsRangePropertyName = GET_MEMBER_NAME_CHECKED(UAvaVisibilityModifier, bTreatAsRange);

	if (PropertyName == IndexPropertyName
		|| PropertyName == TreatAsRangePropertyName
		|| PropertyName == InvertVisibilityPropertyName)
	{
		MarkModifierDirty();
	}
}
#endif // WITH_EDITOR

void UAvaVisibilityModifier::SetTreatAsRange(bool bInTreatAsRange)
{
	if (bTreatAsRange == bInTreatAsRange)
	{
		return;
	}

	bTreatAsRange = bInTreatAsRange;
	MarkModifierDirty();
}

void UAvaVisibilityModifier::SetSkipWhenHidden(bool bInSkip)
{
	if (bSkipWhenHidden == bInSkip)
	{
		return;
	}

	bSkipWhenHidden = bInSkip;
	MarkModifierDirty();
}

void UAvaVisibilityModifier::OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata)
{
	Super::OnModifierCDOSetup(InMetadata);

	InMetadata.SetName(TEXT("Visibility"));
	InMetadata.SetCategory(TEXT("Rendering"));
#if WITH_EDITOR
	InMetadata.SetDisplayName(LOCTEXT("ModifierDisplayName", "Visibility"));
	InMetadata.SetDescription(LOCTEXT("ModifierDescription", "Controls the visibility of a range of child actors by index"));
#endif
}

void UAvaVisibilityModifier::OnModifiedActorTransformed()
{
	// Overwrite parent class behaviour don't do anything when moved
}

void UAvaVisibilityModifier::OnActorVisibilityChanged(AActor* InActor)
{
	Super::OnActorVisibilityChanged(InActor);

	AActor* ActorModified = GetModifiedActor();

	if (!IsValid(ActorModified))
	{
		return;
	}

	// Only handle what is linked to us
	const bool bThisActorUpdated = InActor == ActorModified;
	const bool bActorAttachedToThisUpdated = InActor->IsAttachedTo(ActorModified);

	if (!bThisActorUpdated && !bActorAttachedToThisUpdated)
	{
		return;
	}

	// If no modifier is found above us, then we handle this case otherwise let the other modifier handle it
	const UAvaVisibilityModifier* Modifier = GetFirstModifierAbove(ActorModified);

	if (bThisActorUpdated && Modifier)
	{
		return;
	}

	if (Modifier && Modifier->IsChildActorHidden(InActor))
	{
		return;
	}

	MarkModifierDirty();
}

void UAvaVisibilityModifier::SetInvertVisibility(bool bInInvertVisibility)
{
	if (bInvertVisibility == bInInvertVisibility)
	{
		return;
	}

	bInvertVisibility = bInInvertVisibility;
	MarkModifierDirty();
}

void UAvaVisibilityModifier::SetIndex(int32 InIndex)
{
	if (Index == InIndex)
	{
		return;
	}

	if (Index < 0)
	{
		return;
	}

	Index = InIndex;
	MarkModifierDirty();
}

bool UAvaVisibilityModifier::IsChildActorHidden(AActor* InActor) const
{
	const AActor* ActorModified = GetModifiedActor();

	if (InActor->IsAttachedTo(ActorModified))
	{
		while (InActor->GetAttachParentActor() != ActorModified)
		{
			InActor = InActor->GetAttachParentActor();
		}

		if (!DirectChildrenActorsWeak.Contains(InActor))
		{
			return false;
		}

		return *DirectChildrenActorsWeak.Find(InActor);
	}

	return false;
}

UAvaVisibilityModifier* UAvaVisibilityModifier::GetFirstModifierAbove(AActor* InActor)
{
	UActorModifierVisibilityShared* VisibilityShared = GetShared<UActorModifierVisibilityShared>(false);
	UAvaVisibilityModifier* FirstModifierAbove = nullptr;

	if (!InActor || !VisibilityShared)
	{
		return FirstModifierAbove;
	}

	if (FActorModifierVisibilitySharedActorState* ActorState = VisibilityShared->FindActorState(InActor))
	{
		for (const FActorModifierVisibilitySharedModifierState& ModifierState : ActorState->ModifierStates)
		{
			UActorModifierCoreBase* Modifier = ModifierState.ModifierWeak.Get();
			if (!Modifier || Modifier->GetModifiedActor() != InActor->GetAttachParentActor())
			{
				continue;
			}

			if (UAvaVisibilityModifier* VisibilityModifier = Cast<UAvaVisibilityModifier>(Modifier))
			{
				FirstModifierAbove = VisibilityModifier;
				break;
			}
		}
	}

	if (!FirstModifierAbove)
	{
		FirstModifierAbove = GetFirstModifierAbove(InActor->GetAttachParentActor());
	}

	return FirstModifierAbove;
}

TArray<UAvaVisibilityModifier*> UAvaVisibilityModifier::GetModifiersAbove(AActor* InActor)
{
	TArray<UAvaVisibilityModifier*> Modifiers;
	while (UAvaVisibilityModifier* VisibilityModifier = GetFirstModifierAbove(InActor))
	{
		Modifiers.Add(VisibilityModifier);
		InActor = VisibilityModifier->GetModifiedActor();
	}
	return Modifiers;
}

AActor* UAvaVisibilityModifier::GetDirectChildren(AActor* InParentActor, AActor* InChildActor)
{
	if (!InParentActor || !InChildActor)
	{
		return nullptr;
	}

	if (InChildActor && InChildActor->GetAttachParentActor() == InParentActor)
	{
		return InChildActor;
	}

	return GetDirectChildren(InParentActor, InChildActor->GetAttachParentActor());
}

#undef LOCTEXT_NAMESPACE
