// Copyright Epic Games, Inc. All Rights Reserved.

#include "Profilers/AvaBooleanModifierProfiler.h"

#include "Modifiers/AvaBooleanModifier.h"
#include "Shared/AvaBooleanModifierShared.h"

void FAvaBooleanModifierProfiler::SetupProfilingStats()
{
	FAvaGeometryModifierProfiler::SetupProfilingStats();

	ProfilerStats.AddProperty(ChannelInfo, EPropertyBagPropertyType::Struct, FAvaBooleanModifierSharedChannelInfo::StaticStruct());
}

void FAvaBooleanModifierProfiler::BeginProfiling()
{
	FAvaGeometryModifierProfiler::BeginProfiling();
}

void FAvaBooleanModifierProfiler::EndProfiling()
{
	FAvaGeometryModifierProfiler::EndProfiling();

	if (const UAvaBooleanModifier* BooleanModifier = GetModifier<UAvaBooleanModifier>())
	{
		ProfilerStats.SetValueStruct(ChannelInfo, BooleanModifier->GetChannelInfo());
	}
}

TSet<FName> FAvaBooleanModifierProfiler::GetMainProfilingStats() const
{
	return TSet<FName>
	{
		ExecutionTimeName,
		GET_MEMBER_NAME_CHECKED(FAvaBooleanModifierSharedChannelInfo, ChannelModifierCount),
		GET_MEMBER_NAME_CHECKED(FAvaBooleanModifierSharedChannelInfo, ChannelIntersectCount),
	};
}

FAvaBooleanModifierSharedChannelInfo FAvaBooleanModifierProfiler::GetChannelInfo() const
{
	TValueOrError<FAvaBooleanModifierSharedChannelInfo*, EPropertyBagResult> ValueResult = ProfilerStats.GetValueStruct<FAvaBooleanModifierSharedChannelInfo>(ChannelInfo);
	return ValueResult.HasValue() ? *ValueResult.GetValue() : FAvaBooleanModifierSharedChannelInfo();
}
