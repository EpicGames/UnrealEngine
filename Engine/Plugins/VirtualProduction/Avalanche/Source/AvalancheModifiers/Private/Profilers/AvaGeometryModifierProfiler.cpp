// Copyright Epic Games, Inc. All Rights Reserved.

#include "Profilers/AvaGeometryModifierProfiler.h"

#include "Components/DynamicMeshComponent.h"
#include "Modifiers/AvaGeometryBaseModifier.h"

void FAvaGeometryModifierProfiler::SetupProfilingStats()
{
	FActorModifierCoreProfiler::SetupProfilingStats();

	ProfilerStats.AddProperty(VertexInName, EPropertyBagPropertyType::Int32);
	ProfilerStats.AddProperty(VertexOutName, EPropertyBagPropertyType::Int32);
	ProfilerStats.AddProperty(TriangleInName, EPropertyBagPropertyType::Int32);
	ProfilerStats.AddProperty(TriangleOutName, EPropertyBagPropertyType::Int32);
	ProfilerStats.AddProperty(EdgeInName, EPropertyBagPropertyType::Int32);
	ProfilerStats.AddProperty(EdgeOutName, EPropertyBagPropertyType::Int32);
}

void FAvaGeometryModifierProfiler::BeginProfiling()
{
	FActorModifierCoreProfiler::BeginProfiling();

	const UAvaGeometryBaseModifier* GeometryModifier = GetModifier<UAvaGeometryBaseModifier>();

	if (GeometryModifier && GeometryModifier->IsMeshValid())
	{
		GeometryModifier->GetMeshComponent()->ProcessMesh([this](const FDynamicMesh3& InMesh)
		{
			ProfilerStats.SetValueInt32(VertexInName, InMesh.VertexCount());
			ProfilerStats.SetValueInt32(TriangleInName, InMesh.TriangleCount());
			ProfilerStats.SetValueInt32(EdgeInName, InMesh.EdgeCount());
		});
	}
}

void FAvaGeometryModifierProfiler::EndProfiling()
{
	FActorModifierCoreProfiler::EndProfiling();

	const UAvaGeometryBaseModifier* GeometryModifier = GetModifier<UAvaGeometryBaseModifier>();

	if (GeometryModifier && GeometryModifier->IsMeshValid())
	{
		GeometryModifier->GetMeshComponent()->ProcessMesh([this](const FDynamicMesh3& InMesh)
		{
			ProfilerStats.SetValueInt32(VertexOutName, InMesh.VertexCount());
			ProfilerStats.SetValueInt32(TriangleOutName, InMesh.TriangleCount());
			ProfilerStats.SetValueInt32(EdgeOutName, InMesh.EdgeCount());
		});
	}
}

TSet<FName> FAvaGeometryModifierProfiler::GetMainProfilingStats() const
{
	return TSet<FName>
	{
		ExecutionTimeName,
		TriangleInName,
		TriangleOutName
	};
}

int32 FAvaGeometryModifierProfiler::GetVertexIn() const
{
	TValueOrError<int32, EPropertyBagResult> ValueResult = ProfilerStats.GetValueInt32(VertexInName);
	return ValueResult.HasValue() ? ValueResult.GetValue() : 0;
}

int32 FAvaGeometryModifierProfiler::GetVertexOut() const
{
	TValueOrError<int32, EPropertyBagResult> ValueResult = ProfilerStats.GetValueInt32(VertexOutName);
	return ValueResult.HasValue() ? ValueResult.GetValue() : 0;
}

int32 FAvaGeometryModifierProfiler::GetTriangleIn() const
{
	TValueOrError<int32, EPropertyBagResult> ValueResult = ProfilerStats.GetValueInt32(TriangleInName);
	return ValueResult.HasValue() ? ValueResult.GetValue() : 0;
}

int32 FAvaGeometryModifierProfiler::GetTriangleOut() const
{
	TValueOrError<int32, EPropertyBagResult> ValueResult = ProfilerStats.GetValueInt32(TriangleOutName);
	return ValueResult.HasValue() ? ValueResult.GetValue() : 0;
}

int32 FAvaGeometryModifierProfiler::GetEdgeIn() const
{
	TValueOrError<int32, EPropertyBagResult> ValueResult = ProfilerStats.GetValueInt32(EdgeInName);
	return ValueResult.HasValue() ? ValueResult.GetValue() : 0;
}

int32 FAvaGeometryModifierProfiler::GetEdgeOut() const
{
	TValueOrError<int32, EPropertyBagResult> ValueResult = ProfilerStats.GetValueInt32(EdgeOutName);
	return ValueResult.HasValue() ? ValueResult.GetValue() : 0;
}
