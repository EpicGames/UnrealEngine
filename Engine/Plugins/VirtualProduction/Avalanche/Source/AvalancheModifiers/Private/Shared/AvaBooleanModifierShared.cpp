// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shared/AvaBooleanModifierShared.h"

#include "Async/ParallelFor.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Modifiers/AvaBooleanModifier.h"

TSet<TWeakObjectPtr<UAvaBooleanModifier>> UAvaBooleanModifierShared::GetIntersectingModifiers(const UAvaBooleanModifier* InTargetModifier, FAvaBooleanModifierSharedChannelInfo* OutDesc)
{
	TSet<TWeakObjectPtr<UAvaBooleanModifier>> IntersectingModifiers;
	if (!IsValid(InTargetModifier)
		|| !InTargetModifier->GetMeshComponent()
		|| !InTargetModifier->PreModifierCachedMesh.IsSet())
	{
		return IntersectingModifiers;
	}

	const AActor* TargetActor = InTargetModifier->GetModifiedActor();
	if (!IsValid(TargetActor))
	{
		return IntersectingModifiers;
	}

	const int32 TargetChannel = InTargetModifier->GetChannel();
	const FAvaBooleanModifierSharedChannel* Channel = Channels.Find(TargetChannel);
	if (!Channel)
	{
		return IntersectingModifiers;
	}

	const bool bTargetBooleanMode = InTargetModifier->GetMode() != EAvaBooleanMode::None;
	const FTransform TargetTransform = InTargetModifier->GetMeshComponent()->GetComponentTransform();

	if (OutDesc)
	{
		OutDesc->ChannelModifierCount = Channel->ModifiersWeak.Num();
		OutDesc->ChannelToolCount = 0;
		OutDesc->ChannelTargetCount = 0;
		OutDesc->ChannelIntersectCount = 0;
	}

	for (const TWeakObjectPtr<UAvaBooleanModifier>& OtherModifierWeak : Channel->ModifiersWeak)
	{
		UAvaBooleanModifier* OtherModifier = OtherModifierWeak.Get();
		if (!IsValid(OtherModifier)
			|| !OtherModifier->GetMeshComponent()
			|| !OtherModifier->PreModifierCachedMesh.IsSet())
		{
			continue;
		}

		if (OutDesc)
		{
			if (OtherModifier->GetMode() == EAvaBooleanMode::None)
			{
				OutDesc->ChannelTargetCount++;
			}
			else
			{
				OutDesc->ChannelToolCount++;
			}
		}

		const AActor* OtherActor = OtherModifier->GetModifiedActor();
		if (!OtherActor || TargetActor == OtherActor)
		{
			continue;
		}

		const bool bOtherBooleanMode = OtherModifier->GetMode() != EAvaBooleanMode::None;
		if (bTargetBooleanMode == bOtherBooleanMode)
		{
			continue;
		}

		const FTransform OtherTransform = OtherModifier->GetMeshComponent()->GetComponentTransform();

		// Other is mask
		bool bIsIntersecting = false;
		if (bOtherBooleanMode)
		{
			const FDynamicMesh3& TargetMesh = InTargetModifier->PreModifierCachedMesh.GetValue();

			OtherModifier->GetMeshComponent()->ProcessMesh([this, &bIsIntersecting, &TargetMesh, &OtherTransform, &TargetTransform](const FDynamicMesh3& InOtherMesh)
			{
				bIsIntersecting = TestIntersection(InOtherMesh, OtherTransform, TargetMesh, TargetTransform);
			});
		}
		// Target is mask
		else
		{
			const FDynamicMesh3& OtherMesh = OtherModifier->PreModifierCachedMesh.GetValue();

			InTargetModifier->GetMeshComponent()->ProcessMesh([this, &bIsIntersecting, &OtherMesh, &OtherTransform, &TargetTransform](const FDynamicMesh3& InTargetMesh)
			{
				bIsIntersecting = TestIntersection(InTargetMesh, TargetTransform, OtherMesh, OtherTransform);
			});
		}

		if (bIsIntersecting)
		{
			IntersectingModifiers.Add(OtherModifier);

			if (OutDesc)
			{
				OutDesc->ChannelIntersectCount++;
			}
		}
	}

	if (OutDesc)
	{
		OutDesc->ChannelCount = GetChannelCount();
	}

	// Sort modifier based on mode priority
	static const TMap<EAvaBooleanMode, uint32> ModePriority
	{
		{EAvaBooleanMode::None, 0},
		{EAvaBooleanMode::Intersect, 1},
		{EAvaBooleanMode::Union, 2},
		{EAvaBooleanMode::Subtract, 3}
	};

	IntersectingModifiers.StableSort([](const TWeakObjectPtr<UAvaBooleanModifier>& InModifierA, const TWeakObjectPtr<UAvaBooleanModifier>& InModifierB)
	{
		return ModePriority[InModifierA->GetMode()] < ModePriority[InModifierB->GetMode()];
	});

	return IntersectingModifiers;
}

bool UAvaBooleanModifierShared::TestIntersection(const UE::Geometry::FDynamicMesh3& InToolMesh, const FTransform& InToolTransform, const UE::Geometry::FDynamicMesh3& InTargetMesh, const FTransform& InTargetTransform) const
{
	using namespace UE::Geometry;

	if (InToolMesh.TriangleCount() == 0)
	{
		return false;
	}

	FDynamicMeshAABBTree3 MeshSpatialTrees[2];
	ParallelFor(2, [&MeshSpatialTrees, &InToolMesh, &InTargetMesh](int32 k)
	{
		MeshSpatialTrees[k].SetMesh(k == 0 ? &InToolMesh : &InTargetMesh, true);
	}, EParallelForFlags::Unbalanced);

	const bool bIsIdentityTool = InToolTransform.Equals(FTransform::Identity, 0);
	FTransformSRT3d ToolTransform(InToolTransform);

	const bool bIsIdentityTarget = InTargetTransform.Equals(FTransform::Identity, 0);
	FTransformSRT3d TargetTransform(InTargetTransform);

	bool bIsIntersecting = false;
	if (bIsIdentityTool && bIsIdentityTarget)
	{
		bIsIntersecting = MeshSpatialTrees[0].TestIntersection(MeshSpatialTrees[1]);
	}
	else if (bIsIdentityTool || bIsIdentityTarget)
	{
		const FIndex2i Indices = (bIsIdentityTool) ? FIndex2i(0,1) : FIndex2i(1,0);
		FTransformSRT3d UseTransform = (bIsIdentityTarget) ? ToolTransform : TargetTransform;
		bIsIntersecting = MeshSpatialTrees[Indices.A].TestIntersection(MeshSpatialTrees[Indices.B],
			[&UseTransform](const FVector3d& Pos) { return UseTransform.TransformPosition(Pos); });
	}
	else
	{
		bIsIntersecting = MeshSpatialTrees[0].TestIntersection(MeshSpatialTrees[1],
			[&ToolTransform, &TargetTransform](const FVector3d& Pos) { return ToolTransform.InverseTransformPosition(TargetTransform.TransformPosition(Pos)); });
	}

	// We are not intersecting but target is perhaps contained fully inside tool or partially inside tool (for mesh with disconnected parts)
	if (!bIsIntersecting)
	{
		const FBox ToolBounds = static_cast<FBox>(MeshSpatialTrees[0].GetBoundingBox()).TransformBy(ToolTransform);
		const FBox TargetBounds = static_cast<FBox>(MeshSpatialTrees[1].GetBoundingBox()).TransformBy(TargetTransform);

		// Compute tool centroid
		FVector3d ToolCentroid(0.0, 0.0, 0.0);
		{
			double TotalTotalArea = 0.0;
		
			for (int32 TId : InToolMesh.TriangleIndicesItr())
			{
				FVector3d TriCentroid = InToolMesh.GetTriCentroid(TId);
				double TriArea = InToolMesh.GetTriArea(TId);

				ToolCentroid += TriCentroid * TriArea;
				TotalTotalArea += TriArea;
			}

			if (TotalTotalArea != 0)
			{
				ToolCentroid /= TotalTotalArea;
			}
		}

		// Ensure target bounding box intersect or is inside tool
		if (ToolBounds.Intersect(TargetBounds) || ToolBounds.IsInside(TargetBounds))
		{
			const FVector3d ToolCenter = ToolTransform.TransformPosition(ToolCentroid);

			// Find target nearest triangle from tool center
			double DistSquaredTarget;
			const int32 TargetMeshTId = MeshSpatialTrees[1].FindNearestTriangle(TargetTransform.InverseTransformPosition(ToolCenter), DistSquaredTarget);

			if (TargetMeshTId != INDEX_NONE)
			{
				const FVector3d ClosestTargetCentroid = TargetTransform.TransformPosition(InTargetMesh.GetTriCentroid(TargetMeshTId));

				// Ensure target centroid is inside tool
				if (ToolBounds.IsInside(ClosestTargetCentroid))
				{
					FRay3d ToolRay(ToolCentroid, ToolTransform.InverseTransformPosition(ClosestTargetCentroid).GetSafeNormal());
					double ToolNearestT = 0.f;
					int32 ToolNearestTId = INDEX_NONE;
					FVector3d ToolBaryCoords = FVector3d::ZeroVector;

					// Find tool nearest triangle based on ray projected in nearest target triangle direction
					if (MeshSpatialTrees[0].FindNearestHitTriangle(ToolRay, ToolNearestT, ToolNearestTId, ToolBaryCoords))
					{
						const double DistanceToolCenterToTarget = FVector3d::Distance(ToolCenter, ClosestTargetCentroid);
						
						if (DistanceToolCenterToTarget < ToolNearestT)
						{
							bIsIntersecting = true;
						}
					}
				}
			}
		}
	}

	return bIsIntersecting;
}

void UAvaBooleanModifierShared::TrackModifierChannel(UAvaBooleanModifier* InModifier)
{
	if (!InModifier)
	{
		return;
	}

	FAvaBooleanModifierSharedChannel& Channel = Channels.FindOrAdd(InModifier->GetChannel());
	Channel.ModifiersWeak.Add(InModifier);
}

void UAvaBooleanModifierShared::UntrackModifierChannel(UAvaBooleanModifier* InModifier)
{
	if (!InModifier)
	{
		return;
	}

	for (TMap<uint8, FAvaBooleanModifierSharedChannel>::TIterator It(Channels); It; ++It)
	{
		It->Value.ModifiersWeak.Remove(InModifier);

		// Remove empty channel
		if (It->Value.ModifiersWeak.IsEmpty())
		{
			It.RemoveCurrent();
		}
	}
}

void UAvaBooleanModifierShared::UpdateModifierChannel(UAvaBooleanModifier* InModifier)
{
	if (!InModifier)
	{
		return;
	}

	const uint8 Channel = InModifier->GetChannel();
	for (TMap<uint8, FAvaBooleanModifierSharedChannel>::TIterator It(Channels); It; ++It)
	{
		if (Channel != It->Key)
		{
			It->Value.ModifiersWeak.Remove(InModifier);

			// Remove empty channel
			if (It->Value.ModifiersWeak.IsEmpty())
			{
				It.RemoveCurrent();
			}
		}
	}

	TrackModifierChannel(InModifier);
}

uint8 UAvaBooleanModifierShared::GetChannelCount() const
{
	return Channels.Num();
}

int32 UAvaBooleanModifierShared::GetChannelModifierCount(uint8 InChannel) const
{
	if (const FAvaBooleanModifierSharedChannel* Channel = Channels.Find(InChannel))
	{
		return Channel->ModifiersWeak.Num();
	}
	return 0;
}

int32 UAvaBooleanModifierShared::GetChannelModifierModeCount(uint8 InChannel, EAvaBooleanMode InMode) const
{
	int32 Count = 0;
	if (const FAvaBooleanModifierSharedChannel* Channel = Channels.Find(InChannel))
	{
		for (const TWeakObjectPtr<UAvaBooleanModifier>& ModifierWeak : Channel->ModifiersWeak)
		{
			const UAvaBooleanModifier* Modifier = ModifierWeak.Get();
			if (Modifier && Modifier->GetMode() == InMode)
			{
				Count++;
			}
		}
	}
	return Count;
}
