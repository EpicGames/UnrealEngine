// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shared/AvaTranslucentPriorityModifierShared.h"
#include "AvaActorUtils.h"
#include "AvaSceneItem.h"
#include "AvaSceneTree.h"
#include "AvaSceneTreeNode.h"
#include "Camera/CameraActor.h"
#include "Camera/CameraComponent.h"
#include "Framework/AvaGameInstance.h"
#include "IAvaSceneInterface.h"
#include "Modifiers/ActorModifierRenderStateDirtyEvent.h"
#include "Modifiers/AvaTranslucentPriorityModifier.h"
#include "Version/AvaTranslucentPriorityModifierVersion.h"

#if WITH_EDITOR
#include "AvaOutlinerUtils.h"
#include "IAvaOutliner.h"
#include "Item/AvaOutlinerItemUtils.h"
#endif

void UAvaTranslucentPriorityModifierShared::SetComponentsState(UAvaTranslucentPriorityModifier* InModifierContext, const TSet<TWeakObjectPtr<UPrimitiveComponent>>& InComponents)
{
	if (!IsValid(InModifierContext))
	{
		return;
	}

	// Remove old component states
	for (TSet<FAvaTranslucentPriorityModifierComponentState>::TIterator It(ComponentStates); It; ++It)
	{
		UPrimitiveComponent* const PrimitiveComponent = It->PrimitiveComponent;
		if (PrimitiveComponent && InComponents.Contains(PrimitiveComponent))
		{
			It->Modifier = InModifierContext;
			continue;
		}

		const UAvaTranslucentPriorityModifier* Modifier = It->Modifier.Get();
		if (!PrimitiveComponent || !Modifier || Modifier == InModifierContext)
		{
			It.RemoveCurrent();
		}
	}

	// Add missing components
	for (const TWeakObjectPtr<UPrimitiveComponent>& PrimitiveComponentWeak : InComponents)
	{
		if (UPrimitiveComponent* PrimitiveComponent = PrimitiveComponentWeak.Get())
		{
			SaveComponentState(InModifierContext, PrimitiveComponent, true);
		}
	}
}

void UAvaTranslucentPriorityModifierShared::SaveComponentState(UAvaTranslucentPriorityModifier* InModifierContext, UPrimitiveComponent* InComponent, bool bInOverrideContext)
{
	if (!IsValid(InComponent))
	{
		return;
	}

	FAvaTranslucentPriorityModifierComponentState& ComponentState = ComponentStates.FindOrAdd(FAvaTranslucentPriorityModifierComponentState(InComponent));

	if (!ComponentState.Modifier)
	{
		ComponentState.Save();
		bInOverrideContext = true;
	}

	if (bInOverrideContext)
	{
		ComponentState.Modifier = InModifierContext;
	}
}

void UAvaTranslucentPriorityModifierShared::RestoreComponentState(const UAvaTranslucentPriorityModifier* InModifierContext, UPrimitiveComponent* InComponent, bool bInClearState)
{
	if (!InComponent)
	{
		return;
	}

	if (const FAvaTranslucentPriorityModifierComponentState* ComponentState = ComponentStates.Find(FAvaTranslucentPriorityModifierComponentState(InComponent)))
	{
		if (ComponentState->Modifier == InModifierContext)
		{
			if (bInClearState)
			{
				FAvaTranslucentPriorityModifierComponentState ComponentStateCopy = *ComponentState;
				ComponentStates.Remove(*ComponentState);
				ComponentStateCopy.Restore();
			}
			else
			{
				ComponentState->Restore();
			}
		}
	}
}

void UAvaTranslucentPriorityModifierShared::RestoreComponentsState(const UAvaTranslucentPriorityModifier* InModifierContext, bool bInClearState)
{
	if (!InModifierContext)
	{
		return;
	}

	for (TSet<FAvaTranslucentPriorityModifierComponentState>::TIterator It(ComponentStates); It; ++It)
	{
		if (InModifierContext == It->Modifier)
		{
			if (bInClearState)
			{
				FAvaTranslucentPriorityModifierComponentState ComponentStateCopy = *It;
				It.RemoveCurrent();
				ComponentStateCopy.Restore();
			}
			else
			{
				It->Restore();
			}
		}
	}
}

void UAvaTranslucentPriorityModifierShared::RestoreComponentsState(const UAvaTranslucentPriorityModifier* InModifierContext, const TSet<TWeakObjectPtr<UPrimitiveComponent>>& InComponents, bool bInClearState)
{
	if (!InModifierContext)
	{
		return;
	}

	for (const TWeakObjectPtr<UPrimitiveComponent>& Component : InComponents)
	{
		RestoreComponentState(InModifierContext, Component.Get(), bInClearState);
	}
}

UAvaTranslucentPriorityModifier* UAvaTranslucentPriorityModifierShared::FindModifierContext(UPrimitiveComponent* InComponent) const
{
	UAvaTranslucentPriorityModifier* ModifierContext = nullptr;

	if (!InComponent)
	{
		return ModifierContext;
	}

	if (const FAvaTranslucentPriorityModifierComponentState* ComponentState = ComponentStates.Find(FAvaTranslucentPriorityModifierComponentState(InComponent)))
	{
		ModifierContext = ComponentState->Modifier;
	}

	return ModifierContext;
}

TArray<FAvaTranslucentPriorityModifierComponentState> UAvaTranslucentPriorityModifierShared::GetSortedComponentStates(UAvaTranslucentPriorityModifier* InModifierContext) const
{
	TArray<FAvaTranslucentPriorityModifierComponentState> SortedComponentStates;

	if (!InModifierContext)
	{
		return SortedComponentStates;
	}

	SortedComponentStates.Reserve(ComponentStates.Num());

	const EAvaTranslucentPriorityModifierMode SortMode = InModifierContext->GetMode();

	if (SortMode == EAvaTranslucentPriorityModifierMode::Manual)
	{
		// Only gather components the modifier is handling
		for (const FAvaTranslucentPriorityModifierComponentState& ComponentState : ComponentStates)
		{
			const UPrimitiveComponent* const PrimitiveComponent = ComponentState.PrimitiveComponent;
			if (!ComponentState.Modifier || !PrimitiveComponent)
			{
				continue;
			}

			if (ComponentState.Modifier == InModifierContext)
			{
				SortedComponentStates.Add(ComponentState);
			}
		}
	}
	else if (SortMode == EAvaTranslucentPriorityModifierMode::AutoCameraDistance)
	{
		// Gather components with same camera context
		const ACameraActor* CameraActor = InModifierContext->GetCameraActorWeak().Get();

		if (!CameraActor || CameraActor->GetWorld() != InModifierContext->GetWorld())
		{
			return SortedComponentStates;
		}

		for (const FAvaTranslucentPriorityModifierComponentState& ComponentState : ComponentStates)
		{
			const UPrimitiveComponent* const PrimitiveComponent = ComponentState.PrimitiveComponent;
			if (!ComponentState.Modifier || !PrimitiveComponent)
			{
				continue;
			}

			if (ComponentState.Modifier->GetMode() == InModifierContext->GetMode()
				&& ComponentState.Modifier->CameraActorWeak == CameraActor)
			{
				SortedComponentStates.Add(ComponentState);
			}
		}

		// Sort current modifiers by distance X to a specific camera
		SortedComponentStates.StableSort([CameraActor](const FAvaTranslucentPriorityModifierComponentState& InComponentA, const FAvaTranslucentPriorityModifierComponentState& InComponentB)->bool
		{
			const FVector AForwardComponentLocation = CameraActor->GetActorForwardVector() * InComponentA.GetComponentLocation();
			const float ADist = FVector::Distance(AForwardComponentLocation, CameraActor->GetActorLocation());

			const FVector BForwardComponentLocation = CameraActor->GetActorForwardVector() * InComponentB.GetComponentLocation();
			const float BDist = FVector::Distance(BForwardComponentLocation, CameraActor->GetActorLocation());

			return ADist > BDist;
		});
	}
	else if (SortMode == EAvaTranslucentPriorityModifierMode::AutoOutlinerTree || SortMode == EAvaTranslucentPriorityModifierMode::AutoOutlinerTreeInverted)
	{
		const bool bInvertedTree = SortMode == EAvaTranslucentPriorityModifierMode::AutoOutlinerTreeInverted;

		const AActor* ModifiedActor = InModifierContext->GetModifiedActor();
		UWorld* const World = ModifiedActor->GetTypedOuter<UWorld>();

		for (const FAvaTranslucentPriorityModifierComponentState& ComponentState : ComponentStates)
		{
			const UPrimitiveComponent* const PrimitiveComponent = ComponentState.PrimitiveComponent;
			if (!ComponentState.Modifier || !PrimitiveComponent)
			{
				continue;
			}

			const AActor* OwningActor = ComponentState.Modifier->GetModifiedActor();
			if (!OwningActor)
			{
				continue;
			}

			if (ComponentState.Modifier->GetMode() == InModifierContext->GetMode() && OwningActor->GetTypedOuter<UWorld>() == World)
			{
				SortedComponentStates.Add(ComponentState);
			}
		}

		// Fallback on scene tree sorting
		if (const IAvaSceneInterface* SceneInterface = FAvaActorUtils::GetSceneInterfaceFromActor(ModifiedActor))
		{
			const FAvaSceneTree& SceneTree = SceneInterface->GetSceneTree();

			auto FindTreeNode =
				[&SceneTree, World](AActor* InActor)->const FAvaSceneTreeNode*
					{
						const FAvaSceneTreeNode* TreeNode = nullptr;
						while (InActor && !TreeNode)
						{
							TreeNode = SceneTree.FindObjectTreeNode(InActor, World && InActor->IsIn(World) ? World : nullptr);
							InActor = InActor->GetAttachParentActor();
						}
						return TreeNode;
					};

			SortedComponentStates.Sort(
				[&FindTreeNode, bInvertedTree](const FAvaTranslucentPriorityModifierComponentState& InComponentA, const FAvaTranslucentPriorityModifierComponentState& InComponentB)->bool
				{
					const FAvaSceneTreeNode* SceneItemA = FindTreeNode(InComponentA.GetOwningActor());
					const FAvaSceneTreeNode* SceneItemB = FindTreeNode(InComponentB.GetOwningActor());
					return FAvaSceneTree::CompareTreeItemOrder(SceneItemB, SceneItemA) ^ bInvertedTree;
				});
		}
	}

	return SortedComponentStates;
}

void UAvaTranslucentPriorityModifierShared::SetSortPriorityOffset(int32 InOffset)
{
	if (SortPriorityOffset == InOffset)
	{
		return;
	}

	SortPriorityOffset = InOffset;
	OnLevelGlobalsChangedDelegate.Broadcast();
}

void UAvaTranslucentPriorityModifierShared::SetSortPriorityStep(int32 InStep)
{
	if (SortPriorityStep == InStep)
	{
		return;
	}

	SortPriorityStep = InStep;
	OnLevelGlobalsChangedDelegate.Broadcast();
}

void UAvaTranslucentPriorityModifierShared::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(UE::AvaModifiers::FTranslucentPriorityModifierVersion::Guid);
}

void UAvaTranslucentPriorityModifierShared::PostLoad()
{
	Super::PostLoad();

	const int32 ModifierVersion = GetLinkerCustomVersion(UE::AvaModifiers::FTranslucentPriorityModifierVersion::Guid);

	// Upgrade weak object ptrs
	if (ModifierVersion < UE::AvaModifiers::FTranslucentPriorityModifierVersion::UpgradeFromWeak)
	{
		TSet<FAvaTranslucentPriorityModifierComponentState> OldComponentStates = MoveTemp(ComponentStates);
		ComponentStates.Empty(OldComponentStates.Num());

		for (FAvaTranslucentPriorityModifierComponentState& ComponentState : OldComponentStates)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			ComponentState.PrimitiveComponent = ComponentState.PrimitiveComponentWeak.Get();
			ComponentState.Modifier = ComponentState.ModifierWeak.Get();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			if (ComponentState.PrimitiveComponent)
			{
				ComponentStates.Emplace(MoveTemp(ComponentState));
			}
		}
	}

	// Clean up invalid components
	for (TSet<FAvaTranslucentPriorityModifierComponentState>::TIterator It(ComponentStates); It; ++It)
	{
		if (!It->PrimitiveComponent)
		{
			It.RemoveCurrent();
		}
	}
}

void FAvaTranslucentPriorityModifierComponentState::Save()
{
	if (PrimitiveComponent)
	{
		SortPriority = PrimitiveComponent->TranslucencySortPriority;
	}
}

void FAvaTranslucentPriorityModifierComponentState::Restore() const
{
	if (PrimitiveComponent)
	{
		UE::ActorModifierCore::FRenderStateDirtyReasonScope Scope(UE::ActorModifierCore::ERenderStateDirtyReason::TranslucencyPriority);
		PrimitiveComponent->SetTranslucentSortPriority(SortPriority);
	}
}

AActor* FAvaTranslucentPriorityModifierComponentState::GetOwningActor() const
{
	if (PrimitiveComponent)
	{
		return PrimitiveComponent->GetOwner();
	}

	return nullptr;
}

FVector FAvaTranslucentPriorityModifierComponentState::GetComponentLocation() const
{
	if (PrimitiveComponent)
	{
		return PrimitiveComponent->GetComponentLocation();
	}

	return FVector::ZeroVector;
}
