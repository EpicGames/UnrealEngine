// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/ActorModifierAlignBetweenModifier.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "Tests/Framework/AvaModifiersTestUtils.h"
#include "Tests/Framework/AvaTestDynamicMeshActor.h"
#include "Tests/Framework/AvaTestUtils.h"

BEGIN_DEFINE_SPEC(AvalancheModifiersAlignBetween, "Avalanche.Modifiers.AlignBetween",
                  EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	AAvaTestDynamicMeshActor* ModifiedMeshActor;
	AAvaTestDynamicMeshActor* ReferenceOneMeshActor;
	AAvaTestDynamicMeshActor* ReferenceTwoMeshActor;
	double ReferenceOneWeight = 1;
	double ReferenceTwoWeight = 3;
	double WeightIndex = ReferenceTwoWeight / (ReferenceOneWeight + ReferenceTwoWeight);

	FVector InitialModifiedActorLocation;
	FVector InitialReferenceOneMeshActorLocation;
	FVector InitialReferenceTwoMeshActorLocation;
	FVector ExpectedModifiedActorLocation;

	UActorModifierAlignBetweenModifier* AlignBetweenModifier;

	TSharedPtr<FAvaTestUtils> TestUtils = MakeShared<FAvaTestUtils>();
	TSharedPtr<FAvaModifierTestUtils> ModifierTestUtils = MakeShared<FAvaModifierTestUtils>(TestUtils);

END_DEFINE_SPEC(AvalancheModifiersAlignBetween);

void AvalancheModifiersAlignBetween::Define()
{
	BeforeEach([this]
	{
		TestUtils->Init();

		// Spawn actors
		ModifiedMeshActor = ModifierTestUtils->SpawnTestDynamicMeshActor(FTransform(FVector(-150, 0, 150)));
		ReferenceOneMeshActor = ModifierTestUtils->SpawnTestDynamicMeshActor(FTransform(FVector(0, 0, 0)));
		ReferenceTwoMeshActor = ModifierTestUtils->SpawnTestDynamicMeshActor(FTransform(FVector(100, 100, 100)));

		// Store intial actors' locations
		InitialModifiedActorLocation = ModifiedMeshActor->GetActorTransform().GetLocation();
		InitialReferenceOneMeshActorLocation = ReferenceOneMeshActor->GetActorTransform().GetLocation();
		InitialReferenceTwoMeshActorLocation = ReferenceTwoMeshActor->GetActorTransform().GetLocation();
		ExpectedModifiedActorLocation = FVector(InitialReferenceTwoMeshActorLocation.X * WeightIndex,
		                                        InitialReferenceTwoMeshActorLocation.Y * WeightIndex,
		                                        InitialReferenceTwoMeshActorLocation.Z * WeightIndex);

		// Set up modifier
		const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
		const FName ModifierName = ModifierTestUtils->GetModifierName(UActorModifierAlignBetweenModifier::StaticClass());
		FActorModifierCoreStackInsertOp InsertOp = ModifierTestUtils->GenerateInsertOp(ModifierName);
		UActorModifierCoreStack* ModifierStack = ModifierTestUtils->GenerateModifierStackForActor(ModifiedMeshActor);
		AlignBetweenModifier = Cast<UActorModifierAlignBetweenModifier>(ModifierSubsystem->InsertModifier(ModifierStack, InsertOp));
		check(AlignBetweenModifier);

		// Set up the modifier
		FActorModifierAlignBetweenWeightedActor ReferenceOneActor = FActorModifierAlignBetweenWeightedActor(ReferenceOneMeshActor);
		ReferenceOneActor.bEnabled = true;
		ReferenceOneActor.Weight = ReferenceOneWeight;
		FActorModifierAlignBetweenWeightedActor ReferenceTwoActor = FActorModifierAlignBetweenWeightedActor(ReferenceTwoMeshActor);
		ReferenceTwoActor.bEnabled = true;
		ReferenceTwoActor.Weight = ReferenceTwoWeight;
		TSet<FActorModifierAlignBetweenWeightedActor> ReferenceActors = {
			ReferenceOneActor,
			ReferenceTwoActor
		};
		AlignBetweenModifier->SetReferenceActors(ReferenceActors);
	});

	AfterEach([this]
	{
		TestUtils->Destroy();
	});

	Describe("When Align Between modifier is applied to an actor", [this]
	{
		It("Should align Modified actor between two Reference actors accordingly", [this]
		{
			FVector ActualModifiedActorLocation = ModifiedMeshActor->GetActorTransform().GetLocation();

			TestNotEqual("Modified actor changed location", InitialModifiedActorLocation, ActualModifiedActorLocation);
			TestEqual("Modified actor is located according to modifier settings", ExpectedModifiedActorLocation,
			          ActualModifiedActorLocation);
			TestEqual("ReferenceOne actor location is not effected", InitialReferenceOneMeshActorLocation,
			          ReferenceOneMeshActor->GetActorTransform().GetLocation());
			TestEqual("ReferenceTwo actor location is not effected", InitialReferenceTwoMeshActorLocation,
			          ReferenceTwoMeshActor->GetActorTransform().GetLocation());
		});
	});
}
