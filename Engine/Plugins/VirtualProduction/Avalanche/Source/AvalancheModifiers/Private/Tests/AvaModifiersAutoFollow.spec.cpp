// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Modifiers/ActorModifierCoreBase.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/ActorModifierAutoFollowModifier.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "Tests/Framework/AvaModifiersTestUtils.h"
#include "Tests/Framework/AvaTestDynamicMeshActor.h"
#include "Tests/Framework/AvaTestUtils.h"

BEGIN_DEFINE_SPEC(AvalancheModifiersAutoFollowTests, "Avalanche.Modifiers.Autofollow",
                  EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	FName ModifierAutoFollowName;

	AAvaTestDynamicMeshActor* DynamicMeshActor;
	AAvaTestDynamicMeshActor* DynamicMeshActorToFollow;
	UActorModifierCoreSubsystem* ModifierSubsystem;
	UActorModifierCoreStack* ActorModifierStack;
	UActorModifierAutoFollowModifier* AutoFollowModifier;
	FTransform InitialActorState;

	TSharedPtr<FAvaTestUtils> TestUtils = MakeShared<FAvaTestUtils>();
	TSharedPtr<FAvaModifierTestUtils> ModifierTestUtils = MakeShared<FAvaModifierTestUtils>(TestUtils);

END_DEFINE_SPEC(AvalancheModifiersAutoFollowTests);

void AvalancheModifiersAutoFollowTests::Define()
{
	BeforeEach([this]
	{
		TestUtils->Init();
		DynamicMeshActor = ModifierTestUtils->SpawnTestDynamicMeshActor();
		ModifierSubsystem = UActorModifierCoreSubsystem::Get();
		ModifierAutoFollowName = ModifierTestUtils->GetModifierName(UActorModifierAutoFollowModifier::StaticClass());
		check(ModifierSubsystem);
	});

	AfterEach([this]()
	{
		DynamicMeshActor->Destroy();
		TestUtils->Destroy();
	});

	Describe("Autofollow modifier availability", [this]()
	{
		It("Should be available for Dynamic Mesh Actor", [this]()
		{
			TSet<FName> ActualModifiers = ModifierSubsystem->GetAllowedModifiers(DynamicMeshActor);
			TestTrue("Autofollow modifier is available for Dynamic Mesh Actor",
			         ModifierSubsystem->GetAllowedModifiers(DynamicMeshActor).Contains(ModifierAutoFollowName));
		});
	});

	Describe("Motion Design modifiers. Autofollow modifier application", [this]()
	{
		BeforeEach([this]()
		{
			DynamicMeshActorToFollow = TestUtils->SpawnTestDynamicMeshActor(FTransform(FVector(-100, -100, -100)));
			FText OutFailReason = FText::GetEmpty();
			FText* OutFailReasonPtr = &OutFailReason;
			ActorModifierStack = ModifierSubsystem->AddActorModifierStack(DynamicMeshActor);
			check(ActorModifierStack);
			FActorModifierCoreStackInsertOp InsertOp;

			InsertOp.NewModifierName = ModifierAutoFollowName;
			InsertOp.FailReason = OutFailReasonPtr;
			AutoFollowModifier = Cast<UActorModifierAutoFollowModifier>(ModifierSubsystem->InsertModifier(ActorModifierStack, InsertOp));
		});

		It("Should restore initial Dynamic mesh state once Autofollow modifier is disapplied", [this]()
		{
			FActorModifierCoreStackRemoveOp RemoveOp;
			if (ModifierSubsystem->RemoveModifiers({AutoFollowModifier}, RemoveOp))
			{
				FTransform RestoredActorState = DynamicMeshActor->GetActorTransform();
				TestEqual("Actor returned to initial location", InitialActorState.GetLocation(),
				          RestoredActorState.GetLocation());
			}
			else
			{
				TestTrue("Modifier wasn't removed", false);
			}
		});

		It("Should be able to aply to Dynamic Mesh Actor", [this]()
		{
			TestTrue("Modifier is a stack", ActorModifierStack->IsModifierStack());
			InitialActorState = DynamicMeshActor->GetActorTransform();
			if (AutoFollowModifier)
			{
				TestTrue("Modifier is not a stack", !AutoFollowModifier->IsModifierStack());
				FActorModifierSceneTreeActor ReferenceActor;
				ReferenceActor.ReferenceContainer = EActorModifierReferenceContainer::Other;
				ReferenceActor.ReferenceActorWeak = DynamicMeshActorToFollow;
				AutoFollowModifier->SetReferenceActor(ReferenceActor);
				FTransform NewActorState = DynamicMeshActor->GetActorTransform();
				TestNotEqual("Dynamic mesh position is changed", InitialActorState.GetLocation(),
				             NewActorState.GetLocation());
			}
			else
			{
				TestTrue("Modifier wasn't added", false);
			}
		});
	});
}
