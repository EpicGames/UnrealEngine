// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaModifiersActorUtils.h"
#include "Misc/AutomationTest.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/AvaAutoSizeModifier.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "Tests/Framework/AvaModifiersTestUtils.h"
#include "Tests/Framework/AvaTestDynamicMeshActor.h"
#include "Tests/Framework/AvaTestUtils.h"
#include "Utilities/ActorModifierActorUtils.h"

#if WITH_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(AvalancheModifiersAutoSize, "Avalanche.Modifiers.AutoSize",
                  EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	AAvaTestDynamicMeshActor* ReferenceActor;
	AAvaTestDynamicMeshActor* ModifiedActor;
	FActorModifierSceneTreeActor ActorQuery;

	FBox InitialReferencedActorBounds;
	FBox InitialModifiedActorBounds;

	UAvaAutoSizeModifier* AutoSizeModifier;

	TSharedPtr<FAvaTestUtils> TestUtils = MakeShared<FAvaTestUtils>();
	TSharedPtr<FAvaModifierTestUtils> ModifierTestUtils = MakeShared<FAvaModifierTestUtils>(TestUtils);

END_DEFINE_SPEC(AvalancheModifiersAutoSize);

void AvalancheModifiersAutoSize::Define()
{
	BeforeEach([this]
	{
		TestUtils->Init();

		// Spawn actors 
		ReferenceActor = ModifierTestUtils->SpawnTestDynamicMeshActor(FTransform(FVector(-100, -100, -100)));
		ModifiedActor = ModifierTestUtils->SpawnTestDynamicMeshActor(FTransform(FVector(100, 100, 100)));
		TestUtils->GenerateRectangleForDynamicMesh(ReferenceActor, 120, 70);
		TestUtils->GenerateRectangleForDynamicMesh(ModifiedActor, 10, 20);
		ActorQuery.ReferenceActorWeak = ReferenceActor;

		// Store initial actors data
		InitialReferencedActorBounds = UE::ActorModifier::ActorUtils::GetActorBounds(ReferenceActor);
		InitialModifiedActorBounds = UE::ActorModifier::ActorUtils::GetActorBounds(ModifiedActor);

		// Set up modifier stack
		const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
		const FName ModifierAutoSizeName = ModifierTestUtils->GetModifierName(UAvaAutoSizeModifier::StaticClass());
		FActorModifierCoreStackInsertOp InsertOp = ModifierTestUtils->GenerateInsertOp(ModifierAutoSizeName);
		UActorModifierCoreStack* ModifierStack = ModifierTestUtils->GenerateModifierStackForActor(ModifiedActor);
		AutoSizeModifier = Cast<UAvaAutoSizeModifier>(ModifierSubsystem->InsertModifier(ModifierStack, InsertOp));

		// Set up the modifier
		AutoSizeModifier->SetReferenceActor(ActorQuery);
	});

	AfterEach([this]
	{
		TestUtils->Destroy();
	});

	Describe("When AutoSise modifier is applied to a flat dynamic mesh actor", [this]
	{
		It("Should change it's size according to a reference object and the modifier settings", [this]
		{
			FBox ActualModifiedActorBounds = UE::ActorModifier::ActorUtils::GetActorBounds(ModifiedActor);
			FBox ActualReferenceActorBounds = UE::ActorModifier::ActorUtils::GetActorBounds(ReferenceActor);
			
			TestNotEqual("Modified actor bounds have changed", InitialModifiedActorBounds.Max,
			             ActualModifiedActorBounds.Max);
			TestEqual("Modified actor has the same bounds as the reference one", ActualModifiedActorBounds.Max,
					  ActualReferenceActorBounds.Max);
			TestEqual("Reference actor bounds have not changed", InitialReferencedActorBounds.Max,
			          ActualReferenceActorBounds.Max);
		});
	});
}


#endif
