// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Framework/AvaTestUtils.h"
#include "Tests/Framework/AvaModifiersTestUtils.h"
#include "Tests/Framework/AvaTestData.h"
#include "Tests/Framework/AvaTestDynamicMeshActor.h"
#include "Tests/Framework/AvaTestStaticMeshActor.h"
#include "Misc/AutomationTest.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"

BEGIN_DEFINE_SPEC(AvalancheModifiersCoreTests, "Avalanche.Modifiers.Core",
                  EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	FAvaTestData TestData;
	AAvaTestDynamicMeshActor* DynamicMeshActor;
	AAvaTestStaticMeshActor* StaticMeshActor;
	UActorModifierCoreSubsystem* ModifierSubsystem;

	TSharedPtr<FAvaTestUtils> TestUtils = MakeShared<FAvaTestUtils>();
	TSharedPtr<FAvaModifierTestUtils> ModifierTestUtils = MakeShared<FAvaModifierTestUtils>(TestUtils);

END_DEFINE_SPEC(AvalancheModifiersCoreTests);

void AvalancheModifiersCoreTests::Define()
{
	BeforeEach([this]
	{
		TestUtils->Init();
		DynamicMeshActor = ModifierTestUtils->SpawnTestDynamicMeshActor();
		StaticMeshActor = ModifierTestUtils->SpawnTestStaticMeshActor();
		ModifierSubsystem = UActorModifierCoreSubsystem::Get();
		check(ModifierSubsystem);
	});

	AfterEach([this]()
	{
		DynamicMeshActor->Destroy();
		StaticMeshActor->Destroy();
		TestUtils->Destroy();
	});

	Describe("When modifiers are requested for an actor", [this]()
	{
		It("Should provide correct list of modifiers available for a Dynamic Mesh actor", [this]()
		{
			TestUtils->GenerateRectangleForDynamicMesh(DynamicMeshActor);

			const TSet<FName> ActualAvailableModifiers = ModifierSubsystem->GetAllowedModifiers(DynamicMeshActor);
			const TSet<FName> MissingGeometryModifiers = TestData.ExpectedGeometryModifiers.Difference(
				ActualAvailableModifiers);
			const TSet<FName> MissingLayoutModifiers = TestData.ExpectedLayoutModifiersDynamicMesh.Difference(
				ActualAvailableModifiers);
			const TSet<FName> MissingRenderingModifiers = TestData.ExpectedRenderingModifiers.Difference(
				ActualAvailableModifiers);

			TestFalse("Conversion modifiers aren't included", ActualAvailableModifiers.
			          Includes(TestData.ExpectedConversionModifiers));
			if (!TestTrue("The List of Geometry modifiers is correct",
			              MissingGeometryModifiers.IsEmpty()))
			{
				ModifierTestUtils->LogMissingModifiers(MissingGeometryModifiers);
			}

			if (!TestTrue("The List of Layout modifiers is correct",
			              MissingLayoutModifiers.IsEmpty()))
			{
				ModifierTestUtils->LogMissingModifiers(MissingLayoutModifiers);
			}

			if (!TestTrue("The List of Rendering modifiers is correct",
			              MissingRenderingModifiers.IsEmpty()))
			{
				ModifierTestUtils->LogMissingModifiers(MissingRenderingModifiers);
			}
		});

		It("Should provide correct list of native modifiers for a Static Mesh actor", [this]()
		{
			const TSet<FName> ActualAvailableModifiers = ModifierSubsystem->GetAllowedModifiers(StaticMeshActor);
			const TSet<FName> MissingConversionModifiers = TestData.ExpectedConversionModifiers.Difference(
				ActualAvailableModifiers);
			const TSet<FName> MissingLayoutModifiers = TestData.ExpectedLayoutModifiersStaticMesh.Difference(
				ActualAvailableModifiers);
			const TSet<FName> MissingRenderingModifiers = TestData.ExpectedRenderingModifiers.Difference(
				ActualAvailableModifiers);

			TestFalse("Geometric modifiers for dynamic mesh are not included", ActualAvailableModifiers.
			          Includes(TestData.ExpectedGeometryModifiers));
			if (!TestTrue("The List of Conversion modifiers is correct",
						  MissingConversionModifiers.IsEmpty()))
			{
				ModifierTestUtils->LogMissingModifiers(MissingLayoutModifiers);
			}

			if (!TestTrue("The List of Layout modifiers is correct",
			              MissingLayoutModifiers.IsEmpty()))
			{
				ModifierTestUtils->LogMissingModifiers(MissingLayoutModifiers);
			}

			if (!TestTrue("The List of Rendering modifiers is correct",
			              MissingRenderingModifiers.IsEmpty()))
			{
				ModifierTestUtils->LogMissingModifiers(MissingRenderingModifiers);
			}
		});
	});
}
