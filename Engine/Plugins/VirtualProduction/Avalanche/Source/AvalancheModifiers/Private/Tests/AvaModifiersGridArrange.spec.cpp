// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/ActorModifierGridArrangeModifier.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "Tests/Framework/AvaModifiersTestUtils.h"
#include "Tests/Framework/AvaTestDynamicMeshActor.h"
#include "Tests/Framework/AvaTestUtils.h"

BEGIN_DEFINE_SPEC(AvalancheModifiersGridArrange, "Avalanche.Modifiers.GridArrange",
                  EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	AAvaTestDynamicMeshActor* ParentMeshActor;
	AAvaTestDynamicMeshActor* ChildOneMeshActor;
	AAvaTestDynamicMeshActor* ChildTwoMeshActor;
	AAvaTestDynamicMeshActor* ChildThreeMeshActor;
	AAvaTestDynamicMeshActor* ChildFourMeshActor;

	FTransform InitialParentActorState;
	FTransform InitialChildOneActorState;
	FTransform InitialChildTwoActorState;
	FTransform InitialChildThreeActorState;
	FTransform InitialChildFourActorState;

	UActorModifierGridArrangeModifier* GridArrangeModifier;
	FIntPoint GridCount;
	FVector2d GridSpread;

	TSharedPtr<FAvaTestUtils> TestUtils = MakeShared<FAvaTestUtils>();
	TSharedPtr<FAvaModifierTestUtils> ModifierTestUtils = MakeShared<FAvaModifierTestUtils>(TestUtils);

	void TestActorsLocationChanged(AActor* InActor, FTransform InInitialTransform, FVector InExpectedLocation)
	{
		if (!TestNotEqual((TEXT("Child %s Actor's position is changed"), *InActor->GetActorNameOrLabel()),
		                  InInitialTransform.GetLocation(),
		                  InActor->GetActorTransform().GetLocation()))
		{
			TestUtils->LogActorLocation(InActor);
		}
		else
		{
			TestEqual((TEXT("Child %s was located as expected"), *InActor->GetActorNameOrLabel()),
			          InActor->GetActorTransform().GetLocation(), InExpectedLocation);
		}
	}

	void TestActorsLocationNotChanged(AActor* InActor, FTransform InInitialTransform)
	{
		if (!TestEqual((TEXT("Child %s Actor's position is changed"), *InActor->GetActorNameOrLabel()),
		               InInitialTransform.GetLocation(),
		               InActor->GetActorTransform().GetLocation()))
		{
			TestUtils->LogActorLocation(InActor);
		}
	}

END_DEFINE_SPEC(AvalancheModifiersGridArrange);

void AvalancheModifiersGridArrange::Define()
{
	BeforeEach([this]
	{
		TestUtils->Init();

		// Spawn actors
		ParentMeshActor = ModifierTestUtils->SpawnTestDynamicMeshActor();
		ChildOneMeshActor = ModifierTestUtils->SpawnTestDynamicMeshActor();
		ChildTwoMeshActor = ModifierTestUtils->SpawnTestDynamicMeshActor();
		ChildThreeMeshActor = ModifierTestUtils->SpawnTestDynamicMeshActor();
		ChildFourMeshActor = ModifierTestUtils->SpawnTestDynamicMeshActor();

		TestUtils->GenerateRectangleForDynamicMesh(ParentMeshActor);
		TestUtils->GenerateRectangleForDynamicMesh(ChildOneMeshActor);
		TestUtils->GenerateRectangleForDynamicMesh(ChildTwoMeshActor);
		TestUtils->GenerateRectangleForDynamicMesh(ChildThreeMeshActor);
		TestUtils->GenerateRectangleForDynamicMesh(ChildFourMeshActor);

		// Attach Child actors to a Parent one
		ChildOneMeshActor->AttachToActor(ParentMeshActor, FAttachmentTransformRules::KeepRelativeTransform);
		ChildTwoMeshActor->AttachToActor(ParentMeshActor, FAttachmentTransformRules::KeepRelativeTransform);
		ChildThreeMeshActor->AttachToActor(ParentMeshActor, FAttachmentTransformRules::KeepRelativeTransform);
		ChildFourMeshActor->AttachToActor(ParentMeshActor, FAttachmentTransformRules::KeepRelativeTransform);

		// Store initial objects transform data
		InitialParentActorState = ParentMeshActor->GetActorTransform();
		InitialChildOneActorState = ChildOneMeshActor->GetActorTransform();
		InitialChildTwoActorState = ChildTwoMeshActor->GetActorTransform();
		InitialChildThreeActorState = ChildThreeMeshActor->GetActorTransform();
		InitialChildFourActorState = ChildFourMeshActor->GetActorTransform();

		// Set up modifier
		const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
		const FName ModifierGridArrangeName = ModifierTestUtils->GetModifierName(UActorModifierGridArrangeModifier::StaticClass());
		FActorModifierCoreStackInsertOp InsertOp = ModifierTestUtils->GenerateInsertOp(ModifierGridArrangeName);
		UActorModifierCoreStack* ModifierStack = ModifierTestUtils->GenerateModifierStackForActor(ParentMeshActor);
		GridArrangeModifier = Cast<UActorModifierGridArrangeModifier>(ModifierSubsystem->InsertModifier(ModifierStack, InsertOp));
	});

	AfterEach([this]
	{
		TestUtils->Destroy();
	});

	Describe("When Grid Arrange modifier is applied to a parent mesh actor", [this]
	{
		It("Should align all child actors according to Grid Arrange settings", [this]
		{
			// Set up the modifier
			GridCount = FIntPoint(3, 2);
			GridSpread = FVector2d(200.f, 150.f);
			GridArrangeModifier->SetCount(GridCount);
			GridArrangeModifier->SetSpread(GridSpread);

			// Add expectations
			const FVector ExpectedChildOneLocation = FVector(0, 0, -1 * GridSpread.Y);
			const FVector ExpectedChildTwoLocation = FVector(0, 2 * GridSpread.X, 0);
			const FVector ExpectedChildThreeLocation = FVector(0, 1 * GridSpread.X, 0);

			TestActorsLocationChanged(ChildOneMeshActor, InitialChildOneActorState, ExpectedChildOneLocation);
			TestActorsLocationChanged(ChildTwoMeshActor, InitialChildTwoActorState, ExpectedChildTwoLocation);
			TestActorsLocationChanged(ChildThreeMeshActor, InitialChildThreeActorState, ExpectedChildThreeLocation);
			TestActorsLocationNotChanged(ChildFourMeshActor, InitialChildFourActorState);
			TestActorsLocationNotChanged(ParentMeshActor, InitialParentActorState);
		});
	});
}
