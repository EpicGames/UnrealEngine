// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/ActorModifierJustifyModifier.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "Tests/Framework/AvaModifiersTestUtils.h"
#include "Tests/Framework/AvaTestDynamicMeshActor.h"
#include "Tests/Framework/AvaTestUtils.h"

BEGIN_DEFINE_SPEC(AvalancheModifiersJustify, "Avalanche.Modifiers.Justify",
                  EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	AAvaTestDynamicMeshActor* ParentActor;
	AAvaTestDynamicMeshActor* ChildOneActor;
	AAvaTestDynamicMeshActor* ChildTwoActor;
	AAvaTestDynamicMeshActor* ChildThreeActor;
	double StepBetweenActors = 10;
	double AnchorValue = 15;
	double SquareSideSize = 2;

	FVector InitialParentLocation;
	FVector InitialChilOneLocation;
	FVector InitialChildTwoLocation;
	FVector InitialChildThreeLocation;

	UActorModifierJustifyModifier* JustifyModifier;

	TSharedPtr<FAvaTestUtils> TestUtils = MakeShared<FAvaTestUtils>();
	TSharedPtr<FAvaModifierTestUtils> ModifierTestUtils = MakeShared<FAvaModifierTestUtils>(TestUtils);

	void TestActorsLocationChanged(FVector InInitialActorLocation, FVector CurrentActorLocation,
	                               bool LocationChanged = true)
	{
		if (LocationChanged)
		{
			TestNotEqual("Vertical value is changed", CurrentActorLocation.Z,
			             InInitialActorLocation.Z);
			TestNotEqual("Horizontal value is changed", CurrentActorLocation.Y, InInitialActorLocation.Y);
			TestNotEqual("Depth value is changed", CurrentActorLocation.X, InInitialActorLocation.X);
		}
		else
		{
			TestEqual("Vertical value is not changed", CurrentActorLocation.Z,
			          InInitialActorLocation.Z);
			TestEqual("Horizontal value is not changed", CurrentActorLocation.Y, InInitialActorLocation.Y);
			TestEqual("Depth value is not changed", CurrentActorLocation.X, InInitialActorLocation.X);
		}
	}

END_DEFINE_SPEC(AvalancheModifiersJustify);

void AvalancheModifiersJustify::Define()
{
	BeforeEach([this]
	{
		TestUtils->Init();

		// Spawn actors
		ParentActor = ModifierTestUtils->SpawnTestDynamicMeshActor(FTransform(FVector::ZeroVector));
		ChildOneActor = ModifierTestUtils->SpawnTestDynamicMeshActor(
			FTransform(FVector(StepBetweenActors, StepBetweenActors, StepBetweenActors)));
		ChildTwoActor = ModifierTestUtils->SpawnTestDynamicMeshActor(
			FTransform(FVector(2 * StepBetweenActors, 2 * StepBetweenActors, 2 * StepBetweenActors)));
		ChildThreeActor = ModifierTestUtils->SpawnTestDynamicMeshActor(
			FTransform(FVector(3 * StepBetweenActors, 3 * StepBetweenActors, 3 * StepBetweenActors)));

		TestUtils->GenerateRectangleForDynamicMesh(ChildOneActor, SquareSideSize, SquareSideSize);
		TestUtils->GenerateRectangleForDynamicMesh(ChildTwoActor, SquareSideSize, SquareSideSize);
		TestUtils->GenerateRectangleForDynamicMesh(ChildThreeActor, SquareSideSize, SquareSideSize);

		// Attach Child actors to a Parent one
		ChildOneActor->AttachToActor(ParentActor, FAttachmentTransformRules::KeepRelativeTransform);
		ChildTwoActor->AttachToActor(ParentActor, FAttachmentTransformRules::KeepRelativeTransform);
		ChildThreeActor->AttachToActor(ParentActor, FAttachmentTransformRules::KeepRelativeTransform);

		// Store initial objects data
		InitialParentLocation = ParentActor->GetActorTransform().GetLocation();
		InitialChilOneLocation = ChildOneActor->GetActorTransform().GetLocation();
		InitialChildTwoLocation = ChildTwoActor->GetActorTransform().GetLocation();
		InitialChildThreeLocation = ChildThreeActor->GetActorTransform().GetLocation();

		// Set up modifier
		const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
		const FName ModifierJustifyName = ModifierTestUtils->GetModifierName(UActorModifierJustifyModifier::StaticClass());
		FActorModifierCoreStackInsertOp InsertOp = ModifierTestUtils->GenerateInsertOp(ModifierJustifyName);
		UActorModifierCoreStack* ModifierStack = ModifierTestUtils->GenerateModifierStackForActor(ParentActor);
		JustifyModifier = Cast<UActorModifierJustifyModifier>(ModifierSubsystem->InsertModifier(ModifierStack, InsertOp));
		JustifyModifier->SetHorizontalAlignment(EActorModifierJustifyHorizontal::Right);
		JustifyModifier->SetVerticalAlignment(EActorModifierJustifyVertical::Bottom);
		JustifyModifier->SetDepthAlignment(EActorModifierJustifyDepth::Center);
		JustifyModifier->SetDepthAnchor(AnchorValue);
	});

	AfterEach([this]
	{
		TestUtils->Destroy();
	});

	Describe("When Justify modifier is applied to a parent actor with a group of actors under it", [this]
	{
		It("Should adjust child actors according to the modifier settings", [this]
		{
			const FVector CurrentParentLocation = ParentActor->GetActorTransform().GetLocation();
			const FVector CurrentChildOneLocation = ChildOneActor->GetActorTransform().GetLocation();
			const FVector CurrentChildTwoLocation = ChildTwoActor->GetActorTransform().GetLocation();
			const FVector CurrentChildThreeLocation = ChildThreeActor->GetActorTransform().GetLocation();

			// Child locations are changed
			TestActorsLocationChanged(InitialParentLocation, CurrentParentLocation, false);
			TestActorsLocationChanged(InitialChilOneLocation, CurrentChildOneLocation);
			TestActorsLocationChanged(InitialChildTwoLocation, CurrentChildTwoLocation);
			TestActorsLocationChanged(InitialChildThreeLocation, CurrentChildThreeLocation);


			// Child actors aligned correctly horizontally Right. Distance between actors is respected
			TestEqual("Child one aligned correctly horisontally", CurrentChildOneLocation.Y,
			          InitialParentLocation.Y - SquareSideSize - 2 * StepBetweenActors);
			TestEqual("Child two aligned correctly horisontally", CurrentChildTwoLocation.Y,
			          InitialParentLocation.Y - SquareSideSize - 1 * StepBetweenActors);
			TestEqual("Child three aligned correctly horisontally", CurrentChildThreeLocation.Y,
			          InitialParentLocation.Y - SquareSideSize);

			// Child actors aligned correctly vertically Bottom. Distance between actors is respected
			TestEqual("Child one aligned correctly vertically", CurrentChildOneLocation.Z,
			          InitialParentLocation.Z);
			TestEqual("Child two aligned correctly vertically", CurrentChildTwoLocation.Z,
			          InitialParentLocation.Z + StepBetweenActors);
			TestEqual("Child three aligned correctly vertically", CurrentChildThreeLocation.Z,
			          InitialParentLocation.Z + 2 * StepBetweenActors);

			// Child actors aligned correctly in depth Center with anchor. Distance between actors is respected
			TestEqual("Child one aligned correctly in depth", CurrentChildOneLocation.X,
			          InitialParentLocation.X + AnchorValue - StepBetweenActors);
			TestEqual("Child two aligned correctly in depth", CurrentChildTwoLocation.X,
			          InitialParentLocation.X + AnchorValue);
			TestEqual("Child three aligned correctly in depth", CurrentChildThreeLocation.X,
			          InitialParentLocation.X + AnchorValue + StepBetweenActors);
		});
	});
}
