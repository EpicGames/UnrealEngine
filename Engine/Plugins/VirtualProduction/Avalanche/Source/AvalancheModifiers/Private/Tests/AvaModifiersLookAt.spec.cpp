// Copyright Epic Games, Inc. All Rights Reserved.

#include "Kismet/KismetMathLibrary.h"
#include "Misc/AutomationTest.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/ActorModifierLookAtModifier.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "Tests/Framework/AvaModifiersTestUtils.h"
#include "Tests/Framework/AvaTestDynamicMeshActor.h"
#include "Tests/Framework/AvaTestUtils.h"

BEGIN_DEFINE_SPEC(AvalancheModifiersLookAt, "Avalanche.Modifiers.LookAt",
                  EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	AAvaTestDynamicMeshActor* ModifiedActor;
	AAvaTestDynamicMeshActor* ReferenceActor;
	double RandomMin = -2000;
	double RandomMax = 2000;

	FTransform InitialModifiedState;
	FTransform InitialReferenceSate;

	UActorModifierLookAtModifier* LookAtModifier;

	TSharedPtr<FAvaTestUtils> TestUtils = MakeShared<FAvaTestUtils>();
	TSharedPtr<FAvaModifierTestUtils> ModifierTestUtils = MakeShared<FAvaModifierTestUtils>(TestUtils);

END_DEFINE_SPEC(AvalancheModifiersLookAt);

void AvalancheModifiersLookAt::Define()
{
	BeforeEach([this]
	{
		TestUtils->Init();

		// Spawn actors
		ModifiedActor = ModifierTestUtils->SpawnTestDynamicMeshActor(FTransform(FVector(
			FMath::RandRange(RandomMin, RandomMax),
			FMath::RandRange(RandomMin, RandomMax),
			FMath::RandRange(RandomMin, RandomMax))));
		ReferenceActor = ModifierTestUtils->SpawnTestDynamicMeshActor(FTransform(FVector(
			FMath::RandRange(RandomMin, RandomMax),
			FMath::RandRange(RandomMin, RandomMax),
			FMath::RandRange(RandomMin, RandomMax))));

		// Store initial actors' state
		InitialModifiedState = ModifiedActor->GetActorTransform();
		InitialReferenceSate = ReferenceActor->GetActorTransform();

		// Set up modifier
		const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
		const FName ModifierLookAtName = ModifierTestUtils->GetModifierName(UActorModifierLookAtModifier::StaticClass());
		FActorModifierCoreStackInsertOp InsertOp = ModifierTestUtils->GenerateInsertOp(ModifierLookAtName);
		UActorModifierCoreStack* ModifierStack = ModifierTestUtils->GenerateModifierStackForActor(ModifiedActor);
		LookAtModifier = Cast<UActorModifierLookAtModifier>(ModifierSubsystem->InsertModifier(ModifierStack, InsertOp));
		LookAtModifier->SetReferenceActor(FActorModifierSceneTreeActor(ReferenceActor));
	});

	AfterEach([this]
	{
		TestUtils->Destroy();
	});

	Describe("When LookAt modifier is applied to an actor", [this]
	{
		It("Should rotate modified actor towards the reference actor location", [this]
		{
			const FVector CurrentModifiedLocation = ModifiedActor->GetActorLocation();
			const FVector CurrentReferenceLocation = ReferenceActor->GetActorLocation();
			const FRotator ExpectedRotator = UKismetMathLibrary::FindLookAtRotation(
				CurrentModifiedLocation, CurrentReferenceLocation);

			TestEqual("Modified actor didn't change location",
			          CurrentModifiedLocation, InitialModifiedState.GetLocation());
			TestEqual("Reference actor didn't change location",
			          CurrentReferenceLocation, InitialReferenceSate.GetLocation());
			TestEqual("Rotator value wasn't changed for the Reference actor",
			          ReferenceActor->GetActorRotation(), InitialReferenceSate.Rotator());

			if (!TestNearlyEqual("Modified actor Rotator value is valid",
			                     ModifiedActor->GetActorRotation(), ExpectedRotator))
			{
				UE_LOGF(LogAvaModifiersTest, Display, "Current modified location: %ls",
				       *CurrentModifiedLocation.ToString());
				UE_LOGF(LogAvaModifiersTest, Display, "Current reference  location: %ls",
				       *CurrentReferenceLocation.ToString());
			}
		});
	});
}
