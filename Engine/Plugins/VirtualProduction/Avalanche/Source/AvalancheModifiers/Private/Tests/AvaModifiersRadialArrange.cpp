// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/ActorModifierRadialArrangeModifier.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "Tests/Framework/AvaModifiersTestUtils.h"
#include "Tests/Framework/AvaTestDynamicMeshActor.h"
#include "Tests/Framework/AvaTestUtils.h"

BEGIN_DEFINE_SPEC(AvalancheModifiersRadialArrange, "Avalanche.Modifiers.RadialArrange",
                  EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	AAvaTestDynamicMeshActor* ParentActor;
	TArray<AAvaTestDynamicMeshActor*> ChildActors;
	double InnerRadius;

	FVector InitialParentLocation;

	UActorModifierRadialArrangeModifier* RadialArrangeModifier;

	TSharedPtr<FAvaTestUtils> TestUtils = MakeShared<FAvaTestUtils>();
	TSharedPtr<FAvaModifierTestUtils> ModifierTestUtils = MakeShared<FAvaModifierTestUtils>(TestUtils);

END_DEFINE_SPEC(AvalancheModifiersRadialArrange);

void AvalancheModifiersRadialArrange::Define()
{
	BeforeEach([this]
	{
		TestUtils->Init();

		// Spawn actors
		ParentActor = ModifierTestUtils->SpawnTestDynamicMeshActor(FTransform(FVector::ZeroVector));
		ChildActors = ModifierTestUtils->SpawnTestDynamicMeshActors(4, ParentActor);

		// Store initial actors' state
		InitialParentLocation = ParentActor->GetActorLocation();

		// Set up modifier
		const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
		const FName ModifierName = ModifierTestUtils->GetModifierName(UActorModifierRadialArrangeModifier::StaticClass());
		FActorModifierCoreStackInsertOp InsertOp = ModifierTestUtils->GenerateInsertOp(ModifierName);
		UActorModifierCoreStack* ModifierStack = ModifierTestUtils->GenerateModifierStackForActor(ParentActor);
		RadialArrangeModifier = Cast<UActorModifierRadialArrangeModifier>(ModifierSubsystem->InsertModifier(ModifierStack, InsertOp));
		InnerRadius = FMath::RandRange(-1000, 1000);
		RadialArrangeModifier->SetInnerRadius(InnerRadius);
	});

	AfterEach([this]
	{
		TestUtils->Destroy();
	});

	Describe("When Radial Arrange modifier is applied to a parent actor with children", [this]
	{
		It("Should arrange child actors in a radial layout around parent actor", [this]
		{
			TArray<FVector> ExpectedChildrenLocations = {
				FVector(-InnerRadius, 0, 0),
				FVector(0, InnerRadius, 0),
				FVector(InnerRadius, 0, 0),
				FVector(0, -InnerRadius, 0)
			};

			for (int32 i = 0; i < ChildActors.Num(); i++)
			{
				TestEqual("Child one is placed correctly", ChildActors[i]->GetActorLocation(),
				          ExpectedChildrenLocations[i]);
			}

			TestEqual("Parent didn't change location", ParentActor->GetActorLocation(), InitialParentLocation);
		});
	});
}
