// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Components/PrimitiveComponent.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/AvaTranslucentPriorityModifier.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "Tests/Framework/AvaModifiersTestUtils.h"
#include "Tests/Framework/AvaTestDynamicMeshActor.h"
#include "Tests/Framework/AvaTestUtils.h"

BEGIN_DEFINE_SPEC(AvalancheModifiersTranslucentPriority, "Avalanche.Modifiers.TranslucentPriority",
                  EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)
	AAvaTestDynamicMeshActor* ModifiedActor;
	TArray<UPrimitiveComponent*> ModifiedActorComponents;
	int32 ModifierSortPriority;

	UAvaTranslucentPriorityModifier* TranslucentPriorityModifier;

	TSharedPtr<FAvaTestUtils> TestUtils = MakeShared<FAvaTestUtils>();
	TSharedPtr<FAvaModifierTestUtils> ModifierTestUtils = MakeShared<FAvaModifierTestUtils>(TestUtils);
END_DEFINE_SPEC(AvalancheModifiersTranslucentPriority);

void AvalancheModifiersTranslucentPriority::Define()
{
	BeforeEach([this]
	{
		TestUtils->Init();

		do
		{
			ModifierSortPriority = FMath::RandRange(-100, 100);
		}
		while (ModifierSortPriority == 0);

		ModifiedActor = TestUtils->SpawnTestDynamicMeshActor();

		ModifiedActor->GetComponents(ModifiedActorComponents);

		const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
		const FName ModifierName = ModifierTestUtils->GetModifierName(UAvaTranslucentPriorityModifier::StaticClass());
		FActorModifierCoreStackInsertOp InsertOp = ModifierTestUtils->GenerateInsertOp(ModifierName);
		UActorModifierCoreStack* ModifierStack = ModifierTestUtils->GenerateModifierStackForActor(ModifiedActor);
		TranslucentPriorityModifier = Cast<UAvaTranslucentPriorityModifier>(ModifierSubsystem->InsertModifier(ModifierStack, InsertOp));
		TranslucentPriorityModifier->SetMode(EAvaTranslucentPriorityModifierMode::Manual);
		TranslucentPriorityModifier->SetSortPriorityOffset(0);
		TranslucentPriorityModifier->SetSortPriorityStep(0);
		TranslucentPriorityModifier->SetSortPriority(ModifierSortPriority);
	});

	AfterEach([this]
	{
		TestUtils->Destroy();
	});

	Describe("When TranslucentPriority modifier is applied to an actor in manual mode", [this]
	{
		It("Should update Translucency Sort Priority value of all modified actor primitive components", [this]
		{
			for (const UPrimitiveComponent* PrimitiveComponent : ModifiedActorComponents)
			{
				const int32 CurrentTranslucencySortPriority = PrimitiveComponent->TranslucencySortPriority;
				TestEqual("Translucency Sort Priority is equal to Modifier Sort Priority",
				          CurrentTranslucencySortPriority,
				          ModifierSortPriority);
			}
		});
	});
}
