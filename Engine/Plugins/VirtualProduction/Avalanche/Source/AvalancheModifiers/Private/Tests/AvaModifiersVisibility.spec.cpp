// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/AvaVisibilityModifier.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "Tests/Framework/AvaModifiersTestUtils.h"
#include "Tests/Framework/AvaTestDynamicMeshActor.h"
#include "Tests/Framework/AvaTestUtils.h"

BEGIN_DEFINE_SPEC(AvalancheModifiersVisibility, "Avalanche.Modifiers.Visibility",
                  EAutomationTestFlags::ProductFilter | EAutomationTestFlags_ApplicationContextMask)

	AAvaTestDynamicMeshActor* ParentActor;
	TArray<AAvaTestDynamicMeshActor*> ChildrenActors;
	int32 NumberOfChildren;
	int32 NumberOfChildrenVisible;
	int32 LastVisibleChild;

	UAvaVisibilityModifier* VisibilityModifier;

	TSharedPtr<FAvaTestUtils> TestUtils = MakeShared<FAvaTestUtils>();
	TSharedPtr<FAvaModifierTestUtils> ModifierTestUtils = MakeShared<FAvaModifierTestUtils>(TestUtils);

	void VerifyIfAssertionIsValid(bool AssertionResult, int32 ForLoopIndex)
	{
		if (!AssertionResult)
		{
			UE_LOGF(LogAvaModifiersTest, Warning, "Something whent wrong for child actor index: %d", ForLoopIndex);
		}
	}

END_DEFINE_SPEC(AvalancheModifiersVisibility);

void AvalancheModifiersVisibility::Define()
{
	BeforeEach([this]
	{
		TestUtils->Init();
		NumberOfChildren = FMath::RandRange(1, 20);
		NumberOfChildrenVisible = FMath::RandRange(1, NumberOfChildren);
		LastVisibleChild = NumberOfChildren - NumberOfChildrenVisible;

		// Spawn actors
		ParentActor = ModifierTestUtils->SpawnTestDynamicMeshActor();
		ChildrenActors = ModifierTestUtils->SpawnTestDynamicMeshActors(NumberOfChildren, ParentActor);

		// Set up modifier
		const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
		const FName ModifierName = ModifierTestUtils->GetModifierName(UAvaVisibilityModifier::StaticClass());
		FActorModifierCoreStackInsertOp InsertOp = ModifierTestUtils->GenerateInsertOp(ModifierName);
		UActorModifierCoreStack* ModifierStack = ModifierTestUtils->GenerateModifierStackForActor(ParentActor);
		VisibilityModifier = Cast<UAvaVisibilityModifier>(ModifierSubsystem->InsertModifier(ModifierStack, InsertOp));
		VisibilityModifier->SetIndex(NumberOfChildrenVisible - 1);
	});

	AfterEach([this]
	{
		TestUtils->Destroy();
	});

	Describe("When Visibility modifier is applied to a parent actor with children", [this]
	{
		It("Should set visibility for a range of children according to modifier settings", [this]
		{
			VisibilityModifier->SetTreatAsRange(true);

			// Check visible actors
			for (int32 i = NumberOfChildren - 1; i >= LastVisibleChild; i--)
			{
				VerifyIfAssertionIsValid(TestTrue("The actor is visible", !ChildrenActors[i]->IsHidden()), i);
			}

			// Check hidden actors
			for (int32 i = LastVisibleChild - 1; i >= 0; i--)
			{
				VerifyIfAssertionIsValid(TestTrue("The actor is hidden", ChildrenActors[i]->IsHidden()), i);
			}
		});

		It("Should set visibility for a single child according to modifier settings", [this]
		{
			VisibilityModifier->SetTreatAsRange(false);

			for (int32 i = NumberOfChildren - 1; i >= 0; i--)
			{
				if (i == LastVisibleChild)
				{
					VerifyIfAssertionIsValid(TestTrue("This actor is visible", !ChildrenActors[i]->IsHidden()), i);
				}
				else
				{
					VerifyIfAssertionIsValid(TestTrue("This actor is hidden", ChildrenActors[i]->IsHidden()), i);
				}
			}
		});
	});
}
