// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Framework/AvaModifiersTestUtils.h"

#include "AvaShapeActor.h"
#include "AvaShapeParametricMaterial.h"
#include "AvaShapePrimitiveFunctions.h"
#include "Components/DynamicMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "DynamicMeshes/AvaShapeRectangleDynMesh.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Logging/LogVerbosity.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Misc/AutomationTest.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "Tests/Framework/AvaTestDynamicMeshActor.h"
#include "Tests/Framework/AvaTestStaticMeshActor.h"

// General Motion Design Modifiers unit test log
DEFINE_LOG_CATEGORY(LogAvaModifiersTest);

FAvaModifierTestUtils::FAvaModifierTestUtils(TSharedPtr<FAvaTestUtils> InTestUtils)
	: TestUtils(InTestUtils)
{
}

AAvaShapeActor* FAvaModifierTestUtils::SpawnShapeActor()
{
	AAvaShapeActor* ShapeActor = TestUtils->SpawnActor<AAvaShapeActor>();

	UAvaShapeMeshFunctions::SetRectangle(ShapeActor, { 100, 100 }, FTransform::Identity);

	return ShapeActor;
}

AAvaTestDynamicMeshActor* FAvaModifierTestUtils::SpawnTestDynamicMeshActor(FTransform InTransform)
{
	AAvaTestDynamicMeshActor* Result = TestUtils->SpawnTestDynamicMeshActor(InTransform);

	FAvaShapeParametricMaterial DynamicMaterial;
	Result->GetMeshComponent()->SetMaterial(UAvaShapeDynamicMeshBase::MESH_INDEX_PRIMARY, DynamicMaterial.GetOrCreateMaterial(Result));

	return Result;
}

AAvaTestStaticMeshActor* FAvaModifierTestUtils::SpawnTestStaticMeshActor()
{
	AAvaTestStaticMeshActor* Result = TestUtils->SpawnTestStaticMeshActor();

	FAvaShapeParametricMaterial DynamicMaterial;
	Result->GetMeshComponent()->SetMaterial(UAvaShapeDynamicMeshBase::MESH_INDEX_PRIMARY, DynamicMaterial.GetOrCreateMaterial(Result));

	return Result;
}

TArray<AAvaTestDynamicMeshActor*> FAvaModifierTestUtils::SpawnTestDynamicMeshActors(
	int32 InNumberOfActors
	, AAvaTestDynamicMeshActor* InParentActor)
{
	TArray<AAvaTestDynamicMeshActor*> Result = TestUtils->SpawnTestDynamicMeshActors(InNumberOfActors, InParentActor);

	for (AAvaTestDynamicMeshActor* Actor : Result)
	{
		FAvaShapeParametricMaterial DynamicMaterial;
		Actor->GetMeshComponent()->SetMaterial(UAvaShapeDynamicMeshBase::MESH_INDEX_PRIMARY, DynamicMaterial.GetOrCreateMaterial(Actor));
	}

	return Result;
}

FName FAvaModifierTestUtils::GetModifierName(TSubclassOf<UActorModifierCoreBase> InModifierClass)
{
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	return ModifierSubsystem->GetRegisteredModifierName(InModifierClass.Get());
}

UActorModifierCoreStack* FAvaModifierTestUtils::GenerateModifierStackForActor(AActor* InModifiedActor)
{
	UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	check(ModifierSubsystem);
	UActorModifierCoreStack* ActorModifierStack = ModifierSubsystem->AddActorModifierStack(
		InModifiedActor);
	check(ActorModifierStack);
	return ActorModifierStack;
}

FActorModifierCoreStackInsertOp FAvaModifierTestUtils::GenerateInsertOp(FName InModifierName)
{
	FText OutFailReason = FText::GetEmpty();
	FText* OutFailReasonPtr = &OutFailReason;
	FActorModifierCoreStackInsertOp InsertOp;
	InsertOp.NewModifierName = InModifierName;
	InsertOp.FailReason = OutFailReasonPtr;
	return InsertOp;
}

void FAvaModifierTestUtils::LogMissingModifiers(const TSet<FName>& InMissingModifiers)
{
	FString MismatchingModifiers;

	for (const FName& Elem : InMissingModifiers)
	{
		if (MismatchingModifiers.Len() > 0)
		{
			MismatchingModifiers.Append(", ");
		}
		MismatchingModifiers.Append(*Elem.ToString());
	}

	UE_LOGF(LogAvaModifiersTest, Warning, "List of mismatching modifiers: %ls", *MismatchingModifiers);
}
