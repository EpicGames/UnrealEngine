// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/Framework/AvaTestData.h"

FAvaTestData::FAvaTestData()
{
	ExpectedConversionModifiers =
	{
		"DynamicMeshConverter"
	};

	ExpectedGeometryModifiers =
	{
		"Bend",
		"Bevel",
		"Boolean",
		"Extrude",
		"Mirror",
		"Normal",
		"Pattern",
		"PlaneCut",
		"Subdivide",
		"Taper",
	};

	ExpectedLayoutModifiersDynamicMesh =
	{
		"AlignBetween",
		"AutoFollow",
		"AutoSize",
		"GlobalOpacity",
		"GridArrange",
		"Justify",
		"LookAt",
		"MaterialParameter",
		"RadialArrange",
		"TranslucentSortPriority",
		"Visibility"
	};

	ExpectedLayoutModifiersStaticMesh =
	{
		"AlignBetween",
		"AutoFollow",
		"GlobalOpacity",
		"GridArrange",
		"Justify",
		"LookAt",
		"MaterialParameter",
		"RadialArrange",
		"TranslucentSortPriority",
		"Visibility"
	};

	ExpectedRenderingModifiers =
	{
		"TranslucentSortPriority",
		"Visibility"
	};
}
