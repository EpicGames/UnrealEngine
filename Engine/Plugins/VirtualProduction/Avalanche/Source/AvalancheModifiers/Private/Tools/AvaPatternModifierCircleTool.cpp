// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaPatternModifierCircleTool.h"

void UAvaPatternModifierCircleTool::SetCirclePlane(EAvaPatternModifierPlane InCirclePlane)
{
	if (CirclePlane == InCirclePlane)
	{
		return;
	}

	CirclePlane = InCirclePlane;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierCircleTool::SetCircleRadius(float InCircleRadius)
{
	if (FMath::IsNearlyEqual(CircleRadius, InCircleRadius))
	{
		return;
	}

	CircleRadius = InCircleRadius;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierCircleTool::SetCircleStartAngle(float InCircleStartAngle)
{
	if (FMath::IsNearlyEqual(CircleStartAngle, InCircleStartAngle))
	{
		return;
	}

	CircleStartAngle = InCircleStartAngle;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierCircleTool::SetCircleFullAngle(float InCircleFullAngle)
{
	if (FMath::IsNearlyEqual(CircleFullAngle, InCircleFullAngle))
	{
		return;
	}

	CircleFullAngle = InCircleFullAngle;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierCircleTool::SetCircleCount(int32 InCircleCount)
{
	InCircleCount = FMath::Clamp(InCircleCount, 1, 10000);
	if (CircleCount == InCircleCount)
	{
		return;
	}

	CircleCount = InCircleCount;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierCircleTool::SetCircleAccumulateTransform(bool bInCircleAccumulateTransform)
{
	if (bCircleAccumulateTransform == bInCircleAccumulateTransform)
	{
		return;
	}

	bCircleAccumulateTransform = bInCircleAccumulateTransform;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierCircleTool::SetCircleRotation(const FRotator& InCircleRotation)
{
	if (CircleRotation.Equals(InCircleRotation))
	{
		return;
	}

	CircleRotation = InCircleRotation;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierCircleTool::SetCircleScale(const FVector& InCircleScale)
{
	if (CircleScale.Equals(InCircleScale))
	{
		return;
	}

	CircleScale = InCircleScale;
	OnToolPropertiesChanged();
}

#if WITH_EDITOR
void UAvaPatternModifierCircleTool::PostEditChangeProperty(FPropertyChangedEvent& InEvent)
{
	Super::PostEditChangeProperty(InEvent);

	static const TSet<FName> CirclePropertyNames
	{
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierCircleTool, CirclePlane),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierCircleTool, CircleRadius),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierCircleTool, CircleStartAngle),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierCircleTool, CircleFullAngle),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierCircleTool, CircleCount),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierCircleTool, bCircleAccumulateTransform),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierCircleTool, CircleRotation),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierCircleTool, CircleScale),
	};

	const FName MemberName = InEvent.GetMemberPropertyName();
	const bool bAnyPropertyChanged = MemberName.IsNone();

	if (bAnyPropertyChanged || MemberName == GET_MEMBER_NAME_CHECKED(UAvaPatternModifierCircleTool, CircleCount))
	{
		SetCircleCount(CircleCount);
	}

	if (bAnyPropertyChanged || CirclePropertyNames.Contains(MemberName))
	{
		OnToolPropertiesChanged();
	}
}
#endif

TArray<FTransform> UAvaPatternModifierCircleTool::GetTransformInstances(const FBox& InOriginalBounds) const
{
	TArray<FTransform> Transforms;

	const int32 Count = CircleCount;
	const float Radius = CircleRadius;
	const float StartAngle = FMath::DegreesToRadians(CircleStartAngle);
	const float FullAngle = FMath::DegreesToRadians(CircleFullAngle);
	const float AngleStep = FullAngle / Count;

	const FTransform BaseTransform(CircleRotation.Quaternion(), FVector::ZeroVector, CircleScale);
	FTransform AccumulatedTransform = BaseTransform;
	for (int32 Idx = 0; Idx < CircleCount; Idx++)
	{
		const float CurrentAngle = StartAngle + Idx * AngleStep;
		const float X = Radius * FMath::Cos(CurrentAngle);
		const float Y = Radius * FMath::Sin(CurrentAngle);
        
		FVector Translation = FVector::ZeroVector;
		if (CirclePlane == EAvaPatternModifierPlane::XY)
		{
			Translation = FVector(X, Y, 0);
		}
		else if (CirclePlane == EAvaPatternModifierPlane::YZ)
		{
			Translation = FVector(0, X, Y);
		}
		else if (CirclePlane == EAvaPatternModifierPlane::ZX)
		{
			Translation = FVector(X, 0, Y);
		}

		Transforms.Add(FTransform(AccumulatedTransform.GetRotation(), AccumulatedTransform.GetTranslation() + Translation, AccumulatedTransform.GetScale3D()));

		if (bCircleAccumulateTransform)
		{
			AccumulatedTransform *= BaseTransform;
		}
	}

	return Transforms;
}

FVector UAvaPatternModifierCircleTool::GetCenterAlignmentAxis() const
{
	return FVector::ZeroVector;
}

FName UAvaPatternModifierCircleTool::GetToolName() const
{
	return TEXT("Circle");
}
