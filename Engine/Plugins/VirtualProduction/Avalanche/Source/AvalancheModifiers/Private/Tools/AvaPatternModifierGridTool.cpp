// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaPatternModifierGridTool.h"

void UAvaPatternModifierGridTool::SetGridPlane(EAvaPatternModifierPlane InGridPlane)
{
	if (GridPlane == InGridPlane)
	{
		return;
	}

	GridPlane = InGridPlane;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierGridTool::SetGridAlignment(EAvaPatternModifierGridAlignment InGridAlignment)
{
	if (GridAlignment == InGridAlignment)
	{
		return;
	}

	GridAlignment = InGridAlignment;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierGridTool::SetGridCountX(int32 InGridCountX)
{
	InGridCountX = FMath::Clamp(InGridCountX, 1, 10000);
	if (GridCountX == InGridCountX)
	{
		return;
	}

	GridCountX = InGridCountX;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierGridTool::SetGridCountY(int32 InGridCountY)
{
	InGridCountY = FMath::Clamp(InGridCountY, 1, 10000);
	if (GridCountY == InGridCountY)
	{
		return;
	}

	GridCountY = InGridCountY;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierGridTool::SetGridSpacingX(float InGridSpacingX)
{
	if (FMath::IsNearlyEqual(GridSpacingX, InGridSpacingX))
	{
		return;
	}

	GridSpacingX = InGridSpacingX;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierGridTool::SetGridSpacingY(float InGridSpacingY)
{
	if (FMath::IsNearlyEqual(GridSpacingY, InGridSpacingY))
	{
		return;
	}

	GridSpacingY = InGridSpacingY;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierGridTool::SetGridAccumulateTransform(bool bInGridAccumulateTransform)
{
	if (bGridAccumulateTransform == bInGridAccumulateTransform)
	{
		return;
	}

	bGridAccumulateTransform = bInGridAccumulateTransform;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierGridTool::SetGridRotation(const FRotator& InGridRotation)
{
	if (GridRotation.Equals(InGridRotation))
	{
		return;
	}
	
	GridRotation = InGridRotation;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierGridTool::SetGridScale(const FVector& InGridScale)
{
	if (GridScale.Equals(InGridScale))
	{
		return;
	}

	GridScale = InGridScale;
	OnToolPropertiesChanged();
}

#if WITH_EDITOR
void UAvaPatternModifierGridTool::PostEditChangeProperty(FPropertyChangedEvent& InEvent)
{
	Super::PostEditChangeProperty(InEvent);

	static const TSet<FName> GridPropertyNames
	{
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierGridTool, GridPlane),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierGridTool, GridAlignment),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierGridTool, GridCountX),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierGridTool, GridCountY),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierGridTool, GridSpacingX),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierGridTool, GridSpacingY),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierGridTool, bGridAccumulateTransform),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierGridTool, GridRotation),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierGridTool, GridScale),
	};

	const FName MemberName = InEvent.GetMemberPropertyName();
	const bool bAnyPropertyChanged = MemberName.IsNone();

	if (bAnyPropertyChanged || MemberName == GET_MEMBER_NAME_CHECKED(UAvaPatternModifierGridTool, GridCountX))
	{
		SetGridCountX(GridCountX);
	}
	if (bAnyPropertyChanged || MemberName == GET_MEMBER_NAME_CHECKED(UAvaPatternModifierGridTool, GridCountY))
	{
		SetGridCountY(GridCountY);
	}

	if (bAnyPropertyChanged || GridPropertyNames.Contains(MemberName))
	{
		OnToolPropertiesChanged();
	}
}
#endif

TArray<FTransform> UAvaPatternModifierGridTool::GetTransformInstances(const FBox& InOriginalBounds) const
{
	TArray<FTransform> Transforms;

	FVector ColAxis = (GridPlane == EAvaPatternModifierPlane::XY) ? FVector::YAxisVector :
					  (GridPlane == EAvaPatternModifierPlane::YZ) ? FVector::ZAxisVector :
					  FVector::XAxisVector;

	FVector RowAxis = (GridPlane == EAvaPatternModifierPlane::XY) ? FVector::XAxisVector :
					  (GridPlane == EAvaPatternModifierPlane::YZ) ? FVector::YAxisVector :
					  FVector::ZAxisVector;

	if (GridAlignment == EAvaPatternModifierGridAlignment::TopLeft || GridAlignment == EAvaPatternModifierGridAlignment::TopRight)
	{
		ColAxis *= -1;
	}

	if (GridAlignment == EAvaPatternModifierGridAlignment::BottomRight || GridAlignment == EAvaPatternModifierGridAlignment::TopRight)
	{
		RowAxis *= -1;
	}

	const FVector Size3D = InOriginalBounds.GetSize();
	const FVector ColTranslation = ColAxis * Size3D + ColAxis * GridSpacingY;
	const FVector RowTranslation = RowAxis * Size3D + RowAxis * GridSpacingX;

	const FQuat GridQuat = GridRotation.Quaternion();
	const FTransform BaseTransform(GridQuat, FVector::ZeroVector, GridScale);
	FTransform AccumulatedTransform = BaseTransform;
	for (int32 Row = 0; Row < GridCountX; Row++)
	{
		FTransform RowTransform = AccumulatedTransform;
		for (int32 Col = 0; Col < GridCountY; Col++)
		{
			Transforms.Add(RowTransform);
			if (bGridAccumulateTransform)
			{
				RowTransform *= FTransform(GridQuat, ColTranslation, GridScale);
			}
			else
			{
				RowTransform.SetLocation(RowTransform.GetLocation() + ColTranslation);
			}
		}

		if (bGridAccumulateTransform)
		{
			AccumulatedTransform *= FTransform(GridQuat, RowTranslation, GridScale);
		}
		else
		{
			AccumulatedTransform.SetLocation(AccumulatedTransform.GetLocation() + RowTranslation);
		}
	}

	return Transforms;
}

FVector UAvaPatternModifierGridTool::GetCenterAlignmentAxis() const
{
	FVector CenterAxis = FVector::ZeroVector;
	if (GridAlignment == EAvaPatternModifierGridAlignment::Center)
	{
		if (GridPlane == EAvaPatternModifierPlane::XY)
		{
			CenterAxis.X = 1;
			CenterAxis.Y = 1;
		}
		else if (GridPlane == EAvaPatternModifierPlane::YZ)
		{
			CenterAxis.Y = 1;
			CenterAxis.Z = 1;
		}
		else if (GridPlane == EAvaPatternModifierPlane::ZX)
		{
			CenterAxis.X = 1;
			CenterAxis.Z = 1;
		}
	}
	return CenterAxis;
}

FName UAvaPatternModifierGridTool::GetToolName() const
{
	return TEXT("Grid");
}
