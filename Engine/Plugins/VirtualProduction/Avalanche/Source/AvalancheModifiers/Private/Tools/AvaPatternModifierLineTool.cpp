// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaPatternModifierLineTool.h"

void UAvaPatternModifierLineTool::SetLineAxis(EAvaPatternModifierAxis InLineAxis)
{
	if (LineAxis == InLineAxis)
	{
		return;
	}

	LineAxis = InLineAxis;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierLineTool::SetLineAlignment(EAvaPatternModifierLineAlignment InLineAlignment)
{
	if (LineAlignment == InLineAlignment)
	{
		return;
	}

	LineAlignment = InLineAlignment;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierLineTool::SetLineCount(int32 InLineCount)
{
	InLineCount = FMath::Clamp(InLineCount, 1, 10000);
	if (LineCount == InLineCount)
	{
		return;
	}

	LineCount = InLineCount;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierLineTool::SetLineSpacing(float InLineSpacing)
{
	if (FMath::IsNearlyEqual(LineSpacing, InLineSpacing))
	{
		return;
	}

	LineSpacing = InLineSpacing;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierLineTool::SetLineAccumulateTransform(bool bInLineAccumulateTransform)
{
	if (bLineAccumulateTransform == bInLineAccumulateTransform)
	{
		return;
	}

	bLineAccumulateTransform = bInLineAccumulateTransform;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierLineTool::SetLineRotation(const FRotator& InLineRotation)
{
	if (LineRotation.Equals(InLineRotation))
	{
		return;
	}

	LineRotation = InLineRotation;
	OnToolPropertiesChanged();
}

void UAvaPatternModifierLineTool::SetLineScale(const FVector& InLineScale)
{
	if (LineScale.Equals(InLineScale))
	{
		return;
	}

	LineScale = InLineScale;
	OnToolPropertiesChanged();
}

#if WITH_EDITOR
void UAvaPatternModifierLineTool::PostEditChangeProperty(FPropertyChangedEvent& InEvent)
{
	Super::PostEditChangeProperty(InEvent);

	static const TSet<FName> LinePropertyNames
	{
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierLineTool, LineAxis),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierLineTool, LineAlignment),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierLineTool, LineCount),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierLineTool, LineSpacing),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierLineTool, bLineAccumulateTransform),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierLineTool, LineRotation),
		GET_MEMBER_NAME_CHECKED(UAvaPatternModifierLineTool, LineScale),
	};

	const FName MemberName = InEvent.GetMemberPropertyName();
	const bool bAnyPropertyChanged = MemberName.IsNone();

	if (bAnyPropertyChanged || MemberName == GET_MEMBER_NAME_CHECKED(UAvaPatternModifierLineTool, LineCount))
	{
		SetLineCount(LineCount);
	}

	if (bAnyPropertyChanged || LinePropertyNames.Contains(MemberName))
	{
		OnToolPropertiesChanged();
	}
}
#endif

TArray<FTransform> UAvaPatternModifierLineTool::GetTransformInstances(const FBox& InOriginalBounds) const
{
	TArray<FTransform> Transforms;

	FVector Axis = FVector::ZeroVector;
	switch (LineAxis)
	{
		case EAvaPatternModifierAxis::X:
			Axis = FVector::XAxisVector;
			break;
		case EAvaPatternModifierAxis::Y:
			Axis = FVector::YAxisVector;
			break;
		case EAvaPatternModifierAxis::Z:
			Axis = FVector::ZAxisVector;
			break;
	};

	Axis *= (LineAlignment == EAvaPatternModifierLineAlignment::End ? -1 : 1);

	const FVector Size3D = InOriginalBounds.GetSize();
	const FVector Translation = Axis * Size3D + Axis * LineSpacing;

	const FQuat LineQuat = LineRotation.Quaternion();
	FTransform AccumulatedTransform(LineQuat, FVector::ZeroVector, LineScale);
	for (int32 Idx = 0; Idx < LineCount; Idx++)
	{
		Transforms.Add(AccumulatedTransform);
        
		if (bLineAccumulateTransform)
		{
			AccumulatedTransform *= FTransform(LineQuat, Translation, LineScale);
		}
		else
		{
			AccumulatedTransform.SetLocation(AccumulatedTransform.GetLocation() + Translation);
		}
	}

	return Transforms;
}

FVector UAvaPatternModifierLineTool::GetCenterAlignmentAxis() const
{
	FVector CenterAxis = FVector::ZeroVector;
	if (LineAlignment == EAvaPatternModifierLineAlignment::Center)
	{
		if (LineAxis == EAvaPatternModifierAxis::X)
		{
			CenterAxis.X = 1;
		}
		else if (LineAxis == EAvaPatternModifierAxis::Y)
		{
			CenterAxis.Y = 1;
		}
		else if (LineAxis == EAvaPatternModifierAxis::Z)
		{
			CenterAxis.Z = 1;
		}
	}
	return CenterAxis;
}

FName UAvaPatternModifierLineTool::GetToolName() const
{
	return TEXT("Line");
}
