// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaPatternModifierTool.h"

#include "Modifiers/AvaPatternModifier.h"

void UAvaPatternModifierTool::OnToolPropertiesChanged() const
{
	if (UAvaPatternModifier* PatternModifier = GetTypedOuter<UAvaPatternModifier>())
	{
		if (PatternModifier->GetActiveToolClass() == GetClass())
		{
			PatternModifier->MarkModifierDirty();
		}
	}
}
