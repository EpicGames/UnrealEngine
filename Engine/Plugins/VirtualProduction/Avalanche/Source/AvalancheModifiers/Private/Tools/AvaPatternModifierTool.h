// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaPatternModifierTool.generated.h"

/** Represent a tool supported by the pattern modifier */
UCLASS(Abstract)
class UAvaPatternModifierTool : public UObject
{
	GENERATED_BODY()

public:
	virtual TArray<FTransform> GetTransformInstances(const FBox& InOriginalBounds) const PURE_VIRTUAL(UAvaPatternModifierTool::GetTransformInstances, return {}; );
	virtual FVector GetCenterAlignmentAxis() const PURE_VIRTUAL(UAvaPatternModifierTool::GetCenterAlignmentAxis, return FVector::ZeroVector; );
	virtual FName GetToolName() const PURE_VIRTUAL(UAvaPatternModifierTool::GetToolName, return NAME_None; );

protected:
	void OnToolPropertiesChanged() const;
};
