// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaTaperTool.h"

#include "DeformationOps/LatticeDeformerOp.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "LatticeDeformerTool.h"
#include "Mechanics/LatticeControlPointsMechanic.h"

using namespace UE::Geometry;

bool UAvaTaperTool::Setup(FDynamicMesh3* InMesh, const FAvaTaperSettings& InTaperSettings)
{
	OriginalMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(*InMesh);

	if (OriginalMesh)
	{
		if (OriginalMesh->GetBounds().Width() < KINDA_SMALL_NUMBER)
		{
			return false;
		}

		Settings = InTaperSettings;
	
		InitializeLattice(ControlPoints);

		return true;
	}

	return false;
}

void UAvaTaperTool::SetTaperAmount(float InAmount)
{
	Settings.Amount = FMath::Clamp(InAmount, 0.0, 1.0);
}

void UAvaTaperTool::SetInterpolationType(EAvaTaperInterpolationType InInterpolationType)
{
	Settings.InterpolationType = InInterpolationType;
}

void UAvaTaperTool::SetVerticalResolution(int32 InResolution)
{
	Settings.ZAxisResolution = FMath::Max(2, InResolution);
}

void UAvaTaperTool::SetDeformNormals(bool bInDeformNormals)
{
	Settings.bDeformNormals = bInDeformNormals;
}

void UAvaTaperTool::SetTaperOffset(const FVector2D& InTaperOffset)
{
	Settings.Offset = InTaperOffset;
}

FVector3i UAvaTaperTool::GetLatticeResolution() const
{
	const int32 Res = FMath::Max(2, Settings.ZAxisResolution);
	return FVector3i{Res, Res, Res};
}

double UAvaTaperTool::GetInterpolatedModifier(double InModifier) const
{
	double InterpolatedModifier; 
	switch (Settings.InterpolationType)
	{
		case EAvaTaperInterpolationType::Linear:
			InterpolatedModifier = InModifier;
			break;

		case EAvaTaperInterpolationType::Quadratic:
			InterpolatedModifier = FMath::Pow(InModifier,2.0);
			break;

		case EAvaTaperInterpolationType::Cubic:
			InterpolatedModifier = FMath::Pow(InModifier, 3.0);
			break;

		case EAvaTaperInterpolationType::QuadraticInverse:
			InterpolatedModifier = 1.0 - FMath::Pow(InModifier - 1.0,2.0); // 1 - (x-1)^2
			break;
		
		case EAvaTaperInterpolationType::CubicInverse:
			InterpolatedModifier = 1.0 + FMath::Pow(InModifier - 1.0,3.0); // 1 + (x-1)^3
			break;
		
		default:
			InterpolatedModifier = InModifier;
			break;
	}

	return InterpolatedModifier;
}

TUniquePtr<FDynamicMesh3> UAvaTaperTool::Compute()
{
	if (OriginalMesh)
	{
		if (OriginalMesh->GetBounds().Width() < KINDA_SMALL_NUMBER)
		{
			return nullptr;
		}

		// the taper center point starts from the center of the mesh, plus an optional XY planar offset
		const FVector TaperCenterPoint = OriginalMesh->GetBounds().Center() + FVector(Settings.Offset, 0);
		
		TArray<FVector3d> CurrOpControlPoints = ControlPoints;

		// re-mapping [1.0 - 0.0] / [0.0 - 1.0] min and max taper extent limits to [ZMin - ZMax], so we can apply them
		const double MinExtent = FMath::GetMappedRangeValueClamped(TVector2(1.0, 0.0), TVector2(MinValue, MaxValue), Settings.Extent.X);
		const double MaxExtent = FMath::GetMappedRangeValueClamped(TVector2(0.0, 1.0), TVector2(MinValue, MaxValue), Settings.Extent.Y);
		
		const TVector2 ExtentRange(FMath::Max(MinValue, MinExtent), FMath::Min(MaxValue,MaxExtent));
		const TVector2 AmountRange(0.0, 1.0);

		for (FVector& CurrControlPoint : CurrOpControlPoints)
		{
			const FVector ProjectPoint(TaperCenterPoint.X, TaperCenterPoint.Y, CurrControlPoint.Z);
			double Modifier = FMath::GetMappedRangeValueClamped(ExtentRange, AmountRange, CurrControlPoint.Z);

			Modifier = GetInterpolatedModifier(Modifier);

			FVector ProjectionVec = ProjectPoint - CurrControlPoint;
			ProjectionVec.X *= (Settings.Amount) * Modifier;
			ProjectionVec.Y *= (Settings.Amount) * Modifier;

			CurrControlPoint += ProjectionVec;
		}

		const TUniquePtr<FLatticeDeformerOp> LatticeDeformOp = MakeUnique<FLatticeDeformerOp>(
			OriginalMesh,
			Lattice,
			CurrOpControlPoints,
			ELatticeInterpolation::Linear,
			Settings.bDeformNormals);

		LatticeDeformOp->CalculateResult(nullptr);
		
		return LatticeDeformOp->ExtractResult();
	}

	return nullptr;
}

bool UAvaTaperTool::ApplyTaper(TUniquePtr<FDynamicMesh3>& OutMesh)
{
	OutMesh = Compute();

	if (OutMesh)
	{
		return true;
	}
	else
	{
		return false;
	}
}

void UAvaTaperTool::InitializeLattice(TArray<FVector3d>& OutLatticePoints)
{
	if (OriginalMesh && OriginalMesh.IsValid())
	{
		Lattice = MakeShared<FFFDLattice, ESPMode::ThreadSafe>(GetLatticeResolution(), *OriginalMesh, 0.01);
		Lattice->GenerateInitialLatticePositions(OutLatticePoints);
		UpdateMinMaxZ();
	}
}

void UAvaTaperTool::UpdateMinMaxZ()
{
	double ZMin =  BIG_NUMBER;
	double ZMax = -BIG_NUMBER;

	for (const FVector3d& Point : ControlPoints)
	{
		if (ZMin > Point.Z)
		{
			ZMin = Point.Z;
		}

		if (ZMax < Point.Z)
		{
			ZMax = Point.Z;
		}
	}

	MinValue = ZMin;
	MaxValue = ZMax;
}
