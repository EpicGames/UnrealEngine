// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curve/DynamicGraph3.h"
#include "IntVectorTypes.h"
#include "Operations/FFDLattice.h"
#include "UObject/Object.h"
#include "AvaTaperTool.generated.h"

class UMeshOpPreviewWithBackgroundCompute;
class ULatticeDeformerToolProperties;
class ULatticeControlPointsMechanic;

UENUM()
enum class EAvaTaperInterpolationType : uint8
{
	Linear,
	Quadratic,
	Cubic,
	QuadraticInverse,
	CubicInverse
};

USTRUCT()
struct FAvaTaperSettings
{
	GENERATED_BODY()

	/** the amount of taper to apply [0;1] as min-max range. Values out of this range will be clamped. */
	UPROPERTY()
	float Amount = 1.0;
	
	/** Number of lattice vertices along the Z axis */
	UPROPERTY()
	int ZAxisResolution = 2;

	/** Whether to use linear or cubic interpolation to get new mesh vertex positions from the lattice */
	UPROPERTY()
	EAvaTaperInterpolationType InterpolationType = EAvaTaperInterpolationType::Linear;

	/** Whether to use approximate new vertex normals using the deformer */
	UPROPERTY()
	bool bDeformNormals = false;

	UPROPERTY()
	FVector2D Extent = FVector2D::UnitVector;
	
	UPROPERTY()
	FVector2D Offset = FVector2D::ZeroVector;
};

/**
 * @brief A taper tool based on how ULatticeDeformerTool uses FFFDLattice
 */
UCLASS()
class UAvaTaperTool : public UObject
{
	GENERATED_BODY()
	
public:
	/**
	 * Setup the Taper deformation tool
	 * @param InMesh the dynamic mesh to be tapered
	 * @param InTaperSettings
	 */
	bool Setup(UE::Geometry::FDynamicMesh3* InMesh, const FAvaTaperSettings& InTaperSettings);

	/** sets the amount of taper to apply [0;1] as min-max range. Values out of this range will be clamped. */
	void SetTaperAmount(float InAmount);
	
	/** sets the type of interpolation used by the taper modifier */
	void SetInterpolationType(EAvaTaperInterpolationType InInterpolationType);

	/** sets vertical resolution that directly affects the number of control points used by the taper modifier. Should match the tessellation/resolution of the mesh */
	void SetVerticalResolution(int32 InResolution);

	/** if true, applies normal deformation */
	void SetDeformNormals(bool bInDeformNormals);

	void SetTaperOffset(const FVector2D& InTaperOffset);

	/**
	 * Applies the taper modifier to the mesh specified at Setup time, and stores the result in the specified dynamic mesh
	 * @param OutMesh the dynamic mesh where the taper result will be stored
	 * @return if taper was applied successfully
	 */
	bool ApplyTaper(TUniquePtr<UE::Geometry::FDynamicMesh3>& OutMesh);
	
private:
	/** Input mesh */
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh = nullptr;
	TSharedPtr<UE::Geometry::FFFDLattice, ESPMode::ThreadSafe> Lattice = nullptr;

	UPROPERTY()
	TArray<FVector3d> ControlPoints;

	UPROPERTY()
	FAvaTaperSettings Settings;
	
	/** min and max Z values are used to map the taper along the z axis of the geometry */
	UPROPERTY()
	double MinValue = 0.0;

	/** min and max Z values are used to map the taper along the z axis of the geometry */
	UPROPERTY()
	double MaxValue = 0.0;

	/** Refreshes min and max Z values based on current control points Z position */
	void UpdateMinMaxZ();

	/**
	 * Applies the current interpolation type to the specified Modifier value
	 * @param InModifier the value to be interpolated
	 * @return the interpolated value
	 */
	double GetInterpolatedModifier(double InModifier) const;

	/** Create and store an FFFDLattice. Pass out the lattice's positions */
	void InitializeLattice(TArray<FVector3d>& OutLatticePoints);

	/** Returns this Lattice resolution as a vector. Used internally. */
	UE::Geometry::FVector3i GetLatticeResolution() const;
	
	/**
	 * Internal function to compute the taper. Requires the TaperLatticeTool to be Setup.
	 * @return The tapered mesh (nullptr when failing)
	 */
	TUniquePtr<UE::Geometry::FDynamicMesh3> Compute();
};
