// Copyright Epic Games, Inc. All Rights Reserved.

#include "Version/AvaTranslucentPriorityModifierVersion.h"
#include "Serialization/CustomVersion.h"

namespace UE::AvaModifiers
{

const FGuid FTranslucentPriorityModifierVersion::Guid(0x3A5DCA18, 0x17EF4851, 0xA6156A1B, 0x3A1B0FB2);
FCustomVersionRegistration GRegisterTranslucentPriorityModifierVersion(FTranslucentPriorityModifierVersion::Guid
	, FTranslucentPriorityModifierVersion::LatestVersion
	, TEXT("TranslucentPriorityModifierVersion"));

} // UE::AvaModifiers
