// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Color.h"
#include "UObject/ObjectPtr.h"
#include "AvaModifiersParametricMaterial.generated.h"

class UMaterial;
class UMaterialInstanceDynamic;
class UObject;

/** Use this if you want a parametric material */
USTRUCT()
struct FAvaModifiersParametricMaterial
{
	GENERATED_BODY()

	AVALANCHEMODIFIERS_API FAvaModifiersParametricMaterial();

	AVALANCHEMODIFIERS_API UMaterial* GetDefaultMaterial() const;

	AVALANCHEMODIFIERS_API UMaterialInstanceDynamic* GetMaterial() const;

	AVALANCHEMODIFIERS_API void ApplyChanges(UObject* Outer = nullptr);

	UPROPERTY()
	FLinearColor MaskColor;

private:
	void ApplyParams() const;
	void CreateAndApply(UObject* Outer = nullptr);
	void EnsureCurrentMaterial(UObject* Outer = nullptr);
	UMaterial* LoadResource() const;

	UPROPERTY()
	TObjectPtr<UMaterial> DefaultMaterial;

	UPROPERTY(Transient, Instanced)
	TObjectPtr<UMaterialInstanceDynamic> InstanceMaterial;
};