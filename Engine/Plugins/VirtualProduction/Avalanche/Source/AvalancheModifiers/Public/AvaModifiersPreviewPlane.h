// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaModifiersParametricMaterial.h"
#include "UObject/ObjectPtr.h"
#include "AvaModifiersPreviewPlane.generated.h"

class USceneComponent;
class UStaticMesh;
class UStaticMeshComponent;

/** Use this if you want a preview plane on a specific actor */
USTRUCT()
struct FAvaModifierPreviewPlane
{
	GENERATED_BODY()

	explicit FAvaModifierPreviewPlane();

	/** Create a preview plane attached to this component */
	void Create(USceneComponent* InActorComponent);

	/** Destroy the preview plane */
	void Destroy();

	/** Show the preview plane */
	void Show() const;

	/** Hide the preview plane */
	void Hide() const;

	/** Update relative position of the plane */
	void Update(const FTransform& InRelativeTransform) const;

	UPROPERTY(Transient)
	FAvaModifiersParametricMaterial PreviewDynMaterial;

protected:
	UStaticMesh* LoadPreviewResource() const;

	UPROPERTY(Transient)
	TObjectPtr<UStaticMesh> PreviewStaticMesh = nullptr;

	UPROPERTY(Transient)
	TObjectPtr<UStaticMeshComponent> PreviewComponent = nullptr;
};
