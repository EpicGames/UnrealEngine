// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaGeometryBaseModifier.h"
#include "Extensions/ActorModifierRenderStateUpdateExtension.h"
#include "Extensions/ActorModifierSceneTreeUpdateExtension.h"
#include "Extensions/ActorModifierTransformUpdateExtension.h"
#include "Layout/Margin.h"
#include "AvaAutoSizeModifier.generated.h"

class UActorComponent;
class UAvaShape2DDynMeshBase;

UENUM()
enum class EAvaAutoSizeFitMode : uint8
{
	WidthAndHeight,
	WidthOnly,
	HeightOnly
};

/**
 * Adapts the modified actor geometry size/scale to match reference actor bounds and act as a background
 */
UCLASS(MinimalAPI, BlueprintType)
class UAvaAutoSizeModifier : public UAvaGeometryBaseModifier
	, public IActorModifierTransformUpdateHandler
	, public IActorModifierRenderStateUpdateHandler
	, public IActorModifierSceneTreeUpdateHandler
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|AutoSize")
	AVALANCHEMODIFIERS_API void SetReferenceActor(const FActorModifierSceneTreeActor& InReferenceActor);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|AutoSize")
	const FActorModifierSceneTreeActor& GetReferenceActor() const
	{
		return ReferenceActor;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|AutoSize")
	AVALANCHEMODIFIERS_API void SetPaddingHorizontal(double InPadding);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|AutoSize")
	double GetPaddingHorizontal() const
	{
		return PaddingHorizontal;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|AutoSize")
	AVALANCHEMODIFIERS_API void SetPaddingVertical(double InPadding);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|AutoSize")
	double GetPaddingVertical() const
	{
		return PaddingVertical;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|AutoSize")
	AVALANCHEMODIFIERS_API void SetFitMode(const EAvaAutoSizeFitMode InFitMode);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|AutoSize")
	EAvaAutoSizeFitMode GetFitMode() const
	{
		return FitMode;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|AutoSize")
	AVALANCHEMODIFIERS_API void SetIncludeChildren(bool bInIncludeChildren);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|AutoSize")
	bool GetIncludeChildren() const
	{
		return bIncludeChildren;
	}

protected:
	//~ Begin UObject
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual bool IsModifierDirtyable() const override;
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifiedActorTransformed() override;
	virtual void SavePreState() override;
	virtual void RestorePreState() override;
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierEnabled(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	//~ Begin IAvaTransformUpdatedExtension
	virtual void OnTransformUpdated(AActor* InActor, bool bInParentMoved) override;
	//~ End IAvaTransformUpdatedExtension

	//~ Begin IAvaRenderStateUpdateExtension
	virtual void OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent) override;
	virtual void OnActorVisibilityChanged(AActor* InActor) override;
	//~ End IAvaRenderStateUpdateExtension

	//~ Begin IAvaSceneTreeUpdateModifierExtension
	virtual void OnSceneTreeTrackedActorChanged(int32 InIdx, AActor* InPreviousActor, AActor* InNewActor) override;
	virtual void OnSceneTreeTrackedActorChildrenChanged(int32 InIdx, const TSet<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TSet<TWeakObjectPtr<AActor>>& InNewChildrenActors) override;
	virtual void OnSceneTreeTrackedActorDirectChildrenChanged(int32 InIdx, const TArray<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TArray<TWeakObjectPtr<AActor>>& InNewChildrenActors) override {}
	virtual void OnSceneTreeTrackedActorParentChanged(int32 InIdx, const TArray<TWeakObjectPtr<AActor>>& InPreviousParentActor, const TArray<TWeakObjectPtr<AActor>>& InNewParentActor) override {}
	virtual void OnSceneTreeTrackedActorRearranged(int32 InIdx, AActor* InRearrangedActor) override {}
	//~ End IAvaSceneTreeUpdateModifierExtension

	void OnReferenceActorChanged();

	UPROPERTY(EditInstanceOnly, Setter="SetReferenceActor", Getter="GetReferenceActor", Category="AutoSize", meta=(ShowOnlyInnerProperties, AllowPrivateAccess="true"))
	FActorModifierSceneTreeActor ReferenceActor;

	/** The method for finding a reference actor based on it's position in the parent's hierarchy. */
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use ReferenceActor instead"))
	EActorModifierReferenceContainer ReferenceContainer_DEPRECATED = EActorModifierReferenceContainer::Other;

	/** The actor affecting the modifier. This is user selectable if the Reference Container is set to "Other". */
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use ReferenceActor instead"))
	TWeakObjectPtr<AActor> ReferenceActorWeak_DEPRECATED = nullptr;

	/** If true, will search for the next visible actor based on the selected reference container. */
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use ReferenceActor instead"))
	bool bIgnoreHiddenActors_DEPRECATED = false;

	/** Padding for top and bottom side */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Interp, Category="AutoSize", meta=(AllowPrivateAccess="true"))
	double PaddingVertical = 0.f;

	/** Padding for left and right side */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Interp, Category="AutoSize", meta=(AllowPrivateAccess="true"))
	double PaddingHorizontal = 0.f;

	UPROPERTY(EditInstanceOnly, Setter="SetFitMode", Getter="GetFitMode", Category="AutoSize", meta=(AllowPrivateAccess="true"))
	EAvaAutoSizeFitMode FitMode = EAvaAutoSizeFitMode::WidthAndHeight;

	/** If true, will include children bounds too and compute the new size */
	UPROPERTY(EditInstanceOnly, Setter="SetIncludeChildren", Getter="GetIncludeChildren", Category="AutoSize", meta=(AllowPrivateAccess="true"))
	bool bIncludeChildren = true;

private:
	/** Padding added around reference actor bounds for geometry */
	UPROPERTY()
	FMargin Padding = FMargin(0.f);

	UPROPERTY()
	FVector2D PreModifierShapeDynMesh2DSize;

	UPROPERTY()
	TWeakObjectPtr<UAvaShape2DDynMeshBase> ShapeDynMesh2DWeak;

	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	FVector CachedFollowLocation = FVector::ZeroVector;

	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	FBox CachedReferenceBounds = FBox(EForceInit::ForceInit);

	UPROPERTY()
	bool bDeprecatedPropertiesMigrated = false;
};
