// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaGeometryBaseModifier.h"
#include "AvaBendModifier.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UAvaBendModifier : public UAvaGeometryBaseModifier
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Bend")
	AVALANCHEMODIFIERS_API void SetAngle(float InAngle);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Bend")
	float GetAngle() const
	{
		return Angle;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Bend")
	AVALANCHEMODIFIERS_API void SetExtent(float InExtent);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Bend")
	float GetExtent() const
	{
		return Extent;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Bend")
	AVALANCHEMODIFIERS_API void SetBendPosition(const FVector& InBendPosition);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Bend")
	const FVector& GetBendPosition() const
	{
		return BendPosition;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Bend")
	AVALANCHEMODIFIERS_API void SetBendRotation(const FRotator& InBendRotation);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Bend")
	const FRotator& GetBendRotation() const
	{
		return BendRotation;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Bend")
	AVALANCHEMODIFIERS_API void SetSymmetricExtents(bool bInSymmetricExtents);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Bend")
	bool GetSymmetricExtents() const
	{
		return bSymmetricExtents;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Bend")
	AVALANCHEMODIFIERS_API void SetBidirectional(bool bInBidirectional);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Bend")
	bool GetBidirectional() const
	{
		return bBidirectional;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	void OnBendTransformChanged();
	void OnBendOptionChanged();

	UPROPERTY(EditInstanceOnly, Setter="SetBendPosition", Getter="GetBendPosition", Category="Bend", meta=(AllowPrivateAccess="true"))
	FVector BendPosition = FVector::ZeroVector;

	UPROPERTY(EditInstanceOnly, Setter="SetBendRotation", Getter="GetBendRotation", Category="Bend", meta=(AllowPrivateAccess="true"))
	FRotator BendRotation = FRotator::ZeroRotator;

	UPROPERTY(EditInstanceOnly, Setter="SetAngle", Getter="GetAngle", Category="Bend", meta=(ClampMin="-360.0", ClampMax="360.0", AllowPrivateAccess="true"))
	float Angle = 25;

	UPROPERTY(EditInstanceOnly, Setter="SetExtent", Getter="GetExtent", Category="Bend", meta=(ClampMin="0", ClampMax="1.0", AllowPrivateAccess="true"))
	float Extent = 1.0;

	UPROPERTY(EditInstanceOnly, Setter="SetSymmetricExtents", Getter="GetSymmetricExtents", Category="Bend", meta=(AllowPrivateAccess="true"))
	bool bSymmetricExtents = true;

	UPROPERTY(EditInstanceOnly, Setter="SetBidirectional", Getter="GetBidirectional", Category="Bend", meta=(AllowPrivateAccess="true"))
	bool bBidirectional = false;
};
