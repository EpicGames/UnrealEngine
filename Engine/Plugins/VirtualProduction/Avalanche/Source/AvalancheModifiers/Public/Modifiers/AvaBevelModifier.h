// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaGeometryBaseModifier.h"
#include "AvaBevelModifier.generated.h"

UCLASS(MinimalAPI, BlueprintType)
class UAvaBevelModifier : public UAvaGeometryBaseModifier
{
	GENERATED_BODY()

public:
	static inline const FName BevelPolygroupLayerName = TEXT("Bevel");
	static constexpr float MinInset = 0;
	static constexpr int32 MinIterations = 0;
	static constexpr int32 MaxIterations = 10;
	static constexpr float MinRoundness = -2;
	static constexpr float MaxRoundness = 2;

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Bevel")
	AVALANCHEMODIFIERS_API void SetInset(float InBevel);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Bevel")
	float GetInset() const
	{
		return Inset;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Bevel")
	AVALANCHEMODIFIERS_API void SetIterations(int32 InIterations);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Bevel")
	int32 GetIterations() const
	{
		return Iterations;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Bevel")
	AVALANCHEMODIFIERS_API void SetRoundness(float InRoundness);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Bevel")
	float GetRoundness() const
	{
		return Roundness;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	void OnInsetChanged();
	void OnIterationsChanged();
	void OnRoundnessChanged();

	float GetMaxInsetDistance() const;

	/** Distance used on vertices for beveling, clamped between 0 and (min bound size / 2) */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Bevel", meta=(ClampMin="0", AllowPrivateAccess="true"))
	float Inset = 1.0f;

	/** Amount of subdivisions applied on the bevel, could affect performance the higher this value gets */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Bevel", meta=(ClampMin="0", ClampMax="10", AllowPrivateAccess="true"))
	int32 Iterations = 0;

	/** Roundness of the beveling when multiple iterations are applied : -2 = inner rounded, 0 = flat, 2 = outer rounded */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Bevel", meta=(ClampMin="-2", ClampMax="2", EditCondition="Iterations > 0", AllowPrivateAccess="true"))
	float Roundness = 0;
};
