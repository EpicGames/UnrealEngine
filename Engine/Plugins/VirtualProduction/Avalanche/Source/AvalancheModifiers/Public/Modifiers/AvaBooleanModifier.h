// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaGeometryBaseModifier.h"
#include "AvaModifiersParametricMaterial.h"
#include "Extensions/ActorModifierTransformUpdateExtension.h"
#include "Shared/AvaBooleanModifierShared.h"
#include "AvaBooleanModifier.generated.h"

UENUM(BlueprintType)
enum class EAvaBooleanMode : uint8
{
	/** Does not apply any mode, but get affected by other geometry modes */
	None       UMETA(DisplayName="Target"),
	/** Subtract this geometry from the one it is colliding with */
	Subtract,
	/** Only keep the intersecting geometry from the two colliding geometry */
	Intersect,
	/** Merges the two colliding geometry together */
	Union
};

/** This modifier allows you to apply a mask on a certain shape, this will affect every shape it collides with that matches options */
UCLASS(MinimalAPI, BlueprintType)
class UAvaBooleanModifier : public UAvaGeometryBaseModifier
	, public IActorModifierTransformUpdateHandler
{
	GENERATED_BODY()

	friend class UAvaBooleanModifierShared;

public:
	/** This is the min depth threshold needed on the mask for it to work properly */
	static constexpr float MinDepth = UE_KINDA_SMALL_NUMBER * 2;

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Boolean")
	AVALANCHEMODIFIERS_API void SetMode(EAvaBooleanMode InMode);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Boolean")
	EAvaBooleanMode GetMode() const
	{
		return Mode;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Boolean")
	AVALANCHEMODIFIERS_API void SetChannel(uint8 InChannel);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Boolean")
	uint8 GetChannel() const
	{
		return Channel;
	}

	const FAvaBooleanModifierSharedChannelInfo& GetChannelInfo() const
	{
		return ChannelInfo;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	virtual void SavePreState() override;
	virtual void RestorePreState() override;
	virtual void Apply() override;
	virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;
	virtual void OnModifierEnabled(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierRemoved(EActorModifierCoreDisableReason InReason) override;
	//~ End UActorModifierCoreBase

	//~ Begin IAvaTransformUpdateExtension
	virtual void OnTransformUpdated(AActor* InActor, bool bInParentMoved) override;
	//~ End IAvaTransformUpdateExtension

	void ApplyInternal();

	void OnModeChanged();
	void OnChannelChanged();

	/** Creates depth for a mask shape that has none */
	void CreateMaskDepth() const;

	/** Called by the editor to change visibility of mask in a world */
	void OnMaskVisibilityChange(const UWorld* World, bool bMaskActorVisible) const;
	void OnMaskingOptionsChanged();

	void SaveOriginalMaterials();
	void RestoreOriginalMaterials();

	void UpdateMaskingMaterials();
	void UpdateMaskDelegates();
	void UpdateMaskVisibility();

	static void ApplyTool(EAvaBooleanMode InMode, const UE::Geometry::FDynamicMesh3& InModeTool, const UAvaBooleanModifier* InTarget, UE::Geometry::FDynamicMesh3& OutMesh);

	/** Mode to use when shapes are colliding, none means you will be masked otherwise you are masking */
	UPROPERTY(EditInstanceOnly, Setter="SetMode", Getter="GetMode", Category="Boolean", meta=(AllowPrivateAccess="true"))
	EAvaBooleanMode Mode = EAvaBooleanMode::None;

	/** Channel to only apply this tool on shapes with the same channel */
	UPROPERTY(EditInstanceOnly, Setter="SetChannel", Getter="GetChannel", Category="Boolean", meta=(AllowPrivateAccess="true"))
	uint8 Channel = 0;

	UPROPERTY(Transient)
	FAvaModifiersParametricMaterial ParametricMaskMaterial;

	/** Original materials before we apply the mask material */
	UPROPERTY(DuplicateTransient, NonTransactional)
	TArray<TObjectPtr<UMaterialInterface>> OriginalMaterials;

	/** Colliding actor modifiers to quickly update */
	UPROPERTY(DuplicateTransient, NonTransactional)
	TSet<TWeakObjectPtr<UAvaBooleanModifier>> CollidingModifiers;

	/** General infos on the channel used by this modifier profiler */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	FAvaBooleanModifierSharedChannelInfo ChannelInfo;

private:
	/** Track last transform during update to check against when moved */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	FTransform LastTransform;
};
