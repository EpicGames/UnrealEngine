// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaGeometryBaseModifier.h"
#include "CEMeshBuilder.h"
#include "Extensions/ActorModifierRenderStateUpdateExtension.h"
#include "Extensions/ActorModifierSceneTreeUpdateExtension.h"
#include "Templates/SubclassOf.h"
#include "AvaDynamicMeshConverterModifier.generated.h"

class UBrushComponent;
class UDynamicMeshComponent;
class UMaterialInterface;
class UPrimitiveComponent;
class UProceduralMeshComponent;
class USkeletalMeshComponent;
class UStaticMeshComponent;

/** Components that can be converted to dynamic mesh */
UENUM(BlueprintType, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EAvaDynamicMeshConverterModifierType : uint8
{
	None = 0 UMETA(Hidden),
	StaticMeshComponent = 1 << 0,
	DynamicMeshComponent = 1 << 1,
	SkeletalMeshComponent = 1 << 2,
	BrushComponent = 1 << 3,
	ProceduralMeshComponent = 1 << 4
};
ENUM_CLASS_FLAGS(EAvaDynamicMeshConverterModifierType);

UENUM(BlueprintType)
enum class EAvaDynamicMeshConverterModifierFilter : uint8
{
	None,
	Include,
	Exclude
};

USTRUCT()
struct FAvaDynamicMeshConverterModifierComponentState
{
	GENERATED_BODY()

	explicit FAvaDynamicMeshConverterModifierComponentState() {}
	explicit FAvaDynamicMeshConverterModifierComponentState(UPrimitiveComponent* InPrimitiveComponent);

	void UpdateRelativeTransform(const FTransform& InParentTransform);
	
	friend uint32 GetTypeHash(const FAvaDynamicMeshConverterModifierComponentState& InItem)
	{
		return GetTypeHash(InItem.Component);
	}

	bool operator==(const FAvaDynamicMeshConverterModifierComponentState& InOther) const
	{
		return Component.IsValid() && Component == InOther.Component;
	}

	/** The component we are converting to dynamic mesh */
	UPROPERTY()
	TWeakObjectPtr<UPrimitiveComponent> Component;

	/** The default visibility of the actor converted component in game */
	UPROPERTY()
	bool bComponentHiddenInGame = false;

	/** The default visibility of the converted component in editor */
	UPROPERTY()
	bool bComponentVisible = true;

	/** Transform saved before mesh is converted */
	UPROPERTY()
	FTransform ActorRelativeTransform = FTransform::Identity;

	/** Used for diffs */
	UPROPERTY(Transient)
	TArray<TWeakObjectPtr<UMaterialInterface>> ComponentMaterialsWeak;
};

UCLASS(MinimalAPI, BlueprintType)
class UAvaDynamicMeshConverterModifier
	: public UAvaGeometryBaseModifier
	, public IActorModifierSceneTreeUpdateHandler
	, public IActorModifierRenderStateUpdateHandler
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|DynamicMeshConverter")
	void SetSourceActor(AActor* InActor)
	{
		SetSourceActorWeak(InActor);
	}

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|DynamicMeshConverter")
	AActor* GetSourceActor() const
	{
		return SourceActorWeak.Get();
	}

	AVALANCHEMODIFIERS_API void SetSourceActorWeak(const TWeakObjectPtr<AActor>& InActor);
	TWeakObjectPtr<AActor> GetSourceActorWeak() const
	{
		return SourceActorWeak;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|DynamicMeshConverter")
	AVALANCHEMODIFIERS_API void SetComponentTypes(const TSet<EAvaDynamicMeshConverterModifierType>& InTypes);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|DynamicMeshConverter")
	AVALANCHEMODIFIERS_API TSet<EAvaDynamicMeshConverterModifierType> GetComponentTypes() const;

	AVALANCHEMODIFIERS_API void SetComponentType(int32 InComponentType);

	int32 GetComponentType() const
	{
		return ComponentType;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|DynamicMeshConverter")
	AVALANCHEMODIFIERS_API void SetFilterActorMode(EAvaDynamicMeshConverterModifierFilter InFilter);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|DynamicMeshConverter")
	EAvaDynamicMeshConverterModifierFilter GetFilterActorMode() const
	{
		return FilterActorMode;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|DynamicMeshConverter")
	AVALANCHEMODIFIERS_API void SetFilterActorClasses(const TSet<TSubclassOf<AActor>>& InClasses);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|DynamicMeshConverter")
	const TSet<TSubclassOf<AActor>>& GetFilterActorClasses() const
	{
		return FilterActorClasses;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|DynamicMeshConverter")
	AVALANCHEMODIFIERS_API void SetIncludeAttachedActors(bool bInInclude);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|DynamicMeshConverter")
	bool GetIncludeAttachedActors() const
	{
		return bIncludeAttachedActors;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|DynamicMeshConverter")
	AVALANCHEMODIFIERS_API void SetHideConvertedMesh(bool bInHide);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|DynamicMeshConverter")
	bool GetHideConvertedMesh() const
	{
		return bHideConvertedMesh;
	}
	
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|DynamicMeshConverter")
	AVALANCHEMODIFIERS_API void SetUpdateInterval(float InInterval);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|DynamicMeshConverter")
	float GetUpdateInterval() const
	{
		return UpdateInterval;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/** Export current dynamic mesh to static mesh asset */
	virtual void ConvertToStaticMeshAsset();
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierEnabled(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;
	virtual void RestorePreState() override;
	virtual void Apply() override;
	virtual void OnModifierRemoved(EActorModifierCoreDisableReason InReason) override;
	virtual bool IsModifierDirtyable() const override;
	//~ End UActorModifierCoreBase

	//~ Begin IAvaSceneTreeUpdateHandler
	virtual void OnSceneTreeTrackedActorChanged(int32 InIdx, AActor* InPreviousActor, AActor* InNewActor) {}
	virtual void OnSceneTreeTrackedActorChildrenChanged(int32 InIdx, const TSet<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TSet<TWeakObjectPtr<AActor>>& InNewChildrenActors);
	virtual void OnSceneTreeTrackedActorDirectChildrenChanged(int32 InIdx, const TArray<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TArray<TWeakObjectPtr<AActor>>& InNewChildrenActors) {}
	virtual void OnSceneTreeTrackedActorParentChanged(int32 InIdx, const TArray<TWeakObjectPtr<AActor>>& InPreviousParentActor, const TArray<TWeakObjectPtr<AActor>>& InNewParentActor) {}
	virtual void OnSceneTreeTrackedActorRearranged(int32 InIdx, AActor* InRearrangedActor) {}
	//~ End IAvaSceneTreeUpdateHandler

	//~ Begin IAvaRenderStateUpdateHandler
	virtual void OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent);
	virtual void OnActorVisibilityChanged(AActor* InActor) {}
	//~ End IAvaRenderStateUpdateHandler

	void OnSourceActorChanged();

	bool ConvertComponents(TArray<TWeakObjectPtr<UMaterialInterface>>& OutMaterialsWeak);
	bool HasFlag(EAvaDynamicMeshConverterModifierType InFlag) const;

	void AddDynamicMeshComponent();
	void RemoveDynamicMeshComponent();

	void GetFilteredActors(TArray<AActor*>& OutActors) const;

	/** What actor should we copy from, by default is self */
	UPROPERTY(EditInstanceOnly, Setter="SetSourceActorWeak", Getter="GetSourceActorWeak", Category="DynamicMeshConverter", meta=(DisplayName="SourceActor", AllowPrivateAccess="true"))
	TWeakObjectPtr<AActor> SourceActorWeak = GetModifiedActor();

	/** Which components should we take into account for the conversion */
	UPROPERTY(EditInstanceOnly, Setter="SetComponentType", Getter="GetComponentType", Category="DynamicMeshConverter", meta=(Bitmask, BitmaskEnum="/Script/AvalancheModifiers.EAvaDynamicMeshConverterModifierType", AllowPrivateAccess="true"))
	int32 ComponentType = static_cast<int32>(
		EAvaDynamicMeshConverterModifierType::StaticMeshComponent |
		EAvaDynamicMeshConverterModifierType::DynamicMeshComponent |
		EAvaDynamicMeshConverterModifierType::SkeletalMeshComponent |
		EAvaDynamicMeshConverterModifierType::BrushComponent |
		EAvaDynamicMeshConverterModifierType::ProceduralMeshComponent);

	/** Actor filter mode : none, include or exclude specific actor class */
	UPROPERTY(EditInstanceOnly, Setter="SetFilterActorMode", Getter="GetFilterActorMode", Category="DynamicMeshConverter", meta=(AllowPrivateAccess="true"))
	EAvaDynamicMeshConverterModifierFilter FilterActorMode = EAvaDynamicMeshConverterModifierFilter::None;

	/** Actor class to use as filter when gathering actors to convert */
	UPROPERTY(EditInstanceOnly, Setter="SetFilterActorClasses", Getter="GetFilterActorClasses", Category="DynamicMeshConverter", meta=(EditCondition="FilterActorMode != EAvaDynamicMeshConverterModifierFilter::None", AllowPrivateAccess="true"))
	TSet<TSubclassOf<AActor>> FilterActorClasses;

	/** Checks and convert all attached actors below this actor */
	UPROPERTY(EditInstanceOnly, Setter="SetIncludeAttachedActors", Getter="GetIncludeAttachedActors", Category="DynamicMeshConverter", meta=(AllowPrivateAccess="true"))
	bool bIncludeAttachedActors = true;

	/** Change visibility of source mesh once they are converted to dynamic mesh, by default will convert itself so hide converted mesh is true */
	UPROPERTY(EditInstanceOnly, Setter="SetHideConvertedMesh", Getter="GetHideConvertedMesh", Category="DynamicMeshConverter", meta=(AllowPrivateAccess="true"))
	bool bHideConvertedMesh = true;

	/** Update interval to compare if a transform/material has changed in converted components, when value <= 0 then skipped */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="DynamicMeshConverter", meta=(ClampMin="0.0"))
	float UpdateInterval = 1.f;

	/** Did we create the dynamic mesh component from this modifier or retrieved it */
	UPROPERTY()
	bool bComponentCreated = false;

	/** Components converted to dynamic mesh */
	UPROPERTY()
	TSet<FAvaDynamicMeshConverterModifierComponentState> ConvertedComponents;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional)
	FCEMeshBuilder MeshBuilder;

	FActorModifierSceneTreeActor TrackedActor;

	/** Time elasped since */
	double LastTransformUpdateTime = 0;
};
