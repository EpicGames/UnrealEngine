// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaGeometryBaseModifier.h"
#include "AvaExtrudeModifier.generated.h"

UENUM(BlueprintType)
enum class EAvaExtrudeMode : uint8
{
	Opposite,
	Front,
	Symmetrical
};

/** This modifier extrude triangles from a 2D shape with a specific depth and optionally closes the back */
UCLASS(MinimalAPI, BlueprintType)
class UAvaExtrudeModifier : public UAvaGeometryBaseModifier
{
	GENERATED_BODY()

public:
	static inline const FName ExtrudePolygroupLayerName = TEXT("ExtrudeSide");
	static inline const FName BackPolygroupLayerName = TEXT("ExtrudeBack");

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Extrude")
	AVALANCHEMODIFIERS_API void SetDepth(float InDepth);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Extrude")
	float GetDepth() const
	{
		return Depth;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Extrude")
	AVALANCHEMODIFIERS_API void SetCloseBack(bool bInCloseBack);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Extrude")
	bool GetCloseBack() const
	{
		return bCloseBack;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Extrude")
	AVALANCHEMODIFIERS_API void SetExtrudeMode(EAvaExtrudeMode InExtrudeMode);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Extrude")
	EAvaExtrudeMode GetExtrudeMode() const
	{
		return ExtrudeMode;
	}

protected:
	//~ Begin UObject
	virtual void Serialize(FArchive& InArchive) override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	void OnDepthChanged();
	void OnCloseBackChanged();
	void OnExtrudeModeChanged();
	FVector GetExtrudeDirection() const;

	/** Handles mesh depth to extrude primary section */
	UPROPERTY(EditInstanceOnly, Setter="SetDepth", Getter="GetDepth", Category="Extrude", meta=(ClampMin="0", DisplayName="Extrude Depth", Units="Centimeters", AllowPrivateAccess="true"))
	float Depth = 30.f;

	/** Closes the back of the extrude for a 2D shape for example */
	UPROPERTY(EditInstanceOnly, Setter="SetCloseBack", Getter="GetCloseBack", Category="Extrude", meta=(AllowPrivateAccess="true"))
	bool bCloseBack = true;

	/** Moves the mesh in the opposite extrude direction by the depth distance */
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use ExtrudeMode instead"))
	bool bMoveMeshOppositeDirection_DEPRECATED = true;

	/** Specifies the Extrude direction */
	UPROPERTY(EditInstanceOnly, Setter="SetExtrudeMode", Getter="GetExtrudeMode", Category="Extrude", meta=(AllowPrivateAccess="true"))
	EAvaExtrudeMode ExtrudeMode = EAvaExtrudeMode::Opposite;
};
