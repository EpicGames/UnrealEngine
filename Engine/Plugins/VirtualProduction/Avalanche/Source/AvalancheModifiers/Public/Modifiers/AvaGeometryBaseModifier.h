// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Misc/Optional.h"
#include "Modifiers/ActorModifierCoreBase.h"
#include "UObject/ObjectPtr.h"
#include "AvaGeometryBaseModifier.generated.h"

class UDynamicMesh;
class UDynamicMeshComponent;

/** Base class used for geometry modifier that uses dynamic mesh components */
UCLASS(Abstract)
class UAvaGeometryBaseModifier : public UActorModifierCoreBase
{
	GENERATED_BODY()

	friend class FAvaGeometryModifierProfiler;

public:
	/** Get polygroup layer by name */
    static UE::Geometry::FDynamicMeshPolygroupAttribute* FindOrCreatePolygroupLayer(UE::Geometry::FDynamicMesh3& EditMesh, const FName& InLayerName, TArray<int32>* GroupTriangles = nullptr);

protected:
	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual bool IsModifierReady() const override;
	virtual void SavePreState() override;
	virtual void RestorePreState() override;
	virtual void OnModifierEnabled(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;
	//~ End UActorModifierCoreBase

	void OnMeshChanged();

	/** Gets the cached dynamic mesh component. If null, find the first attached and caches it */
	UDynamicMeshComponent* GetMeshComponent() const;

	/** Gets the dynamic mesh object from the cached mesh component */
	UDynamicMesh* GetMeshObject() const;

	/** Checks if the dynamic mesh component is valid */
	bool IsMeshValid() const;

	/** Get dynamic mesh bounds */
	FBox GetMeshBounds() const;

	/** Cached Mesh to restore to the Pre-Modifier State */
	TOptional<UE::Geometry::FDynamicMesh3> PreModifierCachedMesh;
	TOptional<FBox> PreModifierCachedBounds;

private:
	/** The actor dynamic mesh component */
	UPROPERTY(DuplicateTransient)
	TWeakObjectPtr<UDynamicMeshComponent> MeshComponent = nullptr;
};
