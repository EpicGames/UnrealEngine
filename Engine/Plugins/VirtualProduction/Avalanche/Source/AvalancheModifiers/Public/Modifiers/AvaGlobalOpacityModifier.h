// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/AvaMaterialParameterModifier.h"
#include "AvaGlobalOpacityModifier.generated.h"

class UDMMaterialComponent;
enum class EDMUpdateType : uint8;

/** This modifier sets global opacity parameters on an actor with Material Designer Instances generated with the Material Designer */
UCLASS(MinimalAPI, BlueprintType)
class UAvaGlobalOpacityModifier : public UAvaMaterialParameterModifier
{
	GENERATED_BODY()

public:
	UAvaGlobalOpacityModifier();

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|GlobalOpacity")
	AVALANCHEMODIFIERS_API void SetGlobalOpacity(float InOpacity);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|GlobalOpacity")
	float GetGlobalOpacity() const
	{
		return GlobalOpacity;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void Apply() override;
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	//~ End UActorModifierCoreBase

	void OnGlobalOpacityChanged();

	virtual void OnActorMaterialAdded(UMaterialInstanceDynamic* InAdded) override;
	virtual void OnActorMaterialRemoved(UMaterialInstanceDynamic* InRemoved) override;
	void OnDynamicMaterialValueChanged(UDMMaterialComponent* InComponent, UDMMaterialComponent* InSource, EDMUpdateType InUpdateType);

	/** Global opacity to set on all Material Designer Instances */
	UPROPERTY(EditInstanceOnly, Setter="SetGlobalOpacity", Getter="GetGlobalOpacity", Category="Material Parameter", meta=(ClampMin="0", ClampMax="1", AllowPrivateAccess="true"))
	float GlobalOpacity = 1.f;

	UPROPERTY()
	int32 SupportedOpacityMaterialCount = 0;
};
