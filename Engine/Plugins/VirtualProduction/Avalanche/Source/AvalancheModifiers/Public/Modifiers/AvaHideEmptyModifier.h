// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/ActorModifierArrangeBaseModifier.h"
#include "AvaHideEmptyModifier.generated.h"

class UText3DComponent;

UCLASS(MinimalAPI, BlueprintType)
class UAvaHideEmptyModifier : public UActorModifierArrangeBaseModifier
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|HideEmpty")
	AVALANCHEMODIFIERS_API void SetContainerActor(AActor* InActor)
	{
		SetContainerActorWeak(InActor);
	}

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|HideEmpty")
	AActor* GetContainerActor() const
	{
		return ContainerActorWeak.Get();
	}

	AVALANCHEMODIFIERS_API void SetContainerActorWeak(TWeakObjectPtr<AActor> InContainer);
	TWeakObjectPtr<AActor> GetContainerActorWeak() const
	{
		return ContainerActorWeak;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|HideEmpty")
	AVALANCHEMODIFIERS_API void SetInvertVisibility(bool bInInvert);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|HideEmpty")
	bool GetInvertVisibility() const
	{
		return bInvertVisibility;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierEnabled(EActorModifierCoreEnableReason InReason) override;
	virtual void Apply() override;
	virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;
	virtual void OnModifiedActorTransformed() override;
	//~ End UActorModifierCoreBase

	void OnTextChanged();
	void OnContainerActorChanged();
	void OnInvertVisibilityChanged();

	//~ Begin IAvaSceneTreeUpdateModifierExtension
	virtual void OnSceneTreeTrackedActorChanged(int32 InIdx, AActor* InPreviousActor, AActor* InNewActor) override;
	virtual void OnSceneTreeTrackedActorDirectChildrenChanged(int32 InIdx, const TArray<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TArray<TWeakObjectPtr<AActor>>& InNewChildrenActors) override;
	//~ End IAvaSceneTreeUpdateModifierExtension

	/** The container to hide when text is empty, by default self */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="HideEmpty", meta=(DisplayName="ContainerActor", AllowPrivateAccess="true"))
	TWeakObjectPtr<AActor> ContainerActorWeak;

	/** Invert the behaviour and visibility of the container if text is empty */
	UPROPERTY(EditInstanceOnly, Setter="SetInvertVisibility", Getter="GetInvertVisibility", Category="HideEmpty", meta=(AllowPrivateAccess="true"))
	bool bInvertVisibility = false;

private:
	/** Cached text component */
	UPROPERTY()
	TWeakObjectPtr<UText3DComponent> TextComponent;
};
