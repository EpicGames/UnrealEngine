// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/ActorModifierArrangeBaseModifier.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "AvaMaterialParameterModifier.generated.h"

class UMaterialInstanceDynamic;
class UTexture;

USTRUCT(BlueprintType)
struct FAvaMaterialParameterMapScalar
{
	GENERATED_BODY()

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "MaterialParameter")
	FName Name = NAME_None;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "MaterialParameter")
	float Value = 0.f;
};

USTRUCT(BlueprintType)
struct FAvaMaterialParameterMapVector
{
	GENERATED_BODY()

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "MaterialParameter")
	FName Name = NAME_None;
	
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "MaterialParameter")
	FLinearColor Value = FLinearColor::Black;
};

USTRUCT(BlueprintType)
struct FAvaMaterialParameterMapTexture
{
	GENERATED_BODY()

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "MaterialParameter")
	FName Name = NAME_None;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "MaterialParameter")
	TObjectPtr<UTexture> Value = nullptr;
};

USTRUCT(BlueprintType)
struct FAvaMaterialParameterMap
{
	GENERATED_BODY()

	friend class UAvaMaterialParameterModifier;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAvaMaterialParameterMap() = default;
	FAvaMaterialParameterMap(const FAvaMaterialParameterMap&) = default;
	FAvaMaterialParameterMap(FAvaMaterialParameterMap&&) = default;
	FAvaMaterialParameterMap& operator=(const FAvaMaterialParameterMap&) = default;
	FAvaMaterialParameterMap& operator=(FAvaMaterialParameterMap&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Matches the input parameter key map and removes all unused keys, does not touch current values */
	void MatchKeys(const FAvaMaterialParameterMap& InParameterMap);

	/** Apply our parameters to this Material Designer Instance */
	void PushParametersTo(UMaterialInstanceDynamic* InMaterial);

	/** Read parameters from this Material Designer Instance and save them here. */
	void PullParametersFrom(UMaterialInstanceDynamic* InMaterial);

	/** Returns the scalar parameter struct with the given name. */
	AVALANCHEMODIFIERS_API const FAvaMaterialParameterMapScalar* FindScalarParameter(FName InParameterName) const;

	/** Returns the scalar parameter struct with the given name, potentially creating it if it doesn't exist. */
	AVALANCHEMODIFIERS_API FAvaMaterialParameterMapScalar* FindScalarParameter(FName InParameterName, bool bInCreateIfMissing = false);

	/** Returns the vector parameter struct with the given name. */
	AVALANCHEMODIFIERS_API const FAvaMaterialParameterMapVector* FindVectorParameter(FName InParameterName) const;

	/** Returns the vector parameter struct with the given name, potentially creating it if it doesn't exist. */
	AVALANCHEMODIFIERS_API FAvaMaterialParameterMapVector* FindVectorParameter(FName InParameterName, bool bInCreateIfMissing = false);

	/** Returns the texture parameter struct with the given name. */
	AVALANCHEMODIFIERS_API const FAvaMaterialParameterMapTexture* FindTextureParameter(FName InParameterName) const;

	/** Returns the texture parameter struct with the given name, potentially creating it if it doesn't exist. */
	AVALANCHEMODIFIERS_API FAvaMaterialParameterMapTexture* FindTextureParameter(FName InParameterName, bool bInCreateIfMissing = false);

private:
#if WITH_EDITOR
	void UpdateDeprecatedParameterProperties();
#endif

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "MaterialParameter",
		meta = (DisplayName = "Scalar Parameters", AllowPrivateAccess, TitleProperty = "Name", ShowOnlyInnerProperties))
	TArray<FAvaMaterialParameterMapScalar> ScalarParameterStructs;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "MaterialParameter",
		meta = (DisplayName = "Vector Parameters", AllowPrivateAccess, TitleProperty = "Name", ShowOnlyInnerProperties))
	TArray<FAvaMaterialParameterMapVector> VectorParameterStructs;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "MaterialParameter",
		meta = (DisplayName = "Texture Parameters", AllowPrivateAccess, TitleProperty = "Name", ShowOnlyInnerProperties))
	TArray<FAvaMaterialParameterMapTexture> TextureParameterStructs;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.7, "ScalarParameters has been deprecated. Please use ScalarParameterStructs")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property has been refactored to use an array and struct, instead of a map."))
	TMap<FName, float> ScalarParameters_DEPRECATED;

	UE_DEPRECATED(5.7, "VectorParameters has been deprecated. Please use VectorParameterStructs")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property has been refactored to use an array and struct, instead of a map."))
	TMap<FName, FLinearColor> VectorParameters_DEPRECATED;

	UE_DEPRECATED(5.7, "TextureParameters has been deprecated. Please use TextureParameterStructs")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property has been refactored to use an array and struct, instead of a map."))
	TMap<FName, TObjectPtr<UTexture>> TextureParameters_DEPRECATED;
#endif
};

/** This modifier sets specified dynamic materials parameters on an actor and its children */
UCLASS(MinimalAPI, BlueprintType)
class UAvaMaterialParameterModifier : public UActorModifierArrangeBaseModifier
{
	GENERATED_BODY()

public:
	UAvaMaterialParameterModifier();

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|MaterialParameter")
	AVALANCHEMODIFIERS_API void SetMaterialParameters(const FAvaMaterialParameterMap& InParameterMap);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|MaterialParameter")
	const FAvaMaterialParameterMap& GetMaterialParameters() const
	{
		return MaterialParameters;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|MaterialParameter")
	AVALANCHEMODIFIERS_API void SetUpdateChildren(bool bInUpdateChildren);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|MaterialParameter")
	bool GetUpdateChildren() const
	{
		return bUpdateChildren;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifierEnabled(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;
	virtual void OnModifiedActorTransformed() override;
	virtual void RestorePreState() override;
	virtual void SavePreState() override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	//~ Begin IAvaSceneTreeUpdateModifierExtension
	virtual void OnSceneTreeTrackedActorChildrenChanged(int32 InIdx, const TSet<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TSet<TWeakObjectPtr<AActor>>& InNewChildrenActors) override;
	virtual void OnSceneTreeTrackedActorDirectChildrenChanged(int32 InIdx, const TArray<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TArray<TWeakObjectPtr<AActor>>& InNewChildrenActors) override;
	//~ End IAvaSceneTreeUpdateModifierExtension

	/** Read and save original values */
	void SaveMaterialParameters();
	/** Write and restore original values */
	void RestoreMaterialParameters();

#if WITH_EDITOR
	/** Called when a property changes, used to detect material changes */
	virtual void OnActorPropertyChanged(UObject* InObject, FPropertyChangedEvent& InChangeEvent);
#endif

	void ScanActorMaterials();

	void OnMaterialParametersChanged();
	void OnUpdateChildrenChanged();

	virtual void OnActorMaterialAdded(UMaterialInstanceDynamic* InAdded) {}
	virtual void OnActorMaterialRemoved(UMaterialInstanceDynamic* InRemoved) {}

	/** Checks if this actor has a Material Designer Instance or that we already track one */
	bool IsActorSupported(const AActor* InActor) const;

	/** Retrieves all Material Designer Instance from a primitive component */
	TSet<UMaterialInstanceDynamic*> GetComponentDynamicMaterials(const UPrimitiveComponent* InComponent) const;

	/** Which parameters should we set on the Material Designer Instance */
	UPROPERTY(EditInstanceOnly, Setter="SetMaterialParameters", Getter="GetMaterialParameters", Category="MaterialParameter", meta=(HideEditConditionToggle, EditCondition="bShowMaterialParameters", EditConditionHides, AllowPrivateAccess="true"))
	FAvaMaterialParameterMap MaterialParameters;

	/** Used to restore Material Designer Instance parameters to their original state */
	UPROPERTY()
	TMap<TWeakObjectPtr<UMaterialInstanceDynamic>, FAvaMaterialParameterMap> SavedMaterialParameters;

	/** Filter material type for child modifiers */
	UPROPERTY(Transient)
	TSubclassOf<UMaterialInstanceDynamic> MaterialClass;

	/** Will also look into attached children actors */
	UPROPERTY(EditInstanceOnly, Setter="SetUpdateChildren", Getter="GetUpdateChildren", Category="MaterialParameter", meta=(AllowPrivateAccess="true"))
	bool bUpdateChildren = true;

#if WITH_EDITORONLY_DATA
	/** Used by child classes to override MaterialParameters */
	UPROPERTY(Transient)
	bool bShowMaterialParameters = true;
#endif

private:
#if WITH_EDITOR
	void UpdateDeprecatedParameterProperties();
#endif
};
