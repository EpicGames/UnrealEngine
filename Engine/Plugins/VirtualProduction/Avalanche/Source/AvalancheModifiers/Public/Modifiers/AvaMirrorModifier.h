// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaGeometryBaseModifier.h"
#include "AvaModifiersPreviewPlane.h"
#include "AvaMirrorModifier.generated.h"

class UStaticMesh;

UCLASS(MinimalAPI, BlueprintType)
class UAvaMirrorModifier : public UAvaGeometryBaseModifier
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Mirror")
	AVALANCHEMODIFIERS_API void SetMirrorFramePosition(const FVector& InMirrorFramePosition);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Mirror")
	const FVector& GetMirrorFramePosition() const
	{
		return MirrorFramePosition;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Mirror")
	AVALANCHEMODIFIERS_API void SetMirrorFrameRotation(const FRotator& InMirrorFrameRotation);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Mirror")
	const FRotator& GetMirrorFrameRotation() const
	{
		return MirrorFrameRotation;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Mirror")
	AVALANCHEMODIFIERS_API void SetApplyPlaneCut(bool bInApplyPlaneCut);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Mirror")
	bool GetApplyPlaneCut() const
	{
		return bApplyPlaneCut;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Mirror")
	AVALANCHEMODIFIERS_API void SetFlipCutSide(bool bInFlipCutSide);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Mirror")
	bool GetFlipCutSide() const
	{
		return bFlipCutSide;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Mirror")
	AVALANCHEMODIFIERS_API void SetWeldAlongPlane(bool bInWeldAlongPlane);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Mirror")
	bool GetWeldAlongPlane() const
	{
		return bWeldAlongPlane;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	virtual void Apply() override;
	virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;
	virtual void OnModifierRemoved(EActorModifierCoreDisableReason InReason) override;
	//~ End UActorModifierCoreBase

	void OnMirrorFrameChanged();
	void OnMirrorOptionChanged();

#if WITH_EDITOR
	void CreatePreviewComponent();
	void DestroyPreviewComponent();
	void UpdatePreviewComponent();

	void OnShowMirrorFrameChanged();
#endif

    UPROPERTY(EditInstanceOnly, Setter="SetMirrorFramePosition", Getter="GetMirrorFramePosition", Category="Mirror", meta=(AllowPrivateAccess="true"))
	FVector MirrorFramePosition = FVector::ZeroVector;

	UPROPERTY(EditInstanceOnly, Setter="SetMirrorFrameRotation", Getter="GetMirrorFrameRotation", Category="Mirror", meta=(AllowPrivateAccess="true"))
	FRotator MirrorFrameRotation = FRotator(0, 0, 90);

	UPROPERTY(EditInstanceOnly, Setter="SetApplyPlaneCut", Getter="GetApplyPlaneCut", Category="Mirror", meta=(AllowPrivateAccess="true"))
	bool bApplyPlaneCut = true;

	UPROPERTY(EditInstanceOnly, Setter="SetFlipCutSide", Getter="GetFlipCutSide", Category="Mirror", meta=(AllowPrivateAccess="true"))
	bool bFlipCutSide = false;

	UPROPERTY(EditInstanceOnly, Setter="SetWeldAlongPlane", Getter="GetWeldAlongPlane", Category="Mirror", meta=(AllowPrivateAccess="true"))
	bool bWeldAlongPlane = true;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditInstanceOnly, Category="Mirror", meta=(AllowPrivateAccess="true"))
	bool bShowMirrorFrame = false;

private:
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	FAvaModifierPreviewPlane PreviewPlane;
#endif
};
