// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaGeometryBaseModifier.h"
#include "AvaNormalModifier.generated.h"

UENUM(BlueprintType)
enum class EAvaNormalModifierSplitMethod : uint8
{
	/** Do not split, leave as it is */
	None,
	/** Each vertex will have a split normal between tris */
	Vertex,
	/** Shared vertex between triangles will have a split normal */
	Triangle,
	/** Vertices of a same face grouped together will have a split normal */
	PolyGroup,
	/** Vertices above a certain angle threshold will have a split normal */
	Threshold
};

UCLASS(MinimalAPI, BlueprintType)
class UAvaNormalModifier : public UAvaGeometryBaseModifier
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Normal")
	AVALANCHEMODIFIERS_API void SetAngleWeighted(bool bInAngleWeighted);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Normal")
	bool GetAngleWeighted() const
	{
		return bAngleWeighted;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Normal")
	AVALANCHEMODIFIERS_API void SetAreaWeighted(bool bInAreaWeighted);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Normal")
	bool GetAreaWeighted() const
	{
		return bAreaWeighted;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Normal")
	AVALANCHEMODIFIERS_API void SetInvert(bool bInInvert);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Normal")
	bool GetInvert() const
	{
		return bInvert;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Normal")
	AVALANCHEMODIFIERS_API void SetSplitMethod(EAvaNormalModifierSplitMethod InSplitMethod);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Normal")
	EAvaNormalModifierSplitMethod GetSplitMethod() const
	{
		return SplitMethod;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Normal")
	AVALANCHEMODIFIERS_API void SetAngleThreshold(float InAngleThreshold);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Normal")
	float GetAngleThreshold() const
	{
		return AngleThreshold;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Normal")
	AVALANCHEMODIFIERS_API void SetPolyGroupLayerIdx(int32 InPolyGroupLayer);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Normal")
	AVALANCHEMODIFIERS_API int32 GetPolyGroupLayerIdx() const;

	AVALANCHEMODIFIERS_API void SetPolyGroupLayer(const FString& InPolyGroupLayer);

	FString GetPolyGroupLayer() const
	{
		return PolyGroupLayer;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	void OnAngleWeightedChanged();
	void OnAreaWeightedChanged();
	void OnInvertChanged();
	void OnSplitMethodChanged();
	void OnAngleThresholdChanged();
	void OnPolyGroupLayerChanged();

	UFUNCTION()
	TArray<FString> GetPolyGroupLayers() const;

	/** Recompute normals and weight them by angle */
	UPROPERTY(EditInstanceOnly, Setter="SetAngleWeighted", Getter="GetAngleWeighted", Category="Normal", meta=(AllowPrivateAccess="true"))
	bool bAngleWeighted = true;

	/** Recompute normals and weight them by area */
	UPROPERTY(EditInstanceOnly, Setter="SetAreaWeighted", Getter="GetAreaWeighted", Category="Normal", meta=(AllowPrivateAccess="true"))
	bool bAreaWeighted = true;

	/** Recompute normals and invert normals and triangles */
	UPROPERTY(EditInstanceOnly, Setter="SetInvert", Getter="GetInvert", Category="Normal", meta=(AllowPrivateAccess="true"))
	bool bInvert = false;

	/** Recompute normals and use a split method */
	UPROPERTY(EditInstanceOnly, Setter="SetSplitMethod", Getter="GetSplitMethod", Category="Normal", meta=(AllowPrivateAccess="true"))
	EAvaNormalModifierSplitMethod SplitMethod = EAvaNormalModifierSplitMethod::Threshold;

	/** Angle to compare and split normal when threshold method is chosen */
	UPROPERTY(EditInstanceOnly, Setter="SetAngleThreshold", Getter="GetAngleThreshold", Category="Normal", meta=(ClampMin="0.0", ClampMax="180.0", EditCondition="SplitMethod == EAvaNormalModifierSplitMethod::Threshold", AllowPrivateAccess="true"))
	float AngleThreshold = 60.f;

	/** PolyGroup to use to split normal from when PolyGroup method is chosen */
	UPROPERTY(EditInstanceOnly, Setter="SetPolyGroupLayer", Getter="GetPolyGroupLayer", Category="Normal", meta=(GetOptions="GetPolyGroupLayers", EditCondition="SplitMethod == EAvaNormalModifierSplitMethod::PolyGroup", AllowPrivateAccess="true"))
	FString PolyGroupLayer = TEXT("None");
};
