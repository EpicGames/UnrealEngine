// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaGeometryBaseModifier.h"
#include "AvaOutlineModifier.generated.h"

UENUM(BlueprintType)
enum class EAvaOutlineMode : uint8
{
	Outset,
	Inset
};

/** This modifier adds an outline around a 2D shape with a specific distance */
UCLASS(MinimalAPI, BlueprintType)
class UAvaOutlineModifier : public UAvaGeometryBaseModifier
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Outline")
	AVALANCHEMODIFIERS_API void SetMode(EAvaOutlineMode InMode);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Outline")
	EAvaOutlineMode GetMode() const
	{
		return Mode;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Outline")
	AVALANCHEMODIFIERS_API void SetDistance(float InDistance);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Outline")
	float GetDistance() const
	{
		return Distance;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Outline")
	AVALANCHEMODIFIERS_API void SetRemoveInside(bool bInRemoveInside);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Outline")
	bool GetRemoveInside() const
	{
		return bRemoveInside;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	void OnModeChanged();
	void OnDistanceChanged();
	void OnRemoveInsideChanged();

	float GetMaxInsetDistance() const;

	/** Set the mode like inset or outset */
	UPROPERTY(EditInstanceOnly, Setter="SetMode", Getter="GetMode", Category="Outline", meta=(AllowPrivateAccess="true"))
	EAvaOutlineMode Mode = EAvaOutlineMode::Outset;

	/** Set the distance for the outline */
	UPROPERTY(EditInstanceOnly, Setter="SetDistance", Getter="GetDistance", Category="Outline", meta=(ClampMin="0", AllowPrivateAccess="true"))
	float Distance = 10.f;

	/** Remove the inside part and create a hole in the shape */
	UPROPERTY(EditInstanceOnly, Setter="SetRemoveInside", Getter="GetRemoveInside", Category="Outline", meta=(AllowPrivateAccess="true"))
	bool bRemoveInside = true;
};
