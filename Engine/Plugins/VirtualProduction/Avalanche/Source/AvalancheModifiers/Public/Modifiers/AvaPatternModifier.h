// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaGeometryBaseModifier.h"
#include "Tools/AvaPatternModifierTool.h"
#include "TransformTypes.h"
#include "AvaPatternModifier.generated.h"

UENUM(BlueprintType)
enum class EAvaPatternModifierLayout : uint8
{
	Line = 0,
	Grid = 1,
	Circle = 2
};

UENUM(BlueprintType)
enum class EAvaPatternModifierAxis : uint8
{
	X = 0,
	Y = 1,
	Z = 2
};

UENUM(BlueprintType)
enum class EAvaPatternModifierPlane : uint8
{
	XY = 0,
	ZX = 1,
	YZ = 2
};

UENUM(BlueprintType)
enum class EAvaPatternModifierLineAlignment : uint8
{
	Start,
	Center,
	End
};

UENUM(BlueprintType)
enum class EAvaPatternModifierGridAlignment : uint8
{
	TopLeft,
	TopRight,
	Center,
	BottomLeft,
	BottomRight
};

USTRUCT(BlueprintType)
struct FVector2b
{
	GENERATED_BODY()

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Values")
	bool bX = false;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Values")
	bool bY = false;
};

USTRUCT(BlueprintType)
struct FAvaPatternModifierLineLayoutOptions
{
	GENERATED_BODY()

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	EAvaPatternModifierAxis Axis = EAvaPatternModifierAxis::Y;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	bool bAxisInverted = false;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern", meta=(ClampMin="1"))
	int32 RepeatCount = 4;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	float Spacing = 0.f;

	// Center the layout based on the axis
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	bool bCentered = true;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	bool bAccumulateTransform = false;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	FRotator Rotation = FRotator::ZeroRotator;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	FVector Scale = FVector::OneVector;
};

USTRUCT(BlueprintType)
struct FAvaPatternModifierGridLayoutOptions
{
	GENERATED_BODY()

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	EAvaPatternModifierPlane Plane = EAvaPatternModifierPlane::YZ;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	FVector2b AxisInverted;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern", meta=(ClampMin="1"))
	FIntPoint RepeatCount = FIntPoint(2, 2); // Row, Column

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	FVector2D Spacing = FVector2D(0.f);

	// Center the layout based on the plane
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	bool bCentered = true;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	bool bAccumulateTransform = false;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	FRotator Rotation = FRotator::ZeroRotator;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	FVector Scale = FVector::OneVector;
};

USTRUCT(BlueprintType)
struct FAvaPatternModifierCircleLayoutOptions
{
	GENERATED_BODY()

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	EAvaPatternModifierPlane Plane = EAvaPatternModifierPlane::YZ;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	float Radius = 100.f;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	float StartAngle = 180.f;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	float FullAngle = 360.f;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern", meta=(ClampMin="1"))
	int32 RepeatCount = 4;

	// Center the layout based on the plane
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	bool bCentered = true;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	bool bAccumulateTransform = false;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	FRotator Rotation = FRotator::ZeroRotator;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Pattern")
	FVector Scale = FVector::OneVector;
};

/** This modifier clones a shape following various layouts and options */
UCLASS(MinimalAPI, BlueprintType, AutoExpandCategories=(Pattern))
class UAvaPatternModifier : public UAvaGeometryBaseModifier
{
	GENERATED_BODY()

public:
	UAvaPatternModifier();

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetActiveToolClass(const TSubclassOf<UAvaPatternModifierTool>& InClass);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	TSubclassOf<UAvaPatternModifierTool> GetActiveToolClass() const
	{
		return ActiveToolClass;
	}

protected:
	//~ Begin UObject
	virtual void Serialize(FArchive& InArchive) override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierEnabled(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	/** Finds saved tool or create and saves it */
	UAvaPatternModifierTool* FindOrAddTool(TSubclassOf<UAvaPatternModifierTool> InToolClass);

	template<typename InToolClass>
	InToolClass* FindOrAddTool()
	{
		return Cast<InToolClass>(FindOrAddTool(InToolClass::StaticClass()));
	}

	void OnActiveToolClassChanged();

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern", NoClear, meta=(ShowDisplayNames, AllowPrivateAccess="true"))
	TSubclassOf<UAvaPatternModifierTool> ActiveToolClass;

	UPROPERTY(VisibleInstanceOnly, Instanced, Category="Pattern", Transient, TextExportTransient, DuplicateTransient)
	TObjectPtr<UAvaPatternModifierTool> ActiveTool;

	UPROPERTY()
	TArray<TObjectPtr<UAvaPatternModifierTool>> Tools;

private:
	void MigrateVersion(int32 InCurrentVersion);

	UE_DEPRECATED(5.6, "Moved to ToolName")
	UPROPERTY()
	EAvaPatternModifierLayout Layout = EAvaPatternModifierLayout::Line;

	UE_DEPRECATED(5.6, "Moved to Line Tool")
	UPROPERTY()
	FAvaPatternModifierLineLayoutOptions LineLayoutOptions;

	UE_DEPRECATED(5.6, "Moved to Grid Tool")
	UPROPERTY()
	FAvaPatternModifierGridLayoutOptions GridLayoutOptions;

	UE_DEPRECATED(5.6, "Moved to Circle Tool")
	UPROPERTY()
	FAvaPatternModifierCircleLayoutOptions CircleLayoutOptions;

	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	FBox OriginalMeshBounds;
};
