// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaGeometryBaseModifier.h"
#include "AvaModifiersPreviewPlane.h"
#include "AvaPlaneCutModifier.generated.h"

/** This modifier cuts a shape based on a 2D plane */
UCLASS(MinimalAPI, BlueprintType)
class UAvaPlaneCutModifier : public UAvaGeometryBaseModifier
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|PlaneCut")
	AVALANCHEMODIFIERS_API void SetPlaneOrigin(float InOrigin);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|PlaneCut")
	float GetPlaneOrigin() const
	{
		return PlaneOrigin;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|PlaneCut")
	AVALANCHEMODIFIERS_API void SetPlaneRotation(const FRotator& InRotation);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|PlaneCut")
	const FRotator& GetPlaneRotation() const
	{
		return PlaneRotation;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|PlaneCut")
	AVALANCHEMODIFIERS_API void SetInvertCut(bool bInInvertCut);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|PlaneCut")
	bool GetInvertCut() const
	{
		return bInvertCut;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|PlaneCut")
	AVALANCHEMODIFIERS_API void SetFillHoles(bool bInFillHoles);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|PlaneCut")
	bool GetFillHoles() const
	{
		return bFillHoles;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	virtual void Apply() override;
	virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;
	virtual void OnModifierRemoved(EActorModifierCoreDisableReason InReason) override;
	//~ End UActorModifierCoreBase

	/** Returns actual location of plane bounds restricted */
	FVector GetPlaneLocation() const;

	void OnPlaneRotationChanged();
	void OnFillHolesChanged();
	void OnInvertCutChanged();
	void OnPlaneOriginChanged();

#if WITH_EDITOR
	void OnUsePreviewChanged();
	void CreatePreviewComponent();
	void DestroyPreviewComponent();
	void UpdatePreviewComponent();
#endif

	UPROPERTY(EditInstanceOnly, Setter="SetPlaneOrigin", Getter="GetPlaneOrigin", Category="PlaneCut", meta=(Delta="0.5", LinearDeltaSensitivity="1", AllowPrivateAccess="true"))
	float PlaneOrigin = 0.f;

	UPROPERTY(EditInstanceOnly, Setter="SetPlaneRotation", Getter="GetPlaneRotation", Category="PlaneCut", meta=(AllowPrivateAccess="true"))
	FRotator PlaneRotation = FRotator(0, 0, 90);

	UPROPERTY(EditInstanceOnly, Setter="SetInvertCut", Getter="GetInvertCut", Category="PlaneCut", meta=(AllowPrivateAccess="true"))
	bool bInvertCut = false;

	UPROPERTY(EditInstanceOnly, Setter="SetFillHoles", Getter="GetFillHoles", Category="PlaneCut", meta=(AllowPrivateAccess="true"))
	bool bFillHoles = true;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditInstanceOnly, Category="PlaneCut", meta=(AllowPrivateAccess="true"))
	bool bUsePreview = false;

private:
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	FAvaModifierPreviewPlane PreviewPlane;
#endif
};
