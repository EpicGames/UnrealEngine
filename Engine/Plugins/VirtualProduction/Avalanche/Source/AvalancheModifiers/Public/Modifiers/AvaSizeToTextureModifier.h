// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaGeometryBaseModifier.h"
#include "AvaPropertyChangeDispatcher.h"
#include "Extensions/ActorModifierRenderStateUpdateExtension.h"
#include "AvaSizeToTextureModifier.generated.h"

class UActorComponent;
class UAvaShape2DDynMeshBase;
class UTexture;

UENUM()
enum class EAvaSizeToTextureRule : uint8
{
	/**
	 * Automatically adapts the width based on the height, to maintain texture ratio
	 */
	AdaptiveWidth,
	/**
	 * Automatically adapts the height based on the width, to maintain texture ratio
	 */
	AdaptiveHeight,
	/** Lock height and adapt width */
	FixedHeight    UMETA(Hidden),
	/** Lock width and adapt height */
	FixedWidth     UMETA(Hidden),
};

/**
 * Adapts the modified actor geometry size/scale to match the texture size/ratio
 */
UCLASS(MinimalAPI, BlueprintType)
class UAvaSizeToTextureModifier : public UAvaGeometryBaseModifier
	, public IActorModifierRenderStateUpdateHandler
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|TextureSize")
	AVALANCHEMODIFIERS_API void SetTexture(UTexture* InTexture);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|TextureSize")
	UTexture* GetTexture() const
	{
		return Texture;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|TextureSize")
	AVALANCHEMODIFIERS_API void SetRule(EAvaSizeToTextureRule InRule);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|TextureSize")
	EAvaSizeToTextureRule GetRule() const
	{
		return Rule;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|TextureSize")
	AVALANCHEMODIFIERS_API void SetFixedHeight(float InFixedHeight);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|TextureSize")
	float GetFixedHeight() const
	{
		return FixedHeight;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|TextureSize")
	AVALANCHEMODIFIERS_API void SetFixedWidth(float InFixedWidth);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|TextureSize")
	float GetFixedWidth() const
	{
		return FixedWidth;
	}
	
protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifiedActorTransformed() override;
	virtual void SavePreState() override;
	virtual void RestorePreState() override;
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierEnabled(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	//~ Begin IAvaRenderStateUpdateExtension
	virtual void OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent) override;
	virtual void OnActorVisibilityChanged(AActor* InActor) override;
	//~ End IAvaRenderStateUpdateExtension

	void OnTextureOptionsChanged();

	void GetShapeSizeAndScale(FVector2D& OutShapeSize, FVector2D& OutShapeScale) const;
	void CheckSizeOrScaleChanged();

	/** Texture to resize to */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="TextureSize", meta=(AllowPrivateAccess="true"))
	TObjectPtr<UTexture> Texture;

	/** Rule for texture resize */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="TextureSize", meta=(AllowPrivateAccess="true"))
	EAvaSizeToTextureRule Rule = EAvaSizeToTextureRule::AdaptiveWidth;

	/** The fixed height size */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="TextureSize", meta=(ClampMin="0", EditCondition="Rule == EAvaSizeToTextureRule::FixedHeight", EditConditionHides, AllowPrivateAccess="true"))
	float FixedHeight = 512.f;

	/** The fixed width size */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="TextureSize", meta=(ClampMin="0", EditCondition="Rule == EAvaSizeToTextureRule::FixedWidth", EditConditionHides, AllowPrivateAccess="true"))
	float FixedWidth = 512.f;

private:
	UPROPERTY()
	FVector2D PreModifierShape2DSize = FVector2D::ZeroVector;

	FVector2D CachedScale = FVector2D::ZeroVector;
	FVector2D CachedSize = FVector2D::ZeroVector;

	UPROPERTY()
	TWeakObjectPtr<UAvaShape2DDynMeshBase> Shape2DWeak;

#if WITH_EDITOR
	static const TAvaPropertyChangeDispatcher<UAvaSizeToTextureModifier> PropertyChangeDispatcher;
#endif
};