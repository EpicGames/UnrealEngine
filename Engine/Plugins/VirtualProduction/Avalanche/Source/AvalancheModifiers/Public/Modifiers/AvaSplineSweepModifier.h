// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaGeometryBaseModifier.h"
#include "AvaPropertyChangeDispatcher.h"
#include "Extensions/ActorModifierRenderStateUpdateExtension.h"
#include "AvaSplineSweepModifier.generated.h"

class USplineComponent;

UENUM(BlueprintType)
enum class EAvaSplineSweepSampleMode : uint8
{
	/** Samples the full spline distance : steps on the full distance */
	FullDistance,
	/** Samples a custom distance : steps per custom distance */
	CustomDistance
};

UCLASS(MinimalAPI, BlueprintType)
class UAvaSplineSweepModifier : public UAvaGeometryBaseModifier
	, public IActorModifierRenderStateUpdateHandler
{
	GENERATED_BODY()

public:
	static constexpr int32 MaxSampleCount = 200;

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplineSweep")
	AVALANCHEMODIFIERS_API void SetSplineActor(AActor* InActor)
	{
		SetSplineActorWeak(InActor);
	}

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplineSweep")
	AActor* GetSplineActor() const
	{
		return SplineActorWeak.Get();
	}

	AVALANCHEMODIFIERS_API void SetSplineActorWeak(TWeakObjectPtr<AActor> InSplineActorWeak);
	TWeakObjectPtr<AActor> GetSplineActorWeak() const
	{
		return SplineActorWeak;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplineSweep")
	AVALANCHEMODIFIERS_API void SetSampleMode(EAvaSplineSweepSampleMode InMode);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplineSweep")
	EAvaSplineSweepSampleMode GetSampleMode() const
	{
		return SampleMode;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplineSweep")
	AVALANCHEMODIFIERS_API void SetSampleDistance(float InDistance);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplineSweep")
	float GetSampleDistance() const
	{
		return SampleDistance;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplineSweep")
	AVALANCHEMODIFIERS_API void SetSteps(int32 InSteps);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplineSweep")
	int32 GetSteps() const
	{
		return Steps;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplineSweep")
	AVALANCHEMODIFIERS_API void SetProgressOffset(float InOffset);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplineSweep")
	float GetProgressOffset() const
	{
		return ProgressOffset;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplineSweep")
	AVALANCHEMODIFIERS_API void SetProgressStart(float InStart);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplineSweep")
	float GetProgressStart() const
	{
		return ProgressStart;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplineSweep")
	AVALANCHEMODIFIERS_API void SetProgressEnd(float InEnd);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplineSweep")
	float GetProgressEnd() const
	{
		return ProgressEnd;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplineSweep")
	AVALANCHEMODIFIERS_API void SetScaleStart(float InScaleStart);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplineSweep")
	float GetScaleStart() const
	{
		return ScaleStart;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplineSweep")
	AVALANCHEMODIFIERS_API void SetScaleEnd(float InScaleEnd);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplineSweep")
	float GetScaleEnd() const
	{
		return ScaleEnd;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplineSweep")
	AVALANCHEMODIFIERS_API void SetCapped(bool bInCapped);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplineSweep")
	bool GetCapped() const
	{
		return bCapped;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|SplineSweep")
	AVALANCHEMODIFIERS_API void SetLooped(bool bInLooped);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|SplineSweep")
	bool GetLooped() const
	{
		return bLooped;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin IAvaRenderStateUpdateHandler
	virtual void OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent) override;
	virtual void OnActorVisibilityChanged(AActor* InActor) override {}
	//~ End IAvaRenderStateUpdateHandler

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	void OnModifierOptionsChanged();
	void OnSplineActorWeakChanged();
	void OnSplineOptionsChanged();
	bool SampleSpline();

	/** Spline actor to retrieve the USplineComponent from */
	UPROPERTY(EditInstanceOnly, Setter="SetSplineActorWeak", Getter="GetSplineActorWeak", Category="SplineSweep", meta=(FilterActorByComponentClass="/Script/Engine.SplineComponent", AllowPrivateAccess="true"))
	TWeakObjectPtr<AActor> SplineActorWeak;

	/** How do we sample the spline */
	UPROPERTY(EditInstanceOnly, Setter="SetSampleMode", Getter="GetSampleMode", Category="SplineSweep", meta=(AllowPrivateAccess="true"))
	EAvaSplineSweepSampleMode SampleMode = EAvaSplineSweepSampleMode::FullDistance;

	/** Custom sample distance for steps per distance mode */
	UPROPERTY(EditInstanceOnly, Setter="SetSampleDistance", Getter="GetSampleDistance", Category="SplineSweep", meta=(ClampMin="1", EditCondition="SampleMode == EAvaSplineSweepSampleMode::CustomDistance", EditConditionHides, AllowPrivateAccess="true"))
	float SampleDistance = 1000.f;

	/** The sample count defines the precision of the sweep */
	UPROPERTY(EditInstanceOnly, Setter="SetSteps", Getter="GetSteps", Category="SplineSweep", meta=(ClampMin="3", ClampMax="200", AllowPrivateAccess="true"))
	int32 Steps = 64;

	/** Sample range offset of the spline, for closed loop spline range can wrap around */
	UPROPERTY(EditInstanceOnly, Setter="SetProgressOffset", Getter="GetProgressOffset", Category="SplineSweep", meta=(AllowPrivateAccess="true"))
	float ProgressOffset = 0.f;

	/** Sample start range of the spline */
	UPROPERTY(EditInstanceOnly, Setter="SetProgressStart", Getter="GetProgressStart", Category="SplineSweep", meta=(ClampMin="0", ClampMax="1", AllowPrivateAccess="true"))
	float ProgressStart = 0.f;

	/** Sample end range of the spline */
	UPROPERTY(EditInstanceOnly, Setter="SetProgressEnd", Getter="GetProgressEnd", Category="SplineSweep", meta=(ClampMin="0", ClampMax="1", AllowPrivateAccess="true"))
	float ProgressEnd = 1.f;

	/** Start scale of the spline mesh */
	UPROPERTY(EditInstanceOnly, Setter="SetScaleStart", Getter="GetScaleStart", Category="SplineSweep", meta=(ClampMin="0", AllowPrivateAccess="true"))
	float ScaleStart = 1.f;

	/** End scale of the spline mesh */
	UPROPERTY(EditInstanceOnly, Setter="SetScaleEnd", Getter="GetScaleEnd", Category="SplineSweep", meta=(ClampMin="0", AllowPrivateAccess="true"))
	float ScaleEnd = 1.f;

	/** Whether start and end are closed, loop must be false */
	UPROPERTY(EditInstanceOnly, Setter="SetCapped", Getter="GetCapped", Category="SplineSweep", meta=(EditCondition="!bLooped", AllowPrivateAccess="true"))
	bool bCapped = true;

	/** Whether we close the whole spline path, if spline loops this will be true */
	UPROPERTY(EditInstanceOnly, Setter="SetLooped", Getter="GetLooped", Category="SplineSweep", meta=(AllowPrivateAccess="true"))
	bool bLooped = false;

	/** Spline component to sample from */
	UPROPERTY()
	TWeakObjectPtr<USplineComponent> SplineComponentWeak;

	/** Sample points taken from the spline */
	UPROPERTY(Transient)
	TArray<FTransform> SplineSamples;

private:
#if WITH_EDITOR
	/** Used for PECP */
	static const TAvaPropertyChangeDispatcher<UAvaSplineSweepModifier> PropertyChangeDispatcher;
#endif
};
