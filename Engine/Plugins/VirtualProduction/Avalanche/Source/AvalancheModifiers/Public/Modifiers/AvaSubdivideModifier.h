// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaGeometryBaseModifier.h"
#include "AvaSubdivideModifier.generated.h"

UENUM(BlueprintType)
enum class EAvaSubdivisionType : uint8
{
	Selective,
	Uniform,
	PN
};

UCLASS(MinimalAPI, BlueprintType)
class UAvaSubdivideModifier : public UAvaGeometryBaseModifier
{
	GENERATED_BODY()

public:
	static constexpr int MinSubdivideCuts = 1;
    static constexpr int MaxSubdivideCuts = 15;

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Subdivide")
	AVALANCHEMODIFIERS_API void SetCuts(int32 InCuts);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Subdivide")
	int32 GetCuts() const
	{
		return Cuts;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Subdivide")
	AVALANCHEMODIFIERS_API void SetRecomputeNormals(bool bInRecomputeNormals);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Subdivide")
	bool GetRecomputeNormals() const
	{
		return bRecomputeNormals;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Subdivide")
	AVALANCHEMODIFIERS_API void SetType(EAvaSubdivisionType InType);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Subdivide")
	EAvaSubdivisionType GetType() const
	{
		return Type;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	void OnOptionsChanged();

	UPROPERTY(EditInstanceOnly, Setter="SetCuts", Getter="GetCuts", Category="Subdivide", meta=(ClampMin="1", ClampMax="15", AllowPrivateAccess="true"))
	int32 Cuts = 2;

	UPROPERTY(EditInstanceOnly, Setter="SetRecomputeNormals", Getter="GetRecomputeNormals", Category="Subdivide", meta=(EditCondition="Type == EAvaSubdivisionType::PN", EditConditionHides, AllowPrivateAccess="true"))
	bool bRecomputeNormals = true;

	UPROPERTY(EditInstanceOnly, Setter="SetType", Getter="GetType", Category="Subdivide", meta=(AllowPrivateAccess="true"))
	EAvaSubdivisionType Type = EAvaSubdivisionType::Uniform;
};
