// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaGeometryBaseModifier.h"
#include "Tools/AvaTaperTool.h"
#include "AvaTaperModifier.generated.h"

UENUM(BlueprintType)
enum class EAvaTaperReferenceFrame : uint8
{
	MeshCenter,
	Custom
};

UENUM(BlueprintType)
enum class EAvaTaperExtent : uint8
{
	WholeShape,
	Custom
};

UCLASS(MinimalAPI, BlueprintType)
class UAvaTaperModifier : public UAvaGeometryBaseModifier
{
	GENERATED_BODY()

public:
	static constexpr int MinTaperLatticeResolution = 1;
	static constexpr int MaxTaperLatticeResolution = 20;

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Taper")
	AVALANCHEMODIFIERS_API void SetAmount(float InAmount);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Taper")
	float GetAmount() const
	{
		return Amount;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Taper")
	AVALANCHEMODIFIERS_API void SetUpperExtent(float InUpperExtent);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Taper")
	float GetUpperExtent() const
	{
		return UpperExtent;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Taper")
	AVALANCHEMODIFIERS_API void SetLowerExtent(float InLowerExtent);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Taper")
	float GetLowerExtent() const
	{
		return LowerExtent;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Taper")
	AVALANCHEMODIFIERS_API void SetExtent(EAvaTaperExtent InExtent);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Taper")
	EAvaTaperExtent GetExtent() const
	{
		return Extent;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Taper")
	AVALANCHEMODIFIERS_API void SetInterpolationType(EAvaTaperInterpolationType InInterpolationType);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Taper")
	EAvaTaperInterpolationType GetInterpolationType() const
	{
		return InterpolationType;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Taper")
	AVALANCHEMODIFIERS_API void SetReferenceFrame(EAvaTaperReferenceFrame InReferenceFrame);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Taper")
	EAvaTaperReferenceFrame GetReferenceFrame() const
	{
		return ReferenceFrame;
	}

	UE_DEPRECATED(5.7, "Use subdivide modifier before this modifier instead")
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Taper", meta=(DeprecatedFunction, DeprecatedMessage="Use subdivide modifier before this modifier instead"))
	AVALANCHEMODIFIERS_API void SetResolution(int32 InResolution);

	UE_DEPRECATED(5.7, "Use subdivide modifier before this modifier instead")
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Taper", meta=(DeprecatedFunction, DeprecatedMessage="Use subdivide modifier before this modifier instead"))
	int32 GetResolution() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Resolution;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Taper")
	AVALANCHEMODIFIERS_API void SetOffset(FVector2D InOffset);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Taper")
	FVector2D GetOffset() const
	{
		return Offset;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	void CreateTaperTool();

	void OnParameterChanged();

	FVector2D GetRequiredOffset() const;
	FVector2D GetRequiredExtent() const;

	int32 GetSubdividersCuts() const;

	UPROPERTY(EditInstanceOnly, Setter="SetAmount", Getter="GetAmount", Category="Taper", meta=(ClampMin="0.0", ClampMax="1.0", AllowPrivateAccess="true"))
	float Amount = 0.0;

	UPROPERTY(EditInstanceOnly, Setter="SetExtent", Getter="GetExtent", Category="Taper", meta=(AllowPrivateAccess="true"))
	EAvaTaperExtent Extent = EAvaTaperExtent::WholeShape;

	UPROPERTY(EditInstanceOnly, Setter="SetUpperExtent", Getter="GetUpperExtent", Category="Taper", meta=(EditCondition="Extent == EAvaTaperExtent::Custom", EditConditionHides, ClampMin="0", ClampMax="100", Units="Percent", ToolTip="100%: shape top.\n0%: shape bottom.", AllowPrivateAccess="true"))
	float UpperExtent = 100;

	UPROPERTY(EditInstanceOnly, Setter="SetLowerExtent", Getter="GetLowerExtent", Category="Taper", meta=(EditCondition="Extent == EAvaTaperExtent::Custom", EditConditionHides, ClampMin="0", ClampMax="100", Units="Percent", ToolTip="100%: shape bottom.\n0%: shape top.", AllowPrivateAccess="true"))
	float LowerExtent = 100;

	UPROPERTY(EditInstanceOnly, Setter="SetInterpolationType", Getter="GetInterpolationType", Category="Taper", meta=(AllowPrivateAccess="true"))
	EAvaTaperInterpolationType InterpolationType = EAvaTaperInterpolationType::Linear;

	UE_DEPRECATED(5.7, "Use subdivide modifier before this modifier instead")
	UPROPERTY(Transient, meta=(DeprecatedProperty, DeprecatedMessage="Use subdivide modifier before this modifier instead"))
	int32 Resolution = 5;

	UPROPERTY(EditInstanceOnly, Setter="SetReferenceFrame", Getter="GetReferenceFrame", Category="Taper", meta=(AllowPrivateAccess="true"))
	EAvaTaperReferenceFrame ReferenceFrame = EAvaTaperReferenceFrame::MeshCenter;

	UPROPERTY(EditInstanceOnly, Setter="SetOffset", Getter="GetOffset", Category="Taper", meta=(EditCondition="ReferenceFrame == EAvaTaperReferenceFrame::Custom", EditConditionHides, AllowPrivateAccess="true"))
	FVector2D Offset = FVector2D::ZeroVector;

	UPROPERTY()
	TObjectPtr<UAvaTaperTool> TaperTool = nullptr;
};
