// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/ActorModifierArrangeBaseModifier.h"
#include "Shared/AvaTranslucentPriorityModifierShared.h"
#include "UObject/WeakObjectPtrTemplatesFwd.h"
#include "AvaTranslucentPriorityModifier.generated.h"

class ACameraActor;
class UPrimitiveComponent;

UENUM(BlueprintType)
enum class EAvaTranslucentPriorityModifierMode : uint8
{
	/** The closer you are from the camera based on camera forward axis, the higher your sort priority will be */
	AutoCameraDistance          UMETA(DisplayName = "Camera Distance"),
	/** The higher you are in the outline tree, the higher your sort priority will be */
	AutoOutlinerTree            UMETA(DisplayName = "Outliner Top First"),
	/** The lower you are in the outline tree, the higher your sort priority will be */
	AutoOutlinerTreeInverted    UMETA(DisplayName = "Outliner Bottom First"),
	/** Set it yourself */
	Manual
};

UCLASS(MinimalAPI, BlueprintType)
class UAvaTranslucentPriorityModifier : public UActorModifierArrangeBaseModifier
{
	GENERATED_BODY()

	friend class UAvaTranslucentPriorityModifierShared;

public:
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|TranslucentSortPriority")
    AVALANCHEMODIFIERS_API void SetMode(EAvaTranslucentPriorityModifierMode InMode);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|TranslucentSortPriority")
    EAvaTranslucentPriorityModifierMode GetMode() const
    {
	    return Mode;
    }

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|TranslucentSortPriority")
	AVALANCHEMODIFIERS_API void SetCameraActor(ACameraActor* InCameraActor);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|TranslucentSortPriority")
	ACameraActor* GetCameraActor() const
	{
		return CameraActorWeak.Get();
	}

    AVALANCHEMODIFIERS_API void SetCameraActorWeak(const TWeakObjectPtr<ACameraActor>& InCameraActor);

	TWeakObjectPtr<ACameraActor> GetCameraActorWeak() const
	{
		return CameraActorWeak;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|TranslucentSortPriority")
    AVALANCHEMODIFIERS_API void SetSortPriority(int32 InSortPriority);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|TranslucentSortPriority")
    int32 GetSortPriority() const
    {
	    return SortPriority;
    }

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|TranslucentSortPriority")
	AVALANCHEMODIFIERS_API void SetSortPriorityOffset(int32 InOffset);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|TranslucentSortPriority")
	int32 GetSortPriorityOffset() const
	{
		return SortPriorityOffset;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|TranslucentSortPriority")
	AVALANCHEMODIFIERS_API void SetSortPriorityStep(int32 InStep);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|TranslucentSortPriority")
	int32 GetSortPriorityStep() const
	{
		return SortPriorityStep;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|TranslucentSortPriority")
	AVALANCHEMODIFIERS_API void SetIncludeChildren(bool bInIncludeChildren);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|TranslucentSortPriority")
	bool GetIncludeChildren() const
	{
		return bIncludeChildren;
	}

protected:
	//~ Begin UObject
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifierAdded(EActorModifierCoreEnableReason InReason) override;
	virtual void OnModifierRemoved(EActorModifierCoreDisableReason InReason) override;
	virtual void SavePreState() override;
	virtual void RestorePreState() override;
	virtual void Apply() override;
	virtual void OnModifierDisabled(EActorModifierCoreDisableReason InReason) override;
	virtual void OnModifiedActorTransformed() override;
	//~ End UActorModifierCoreBase

	//~ Begin IAvaSceneTreeUpdateModifierExtension
	virtual void OnSceneTreeTrackedActorChildrenChanged(int32 InIdx, const TSet<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TSet<TWeakObjectPtr<AActor>>& InNewChildrenActors) override;
	virtual void OnSceneTreeTrackedActorDirectChildrenChanged(int32 InIdx, const TArray<TWeakObjectPtr<AActor>>& InPreviousChildrenActors, const TArray<TWeakObjectPtr<AActor>>& InNewChildrenActors) override;
	virtual void OnSceneTreeTrackedActorRearranged(int32 InIdx, AActor* InRearrangedActor) override;
	//~ End IAvaSceneTreeUpdateModifierExtension

	//~ Begin IAvaRenderStateUpdateExtension
	virtual UE::ActorModifierCore::ERenderStateDirtyReason GetRenderStateDirtyEventTypes() const override;
	virtual void OnRenderStateUpdated(AActor* InActor, UActorComponent* InComponent) override;
	//~ End IAvaRenderStateUpdateExtension

	//~ Begin IAvaTransformUpdateExtension
	virtual void OnTransformUpdated(AActor* InActor, bool bInParentMoved) override;
	//~ End IAvaTransformUpdateExtension

	void OnModeChanged();
	void OnCameraActorChanged();
	void OnSortPriorityChanged();
	void OnSortPriorityLevelGlobalsChanged() const;
	void OnIncludeChildrenChanged();

	void OnGlobalSortPriorityOffsetChanged();

	ACameraActor* GetDefaultCameraActor() const;

	/** The sort mode we are currently in */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="TranslucentSortPriority", meta=(DisplayName="Sort Mode", AllowPrivateAccess="true"))
	EAvaTranslucentPriorityModifierMode Mode = EAvaTranslucentPriorityModifierMode::Manual;

	/** The camera actor to compute the distance from */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="TranslucentSortPriority", meta=(DisplayName="Camera Actor", EditCondition="Mode == EAvaTranslucentPriorityModifierMode::AutoCameraDistance", AllowPrivateAccess="true"))
	TWeakObjectPtr<ACameraActor> CameraActorWeak = nullptr;

	/** The sort priority that will be set on the primitive component for manual mode */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="TranslucentSortPriority", meta=(EditCondition="Mode == EAvaTranslucentPriorityModifierMode::Manual", EditConditionHides, AllowPrivateAccess="true"))
	int32 SortPriority = 0;

	/** Sort priority offset shared across all modifiers in this same level */
	UPROPERTY(Transient, EditInstanceOnly, Setter, Getter, Category="TranslucentSortPriority", meta=(AllowPrivateAccess="true"))
	int32 SortPriorityOffset = 0;

	/** Sort priority incremental step shared across all modifiers in this same level */
	UPROPERTY(Transient, EditInstanceOnly, Setter, Getter, Category="TranslucentSortPriority", meta=(AllowPrivateAccess="true"))
	int32 SortPriorityStep = 1;

	/** If true, will include children too and update their sort priority */
	UPROPERTY(EditInstanceOnly, Setter="SetIncludeChildren", Getter="GetIncludeChildren", Category="TranslucentSortPriority", meta=(AllowPrivateAccess="true"))
	bool bIncludeChildren = true;

	/** The components this modifier is managing */
	UPROPERTY()
	TSet<TWeakObjectPtr<UPrimitiveComponent>> PrimitiveComponentsWeak;

private:
	/** The previous sort priority to restore when disabling this modifier */
	UPROPERTY()
	TMap<TWeakObjectPtr<UPrimitiveComponent>, int32> PreviousSortPriorities;

	/** Last primitive components assigned sort priority, used for comparison on change */
	TMap<FObjectKey, int32> LastSortPriorities;

	/** Used to avoid querying again the full list of component states */
	TArray<FAvaTranslucentPriorityModifierComponentState> CachedSortedComponentStates;
};
