// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/ActorModifierArrangeBaseModifier.h"
#include "AvaVisibilityModifier.generated.h"

class AActor;

/**
 * Controls the visibility of a range of child actors by index.
 */
UCLASS(MinimalAPI, BlueprintType)
class UAvaVisibilityModifier : public UActorModifierArrangeBaseModifier
{
	GENERATED_BODY()

public:
	/** Sets the child index range to hide instead of showing. */
	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Visibility")
	AVALANCHEMODIFIERS_API void SetInvertVisibility(bool bInInvertVisibility);

	/** Returns true if hiding the child index range instead of showing. */
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Visibility")
	bool GetInvertVisibility() const
	{
		return bInvertVisibility;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Visibility")
	AVALANCHEMODIFIERS_API void SetIndex(int32 InIndex);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Visibility")
	int32 GetIndex() const
	{
		return Index;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Visibility")
	AVALANCHEMODIFIERS_API void SetTreatAsRange(bool bInTreatAsRange);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Visibility")
	bool GetTreatAsRange() const
	{
		return bTreatAsRange;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Visibility")
	AVALANCHEMODIFIERS_API void SetSkipWhenHidden(bool bInSkip);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Visibility")
	bool GetSkipWhenHidden() const
	{
		return bSkipWhenHidden;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UActorModifierCoreBase
	virtual void OnModifierCDOSetup(FActorModifierCoreMetadata& InMetadata) override;
	virtual void OnModifiedActorTransformed() override;
	virtual void Apply() override;
	//~ End UActorModifierCoreBase

	//~ Begin IAvaRenderStateUpdateExtension
	virtual void OnActorVisibilityChanged(AActor* InActor) override;
	//~ End IAvaRenderStateUpdateExtension

	/** Used by other modifiers of this class to check if we are hiding an actor */
	bool IsChildActorHidden(AActor* InActor) const;

	/** Get first visibility modifier found above this actor */
	UAvaVisibilityModifier* GetFirstModifierAbove(AActor* InActor);
	TArray<UAvaVisibilityModifier*> GetModifiersAbove(AActor* InActor);

	/** Will find the direct children of InParentActor that contains this child actor */
	AActor* GetDirectChildren(AActor* InParentActor, AActor* InChildActor);

	/** Child index to set visibility, visible if bInvertVisibility is false else hidden */
	UPROPERTY(EditInstanceOnly, Setter="SetIndex", Getter="GetIndex", Category="Visibility", meta=(ClampMin="0", UIMin="0", AllowPrivateAccess="true"))
	int32 Index = 0;

	/** Treat index as a range from 0 to index */
	UPROPERTY(EditInstanceOnly, Setter="SetTreatAsRange", Getter="GetTreatAsRange", Category="Visibility", meta=(DisplayName="Keep Visible", AllowPrivateAccess="true"))
	bool bTreatAsRange = false;

	/** If true, will hide the child index range instead of showing. */
	UPROPERTY(EditInstanceOnly, Setter="SetInvertVisibility", Getter="GetInvertVisibility", Category="Visibility", meta=(AllowPrivateAccess="true"))
	bool bInvertVisibility = false;

	/** If true, when the top actor having this modifier is hidden, children actor visibility state will not be handled */
	UPROPERTY(EditInstanceOnly, Setter="SetSkipWhenHidden", Getter="GetSkipWhenHidden", Category="Visibility", meta=(AllowPrivateAccess="true"))
	bool bSkipWhenHidden = true;

private:
	/** Visibility defined for direct children of this actor, used by other modifiers of this type to check */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TMap<TWeakObjectPtr<AActor>, bool> DirectChildrenActorsWeak;
};
