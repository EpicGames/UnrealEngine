// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Profilers/AvaGeometryModifierProfiler.h"

struct FAvaBooleanModifierSharedChannelInfo;

/** Modifier profiler used by boolean modifiers */
class FAvaBooleanModifierProfiler : public FAvaGeometryModifierProfiler
{
public:
	static inline const FName ChannelInfo = TEXT("ChannelInfo");

	//~ Begin FActorModifierCoreProfiler
	virtual void SetupProfilingStats() override;
	virtual void BeginProfiling() override;
	virtual void EndProfiling() override;
	virtual TSet<FName> GetMainProfilingStats() const override;
	//~ End FActorModifierCoreProfiler

	AVALANCHEMODIFIERS_API FAvaBooleanModifierSharedChannelInfo GetChannelInfo() const;
};