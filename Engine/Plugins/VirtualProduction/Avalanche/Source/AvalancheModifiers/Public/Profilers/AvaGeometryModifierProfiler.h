// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/Profiler/ActorModifierCoreProfiler.h"

/** Modifier profiler used by geometry modifiers */
class FAvaGeometryModifierProfiler : public FActorModifierCoreProfiler
{
public:
	static inline const FName VertexInName = TEXT("VertexIn");
	static inline const FName VertexOutName = TEXT("VertexOut");
	static inline const FName TriangleInName = TEXT("TriIn");
	static inline const FName TriangleOutName = TEXT("TriOut");
	static inline const FName EdgeInName = TEXT("EdgeIn");
	static inline const FName EdgeOutName = TEXT("EdgeOut");

	//~ Begin FActorModifierCoreProfiler
	virtual void SetupProfilingStats() override;
	virtual void BeginProfiling() override;
	virtual void EndProfiling() override;
	virtual TSet<FName> GetMainProfilingStats() const override;
	//~ End FActorModifierCoreProfiler

	AVALANCHEMODIFIERS_API int32 GetVertexIn() const;
	AVALANCHEMODIFIERS_API int32 GetVertexOut() const;
	AVALANCHEMODIFIERS_API int32 GetTriangleIn() const;
	AVALANCHEMODIFIERS_API int32 GetTriangleOut() const;
	AVALANCHEMODIFIERS_API int32 GetEdgeIn() const;
	AVALANCHEMODIFIERS_API int32 GetEdgeOut() const;
};
