// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "Modifiers/ActorModifierCoreSharedObject.h"
#include "AvaBooleanModifierShared.generated.h"

class UAvaBooleanModifier;
enum class EAvaBooleanMode : uint8;

USTRUCT()
struct FAvaBooleanModifierSharedChannel
{
	GENERATED_BODY()

	UPROPERTY()
	TSet<TWeakObjectPtr<UAvaBooleanModifier>> ModifiersWeak;
};

USTRUCT(BlueprintType)
struct FAvaBooleanModifierSharedChannelInfo
{
	GENERATED_BODY()

	/** The number of channel currently active */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Channel")
	int32 ChannelCount = 0;

	/** The number of modifier on that channel, the more there are, the more impact on performance */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Channel")
	int32 ChannelModifierCount = 0;

	/** The number of modifier on that channel that are used to mask other non mask modifier */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Channel")
	int32 ChannelToolCount = 0;

	/** The number of modifier on that channel that are masked by other mask modifier */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Channel")
	int32 ChannelTargetCount = 0;

	/** The number of modifier intersecting with this modifier */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Channel")
	int32 ChannelIntersectCount = 0;
};

/**
 * Singleton class for boolean modifiers to share data
 * Transient because we can rebuild it, no need to save the asset
 */
UCLASS(Transient, Hidden)
class UAvaBooleanModifierShared : public UActorModifierCoreSharedObject
{
	GENERATED_BODY()

public:
	/** Track a modifier on a channel */
	void TrackModifierChannel(UAvaBooleanModifier* InModifier);

	/** Untrack a modifier from a channel */
	void UntrackModifierChannel(UAvaBooleanModifier* InModifier);

	/** Updates and refresh modifier channel */
	void UpdateModifierChannel(UAvaBooleanModifier* InModifier);

	/** Get number of active channel */
	uint8 GetChannelCount() const;

	/** Get number of modifier active on that channel */
	int32 GetChannelModifierCount(uint8 InChannel) const;

	/** Get number of modifier active on that channel with that mode set */
	int32 GetChannelModifierModeCount(uint8 InChannel, EAvaBooleanMode InMode) const;

	/** Get all intersecting modifiers with the input target modifier */
	TSet<TWeakObjectPtr<UAvaBooleanModifier>> GetIntersectingModifiers(const UAvaBooleanModifier* InTargetModifier, FAvaBooleanModifierSharedChannelInfo* OutDesc = nullptr);

private:
	/** Tests if two meshes are intersecting, always pass the tool/mask mesh first */
	bool TestIntersection(const UE::Geometry::FDynamicMesh3& InToolMesh, const FTransform& InToolTransform, const UE::Geometry::FDynamicMesh3& InTargetMesh, const FTransform& InTargetTransform) const;

	UPROPERTY(Transient)
	TMap<uint8, FAvaBooleanModifierSharedChannel> Channels;
};
