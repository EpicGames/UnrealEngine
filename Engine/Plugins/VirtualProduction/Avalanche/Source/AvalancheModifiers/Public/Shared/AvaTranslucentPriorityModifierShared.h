// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PrimitiveComponent.h"
#include "Containers/Set.h"
#include "Modifiers/ActorModifierCoreSharedObject.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "AvaTranslucentPriorityModifierShared.generated.h"

class AActor;
class ACameraActor;
class UAvaTranslucentPriorityModifier;

USTRUCT()
struct FAvaTranslucentPriorityModifierComponentState
{
	GENERATED_BODY()

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAvaTranslucentPriorityModifierComponentState() = default;
	explicit FAvaTranslucentPriorityModifierComponentState(UPrimitiveComponent* InComponent)
		: PrimitiveComponent(InComponent)
	{
	}
	FAvaTranslucentPriorityModifierComponentState(const FAvaTranslucentPriorityModifierComponentState&) = default;
	FAvaTranslucentPriorityModifierComponentState(FAvaTranslucentPriorityModifierComponentState&&) = default;
	FAvaTranslucentPriorityModifierComponentState& operator=(const FAvaTranslucentPriorityModifierComponentState&) = default;
	FAvaTranslucentPriorityModifierComponentState& operator=(FAvaTranslucentPriorityModifierComponentState&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Save this component state if valid */
	void Save();

	/** Restore this component state if valid */
	void Restore() const;

	/** Get component owning actor */
	AActor* GetOwningActor() const;

	/** Get component world location */
	FVector GetComponentLocation() const;

	friend uint32 GetTypeHash(const FAvaTranslucentPriorityModifierComponentState& InItem)
	{
		return GetTypeHash(InItem.PrimitiveComponent);
	}

	bool operator==(const FAvaTranslucentPriorityModifierComponentState& InOther) const
	{
		return PrimitiveComponent == InOther.PrimitiveComponent;
	}

	UE_DEPRECATED(5.8,"Use Modifier instead")
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use Modifier instead"))
	TWeakObjectPtr<UAvaTranslucentPriorityModifier> ModifierWeak;

	UE_DEPRECATED(5.8,"Use PrimitiveComponent instead")
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use PrimitiveComponent instead"))
	TWeakObjectPtr<UPrimitiveComponent> PrimitiveComponentWeak;

	/** Modifier managing this component state */
	UPROPERTY()
	TObjectPtr<UAvaTranslucentPriorityModifier> Modifier;

	/** Primitive component that this state is describing */
	UPROPERTY()
	TObjectPtr<UPrimitiveComponent> PrimitiveComponent;

	/** Previous sort priority to restore */
	UPROPERTY()
	int32 SortPriority = 0;
};

/**
 * Singleton class for translucent priority modifiers to share data about component state
 */
UCLASS(Hidden)
class UAvaTranslucentPriorityModifierShared : public UActorModifierCoreSharedObject
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE(FOnLevelGlobalsChanged)
	FOnLevelGlobalsChanged OnLevelGlobalsChangedDelegate;

	/** Replaces the component linked to this context by new components set */
	void SetComponentsState(UAvaTranslucentPriorityModifier* InModifierContext, const TSet<TWeakObjectPtr<UPrimitiveComponent>>& InComponents);

	/** Save component state, adds it if it is not tracked, optionally override the context */
	void SaveComponentState(UAvaTranslucentPriorityModifier* InModifierContext, UPrimitiveComponent* InComponent, bool bInOverrideContext);

	/** Restore component state, removes it if no other modifier track that state */
	void RestoreComponentState(const UAvaTranslucentPriorityModifier* InModifierContext, UPrimitiveComponent* InComponent, bool bInClearState);

	/** Restore components state linked to this modifier */
	void RestoreComponentsState(const UAvaTranslucentPriorityModifier* InModifierContext, bool bInClearState);

	/** Restore components state linked to this modifier */
	void RestoreComponentsState(const UAvaTranslucentPriorityModifier* InModifierContext, const TSet<TWeakObjectPtr<UPrimitiveComponent>>& InComponents, bool bInClearState);

	/** Find the modifier linked to a component */
	UAvaTranslucentPriorityModifier* FindModifierContext(UPrimitiveComponent* InComponent) const;

	/** Get sorted components state based on modifier context */
	TArray<FAvaTranslucentPriorityModifierComponentState> GetSortedComponentStates(UAvaTranslucentPriorityModifier* InModifierContext) const;

	void SetSortPriorityOffset(int32 InOffset);

	int32 GetSortPriorityOffset() const
	{
		return SortPriorityOffset;
	}

	void SetSortPriorityStep(int32 InStep);

	int32 GetSortPriorityStep() const
	{
		return SortPriorityStep;
	}

private:
	//~ Begin UObject
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	//~ End UObject

	UPROPERTY()
	TSet<FAvaTranslucentPriorityModifierComponentState> ComponentStates;

	/** Offset for the whole level */
	UPROPERTY()
	int32 SortPriorityOffset = 0;

	/** Incremental step for the whole level */
	UPROPERTY()
	int32 SortPriorityStep = 1;
};
