// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaPatternModifierTool.h"
#include "Modifiers/AvaPatternModifier.h"
#include "AvaPatternModifierCircleTool.generated.h"

/** Circle tool for the pattern modifier */
UCLASS(DisplayName="Circle", ClassGroup="Modifiers")
class UAvaPatternModifierCircleTool : public UAvaPatternModifierTool
{
	GENERATED_BODY()

	friend class UAvaPatternModifier;

public:
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	EAvaPatternModifierPlane GetCirclePlane() const
	{
		return CirclePlane;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetCirclePlane(EAvaPatternModifierPlane InCirclePlane);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	float GetCircleRadius() const
	{
		return CircleRadius;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetCircleRadius(float InCircleRadius);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	float GetCircleStartAngle() const
	{
		return CircleStartAngle;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetCircleStartAngle(float InCircleStartAngle);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	float GetCircleFullAngle() const
	{
		return CircleFullAngle;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetCircleFullAngle(float InCircleFullAngle);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	int32 GetCircleCount() const
	{
		return CircleCount;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetCircleCount(int32 InCircleCount);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	bool GetCircleAccumulateTransform() const
	{
		return bCircleAccumulateTransform;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetCircleAccumulateTransform(bool bInCircleAccumulateTransform);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	FRotator GetCircleRotation() const
	{
		return CircleRotation;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetCircleRotation(const FRotator& InCircleRotation);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	FVector GetCircleScale() const
	{
		return CircleScale;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetCircleScale(const FVector& InCircleScale);

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InEvent) override;
#endif
	//~ End UObject

	//~ Begin UAvaPatternModifierTool
	virtual TArray<FTransform> GetTransformInstances(const FBox& InOriginalBounds) const;
	virtual FVector GetCenterAlignmentAxis() const override;
	virtual FName GetToolName() const override;
	//~ End UAvaPatternModifierTool

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern")
	EAvaPatternModifierPlane CirclePlane = EAvaPatternModifierPlane::YZ;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern")
	float CircleRadius = 100.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern")
	float CircleStartAngle = 180.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern")
	float CircleFullAngle = 360.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern", meta=(ClampMin="1", ClampMax="10000"))
	int32 CircleCount = 4;

	UPROPERTY(EditInstanceOnly, Setter="SetCircleAccumulateTransform", Getter="GetCircleAccumulateTransform", Category="Pattern")
	bool bCircleAccumulateTransform = false;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern")
	FRotator CircleRotation = FRotator::ZeroRotator;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern", meta=(MotionDesignVectorWidget, AllowPreserveRatio="XYZ"))
	FVector CircleScale = FVector::OneVector;
};
