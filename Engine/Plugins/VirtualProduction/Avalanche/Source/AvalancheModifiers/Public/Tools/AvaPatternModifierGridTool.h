// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaPatternModifierTool.h"
#include "Modifiers/AvaPatternModifier.h"
#include "AvaPatternModifierGridTool.generated.h"

/** Grid tool for the pattern modifier */
UCLASS(DisplayName="Grid", ClassGroup="Modifiers")
class UAvaPatternModifierGridTool : public UAvaPatternModifierTool
{
	GENERATED_BODY()

	friend class UAvaPatternModifier;

public:
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	EAvaPatternModifierPlane GetGridPlane() const
	{
		return GridPlane;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetGridPlane(EAvaPatternModifierPlane InGridPlane);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	EAvaPatternModifierGridAlignment GetGridAlignment() const
	{
		return GridAlignment;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetGridAlignment(EAvaPatternModifierGridAlignment InGridAlignment);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	int32 GetGridCountX() const
	{
		return GridCountX;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetGridCountX(int32 InGridCountX);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	int32 GetGridCountY() const
	{
		return GridCountY;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetGridCountY(int32 InGridCountY);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	float GetGridSpacingX() const
	{
		return GridSpacingX;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetGridSpacingX(float InGridSpacingX);
	
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	float GetGridSpacingY() const
	{
		return GridSpacingY;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetGridSpacingY(float InGridSpacingY);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	bool GetGridAccumulateTransform() const
	{
		return bGridAccumulateTransform;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetGridAccumulateTransform(bool bInGridAccumulateTransform);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	FRotator GetGridRotation() const
	{
		return GridRotation;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetGridRotation(const FRotator& InGridRotation);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	FVector GetGridScale() const
	{
		return GridScale;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetGridScale(const FVector& InGridScale);

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InEvent) override;
#endif
	//~ End UObject

	//~ Begin UAvaPatternModifierTool
	virtual TArray<FTransform> GetTransformInstances(const FBox& InOriginalBounds) const;
	virtual FVector GetCenterAlignmentAxis() const override;
	virtual FName GetToolName() const override;
	//~ End UAvaPatternModifierTool

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern")
	EAvaPatternModifierPlane GridPlane = EAvaPatternModifierPlane::YZ;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern")
	EAvaPatternModifierGridAlignment GridAlignment = EAvaPatternModifierGridAlignment::Center;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern", meta=(ClampMin="1", ClampMax="10000"))
	int32 GridCountX = 2;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern", meta=(ClampMin="1", ClampMax="10000"))
	int32 GridCountY = 2;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern")
	float GridSpacingX = 0.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern")
	float GridSpacingY = 0.f;

	UPROPERTY(EditInstanceOnly, Setter="SetGridAccumulateTransform", Getter="GetGridAccumulateTransform", Category="Pattern")
	bool bGridAccumulateTransform = false;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern")
	FRotator GridRotation = FRotator::ZeroRotator;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern", meta=(MotionDesignVectorWidget, AllowPreserveRatio="XYZ"))
	FVector GridScale = FVector::OneVector;
};