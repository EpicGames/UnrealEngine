// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaPatternModifierTool.h"
#include "Modifiers/AvaPatternModifier.h"
#include "AvaPatternModifierLineTool.generated.h"

/** Line tool for the pattern modifier */
UCLASS(DisplayName="Line", ClassGroup="Modifiers")
class UAvaPatternModifierLineTool : public UAvaPatternModifierTool
{
	GENERATED_BODY()

	friend class UAvaPatternModifier;

public:
	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	EAvaPatternModifierAxis GetLineAxis() const
	{
		return LineAxis;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetLineAxis(EAvaPatternModifierAxis InLineAxis);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	EAvaPatternModifierLineAlignment GetLineAlignment() const
	{
		return LineAlignment;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetLineAlignment(EAvaPatternModifierLineAlignment InLineAlignment);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	int32 GetLineCount() const
	{
		return LineCount;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetLineCount(int32 InLineCount);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	float GetLineSpacing() const
	{
		return LineSpacing;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetLineSpacing(float InLineSpacing);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	bool GetLineAccumulateTransform() const
	{
		return bLineAccumulateTransform;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetLineAccumulateTransform(bool bInLineAccumulateTransform);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	FRotator GetLineRotation() const
	{
		return LineRotation;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetLineRotation(const FRotator& InLineRotation);

	UFUNCTION(BlueprintPure, Category="Motion Design|Modifiers|Pattern")
	FVector GetLineScale() const
	{
		return LineScale;
	}

	UFUNCTION(BlueprintCallable, Category="Motion Design|Modifiers|Pattern")
	void SetLineScale(const FVector& InLineScale);

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InEvent) override;
#endif
	//~ End UObject

	//~ Begin UAvaPatternModifierTool
	virtual TArray<FTransform> GetTransformInstances(const FBox& InOriginalBounds) const;
	virtual FVector GetCenterAlignmentAxis() const override;
	virtual FName GetToolName() const override;
	//~ End UAvaPatternModifierTool

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern")
	EAvaPatternModifierAxis LineAxis = EAvaPatternModifierAxis::Y;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern")
	EAvaPatternModifierLineAlignment LineAlignment = EAvaPatternModifierLineAlignment::Center;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern", meta=(ClampMin="1", ClampMax="10000"))
	int32 LineCount = 4;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern")
	float LineSpacing = 0.f;

	UPROPERTY(EditInstanceOnly, Setter="SetLineAccumulateTransform", Getter="GetLineAccumulateTransform", Category="Pattern")
	bool bLineAccumulateTransform = false;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern")
	FRotator LineRotation = FRotator::ZeroRotator;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Pattern", meta=(MotionDesignVectorWidget, AllowPreserveRatio="XYZ"))
	FVector LineScale = FVector::OneVector;
};
