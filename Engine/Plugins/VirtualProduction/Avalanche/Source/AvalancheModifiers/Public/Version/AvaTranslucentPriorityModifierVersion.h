// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/Guid.h"

#define UE_API AVALANCHEMODIFIERS_API

namespace UE::AvaModifiers
{

/** Describes the versions of the translucent priority modifier */
struct FTranslucentPriorityModifierVersion
{
private:
	FTranslucentPriorityModifierVersion() = delete;

public:
	enum Type : uint8
	{
		PreVersioning = 0,

		/** Upgraded TWeakObjectPtr to TObjectPtr in UAvaTranslucentPriorityModifierShared */
		UpgradeFromWeak,

		/* ------------------------------------------------------ */
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	UE_API const static FGuid Guid;
};

} // UE::AvaModifiers

#undef UE_API
