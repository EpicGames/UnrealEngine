// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaModifiersEditorModule.h"
#include "AvaModifiersEditorStyle.h"
#include "DragDropOps/AvaOutlinerItemDragDropOp.h"
#include "IAvaOutliner.h"
#include "IAvaOutlinerModule.h"
#include "Item/AvaOutlinerActor.h"
#include "ItemProxies/AvaOutlinerItemProxyRegistry.h"
#include "Modules/ModuleManager.h"
#include "Outliner/AvaOutlinerModifierContextMenu.h"
#include "Outliner/AvaOutlinerModifierDropHandler.h"
#include "Outliner/AvaOutlinerModifierProxy.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "AvaModifiersEditorModule"

void FAvaModifiersEditorModule::StartupModule()
{
	// Initialize it here, otherwise never used elsewhere
	FAvaModifiersEditorStyle::Get();

	RegisterOutlinerItems();
}

void FAvaModifiersEditorModule::ShutdownModule()
{
    UnregisterOutlinerItems();
}

void FAvaModifiersEditorModule::RegisterOutlinerItems()
{
	FAvaOutlinerItemProxyRegistry& ItemProxyRegistry = IAvaOutlinerModule::Get().GetItemProxyRegistry();

	ItemProxyRegistry.RegisterItemProxyWithDefaultFactory<FAvaOutlinerModifierProxy, 60>();

	OutlinerProxiesExtensionDelegateHandle = IAvaOutlinerModule::Get().GetOnExtendItemProxiesForItem().AddLambda(
		[](IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InItem, TArray<TSharedPtr<FAvaOutlinerItemProxy>>& OutItemProxies)
		{
			if (InItem->IsA<FAvaOutlinerActor>())
			{
				if (TSharedPtr<FAvaOutlinerItemProxy> ModifierProxy = InOutliner.GetOrCreateItemProxy<FAvaOutlinerModifierProxy>(InItem))
				{
					OutItemProxies.Add(ModifierProxy);
				}
			}
		});

	OutlinerContextDelegateHandle = IAvaOutlinerModule::Get().GetOnExtendOutlinerItemContextMenu()
		.AddStatic(&FAvaOutlinerModifierContextMenu::OnExtendOutlinerContextMenu);

	OutlinerDropHandlerDelegateHandle = FAvaOutlinerItemDragDropOp::OnItemDragDropOpInitialized().AddStatic(
		[](FAvaOutlinerItemDragDropOp& InDragDropOp)
		{
			InDragDropOp.AddDropHandler<FAvaOutlinerModifierDropHandler>();
		});
}

void FAvaModifiersEditorModule::UnregisterOutlinerItems()
{
	if (IAvaOutlinerModule::IsLoaded())
	{
		IAvaOutlinerModule& OutlinerModule = IAvaOutlinerModule::Get();

		FAvaOutlinerItemProxyRegistry& ItemProxyRegistry = OutlinerModule.GetItemProxyRegistry();
		ItemProxyRegistry.UnregisterItemProxyFactory<FAvaOutlinerModifierProxy>();

		OutlinerModule.GetOnExtendItemProxiesForItem().Remove(OutlinerProxiesExtensionDelegateHandle);
		OutlinerProxiesExtensionDelegateHandle.Reset();

		OutlinerModule.GetOnExtendOutlinerItemContextMenu().Remove(OutlinerContextDelegateHandle);
		OutlinerContextDelegateHandle.Reset();

	}

	FAvaOutlinerItemDragDropOp::OnItemDragDropOpInitialized().Remove(OutlinerDropHandlerDelegateHandle);
	OutlinerDropHandlerDelegateHandle.Reset();
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FAvaModifiersEditorModule, AvalancheModifiersEditor)
