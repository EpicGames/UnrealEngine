// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaModifiersEditorStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/IPluginManager.h"
#include "Math/MathFwd.h"
#include "Modifiers/ActorModifierCoreBase.h"
#include "Styles/ActorModifierEditorStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"

FAvaModifiersEditorStyle::FAvaModifiersEditorStyle()
	: FSlateStyleSet(UE_MODULE_NAME)
{
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	SetContentRoot(Plugin->GetBaseDir() / TEXT("Resources"));

	if (const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get())
	{
		// loop through already registered factories if subsystem exists
		ModifierSubsystem->ForEachModifierMetadata([this](const FActorModifierCoreMetadata& InMetadata)->bool
		{
			OnModifierClassRegistered(InMetadata);
			return true;
		});
	}

	FSlateStyleRegistry::RegisterSlateStyle(*this);

	UActorModifierCoreSubsystem::OnModifierClassRegistered().AddRaw(this, &FAvaModifiersEditorStyle::OnModifierClassRegistered);
}

void FAvaModifiersEditorStyle::OnModifierClassRegistered(const FActorModifierCoreMetadata& InMetadata)
{
	// We need to update the metadata to set various options
	FActorModifierCoreMetadata& MutableMetadata = const_cast<FActorModifierCoreMetadata&>(InMetadata);

	const FVector2f Icon16x16(16.f, 16.f);
	const FString ModifierIconPath = TEXT("Icons/ModifierIcons/") + InMetadata.GetName().ToString() + TEXT("Modifier");
	
	MutableMetadata.SetColor(FActorModifierEditorStyle::GetModifierCategoryColor(InMetadata.GetCategory()).GetSpecifiedColor());

	if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*RootToContentDir(ModifierIconPath, TEXT(".svg"))))
	{
		const FName ModifierClassStyleName(TEXT("ClassIcon.") + InMetadata.GetClass()->GetName());
		Set(ModifierClassStyleName, new IMAGE_BRUSH_SVG(ModifierIconPath, Icon16x16));
		MutableMetadata.SetIcon(FSlateIcon(GetStyleSetName(), ModifierClassStyleName));
	}
}

FAvaModifiersEditorStyle::~FAvaModifiersEditorStyle()
{
	UActorModifierCoreSubsystem::OnModifierClassRegistered().RemoveAll(this);

	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
