// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Styling/SlateStyle.h"

struct FActorModifierCoreMetadata;

class FAvaModifiersEditorStyle final : public FSlateStyleSet
{
public:
	static FAvaModifiersEditorStyle& Get()
	{
		static FAvaModifiersEditorStyle Instance;
		return Instance;
	}

	FAvaModifiersEditorStyle();
	virtual ~FAvaModifiersEditorStyle() override;

private:
	void OnModifierClassRegistered(const FActorModifierCoreMetadata& InMetadata);
};
