// Copyright Epic Games, Inc. All Rights Reserved.

#include "Outliner/AvaOutlinerModifier.h"
#include "Modifiers/ActorModifierCoreBase.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Selection/AvaOutlinerScopedSelection.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/ActorModifierCoreEditorSubsystem.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"

FAvaOutlinerModifier::FAvaOutlinerModifier(IAvaOutliner& InOutliner, UActorModifierCoreBase* InModifier)
	: FAvaOutlinerObject(InOutliner, InModifier)
	, Modifier(InModifier)
{
	ModifierName = FText::FromName(InModifier->GetModifierName());
	ModifierIcon = FSlateIconFinder::FindIconForClass(UActorModifierCoreBase::StaticClass());
	ModifierTooltip = FText::GetEmpty();

	if (const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get())
	{
		ModifierSubsystem->ProcessModifierMetadata(InModifier->GetModifierName(), [this](const FActorModifierCoreMetadata& InMetadata)->bool
		{
			ModifierName = InMetadata.GetDisplayName();
			ModifierIcon = InMetadata.GetIcon();
			ModifierTooltip = InMetadata.GetDescription();
			return true;
		});
	}
}

void FAvaOutlinerModifier::Select(FAvaOutlinerScopedSelection& InSelection) const
{
	const UActorModifierCoreBase* const UnderlyingModifier = GetModifier();
	
	if (!UnderlyingModifier)
	{
		return;
	}

	UActorModifierCoreStack* const RootModifierStack = UnderlyingModifier->GetRootModifierStack();
	const AActor* const ActorModified = UnderlyingModifier->GetModifiedActor();

	if (!InSelection.IsSelected(ActorModified))
	{
		InSelection.Select(RootModifierStack);
	}
}

FText FAvaOutlinerModifier::GetDisplayName() const
{
	return ModifierName;
}

FText FAvaOutlinerModifier::GetIconTooltipText() const
{
	return ModifierTooltip;
}

FSlateIcon FAvaOutlinerModifier::GetIcon() const
{
	return ModifierIcon;
}

bool FAvaOutlinerModifier::ShowVisibility(EAvaOutlinerVisibilityType InVisibilityType) const
{
	return InVisibilityType == EAvaOutlinerVisibilityType::Runtime;
}

bool FAvaOutlinerModifier::GetVisibility(EAvaOutlinerVisibilityType InVisibilityType) const
{
	return InVisibilityType == EAvaOutlinerVisibilityType::Runtime
		&& Modifier.IsValid()
		&& Modifier->IsModifierEnabled();
}

void FAvaOutlinerModifier::OnVisibilityChanged(EAvaOutlinerVisibilityType InVisibilityType, bool bInNewVisibility)
{
	if (InVisibilityType == EAvaOutlinerVisibilityType::Runtime && Modifier.IsValid())
	{
		if (const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get())
		{
			ModifierSubsystem->EnableModifiers({Modifier.Get()}, bInNewVisibility, true);
		}
	}
}

bool FAvaOutlinerModifier::CanDelete() const
{
	return Modifier.IsValid();
}

bool FAvaOutlinerModifier::Delete()
{
	if (const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get())
	{
		FText FailReason;
		FActorModifierCoreStackRemoveOp RemoveOp;
		RemoveOp.FailReason = &FailReason;
		RemoveOp.bShouldTransact = false;

		return ModifierSubsystem->RemoveModifiers({Modifier.Get()}, RemoveOp);
	}

	return false;
}

void FAvaOutlinerModifier::SetObject_Impl(UObject* InObject)
{
	FAvaOutlinerObject::SetObject_Impl(InObject);
	Modifier = Cast<UActorModifierCoreBase>(InObject);
}
