// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Item/AvaOutlinerObject.h"
#include "Textures/SlateIcon.h"
#include "UObject/WeakObjectPtr.h"

class UActorModifierCoreBase;

/** Item representing a Modifier Item */
class AVALANCHEMODIFIERSEDITOR_API FAvaOutlinerModifier : public FAvaOutlinerObject
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerModifier, FAvaOutlinerObject);

	FAvaOutlinerModifier(IAvaOutliner& InOutliner, UActorModifierCoreBase* InObject);

	UActorModifierCoreBase* GetModifier() const { return Modifier.Get(); }

	//~ Begin IAvaOutlinerItem
	virtual void Select(FAvaOutlinerScopedSelection& InSelection) const override;
	virtual FText GetDisplayName() const override;
	virtual FText GetIconTooltipText() const override;
	virtual FSlateIcon GetIcon() const override;
	virtual bool ShowVisibility(EAvaOutlinerVisibilityType InVisibilityType) const override;
	virtual bool GetVisibility(EAvaOutlinerVisibilityType InVisibilityType) const override;
	virtual void OnVisibilityChanged(EAvaOutlinerVisibilityType InVisibilityType, bool bInNewVisibility) override;
	virtual bool CanDelete() const override;
	virtual bool Delete() override;
	//~ End IAvaOutlinerItem

protected:
	//~ Begin FAvaOutlinerObjectItem
	virtual void SetObject_Impl(UObject* InObject) override;
	//~ End FAvaOutlinerObjectItem

	TWeakObjectPtr<UActorModifierCoreBase> Modifier;

	FText ModifierName;
	FSlateIcon ModifierIcon;
	FText ModifierTooltip;
};
