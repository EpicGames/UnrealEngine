// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerModifierContextMenu.h"
#include "GameFramework/Actor.h"
#include "Item/AvaOutlinerActor.h"
#include "Outliner/AvaOutlinerModifier.h"
#include "Outliner/AvaOutlinerModifierProxy.h"
#include "Subsystems/ActorModifierCoreEditorSubsystem.h"
#include "ToolMenuContext/AvaOutlinerItemsContext.h"
#include "ToolMenus.h"
#include "Modifiers/ActorModifierCoreStack.h"

void FAvaOutlinerModifierContextMenu::OnExtendOutlinerContextMenu(UToolMenu* InToolMenu)
{
	if (!InToolMenu)
	{
		return;
	}
	
	UAvaOutlinerItemsContext* OutlinerItemsContext = InToolMenu->Context.FindContext<UAvaOutlinerItemsContext>();

	if (!IsValid(OutlinerItemsContext))
	{
		return;
	}

	const UActorModifierCoreEditorSubsystem* ExtensionSubsystem = UActorModifierCoreEditorSubsystem::Get();

	if (!IsValid(ExtensionSubsystem))
	{
		return;
	}
	
	TSet<TWeakObjectPtr<UObject>> ContextObjects;
	GetContextObjects(OutlinerItemsContext, ContextObjects);
	const FActorModifierCoreEditorMenuContext MenuContext(ContextObjects);

	static const FActorModifierCoreEditorMenuOptions AddMenuOptions(EActorModifierCoreEditorMenuType::Add);
	ExtensionSubsystem->FillModifierMenu(InToolMenu, MenuContext, AddMenuOptions);
	
	static const FActorModifierCoreEditorMenuOptions DeleteMenuOptions(EActorModifierCoreEditorMenuType::Delete);
	ExtensionSubsystem->FillModifierMenu(InToolMenu, MenuContext, DeleteMenuOptions);
	
	static const FActorModifierCoreEditorMenuOptions MoveMenuOptions(EActorModifierCoreEditorMenuType::Move);
	ExtensionSubsystem->FillModifierMenu(InToolMenu, MenuContext, MoveMenuOptions);
	
	static const FActorModifierCoreEditorMenuOptions InsertAfterMenuOptions(EActorModifierCoreEditorMenuType::InsertAfter);
	ExtensionSubsystem->FillModifierMenu(InToolMenu, MenuContext, InsertAfterMenuOptions);
	
	static const FActorModifierCoreEditorMenuOptions InsertBeforeMenuOptions(EActorModifierCoreEditorMenuType::InsertBefore);
	ExtensionSubsystem->FillModifierMenu(InToolMenu, MenuContext, InsertBeforeMenuOptions);
	
	static const FActorModifierCoreEditorMenuOptions DisableMenuOptions(EActorModifierCoreEditorMenuType::Disable);
	ExtensionSubsystem->FillModifierMenu(InToolMenu, MenuContext, DisableMenuOptions);
	
	static const FActorModifierCoreEditorMenuOptions EnableMenuOptions(EActorModifierCoreEditorMenuType::Enable);
	ExtensionSubsystem->FillModifierMenu(InToolMenu, MenuContext, EnableMenuOptions);
}

void FAvaOutlinerModifierContextMenu::GetContextObjects(const UAvaOutlinerItemsContext* ItemsContext, TSet<TWeakObjectPtr<UObject>>& OutObjects)
{
	OutObjects.Empty();
	
	if (!IsValid(ItemsContext) || ItemsContext->GetItems().IsEmpty())
	{
		return;
	}
	
	for (FAvaOutlinerItemWeakPtr ItemWeak : ItemsContext->GetItems())
	{
		const TSharedPtr<IAvaOutlinerItem> Item = ItemWeak.Pin();

		// is it an actor
		if (const FAvaOutlinerActor* ActorItem = Item->CastTo<FAvaOutlinerActor>())
		{
			if (AActor* Actor = ActorItem->GetActor())
			{
				OutObjects.Add(Actor);
			}
		}
		// is it a stack
		else if (const FAvaOutlinerModifierProxy* ModifierProxy = Item->CastTo<FAvaOutlinerModifierProxy>())
		{
			if (const UActorModifierCoreStack* Stack = ModifierProxy->GetModifierStack())
			{
				OutObjects.Add(Stack->GetModifiedActor());
			}
		}
		// is it a modifier
		else if (const FAvaOutlinerModifier* ModifierItem = Item->CastTo<FAvaOutlinerModifier>())
		{
			if (UActorModifierCoreBase* Modifier = ModifierItem->GetModifier())
			{
				OutObjects.Add(Modifier);
			}
		}
	}
}
