// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerModifierDropHandler.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Item/AvaOutlinerActor.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Outliner/AvaOutlinerModifier.h"
#include "Outliner/AvaOutlinerModifierProxy.h"
#include "Subsystems/ActorModifierCoreEditorSubsystem.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerModifierDropHandler"

DEFINE_LOG_CATEGORY_STATIC(LogAvaOutlinerModifierDropHandler, Log, All);

bool FAvaOutlinerModifierDropHandler::IsDraggedItemSupported(const FAvaOutlinerItemPtr& InDraggedItem) const
{
	return InDraggedItem->IsA<FAvaOutlinerModifier>() || InDraggedItem->IsA<FAvaOutlinerModifierProxy>();
}

TOptional<EItemDropZone> FAvaOutlinerModifierDropHandler::CanDrop(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem) const
{
	// Dropping on actor directly
	if (const FAvaOutlinerActor* const TargetActorItem = InTargetItem->CastTo<FAvaOutlinerActor>())
	{
		return CanDropOnActor(TargetActorItem->GetActor(), InDropZone);
	}

	// Dropping onto the Modifier Proxy itself has the same Effects as Dropping on Actor
	if (const FAvaOutlinerModifierProxy* const TargetModifierProxy = InTargetItem->CastTo<FAvaOutlinerModifierProxy>())
	{
		if (const UActorModifierCoreStack* const ModifierStack = TargetModifierProxy->GetModifierStack())
		{
			return CanDropOnActor(ModifierStack->GetModifiedActor(), InDropZone);
		}
		return TOptional<EItemDropZone>();
	}

	// If Target Item is none of the above, nor a Modifier Item, then it's not a supported Target
	const FAvaOutlinerModifier* const TargetModifierItem = InTargetItem->CastTo<FAvaOutlinerModifier>();
	if (!TargetModifierItem || !TargetModifierItem->GetModifier())
	{
		return TOptional<EItemDropZone>();
	}

	UActorModifierCoreBase* TargetModifier = TargetModifierItem->GetModifier();
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	const EActorModifierCoreStackPosition Position = InDropZone == EItemDropZone::AboveItem ? EActorModifierCoreStackPosition::Before : EActorModifierCoreStackPosition::After;

	TArray<UActorModifierCoreBase*> MoveModifiers;
	TArray<UActorModifierCoreBase*> CloneModifiers;
	ModifierSubsystem->GetSortedModifiers(GetDraggedModifiers(), TargetModifier->GetModifiedActor(), TargetModifier, Position, MoveModifiers, CloneModifiers);

	// If the Modifiers are Empty, return fail early
	if (MoveModifiers.IsEmpty() && CloneModifiers.IsEmpty())
	{
		return TOptional<EItemDropZone>();
	}

	return InDropZone;
}

bool FAvaOutlinerModifierDropHandler::Drop(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem)
{
	if (const FAvaOutlinerActor* const TargetActorItem = InTargetItem->CastTo<FAvaOutlinerActor>())
	{
		return DropModifiersInActor(TargetActorItem->GetActor(), InDropZone);
	}

	// Dropping onto the Modifier Proxy itself has the same Effects as Dropping on Actor
	if (const FAvaOutlinerModifierProxy* const TargetModifierProxy = InTargetItem->CastTo<FAvaOutlinerModifierProxy>())
	{
		if (const UActorModifierCoreStack* const ModifierStack = TargetModifierProxy->GetModifierStack())
		{
			return DropModifiersInActor(ModifierStack->GetModifiedActor(), InDropZone);
		}
		return false;
	}

	// If Target Item is none of the above, nor a Modifier Item, then it's not a supported Target
	const FAvaOutlinerModifier* const TargetModifierItem = InTargetItem->CastTo<FAvaOutlinerModifier>();
	if (!TargetModifierItem || !TargetModifierItem->GetModifier())
	{
		return false;
	}

	UActorModifierCoreBase* TargetModifier = TargetModifierItem->GetModifier();
	return DropModifiersInModifier(TargetModifier, InDropZone);
}

TSet<UActorModifierCoreBase*> FAvaOutlinerModifierDropHandler::GetDraggedModifiers() const
{
	TSet<UActorModifierCoreBase*> OutDraggedModifiers;

	// Rather than iterating separately, keep order of how they were dragged
	ForEachItem<IAvaOutlinerItem>([&OutDraggedModifiers](IAvaOutlinerItem& InItem)->EIterationResult
	{
		if (const FAvaOutlinerModifierProxy* const ModifierItemProxy = InItem.CastTo<FAvaOutlinerModifierProxy>())
		{
			if (const UActorModifierCoreStack* const ModifierStack = ModifierItemProxy->GetModifierStack())
			{
				OutDraggedModifiers.Append(ModifierStack->GetModifiers());
			}
		}
		else if (const FAvaOutlinerModifier* const ModifierItem = InItem.CastTo<FAvaOutlinerModifier>())
		{
			if (UActorModifierCoreBase* const Modifier = ModifierItem->GetModifier())
			{
				OutDraggedModifiers.Add(Modifier);
			}
		}

		return EIterationResult::Continue;
	});

	return OutDraggedModifiers;
}

TOptional<EItemDropZone> FAvaOutlinerModifierDropHandler::CanDropOnActor(AActor* InActor, EItemDropZone InDropZone) const
{
	if (!IsValid(InActor))
	{
		return TOptional<EItemDropZone>();
	}

	const UActorModifierCoreSubsystem* const ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	if (!IsValid(ModifierSubsystem))
	{
		return TOptional<EItemDropZone>();
	}

	const TSet<FName> AllowedModifiers = ModifierSubsystem->GetAllowedModifiers(InActor);
	const TSet<UActorModifierCoreBase*> DraggedModifiers = GetDraggedModifiers();

	// Support if there's at least one Modifier that is Allowed
	bool bHasAllowedModifiers = false;
	for (const UActorModifierCoreBase* Modifier : DraggedModifiers)
	{
		if (AllowedModifiers.Contains(Modifier->GetModifierName()))
		{
			bHasAllowedModifiers = true;
			break;
		}
	}

	// For Actor Items, Drop Zone can only be Onto the Actor
	return bHasAllowedModifiers
		? EItemDropZone::OntoItem
		: TOptional<EItemDropZone>();
}

bool FAvaOutlinerModifierDropHandler::DropModifiersInActor(AActor* InActor, EItemDropZone InDropZone) const
{
	const UActorModifierCoreSubsystem* const ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!IsValid(ModifierSubsystem))
	{
		return false;
	}

	const EActorModifierCoreStackPosition Position = InDropZone == EItemDropZone::AboveItem ? EActorModifierCoreStackPosition::Before : EActorModifierCoreStackPosition::After;

	TArray<UActorModifierCoreBase*> MoveModifiers;
	TArray<UActorModifierCoreBase*> CloneModifiers;
	ModifierSubsystem->GetSortedModifiers(GetDraggedModifiers(), InActor, nullptr, Position, MoveModifiers, CloneModifiers);

	if (CloneModifiers.IsEmpty())
	{
		return false;
	}

	UE_LOGF(LogAvaOutlinerModifierDropHandler, Log, "Dropping %i modifier(s) on actor %ls (Clone)"
		, CloneModifiers.Num()
		, *InActor->GetActorNameOrLabel());

	bool bSuccess = false;

	if (UActorModifierCoreStack* const Stack = ModifierSubsystem->AddActorModifierStack(InActor))
	{
		FText FailReason;
		FActorModifierCoreStackCloneOp CloneOp;
		CloneOp.bShouldTransact = true;
		CloneOp.FailReason = &FailReason;

		bSuccess = ModifierSubsystem->CloneModifiers(CloneModifiers, Stack, CloneOp).Num() == CloneModifiers.Num();

		// When ALT is not pressed : perform a move operation between actors (copy + delete)
		// When ALT is pressed : perform a clone operation between actors
		if (bSuccess && !FSlateApplication::Get().GetModifierKeys().IsAltDown())
		{
			FActorModifierCoreStackRemoveOp RemoveOp;
			RemoveOp.bShouldTransact = true;
			RemoveOp.FailReason = &FailReason;

			bSuccess = ModifierSubsystem->RemoveModifiers(TSet<UActorModifierCoreBase*>(CloneModifiers), RemoveOp);
		}

		if (!FailReason.IsEmpty())
		{
			UE_LOGF(LogAvaOutlinerModifierDropHandler, Warning, "Dropping %i modifier(s) on actor %ls (Clone) failed with reason : %ls",
				CloneModifiers.Num(),
				*InActor->GetActorNameOrLabel(),
				*FailReason.ToString());

			FNotificationInfo NotificationInfo(FailReason);
			NotificationInfo.ExpireDuration = 3.f;
			NotificationInfo.bFireAndForget = true;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
	}

	return bSuccess;
}

bool FAvaOutlinerModifierDropHandler::DropModifiersInModifier(UActorModifierCoreBase* InTargetModifier, EItemDropZone InDropZone) const
{
	const UActorModifierCoreSubsystem* const ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	const UActorModifierCoreStack* const TargetStack = InTargetModifier->GetModifierStack();

	if (!IsValid(ModifierSubsystem) || !IsValid(TargetStack))
	{
		return false;
	}

	AActor* const TargetActor = InTargetModifier->GetModifiedActor();
	const EActorModifierCoreStackPosition Position = InDropZone == EItemDropZone::AboveItem ? EActorModifierCoreStackPosition::Before : EActorModifierCoreStackPosition::After;

	TArray<UActorModifierCoreBase*> MoveModifiers;
	TArray<UActorModifierCoreBase*> CloneModifiers;
	ModifierSubsystem->GetSortedModifiers(GetDraggedModifiers(), TargetActor, InTargetModifier, Position, MoveModifiers, CloneModifiers);

	bool bSuccess = false;
	FText FailReason;

	if (!MoveModifiers.IsEmpty())
	{
		UE_LOGF(LogAvaOutlinerModifierDropHandler, Log, "Dropping %i modifier(s) %ls modifier %ls on actor %ls (Move)"
		, MoveModifiers.Num()
		, Position == EActorModifierCoreStackPosition::After ? TEXT("after") : TEXT("before")
		, *InTargetModifier->GetModifierName().ToString()
		, *TargetActor->GetActorNameOrLabel());

		FActorModifierCoreStackMoveOp MoveOp;
		MoveOp.bShouldTransact = true;
		MoveOp.FailReason = &FailReason;
		MoveOp.MovePosition = Position;
		MoveOp.MovePositionContext = InTargetModifier;

		bSuccess = ModifierSubsystem->MoveModifiers(MoveModifiers, InTargetModifier->GetModifierStack(), MoveOp);
	}
	else if (!CloneModifiers.IsEmpty())
	{
		UE_LOGF(LogAvaOutlinerModifierDropHandler, Log, "Dropping %i modifier(s) %ls modifier %ls on actor %ls (Clone)"
		, CloneModifiers.Num()
		, Position == EActorModifierCoreStackPosition::After ? TEXT("after") : TEXT("before")
		, *InTargetModifier->GetModifierName().ToString()
		, *TargetActor->GetActorNameOrLabel());

		FActorModifierCoreStackCloneOp CloneOp;
		CloneOp.bShouldTransact = true;
		CloneOp.FailReason = &FailReason;
		CloneOp.ClonePosition = Position;
		CloneOp.ClonePositionContext = InTargetModifier;

		bSuccess = ModifierSubsystem->CloneModifiers(CloneModifiers, InTargetModifier->GetModifierStack(), CloneOp).Num() == CloneModifiers.Num();

		// When ALT is not pressed : perform a move operation between actors (copy + delete)
		// When ALT is pressed : perform a clone operation between actors
		if (bSuccess && !FSlateApplication::Get().GetModifierKeys().IsAltDown())
		{
			FActorModifierCoreStackRemoveOp RemoveOp;
			RemoveOp.bShouldTransact = true;
			RemoveOp.FailReason = &FailReason;

			bSuccess = ModifierSubsystem->RemoveModifiers(TSet<UActorModifierCoreBase*>(CloneModifiers), RemoveOp);
		}
	}

	if (!FailReason.IsEmpty())
	{
		UE_LOGF(LogAvaOutlinerModifierDropHandler, Warning, "Dropping %i modifier(s) %ls modifier %ls on actor %ls (%ls) failed with reason : %ls"
			, MoveModifiers.IsEmpty() ? CloneModifiers.Num() : MoveModifiers.Num()
			, Position == EActorModifierCoreStackPosition::After ? TEXT("after") : TEXT("before")
			, *InTargetModifier->GetModifierName().ToString()
			, *TargetActor->GetActorNameOrLabel()
			, MoveModifiers.IsEmpty() ? TEXT("Clone") : TEXT("Move")
			, *FailReason.ToString());

		FNotificationInfo NotificationInfo(FailReason);
		NotificationInfo.ExpireDuration = 3.f;
		NotificationInfo.bFireAndForget = true;
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);
	}

	return bSuccess;
}

#undef LOCTEXT_NAMESPACE
