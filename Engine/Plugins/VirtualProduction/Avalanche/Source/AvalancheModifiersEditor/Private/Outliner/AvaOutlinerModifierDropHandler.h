// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragDropOps/Handlers/AvaOutlinerItemDropHandler.h"

class UActorModifierCoreBase;
class UActorModifierCoreStack;
class UActorModifierCoreSubsystem;

/** Class that handles dropping modifiers item into a target item */
class FAvaOutlinerModifierDropHandler : public FAvaOutlinerItemDropHandler
{
public:
	UE_AVA_INHERITS(FAvaOutlinerModifierDropHandler, FAvaOutlinerItemDropHandler);

protected:
	//~ Begin FAvaOutlinerItemDropHandler
	virtual bool IsDraggedItemSupported(const FAvaOutlinerItemPtr& InDraggedItem) const override;
	virtual TOptional<EItemDropZone> CanDrop(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem) const override;
	virtual bool Drop(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem) override;
	//~ End FAvaOutlinerItemDropHandler

	/** Get raw dragged modifiers set */
	TSet<UActorModifierCoreBase*> GetDraggedModifiers() const;
	
	TOptional<EItemDropZone> CanDropOnActor(AActor* InActor, EItemDropZone InDropZone) const;
	bool DropModifiersInActor(AActor* InActor, EItemDropZone InDropZone) const;
	bool DropModifiersInModifier(UActorModifierCoreBase* InTargetModifier, EItemDropZone InDropZone) const;
};
