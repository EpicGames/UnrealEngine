// Copyright Epic Games, Inc. All Rights Reserved.

#include "Outliner/AvaOutlinerModifierProxy.h"
#include "IAvaOutliner.h"
#include "Item/AvaOutlinerActor.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Outliner/AvaOutlinerModifier.h"
#include "Selection/AvaOutlinerScopedSelection.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerModifierProxy"

FAvaOutlinerModifierProxy::FAvaOutlinerModifierProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem)
	: Super(InOutliner, InParentItem)
{
	ModifierIcon = FSlateIconFinder::FindIconForClass(UActorModifierCoreBase::StaticClass());
}

UActorModifierCoreStack* FAvaOutlinerModifierProxy::GetModifierStack() const
{
	const AActor* Actor = GetActor();

	if (!Actor)
	{
		return nullptr;
	}

	const UActorModifierCoreSubsystem* const ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (!ModifierSubsystem)
	{
		return nullptr;
	}

	return ModifierSubsystem->GetActorModifierStack(Actor);
}

AActor* FAvaOutlinerModifierProxy::GetActor() const
{
	const FAvaOutlinerItemPtr Parent = GetParent();

	if (!Parent.IsValid())
	{
		return nullptr;
	}

	if (const FAvaOutlinerActor* const ActorItem = Parent->CastTo<FAvaOutlinerActor>())
	{
		return ActorItem->GetActor();
	}

	return nullptr;
}

void FAvaOutlinerModifierProxy::OnItemRegistered()
{
	Super::OnItemRegistered();
	BindDelegates();
}

void FAvaOutlinerModifierProxy::OnItemUnregistered()
{
	Super::OnItemUnregistered();
	UnbindDelegates();
}

void FAvaOutlinerModifierProxy::Select(FAvaOutlinerScopedSelection& InSelection) const
{
	if (UActorModifierCoreStack* const ModifierStack = GetModifierStack())
	{
		InSelection.Select(ModifierStack);
	}
}

FText FAvaOutlinerModifierProxy::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Modifiers");
}

FSlateIcon FAvaOutlinerModifierProxy::GetIcon() const
{
	return ModifierIcon;
}

FText FAvaOutlinerModifierProxy::GetIconTooltipText() const
{
	return LOCTEXT("Tooltip", "Shows all the Modifiers found in the Root Stack of an Actor");
}

bool FAvaOutlinerModifierProxy::CanDelete() const
{
	return GetModifierStack() != nullptr;
}

bool FAvaOutlinerModifierProxy::Delete()
{
	UActorModifierCoreStack* ModifierStack = GetModifierStack();
	const UActorModifierCoreSubsystem* ModifierSubsystem = UActorModifierCoreSubsystem::Get();

	if (ModifierStack && ModifierSubsystem)
	{
		FText FailReason;
		FActorModifierCoreStackRemoveOp RemoveOp;
		RemoveOp.FailReason = &FailReason;
		RemoveOp.bShouldTransact = false;

		return ModifierSubsystem->RemoveModifiers(TSet<UActorModifierCoreBase*>{ModifierStack->GetModifiers()}, RemoveOp);
	}

	return false;
}

void FAvaOutlinerModifierProxy::GetProxiedItems(const TSharedRef<IAvaOutlinerItem>& InParent, TArray<FAvaOutlinerItemPtr>& OutChildren, bool bInRecursive)
{
	if (const UActorModifierCoreStack* const ModifierStack = GetModifierStack())
	{
		for (UActorModifierCoreBase* const Modifier : ModifierStack->GetModifiers())
		{
			const FAvaOutlinerItemPtr ModifierItem = Outliner.FindOrAdd<FAvaOutlinerModifier>(Modifier);
			ModifierItem->SetParent(SharedThis(this));

			OutChildren.Add(ModifierItem);

			if (bInRecursive)
			{
				ModifierItem->FindChildren(OutChildren, bInRecursive);
			}
		}
	}
}

void FAvaOutlinerModifierProxy::BindDelegates()
{
	UnbindDelegates();
	UActorModifierCoreStack::OnModifierAdded().AddSP(this, &FAvaOutlinerModifierProxy::OnModifierAdded);
	UActorModifierCoreStack::OnModifierRemoved().AddSP(this, &FAvaOutlinerModifierProxy::OnModifierRemoved);
	UActorModifierCoreStack::OnModifierMoved().AddSP(this, &FAvaOutlinerModifierProxy::OnModifierUpdated);
	UActorModifierCoreStack::OnModifierReplaced().AddSP(this, &FAvaOutlinerModifierProxy::OnModifierUpdated);
}

void FAvaOutlinerModifierProxy::UnbindDelegates()
{
	UActorModifierCoreStack::OnModifierAdded().RemoveAll(this);
	UActorModifierCoreStack::OnModifierRemoved().RemoveAll(this);
	UActorModifierCoreStack::OnModifierMoved().RemoveAll(this);
	UActorModifierCoreStack::OnModifierReplaced().RemoveAll(this);
}

void FAvaOutlinerModifierProxy::OnModifierAdded(UActorModifierCoreBase* InItemChanged, EActorModifierCoreEnableReason InReason)
{
	OnModifierUpdated(InItemChanged);
}

void FAvaOutlinerModifierProxy::OnModifierRemoved(UActorModifierCoreBase* InItemChanged, EActorModifierCoreDisableReason InReason)
{
	OnModifierUpdated(InItemChanged);
}

void FAvaOutlinerModifierProxy::OnModifierUpdated(UActorModifierCoreBase* ItemChanged)
{
	const UActorModifierCoreStack* ThisStack = GetModifierStack();
	const AActor* ThisActor = GetActor();

	if (IsValid(ItemChanged) && (IsValid(ThisStack) || IsValid(ThisActor)))
	{
		const UActorModifierCoreStack* UpdatedStack = ItemChanged->GetModifierStack();

		if (UpdatedStack == ThisStack || ItemChanged->GetModifiedActor() == ThisActor)
		{
			RefreshChildren();
			Outliner.RequestRefresh();
		}
	}
}

#undef LOCTEXT_NAMESPACE
