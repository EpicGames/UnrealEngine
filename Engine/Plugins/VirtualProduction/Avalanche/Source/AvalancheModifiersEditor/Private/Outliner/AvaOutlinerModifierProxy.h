// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Item/AvaOutlinerItemProxy.h"
#include "Textures/SlateIcon.h"

class AActor;
class UActorModifierCoreBase;
class UActorModifierCoreStack;
enum class EActorModifierCoreDisableReason : uint8;
enum class EActorModifierCoreEnableReason : uint8;

/** Creates Modifier Items based on all the Modifiers found in the Root Stack of an Actor */
class AVALANCHEMODIFIERSEDITOR_API FAvaOutlinerModifierProxy : public FAvaOutlinerItemProxy
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerModifierProxy, FAvaOutlinerItemProxy);

	FAvaOutlinerModifierProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem);

	/** Gets the Modifier Stack to use (e.g. for an Actor it would be the Root Modifier Stack) */
	UActorModifierCoreStack* GetModifierStack() const;
	AActor* GetActor() const;

	//~ Begin IAvaOutlinerItem
	virtual void OnItemRegistered() override;
	virtual void OnItemUnregistered() override;
	virtual void Select(FAvaOutlinerScopedSelection& InSelection) const override;
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	virtual FText GetIconTooltipText() const override;
	virtual bool CanDelete() const override;
	virtual bool Delete() override;
	//~ End IAvaOutlinerItem

	//~ Begin FAvaOutlinerItemProxy
	virtual void GetProxiedItems(const TSharedRef<IAvaOutlinerItem>& InParent, TArray<FAvaOutlinerItemPtr>& OutChildren, bool bInRecursive) override;
	//~ End FAvaOutlinerItemProxy

protected:
	void BindDelegates();
	void UnbindDelegates();

	void OnModifierAdded(UActorModifierCoreBase* InItemChanged, EActorModifierCoreEnableReason InReason);
	void OnModifierRemoved(UActorModifierCoreBase* InItemChanged, EActorModifierCoreDisableReason InReason);
	void OnModifierUpdated(UActorModifierCoreBase* ItemChanged);

	FSlateIcon ModifierIcon;
};
