// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheOutliner : ModuleRules
{
	public AvalancheOutliner(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AvalancheCore",
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"Slate",
				"SlateCore",
				"UnrealEd",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AvalancheEditorCore",
				"AvalancheSceneTree",
				"BlueprintGraph",
				"EditorWidgets",
				"InputCore",
				"Projects",
				"ToolMenus",
				"ToolWidgets",
				"TypedElementRuntime",
			}
		);
	}
}
