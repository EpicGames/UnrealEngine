// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutliner.h"
#include "Algo/Reverse.h"
#include "AvaOutlinerSettings.h"
#include "AvaOutlinerSubsystem.h"
#include "AvaOutlinerView.h"
#include "AvaSceneTree.h"
#include "BlueprintEditorSettings.h"
#include "Clipboard/AvaOutlinerExporter.h"
#include "Clipboard/AvaOutlinerImporter.h"
#include "Components/ActorComponent.h"
#include "Data/AvaOutlinerSaveState.h"
#include "Data/AvaOutlinerVersion.h"
#include "Editor/Transactor.h"
#include "EditorModeManager.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "GameFramework/PhysicsVolume.h"
#include "GameFramework/WorldSettings.h"
#include "IAvaOutlinerProvider.h"
#include "Item/AvaOutlinerActor.h"
#include "Item/AvaOutlinerComponent.h"
#include "Item/AvaOutlinerItemUtils.h"
#include "Item/AvaOutlinerLevel.h"
#include "Item/AvaOutlinerMaterial.h"
#include "Item/AvaOutlinerTreeRoot.h"
#include "ItemActions/AvaOutlinerAddItem.h"
#include "ItemActions/AvaOutlinerItemAction.h"
#include "ItemActions/AvaOutlinerRemoveItem.h"
#include "ItemProxies/AvaOutlinerItemProxyRegistry.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "LevelInstance/LevelInstanceComponent.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Particles/ParticleEventManager.h"
#include "ScopedTransaction.h"
#include "Selection/AvaEditorSelection.h"
#include "Selection/AvaOutlinerScopedSelection.h"
#include "Textures/SlateIcon.h"
#include "UObject/PropertyIterator.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "AvaOutliner"

FAvaOutliner::FAvaOutliner(IAvaOutlinerProvider& InOutlinerProvider)
	: OutlinerProvider(InOutlinerProvider)
	, RootItem(MakeShared<FAvaOutlinerTreeRoot>(*this))
	, SaveState(MakeShared<FAvaOutlinerSaveState>())
{
	if (GEngine)
	{
		GEngine->OnLevelActorAdded().AddRaw(this, &FAvaOutliner::OnActorSpawned);
		GEngine->OnLevelActorDeleted().AddRaw(this, &FAvaOutliner::OnActorDestroyed);
		GEngine->OnLevelActorAttached().AddRaw(this, &FAvaOutliner::OnActorAttachmentChanged, true);
		GEngine->OnLevelActorDetached().AddRaw(this, &FAvaOutliner::OnActorAttachmentChanged, false);
	}

	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}

	//Listen to Object Replacement Changes
	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FAvaOutliner::OnObjectsReplaced);
	FEditorDelegates::OnEditorActorReplaced.AddRaw(this, &FAvaOutliner::OnActorReplaced);
}

FAvaOutliner::~FAvaOutliner()
{
	if (GEngine)
	{
		GEngine->OnLevelActorAdded().RemoveAll(this);
		GEngine->OnLevelActorDeleted().RemoveAll(this);
		GEngine->OnLevelActorAttached().RemoveAll(this);
		GEngine->OnLevelActorDetached().RemoveAll(this);
	}

	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}

	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	FEditorDelegates::OnEditorActorReplaced.RemoveAll(this);

	UnbindWorldDelegates();
}

UAvaOutlinerSubsystem* FAvaOutliner::GetOutlinerSubsystem() const
{
	if (UWorld* const World = GetWorld())
	{
		return World->GetSubsystem<UAvaOutlinerSubsystem>();
	}
	return nullptr;
}

bool FAvaOutliner::IsActorAllowedInOutliner(const AActor* InActor) const
{
	if (!InActor)
	{
		return false;
	}

	const UWorld* const ActorWorld = InActor->GetWorld();
	if (!ActorWorld)
	{
		return false;
	}

	const UWorld* const OutlinerWorld = GetWorld();
	if (!IsValid(OutlinerWorld))
	{
		return false;
	}

	// Note: RF_Transient is no longer checked (to hide transient actors).
	// It could be re-added with Outliner parameters to show/hide Transient actors (as seen in FActorMode::IsActorDisplayable)
	// However, in SLevelEditor::CreateSceneOutliner bShowTransient is set to true.

	// Do not show Actors that aren't editable or not meant to be listed in Outliner
	if (!InActor->IsEditable() || !InActor->IsListedInSceneOutliner())
	{
		return false;
	}

	// Only consider Actors that are in the Outliner World
	if (ActorWorld != OutlinerWorld)
	{
		return false;
	}

	// Check if actor is in a streaming level
	for (ULevelStreaming* const LevelStreaming : OutlinerWorld->GetStreamingLevels())
	{
		if (const TSoftObjectPtr<UWorld>& WorldAsset = LevelStreaming->GetWorldAsset())
		{
			if (WorldAsset && WorldAsset->PersistentLevel
				&& WorldAsset->PersistentLevel->Actors.Contains(InActor))
			{
				return !IsDefaultWorldActorToHide(WorldAsset.Get(), InActor);
			}
		}
	}

	// Make sure the Actor is none of these Default World Actors
	return !IsDefaultWorldActorToHide(ActorWorld, InActor);
}

bool FAvaOutliner::IsDefaultWorldActorToHide(const UWorld* const InWorld, const AActor* const InActor) const
{
	if (!IsValid(InWorld) || !IsValid(InActor))
	{
		return false;
	}

	// Make sure the Actor is none of these Default World Actors
	return (InWorld->HasDefaultPhysicsVolume() && InActor == InWorld->GetDefaultPhysicsVolume())
		|| InActor == InWorld->GetDefaultBrush()
		|| InActor == InWorld->GetWorldSettings()
		|| InActor == InWorld->MyParticleEventManager;
}

bool FAvaOutliner::IsComponentAllowedInOutliner(const USceneComponent* InComponent) const
{
	if (InComponent)
	{
		const bool bHideConstructionScriptComponents = GetDefault<UBlueprintEditorSettings>()->bHideConstructionScriptComponentsInDetailsView;
		const USceneComponent* const AttachParent = InComponent->GetAttachParent();

		return !InComponent->IsVisualizationComponent()
			&& (InComponent->CreationMethod != EComponentCreationMethod::UserConstructionScript || !bHideConstructionScriptComponents)
			// NOTE: This prevents Native Comps (e.g. Static Mesh Actor's Comps) that are Attached to a BP Actor from appearing.
			// For now, allow this type of Scenario, although we can prevent Native Comps from Attaching in the First Place
			//	&& (AttachParent == nullptr || !AttachParent->IsCreatedByConstructionScript() || !InComponent->HasAnyFlags(RF_DefaultSubObject))
			&& (InComponent->CreationMethod != EComponentCreationMethod::Native ||
				FComponentEditorUtils::GetPropertyForEditableNativeComponent(InComponent));
	}
	return false;
}

bool FAvaOutliner::CanProcessActorSpawn(AActor* InActor) const
{
	return IsActorAllowedInOutliner(InActor)
		&& OutlinerProvider.CanOutlinerProcessActorSpawn(InActor);
}

TSharedPtr<FUICommandList> FAvaOutliner::GetBaseCommandList() const
{
	return BaseCommandListWeak.Pin();
}

TArray<FName> FAvaOutliner::GetRegisteredItemProxyTypeNames() const
{
	TArray<FName> OutItemProxyTypeNames;
	{
		// Get Outliner-registered Item Types first
		TSet<FName> NameSet;
		ItemProxyRegistry.GetRegisteredItemProxyTypeNames(NameSet);

		// Get the Module-registered Item Types second
		TSet<FName> ModuleNameSet;
		IAvaOutlinerModule::Get().GetItemProxyRegistry().GetRegisteredItemProxyTypeNames(ModuleNameSet);

		NameSet.Append(ModuleNameSet);

		OutItemProxyTypeNames = NameSet.Array();
	}

	OutItemProxyTypeNames.Sort(FNameLexicalLess());

	return OutItemProxyTypeNames;
}

void FAvaOutliner::GetItemProxiesForItem(const FAvaOutlinerItemPtr& InItem, TArray<TSharedPtr<FAvaOutlinerItemProxy>>& OutItemProxies)
{
	// No Item Proxy support for Root
	if (!InItem.IsValid() || InItem == RootItem)
	{
		return;
	}

	InItem->GetItemProxies(OutItemProxies);
	
	IAvaOutlinerModule::Get().GetOnExtendItemProxiesForItem().Broadcast(*this, InItem, OutItemProxies);	

	// Clean up any invalid Item Proxy
	OutItemProxies.RemoveAll([](const TSharedPtr<FAvaOutlinerItemProxy>& InItemProxy) { return !InItemProxy.IsValid(); });
	
	// Sort Proxies by their Priority
	OutItemProxies.Sort([](const TSharedPtr<FAvaOutlinerItemProxy>& InItemProxyA, const TSharedPtr<FAvaOutlinerItemProxy>& InItemProxyB)
	{
		return InItemProxyA->GetPriority() > InItemProxyB->GetPriority();
	});
}

IAvaOutlinerItemProxyFactory* FAvaOutliner::GetItemProxyFactory(FName InItemProxyTypeName) const
{
	// First look for the Registry in Outliner
	if (IAvaOutlinerItemProxyFactory* Factory = ItemProxyRegistry.GetItemProxyFactory(InItemProxyTypeName))
	{
		return Factory;
	}
	// Fallback to finding the Factory in the Module if the Outliner did not find it
	return IAvaOutlinerModule::Get().GetItemProxyRegistry().GetItemProxyFactory(InItemProxyTypeName);
}

const TSharedRef<FAvaOutlinerSaveState>& FAvaOutliner::GetSaveState() const
{
	return SaveState;
}

bool FAvaOutliner::IsOutlinerLocked() const
{
	return OutlinerProvider.ShouldLockOutliner();
}

void FAvaOutliner::HandleUndoRedoTransaction(const FTransaction* Transaction, bool bIsUndo)
{
	RequestRefresh();
}

void FAvaOutliner::SetBaseCommandList(const TSharedPtr<FUICommandList>& InBaseCommandList)
{
	BaseCommandListWeak = InBaseCommandList;
}

void FAvaOutliner::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FAvaOutlinerVersion::GUID);
	
	FCustomVersionContainer CustomVersionContainer = Ar.GetCustomVersions();
	CustomVersionContainer.Serialize(Ar);
	Ar.SetCustomVersions(CustomVersionContainer);
	
	SaveState->Serialize(*this, Ar);
	
	if (Ar.IsLoading())
	{
		Refresh();
		bOutlinerDirty = false;
		OnOutlinerLoaded.Broadcast();
	}
}

TSharedPtr<IAvaOutlinerView> FAvaOutliner::RegisterOutlinerView(int32 InOutlinerViewId)
{
	const bool bShouldCreateWidget = OutlinerProvider.ShouldCreateWidget();
	
	TSharedPtr<FAvaOutlinerView> OutlinerView = FAvaOutlinerView::CreateInstance(InOutlinerViewId
		, SharedThis(this)
		, bShouldCreateWidget);

	TSharedPtr<FUICommandList> BaseCommandList = GetBaseCommandList();
	OutlinerView->BindCommands(BaseCommandList);
	
	OutlinerViews.Add(InOutlinerViewId, OutlinerView);

	return OutlinerView;
}

TSharedPtr<IAvaOutlinerView> FAvaOutliner::GetOutlinerView(int32 InOutlinerViewId) const
{
	if (const TSharedPtr<FAvaOutlinerView>* const FoundOutlinerView = OutlinerViews.Find(InOutlinerViewId))
	{
		return *FoundOutlinerView;
	}
	return nullptr;
}

void FAvaOutliner::RegisterItem(const FAvaOutlinerItemPtr& InItem)
{
	if (InItem.IsValid())
	{
		const FAvaOutlinerItemId ItemId = InItem->GetItemId();
		const FAvaOutlinerItemPtr ExistingItem = FindItem(ItemId);

		// if there's no existing item or the existing item does not match its new replacement,
		// then call OnItemRegistered and Refresh Outliner
		if (!ExistingItem.IsValid() || ExistingItem != InItem)
		{
			InItem->OnItemRegistered();
			AddItem(InItem);
			RequestRefresh();
		}
	}
}

void FAvaOutliner::UnregisterItem(const FAvaOutlinerItemId& InItemId)
{
	if (const FAvaOutlinerItemPtr FoundItem = FindItem(InItemId))
	{
		FoundItem->OnItemUnregistered();
		RemoveItem(InItemId);
		RequestRefresh();
	}
}

void FAvaOutliner::RequestRefresh()
{
	bRefreshRequested = true;
}

void FAvaOutliner::Refresh()
{
	TGuardValue RefreshGuard(bRefreshing, true);
	bRefreshRequested = false;

	// Flush Pending Actions
	{
		// Make a Transaction if there's a Pending Action Requesting it
		TUniquePtr<FScopedTransaction> Transaction;

		if (!GIsTransacting)
		{
			const bool bShouldTransact = PendingActions.ContainsByPredicate(
				[](const TSharedPtr<IAvaOutlinerAction>& InAction)
				{
					return InAction.IsValid() && InAction->ShouldTransact();
				});

			if (bShouldTransact)
			{
				Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("OutlinerItemAction", "Outliner Item Action"));
			}
		}

		// Execute Pending Actions
		for (const TSharedPtr<IAvaOutlinerAction>& Action : PendingActions)
		{
			if (Action.IsValid())
			{
				Action->Execute(*this);
			}
		}

		PendingActions.Empty();
	}

	// Save the Current Item ordering before Refreshing Children
	// Do not reset tree just yet as there might be actors that still need to be discovered on the next pass
	// This is done to save the items added from the Queued Actions and be considered when adding new items from Discovery
	SaveState->SaveSceneTree(*this, /*bInResetTree*/false);

	SceneOutlinerParentMap.Reset();
	if (UWorld* const World = GetWorld())
	{
		// Ensure the latest world has its delegates bound
		BindWorldDelegates(World);

		auto AddActorToParentMap = [this](UWorld* const InWorld, AActor* const InActor)
			{
				if (!IsValid(InWorld) || !IsValid(InActor))
				{
					return;
				}

				ULevelInstanceSubsystem* const LevelInstanceSubsystem = InWorld->GetSubsystem<ULevelInstanceSubsystem>();

				ULevel* const Level = InActor->GetLevel();

				// Ensure that the Actor's Level is registered as an Item
				FindOrAdd<FAvaOutlinerLevel>(Level);

				AActor* Parent = InActor->GetSceneOutlinerParent();

				// Try to find the Level Instance Actor to use as Parent for actors that aren't attached to anything
				// and belong to sub-levels
				if (!Parent && Level != InWorld->PersistentLevel)
				{
					if (const ILevelInstanceInterface* LevelInstance = LevelInstanceSubsystem->GetOwningLevelInstance(Level))
					{
						if (const ULevelInstanceComponent* LevelInstanceComponent = LevelInstance->GetLevelInstanceComponent())
						{
							Parent = LevelInstanceComponent->GetOwner();
						}
					}
				}

				SceneOutlinerParentMap.FindOrAdd(Parent).AddUnique(InActor);
				FindOrAdd<FAvaOutlinerActor>(InActor);
			};

		// 1) Update the Scene Outliner Parent Map for its Parent to know this Actor
		// 2) Make sure this Actor has an assigned Outliner Item
		for (AActor* const Actor : TActorRange<AActor>(World))
		{
			AddActorToParentMap(World, Actor);
		}

		// Add all actors from streaming levels
		for (ULevelStreaming* const LevelStreaming : World->GetStreamingLevels())
		{
			if (const TSoftObjectPtr<UWorld>& WorldAsset = LevelStreaming->GetWorldAsset())
			{
				if (WorldAsset && WorldAsset->PersistentLevel)
				{
					for (AActor* const Actor : WorldAsset->PersistentLevel->Actors)
					{
						AddActorToParentMap(WorldAsset.Get(), Actor);
					}
				}
			}
		}
	}

	// Refresh each item's children. This also Updates each Child's parent var
	ForEachItem([](const FAvaOutlinerItemPtr& InItem)
	{
		InItem->RefreshChildren();
	});

	RootItem->RefreshChildren();

	ForEachOutlinerView([](const TSharedPtr<FAvaOutlinerView>& InOutlinerView)
	{
		InOutlinerView->Refresh();
	});

	// Save so that the Tree is updated to the latest Outliner State
	SaveState->SaveSceneTree(*this, /*bInResetTree*/true);

	if (UAvaOutlinerSubsystem* const OutlinerSubsystem = GetOutlinerSubsystem())
	{
		const TArray<FActorHierarchyChange> ActorHierarchyChanges = MoveTemp(QueuedActorHierarchyChanges);
		QueuedActorHierarchyChanges.Empty();

		for (const FActorHierarchyChange& Change : ActorHierarchyChanges)
		{
			OutlinerSubsystem->BroadcastActorHierarchyChanged(Change.ActorWeak.Get(), Change.ParentWeak.Get(), Change.ChangeType);
		}
	}
}

TSharedRef<FAvaOutlinerItem> FAvaOutliner::GetTreeRoot() const
{
	return RootItem;
}

FAvaOutlinerItemPtr FAvaOutliner::FindItem(const FAvaOutlinerItemId& InItemId) const
{
	if (bIteratingItemMap)
	{
		if (const FAvaOutlinerItemPtr* const FoundItem = ItemsPendingAdd.Find(InItemId))
		{
			return *FoundItem;
		}
	}

	if (const FAvaOutlinerItemPtr* const FoundItem = ItemMap.Find(InItemId))
	{
		return *FoundItem;
	}

	return nullptr;
}

void FAvaOutliner::SetIgnoreNotify(EAvaOutlinerIgnoreNotifyFlags InFlag, bool bIgnore)
{
	if (bIgnore)
	{
		EnumAddFlags(IgnoreNotifyFlags, InFlag);
	}
	else
	{
		EnumRemoveFlags(IgnoreNotifyFlags, InFlag);
	}
}

void FAvaOutliner::OnActorsCopied(FString& InOutCopiedData, TConstArrayView<AActor*> InCopiedActors)
{
	FAvaOutlinerExporter Exporter(SharedThis(this));
	Exporter.ExportText(InOutCopiedData, InCopiedActors);
}

void FAvaOutliner::OnActorsPasted(FStringView InPastedData, const TMap<FName, AActor*>& InPastedActors)
{
	FAvaOutlinerImporter Importer(SharedThis(this));
	Importer.ImportText(InPastedData, InPastedActors);
}

void FAvaOutliner::OnActorsDuplicated(const TMap<AActor*, AActor*>& InDuplicateActorMap
	, FAvaOutlinerItemPtr InRelativeItem
	, TOptional<EItemDropZone> InRelativeDropZone)
{
	if (InDuplicateActorMap.IsEmpty())
	{
		return;
	}

	// Bail if we're currently Ignoring Duplication Notifies
	if (EnumHasAnyFlags(IgnoreNotifyFlags, EAvaOutlinerIgnoreNotifyFlags::Duplication))
	{
		return;
	}

	// Map of the Duplicate Actor to the Template Actor's Item
	TMap<AActor*, FAvaOutlinerItemPtr> DuplicateItemMap;
	
	// Try finding the Template Item with the Lowest Index (i.e. Highest in the Tree) to use as Placeholder Relative Item
	if (!InRelativeItem.IsValid())
	{
		DuplicateItemMap.Reserve(InDuplicateActorMap.Num());
		for (const TPair<AActor*, AActor*>& Pair : InDuplicateActorMap)
		{
			AActor* const TemplateActor = Pair.Value;
			if (IsValid(TemplateActor))
			{
				if (FAvaOutlinerItemPtr FoundTemplateItem = FindItem(TemplateActor))
				{
					DuplicateItemMap.Emplace(Pair.Key, FoundTemplateItem);
					if (!InRelativeItem.IsValid() || UE::AvaOutliner::CompareOutlinerItemOrder(FoundTemplateItem, InRelativeItem))
					{
						InRelativeItem = FoundTemplateItem;
					}
				}
			}
		}
	}

	const EItemDropZone RelativeDropZone = InRelativeDropZone.Get(EItemDropZone::BelowItem);

	// Newly Duplicated Items don't have a Parent Item set yet, so it's not going to call RemoveFromParent, which handles Detachment
	bool bShouldDetachActors = false;
	if (InRelativeItem.IsValid())
	{
		const bool bRelativeItemIsRoot        = InRelativeItem == RootItem;
		const bool bRelativeItemIsChildOfRoot = InRelativeItem->GetParent() == RootItem;
		
		bShouldDetachActors = bRelativeItemIsRoot || (bRelativeItemIsChildOfRoot && RelativeDropZone != EItemDropZone::OntoItem);
	}

	TArray<AActor*> DuplicateActors;
	InDuplicateActorMap.GetKeys(DuplicateActors);

	// Sort based on the ordering of the template actor items in outliner
	DuplicateActors.Sort([&InDuplicateActorMap, this](const AActor& A, const AActor& B)
	{
		AActor* const * const TemplateActorA = InDuplicateActorMap.Find(&A);
		AActor* const * const TemplateActorB = InDuplicateActorMap.Find(&B);

		const FAvaOutlinerItemPtr TemplateItemA = FindItem(*TemplateActorA);
		const FAvaOutlinerItemPtr TemplateItemB = FindItem(*TemplateActorB);

		return UE::AvaOutliner::CompareOutlinerItemOrder(TemplateItemA, TemplateItemB);
	});

	TArray<TSharedPtr<IAvaOutlinerAction>> ItemActions;
	ItemActions.Reserve(DuplicateActors.Num());

	FAvaOutlinerAddItemParams Params;
	Params.RelativeDropZone = RelativeDropZone;
	Params.Flags            = EAvaOutlinerAddItemFlags::AddChildren;

	// Finally, Enqueue the Add Item Action to these new Actors
	for (AActor* const DuplicateActor : DuplicateActors)
	{
		Params.RelativeItem.Reset();
		
		//We don't need to worry about Child Items since the Parent Item will do all the Attachment Work
		const AActor* const ParentActor = DuplicateActor->GetAttachParentActor();

		const bool bIsParentDuplicated = InDuplicateActorMap.Contains(ParentActor);
		if (!bIsParentDuplicated && bShouldDetachActors)
		{
			DuplicateActor->DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
		}

		FAvaOutlinerItemPtr Item = FindOrAdd<FAvaOutlinerActor>(DuplicateActor);
		Params.Items = { Item };
		
		if (FAvaOutlinerItemPtr* const FoundTemplateItem = DuplicateItemMap.Find(DuplicateActor))
		{
			FAvaOutlinerItemPtr TemplateItem = *FoundTemplateItem;

			if (!bIsParentDuplicated)
			{
				Params.RelativeItem = TemplateItem;
			}
			
			//Sanitize the Flags by Removing the Temp Ones
			EAvaOutlinerItemFlags TemplateItemFlags = TemplateItem->GetFlags();
			EnumRemoveFlags(TemplateItemFlags, EAvaOutlinerItemFlags::PendingRemoval | EAvaOutlinerItemFlags::IgnorePendingKill);
			
			Item->SetFlags(TemplateItemFlags);

			//Copy the template flags of each outliner view into the new item
			ForEachOutlinerView([TemplateItem, Item](const TSharedPtr<FAvaOutlinerView>& InOutlinerView)
			{
				InOutlinerView->SetViewItemFlags(Item, InOutlinerView->GetViewItemFlags(TemplateItem));
			});			
		}
		
		//If a Relative Item hasn't been set yet (i.e. no Template Item found for the Item)
		if (!Params.RelativeItem.IsValid() && !bIsParentDuplicated)
		{
			Params.RelativeItem = InRelativeItem;
		}

		TSharedRef<FAvaOutlinerAddItem> AddItemAction = NewItemAction<FAvaOutlinerAddItem>(Params);
		ItemActions.Add(AddItemAction);

		// Append after first selected item is added
		Params.SelectionFlags |= EAvaOutlinerItemSelectionFlags::AppendToCurrentSelection;
	}
	
	EnqueueItemActions(MoveTemp(ItemActions));
}

void FAvaOutliner::GroupSelection(AActor* InGroupingActor, const TOptional<FAttachmentTransformRules>& InTransformRules)
{
	if (!InGroupingActor)
	{
		return;
	}

	FAvaOutlinerItemPtr GroupingItem = FindItem(InGroupingActor);
	bool bIsGroupingItemNew = false;	
	if (!GroupingItem.IsValid())
	{
		GroupingItem = FindOrAdd<FAvaOutlinerActor>(InGroupingActor);
		bIsGroupingItemNew = true;
	}
	GroupingItem->SetFlags(EAvaOutlinerItemFlags::Expanded);

	TArray<FAvaOutlinerItemPtr> SelectedItems = GetSelectedItems();
	NormalizeItems(SelectedItems);

	// Enqueue the Grouping Item if New
	if (bIsGroupingItemNew)
	{
		FAvaOutlinerAddItemParams AddGroupingItemParams;
		AddGroupingItemParams.Items = { GroupingItem };
		AddGroupingItemParams.Flags = EAvaOutlinerAddItemFlags::AddChildren;

		AddGroupingItemParams.AttachmentTransformRules = InTransformRules;

		if (const FAvaOutlinerItemPtr LowestCommonAncestor = FindLowestCommonAncestor(SelectedItems))
		{
			//Find the first path that leads to a Selected Item
			TArray<FAvaOutlinerItemPtr> Descendants = LowestCommonAncestor->FindPath(SelectedItems);
			if (Descendants.Num() > 0)
			{
				AddGroupingItemParams.RelativeItem     = Descendants[0];
				AddGroupingItemParams.RelativeDropZone = EItemDropZone::BelowItem;
			}
			else
			{
				AddGroupingItemParams.RelativeItem     = LowestCommonAncestor;
				AddGroupingItemParams.RelativeDropZone = EItemDropZone::OntoItem;
			}
		}

		EnqueueItemAction<FAvaOutlinerAddItem>(AddGroupingItemParams);
	}	
	
	//If there are no Selected Items, stop here
	if (SelectedItems.IsEmpty())
	{
		return;
	}

	SortItems(SelectedItems);

	FAvaOutlinerAddItemParams AddChildItemParams;
	AddChildItemParams.RelativeItem     = GroupingItem;
	AddChildItemParams.RelativeDropZone = EItemDropZone::OntoItem;
	AddChildItemParams.SelectionFlags   = EAvaOutlinerItemSelectionFlags::AppendToCurrentSelection | EAvaOutlinerItemSelectionFlags::ScrollIntoView;

	AddChildItemParams.AttachmentTransformRules = InTransformRules;
	AddChildItemParams.Items = SelectedItems;

	EnqueueItemAction<FAvaOutlinerAddItem>(AddChildItemParams);
}

void FAvaOutliner::PostUndo(bool bSuccess)
{
	if (bSuccess)
	{
		const int32 QueueIndex = GEditor->Trans->GetQueueLength() - GEditor->Trans->GetUndoCount();
		const FTransaction* Transaction = GEditor->Trans->GetTransaction(QueueIndex);
		HandleUndoRedoTransaction(Transaction, true);
	}
}

void FAvaOutliner::PostRedo(bool bSuccess)
{
	if (bSuccess)
	{
		const int32 QueueIndex = GEditor->Trans->GetQueueLength() - GEditor->Trans->GetUndoCount();
		const FTransaction* Transaction = GEditor->Trans->GetTransaction(QueueIndex);
		HandleUndoRedoTransaction(Transaction, false);
	}
}

TStatId FAvaOutliner::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FAvaOutliner, STATGROUP_Tickables);
}

void FAvaOutliner::Tick(float InDeltaTime)
{
	if (NeedsRefresh())
	{
		Refresh();
	}

	if (bOutlinerDirty)
	{
		bOutlinerDirty = false;
	}

	// Select Objects Pending Selection
	if (ObjectsLastSelected.IsValid())
	{
		TArray<FAvaOutlinerItemPtr> ItemsToSelect;

		if (ObjectsLastSelected->Num() > 0)
		{
			ItemsToSelect.Reserve(ObjectsLastSelected->Num());
			for (TWeakObjectPtr<UObject> Object : *ObjectsLastSelected)
			{
				if (Object.IsValid())
				{
					if (FAvaOutlinerItemPtr Item = FindItem(FAvaOutlinerItemId(Object.Get())))
					{
						// If it's a Shared Object, select all its References as Shared Objects are not in the Outliner directly
						// and the outliner does not know which Object Reference should be selected
						if (const FAvaOutlinerSharedObject* const SharedObjectItem = Item->CastTo<FAvaOutlinerSharedObject>())
						{
							ItemsToSelect.Append(SharedObjectItem->GetObjectReferences());
						}
						else
						{
							ItemsToSelect.Add(Item);
						}
					}
				}
			}
		}

		// Only Scroll Into View, don't signal selection since we just come from the selection notify itself
		SelectItems(ItemsToSelect, EAvaOutlinerItemSelectionFlags::ScrollIntoView);
		ObjectsLastSelected.Reset();
	}

	ForEachOutlinerView([InDeltaTime](const TSharedPtr<FAvaOutlinerView>& InOutlinerView)
	{
		InOutlinerView->Tick(InDeltaTime);
	});
}

void FAvaOutliner::DuplicateItems(TArray<FAvaOutlinerItemPtr> InItems
	, FAvaOutlinerItemPtr InRelativeItem
	, TOptional<EItemDropZone> InRelativeDropZone)
{
	SortItems(InItems);

	TArray<AActor*> TemplateActors;
	for (const FAvaOutlinerItemPtr& Item : InItems)
	{
		if (!Item.IsValid())
		{
			continue;
		}
		
		if (const FAvaOutlinerActor* const ActorItem = Item->CastTo<FAvaOutlinerActor>())
		{
			TemplateActors.Add(ActorItem->GetActor());
		}
	}

	if (TemplateActors.IsEmpty())
	{
		return;
	}

	OutlinerProvider.OutlinerDuplicateActors(TemplateActors);
}

void FAvaOutliner::DeleteItems(TArray<FAvaOutlinerItemPtr> InItems)
{
	SortItems(InItems);

	TArray<AActor*> DeleteActors;
	DeleteActors.Reserve(InItems.Num());

	for (TArray<FAvaOutlinerItemPtr>::TIterator It(InItems); It; ++It)
	{
		FAvaOutlinerItemPtr Item = *It;

		if (!Item.IsValid() || !Item->CanDelete())
		{
			It.RemoveCurrent();
		}

		if (const FAvaOutlinerActor* const ActorItem = Item->CastTo<FAvaOutlinerActor>())
		{
			DeleteActors.Add(ActorItem->GetActor());
		}
	}

	if (InItems.IsEmpty())
	{
		return;
	}

	FScopedTransaction DeleteTransaction(LOCTEXT("OutlinerItemDeleteAction", "Outliner Delete Item(s)"), !GIsTransacting);

	if (!DeleteActors.IsEmpty())
	{
		OutlinerProvider.OutlinerDeleteActors(DeleteActors);
	}

	bool bRequestRefresh = false;
	for (FAvaOutlinerItemPtr& Item : InItems)
	{
		if (Item && Item->Delete())
		{
			bRequestRefresh = true;
		}
	}

	if (bRequestRefresh)
	{
		RequestRefresh();
	}
}

void FAvaOutliner::UnregisterOutlinerView(int32 InOutlinerViewId)
{
	OutlinerViews.Remove(InOutlinerViewId);
}

void FAvaOutliner::UpdateRecentOutlinerViews(int32 InOutlinerViewId)
{
	RecentOutlinerViews.Remove(InOutlinerViewId);
	RecentOutlinerViews.Add(InOutlinerViewId);
}

TSharedPtr<FAvaOutlinerView> FAvaOutliner::GetMostRecentOutlinerView() const
{
	for (int32 Index = RecentOutlinerViews.Num() - 1; Index >= 0; --Index)
	{
		TSharedPtr<IAvaOutlinerView> OutlinerView = GetOutlinerView(RecentOutlinerViews[Index]);
		if (OutlinerView.IsValid())
		{
			return StaticCastSharedPtr<FAvaOutlinerView>(OutlinerView);
		}
	}
	return nullptr;
}

void FAvaOutliner::ForEachOutlinerView(const TFunction<void(const TSharedPtr<FAvaOutlinerView>& InOutlinerView)>& InPredicate) const
{
	FAvaOutliner* const MutableThis = const_cast<FAvaOutliner*>(this);
	for (TMap<int32, TSharedPtr<FAvaOutlinerView>>::TIterator Iter(MutableThis->OutlinerViews); Iter; ++Iter)
	{
		const TSharedPtr<FAvaOutlinerView>& OutlinerView = Iter.Value();
		if (OutlinerView.IsValid())
		{
			InPredicate(OutlinerView);
		}
		else
		{
			Iter.RemoveCurrent();
		}
	}
}

void FAvaOutliner::EnqueueItemActions(TArray<TSharedPtr<IAvaOutlinerAction>>&& InItemActions) noexcept
{
	PendingActions.Append(MoveTemp(InItemActions));
}

int32 FAvaOutliner::GetPendingItemActionCount() const
{
	return PendingActions.Num();
}

bool FAvaOutliner::NeedsRefresh() const
{
	//Return false if we're already refreshing
	if (bRefreshing)
	{
		return false;
	}

	if (bRefreshRequested || PendingActions.Num() > 0)
	{
		return true;
	}

	return false;
}

TOptional<FAvaOutlinerColorPair> FAvaOutliner::FindItemColor(const FAvaOutlinerItemPtr& InItem, bool bRecurseParent) const
{
	if (InItem.IsValid())
	{
		if (const FName* const FoundColorName = SaveState->ItemColoring.Find(InItem->GetItemId().GetStringId()))
		{
			if (const FLinearColor* const FoundColor = GetColorMap().Find(*FoundColorName))
			{
				return FAvaOutlinerColorPair(*FoundColorName, *FoundColor);
			}
		}

		//If no Item Coloring Mapping was found for the Specific Item, then try find the Item Color of the Parent
		if (bRecurseParent)
		{
			return FindItemColor(InItem->GetParent(), bRecurseParent);
		}
	}
	return TOptional<FAvaOutlinerColorPair>();
}

void FAvaOutliner::SetItemColor(const FAvaOutlinerItemPtr& InItem, const FName& InColorName)
{
	if (InItem.IsValid())
	{
		bool bShouldChangeItemColor = true;

		TOptional<FAvaOutlinerColorPair> InheritedColor = FindItemColor(InItem, true);
		const bool bHasInheritedColor = InheritedColor.IsSet();

		if (bHasInheritedColor)
		{
			//Make sure the Inherited Color is different than the Color we're trying to Set.
			bShouldChangeItemColor = InheritedColor->Key != InColorName;
		}

		if (bShouldChangeItemColor)
		{
			TOptional<FAvaOutlinerColorPair> ParentInheritedColor = FindItemColor(InItem->GetParent(), true);

			//First check if the Color to Set matches the one inherited from the Parent
			if (ParentInheritedColor.IsSet() && ParentInheritedColor->Key == InColorName)
			{
				//If it matches, remove this Item from the map as we will inherit from parent
				SaveState->ItemColoring.Remove(InItem->GetItemId().GetStringId());
			}
			else
			{
				SaveState->ItemColoring.Add(InItem->GetItemId().GetStringId(), InColorName);
			}

			//Recurse Children to Remove their Mapping if same color with parent
			TArray<FAvaOutlinerItemPtr> RemainingChildren = InItem->GetChildren();
			while (RemainingChildren.Num() > 0)
			{
				FAvaOutlinerItemPtr Child = RemainingChildren.Pop();
				if (Child.IsValid())
				{
					TOptional<FAvaOutlinerColorPair> ChildColor = FindItemColor(Child, false);
					if (ChildColor.IsSet() && ChildColor->Key == InColorName)
					{
						RemoveItemColor(Child);

						//Only check Grand Children if Child had same Color.
						//Allow the situation where Parent is ColorA, Child is ColorB and GrandChild is ColorA.
						RemainingChildren.Append(Child->GetChildren());
					}
				}
			}
			SetOutlinerModified();
		}
	}
}

void FAvaOutliner::RemoveItemColor(const FAvaOutlinerItemPtr& InItem)
{
	if (InItem.IsValid())
	{
		const bool bRemoved = SaveState->ItemColoring.Remove(InItem->GetItemId().GetStringId()) > 0;
		if (bRemoved)
		{
			SetOutlinerModified();
		}
	}
}

const TMap<FName, FLinearColor>& FAvaOutliner::GetColorMap() const
{
	return UAvaOutlinerSettings::Get()->GetColorMap();
}

void FAvaOutliner::NotifyItemIdChanged(const FAvaOutlinerItemId& OldId, const FAvaOutlinerItemPtr& InItem)
{
	const FAvaOutlinerItemId NewId = InItem->GetItemId();
	if (OldId == NewId)
	{
		return;
	}

	const FAvaOutlinerItemPtr FoundItem = FindItem(OldId);
	if (FoundItem.IsValid() && FoundItem == InItem)
	{
		AddItem(InItem);
		RemoveItem(OldId);
		SetOutlinerModified();
	}
}

TArray<FAvaOutlinerItemPtr> FAvaOutliner::GetSelectedItems() const
{
	if (const TSharedPtr<FAvaOutlinerView>& OutlinerView = GetMostRecentOutlinerView())
	{
		return OutlinerView->GetViewSelectedItems();
	}
	return TArray<FAvaOutlinerItemPtr>();
}

int32 FAvaOutliner::GetSelectedItemCount() const
{
	if (const TSharedPtr<FAvaOutlinerView>& OutlinerView = GetMostRecentOutlinerView())
	{
		return OutlinerView->GetViewSelectedItemCount();
	}
	return -1;
}

void FAvaOutliner::SelectItems(const TArray<FAvaOutlinerItemPtr>& InItems, EAvaOutlinerItemSelectionFlags InFlags) const
{
	ForEachOutlinerView([InItems, InFlags](const TSharedPtr<FAvaOutlinerView>& InOutlinerView)
	{
		InOutlinerView->SelectItems(InItems, InFlags);
	});
}

void FAvaOutliner::ClearItemSelection(bool bSignalSelectionChange) const
{
	ForEachOutlinerView([bSignalSelectionChange](const TSharedPtr<FAvaOutlinerView>& InOutlinerView)
	{
		InOutlinerView->ClearItemSelection(bSignalSelectionChange);
	});
}

FAvaOutlinerItemPtr FAvaOutliner::FindLowestCommonAncestor(const TArray<FAvaOutlinerItemPtr>& Items)
{
	TSet<FAvaOutlinerItemPtr> IntersectedAncestors;

	for (const FAvaOutlinerItemPtr& Item : Items)
	{
		FAvaOutlinerItemPtr Parent = Item->GetParent();
		TSet<FAvaOutlinerItemPtr> ItemAncestors;

		//Add all Item's Ancestors
		while (Parent.IsValid())
		{
			ItemAncestors.Add(Parent);
			Parent = Parent->GetParent();
		}

		//cant check for intersection if empty so just init
		if (IntersectedAncestors.Num() == 0)
		{
			IntersectedAncestors = ItemAncestors;
		}
		else
		{
			IntersectedAncestors = IntersectedAncestors.Intersect(ItemAncestors);

			//We are sure the intersection is the Root if only one item is remaining. Stop iterating
			if (IntersectedAncestors.Num() == 1)
			{
				break;
			}
		}
	}

	FAvaOutlinerItemPtr LowestCommonAncestor;
	for (const FAvaOutlinerItemPtr& Item : IntersectedAncestors)
	{
		//Find Item with most tree height (i.e. lowest down the tree, closer to the selected nodes)
		if (!LowestCommonAncestor.IsValid() || Item->GetItemTreeHeight() > LowestCommonAncestor->GetItemTreeHeight())
		{
			LowestCommonAncestor = Item;
		}
	}
	return LowestCommonAncestor;
}

FAvaSceneItem FAvaOutliner::MakeSceneItemFromOutlinerItem(const FAvaOutlinerItemPtr& InItem)
{
	if (!InItem.IsValid())
	{
		return FAvaSceneItem();
	}

	if (InItem->IsA<FAvaOutlinerObject>())
	{
		const TSharedPtr<FAvaOutlinerObject> ObjectItem = StaticCastSharedPtr<FAvaOutlinerObject>(InItem);

		if (UObject* const Object = ObjectItem->GetObject())
		{
			UObject* const World     = ObjectItem->GetOwnerOutliner()->GetWorld();
			UObject* const StopOuter = World && Object->IsIn(World) ? World : nullptr;

			return FAvaSceneItem(Object, StopOuter);
		}
	}

	return FAvaSceneItem(InItem->GetItemId().GetStringId());
}

void FAvaOutliner::SortItems(TArray<FAvaOutlinerItemPtr>& OutOutlinerItems)
{
	OutOutlinerItems.Sort([](const FAvaOutlinerItemPtr& ItemA, const FAvaOutlinerItemPtr& ItemB)
	{
		return UE::AvaOutliner::CompareOutlinerItemOrder(ItemA, ItemB);
	});
}

void FAvaOutliner::NormalizeItems(TArray<FAvaOutlinerItemPtr>& InOutItems)
{
	if (InOutItems.IsEmpty())
	{
		return;
	}

	//Set for Quick Lookup
	const TSet<FAvaOutlinerItemPtr> SelectedItemSet(InOutItems);

	//Normalize Selection: Remove all Items that have Parents that are in the Selection. Swapping since we're sorting afterwards
	InOutItems.RemoveAllSwap([&SelectedItemSet](const FAvaOutlinerItemPtr& Item)
	{
		FAvaOutlinerItemPtr Parent = Item->GetParent();
		while (Parent.IsValid())
		{
			if (SelectedItemSet.Contains(Parent))
			{
				return true;
			}
			Parent = Parent->GetParent();
		}
		return false;
	});
}

FEditorModeTools* FAvaOutliner::GetModeTools() const
{
	return OutlinerProvider.GetOutlinerModeTools();
}

void FAvaOutliner::SyncModeToolsSelection(const TArray<FAvaOutlinerItemPtr>& InSelectedItems) const
{
	const FEditorModeTools* const ModeTools = GetModeTools();

	if (!ModeTools)
	{
		return;
	}

	FAvaOutlinerScopedSelection ScopedSelection(*ModeTools, EAvaOutlinerScopedSelectionPurpose::Sync);
	for (const FAvaOutlinerItemPtr& Item : InSelectedItems)
	{
		Item->Select(ScopedSelection);
	}
}

void FAvaOutliner::OnObjectSelectionChanged(const FAvaEditorSelection& InEditorSelection)
{
	bool bIsSyncingItemSelection = false;

	// find if any outliner view is syncing item selection
	// if we are, ignore object selection or it will cause another round of selections next tick
	ForEachOutlinerView([&bIsSyncingItemSelection](const TSharedPtr<FAvaOutlinerView>& InOutlinerView)
	{
		bIsSyncingItemSelection |= InOutlinerView->IsSyncingItemSelection();
	});

	if (bIsSyncingItemSelection)
	{
		return;
	}
	
	if (!ObjectsLastSelected.IsValid())
	{
		ObjectsLastSelected = MakeShared<TArray<TWeakObjectPtr<UObject>>>();
	}

	TArray<UObject*> SelectedObjects = InEditorSelection.GetSelectedObjects<UObject, EAvaSelectionSource::All>();
	ObjectsLastSelected->Reserve(ObjectsLastSelected->Num() + SelectedObjects.Num());

	for (UObject* const Object : SelectedObjects)
	{
		ObjectsLastSelected->AddUnique(Object);
	}
}

UWorld* FAvaOutliner::GetWorld() const
{
	return OutlinerProvider.GetOutlinerWorld();
}

const FAvaOutlinerItemProxyRegistry& FAvaOutliner::GetItemProxyRegistry() const
{
	return ItemProxyRegistry;
}

FAvaOutlinerItemProxyRegistry& FAvaOutliner::GetItemProxyRegistry()
{
	return ItemProxyRegistry;
}

TArray<TWeakObjectPtr<AActor>> FAvaOutliner::GetActorSceneOutlinerChildren(AActor* InParentActor) const
{
	if (const TArray<TWeakObjectPtr<AActor>>* const FoundChildren = SceneOutlinerParentMap.Find(InParentActor))
	{
		return *FoundChildren;
	}
	return TArray<TWeakObjectPtr<AActor>>();
}

void FAvaOutliner::OnActorSpawned(AActor* InActor)
{
	const bool bSpawnNotifyIgnored = EnumHasAnyFlags(IgnoreNotifyFlags, EAvaOutlinerIgnoreNotifyFlags::Spawn);
	if (!bSpawnNotifyIgnored && IsValid(InActor) && InActor->GetWorld() == GetWorld())
	{
		if (CanProcessActorSpawn(InActor) && !FindItem(FAvaOutlinerItemId(InActor)))
		{
			FAvaOutlinerAddItemParams Params;
			Params.Items            = { FindOrAdd<FAvaOutlinerActor>(InActor) };
			Params.RelativeItem     = nullptr;
			Params.RelativeDropZone = TOptional<EItemDropZone>();
			Params.Flags            = EAvaOutlinerAddItemFlags::AddChildren;
			Params.SelectionFlags   = EAvaOutlinerItemSelectionFlags::AppendToCurrentSelection | EAvaOutlinerItemSelectionFlags::ScrollIntoView;

			EnqueueItemAction<FAvaOutlinerAddItem>(MoveTemp(Params));
		}
	}
}

void FAvaOutliner::OnActorDestroyed(AActor* InActor)
{
	const FAvaOutlinerItemPtr ActorItem = FindItem(FAvaOutlinerItemId(InActor));
	if (ActorItem.IsValid())
	{
		FAvaOutlinerRemoveItemParams Params(ActorItem);
		EnqueueItemAction<FAvaOutlinerRemoveItem>(MoveTemp(Params));
	}
}

void FAvaOutliner::OnActorAttachmentChanged(AActor* InActor, const AActor* InParent, bool bAttach)
{
	//If Parent is Pending Kill, it probably means that this was called while the Parent is being destroyed.
	if (GIsTransacting || (!bRefreshing && IsValid(InParent)))
	{
		RequestRefresh();
	}
}

void FAvaOutliner::OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap)
{
	ForEachItem([&InReplacementMap](const FAvaOutlinerItemPtr& InItem)
	{
		// Recursive not needed since we're calling it on all Items in Map Anyway
		InItem->OnObjectsReplaced(InReplacementMap, /*bRecursive*/false);
	});

	if (ObjectsLastSelected.IsValid())
	{
		for (TWeakObjectPtr<UObject>& Object : *ObjectsLastSelected)
		{
			if (Object.IsValid() && InReplacementMap.Contains(Object.Get()))
			{
				Object = InReplacementMap[Object.Get()];
			}
		}
	}

	for (const TSharedPtr<IAvaOutlinerAction>& Action : PendingActions)
	{
		//Recursive needed since we only have direct reference to the underlying item in the Action, not its children
		if (Action.IsValid())
		{
			Action->OnObjectsReplaced(InReplacementMap, true);
		}
	}

	ForEachOutlinerView([](const TSharedPtr<FAvaOutlinerView>& InOutlinerView)
	{
		InOutlinerView->NotifyObjectsReplaced();
	});
}

void FAvaOutliner::OnActorReplaced(AActor* InOldActor, AActor* InNewActor)
{
	TMap<UObject*, UObject*> ReplacementMap;
	ReplacementMap.Add(InOldActor, InNewActor);
	OnObjectsReplaced(ReplacementMap);
}

void FAvaOutliner::SetOutlinerModified()
{
	if (!bOutlinerDirty)
	{
		bOutlinerDirty = true;
	}
}

void FAvaOutliner::QueueActorHierarchyChange(AActor* InActor, const AActor* InParentActor, EAvaOutlinerHierarchyChangeType InChangeType)
{
	FActorHierarchyChange& Change = QueuedActorHierarchyChanges.AddDefaulted_GetRef();
	Change.ActorWeak = InActor;
	Change.ParentWeak = InParentActor;
	Change.ChangeType = InChangeType;
}

void FAvaOutliner::BindWorldDelegates(UWorld* InWorld)
{
	if (BoundWorldWeak == InWorld)
	{
		return;
	}

	UnbindWorldDelegates();
	BoundWorldWeak = InWorld;

	if (InWorld)
	{
		InWorld->OnLevelsChanged().AddSP(this, &FAvaOutliner::RequestRefresh);
	}
}

void FAvaOutliner::UnbindWorldDelegates()
{
	if (UWorld* const BoundWorld = BoundWorldWeak.Get())
	{
		BoundWorld->OnLevelsChanged().RemoveAll(this);
	}
}

void FAvaOutliner::AddItem(const FAvaOutlinerItemPtr& InItem)
{
	const FAvaOutlinerItemId ItemId = InItem->GetItemId();

	ItemsPendingRemove.Remove(ItemId);

	if (bIteratingItemMap)
	{
		ItemsPendingAdd.Add(ItemId, InItem);
	}
	else
	{
		ItemMap.Add(ItemId, InItem);
	}
}

void FAvaOutliner::RemoveItem(const FAvaOutlinerItemId& InItemId)
{
	ItemsPendingAdd.Remove(InItemId);

	if (bIteratingItemMap)
	{
		ItemsPendingRemove.Add(InItemId);
	}
	else
	{
		ItemMap.Remove(InItemId);
	}
}

void FAvaOutliner::ForEachItem(TFunctionRef<void(const FAvaOutlinerItemPtr&)> InFunc)
{
	// iteration scope, allowing for nested for-each
	{
		TGuardValue<bool> Guard(bIteratingItemMap, true);
		for (const TPair<FAvaOutlinerItemId, FAvaOutlinerItemPtr>& Pair : ItemMap)
		{
			InFunc(Pair.Value);
		}
	}

	if (!bIteratingItemMap && (!ItemsPendingAdd.IsEmpty() || !ItemsPendingRemove.IsEmpty()))
	{
		for (const TPair<FAvaOutlinerItemId, FAvaOutlinerItemPtr>& ItemToAdd : ItemsPendingAdd)
		{
			ItemMap.Add(ItemToAdd);
		}
		ItemsPendingAdd.Empty();

		for (const FAvaOutlinerItemId& ItemIdToRemove : ItemsPendingRemove)
		{
			ItemMap.Remove(ItemIdToRemove);
		}
		ItemsPendingRemove.Empty();
	}
}

#undef LOCTEXT_NAMESPACE
