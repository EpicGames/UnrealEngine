// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "EditorUndoClient.h"
#include "Engine/EngineTypes.h"
#include "IAvaOutliner.h"
#include "IAvaOutlinerModule.h"
#include "Item/AvaOutlinerItemId.h"
#include "Item/AvaOutlinerItemProxy.h"
#include "Item/IAvaOutlinerItem.h"
#include "ItemProxies/AvaOutlinerItemProxyRegistry.h"
#include "ItemProxies/IAvaOutlinerItemProxyFactory.h"
#include "TickableEditorObject.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class FAvaEditorSelection;
class FAvaOutlinerTreeRoot;
class FAvaOutlinerView;
class FEditorModeTools;
class FTransaction;
class FUICommandList;
class IAvaOutlinerAction;
class IAvaOutlinerProvider;
class IAvaOutlinerView;
class UAvaOutlinerSubsystem;
class ULevelStreaming;
enum class EAvaOutlinerHierarchyChangeType : uint8;
enum class EItemDropZone;
struct FAttachmentTransformRules;
struct FAvaOutlinerSaveState;
struct FAvaSceneItem;

class FAvaOutliner
	: public IAvaOutliner
	, public FTickableEditorObject
	, public FEditorUndoClient
{
public:
	FAvaOutliner(IAvaOutlinerProvider& InOutlinerProvider);

	virtual ~FAvaOutliner() override;

	/**
	 * Gets the Outliner Subsystem of the World this Outliner is responsible for
	 * NOTE: The OutlinerSubsystem's Outliner Instance should be the same as this one in the default implementation 
	 * but can differ if there are multiple FAvaOutliner instances being used in custom implementations
	 */
	UAvaOutlinerSubsystem* GetOutlinerSubsystem() const;

	/**
	 * Determines whether the given actor can be presented in the Outliner, at all.
	 * This is a permanent check unlike filters that are temporary.
	 */
	bool IsActorAllowedInOutliner(const AActor* InActor) const;

	/**
	 * Determines whether the given scene component can be presented in the Outliner, at all.
	 * This is a permanent check unlike filters that are temporary.
	 */
	bool IsComponentAllowedInOutliner(const USceneComponent* InComponent) const;

	bool CanProcessActorSpawn(AActor* InActor) const;

	TSharedPtr<FUICommandList> GetBaseCommandList() const;

	FOnOutlinerLoaded OnOutlinerLoaded;

	/** Gathers the Type Names of all the Item Proxies that are registered both in the outliner proxy registry and the module's */
	TArray<FName> GetRegisteredItemProxyTypeNames() const;

	/** Gathers all previously existing and new Item Proxies for a given Item */
	void GetItemProxiesForItem(const FAvaOutlinerItemPtr& InItem, TArray<TSharedPtr<FAvaOutlinerItemProxy>>& OutItemProxies);

	/** Tries to find the Item Proxy Factory for the given Item Proxy Type Name */
	IAvaOutlinerItemProxyFactory* GetItemProxyFactory(FName InItemProxyTypeName) const;

	const TSharedRef<FAvaOutlinerSaveState>& GetSaveState() const;

	/** Returns whether the Outliner is in Read-only mode */
	bool IsOutlinerLocked() const;

	void HandleUndoRedoTransaction(const FTransaction* Transaction, bool bIsUndo);

	//~ Begin IAvaOutliner
	virtual FOnOutlinerLoaded& GetOnOutlinerLoaded() override { return OnOutlinerLoaded; }
	virtual IAvaOutlinerProvider& GetProvider() const override { return OutlinerProvider; }
	virtual void SetBaseCommandList(const TSharedPtr<FUICommandList>& InBaseCommandList) override;
	virtual void Serialize(FArchive& Ar) override;
	virtual TSharedPtr<IAvaOutlinerView> RegisterOutlinerView(int32 InOutlinerViewId) override;
	virtual TSharedPtr<IAvaOutlinerView> GetOutlinerView(int32 InOutlinerViewId) const override;
	virtual void RegisterItem(const FAvaOutlinerItemPtr& InItem) override;
	virtual void UnregisterItem(const FAvaOutlinerItemId& InItemId) override;
	virtual void RequestRefresh() override;
	virtual void Refresh() override;
	virtual TSharedRef<FAvaOutlinerItem> GetTreeRoot() const override;
	virtual FAvaOutlinerItemPtr FindItem(const FAvaOutlinerItemId& InItemId) const override;
	virtual void SetIgnoreNotify(EAvaOutlinerIgnoreNotifyFlags InFlag, bool bIgnore) override;
	virtual void OnActorsCopied(FString& InOutCopiedData, TConstArrayView<AActor*> InCopiedActors) override;
	virtual void OnActorsPasted(FStringView InPastedData, const TMap<FName, AActor*>& InPastedActors) override;
	virtual void OnActorsDuplicated(const TMap<AActor*, AActor*>& InDuplicateActorMap, FAvaOutlinerItemPtr InRelativeItem = nullptr, TOptional<EItemDropZone> InRelativeDropZone = TOptional<EItemDropZone>()) override;
	virtual void GroupSelection(AActor* InGroupingActor, const TOptional<FAttachmentTransformRules>& InTransformRules = TOptional<FAttachmentTransformRules>()) override;
	virtual void OnObjectSelectionChanged(const FAvaEditorSelection& InEditorSelection) override;
	virtual UWorld* GetWorld() const override;
	virtual const FAvaOutlinerItemProxyRegistry& GetItemProxyRegistry() const override;
	virtual bool IsDefaultWorldActorToHide(const UWorld* const InWorld, const AActor* const InActor) const override;
	//~ End IAvaOutliner

	FAvaOutlinerItemProxyRegistry& GetItemProxyRegistry();

	//~ Begin FEditorUndoClient
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FEditorUndoClient

	//~ Begin FTickableObjectBase
	virtual TStatId GetStatId() const override;
	virtual void Tick(float InDeltaTime) override;
	//~ End FTickableObjectBase

	/**
	 * Creates a new set of items in the outliner based on the given template items
	 * @param InItems the template items to use for duplication
	 * @param InRelativeItem the existing outliner item that items should use as positional reference
	 * @param InRelativeDropZone where to put the duplicate items with relation to InRelativeItem (above, below, onto)
	 */
	void DuplicateItems(TArray<FAvaOutlinerItemPtr> InItems
		, FAvaOutlinerItemPtr InRelativeItem
		, TOptional<EItemDropZone> InRelativeDropZone);

	/**
	 * Delete a set of items in the outliner by calling their custom delete handler
	 */
	void DeleteItems(TArray<FAvaOutlinerItemPtr> InItems);

	/** Unregisters the Outliner View bound to the given id */
	void UnregisterOutlinerView(int32 InOutlinerViewId);

	/** Sets the given Outliner View Id as the most recent Outliner View */
	void UpdateRecentOutlinerViews(int32 InOutlinerViewId);

	/** Gets the outliner view that was most recently used (i.e. called FAvaOutliner::UpdateRecentOutlinerViews) */
	TSharedPtr<FAvaOutlinerView> GetMostRecentOutlinerView() const;

	/** Executes the given predicate for each Outliner View registered */
	void ForEachOutlinerView(const TFunction<void(const TSharedPtr<FAvaOutlinerView>& InOutlinerView)>& InPredicate) const;

	/**
	 * Instantiates a new item action without adding it to the Pending Actions Queue.
	 * This should only be used directly when planning to enqueue multiple actions.
	 * @see FAvaOutliner::EnqueueItemActions
	 */
	template<typename InItemActionType, typename = typename TEnableIf<TIsDerivedFrom<InItemActionType, IAvaOutlinerAction>::IsDerived>::Type, typename ...InArgTypes>
	TSharedRef<InItemActionType> NewItemAction(InArgTypes&&... InArgs)
	{
		return MakeShared<InItemActionType>(Forward<InArgTypes>(InArgs)...);
	}

	/**
	 * Instantiates a single new item action and immediately adds it to the Pending Actions Queue.
	 * Ideal for when dealing with a single action.
	 * For multiple actions use FAvaOutliner::EnqueueItemActions.
	 */
	template<typename InItemActionType, typename = typename TEnableIf<TIsDerivedFrom<InItemActionType, IAvaOutlinerAction>::IsDerived>::Type, typename ...InArgTypes>
	void EnqueueItemAction(InArgTypes&&... InArgs)
	{
		EnqueueItemActions({ NewItemAction<InItemActionType>(Forward<InArgTypes>(InArgs)...) });
	}

	/** Adds the given actions to the Pending Action Queue */
	void EnqueueItemActions(TArray<TSharedPtr<IAvaOutlinerAction>>&& InItemActions) noexcept;

	/** Returns the number of actions that been added to the queue so far before triggering a refresh */
	int32 GetPendingItemActionCount() const;

	/** Returns whether the Outliner is currently in need of a Refresh */
	bool NeedsRefresh() const;

	/**
	 * Gets the color pair (color name, linear color) related to the Item
	 * @param InItem the item to query
	 * @param bRecurseParent whether to get the color of the parent (recursively) if the given item does not have a color by itself
	 * @returns the matching color pair or unset if item is invalid or no color could be found.
	 */
	TOptional<FAvaOutlinerColorPair> FindItemColor(const FAvaOutlinerItemPtr& InItem, bool bRecurseParent = true) const;

	/** Pairs the Item with the given color name, overriding the inherited color if different */
	void SetItemColor(const FAvaOutlinerItemPtr& InItem, const FName& InColorName);

	/** Removes the Color pairing of the given Item (can still have an inherited color though) */
	void RemoveItemColor(const FAvaOutlinerItemPtr& InItem);

	/** Returns the color map from the Outliner Settings */
	const TMap<FName, FLinearColor>& GetColorMap() const;

	/**
	 * Replaces the Item's Id in the Item Map. This can be due to an object item changing it's object
	 * (e.g. a bp component getting destroyed and recreated, the item should be the same but the underlying component will not be)
	 */
	void NotifyItemIdChanged(const FAvaOutlinerItemId& OldId, const FAvaOutlinerItemPtr& InItem);

	/** Returns the currently selected items in the most recent outliner view (since this list can vary between outliner views) */
	TArray<FAvaOutlinerItemPtr> GetSelectedItems() const;

	/** Returns the number of currently selected items in the most recent outliner view */
	int32 GetSelectedItemCount() const;

	/**
	 * Selects the given Items on all Outliner Views
	 * @param InItems the items to select
	 * @param InFlags how the items should be selected (appended, notify of selections, etc)
	 */
	void SelectItems(const TArray<FAvaOutlinerItemPtr>& InItems, EAvaOutlinerItemSelectionFlags InFlags = EAvaOutlinerItemSelectionFlags::SignalSelectionChange) const;

	/**
	 * Clears the Item Selection from all Outliner Views
	 * @param bSignalSelectionChange whether to notify the change in selection
	 */
	void ClearItemSelection(bool bSignalSelectionChange) const;

	/** Gets the closest item to all the given items while also being their common ancestor */
	static FAvaOutlinerItemPtr FindLowestCommonAncestor(const TArray<FAvaOutlinerItemPtr>& Items);

	/** Converts the given Outliner Item to a Scene Item that can be serialized in the Scene Tree */
	static FAvaSceneItem MakeSceneItemFromOutlinerItem(const FAvaOutlinerItemPtr& InItem);

	/**
	 * Helper function to sort the given array of items based on their ordering in the Outliner
	 * @see FAvaOutliner::CompareOutlinerItemOrder
	 */
	static void SortItems(TArray<FAvaOutlinerItemPtr>& OutOutlinerItems);

	/** Normalizes the given Items by removing selected items that have their parent item also in the selection */
	static void NormalizeItems(TArray<FAvaOutlinerItemPtr>& InOutItems);

	/** Gets the Editor Mode Tools used to handle selections */
	FEditorModeTools* GetModeTools() const;

	/** Have the given Selected Items sync to the USelection Instances of Mode Tools */
	void SyncModeToolsSelection(const TArray<FAvaOutlinerItemPtr>& InSelectedItems) const;

	/** Gets all the Actors that have as their AActor::GetSceneOutlinerParent the given InParentActor */
	TArray<TWeakObjectPtr<AActor>> GetActorSceneOutlinerChildren(AActor* InParentActor) const;

	/** Tries to add the new Actor to the Outliner if not added already and if the Outliner allows the given actor to be added */
	void OnActorSpawned(AActor* InActor);

	/** Called when an Actor has been destroyed. This enqueues the Removal the Actor Item to the Pending Action Queue */
	void OnActorDestroyed(AActor* InActor);

	/** Called when an Actor's attachment has changed. This triggers a refresh */
	void OnActorAttachmentChanged(AActor* InActor, const AActor* InParent, bool bAttach);

	/** Called when the engine replaces an object. A common example is when a BP Component is destroyed, and replaced */
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap);

	/** Called when the engine replaces an Actor */
	void OnActorReplaced(AActor* InOldActor, AActor* InNewActor);

	/** Marks the Outliner dirty. This triggers IAvaOutlinerProvider::OnOutlinerModified on next tick */
	void SetOutlinerModified();

	void QueueActorHierarchyChange(AActor* InActor, const AActor* InParentActor, EAvaOutlinerHierarchyChangeType InChangeType);

private:
	/** Binds a given worlds delegates to the outliner, unbinding the previous bound world's */
	void BindWorldDelegates(UWorld* InWorld);
	/** Unbinds the last bound world's delegates from outliner */
	void UnbindWorldDelegates();

	void AddItem(const FAvaOutlinerItemPtr& InItem);
	void RemoveItem(const FAvaOutlinerItemId& InItemId);
	void ForEachItem(TFunctionRef<void(const FAvaOutlinerItemPtr&)> InFunc);

	/** Interface providing the Outliner with logic such as Actor Duplication, Mode Tools, and extensibility options */
	IAvaOutlinerProvider& OutlinerProvider;

	/** the base command list gotten from the outliner provider to append our command list to */
	TWeakPtr<FUICommandList> BaseCommandListWeak;

	/** Latest world with bound delegates. */
	TWeakObjectPtr<UWorld> BoundWorldWeak;

	/** the map of the registered items */
	TMap<FAvaOutlinerItemId, FAvaOutlinerItemPtr> ItemMap;

	TMap<FAvaOutlinerItemId, FAvaOutlinerItemPtr> ItemsPendingAdd;

	TSet<FAvaOutlinerItemId> ItemsPendingRemove;

	/** Map of an Actor to its Child Actors defined as such via AActor::GetSceneOutlinerParent */
	TMap<TWeakObjectPtr<AActor>, TArray<TWeakObjectPtr<AActor>>> SceneOutlinerParentMap;

	/** The root of all the items in the outliner */
	TSharedRef<FAvaOutlinerTreeRoot> RootItem;

	/** Outliner's Item Proxy Factory Registry Instance. This takes precedence over the Module's Factory Registry */
	FAvaOutlinerItemProxyRegistry ItemProxyRegistry;
	
	/** the current pending actions before refresh is called */
	TArray<TSharedPtr<IAvaOutlinerAction>> PendingActions;

	/** the list of objects pending selection processing. filled in when getting notifies from the USelection instances */
	TSharedPtr<TArray<TWeakObjectPtr<UObject>>> ObjectsLastSelected;

	/** the save state object to help serialize the outliner state */
	TSharedRef<FAvaOutlinerSaveState> SaveState;

	/** the map of registered outliner views */
	TMap<int32, TSharedPtr<FAvaOutlinerView>> OutlinerViews;

	/** List of Outliner View Ids in order from least recent to most recent (i.e. Index 0 is least recent) */
	TArray<int32> RecentOutlinerViews;

	struct FActorHierarchyChange
	{
		TWeakObjectPtr<AActor> ActorWeak;
		TWeakObjectPtr<const AActor> ParentWeak;
		EAvaOutlinerHierarchyChangeType ChangeType;
	};
	TArray<FActorHierarchyChange> QueuedActorHierarchyChanges;

	/** the current events to ignore and not handle automatically */
	EAvaOutlinerIgnoreNotifyFlags IgnoreNotifyFlags = EAvaOutlinerIgnoreNotifyFlags::None;

	/** Flag indicating whether the Outliner has been changed this tick and should call IAvaOutlinerProvider::OnOutlinerModified next tick */
	bool bOutlinerDirty = false;
	
	/** Flag indicating Refreshing is taking place */
	bool bRefreshing = false;

	/** Flag indicating that a refresh must take place next tick */
	bool bRefreshRequested = false;

	/** Flag indicating that the Item Map is iterating */
	bool bIteratingItemMap = false;
};
