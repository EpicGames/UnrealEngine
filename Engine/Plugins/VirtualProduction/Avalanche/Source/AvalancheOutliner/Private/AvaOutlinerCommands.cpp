// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerCommands.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerCommands"

void FAvaOutlinerCommands::RegisterCommands()
{
	UI_COMMAND(SelectAllChildren
		, "Select All Children"
		, "Selects all the children (recursively) of each selection"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SelectImmediateChildren
		, "Select Immediate Children"
		, "Selects only the immediate children of each selection"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SelectParent
		, "Select Parent"
		, "Selects the parent item of each selection"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Shift, EKeys::Left));

	UI_COMMAND(SelectFirstChild
		, "Select First Child"
		, "Selects the first child item of each selection"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Shift, EKeys::Right));

	UI_COMMAND(SelectPreviousSibling
		, "Select Previous Sibling"
		, "Selects the previous sibling of each selection"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Shift, EKeys::Up));

	UI_COMMAND(SelectNextSibling
		, "Select Next Sibling"
		, "Selects the next sibling item of each selection"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Shift, EKeys::Down));

	UI_COMMAND(ExpandAll
		, "Expand All"
		, "Expands all items in outliner"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::RightBracket));

	UI_COMMAND(CollapseAll
		, "Collapse All"
		, "Collapses all items in outliner"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::LeftBracket));

	UI_COMMAND(ScrollNextSelectionIntoView
		, "Scroll to Next"
		, "Scrolls the next selection into view"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Alt, EKeys::N));

	UI_COMMAND(ToggleMutedHierarchy
		, "Muted Hierarchy"
		, "Show the parent of the shown items, even if the parents are filtered out"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(ToggleAutoExpandToSelection
		, "Auto Expand to Selection"
		, "Auto expand the hierarchy to show the item when selected"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Refresh
		, "Refresh"
		, "Refreshes the outliner view"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::F5));
}

#undef LOCTEXT_NAMESPACE
