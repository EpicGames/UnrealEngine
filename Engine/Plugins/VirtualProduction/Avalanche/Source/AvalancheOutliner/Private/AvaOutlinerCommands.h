// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerCommands"

class FAvaOutlinerCommands
	: public TCommands<FAvaOutlinerCommands>
{
public:
	FAvaOutlinerCommands()
		: TCommands<FAvaOutlinerCommands>(TEXT("AvaOutliner")
		, LOCTEXT("MotionDesignOutliner", "Motion Design Outliner")
		, NAME_None
		, FAppStyle::GetAppStyleSetName())
	{
	}

	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> Refresh;
	
	TSharedPtr<FUICommandInfo> SelectAllChildren;
	
	TSharedPtr<FUICommandInfo> SelectImmediateChildren;
	
	TSharedPtr<FUICommandInfo> SelectParent;
	
	TSharedPtr<FUICommandInfo> SelectFirstChild;

	TSharedPtr<FUICommandInfo> SelectPreviousSibling;
	
	TSharedPtr<FUICommandInfo> SelectNextSibling;

	TSharedPtr<FUICommandInfo> ExpandAll;
	
	TSharedPtr<FUICommandInfo> CollapseAll;

	TSharedPtr<FUICommandInfo> ExpandSelection;
	
	TSharedPtr<FUICommandInfo> CollapseSelection;

	TSharedPtr<FUICommandInfo> ScrollNextSelectionIntoView;
	
	TSharedPtr<FUICommandInfo> ToggleMutedHierarchy;

	TSharedPtr<FUICommandInfo> ToggleAutoExpandToSelection;
};

#undef LOCTEXT_NAMESPACE
