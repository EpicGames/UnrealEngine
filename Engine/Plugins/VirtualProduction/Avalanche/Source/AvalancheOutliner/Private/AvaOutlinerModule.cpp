// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerModule.h"
#include "AvaOutlinerCommands.h"
#include "AvaOutlinerStyle.h"
#include "Filters/Expressions/AvaFilterColorExpressionFactory.h"
#include "Filters/Expressions/AvaFilterNameExpressionFactory.h"
#include "Filters/Expressions/AvaFilterTagExpressionFactory.h"
#include "Filters/Expressions/AvaFilterTypeExpressionFactory.h"
#include "Filters/Suggestions/AvaFilterColorSuggestionFactory.h"
#include "Filters/Suggestions/AvaFilterNameSuggestionFactory.h"
#include "Filters/Suggestions/AvaFilterTagSuggestionFactory.h"
#include "Filters/Suggestions/AvaFilterTypeSuggestionFactory.h"
#include "Item/AvaOutlinerComponent.h"
#include "Item/AvaOutlinerComponentProxy.h"
#include "Item/AvaOutlinerMaterial.h"
#include "Item/AvaOutlinerMaterialProxy.h"
#include "ItemProxies/Factories/AvaOutlinerItemProxyDefaultFactory.h"
#include "Misc/TextFilterUtils.h"
#include "Modules/ModuleManager.h"

void FAvaOutlinerModule::StartupModule()
{
	FAvaOutlinerStyle::Get();
	FAvaOutlinerCommands::Register();
	ItemProxyRegistry.RegisterItemProxyWithDefaultFactory<FAvaOutlinerMaterialProxy, 10>();
	ItemProxyRegistry.RegisterItemProxyWithDefaultFactory<FAvaOutlinerComponentProxy, 20>();

	RegisterFilterExpressionFactories();
	RegisterFilterSuggestionFactories();
}

void FAvaOutlinerModule::ShutdownModule()
{
	FAvaOutlinerCommands::Unregister();
}

bool FAvaOutlinerModule::CanFilterSupportComparisonOperation(const FName& InFilterKey, const ETextFilterComparisonOperation InOperation) const
{
	if (const TSharedPtr<IAvaFilterExpressionFactory>* FilterExpressionFactory = FilterExpressionFactories.Find(InFilterKey))
	{
		return FilterExpressionFactory->Get()->SupportsComparisonOperation(InOperation);
	}
	return false;
}

bool FAvaOutlinerModule::FilterExpression(const FName& InFilterKey, const IAvaOutlinerItem& InItem, const FAvaTextFilterArgs& InArgs) const
{
	if (const TSharedPtr<IAvaFilterExpressionFactory>* FilterExpressionFactory = FilterExpressionFactories.Find(InFilterKey))
	{
		return FilterExpressionFactory->Get()->FilterExpression(InItem, InArgs);
	}
	return false;
}

TArray<TSharedPtr<IAvaFilterSuggestionFactory>> FAvaOutlinerModule::GetSuggestions(const EAvaFilterSuggestionType& InSuggestionType) const
{
	TArray<TSharedPtr<IAvaFilterSuggestionFactory>> OutArray;

	for (const TPair<FName, TSharedPtr<IAvaFilterSuggestionFactory>>& Suggestion : FilterSuggestionsFactories)
	{
		if ((Suggestion.Value->GetSuggestionType() & InSuggestionType) != EAvaFilterSuggestionType::None)
		{
			OutArray.Add(Suggestion.Value);
		}
	}

	return OutArray;
}

FSlateIcon FAvaOutlinerModule::FindOverrideIcon(TSharedPtr<const FAvaOutlinerItem> InItem) const
{
	if (InItem.IsValid())
	{
		const TSharedPtr<IAvaOutlinerIconCustomization> CustomizationToUse = GetCustomizationForItem(InItem);

		if (CustomizationToUse.IsValid())
		{
			return CustomizationToUse->GetOverrideIcon(InItem);
		}
	}
	return FSlateIcon();
}

void FAvaOutlinerModule::UnregisterOverriddenIcon_Internal(const FAvaTypeId& InItemTypeId, const FName& InSpecializationIdentifier)
{
	FIconCustomizationKey Key;
	Key.ItemTypeId = InItemTypeId;
	Key.CustomizationSpecializationIdentifier = InSpecializationIdentifier;

	IconRegistry.Remove(Key);
}

void FAvaOutlinerModule::RegisterOverridenIcon_Internal(const FAvaTypeId& InItemTypeId, const TSharedRef<IAvaOutlinerIconCustomization>& InAvaOutlinerIconCustomization)
{
	FIconCustomizationKey Key;
	Key.ItemTypeId = InItemTypeId;
	Key.CustomizationSpecializationIdentifier = InAvaOutlinerIconCustomization->GetOutlinerItemIdentifier();

	if (!IconRegistry.Contains(Key))
	{
		IconRegistry.Add(Key, InAvaOutlinerIconCustomization);
	}
}

void FAvaOutlinerModule::RegisterFilterExpressionFactory_Internal(const TSharedRef<IAvaFilterExpressionFactory>& InFilterExpressionFactory)
{
	if (!FilterExpressionFactories.Contains(InFilterExpressionFactory->GetFilterIdentifier()))
	{
		FilterExpressionFactories.Add(InFilterExpressionFactory->GetFilterIdentifier(), InFilterExpressionFactory);
	}
}

void FAvaOutlinerModule::RegisterFilterSuggestionFactory_Internal(const TSharedRef<IAvaFilterSuggestionFactory>& InFilterSuggestionFactory)
{
	if (!FilterSuggestionsFactories.Contains(InFilterSuggestionFactory->GetSuggestionIdentifier()))
	{
		FilterSuggestionsFactories.Add(InFilterSuggestionFactory->GetSuggestionIdentifier(), InFilterSuggestionFactory);
	}
}

TSharedPtr<IAvaOutlinerIconCustomization> FAvaOutlinerModule::GetCustomizationForItem(const TSharedPtr<const FAvaOutlinerItem>& InItem) const
{
	// There is gonna always be just one icon customization that support the item since it checks specific thing for the SupportCustomization
	for (const TPair<FIconCustomizationKey, TSharedPtr<IAvaOutlinerIconCustomization>>& IconCustomization : IconRegistry)
	{
		if (IconCustomization.Key.ItemTypeId == InItem->GetTypeId() && IconCustomization.Value->HasOverrideIcon(InItem))
		{
			return IconCustomization.Value;
		}
	}

	return nullptr;
}

void FAvaOutlinerModule::RegisterFilterExpressionFactories()
{
	RegisterFilterExpressionFactory<FAvaFilterNameExpressionFactory>();
	RegisterFilterExpressionFactory<FAvaFilterTypeExpressionFactory>();
	RegisterFilterExpressionFactory<FAvaFilterTagExpressionFactory>();
	RegisterFilterExpressionFactory<FAvaFilterColorExpressionFactory>();
}

void FAvaOutlinerModule::RegisterFilterSuggestionFactories()
{
	RegisterFilterSuggestionFactory<FAvaFilterNameSuggestionFactory>();
	RegisterFilterSuggestionFactory<FAvaFilterTypeSuggestionFactory>();
	RegisterFilterSuggestionFactory<FAvaFilterTagSuggestionFactory>();
	RegisterFilterSuggestionFactory<FAvaFilterColorSuggestionFactory>();
}

IMPLEMENT_MODULE(FAvaOutlinerModule, AvalancheOutliner)
