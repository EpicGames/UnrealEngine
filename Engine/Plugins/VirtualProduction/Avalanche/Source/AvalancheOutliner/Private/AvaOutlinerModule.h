// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/IAvaFilterExpressionFactory.h"
#include "Filters/IAvaFilterSuggestionFactory.h"
#include "IAvaOutlinerModule.h"
#include "ItemProxies/AvaOutlinerItemProxyRegistry.h"

class IAvaFilterExpressionFactory;

class FAvaOutlinerModule : public IAvaOutlinerModule
{
public:
	static FAvaOutlinerModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FAvaOutlinerModule>(TEXT("AvalancheOutliner"));
	};

	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	//~ Begin IAvaOutlinerModule
	virtual FOnExtendOutlinerToolMenu& GetOnExtendOutlinerItemContextMenu() override { return OnExtendOutlinerItemContextMenu; }
	virtual FAvaOutlinerItemProxyRegistry& GetItemProxyRegistry() override { return ItemProxyRegistry; }
	virtual FOnExtendItemProxiesForItem& GetOnExtendItemProxiesForItem() override { return OnGetItemProxiesForItem; }
	//~ End IAvaOutlinerModule

	/**
 	 * Create and Register the given filter expression factory
 	 * @tparam InFilterExpressionFactoryType The type of filter expression factory to create and register
 	 * @tparam InArgsType Constructor args types if needed
 	 * @param InArgs Additional args passed to the constructor of the filter expression factory if needed
 	 */
	template<typename InFilterExpressionFactoryType, typename... InArgsType
			UE_REQUIRES(TIsDerivedFrom<InFilterExpressionFactoryType, IAvaFilterExpressionFactory>::Value)>
	void RegisterFilterExpressionFactory(InArgsType&&... InArgs)
	{
		RegisterFilterExpressionFactory_Internal(IAvaFilterExpressionFactory::MakeInstance<InFilterExpressionFactoryType>(Forward<InArgsType>(InArgs)...));
	}

	/**
  	 * Create and Register the given filter suggestion factory
  	 * @tparam InSuggestionFactoryType The type of filter suggestion factory to create and register
  	 * @tparam InArgsType Constructor args types if needed
  	 * @param InArgs Additional args passed to the constructor of the filter suggestion factory if needed
  	 */
	template<typename InSuggestionFactoryType, typename... InArgsType
			UE_REQUIRES(TIsDerivedFrom<InSuggestionFactoryType, IAvaFilterSuggestionFactory>::Value)>
	void RegisterFilterSuggestionFactory(InArgsType&&... InArgs)
	{
		RegisterFilterSuggestionFactory_Internal(IAvaFilterSuggestionFactory::MakeInstance<InSuggestionFactoryType>(Forward<InArgsType>(InArgs)...));
	}

	/**
	 * Check if current filter expression factory support the comparison operation
	 * @param InFilterKey Filter key to get the filter expression factory needed
	 * @param InOperation Operation to check if supported
	 * @return True if operation is supported, False otherwise
	 */
	bool CanFilterSupportComparisonOperation(const FName& InFilterKey, const ETextFilterComparisonOperation InOperation) const;

	/**
	 * Evaluate the expression and return the result 
	 * @param InFilterKey Filter Key to get the Factory
	 * @param InItem Item that is currently checked
	 * @param InArgs Args to evaluate the expression see FAvaTextFilterArgs for more information
	 * @return True if the expression evaluated to True, False otherwise
	 */
	bool FilterExpression(const FName& InFilterKey, const IAvaOutlinerItem& InItem, const FAvaTextFilterArgs& InArgs) const;

	/**
	 * Get all suggestions with the given type (Generic/ItemBased/All)
	 * @param InSuggestionType Type of suggestion to get, see EFilterSuggestionType for more information
	 * @return An Array containing all suggestions of the given type
	 */
	TArray<TSharedPtr<IAvaFilterSuggestionFactory>> GetSuggestions(const EAvaFilterSuggestionType& InSuggestionType) const;

	/**
	 * Get the custom icon for the item if a customization for it exist
	 * @param InItem The item to search the icon for
	 * @return The custom icon for the item if a customization is found, FSlateIcon() otherwise
	 */
	FSlateIcon FindOverrideIcon(TSharedPtr<const FAvaOutlinerItem> InItem) const;

protected:
	//~ Begin IAvaOutlinerModule
	virtual void UnregisterOverriddenIcon_Internal(const FAvaTypeId& InItemTypeId, const FName& InSpecializationIdentifier) override;
	virtual void RegisterOverridenIcon_Internal(const FAvaTypeId& InItemTypeId, const TSharedRef<IAvaOutlinerIconCustomization>& InAvaOutlinerIconCustomization) override;
	//~ End IAvaOutlinerModule

private:
	/**
	 * Adds the given factory to the Map containing all of them if not already added.
	 * @param InFilterExpressionFactory Filter expression factory to register.
	 */
	void RegisterFilterExpressionFactory_Internal(const TSharedRef<IAvaFilterExpressionFactory>& InFilterExpressionFactory);

	/**
	 * Adds the given factory to the Map containing all of them if not already added.
	 * @param InFilterSuggestionFactory Filter suggestion factory to register.
	 */
	void RegisterFilterSuggestionFactory_Internal(const TSharedRef<IAvaFilterSuggestionFactory>& InFilterSuggestionFactory) ;

	/**
	 * Get the customization for the given item if any
	 * @param InItem Item to search the customization for
	 * @return The customization for the item if any, nullptr otherwise
	 */
	TSharedPtr<IAvaOutlinerIconCustomization> GetCustomizationForItem(const TSharedPtr<const FAvaOutlinerItem>& InItem) const;

	/**
	 * Registers default filter expression factories.
	 */
	void RegisterFilterExpressionFactories();

	/**
	  * Registers default filter suggestion factories.
	  */
	void RegisterFilterSuggestionFactories();

	/** Delegate to extend the Item Context Menu */
	FOnExtendOutlinerToolMenu OnExtendOutlinerItemContextMenu;

	/** Delegate to add in Item Proxies for a given Item / Item Type */
	FOnExtendItemProxiesForItem OnGetItemProxiesForItem;

	/** Module Instance of the Item Registry */
	FAvaOutlinerItemProxyRegistry ItemProxyRegistry;

	/** Holds all the FilterExpressionFactory */
	TMap<FName, TSharedPtr<IAvaFilterExpressionFactory>> FilterExpressionFactories;

	/** Holds all the FilterSuggestionFactory */
	TMap<FName, TSharedPtr<IAvaFilterSuggestionFactory>> FilterSuggestionsFactories;

	/**
	 * Hold the key of the map containing the IconCustomizations
	 */
	struct FIconCustomizationKey
	{
		/** OutlinerClass FName (ex. AvaOutlinerItem) */
		FAvaTypeId ItemTypeId = FAvaTypeId::Invalid();

		/** Specialization identifier (ex. AvaShapeActor class FName for ActorIcon customization) */
		FName CustomizationSpecializationIdentifier;

		bool operator==(const FIconCustomizationKey& InOther) const
		{
			return ItemTypeId == InOther.ItemTypeId
				&& CustomizationSpecializationIdentifier == InOther.CustomizationSpecializationIdentifier;
		}

		friend uint32 GetTypeHash(FIconCustomizationKey const& InCustomizationKey)
		{
			return HashCombine(GetTypeHash(InCustomizationKey.ItemTypeId)
				, GetTypeHash(InCustomizationKey.CustomizationSpecializationIdentifier));
		}
	};

	/** Holds all the IconCustomizations */
	TMap<FIconCustomizationKey, TSharedPtr<IAvaOutlinerIconCustomization>> IconRegistry;
};
