// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerSettings.h"
#include "Styling/StyleColors.h"
#include "UObject/PropertyIterator.h"

UAvaOutlinerSettings::UAvaOutlinerSettings()
{
	CategoryName = TEXT("Motion Design");
	SectionName  = TEXT("Outliner");
	
	// Default Item View Modes
	ItemProxyViewMode   = static_cast<int32>(EAvaOutlinerItemViewMode::None);
	ItemDefaultViewMode = static_cast<int32>(EAvaOutlinerItemViewMode::HorizontalItemList);
	
	// Default Colors
	ItemColorMap.Emplace(TEXT("Blue"),   FStyleColors::AccentBlue.GetSpecifiedColor());
	ItemColorMap.Emplace(TEXT("Purple"), FStyleColors::AccentPurple.GetSpecifiedColor());
	ItemColorMap.Emplace(TEXT("Pink"),   FStyleColors::AccentPink.GetSpecifiedColor());
	ItemColorMap.Emplace(TEXT("Red"),    FStyleColors::AccentRed.GetSpecifiedColor());
	ItemColorMap.Emplace(TEXT("Orange"), FStyleColors::AccentOrange.GetSpecifiedColor());
	ItemColorMap.Emplace(TEXT("Yellow"), FStyleColors::AccentYellow.GetSpecifiedColor());
	ItemColorMap.Emplace(TEXT("Green"),  FStyleColors::AccentGreen.GetSpecifiedColor());
	ItemColorMap.Emplace(TEXT("Brown"),  FStyleColors::AccentBrown.GetSpecifiedColor());
	ItemColorMap.Emplace(TEXT("Black"),  FStyleColors::AccentBlack.GetSpecifiedColor());
	ItemColorMap.Emplace(TEXT("Gray"),   FStyleColors::AccentGray.GetSpecifiedColor());
	ItemColorMap.Emplace(TEXT("White"),  FStyleColors::AccentWhite.GetSpecifiedColor());
}

UAvaOutlinerSettings* UAvaOutlinerSettings::Get()
{
	UAvaOutlinerSettings* DefaultSettings = GetMutableDefault<UAvaOutlinerSettings>();
	static bool bInitialized = false;
	if (!bInitialized)
	{
		bInitialized = true;
		DefaultSettings->SetFlags(RF_Transactional);
	}
	return DefaultSettings;
}

FName UAvaOutlinerSettings::GetCustomItemTypeFiltersName()
{
	return GET_MEMBER_NAME_CHECKED(UAvaOutlinerSettings, CustomItemTypeFilters);
}

bool UAvaOutlinerSettings::AddCustomItemTypeFilter(FName InKey, FAvaOutlinerItemTypeFilterData& InFilter)
{
	if (CustomItemTypeFilters.Contains(InKey))
	{
		return false;
	}
	
	CustomItemTypeFilters.Add(InKey, InFilter);
	return true;
}

void UAvaOutlinerSettings::PostInitProperties()
{
	Super::PostInitProperties();
	HideItemColorMapAlphaChannel();
}

void UAvaOutlinerSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UAvaOutlinerSettings, ItemColorMap))
	{
		//Force all Item Color Map entries to have Alpha value of 1
		for (TPair<FName, FLinearColor>& Pair : ItemColorMap)
		{
			Pair.Value.A = 1.f;
		}
	}
}

void UAvaOutlinerSettings::HideItemColorMapAlphaChannel() const
{
	const FMapProperty* ItemColorMapProperty = nullptr;
	for (TFieldIterator<FProperty> PropertyIt(StaticClass()); PropertyIt; ++PropertyIt)
	{
		FProperty* const Property = CastField<FProperty>(*PropertyIt);
		if (Property->IsA<FMapProperty>() && Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAvaOutlinerSettings, ItemColorMap))
		{
			ItemColorMapProperty = CastField<FMapProperty>(Property);
			break;
		}
	}
	if (ItemColorMapProperty && ItemColorMapProperty->ValueProp)
	{
		ItemColorMapProperty->ValueProp->SetMetaData(TEXT("HideAlphaChannel"), TEXT("true"));
	}
}
