// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Filters/AvaOutlinerItemTypeFilter.h"
#include "AvaOutlinerSettings.generated.h"

UCLASS(config=EditorPerProjectUserSettings, meta=(DisplayName="Outliner"))
class UAvaOutlinerSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UAvaOutlinerSettings();

	static UAvaOutlinerSettings* Get();

	const TMap<FName, FLinearColor>& GetColorMap() const { return ItemColorMap; }

	static FName GetCustomItemTypeFiltersName();

	const TMap<FName, FAvaOutlinerItemTypeFilterData>& GetCustomItemTypeFilters() const { return CustomItemTypeFilters; }

	bool AddCustomItemTypeFilter(FName InKey, FAvaOutlinerItemTypeFilterData& InFilter);

	bool ShouldUseMutedHierarchy() const { return bUseMutedHierarchy; }

	bool ShouldAutoExpandToSelection() const { return bAutoExpandToSelection; }

	bool ShouldAlwaysShowVisibilityState() const { return bAlwaysShowVisibilityState; }

	bool ShouldAlwaysShowLockState() const { return bAlwaysShowLockState; }

	EAvaOutlinerItemViewMode GetItemDefaultViewMode() const { return static_cast<EAvaOutlinerItemViewMode>(ItemDefaultViewMode); }

	EAvaOutlinerItemViewMode GetItemProxyViewMode() const { return static_cast<EAvaOutlinerItemViewMode>(ItemProxyViewMode); }

private:
	virtual void PostInitProperties() override;

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/**
	 * Hides the Alpha Channel for the Value Color Property in ItemColorMap
	 * this has to be done manually as putting in meta=(HideAlphaChannel) in a map doesn't apply to the Value Property
	 */
	void HideItemColorMapAlphaChannel() const;

	UPROPERTY(Config, EditAnywhere, Category = "Color")
	TMap<FName, FLinearColor> ItemColorMap;

	UPROPERTY(Config, EditAnywhere, Category = "Filters")
	TMap<FName, FAvaOutlinerItemTypeFilterData> CustomItemTypeFilters;

	/** Whether to show the visibility state always, rather than only showing when the item is hidden or hovered */
	UPROPERTY(Config, EditAnywhere, Category = "Editor UX")
	bool bAlwaysShowVisibilityState = false;

	/** Whether to show the lock state always, rather than only showing when the item is locked or hovered */
	UPROPERTY(Config, EditAnywhere, Category = "Editor UX")
	bool bAlwaysShowLockState = false;

	/** Whether to show the parent of the shown items, even if the parents are filtered out */
	UPROPERTY(Config, EditAnywhere, Category = "Initial Values")
	bool bUseMutedHierarchy = true;

	/** Whether to auto expand the hierarchy to show the item when selected */
	UPROPERTY(Config, EditAnywhere, Category = "Initial Values")
	bool bAutoExpandToSelection = true;

	/** The View Mode a Non-Actor / Non-Component Item supports by default */
	UPROPERTY(Config, EditAnywhere, Category = "Initial Values", meta=(Bitmask, BitmaskEnum="/Script/AvalancheOutliner.EAvaOutlinerItemViewMode"))
	int32 ItemDefaultViewMode;

	/** The View Mode a Proxy Item supports by default */
	UPROPERTY(Config, EditAnywhere, Category = "Initial Values", meta=(Bitmask, BitmaskEnum="/Script/AvalancheOutliner.EAvaOutlinerItemViewMode"))
	int32 ItemProxyViewMode;
};
