// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerStyle.h"
#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Brushes/SlateNoResource.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/ToolBarStyle.h"

FAvaOutlinerStyle::FAvaOutlinerStyle()
	: FSlateStyleSet(TEXT("AvaOutliner"))
{
	const FVector2D Icon20x20(20.0f, 20.0f);

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	SetContentRoot(FPaths::Combine(Plugin->GetBaseDir(), TEXT("Resources")));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
	
	Set("AvaOutliner.FilterIcon", new IMAGE_BRUSH("Icons/OutlinerIcons/FilterIcon", Icon20x20));
	
	// Table View Row Style
	const FTableRowStyle& TableRowStyle = FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow");
	Set("AvaOutliner.TableViewRow", FTableRowStyle(TableRowStyle)
		.SetEvenRowBackgroundBrush(FSlateNoResource())
		.SetOddRowBackgroundBrush(FSlateColorBrush(FStyleColors::Background)));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FAvaOutlinerStyle::~FAvaOutlinerStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
