// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FAvaOutlinerStyle final : public FSlateStyleSet
{
public:
	static FAvaOutlinerStyle& Get()
	{
		static FAvaOutlinerStyle Instance;
		return Instance;
	}

	FAvaOutlinerStyle();
	virtual ~FAvaOutlinerStyle() override;
};
