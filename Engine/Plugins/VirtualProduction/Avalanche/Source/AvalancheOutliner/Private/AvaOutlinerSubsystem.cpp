// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerSubsystem.h"
#include "AvaOutliner.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"

DEFINE_LOG_CATEGORY_STATIC(LogAvaOutlinerSubsystem, Log, All);

TSharedRef<IAvaOutliner> UAvaOutlinerSubsystem::GetOrCreateOutliner(IAvaOutlinerProvider& InProvider, bool bInForceCreate)
{
	TSharedPtr<IAvaOutliner> Outliner = OutlinerWeak.Pin();
	if (!Outliner.IsValid() || &StaticCastSharedPtr<FAvaOutliner>(Outliner)->GetProvider() != &InProvider || bInForceCreate)
	{
		Outliner = MakeShared<FAvaOutliner>(InProvider);
	}
	OutlinerWeak = Outliner;
	return Outliner.ToSharedRef();
}

void UAvaOutlinerSubsystem::BroadcastActorHierarchyChanged(AActor* InActor
	, const AActor* InParentActor
	, EAvaOutlinerHierarchyChangeType InChangeType) const
{
	static const UEnum* const ChangeTypeEnumClass = StaticEnum<EAvaOutlinerHierarchyChangeType>();
	UE_LOGF(LogAvaOutlinerSubsystem, Verbose, "Actor Hierarchy Changed: (Child: %ls)  (Parent: %ls)  Change Type: %ls"
		, InActor ? *InActor->GetName() : TEXT("invalid")
		, InParentActor ? *InParentActor->GetName() : TEXT("invalid")
		, *ChangeTypeEnumClass->GetNameByValue(static_cast<int64>(InChangeType)).ToString());

	switch (InChangeType)
	{
	case EAvaOutlinerHierarchyChangeType::Attached:
		GEngine->BroadcastLevelActorAttached(InActor, InParentActor);
		break;

	case EAvaOutlinerHierarchyChangeType::Detached:
		GEngine->BroadcastLevelActorDetached(InActor, InParentActor);
		break;
	}

	ActorHierarchyChangedEvent.Broadcast(InActor, InParentActor, InChangeType);
}

bool UAvaOutlinerSubsystem::DoesSupportWorldType(const EWorldType::Type InWorldType) const
{
	return InWorldType == EWorldType::Type::Editor;
}
