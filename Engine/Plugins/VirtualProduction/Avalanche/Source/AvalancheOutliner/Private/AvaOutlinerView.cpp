// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerView.h"
#include "Algo/ForEach.h"
#include "AvaOutliner.h"
#include "AvaOutlinerCommands.h"
#include "AvaOutlinerSettings.h"
#include "Columns/AvaOutlinerColorColumn.h"
#include "Columns/AvaOutlinerColumnExtender.h"
#include "Columns/AvaOutlinerItemsColumn.h"
#include "Columns/AvaOutlinerLabelColumn.h"
#include "Columns/AvaOutlinerLockColumn.h"
#include "Columns/AvaOutlinerTagColumn.h"
#include "Columns/AvaOutlinerVisibilityColumn.h"
#include "Data/AvaOutlinerSaveState.h"
#include "DragDropOps/AvaOutlinerItemDragDropOp.h"
#include "Filters/AvaOutlinerItemTypeFilter.h"
#include "Filters/AvaOutlinerTextFilter.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IAvaOutlinerModule.h"
#include "IAvaOutlinerProvider.h"
#include "Item/AvaOutlinerActor.h"
#include "Item/AvaOutlinerComponent.h"
#include "Item/AvaOutlinerComponentProxy.h"
#include "Item/AvaOutlinerTreeRoot.h"
#include "Menu/AvaOutlinerViewToolbarContext.h"
#include "Slate/SAvaOutliner.h"
#include "Slate/SAvaOutlinerTreeView.h"
#include "Stats/AvaOutlinerStats.h"
#include "Styling/SlateTypes.h"
#include "ToolMenu.h"
#include "ToolMenuContext/AvaOutlinerItemsContext.h"
#include "ToolMenus.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerView"

FAvaOutlinerView::FAvaOutlinerView(FPrivateToken)
	: TextFilter(MakeShared<FAvaOutlinerTextFilter>())
	, OutlinerStats(MakeShared<FAvaOutlinerStats>())
{
	HideItemType<FAvaOutlinerComponentProxy>();
}

FAvaOutlinerView::~FAvaOutlinerView()
{
	SaveState();

	if (OutlinerWidgetContainer.IsValid())
	{
		OutlinerWidgetContainer->ClearContent();

		const int32 OutlinerWidgetReferenceCount = OutlinerWidget.GetSharedReferenceCount();
		ensureMsgf(OutlinerWidgetReferenceCount == 1, TEXT("Outliner Widget expected to have 1 shared reference count but had %d"), OutlinerWidgetReferenceCount);
	}

	OutlinerWidget.Reset();

	if (UObjectInitialized())
	{
		if (UAvaOutlinerSettings* const OutlinerSettings = UAvaOutlinerSettings::Get())
		{
			OutlinerSettings->OnSettingChanged().RemoveAll(this);
		}
	}
}

FName FAvaOutlinerView::GetOutlinerToolbarName()
{
	static const FName ToolBarName(TEXT("AvalancheOutliner.MainToolBar"));
	return ToolBarName;
}

FName FAvaOutlinerView::GetOutlinerItemContextMenuName()
{
	static const FName ItemContextMenu(TEXT("AvalancheOutliner.ItemContextMenu"));
	return ItemContextMenu;
}

void FAvaOutlinerView::Init(const TSharedRef<FAvaOutliner>& InOutliner, bool bCreateOutlinerWidget)
{
	OutlinerWeak = InOutliner;

	LoadState();

	CreateColumns();

	InOutliner->GetProvider().ExtendOutlinerItemFilters(ItemFilters);
	UAvaOutlinerSettings::Get()->OnSettingChanged().AddSP(this, &FAvaOutlinerView::OnOutlinerSettingsChanged);
	UpdateCustomFilters();

	TextFilter->OnChanged().AddSP(this, &FAvaOutlinerView::OnFilterChanged);

	if (bCreateOutlinerWidget)
	{
		OutlinerWidgetContainer = SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.Padding(0)
			[
				SAssignNew(OutlinerWidget, SAvaOutliner, SharedThis(this))
			];

		CreateToolbar();
	}

	UpdateRecentOutlinerViews();
}

void FAvaOutlinerView::CreateColumns()
{
	FAvaOutlinerColumnExtender ColumnExtender;

	// Default Columns
	ColumnExtender.AddColumn<FAvaOutlinerEditorVisibilityColumn>();
	ColumnExtender.AddColumn<FAvaOutlinerRuntimeVisibilityColumn>();
	ColumnExtender.AddColumn<FAvaOutlinerLockColumn>();
	ColumnExtender.AddColumn<FAvaOutlinerColorColumn>();
	ColumnExtender.AddColumn<FAvaOutlinerLabelColumn>();
	ColumnExtender.AddColumn<FAvaOutlinerItemsColumn>();
	ColumnExtender.AddColumn<FAvaOutlinerTagColumn>();

	const TArray<TSharedPtr<IAvaOutlinerColumn>>& FoundColumns = ColumnExtender.GetColumns();
	
	Columns.Empty(FoundColumns.Num());

	for (const TSharedPtr<IAvaOutlinerColumn>& Column : FoundColumns)
	{
		const FName ColumnId = Column->GetColumnId();
		Columns.Add(ColumnId, Column);
	}
}

TSharedRef<FAvaOutlinerView> FAvaOutlinerView::CreateInstance(int32 InOutlinerViewId
	, const TSharedRef<FAvaOutliner>& InOutliner
	, bool bCreateOutlinerWidget)
{
	TSharedRef<FAvaOutlinerView> Instance = MakeShared<FAvaOutlinerView>(FAvaOutlinerView::FPrivateToken{});
	Instance->OutlinerViewId = InOutlinerViewId;
	Instance->Init(InOutliner, bCreateOutlinerWidget);
	return Instance;
}

void FAvaOutlinerView::PostLoad()
{
	if (OutlinerWidget.IsValid())
	{
		OutlinerWidget->ReconstructColumns();
	}

	if (TSharedPtr<FAvaOutliner> Outliner = GetOutliner())
	{
		const TSet<FName> RegisteredTypeNames(Outliner->GetRegisteredItemProxyTypeNames());

		// There's no user interface to hide/show any item other than the Registered Item Proxies (In View Options in Toolbar)
		// To avoid hiding an Item Type and having no visual cue for this, remove all the loaded Hidden Item Types that are not in this list
		for (TSet<FName>::TIterator Iter(HiddenItemTypes); Iter; ++Iter)
		{
			if (!RegisteredTypeNames.Contains(*Iter))
			{
				Iter.RemoveCurrent();
			}
		}
	}
}

void FAvaOutlinerView::OnOutlinerSettingsChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	const FName PropertyName = InPropertyChangedEvent.GetMemberPropertyName();
	if (PropertyName == UAvaOutlinerSettings::GetCustomItemTypeFiltersName())
	{
		UpdateCustomFilters();
	}
	RefreshOutliner(false);
}

void FAvaOutlinerView::UpdateCustomFilters()
{
	UAvaOutlinerSettings* const OutlinerSettings = UAvaOutlinerSettings::Get();
	check(OutlinerSettings);
		
	const TMap<FName, FAvaOutlinerItemTypeFilterData>& CustomFilterMap = OutlinerSettings->GetCustomItemTypeFilters();

	TSet<FName> EnabledFilterIds;
		
	//If currently enabled, these must be turned off, but save id to enable the new instances
	for (const TSharedPtr<IAvaOutlinerItemFilter>& CustomItemFilter : CustomItemFilters)
	{
		if (IsItemFilterEnabled(CustomItemFilter))
		{
			EnabledFilterIds.Add(CustomItemFilter->GetFilterId());
			DisableItemFilter(CustomItemFilter, false); //don't refresh outliner here as we do it at the end
		}
	}
		
	CustomItemFilters.Reset(CustomFilterMap.Num());
		
	for (const TPair<FName, FAvaOutlinerItemTypeFilterData>& Pair : CustomFilterMap)
	{
		const FAvaOutlinerItemTypeFilterData& FilterData = Pair.Value;
		
		if (!FilterData.HasValidFilterData())
		{
			continue;
		}

		TSharedPtr<IAvaOutlinerItemFilter> CustomItemFilter = MakeShared<FAvaOutlinerItemTypeFilter>(Pair.Key, FilterData);
		CustomItemFilters.Emplace(CustomItemFilter);

		//activate the new instance if the previous one was active
		if (EnabledFilterIds.Contains(CustomItemFilter->GetFilterId()))
		{
			EnableItemFilter(CustomItemFilter, false); //don't refresh outliner here as we do it at the end
		}
	}
		
	OnCustomFiltersChanged.Broadcast();
	Refresh();
}

void FAvaOutlinerView::Tick(float InDeltaTime)
{
	if (bRefreshRequested)
	{
		bRefreshRequested = false;
		Refresh();
	}

	for (TPair<FName, TSharedPtr<IAvaOutlinerColumn>>& Pair : Columns)
	{
		if (const TSharedPtr<IAvaOutlinerColumn>& Column = Pair.Value)
		{
			Column->Tick(InDeltaTime);
		}
	}

	//Check if we have pending items to rename and we are not currently renaming an item
	if (bRenamingItems && ItemsRemainingRename.Num() > 0 && !CurrentItemRenaming.IsValid())
	{
		CurrentItemRenaming = ItemsRemainingRename[0];
		ItemsRemainingRename.RemoveAt(0);

		if (CurrentItemRenaming.IsValid())
		{
			CurrentItemRenaming->OnRenameAction().AddSP(this, &FAvaOutlinerView::OnItemRenameAction);
			CurrentItemRenaming->OnRenameAction().Broadcast(EAvaOutlinerRenameAction::Requested, SharedThis(this));
		}
	}

	if (bRequestedRename)
	{
		bRequestedRename = false;
		RenameSelected();
	}
	
	OutlinerStats->Tick(*this);
}

void FAvaOutlinerView::SaveState()
{
	if (TSharedPtr<FAvaOutliner> Outliner = OutlinerWeak.Pin())
	{
		Outliner->GetSaveState()->SaveOutlinerViewState(*Outliner, *this);
	}
}

void FAvaOutlinerView::LoadState()
{
	if (TSharedPtr<FAvaOutliner> Outliner = OutlinerWeak.Pin())
	{
		Outliner->GetSaveState()->LoadOutlinerViewState(*Outliner, *this);
	}
}

void FAvaOutlinerView::BindCommands(const TSharedPtr<FUICommandList>& InBaseCommandList)
{
	const FGenericCommands& GenericCommands = FGenericCommands::Get();
	const FAvaOutlinerCommands& OutlinerCommands = FAvaOutlinerCommands::Get();

	ViewCommandList = MakeShared<FUICommandList>();

	if (InBaseCommandList.IsValid())
	{
		InBaseCommandList->Append(ViewCommandList.ToSharedRef());
	}

	ViewCommandList->MapAction(GenericCommands.Rename
		, FExecuteAction::CreateSP(this, &FAvaOutlinerView::RenameSelected)
		, FCanExecuteAction::CreateSP(this, &FAvaOutlinerView::CanRenameSelected));

	ViewCommandList->MapAction(GenericCommands.Delete
		, FExecuteAction::CreateSP(this, &FAvaOutlinerView::DeleteSelected)
		, FCanExecuteAction::CreateSP(this, &FAvaOutlinerView::CanDeleteSelected));

	ViewCommandList->MapAction(GenericCommands.Duplicate
		, FExecuteAction::CreateSP(this, &FAvaOutlinerView::DuplicateSelected)
		, FCanExecuteAction::CreateSP(this, &FAvaOutlinerView::CanDuplicateSelected));

	ViewCommandList->MapAction(OutlinerCommands.SelectAllChildren
		, FExecuteAction::CreateSP(this, &FAvaOutlinerView::SelectChildren, true)
		, FCanExecuteAction::CreateSP(this, &FAvaOutlinerView::CanSelectChildren));

	ViewCommandList->MapAction(OutlinerCommands.SelectImmediateChildren
		, FExecuteAction::CreateSP(this, &FAvaOutlinerView::SelectChildren, false)
		, FCanExecuteAction::CreateSP(this, &FAvaOutlinerView::CanSelectChildren));

	ViewCommandList->MapAction(OutlinerCommands.SelectParent
		, FExecuteAction::CreateSP(this, &FAvaOutlinerView::SelectParent)
		, FCanExecuteAction::CreateSP(this, &FAvaOutlinerView::CanSelectParent));

	ViewCommandList->MapAction(OutlinerCommands.SelectFirstChild
		, FExecuteAction::CreateSP(this, &FAvaOutlinerView::SelectFirstChild)
		, FCanExecuteAction::CreateSP(this, &FAvaOutlinerView::CanSelectFirstChild));

	ViewCommandList->MapAction(OutlinerCommands.SelectNextSibling
		, FExecuteAction::CreateSP(this, &FAvaOutlinerView::SelectSibling, +1)
		, FCanExecuteAction::CreateSP(this, &FAvaOutlinerView::CanSelectSibling));

	ViewCommandList->MapAction(OutlinerCommands.SelectPreviousSibling
		, FExecuteAction::CreateSP(this, &FAvaOutlinerView::SelectSibling, -1)
		, FCanExecuteAction::CreateSP(this, &FAvaOutlinerView::CanSelectSibling));

	ViewCommandList->MapAction(OutlinerCommands.ExpandAll
		, FExecuteAction::CreateSP(this, &FAvaOutlinerView::ExpandAll)
		, FCanExecuteAction::CreateSP(this, &FAvaOutlinerView::CanExpandAll));

	ViewCommandList->MapAction(OutlinerCommands.CollapseAll
		, FExecuteAction::CreateSP(this, &FAvaOutlinerView::CollapseAll)
		, FCanExecuteAction::CreateSP(this, &FAvaOutlinerView::CanCollapseAll));

	ViewCommandList->MapAction(OutlinerCommands.ScrollNextSelectionIntoView
		, FExecuteAction::CreateSP(this, &FAvaOutlinerView::ScrollNextIntoView)
		, FCanExecuteAction::CreateSP(this, &FAvaOutlinerView::CanScrollNextIntoView));

	ViewCommandList->MapAction(OutlinerCommands.ToggleMutedHierarchy
		, FExecuteAction::CreateSP(this, &FAvaOutlinerView::ToggleMutedHierarchy)
		, FCanExecuteAction::CreateSP(this, &FAvaOutlinerView::CanToggleMutedHierarchy)
		, FIsActionChecked::CreateSP(this, &FAvaOutlinerView::IsMutedHierarchyActive));

	ViewCommandList->MapAction(OutlinerCommands.ToggleAutoExpandToSelection
		, FExecuteAction::CreateSP(this, &FAvaOutlinerView::ToggleAutoExpandToSelection)
		, FCanExecuteAction::CreateSP(this, &FAvaOutlinerView::CanToggleAutoExpandToSelection)
		, FIsActionChecked::CreateSP(this, &FAvaOutlinerView::ShouldAutoExpandToSelection));

	ViewCommandList->MapAction(OutlinerCommands.Refresh
		, FExecuteAction::CreateSP(this, &FAvaOutlinerView::RefreshOutliner, true));
}

TSharedPtr<FUICommandList> FAvaOutlinerView::GetBaseCommandList() const
{
	if (const TSharedPtr<FAvaOutliner> Outliner = OutlinerWeak.Pin())
	{
		return Outliner->GetBaseCommandList();
	}
	return nullptr;
}

void FAvaOutlinerView::UpdateRecentOutlinerViews()
{
	if (const TSharedPtr<FAvaOutliner> Outliner = OutlinerWeak.Pin())
	{
		Outliner->UpdateRecentOutlinerViews(OutlinerViewId);
	}
}

bool FAvaOutlinerView::IsMostRecentOutlinerView() const
{
	return OutlinerWeak.IsValid() && OutlinerWeak.Pin()->GetMostRecentOutlinerView().Get() == this;
}

TSharedRef<SWidget> FAvaOutlinerView::GetOutlinerWidget() const
{
	if (OutlinerWidgetContainer.IsValid())
	{
		return OutlinerWidgetContainer.ToSharedRef();
	}
	return SNullWidget::NullWidget;
}

TSharedPtr<IAvaOutliner> FAvaOutlinerView::GetOwnerOutliner() const
{
	return GetOutliner();
}

void FAvaOutlinerView::CreateToolbar()
{
	UToolMenus* const ToolMenus = UToolMenus::Get();

	const FName ToolbarName = FAvaOutlinerView::GetOutlinerToolbarName();

	if (!ToolMenus->IsMenuRegistered(ToolbarName))
	{
		UToolMenu* const ToolBar = ToolMenus->RegisterMenu(ToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		ToolBar->StyleName               = "StatusBarToolBar";
		ToolBar->bToolBarForceSmallIcons = true;
		ToolBar->bToolBarIsFocusable     = true;
		ToolBar->AddDynamicSection("PopulateToolBar", FNewToolMenuDelegate::CreateStatic(&FAvaOutlinerView::PopulateToolBar));
	}

	TSharedPtr<FExtender> Extender;

	UAvaOutlinerViewToolbarContext* const ContextObject = NewObject<UAvaOutlinerViewToolbarContext>();
	ContextObject->OutlinerViewWeak = SharedThis(this);

	FToolMenuContext Context(GetBaseCommandList(), Extender, ContextObject);
	if (OutlinerWidget.IsValid())
	{
		OutlinerWidget->SetToolBarWidget(ToolMenus->GenerateWidget(ToolbarName, Context));
	}
}

TSharedPtr<SWidget> FAvaOutlinerView::CreateItemContextMenu()
{
	UToolMenus* const ToolMenus = UToolMenus::Get();

	const FName ItemContextMenuName = FAvaOutlinerView::GetOutlinerItemContextMenuName();

	if (!ToolMenus->IsMenuRegistered(ItemContextMenuName))
	{
		UToolMenu* const ContextMenu = ToolMenus->RegisterMenu(ItemContextMenuName, NAME_None, EMultiBoxType::Menu);
		ContextMenu->AddDynamicSection("PopulateContextMenu", FNewToolMenuDelegate::CreateStatic(&FAvaOutlinerView::PopulateItemContextMenu));
	}

	TSharedPtr<FExtender> Extender;

	UAvaOutlinerItemsContext* const ContextObject = NewObject<UAvaOutlinerItemsContext>();
	ContextObject->OutlinerWeak = GetOutliner();
	ContextObject->ItemListWeak.Append(SelectedItems);

	FToolMenuContext Context(GetViewCommandList(), Extender, ContextObject);
	return ToolMenus->GenerateWidget(ItemContextMenuName, Context);
}

TSharedRef<SWidget> FAvaOutlinerView::CreateOutlinerSettingsMenu()
{
	const FAvaOutlinerCommands& OutlinerCommands = FAvaOutlinerCommands::Get();
	
	FMenuBuilder MenuBuilder(true, ViewCommandList);
	MenuBuilder.BeginSection(TEXT("HierarchySettings"), LOCTEXT("HierarchyHeading", "Hierarchy"));
	{
		MenuBuilder.AddMenuEntry(OutlinerCommands.ExpandAll);
		MenuBuilder.AddMenuEntry(OutlinerCommands.CollapseAll);
		MenuBuilder.AddMenuEntry(OutlinerCommands.ToggleMutedHierarchy);
		MenuBuilder.AddMenuEntry(OutlinerCommands.ToggleAutoExpandToSelection);
	}
	MenuBuilder.EndSection();
	
	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FAvaOutlinerView::CreateOutlinerViewOptionsMenu()
{
	const TSharedPtr<FAvaOutliner> Outliner = GetOutliner();

	if (!Outliner.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	FMenuBuilder MenuBuilder(false, nullptr);

	// Toggle Proxy and their Items ON/OFF
	MenuBuilder.BeginSection(TEXT("Item Types"), LOCTEXT("ItemTypesLabel", "Item Types"));
	{
		for (FName RegisteredTypeName : Outliner->GetRegisteredItemProxyTypeNames())
		{
			IAvaOutlinerItemProxyFactory* const ItemProxyFactory = Outliner->GetItemProxyFactory(RegisteredTypeName);
			check(ItemProxyFactory);

			TSharedPtr<FAvaOutlinerItemProxy> TemplateProxyItem = ItemProxyFactory->CreateItemProxy(*Outliner, nullptr);

			// Template Proxy Item creation might fail if ItemProxyFactory is not the default (where it always returns a constructed item)
			if (!TemplateProxyItem.IsValid())
			{
				continue;
			}

			const FName ItemProxyTypeName = TemplateProxyItem->GetTypeId().ToName();

			// Type mismatch here can continue but recommended to be addressed for Proxy Items / Factories
			// not correctly overriding Type Name Functions
			ensureMsgf(ItemProxyTypeName == RegisteredTypeName
				, TEXT("Item Proxy Type (%s) does not match Factory registered type (%s)! Check override of the Type Name Getters for both")
				, *ItemProxyTypeName.ToString()
				, *RegisteredTypeName.ToString());

			FUIAction UIAction;
			UIAction.ExecuteAction = FExecuteAction::CreateSP(this, &FAvaOutlinerView::ToggleHideItemTypes, ItemProxyTypeName);
			UIAction.GetActionCheckState = FGetActionCheckState::CreateSP(this, &FAvaOutlinerView::GetToggleHideItemTypesState, ItemProxyTypeName);

			MenuBuilder.AddMenuEntry(TemplateProxyItem->GetDisplayName()
				, TemplateProxyItem->GetIconTooltipText()
				, TemplateProxyItem->GetIcon()
				, UIAction, NAME_None, EUserInterfaceActionType::ToggleButton);
		}
	}
	MenuBuilder.EndSection();

	auto AddViewModeEntries = [this, &MenuBuilder](TMemFunPtrType<false, FAvaOutlinerView, void(EAvaOutlinerItemViewMode)>::Type InExecuteFunction
		, TMemFunPtrType<true, FAvaOutlinerView, ECheckBoxState(EAvaOutlinerItemViewMode)>::Type InGetActionCheckState)
		{
			const UEnum* const ViewModeEnum = StaticEnum<EAvaOutlinerItemViewMode>();
			check(ViewModeEnum);
			
			for (EAvaOutlinerItemViewMode ViewModeFlags : MakeFlagsRange(EAvaOutlinerItemViewMode::All))
			{
				const int32 EnumIndex = ViewModeEnum->GetIndexByValue(static_cast<int64>(ViewModeFlags));
				
				FUIAction Action;
				Action.ExecuteAction = FExecuteAction::CreateSP(this, InExecuteFunction, ViewModeFlags);
				Action.CanExecuteAction = FCanExecuteAction::CreateLambda([](){ return true; });
				Action.GetActionCheckState = FGetActionCheckState::CreateSP(this, InGetActionCheckState, ViewModeFlags);
				
				MenuBuilder.AddMenuEntry(ViewModeEnum->GetDisplayNameTextByIndex(EnumIndex)
					, ViewModeEnum->GetToolTipTextByIndex(EnumIndex)
					, FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::ToggleButton);
			}
		};
	
	MenuBuilder.BeginSection(TEXT("ItemDefaultViewMode"), LOCTEXT("ItemDefaultViewMode", "Default Item View Mode"));
	{
		AddViewModeEntries(&FAvaOutlinerView::ToggleItemDefaultViewModeSupport
			, &FAvaOutlinerView::GetItemDefaultViewModeCheckState);
	}
	MenuBuilder.EndSection();
	
	MenuBuilder.BeginSection(TEXT("ItemProxyViewMode"), LOCTEXT("ItemProxyViewMode", "Proxy Item View Mode"));
	{
		AddViewModeEntries(&FAvaOutlinerView::ToggleItemProxyViewModeSupport
			, &FAvaOutlinerView::GetItemProxyViewModeCheckState);
	}
	MenuBuilder.EndSection();
	
	return MenuBuilder.MakeWidget();
}

bool FAvaOutlinerView::ShouldShowColumnByDefault(const TSharedPtr<IAvaOutlinerColumn>& InColumn) const
{
	if (!InColumn.IsValid())
	{
		return false;
	}
	
	if (const bool* const bFoundVisibility = ColumnVisibility.Find(InColumn->GetColumnId()))
	{
		return *bFoundVisibility;
	}
	
	return InColumn->ShouldShowColumnByDefault();
}

void FAvaOutlinerView::UpdateColumnVisibilityMap()
{
	if (OutlinerWidget.IsValid())
	{
		OutlinerWidget->GenerateColumnVisibilityMap(ColumnVisibility);
	}
}

void FAvaOutlinerView::RequestRefresh()
{
	bRefreshRequested = true;
}

void FAvaOutlinerView::Refresh()
{
	UpdateRootVisibleItems();
	
	UpdateItemExpansions();
	
	if (OutlinerWidget.IsValid())
	{
		OutlinerWidget->Refresh();
	}
	
	OnOutlinerViewRefreshed.Broadcast();
}

void FAvaOutlinerView::SetKeyboardFocus()
{
	if (OutlinerWidget.IsValid())
	{
		OutlinerWidget->SetKeyboardFocus();
	}
}

void FAvaOutlinerView::UpdateRootVisibleItems()
{
	RootVisibleItems.Reset();
	ReadOnlyItems.Reset();
	
	if (OutlinerWeak.IsValid())
	{
		GetChildrenOfItem(OutlinerWeak.Pin()->GetTreeRoot(), RootVisibleItems);
	}
	
	OutlinerStats->MarkCountTypeDirty(EAvaOutlinerStatCountType::VisibleItemCount);
}

void FAvaOutlinerView::UpdateItemExpansions()
{
	TArray<FAvaOutlinerItemPtr> Items = RootVisibleItems;
	
	while (Items.Num() > 0)
	{
		FAvaOutlinerItemPtr Item = Items.Pop();
		const EAvaOutlinerItemFlags ItemFlags = GetViewItemFlags(Item);
		SetItemExpansion(Item, EnumHasAnyFlags(ItemFlags, EAvaOutlinerItemFlags::Expanded));
		Items.Append(Item->GetChildren());
	}

	Items = RootVisibleItems;

	while (Items.Num() > 0)
	{
		const FAvaOutlinerItemPtr Item = Items.Pop();
		if (OutlinerWidget.IsValid())
		{
			OutlinerWidget->UpdateItemExpansions(Item);
		}
		Items.Append(Item->GetChildren());
	}
}

void FAvaOutlinerView::NotifyObjectsReplaced()
{
	if (OutlinerWidget.IsValid())
	{
		OutlinerWidget->Invalidate(EInvalidateWidgetReason::Paint);	
	}
}

void FAvaOutlinerView::OnFilterChanged()
{
	Refresh();
}

FAvaOutlinerItemPtr FAvaOutlinerView::GetRootItem() const
{
	if (OutlinerWeak.IsValid())
	{
		return OutlinerWeak.Pin()->GetTreeRoot();
	}
	return nullptr;
}

void FAvaOutlinerView::SetViewItemFlags(const FAvaOutlinerItemPtr& InItem, EAvaOutlinerItemFlags InFlags)
{
	if (InItem.IsValid())
	{
		if (InFlags == EAvaOutlinerItemFlags::None)
		{
			ViewItemFlags.Remove(InItem->GetItemId());
		}
		else
		{	
			ViewItemFlags.Add(InItem->GetItemId(), InFlags);
		}
	}
}

EAvaOutlinerItemFlags FAvaOutlinerView::GetViewItemFlags(const FAvaOutlinerItemPtr& InItem) const
{
	if (InItem.IsValid())
	{
		if (const EAvaOutlinerItemFlags* const OverrideFlags = ViewItemFlags.Find(InItem->GetItemId()))
		{
			return *OverrideFlags;
		}
	}
	return EAvaOutlinerItemFlags::None;
}

void FAvaOutlinerView::GetChildrenOfItem(FAvaOutlinerItemPtr InItem, TArray<FAvaOutlinerItemPtr>& OutChildren) const
{
	static const TSet<FAvaOutlinerItemPtr> EmptySet;
	GetChildrenOfItem(InItem, OutChildren, EAvaOutlinerItemViewMode::ItemTree, EmptySet);
}

void FAvaOutlinerView::GetChildrenOfItem(FAvaOutlinerItemPtr InItem
	, TArray<FAvaOutlinerItemPtr>& OutChildren
	, EAvaOutlinerItemViewMode InViewMode
	, const TSet<FAvaOutlinerItemPtr>& InRecursionDisallowedItems) const
{
	if (!InItem.IsValid())
	{
		return;
	}

	for (const FAvaOutlinerItemPtr& ChildItem : InItem->GetChildren())
	{
		if (ChildItem.IsValid())
		{
			if (ShouldShowItem(ChildItem, true, InViewMode))
			{
				// if current item is visible in outliner, add it to the children
				OutChildren.Add(ChildItem);
			}
			else if (!InRecursionDisallowedItems.Contains(ChildItem))
			{
				TArray<FAvaOutlinerItemPtr> GrandChildren;

				// for Muted Hierarchy to be in effect, not only does it have to be on
				// but also the item should be shown (without counting the filter pass)
				const bool bShouldMuteItem = bUseMutedHierarchy && ShouldShowItem(ChildItem, false, InViewMode);

				// If it's muted hierarchy, there might be ONLY grand children that are just visible in other view modes
				// so instead of just filtering out the child item, check that there are no grand children from other view modes passing filter tests
				// If it's NOT muted hierarchy, just get the grand children visible in the requested view mode, as this ChildItem is guaranteed to be hidden
				const EAvaOutlinerItemViewMode ViewModeToUse = bShouldMuteItem ? EAvaOutlinerItemViewMode::All : InViewMode;

				GetChildrenOfItem(ChildItem, GrandChildren, ViewModeToUse, InRecursionDisallowedItems);

				if (!GrandChildren.IsEmpty())
				{
					if (bShouldMuteItem)
					{
						ReadOnlyItems.Add(ChildItem);	
						OutChildren.Add(ChildItem);
					}
					else
					{
						// we can append them knowing that the ViewMode to use is the one passed in and there's no
						// child that leaked from another view mode
						ensure(ViewModeToUse == InViewMode);
						OutChildren.Append(GrandChildren);
					}
				}
			}
		}
	}
}

FLinearColor FAvaOutlinerView::GetItemBrushColor(FAvaOutlinerItemPtr InItem) const
{
	if (InItem.IsValid())
	{
		FLinearColor OutColor = InItem->GetItemColor();

		// If NextSelectedItemIntoView is valid, it means we're scrolling items into view with Next/Previous
		// So Make everything that's not the Current Item a bit more translucent to make the Current Item stand out
		if (SortedSelectedItems.IsValidIndex(NextSelectedItemIntoView)
			&& SortedSelectedItems[NextSelectedItemIntoView] != InItem)
		{
			OutColor.A *= 0.5f;
		}

		return OutColor;
	}
	return FLinearColor::White;
}

TArray<FAvaOutlinerItemPtr> FAvaOutlinerView::GetViewSelectedItems() const
{
	return SelectedItems;
}

int32 FAvaOutlinerView::GetViewSelectedItemCount() const
{
	return SelectedItems.Num();
}

int32 FAvaOutlinerView::CalculateVisibleItemCount() const
{
	TArray<FAvaOutlinerItemPtr> RemainingItems = RootVisibleItems;
	
	int32 VisibleItemCount = RemainingItems.Num();

	while (RemainingItems.Num() > 0)
	{
		const FAvaOutlinerItemPtr Item = RemainingItems.Pop();
		
		TArray<FAvaOutlinerItemPtr> ChildItems;
		GetChildrenOfItem(Item, ChildItems);
		VisibleItemCount += ChildItems.Num();
		RemainingItems.Append(MoveTemp(ChildItems));
	}

	//Remove the Read Only Items as they are filtered out items that are still shown because of Hierarchy Viz
	VisibleItemCount -= ReadOnlyItems.Num();

	return VisibleItemCount;
}

void FAvaOutlinerView::SelectItems(TArray<FAvaOutlinerItemPtr> InItems, EAvaOutlinerItemSelectionFlags InFlags)
{
	// Remove Duplicate Items
	TSet<FAvaOutlinerItemPtr> SeenItems;
	SeenItems.Reserve(InItems.Num());
	for (TArray<FAvaOutlinerItemPtr>::TIterator Iter(InItems); Iter; ++Iter)
	{
		if (SeenItems.Contains(*Iter))
		{
			Iter.RemoveCurrent();
		}
		else
		{
			SeenItems.Add(*Iter);
		}
	}

	// Add the Children of the Items given
	if (EnumHasAnyFlags(InFlags, EAvaOutlinerItemSelectionFlags::IncludeChildren))
	{
		TArray<FAvaOutlinerItemPtr> ChildItemsRemaining(InItems);
		while (ChildItemsRemaining.Num() > 0)
		{
			if (FAvaOutlinerItemPtr ChildItem = ChildItemsRemaining.Pop())
			{
				TArray<FAvaOutlinerItemPtr> Children;
				GetChildrenOfItem(ChildItem, Children);

				InItems.Append(Children);
				ChildItemsRemaining.Append(Children);
			}
		}
	}

	if (EnumHasAnyFlags(InFlags, EAvaOutlinerItemSelectionFlags::AppendToCurrentSelection))
	{
		//Remove all repeated items to avoid duplicated entries
		SelectedItems.RemoveAll([&SeenItems](const FAvaOutlinerItemPtr& Item)
		{
			return SeenItems.Contains(Item);
		});	
		TArray<FAvaOutlinerItemPtr> Items = MoveTemp(SelectedItems);
		Items.Append(MoveTemp(InItems));
		InItems = MoveTemp(Items);
	}

	if (!InItems.IsEmpty() && EnumHasAnyFlags(InFlags, EAvaOutlinerItemSelectionFlags::ScrollIntoView))
	{
		ScrollItemIntoView(InItems[0]);
	}

	const bool bSignalSelectionChange = EnumHasAnyFlags(InFlags, EAvaOutlinerItemSelectionFlags::SignalSelectionChange);
	SetItemSelectionImpl(MoveTemp(InItems), bSignalSelectionChange);
}

void FAvaOutlinerView::ClearItemSelection(bool bSignalSelectionChange)
{
	SetItemSelectionImpl({}, bSignalSelectionChange);
}

void FAvaOutlinerView::SetItemSelectionImpl(TArray<FAvaOutlinerItemPtr>&& InItems, bool bSignalSelectionChange)
{
	if (ShouldAutoExpandToSelection())
	{
		for (const FAvaOutlinerItemPtr& Item : InItems)
		{
			SetParentItemExpansions(Item, true);
		}
	}

	SelectedItems = MoveTemp(InItems);

	OutlinerStats->MarkCountTypeDirty(EAvaOutlinerStatCountType::SelectedItemCount);

	if (OutlinerWidget.IsValid())
	{
		OutlinerWidget->SetItemSelection(SelectedItems, bSignalSelectionChange);
	}
	else if (bSignalSelectionChange)
	{
		NotifyItemSelectionChanged(SelectedItems, nullptr, true);
	}
	
	Refresh();
}

void FAvaOutlinerView::NotifyItemSelectionChanged(const TArray<FAvaOutlinerItemPtr>& InSelectedItems
	, const FAvaOutlinerItemPtr& InItem
	, bool bUpdateModeTools)
{
	if (bSyncingItemSelection)
	{
		return;
	}
	TGuardValue<bool> Guard(bSyncingItemSelection, true);
	
	SelectedItems            = InSelectedItems;
	SortedSelectedItems      = InSelectedItems;
	NextSelectedItemIntoView = -1;
	
	FAvaOutliner::SortItems(SortedSelectedItems);

	OutlinerStats->MarkCountTypeDirty(EAvaOutlinerStatCountType::SelectedItemCount);

	//If we have Pending Items Remaining but we switched Selection via Navigation, treat it as "I want to rename this too"
	if (bRenamingItems && InItem.IsValid() && InItem != CurrentItemRenaming)
	{
		bRequestedRename = true;
	}
	
	TSharedPtr<FAvaOutliner> Outliner = OutlinerWeak.Pin();
	if (bUpdateModeTools && Outliner.IsValid())
	{
		Outliner->SyncModeToolsSelection(SelectedItems);
		Outliner->SelectItems(SelectedItems, EAvaOutlinerItemSelectionFlags::None);
	}
}

bool FAvaOutlinerView::IsItemReadOnly(const FAvaOutlinerItemPtr& InItem) const
{
	return ReadOnlyItems.Contains(InItem);;
}

bool FAvaOutlinerView::CanSelectItem(const FAvaOutlinerItemPtr& InItem) const
{
	const bool bIsSelectable = InItem.IsValid() && InItem->IsSelectable();
	return bIsSelectable && !IsItemReadOnly(InItem);
}

bool FAvaOutlinerView::IsItemSelected(const FAvaOutlinerItemPtr& InItem) const
{
	return SelectedItems.Contains(InItem);
}

void FAvaOutlinerView::SetParentItemExpansions(const FAvaOutlinerItemPtr& InItem, bool bIsExpanded)
{
	if (!InItem.IsValid())
	{
		return;
	}
	
	TArray<FAvaOutlinerItemPtr> ItemsToExpand;
	
	// Don't auto expand at all if there's a parent preventing it
	FAvaOutlinerItemPtr ParentItem = InItem->GetParent();
	while (ParentItem.IsValid())
	{
		if (!ParentItem->CanAutoExpand())
		{
			return;
		}
		ItemsToExpand.Add(ParentItem);
		ParentItem = ParentItem->GetParent();
	}

	for (const FAvaOutlinerItemPtr& Item : ItemsToExpand)
	{
		SetItemExpansion(Item, bIsExpanded);
	}
}

void FAvaOutlinerView::SetItemExpansion(const FAvaOutlinerItemPtr& InItem, bool bIsExpanded, bool bInUseFilter)
{
	// Don't continue if Item should be hidden in view.
	// the tree view still calls OnItemExpansionChanged even if it doesn't contain the item
	// so this preemptive check is needed
	if (!ShouldShowItem(InItem, bInUseFilter, EAvaOutlinerItemViewMode::ItemTree))
	{
		return;
	}
	
	if (OutlinerWidget.IsValid())
	{
		OutlinerWidget->SetItemExpansion(InItem, bIsExpanded);
	}
	else
	{
		OnItemExpansionChanged(InItem, bIsExpanded);
	}
}

void FAvaOutlinerView::SetItemExpansionRecursive(FAvaOutlinerItemPtr InItem, bool bIsExpanded)
{
	SetItemExpansion(InItem, bIsExpanded, false);
	
	for (const FAvaOutlinerItemPtr& Child : InItem->GetChildren())
	{
		if (Child.IsValid())
		{
			SetItemExpansionRecursive(Child, bIsExpanded);
		}
	}
}

void FAvaOutlinerView::OnItemExpansionChanged(FAvaOutlinerItemPtr InItem, bool bIsExpanded)
{
	const EAvaOutlinerItemFlags CurrentFlags = GetViewItemFlags(InItem);
	
	EAvaOutlinerItemFlags TargetFlags = CurrentFlags;
	
	if (bIsExpanded)
	{
		TargetFlags |= EAvaOutlinerItemFlags::Expanded;
	}
	else
	{
		TargetFlags &= ~EAvaOutlinerItemFlags::Expanded;
	}

	SetViewItemFlags(InItem, TargetFlags);
	
	if (CurrentFlags != TargetFlags)
	{
		InItem->OnExpansionChanged().Broadcast(SharedThis(this), bIsExpanded);
	}
}

bool FAvaOutlinerView::ShouldShowItem(const FAvaOutlinerItemPtr& InItem, bool bInUseFilters
	, EAvaOutlinerItemViewMode InViewMode) const
{
	// Check #1: Item Valid, Allowed in Outliner etc
	
	// can't show an invalid item, or an item that is a tree root
	if (!InItem.IsValid() || InItem->IsA<FAvaOutlinerTreeRoot>())
	{
		return false;
	}

	if (!InItem->IsAllowedInOutliner() || !InItem->IsViewModeSupported(InViewMode, *this))
	{
		return false;
	}

	TSharedPtr<FAvaOutliner> Outliner = OutlinerWeak.Pin();

	// allow provider to determine whether the item should be hidden
	if (!Outliner.IsValid() || Outliner->GetProvider().ShouldHideItem(InItem))
	{
		return false;
	}

	if (IsItemTypeHidden(InItem))
	{
		return false;
	}

	// Extra pass for Non-Item Proxies that are parented under an Item Proxy
	// Hiding an Item Proxy Type should affect all the rest of the items below it
	if (!InItem->IsA<FAvaOutlinerItemProxy>())
	{
		FAvaOutlinerItemPtr ItemParent = InItem->GetParent();
		while (ItemParent.IsValid())
		{
			if (ItemParent->IsA<FAvaOutlinerItemProxy>())
			{
				if (IsItemTypeHidden(ItemParent))
				{
					return false;
				}
				// Stop at the first Item Proxy parent found
				break;
			}
			ItemParent = ItemParent->GetParent();
		}
	}

	// if we are not doing a filter check stop here and return true as the rest below are just filters
	if (!bInUseFilters)
	{
		return true;
	}

	// Check #2: Filters
	
	if (!TextFilter->PassesFilter(*InItem))
	{
		return false;
	}

	//Return true if we find 1 item filter where the item succeed filter test
	//These filters behave more like an OR than an AND, so 1 succeeding = true
	for (const TSharedPtr<IAvaOutlinerItemFilter>& ItemFilter : ActiveItemFilters)
	{
		if (ItemFilter.IsValid() && ItemFilter->PassesFilter(*InItem))
		{
			return true;
		}
	}

	//If we had filters, return false since none of the filters passed, else return true since there were no active filters
	return ActiveItemFilters.Num() == 0;
}

int32 FAvaOutlinerView::GetVisibleChildIndex(const FAvaOutlinerItemPtr& InParentItem, const FAvaOutlinerItemPtr& InChildItem) const
{
	if (InParentItem.IsValid())
	{
		TArray<FAvaOutlinerItemPtr> Children;
		GetChildrenOfItem(InParentItem, Children);
		return Children.Find(InChildItem);
	}
	return INDEX_NONE;
}

FAvaOutlinerItemPtr FAvaOutlinerView::GetVisibleChildAt(const FAvaOutlinerItemPtr& InParentItem, int32 InChildIndex) const
{
	if (InParentItem.IsValid())
	{
		TArray<FAvaOutlinerItemPtr> Children;
		GetChildrenOfItem(InParentItem, Children);
		if (Children.IsValidIndex(InChildIndex))
		{
			return Children[InChildIndex];
		}
	}
	return nullptr;
}

bool FAvaOutlinerView::IsOutlinerLocked() const
{
	return OutlinerWeak.IsValid() && OutlinerWeak.Pin()->IsOutlinerLocked();
}

void FAvaOutlinerView::EnableItemFilter(const TSharedPtr<IAvaOutlinerItemFilter>& InFilter, const bool bInRefreshOutliner)
{
	bool bAlreadyInSet = false;
	ActiveItemFilters.Add(InFilter, &bAlreadyInSet);
	if (bInRefreshOutliner && !bAlreadyInSet)
	{
		Refresh();
	}
}

void FAvaOutlinerView::EnableItemFilterById(const FName& InFilterId, const bool bInRefreshOutliner)
{
	for (const TSharedPtr<IAvaOutlinerItemFilter>& CustomItemFilter : GetCustomItemFilters())
	{
		if (CustomItemFilter->GetFilterId() == InFilterId)
		{
			// We can return after the first since the Id is unique
			EnableItemFilter(CustomItemFilter, bInRefreshOutliner);
			return;
		}
	}
}

void FAvaOutlinerView::DisableItemFilter(const TSharedPtr<IAvaOutlinerItemFilter>& InFilter, const bool bInRefreshOutliner)
{
	const int32 RemoveCount = ActiveItemFilters.Remove(InFilter);
	if (bInRefreshOutliner && RemoveCount > 0)
	{
		Refresh();
	}
}

bool FAvaOutlinerView::IsItemFilterEnabled(const TSharedPtr<IAvaOutlinerItemFilter>& InFilter) const
{
	return ActiveItemFilters.Contains(InFilter);
}

void FAvaOutlinerView::SetSearchText(const FText& InText)
{
	TextFilter->SetFilterText(InText);
}

FText FAvaOutlinerView::GetSearchText() const
{
	return OutlinerWidget.IsValid()
		? OutlinerWidget->GetSearchText()
		: FText::GetEmpty();
}

void FAvaOutlinerView::ToggleViewModeSupport(EAvaOutlinerItemViewMode& InOutViewMode, EAvaOutlinerItemViewMode InFlags)
{
	if (EnumHasAnyFlags(InOutViewMode, InFlags))
	{
		EnumRemoveFlags(InOutViewMode, InFlags);
	}
	else
	{
		EnumAddFlags(InOutViewMode, InFlags);
	}
	RefreshOutliner(false);
}

void FAvaOutlinerView::ToggleItemDefaultViewModeSupport(EAvaOutlinerItemViewMode InFlags)
{
	ToggleViewModeSupport(ItemDefaultViewMode, InFlags);	
}

void FAvaOutlinerView::ToggleItemProxyViewModeSupport(EAvaOutlinerItemViewMode InFlags)
{
	ToggleViewModeSupport(ItemProxyViewMode, InFlags);
}

ECheckBoxState FAvaOutlinerView::GetViewModeCheckState(EAvaOutlinerItemViewMode InViewMode, EAvaOutlinerItemViewMode InFlags) const
{
	const EAvaOutlinerItemViewMode Result = InViewMode & InFlags;
	
	if (Result == InFlags)
	{
		return ECheckBoxState::Checked;
	}
	
	if (Result != EAvaOutlinerItemViewMode::None)
	{
		return ECheckBoxState::Undetermined;
	}
	
	return ECheckBoxState::Unchecked;
}

ECheckBoxState FAvaOutlinerView::GetItemDefaultViewModeCheckState(EAvaOutlinerItemViewMode InFlags) const
{
	return GetViewModeCheckState(ItemDefaultViewMode, InFlags);
}

ECheckBoxState FAvaOutlinerView::GetItemProxyViewModeCheckState(EAvaOutlinerItemViewMode InFlags) const
{
	return GetViewModeCheckState(ItemProxyViewMode, InFlags);
}

void FAvaOutlinerView::ToggleMutedHierarchy()
{
	bUseMutedHierarchy = !bUseMutedHierarchy;
	Refresh();
}

void FAvaOutlinerView::ToggleAutoExpandToSelection()
{
	bAutoExpandToSelection = !bAutoExpandToSelection;
}

void FAvaOutlinerView::ToggleShowItemFilters()
{
	//Note: Not Marking Outliner Instance as Modified because this is not saved.
	bShowItemFilters = !bShowItemFilters;
}

void FAvaOutlinerView::SetItemTypeHidden(FName InItemTypeName, bool bHidden)
{
	if (IsItemTypeHidden(InItemTypeName) != bHidden)
	{
		if (bHidden)
		{
			HiddenItemTypes.Add(InItemTypeName);
		}
		else
		{
			HiddenItemTypes.Remove(InItemTypeName);
		}
		RequestRefresh();
	}	
}

void FAvaOutlinerView::ToggleHideItemTypes(FName InItemTypeName)
{
	SetItemTypeHidden(InItemTypeName, !IsItemTypeHidden(InItemTypeName));
}

ECheckBoxState FAvaOutlinerView::GetToggleHideItemTypesState(FName InItemTypeName) const
{
	return IsItemTypeHidden(InItemTypeName) ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
}

bool FAvaOutlinerView::IsItemTypeHidden(FName InItemTypeName) const
{
	return HiddenItemTypes.Contains(InItemTypeName);
}

bool FAvaOutlinerView::IsItemTypeHidden(const FAvaOutlinerItemPtr& InItem) const
{
	return IsItemTypeHidden(InItem->GetTypeId().ToName());
}

void FAvaOutlinerView::OnDragEnter(const FDragDropEvent& InDragDropEvent, FAvaOutlinerItemPtr InTargetItem)
{
	if (!InTargetItem.IsValid() && OutlinerWeak.IsValid())
	{
		TSharedRef<FAvaOutlinerItem> TreeRoot = OutlinerWeak.Pin()->GetTreeRoot();
		const bool bCanAcceptDrop = TreeRoot->CanAcceptDrop(InDragDropEvent, EItemDropZone::OntoItem).IsSet();
		SetDragIntoTreeRoot(bCanAcceptDrop);
	}
	else
	{
		SetDragIntoTreeRoot(false);
	}
}

void FAvaOutlinerView::OnDragLeave(const FDragDropEvent& InDragDropEvent, FAvaOutlinerItemPtr InTargetItem)
{
	// If drag left an item, set the drag into tree root to false (this will set it back to false if a valid item receives DragEnter)
	SetDragIntoTreeRoot(InTargetItem.IsValid());
}

FReply FAvaOutlinerView::OnDragDetected(const FGeometry& InGeometry
	, const FPointerEvent& InMouseEvent
	, FAvaOutlinerItemPtr InTargetItem)
{
	if (!IsOutlinerLocked())
	{
		// Only Select Target if it hasn't already been selected
		if (!IsItemSelected(InTargetItem))
		{
			const EAvaOutlinerItemSelectionFlags SelectionFlags = InMouseEvent.IsControlDown()
			? EAvaOutlinerItemSelectionFlags::AppendToCurrentSelection
			: EAvaOutlinerItemSelectionFlags::None;

			SelectItems({ InTargetItem }, SelectionFlags);
		}

		// Get all Selected Items that are in a state where they can be selected again (i.e. not Read Only)
		TArray<FAvaOutlinerItemPtr> Items = GetViewSelectedItems();
		Items.RemoveAll([this](FAvaOutlinerItemPtr InItem)->bool
		{
			return !CanSelectItem(InItem);
		});

		if (Items.Num() > 0)
		{
			const EAvaOutlinerDragDropActionType ActionType = InMouseEvent.IsAltDown()
				? EAvaOutlinerDragDropActionType::Copy
				: EAvaOutlinerDragDropActionType::Move;

			const TSharedRef<FAvaOutlinerItemDragDropOp> DragDropOp = FAvaOutlinerItemDragDropOp::New(Items, SharedThis(this), ActionType);
			return FReply::Handled().BeginDragDrop(DragDropOp);
		}
	}
	return FReply::Unhandled();
}

FReply FAvaOutlinerView::OnDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem)
{
	SetDragIntoTreeRoot(false);

	if (InTargetItem.IsValid())
	{
		return InTargetItem->AcceptDrop(InDragDropEvent, InDropZone);
	}

	TSharedPtr<FAvaOutlinerItem> TreeRoot;

	if (OutlinerWeak.IsValid())
	{
		TreeRoot = OutlinerWeak.Pin()->GetTreeRoot();
	}

	if (TreeRoot.IsValid() && TreeRoot->CanAcceptDrop(InDragDropEvent, EItemDropZone::OntoItem))
	{
		return TreeRoot->AcceptDrop(InDragDropEvent, EItemDropZone::OntoItem);
	}
	
	return FReply::Unhandled();
}

TOptional<EItemDropZone> FAvaOutlinerView::OnCanDrop(const FDragDropEvent& InDragDropEvent
	, EItemDropZone InDropZone
	, FAvaOutlinerItemPtr InTargetItem) const
{	
	if (!IsOutlinerLocked() && InTargetItem.IsValid() && CanSelectItem(InTargetItem))
	{
		return InTargetItem->CanAcceptDrop(InDragDropEvent, InDropZone);
	}
	return TOptional<EItemDropZone>();
}

void FAvaOutlinerView::SetDragIntoTreeRoot(bool bInIsDraggingIntoTreeRoot)
{
	if (OutlinerWidget.IsValid())
	{
		OutlinerWidget->SetTreeBorderVisibility(bInIsDraggingIntoTreeRoot);
	}
}

void FAvaOutlinerView::RenameSelected()
{
	if (TSharedPtr<FAvaOutliner> Outliner = OutlinerWeak.Pin())
	{
		TArray<FAvaOutlinerItemPtr> Items = GetViewSelectedItems();
		
		if (Items.IsEmpty())
		{
			return;
		}
	
		//Assume we have an Item Currently Renaming
		ResetRenaming();

		//Remove Items that are Invalid or can't rename
		Items.RemoveAll([](const FAvaOutlinerItemPtr& InItem) { return !InItem.IsValid() || !InItem->CanRename(); });
		
		ItemsRemainingRename = MoveTemp(Items);

		if (ItemsRemainingRename.Num() > 0)
		{
			FAvaOutliner::SortItems(ItemsRemainingRename);
			bRenamingItems = true;
		}
	}
}

void FAvaOutlinerView::ResetRenaming()
{
	if (CurrentItemRenaming.IsValid())
	{
		CurrentItemRenaming->OnRenameAction().RemoveAll(this);
		CurrentItemRenaming.Reset();
	}

	if (ItemsRemainingRename.IsEmpty())
	{
		bRenamingItems = false;
	}
}

void FAvaOutlinerView::OnItemRenameAction(EAvaOutlinerRenameAction InRenameAction, const TSharedPtr<FAvaOutlinerView>& InOutlinerView)
{
	if (InOutlinerView.Get() != this)
	{
		return;
	}
	
	switch (InRenameAction)
	{
	case EAvaOutlinerRenameAction::None:
		break;
	
	case EAvaOutlinerRenameAction::Requested:
		break;
	
	case EAvaOutlinerRenameAction::Cancelled:
		ItemsRemainingRename.Reset();
		ResetRenaming();
		break;
	
	case EAvaOutlinerRenameAction::Completed:
		ResetRenaming();
		break;
	
	default:
		break;
	}
}

bool FAvaOutlinerView::CanRenameSelected() const
{
	return GetViewSelectedItemCount() > 0;
}

void FAvaOutlinerView::DeleteSelected()
{
	if (TSharedPtr<FAvaOutliner> Outliner = OutlinerWeak.Pin())
	{
		TArray<FAvaOutlinerItemPtr> Items = GetViewSelectedItems();

		Items.RemoveAll([](const FAvaOutlinerItemPtr& InItem)
		{
			return !InItem.IsValid() || !InItem->CanDelete();
		});

		if (Items.IsEmpty())
		{
			return;
		}

		Outliner->DeleteItems(Items);
	}
}

bool FAvaOutlinerView::CanDeleteSelected() const
{
	for (const FAvaOutlinerItemPtr& Item : GetViewSelectedItems())
	{
		if (Item && Item->CanDelete())
		{
			return true;
		}
	}

	return false;
}

void FAvaOutlinerView::DuplicateSelected()
{
	if (TSharedPtr<FAvaOutliner> Outliner = OutlinerWeak.Pin())
	{
		Outliner->DuplicateItems(GetViewSelectedItems(), nullptr, TOptional<EItemDropZone>());
	}
}

bool FAvaOutlinerView::CanDuplicateSelected() const
{
	for (const FAvaOutlinerItemPtr& Item : GetViewSelectedItems())
	{
		if (Item && Item->IsA<FAvaOutlinerActor>())
		{
			return true;
		}
	}
	return false;
}

void FAvaOutlinerView::SelectChildren(bool bIsRecursive)
{
	TArray<FAvaOutlinerItemPtr> ItemsToSelect;
	TArray<FAvaOutlinerItemPtr> RemainingItems = GetViewSelectedItems();

	while (RemainingItems.Num() > 0)
	{
		//Note: Pop here will affect order of Children in Selection
		const FAvaOutlinerItemPtr ParentItem = RemainingItems.Pop();

		TArray<FAvaOutlinerItemPtr> ChildItems;
		GetChildrenOfItem(ParentItem, ChildItems);
		if (bIsRecursive)
		{
			RemainingItems.Append(ChildItems);
		}
		ItemsToSelect.Append(ChildItems);
	}

	SelectItems(ItemsToSelect
		, EAvaOutlinerItemSelectionFlags::AppendToCurrentSelection
		| EAvaOutlinerItemSelectionFlags::SignalSelectionChange);
}

bool FAvaOutlinerView::CanSelectChildren() const
{
	return GetViewSelectedItemCount() > 0;
}

void FAvaOutlinerView::SelectParent()
{
	TSet<FAvaOutlinerItemPtr> Items(GetViewSelectedItems());
	TSet<FAvaOutlinerItemPtr> ParentItemsToSelect;
	ParentItemsToSelect.Reserve(Items.Num());

	FAvaOutlinerItemPtr RootItem = GetRootItem();
	
	//Add only Valid Parents that are not Root and are not part of the Original Selection!
	for (const FAvaOutlinerItemPtr& Item : Items)
	{
		if (Item.IsValid())
		{
			FAvaOutlinerItemPtr ParentItem = Item->GetParent();
			if (ParentItem.IsValid() && ParentItem != RootItem && !Items.Contains(ParentItem))
			{
				ParentItemsToSelect.Add(ParentItem);
			}
		}
	}

	SortAndSelectItems(ParentItemsToSelect.Array());
}

bool FAvaOutlinerView::CanSelectParent() const
{
	return GetViewSelectedItemCount() == 1;
}

void FAvaOutlinerView::SelectFirstChild()
{
	TArray<FAvaOutlinerItemPtr> Items = GetViewSelectedItems();
	TSet<FAvaOutlinerItemPtr> FirstChildItemsToSelect;
	FirstChildItemsToSelect.Reserve(Items.Num());

	for (const FAvaOutlinerItemPtr& Item : Items)
	{
		if (Item.IsValid())
		{
			FAvaOutlinerItemPtr FirstChildItem = GetVisibleChildAt(Item, 0);

			//Don't select Component items! (Component items on selection also select their owner actor items, which can cause undesired issues)
			if (FirstChildItem.IsValid() && !FirstChildItem->IsA<FAvaOutlinerComponent>())
			{
				FirstChildItemsToSelect.Add(FirstChildItem);
			}
		}
	}
	
	SortAndSelectItems(FirstChildItemsToSelect.Array());
}

bool FAvaOutlinerView::CanSelectFirstChild() const
{
	return GetViewSelectedItemCount() == 1;
}

void FAvaOutlinerView::SelectSibling(int32 InDeltaIndex)
{
	TArray<FAvaOutlinerItemPtr> Items = GetViewSelectedItems();
	TSet<FAvaOutlinerItemPtr> SiblingItemsToSelect;
	SiblingItemsToSelect.Reserve(Items.Num());

	for (const FAvaOutlinerItemPtr& Item : Items)
	{
		if (Item.IsValid() && Item->GetParent())
		{
			FAvaOutlinerItemPtr ParentItem = Item->GetParent();

			const int32 ItemIndex   = GetVisibleChildIndex(ParentItem, Item);
			const int32 TargetIndex = ItemIndex + InDeltaIndex;

			//Don't try to Normalize Index, if it's Invalid, we won't cycle and just skip that selection
			FAvaOutlinerItemPtr SiblingToSelect = GetVisibleChildAt(ParentItem, TargetIndex);

			//Don't select Component items! (Component items on selection also select their owner actor items, which can cause undesired issues)
			if (SiblingToSelect.IsValid() && !SiblingToSelect->IsA<FAvaOutlinerComponent>())
			{
				SiblingItemsToSelect.Add(SiblingToSelect);
			}
		}
	}
	SortAndSelectItems(SiblingItemsToSelect.Array());
}

bool FAvaOutlinerView::CanSelectSibling() const
{
	return GetViewSelectedItemCount() == 1;
}

void FAvaOutlinerView::ExpandAll()
{
	for (const FAvaOutlinerItemPtr& Item : RootVisibleItems)
	{
		SetItemExpansionRecursive(Item, true);
	}
}

bool FAvaOutlinerView::CanExpandAll() const
{
	return true;
}

void FAvaOutlinerView::CollapseAll()
{
	for (const FAvaOutlinerItemPtr& Item : RootVisibleItems)
	{
		SetItemExpansionRecursive(Item, false);
	}
}

bool FAvaOutlinerView::CanCollapseAll() const
{
	return true;
}

void FAvaOutlinerView::ScrollNextIntoView()
{
	ScrollDeltaIndexIntoView(+1);
}

void FAvaOutlinerView::ScrollPrevIntoView()
{
	ScrollDeltaIndexIntoView(-1);
}

bool FAvaOutlinerView::CanScrollNextIntoView() const
{
	return GetViewSelectedItemCount() > 0;
}

void FAvaOutlinerView::ScrollDeltaIndexIntoView(int32 DeltaIndex)
{
	const int32 SelectedItemCount = SortedSelectedItems.Num();
	if (SortedSelectedItems.Num() > 0)
	{
		const int32 TargetIndex  = NextSelectedItemIntoView + DeltaIndex;
		NextSelectedItemIntoView = TargetIndex % SelectedItemCount;
		if (NextSelectedItemIntoView < 0)
		{
			NextSelectedItemIntoView += SelectedItemCount;
		}
		ScrollItemIntoView(SortedSelectedItems[NextSelectedItemIntoView]);
	}
}

void FAvaOutlinerView::ScrollItemIntoView(const FAvaOutlinerItemPtr& InItem)
{
	if (InItem.IsValid())
	{
		SetParentItemExpansions(InItem, true);
		if (OutlinerWidget.IsValid() && OutlinerWidget->GetTreeView())
		{
			OutlinerWidget->GetTreeView()->FocusOnItem(InItem);
			OutlinerWidget->ScrollItemIntoView(InItem);
		}
	}
}

void FAvaOutlinerView::SortAndSelectItems(TArray<FAvaOutlinerItemPtr> InItemsToSelect)
{
	if (!InItemsToSelect.IsEmpty())
	{
		FAvaOutliner::SortItems(InItemsToSelect);

		SelectItems(InItemsToSelect
			, EAvaOutlinerItemSelectionFlags::SignalSelectionChange
			| EAvaOutlinerItemSelectionFlags::ScrollIntoView);
	}
}

void FAvaOutlinerView::PopulateItemContextMenu(UToolMenu* InToolMenu)
{
	if (!InToolMenu)
	{
		return;
	}

	UAvaOutlinerItemsContext* const ItemsContext = InToolMenu->FindContext<UAvaOutlinerItemsContext>();
	if (!ItemsContext)
	{
		return;
	}

	// Generic Actions
	{
		FToolMenuSection& GenericActionsSection = InToolMenu->AddSection(TEXT("GenericActions"), LOCTEXT("OutlinerGenericActionsHeader", "Generic Actions"));
		
		const FGenericCommands& GenericCommands = FGenericCommands::Get();
		GenericActionsSection.AddMenuEntry(GenericCommands.Cut);
		GenericActionsSection.AddMenuEntry(GenericCommands.Copy);
		GenericActionsSection.AddMenuEntry(GenericCommands.Paste);
		GenericActionsSection.AddMenuEntry(GenericCommands.Duplicate);
		GenericActionsSection.AddMenuEntry(GenericCommands.Delete);
		GenericActionsSection.AddMenuEntry(GenericCommands.Rename);
	}

	// Outliner Commands
	{
		FToolMenuSection& OutlinerActionsSection = InToolMenu->AddSection(TEXT("OutlinerActions"), LOCTEXT("OutlinerActionsHeader", "Outliner Actions"));
		
		const FAvaOutlinerCommands& OutlinerCommands = FAvaOutlinerCommands::Get();
		OutlinerActionsSection.AddMenuEntry(OutlinerCommands.ExpandAll);
		OutlinerActionsSection.AddMenuEntry(OutlinerCommands.CollapseAll);
		OutlinerActionsSection.AddMenuEntry(OutlinerCommands.SelectAllChildren);
		OutlinerActionsSection.AddMenuEntry(OutlinerCommands.SelectImmediateChildren);
	}

	const TSharedPtr<IAvaOutliner> Outliner = ItemsContext->GetOutliner();
	if (!Outliner.IsValid())
	{
		return;
	}

	// Give Option to Quickly Extend without having to implement UToolMenus::Get()->ExtendMenu
	Outliner->GetProvider().ExtendOutlinerItemContextMenu(InToolMenu);
	IAvaOutlinerModule::Get().GetOnExtendOutlinerItemContextMenu().Broadcast(InToolMenu);
}

void FAvaOutlinerView::PopulateToolBar(UToolMenu* InToolMenu)
{
	if (!InToolMenu)
	{
		return;
	}

	UAvaOutlinerViewToolbarContext* const ToolBarContext = InToolMenu->FindContext<UAvaOutlinerViewToolbarContext>();
	if (!ToolBarContext)
	{
		return;
	}

	if (!ToolBarContext->GetOutlinerView().IsValid())
	{
		return;
	}

	TSharedRef<FAvaOutlinerView> OutlinerView = ToolBarContext->GetOutlinerView().ToSharedRef();

	TSharedRef<SWidget> Widget = OutlinerView->GetOutlinerWidget();
	if (Widget == SNullWidget::NullWidget)
	{
		return;
	}

	TSharedRef<SAvaOutliner> OutlinerWidget = StaticCastSharedRef<SAvaOutliner>(Widget);

	FToolMenuSection& MainSection = InToolMenu->AddSection(TEXT("Main"));

	// View Options
	const FToolMenuEntry ViewOptionsEntry = FToolMenuEntry::InitComboButton("ViewOptions", FUIAction()
			, FOnGetContent::CreateSP(OutlinerView, &FAvaOutlinerView::CreateOutlinerViewOptionsMenu)
			, LOCTEXT("ViewOptionsLabel", "View Options")
			, LOCTEXT("ViewOptionsToolTip", "View Options")
			, FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Visibility"));
	MainSection.AddEntry(ViewOptionsEntry);

	// Settings
	const FToolMenuEntry SettingsEntry = FToolMenuEntry::InitComboButton("Settings", FUIAction()
			, FOnGetContent::CreateSP(OutlinerView, &FAvaOutlinerView::CreateOutlinerSettingsMenu)
			, LOCTEXT("SettingsLabel", "Settings")
			, LOCTEXT("SettingsToolTip", "Settings")
			, FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Settings"));
	MainSection.AddEntry(SettingsEntry);

	// Give Option to Quickly Extend without having to implement UToolMenus::Get()->ExtendMenu
	if (TSharedPtr<FAvaOutliner> Outliner = OutlinerView->GetOutliner())
	{
		Outliner->GetProvider().ExtendOutlinerToolBar(InToolMenu);
	}
}

void FAvaOutlinerView::RefreshOutliner(bool bInImmediateRefresh)
{
	if (TSharedPtr<FAvaOutliner> Outliner = GetOutliner())
	{
		if (bInImmediateRefresh)
		{
			Outliner->Refresh();
		}
		else
		{
			Outliner->RequestRefresh();
		}
	}
}

#undef LOCTEXT_NAMESPACE
