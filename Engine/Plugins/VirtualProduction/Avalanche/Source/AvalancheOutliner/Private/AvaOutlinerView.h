// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "IAvaOutlinerView.h"
#include "Item/AvaOutlinerItemId.h"

class FAvaOutliner;
class FAvaOutlinerTextFilter;
class FDragDropEvent;
class FReply;
class FUICommandList;
class IAvaOutlinerColumn;
class IAvaOutlinerItemFilter;
class SAvaOutliner;
class SBorder;
class SHorizontalBox;
class SWidget;
class UToolMenu;
enum class ECheckBoxState : uint8;
enum class EItemDropZone;
struct FAvaOutlinerSaveState;
struct FAvaOutlinerStats;
struct FGeometry;
struct FPointerEvent;

/**
  * A view instance of the Outliner, that handles viewing a subset of the outliner items based on
  * item filters, search text, hierarchy type, etc
  */
class FAvaOutlinerView : public IAvaOutlinerView
{
	friend FAvaOutlinerSaveState;
	
	struct FPrivateToken { explicit FPrivateToken() = default; };

	/** Initializes the Outliner Instance. Only executed once */
	void Init(const TSharedRef<FAvaOutliner>& InOutliner, bool bCreateOutlinerWidget);

	/** Creates the Base Columns and any other columns gotten via the Module or Provider */
	void CreateColumns();
	
public:
	explicit FAvaOutlinerView(FPrivateToken);

	virtual ~FAvaOutlinerView() override;

	/** Gets the Tool Menu name for the ToolBar in each Outliner View Widget */
	static FName GetOutlinerToolbarName();

	/** Gets the Tool Menu name for the Item Context Menu */
	static FName GetOutlinerItemContextMenuName();
	
	/**
	 * Creates an Outliner View Instance and register it to the Outliner
	 * @param InOutlinerViewId the Id assigned to the Instance created
	 * @param InOutliner the Outliner this Instance View belongs to
	 * @param bCreateOutlinerWidget whether to create an SAvaOutliner to this view. Can be false if used for testing
	 */
	static TSharedRef<FAvaOutlinerView> CreateInstance(int32 InOutlinerViewId
		, const TSharedRef<FAvaOutliner>& InOutliner
		, bool bCreateOutlinerWidget);

	void PostLoad();
	
	/** Called when the UAvaOutlinerSettings has a property change */
	void OnOutlinerSettingsChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);

	/**
	 * Retrieves the latest list of Custom Filters specified in UAvaOutlinerSettings
	 * and recreates the filter instances for these
	 */
	void UpdateCustomFilters();
	
	DECLARE_MULTICAST_DELEGATE(FOnCustomFiltersChanged)
	FOnCustomFiltersChanged& GetOnCustomFiltersChanged() { return OnCustomFiltersChanged; }
	
	void Tick(float InDeltaTime);

	/**
	 * Saves the Outliner View Data into the Save State object in FAvaOutliner
	 * @see FAvaOutlinerSaveState::SaveOutlinerViewState
	 */
	void SaveState();

	/**
	 * Loads the data found in the FAvaOutliner's SaveState into the respective properties in this View
	 * @see FAvaOutlinerSaveState::LoadOutlinerViewState
	 */
	void LoadState();
	
	void BindCommands(const TSharedPtr<FUICommandList>& InBaseCommandList);

	TSharedPtr<FUICommandList> GetBaseCommandList() const;

	TSharedPtr<FUICommandList> GetViewCommandList() const { return ViewCommandList; }

	/** Notifies FAvaOutliner so that this view instance becomes the most recent interacted outliner view */
	void UpdateRecentOutlinerViews();

	/** Returns whether this view instance is the most recently interacted between the instances in FAvaOutliner */
	bool IsMostRecentOutlinerView() const;
	
	int32 GetOutlinerViewId() const { return OutlinerViewId; }

	TSharedPtr<FAvaOutliner> GetOutliner() const { return OutlinerWeak.Pin(); }

	TSharedRef<FAvaOutlinerStats> GetOutlinerStats() const { return OutlinerStats; }

	//~ Begin IAvaOutlinerView
	virtual TSharedRef<SWidget> GetOutlinerWidget() const override;
	virtual TSharedPtr<IAvaOutliner> GetOwnerOutliner() const override;
	//~ End IAvaOutlinerView

	void CreateToolbar();
	
	TSharedPtr<SWidget> CreateItemContextMenu();
	
	TSharedRef<SWidget> CreateOutlinerSettingsMenu();

	TSharedRef<SWidget> CreateOutlinerViewOptionsMenu();
	
	/** Get the Columns currently allocated in this Outliner Instance */
	const TMap<FName, TSharedPtr<IAvaOutlinerColumn>>& GetColumns() const { return Columns; }

	/** Whether the given column should be shown by default. This is only used at the start when Creating the Columns */
	bool ShouldShowColumnByDefault(const TSharedPtr<IAvaOutlinerColumn>& InColumn) const;

	/** Updates the Cached Column Visibility Map to the latest current state of this View's Widget Columns */
	void UpdateColumnVisibilityMap();

	/** Marks the Outliner View to be refreshed on Next Tick */
	void RequestRefresh();
	
	DECLARE_MULTICAST_DELEGATE(FOnOutlinerViewRefreshed);
	FOnOutlinerViewRefreshed& GetOnOutlinerViewRefreshed() { return OnOutlinerViewRefreshed; }
	
	/** Refreshes the Items visible in this View, and refreshes the Outliner Widget if it was created */
	void Refresh();

	void SetKeyboardFocus();
	
	/** Refreshes the Items that will be at the Top Level of the Tree (e.g. Actors with no parent item) */
	void UpdateRootVisibleItems();

	/** Updates the Item Expansions in the Tree Widget based on whether the Items have the Expanded flag or not */
	void UpdateItemExpansions();

	/** Called when Object Replacement has taken place. Used to invalidate the widget for painting */
	void NotifyObjectsReplaced();

	/** Called when the Text Filter has changed and thus needing to refresh to visualize the new filtered items */
	void OnFilterChanged();

	/** The Root Item of the Tree. Convenience function to get the RootItem from FAvaOutliner */
	FAvaOutlinerItemPtr GetRootItem() const;

	/** Gets the Top Level Items that should be visible in the Tree. This can differ between views depending on Filters and other factors */
	const TArray<FAvaOutlinerItemPtr>& GetRootVisibleItems() const { return RootVisibleItems; }

	void SetViewItemFlags(const FAvaOutlinerItemPtr& InItem, EAvaOutlinerItemFlags InFlags);
	
	EAvaOutlinerItemFlags GetViewItemFlags(const FAvaOutlinerItemPtr& InItem) const;

	void GetChildrenOfItem(FAvaOutlinerItemPtr InItem, TArray<FAvaOutlinerItemPtr>& OutChildren) const;

	/**
	 * Gets the Children of a given Item. Can recurse if the immediate child is hidden (the children of these hidden items should still be given the opportunity to show up)
	 * @param InItem the Item to get the children of
	 * @param OutChildren the visible children in the give view mode.
	 * @param InViewMode the view mode(s) that the children should support to be added to OutChildren
	 * @param InRecursionDisallowedItems the items where recursion should not be performed
	 */
	void GetChildrenOfItem(FAvaOutlinerItemPtr InItem
		, TArray<FAvaOutlinerItemPtr>& OutChildren
		, EAvaOutlinerItemViewMode InViewMode
		, const TSet<FAvaOutlinerItemPtr>& InRecursionDisallowedItems) const;

	FLinearColor GetItemBrushColor(FAvaOutlinerItemPtr InItem) const;
	
	const TArray<TSharedPtr<IAvaOutlinerItemFilter>>& GetItemFilters() const { return ItemFilters; }
	
	const TArray<TSharedPtr<IAvaOutlinerItemFilter>>& GetCustomItemFilters() const { return CustomItemFilters; }

	const TSet<TSharedPtr<IAvaOutlinerItemFilter>>& GetActiveItemFilters() const { return ActiveItemFilters; }

	TSharedRef<FAvaOutlinerTextFilter> GetTextFilter() const { return TextFilter; }
	
	/** Gets the Currently Selected Items in the Tree View */
	TArray<FAvaOutlinerItemPtr> GetViewSelectedItems() const;

	/** Gets the Currently Selected Item Count in the Tree View */
	int32 GetViewSelectedItemCount() const;

	/** Calculates the amount of Items that are visible in this Outliner View */
	int32 CalculateVisibleItemCount() const;
	
	/** Selects the Item in this Outliner View */
	void SelectItems(TArray<FAvaOutlinerItemPtr> InItems, EAvaOutlinerItemSelectionFlags InFlags = EAvaOutlinerItemSelectionFlags::SignalSelectionChange);

	/** Clears the currently Selected Items in the Outliner View */
	void ClearItemSelection(bool bSignalSelectionChange = true);

	/** Whether Sync Selection is currently taking place */
	bool IsSyncingItemSelection() const { return bSyncingItemSelection; }
	
private:
	/** Sets the Selected Items to be the given Items in this List. */
	void SetItemSelectionImpl(TArray<FAvaOutlinerItemPtr>&& InItems
		, bool bSignalSelectionChange);

public:
	/** Called when Item selection has changed. This can be called by the OutlinerWidget or by the View itself if there's no OutlinerWidget */
	void NotifyItemSelectionChanged(const TArray<FAvaOutlinerItemPtr>& InSelectedItems
		, const FAvaOutlinerItemPtr& InItem
		, bool bUpdateModeTools);

	/** Whether the given Item is explicitly marked as Read-only in the Outliner */
	bool IsItemReadOnly(const FAvaOutlinerItemPtr& InItem) const;
	
	/** Whether the given Item can be selected in this Outliner View / Widget */
	bool CanSelectItem(const FAvaOutlinerItemPtr& InItem) const;

	/** Whether the current item is selected in this Outliner View / Widget */
	bool IsItemSelected(const FAvaOutlinerItemPtr& InItem) const;

	/** Change the expansion state of the parents (recursively) of the given item */
	void SetParentItemExpansions(const FAvaOutlinerItemPtr& InItem, bool bIsExpanded);

	/** Expands / Collapses the given Item */
	void SetItemExpansion(const FAvaOutlinerItemPtr& InItem, bool bIsExpanded, bool bInUseFilter = true);

	/** Expands / Collapses the given Item and its children recursively */
	void SetItemExpansionRecursive(FAvaOutlinerItemPtr InItem, bool bIsExpanded);

	/** Called when the given item's expansion state (expanded/collapsed) has changed */
	void OnItemExpansionChanged(FAvaOutlinerItemPtr InItem, bool bIsExpanded);

	/** Return whether the given item should be visible in this Outliner Instance */
	bool ShouldShowItem(const FAvaOutlinerItemPtr& InItem, bool bInUseFilters, EAvaOutlinerItemViewMode InViewMode) const;

	/** Returns the Index of the Child from the Parent's Children Visible List (Visible List can vary between Outliner Instances) */
	int32 GetVisibleChildIndex(const FAvaOutlinerItemPtr& InParentItem, const FAvaOutlinerItemPtr& InChildItem) const;

	/** Returns the Child from the Parent's Children Visible List at the given Index (Visible List can vary between Outliner Instances) */
	FAvaOutlinerItemPtr GetVisibleChildAt(const FAvaOutlinerItemPtr& InParentItem, int32 InChildIndex) const;

	/** Returns whether the Outliner is Locked. See FAvaOutliner::IsOutlinerLocked */
	bool IsOutlinerLocked() const;

	/**
	 * Activate the given Item Filter
	 * @param InFilter the filter to activate
	 * @param bInRefreshOutliner whether to refresh this view if the filter was added. false recommended if doing operations in bulk
	 */
	void EnableItemFilter(const TSharedPtr<IAvaOutlinerItemFilter>& InFilter, const bool bInRefreshOutliner = true);

	/**
 	* Activate the Item Filter with the given ID
 	* @param InFilterId the filter id associated with the Filter to activate
 	* @param bInRefreshOutliner whether to refresh this view if the filter was added. false recommended if doing operations in bulk
 	*/
	void EnableItemFilterById(const FName& InFilterId, const bool bInRefreshOutliner = true);

	/**
	 * Deactivates the given Item Filter
	 * @param InFilter the filter to remove from the active filter list
	 * @param bInRefreshOutliner whether to refresh this view if the filter was removed. false recommended if doing operations in bulk
	 */
	void DisableItemFilter(const TSharedPtr<IAvaOutlinerItemFilter>& InFilter, const bool bInRefreshOutliner = true);

	/** Whether the given Filter is in the Active Filter List */
	bool IsItemFilterEnabled(const TSharedPtr<IAvaOutlinerItemFilter>& InFilter) const;

	/** Sets the Filter Text */
	void SetSearchText(const FText& InText);

	/** Gets the Current Filter Text */
	FText GetSearchText() const;

	EAvaOutlinerItemViewMode GetItemDefaultViewMode() const { return ItemDefaultViewMode; }
	EAvaOutlinerItemViewMode GetItemProxyViewMode() const { return ItemProxyViewMode; }

	void ToggleViewModeSupport(EAvaOutlinerItemViewMode& InOutViewMode, EAvaOutlinerItemViewMode InFlags);
	void ToggleItemDefaultViewModeSupport(EAvaOutlinerItemViewMode InFlags);
	void ToggleItemProxyViewModeSupport(EAvaOutlinerItemViewMode InFlags);

	ECheckBoxState GetViewModeCheckState(EAvaOutlinerItemViewMode InViewMode, EAvaOutlinerItemViewMode InFlags) const;
	ECheckBoxState GetItemDefaultViewModeCheckState(EAvaOutlinerItemViewMode InFlags) const;
	ECheckBoxState GetItemProxyViewModeCheckState(EAvaOutlinerItemViewMode InFlags) const;
	
	/**
	 * Action to turn Muted Hierarchy on or off
	 * @see bUseMutedHierarchy
	 */
	void ToggleMutedHierarchy();
	bool CanToggleMutedHierarchy() const { return true; }
	bool IsMutedHierarchyActive() const { return bUseMutedHierarchy; }

	/**
	 * Action to turn Auto Expand to Selection on or off
	 * @see bAutoExpandToSelection
	 */
	void ToggleAutoExpandToSelection();
	bool CanToggleAutoExpandToSelection() const { return true; }
	bool ShouldAutoExpandToSelection() const { return bAutoExpandToSelection; }

	void ToggleShowItemFilters();
	bool CanToggleShowItemFilters() const { return true; }
	bool ShouldShowItemFilters() const { return bShowItemFilters; }

	/** Sets whether the given item type should be hidden or not */
	void SetItemTypeHidden(FName InItemTypeName, bool bHidden);
	
	/** Toggles the given Item Types to Hide/Show */
	void ToggleHideItemTypes(FName InItemTypeName);
	ECheckBoxState GetToggleHideItemTypesState(FName InItemTypeName) const;
	
	/** Hides the given Outliner Item Type from showing in this Outliner View (e.g. Hide Components is used) */
	template<typename InItemType, typename = typename TEnableIf<TIsDerivedFrom<InItemType, IAvaOutlinerItem>::IsDerived>::Type>
	void HideItemType()
	{
		SetItemTypeHidden(TAvaType<InItemType>::GetTypeId().ToName(), true);
	}
	
	/** Shows the given Outliner Item Type if it was hidden in this Outliner View (e.g. Showing Components again) */
	template<typename InItemType, typename = typename TEnableIf<TIsDerivedFrom<InItemType, IAvaOutlinerItem>::IsDerived>::Type>
	void ExposeItemType()
	{
		SetItemTypeHidden(TAvaType<InItemType>::GetTypeId().ToName(), false);
	}
	
	/** Whether the given Outliner Item Type is currently hidden in this Outliner View */
	bool IsItemTypeHidden(FName InItemTypeName) const;
	bool IsItemTypeHidden(const FAvaOutlinerItemPtr& InItem) const;
	
	template<typename InItemType, typename = typename TEnableIf<TIsDerivedFrom<InItemType, IAvaOutlinerItem>::IsDerived>::Type>
	bool IsItemTypeHidden() const
	{
		return IsItemTypeHidden(InItemType::GetStaticTypeName());
	}

	/** Called when a Drag enters the Outliner Widgets for the given Item (if null, treat as root) */
	void OnDragEnter(const FDragDropEvent& InDragDropEvent, FAvaOutlinerItemPtr InTargetItem);

	/** Called when a Drag leaves the Outliner Widgets for the given Item (if null, treat as root) */
	void OnDragLeave(const FDragDropEvent& InDragDropEvent, FAvaOutlinerItemPtr InTargetItem);
	
	/** Called a Drag is attempted for the selected items in view */
	FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent, FAvaOutlinerItemPtr InTargetItem);

	/** Processes the Drag and Drop for the given Item if valid, else it will process it for the Root Item */
	FReply OnDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem);

	/** Determines whether the Drag and Drop can be processed by the given item */
	TOptional<EItemDropZone> OnCanDrop(const FDragDropEvent& InDragDropEvent
		, EItemDropZone InDropZone
		, FAvaOutlinerItemPtr InTargetItem) const;

	/** Called when there's a Drag being started/ended to the Outliner Widget as a whole (i.e. root) and not an individual item */
	void SetDragIntoTreeRoot(bool bInIsDraggingIntoTreeRoot);
	
	void RenameSelected();
	void ResetRenaming();
	void OnItemRenameAction(EAvaOutlinerRenameAction InRenameAction, const TSharedPtr<FAvaOutlinerView>& InOutlinerView);
	bool CanRenameSelected() const;

	void DeleteSelected();
	bool CanDeleteSelected() const;

	void DuplicateSelected();
	bool CanDuplicateSelected() const;

	void SelectChildren(bool bIsRecursive);
	bool CanSelectChildren() const;

	void SelectParent();
	bool CanSelectParent() const;

	void SelectFirstChild();
	bool CanSelectFirstChild() const;

	void SelectSibling(int32 InDeltaIndex);
	bool CanSelectSibling() const;

	void ExpandAll();
	bool CanExpandAll() const;

	void CollapseAll();
	bool CanCollapseAll() const;

	void ScrollNextIntoView();
	void ScrollPrevIntoView();
	bool CanScrollNextIntoView() const;
	void ScrollDeltaIndexIntoView(int32 DeltaIndex);
	void ScrollItemIntoView(const FAvaOutlinerItemPtr& InItem);

	/** Sorts the given Items in the user-defined order and selects them, and if the Widget is valid focusing to the item in the Widget */
	void SortAndSelectItems(TArray<FAvaOutlinerItemPtr> InItemsToSelect);
	
private:
	/** Static Function to Populate the Outliner Item Context Menu. The Outliner View and Context Items are gotten via UAvaOutlinerItemsContext */
	static void PopulateItemContextMenu(UToolMenu* InToolMenu);
	
	/** Static Function to Populate the Outliner Main Toolbar. The Outliner View is gotten via UAvaOutlinerToolbarContext */
	static void PopulateToolBar(UToolMenu* InToolMenu);
	
	/** Triggers a Refresh on the FAvaOutliner */
	void RefreshOutliner(bool bInImmediateRefresh);
	
	/** Local Identifier of this Instance */
	int32 OutlinerViewId = -1;

	/** Weak pointer to the outliner this is a view of */
	TWeakPtr<FAvaOutliner> OutlinerWeak;

	/** Widget wrapping the outliner widget. Used so that the outliner widget's lifetime can be controlled by this container */
	TSharedPtr<SBorder> OutlinerWidgetContainer;

	/** Widget showing the View. Can be null if instanced for testing (i.e. bCreateOutlinerWidget is false in Init) */
	TSharedPtr<SAvaOutliner> OutlinerWidget;

	/** Command List mapped to the View to handle things like Selected Items */
	TSharedPtr<FUICommandList> ViewCommandList;
	
	/** Root Items from Outliner visible to this Instance. i.e. a Subset of all the Items in FAvaOutliner */
	TArray<FAvaOutlinerItemPtr> RootVisibleItems;

	/** A list of the Selected Items in this Outliner View */
	TArray<FAvaOutlinerItemPtr> SelectedItems;
	
	/** Set of Items that are Currently Read Only in this Instance. Mutable as this changes on GetChildrenOfItem const func */
	mutable TSet<FAvaOutlinerItemPtr> ReadOnlyItems;

	/** Items specific to this Outliner Instance, rather than being shared across Outliners (e.g. Expansion flags) */
	TMap<FAvaOutlinerItemId, EAvaOutlinerItemFlags> ViewItemFlags;

	TSharedRef<FAvaOutlinerTextFilter> TextFilter;

	/** Native-constructed Item Filters in the Outliner */
	TArray<TSharedPtr<IAvaOutlinerItemFilter>> ItemFilters;

	/** Custom Item Filters constructed from UAvaOutlinerSettings */
	TArray<TSharedPtr<IAvaOutlinerItemFilter>> CustomItemFilters;

	/** Delegate called when the Custom Item Filters have been modified, as these can be instanced after */
	FOnCustomFiltersChanged OnCustomFiltersChanged;

	/** Delegate called at the end of the Outliner View Refresh */
	FOnOutlinerViewRefreshed OnOutlinerViewRefreshed;
	
	/** Active Item Filters. Subset of ItemFilters */
	TSet<TSharedPtr<IAvaOutlinerItemFilter>> ActiveItemFilters;

	/** Map of the Column Ids and the Colum with that Id. This is filled in FAvaOutlinerView::CreateColumns */
	TMap<FName, TSharedPtr<IAvaOutlinerColumn>> Columns;

	/** Map of the Column Ids to their Override (i.e. saved) Visibility */
	TMap<FName, bool> ColumnVisibility;
	
	/** A set of the Outliner Item Type Names that should be hidden in the Outliner */
	TSet<FName> HiddenItemTypes;

	/** The Index of the Item that will be scrolled into View */
	int32 NextSelectedItemIntoView = -1;

	/** Selected Items Sorted from Top To Bottom */
	TArray<FAvaOutlinerItemPtr> SortedSelectedItems;

	/** Stats for the Outliner Widget */
	TSharedRef<FAvaOutlinerStats> OutlinerStats;

	/** The list of items that need renaming and are waiting for their turn to be renamed */
	TArray<FAvaOutlinerItemPtr> ItemsRemainingRename;
	
	/** The current item in the process of renaming. Null if no renaming is taking place */
	FAvaOutlinerItemPtr CurrentItemRenaming;
	
	/** This Outliner View's Setting for the Default Item's Default View Mode */
	EAvaOutlinerItemViewMode ItemDefaultViewMode = EAvaOutlinerItemViewMode::HorizontalItemList;
	
	/** This Outliner View's Setting for the Item Proxy's Default View Mode */
	EAvaOutlinerItemViewMode ItemProxyViewMode = EAvaOutlinerItemViewMode::None;
	
	/** Whether the expand the parents of an item when it's selected */
	bool bAutoExpandToSelection = false;
	
	/** Whether to also show the Parent Items in read-only mode when these are filtered out */
	bool bUseMutedHierarchy = false;
	
	/** Flag to call FAvaOutlinerView::Refresh next tick */
	bool bRefreshRequested = false;
	
	/** Whether Item renaming is taking place in this View */
	bool bRenamingItems = false;
	
	/** Whether Renaming Items should be processed next tick */
	bool bRequestedRename = false;

	/** Whether Items are currently being synced to what the Widget or Interface sent. Used as a re-enter guard */
	bool bSyncingItemSelection = false;
	
	/** Flag used for the Outliner Widget to determine if the Item Filter Bar should open and show the Item Filters */
	bool bShowItemFilters = false;
};
