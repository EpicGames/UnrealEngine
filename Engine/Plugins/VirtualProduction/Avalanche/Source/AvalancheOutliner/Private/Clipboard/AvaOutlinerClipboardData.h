// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "UObject/SoftObjectPtr.h"
#include "AvaOutlinerClipboardData.generated.h"

class AActor;

UCLASS()
class UAvaOutlinerClipboardData : public UObject
{
	GENERATED_BODY()

public:
	/** Copied Actor names in the order that they should be processed by Outliner */
	UPROPERTY()
	TArray<FName> ActorNames;
};
