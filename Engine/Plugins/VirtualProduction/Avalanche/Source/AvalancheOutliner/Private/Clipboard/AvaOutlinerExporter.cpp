// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerExporter.h"
#include "AvaOutliner.h"
#include "AvaOutlinerClipboardData.h"
#include "AvaOutlinerDefines.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Item/AvaOutlinerActor.h"
#include "Misc/StringOutputDevice.h"
#include "UObject/Package.h"
#include "UnrealExporter.h"

FAvaOutlinerExporter::FAvaOutlinerExporter(const TSharedRef<FAvaOutliner>& InAvaOutliner)
	: AvaOutlinerWeak(InAvaOutliner)
{
}

void FAvaOutlinerExporter::ExportText(FString& InOutCopiedData, TConstArrayView<AActor*> InCopiedActors)
{
	UAvaOutlinerClipboardData* const ClipboardData = CreateClipboardData(InCopiedActors);
	if (!ensureAlways(ClipboardData))
	{
		return;
	}

	FExportObjectInnerContext ExportContext;

	constexpr const TCHAR* Filetype = TEXT("copy");
	constexpr uint32 PortFlags      = PPF_DeepCompareInstances | PPF_ExportsNotFullyQualified;
	constexpr int32 IndentLevel     = 0;

	FStringOutputDevice Ar;
	Ar.Logf(TEXT("%sBegin Outliner\r\n"), FCString::Spc(IndentLevel));
	UExporter::ExportToOutputDevice(&ExportContext, ClipboardData, nullptr, Ar, Filetype, IndentLevel + 3, PortFlags);
	Ar.Logf(TEXT("%sEnd Outliner\r\n"), FCString::Spc(IndentLevel));
	InOutCopiedData += Ar;
}

UAvaOutlinerClipboardData* FAvaOutlinerExporter::CreateClipboardData(TConstArrayView<AActor*> InCopiedActors)
{
	TSharedPtr<FAvaOutliner> AvaOutliner = AvaOutlinerWeak.Pin();
	if (!AvaOutliner.IsValid())
	{
		return nullptr;
	}

	UAvaOutlinerClipboardData* const ClipboardData = NewObject<UAvaOutlinerClipboardData>(GetTransientPackage());
	if (!ClipboardData)
	{
		return nullptr;
	}

	TArray<FAvaOutlinerItemPtr> Items;
	Items.Reserve(InCopiedActors.Num());
	ClipboardData->ActorNames.Reserve(InCopiedActors.Num());

	for (AActor* const Actor : InCopiedActors)
	{
		if (!Actor)
		{
			continue;
		}

		FAvaOutlinerItemPtr ActorItem = AvaOutliner->FindOrAdd<FAvaOutlinerActor>(Actor);
		if (ActorItem->IsAllowedInOutliner())
		{
			Items.Add(ActorItem);
		}
	}

	FAvaOutliner::SortItems(Items);

	for (const FAvaOutlinerItemPtr& Item : Items)
	{
		FAvaOutlinerActor* const ActorItem = Item->CastTo<FAvaOutlinerActor>();
		if (!ActorItem)
		{
			continue;
		}

		AActor* const Actor = ActorItem->GetActor();
		if (!Actor)
		{
			continue;
		}

		ClipboardData->ActorNames.Add(*Actor->GetPathName(Actor->GetWorld()));
	}

	return ClipboardData;
}
