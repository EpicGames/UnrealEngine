// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Exporters/Exporter.h"
#include "Templates/SharedPointer.h"

class UAvaOutlinerClipboardData;
class AActor;
class FAvaOutliner;

/** Handles exporting the given Copied Actors' Outliner Data to String Text */
class FAvaOutlinerExporter
{
public:
	explicit FAvaOutlinerExporter(const TSharedRef<FAvaOutliner>& InAvaOutliner);

	void ExportText(FString& InOutCopiedData, TConstArrayView<AActor*> InCopiedActors);

private:
	UAvaOutlinerClipboardData* CreateClipboardData(TConstArrayView<AActor*> InCopiedActors);

	TWeakPtr<FAvaOutliner> AvaOutlinerWeak;
};
