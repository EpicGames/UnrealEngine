// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerImporter.h"
#include "AvaOutliner.h"
#include "AvaOutlinerClipboardData.h"
#include "Factories.h"
#include "UObject/Package.h"

FAvaOutlinerImporter::FAvaOutlinerImporter(const TSharedRef<FAvaOutliner>& InAvaOutliner)
	: AvaOutlinerWeak(InAvaOutliner)
{
}

void FAvaOutlinerImporter::ImportText(FStringView InPastedData, const TMap<FName, AActor*>& InPastedActors)
{
	if (InPastedActors.IsEmpty())
	{
		return;
	}

	TSharedPtr<FAvaOutliner> Outliner = AvaOutlinerWeak.Pin();
	if (!Outliner.IsValid())
	{
		return;
	}

	const TCHAR* Buffer = InPastedData.GetData();

	TArray<FName> ActorNames;

	FString StringLine;
	while (FParse::Line(&Buffer, StringLine))
	{
		const TCHAR* Str = *StringLine;
		if (!FAvaOutlinerImporter::ParseCommand(&Str, TEXT("Begin")))
		{
			continue;
		}

		FString OutlinerData, OutlinerDataLine;
		while (!FAvaOutlinerImporter::ParseCommand(&Buffer, TEXT("End")) && FParse::Line(&Buffer, OutlinerDataLine))
		{
			OutlinerData += *OutlinerDataLine;
			OutlinerData += TEXT("\r\n");
		}

		ActorNames = ImportOutlinerData(OutlinerData);
		break;
	}

	// Do not ignore Spawn here
	Outliner->SetIgnoreNotify(EAvaOutlinerIgnoreNotifyFlags::Spawn, false);
	for (const FName& ActorName : ActorNames)
	{
		if (AActor* const * const SpawnedActor = InPastedActors.Find(ActorName))
		{
			Outliner->OnActorSpawned(*SpawnedActor);
		}
	}
}

bool FAvaOutlinerImporter::ParseCommand(const TCHAR** InStream, const TCHAR* InToken)
{
	constexpr const TCHAR* const OutlinerToken = TEXT("Outliner");
	const TCHAR* Original = *InStream;

	if (FParse::Command(InStream, InToken) && FParse::Command(InStream, OutlinerToken))
	{
		return true;
	}

	*InStream = Original;

	return false;
}

TArray<FName> FAvaOutlinerImporter::ImportOutlinerData(const FString& InBuffer)
{
	class FAvaOutlinerClipboardDataTextFactory : public FCustomizableTextObjectFactory
	{
	public:
		FAvaOutlinerClipboardDataTextFactory() : FCustomizableTextObjectFactory(GWarn) {}

		virtual bool CanCreateClass(UClass* InObjectClass, bool& bOutOmitSubObjects) const override
		{
			return ActorNames.IsEmpty() && InObjectClass->IsChildOf<UAvaOutlinerClipboardData>();
		}

		virtual void ProcessConstructedObject(UObject* InObject) override
		{
			ActorNames = CastChecked<UAvaOutlinerClipboardData>(InObject)->ActorNames;
		}

		TArray<FName> ActorNames;
	};

	UPackage* const TempPackage = NewObject<UPackage>(nullptr, TEXT("/Temp/MotionDesignOutliner/Import"), RF_Transient);
	TempPackage->AddToRoot();

	// Turn the text buffer into objects
	FAvaOutlinerClipboardDataTextFactory Factory;
	Factory.ProcessBuffer(TempPackage, RF_Transactional, InBuffer);

	// Remove the temp package from the root now that it has served its purpose
	TempPackage->RemoveFromRoot();

	return Factory.ActorNames;
}
