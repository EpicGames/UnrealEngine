// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointer.h"
#include "UObject/SoftObjectPtr.h"

class AActor;
class FAvaOutliner;

/** Handles parsing String Data to import outliner data */
class FAvaOutlinerImporter
{
public:
	explicit FAvaOutlinerImporter(const TSharedRef<FAvaOutliner>& InAvaOutliner);

	void ImportText(FStringView InPastedData, const TMap<FName, AActor*>& InPastedActors);

private:
	static bool ParseCommand(const TCHAR** InStream, const TCHAR* InToken);

	TArray<FName> ImportOutlinerData(const FString& InBuffer);

	TWeakPtr<FAvaOutliner> AvaOutlinerWeak;
};
