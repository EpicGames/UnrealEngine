// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/AvaOutlinerColorColumn.h"
#include "Columns/Slate/SAvaOutlinerColor.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerColorColumn"

FText FAvaOutlinerColorColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("ColorColumn", "Color");
}

SHeaderRow::FColumn::FArguments FAvaOutlinerColorColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
	    .FixedWidth(24.f)
	    .HAlignCell(HAlign_Center)
	    .VAlignCell(VAlign_Center)
	    .DefaultLabel(GetColumnDisplayNameText())
	    .DefaultTooltip(GetColumnDisplayNameText())
		[
			SNew(SBox)
		];
}

TSharedRef<SWidget> FAvaOutlinerColorColumn::ConstructRowWidget(FAvaOutlinerItemPtr InItem
	, const TSharedRef<FAvaOutlinerView>& InOutlinerView
	, const TSharedRef<SAvaOutlinerTreeRow>& InRow)
{
	return SNew(SAvaOutlinerColor, InItem, InOutlinerView, InRow);
}

#undef LOCTEXT_NAMESPACE
