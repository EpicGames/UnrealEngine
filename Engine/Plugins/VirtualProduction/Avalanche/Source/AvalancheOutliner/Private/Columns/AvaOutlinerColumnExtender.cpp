// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/AvaOutlinerColumnExtender.h"
#include "Columns/IAvaOutlinerColumn.h"

void FAvaOutlinerColumnExtender::AddColumn(FAvaOutlinerColumnPtr InColumn
	, EAvaOutlinerExtensionPosition InExtensionPosition
	, FName InReferenceColumnId)
{
	//By default set the Placement Index to be the next index after the last.
	int32 PlacementIndex = Columns.Num();

	if (InReferenceColumnId != NAME_None)
	{
		const int32 ReferenceColumnIndex = FindColumnIndex(InReferenceColumnId);
		if (ReferenceColumnIndex != INDEX_NONE)
		{
			PlacementIndex = InExtensionPosition == EAvaOutlinerExtensionPosition::Before
				? ReferenceColumnIndex
				: ReferenceColumnIndex + 1;
		}
	}

	Columns.EmplaceAt(PlacementIndex, InColumn);
}

int32 FAvaOutlinerColumnExtender::FindColumnIndex(FName InColumnId) const
{
	const FAvaOutlinerColumnPtr* const FoundColumn = Columns.FindByPredicate([InColumnId](const FAvaOutlinerColumnPtr& InColumn)
	{
		return InColumn.IsValid() && InColumn->GetColumnId() == InColumnId;
	});

	if (FoundColumn)
	{
		return static_cast<int32>(FoundColumn - Columns.GetData());
	}
	return INDEX_NONE;
}
