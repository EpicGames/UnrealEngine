// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Templates/UnrealTypeTraits.h"

class IAvaOutlinerColumn;

/**
 * Extension Class to add Outliner Columns
 *
 * Example #1:
 * This would create a Tag Column at the end of the current column list (order matters!)
 *    ColumnExtender.AddColumn<FAvaOutlinerTagColumn>();
 *
 * Example #2:
 * This would create a Tag Column before the Label Column (if it doesn't exist, it's the same behavior as the above example)
 *    ColumnExtender.AddColumn<FAvaOutlinerTagColumn, EAvaOutlinerExtensionPosition::Before, FAvaOutlinerLabelColumn>();
 */
class FAvaOutlinerColumnExtender
{
	using FAvaOutlinerColumnPtr = TSharedPtr<IAvaOutlinerColumn>;

public:
	template<typename InColumnType, EAvaOutlinerExtensionPosition InExtensionPosition, typename InRefColumnType
		UE_REQUIRES(TIsDerivedFrom<InColumnType, IAvaOutlinerColumn>::Value && TIsDerivedFrom<InRefColumnType, IAvaOutlinerColumn>::Value)>
	void AddColumn()
	{
		AddColumn(MakeShared<InColumnType>(), InExtensionPosition, InRefColumnType::GetStaticTypeName());
	}

	template<typename InColumnType
		UE_REQUIRES(TIsDerivedFrom<InColumnType, IAvaOutlinerColumn>::Value)>
	void AddColumn()
	{
		AddColumn(MakeShared<InColumnType>(), EAvaOutlinerExtensionPosition::Before, NAME_None);
	}

	const TArray<FAvaOutlinerColumnPtr>& GetColumns() const { return Columns; }

private:
	void AddColumn(FAvaOutlinerColumnPtr InColumn
		, EAvaOutlinerExtensionPosition InExtensionPosition
		, FName InReferenceColumnId);

	int32 FindColumnIndex(FName InColumnId) const;

	TArray<FAvaOutlinerColumnPtr> Columns;
};
