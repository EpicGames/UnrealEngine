// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/AvaOutlinerItemsColumn.h"
#include "Slate/SAvaOutlinerItemList.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerItemsColumn"

FText FAvaOutlinerItemsColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("DisplayName", "Items");
}

SHeaderRow::FColumn::FArguments FAvaOutlinerItemsColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.FillWidth(3.0f)
		.DefaultLabel(GetColumnDisplayNameText());
}

TSharedRef<SWidget> FAvaOutlinerItemsColumn::ConstructRowWidget(FAvaOutlinerItemPtr InItem
	, const TSharedRef<FAvaOutlinerView>& InOutlinerView
	, const TSharedRef<SAvaOutlinerTreeRow>& InRow)
{
	return SNew(SAvaOutlinerItemList, InItem, InOutlinerView, InRow);
}

#undef LOCTEXT_NAMESPACE
