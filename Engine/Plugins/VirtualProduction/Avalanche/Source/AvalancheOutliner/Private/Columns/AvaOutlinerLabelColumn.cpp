// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/AvaOutlinerLabelColumn.h"
#include "Item/IAvaOutlinerItem.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerLabelColumn"

FText FAvaOutlinerLabelColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("LabelColumn", "Label");
}

SHeaderRow::FColumn::FArguments FAvaOutlinerLabelColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.FillWidth(5.0f)
		.DefaultLabel(GetColumnDisplayNameText())
		.ShouldGenerateWidget(true);
}

TSharedRef<SWidget> FAvaOutlinerLabelColumn::ConstructRowWidget(FAvaOutlinerItemPtr InItem
	, const TSharedRef<FAvaOutlinerView>& InOutlinerView
	, const TSharedRef<SAvaOutlinerTreeRow>& InRow)
{
	return InItem->GenerateLabelWidget(InRow);
}

#undef LOCTEXT_NAMESPACE
