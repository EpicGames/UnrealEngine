// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/AvaOutlinerLockColumn.h"
#include "Item/IAvaOutlinerItem.h"
#include "Slate/SAvaOutlinerLock.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerLockColumn"

FText FAvaOutlinerLockColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("LockColumn", "Lock");
}

SHeaderRow::FColumn::FArguments FAvaOutlinerLockColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnDisplayNameText())
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::GetBrush("Icons.Lock"))
		];
}

TSharedRef<SWidget> FAvaOutlinerLockColumn::ConstructRowWidget(FAvaOutlinerItemPtr InItem
	, const TSharedRef<FAvaOutlinerView>& InOutlinerView
	, const TSharedRef<SAvaOutlinerTreeRow>& InRow)
{
	if (InItem->CanLock())
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SAvaOutlinerLock, InOutlinerView, InItem, InRow)
			];
	}
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE
