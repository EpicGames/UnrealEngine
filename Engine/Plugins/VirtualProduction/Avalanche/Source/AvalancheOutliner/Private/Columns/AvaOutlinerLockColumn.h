// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Columns/IAvaOutlinerColumn.h"

class FAvaOutlinerLockColumn : public IAvaOutlinerColumn
{
public:
	UE_AVA_INHERITS(FAvaOutlinerLockColumn, IAvaOutlinerColumn);
	
protected:
	//~ Begin IAvaOutlinerColumn
	virtual FText GetColumnDisplayNameText() const override;
	virtual bool ShouldShowColumnByDefault() const override { return true; }
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual TSharedRef<SWidget> ConstructRowWidget(FAvaOutlinerItemPtr InItem
		, const TSharedRef<FAvaOutlinerView>& InOutlinerView
		, const TSharedRef<SAvaOutlinerTreeRow>& InRow) override;
	//~ End IAvaOutlinerColumn
};
