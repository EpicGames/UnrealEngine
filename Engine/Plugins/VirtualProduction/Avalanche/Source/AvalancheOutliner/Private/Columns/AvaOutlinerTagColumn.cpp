// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/AvaOutlinerTagColumn.h"
#include "Columns/Slate/SAvaOutlinerTag.h"
#include "Item/IAvaOutlinerItem.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerTagColumn"

FText FAvaOutlinerTagColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("TagColumn", "Tags");
}

SHeaderRow::FColumn::FArguments FAvaOutlinerTagColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.FillWidth(1.0f)
		.DefaultLabel(GetColumnDisplayNameText());
}

TSharedRef<SWidget> FAvaOutlinerTagColumn::ConstructRowWidget(FAvaOutlinerItemPtr InItem
	, const TSharedRef<FAvaOutlinerView>& InOutlinerView
	, const TSharedRef<SAvaOutlinerTreeRow>& InRow)
{
	return SNew(SAvaOutlinerTag, InItem, InRow);
}

#undef LOCTEXT_NAMESPACE
