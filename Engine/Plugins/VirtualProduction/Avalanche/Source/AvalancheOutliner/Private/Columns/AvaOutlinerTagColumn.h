// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Columns/IAvaOutlinerColumn.h"

class FAvaOutlinerTagColumn : public IAvaOutlinerColumn
{
public:
	UE_AVA_INHERITS(FAvaOutlinerTagColumn, IAvaOutlinerColumn);

protected:
	//~ Begin IAvaOutlinerColumn
	virtual FText GetColumnDisplayNameText() const override;
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual TSharedRef<SWidget> ConstructRowWidget(FAvaOutlinerItemPtr InItem
		,const TSharedRef<FAvaOutlinerView>& InOutlinerView
		, const TSharedRef<SAvaOutlinerTreeRow>& InRow) override;
	//~ End IAvaOutlinerColumn
};
