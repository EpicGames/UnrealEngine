// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/AvaOutlinerVisibilityColumn.h"
#include "Columns/Slate/SAvaOutlinerVisibility.h"
#include "Item/IAvaOutlinerItem.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerVisibilityCache"

bool FAvaOutlinerVisibilityCache::GetVisibility(EAvaOutlinerVisibilityType Type, const FAvaOutlinerItemPtr& Item) const
{
	if (const FVisibilityInfo* const Info = VisibilityInfo.Find(Item))
	{
		return Info->GetVisibility(Type);
	}
	
	const bool bIsVisible = Item->GetVisibility(Type);
	
	VisibilityInfo.Add(Item, FVisibilityInfo(Type, bIsVisible));
	
	return bIsVisible;
}

SHeaderRow::FColumn::FArguments FAvaOutlinerVisibilityColumn::ConstructHeaderRowColumn()
{
	FText VisibilityType;
	
	switch (GetVisibilityType())
	{
		case EAvaOutlinerVisibilityType::Editor: VisibilityType = LOCTEXT("VisibilityTypeEditor", "E");
			break;

		case EAvaOutlinerVisibilityType::Runtime: VisibilityType = LOCTEXT("VisibilityTypeRuntime", "R");
			break;
	}

	return SHeaderRow::Column(GetColumnId())
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnDisplayNameText())
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			.Padding(0.f, 0.f, 3.f, 3.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush("Level.VisibleIcon16x"))
			]
			+ SOverlay::Slot()
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Right)
			[
				SNew(SScaleBox)
				.UserSpecifiedScale(0.75f)
				.Stretch(EStretch::UserSpecified)
				[
					SNew(STextBlock)
					.Text(VisibilityType)
				]
			]
		];
}

TSharedRef<SWidget> FAvaOutlinerVisibilityColumn::ConstructRowWidget(FAvaOutlinerItemPtr InItem
	, const TSharedRef<FAvaOutlinerView>& InOutlinerView
	, const TSharedRef<SAvaOutlinerTreeRow>& InRow)
{
	if (InItem->ShowVisibility(GetVisibilityType()))
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SAvaOutlinerVisibility, SharedThis(this), InOutlinerView, InItem, InRow)
			];
	}
	
	return SNullWidget::NullWidget;
}

bool FAvaOutlinerVisibilityColumn::IsItemVisible(const FAvaOutlinerItemPtr& Item)
{
	return VisibilityCache.GetVisibility(GetVisibilityType(), Item);
}

void FAvaOutlinerVisibilityColumn::Tick(float InDeltaTime)
{
	VisibilityCache.VisibilityInfo.Empty();
}

#undef LOCTEXT_NAMESPACE
