// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Columns/IAvaOutlinerColumn.h"

/** A struct which gets, and caches the visibility of an item */
struct FAvaOutlinerVisibilityCache
{
	struct FVisibilityInfo
	{
		FVisibilityInfo(EAvaOutlinerVisibilityType Type, bool bVisible)
		{
			Visibility = {{Type, bVisible}};
		}

		TMap<EAvaOutlinerVisibilityType, bool> Visibility;

		bool GetVisibility(EAvaOutlinerVisibilityType Type) const
		{
			if (const bool* const bVisibility = Visibility.Find(Type))
			{
				return *bVisibility;
			}
			return false;
		}
	};

	//Map of tree item to visibility
	mutable TMap<TWeakPtr<IAvaOutlinerItem>, FVisibilityInfo> VisibilityInfo;

	bool GetVisibility(EAvaOutlinerVisibilityType Type, const FAvaOutlinerItemPtr& Item) const;
};

class FAvaOutlinerVisibilityColumn : public IAvaOutlinerColumn
{
public:
	UE_AVA_INHERITS(FAvaOutlinerVisibilityColumn, IAvaOutlinerColumn);

	//~ Begin IAvaOutlinerColumn
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual bool ShouldShowColumnByDefault() const override { return true; }
	virtual TSharedRef<SWidget> ConstructRowWidget(FAvaOutlinerItemPtr InItem
		, const TSharedRef<FAvaOutlinerView>& InOutlinerView
		, const TSharedRef<SAvaOutlinerTreeRow>& InRow) override;
	//~ End IAvaOutlinerColumn

	virtual bool IsItemVisible(const FAvaOutlinerItemPtr& Item);

	virtual EAvaOutlinerVisibilityType GetVisibilityType() const = 0;
	
	virtual void Tick(float InDeltaTime) override;

protected:
	FAvaOutlinerVisibilityCache VisibilityCache;
};

class FAvaOutlinerEditorVisibilityColumn : public FAvaOutlinerVisibilityColumn
{
public:
	UE_AVA_INHERITS(FAvaOutlinerEditorVisibilityColumn, FAvaOutlinerVisibilityColumn);

	virtual FText GetColumnDisplayNameText() const override
	{
		return NSLOCTEXT("AvaOutlinerVisibilityColumn", "OutlinerEditorVisibilityColumn", "Editor Visibility");
	}

	virtual EAvaOutlinerVisibilityType GetVisibilityType() const override
	{
		return EAvaOutlinerVisibilityType::Editor;
	}
};

class FAvaOutlinerRuntimeVisibilityColumn : public FAvaOutlinerVisibilityColumn
{
public:
	UE_AVA_INHERITS(FAvaOutlinerRuntimeVisibilityColumn, FAvaOutlinerVisibilityColumn);

	virtual FText GetColumnDisplayNameText() const override
	{
		return NSLOCTEXT("AvaOutlinerVisibilityColumn", "OutlinerRuntimeVisibilityColumn", "Runtime Visibility");
	}

	virtual EAvaOutlinerVisibilityType GetVisibilityType() const override
	{
		return EAvaOutlinerVisibilityType::Runtime;
	}
};
