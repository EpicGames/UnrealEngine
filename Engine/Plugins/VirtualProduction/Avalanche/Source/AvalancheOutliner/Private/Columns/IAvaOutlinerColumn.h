// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "AvaType.h"
#include "Widgets/Views/SHeaderRow.h"

class FAvaOutlinerView;
class SAvaOutlinerTreeRow;

class IAvaOutlinerColumn : public IAvaTypeCastable, public TSharedFromThis<IAvaOutlinerColumn>
{
public:	
	UE_AVA_INHERITS(IAvaOutlinerColumn, IAvaTypeCastable);

	FName GetColumnId() const { return GetTypeId().ToName(); }

	virtual FText GetColumnDisplayNameText() const = 0;

	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() = 0;

	/*
	 * Determines whether the Column should be Showing by Default while still be able to toggle it on/off.
	 * Used when calling SHeaderRow::SetShowGeneratedColumn (requires ShouldGenerateWidget to not be set).
	 */
	virtual bool ShouldShowColumnByDefault() const { return false; }
	
	virtual TSharedRef<SWidget> ConstructRowWidget(FAvaOutlinerItemPtr InItem
		, const TSharedRef<FAvaOutlinerView>& InOutlinerView
		, const TSharedRef<SAvaOutlinerTreeRow>& InRow) = 0;

	virtual void Tick(float InDeltaTime) {}
};
