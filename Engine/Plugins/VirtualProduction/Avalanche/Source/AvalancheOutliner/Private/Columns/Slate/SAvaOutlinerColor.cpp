// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaOutlinerColor.h"
#include "AvaOutliner.h"
#include "AvaOutlinerView.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Slate/SAvaOutlinerTreeRow.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "SAvaOutlinerColor"

DECLARE_DELEGATE_RetVal_OneParam(FReply, FOnColorEntrySelected, FName EntryName);

class SAvaOutlinerColorEntry : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaOutlinerColorEntry) {}
		SLATE_EVENT(FOnColorEntrySelected, OnColorEntrySelected)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, FName InEntryName, const FLinearColor& InEntryColor)
	{
		ColorEntryName  = InEntryName;
		MenuButtonStyle = &FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Menu.Button");

		OnColorEntrySelected = InArgs._OnColorEntrySelected;

		ChildSlot
		[
			SNew(SBox)
			.WidthOverride(120.f)
			[
				SNew(SBorder)
				.BorderImage(this, &SAvaOutlinerColorEntry::GetBorderImage)
				.Padding(FMargin(12.f, 1.f, 12.f, 1.f))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					  .FillWidth(0.8f)
					  .Padding(FMargin(0.f, 2.f, 0.f, 0.f))
					[
						SNew(STextBlock)
						.Text(FText::FromName(InEntryName))
						.ColorAndOpacity(FLinearColor::White)
					]
					+ SHorizontalBox::Slot()
					.FillWidth(0.2f)
					[
						SNew(SBorder)
						[
							SNew(SColorBlock)
							.Color(InEntryColor)
						]
					]
				]

			]
		];
	}

	const FSlateBrush* GetBorderImage() const
	{
		if (IsHovered())
		{
			return &MenuButtonStyle->Hovered;
		}
		return &MenuButtonStyle->Normal;
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (OnColorEntrySelected.IsBound())
		{
			return OnColorEntrySelected.Execute(ColorEntryName);
		}
		return FReply::Handled();
	}

protected:
	FName ColorEntryName;
	FOnColorEntrySelected OnColorEntrySelected;
	const FButtonStyle* MenuButtonStyle = nullptr;
};

class SAvaOutlinerColorOptions : public SMenuAnchor
{
public:
	SLATE_BEGIN_ARGS(SAvaOutlinerColorOptions)
		: _MenuPlacement(MenuPlacement_MenuRight)
		, _CollapseMenuOnParentFocus(false)
	{}

	SLATE_NAMED_SLOT(FArguments, ButtonContent)
	SLATE_NAMED_SLOT(FArguments, MenuContent)
	SLATE_EVENT(FOnGetContent, OnGetMenuContent)
	SLATE_EVENT(FOnIsOpenChanged, OnMenuOpenChanged)
	SLATE_ATTRIBUTE(EMenuPlacement, MenuPlacement)
	SLATE_ARGUMENT(TOptional<EPopupMethod>, Method)
	SLATE_ARGUMENT(bool, CollapseMenuOnParentFocus)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		SMenuAnchor::Construct(SMenuAnchor::FArguments()
			.Placement(InArgs._MenuPlacement)
			.Method(InArgs._Method)
			.OnMenuOpenChanged(InArgs._OnMenuOpenChanged)
			.OnGetMenuContent(InArgs._OnGetMenuContent)
			.IsCollapsedByParent(InArgs._CollapseMenuOnParentFocus)
			[
				InArgs._ButtonContent.Widget
			]);

		SetMenuContent(InArgs._MenuContent.Widget);
	}
};

class FColorDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FColorDragDropOp, FDragDropOperation)

	FName Color;

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return SNullWidget::NullWidget;
	}

	static TSharedRef<FColorDragDropOp> New(const FName& Color)
	{
		TSharedRef<FColorDragDropOp> Operation = MakeShared<FColorDragDropOp>();
		Operation->Color = Color;
		Operation->Construct();
		return Operation;
	}
};

void SAvaOutlinerColor::Construct(const FArguments& InArgs
	, const FAvaOutlinerItemPtr& InItem
	, const TSharedRef<FAvaOutlinerView>& InOutlinerView
	, const TSharedRef<SAvaOutlinerTreeRow>& InRow)
{
	ItemWeak         = InItem;
	RowWeak          = InRow;
	OutlinerViewWeak = InOutlinerView;
	OutlinerWeak     = InOutlinerView->GetOutliner();

	SetColorAndOpacity(TAttribute<FLinearColor>(this, &SAvaOutlinerColor::GetStateColorAndOpacity));

	ChildSlot
	[
		SAssignNew(ColorOptions, SAvaOutlinerColorOptions)
		.OnGetMenuContent(this, &SAvaOutlinerColor::GetOutlinerColorOptions)
		.ButtonContent()
		[
			SNew(SBox)
			[
				SNew(SBorder)
				.Padding(FMargin(1.f))
				.BorderBackgroundColor(this, &SAvaOutlinerColor::GetBorderColor)
				[
					SNew(SColorBlock)
					.Color(this, &SAvaOutlinerColor::FindItemColor)
					.Size(FVector2D(12.f, 12.f))
				]
			]
		]
	];
}

FLinearColor SAvaOutlinerColor::FindItemColor() const
{
	ItemColor = NAME_None;
	if (const TSharedPtr<FAvaOutliner> Outliner = OutlinerWeak.Pin())
	{
		TOptional<FAvaOutlinerColorPair> ItemColorPair = Outliner->FindItemColor(ItemWeak.Pin());
		if (ItemColorPair.IsSet())
		{
			ItemColor = ItemColorPair->Key;
			return ItemColorPair->Value;
		}
	}
	return FLinearColor::Transparent;
}

void SAvaOutlinerColor::RemoveItemColor() const
{
	const TSharedPtr<FAvaOutliner> Outliner = OutlinerWeak.Pin();
	check(Outliner.IsValid());
	Outliner->RemoveItemColor(ItemWeak.Pin());
}

FSlateColor SAvaOutlinerColor::GetBorderColor() const
{
	if (IsHovered())
	{
		return FSlateColor::UseForeground();
	}
	return FSlateColor::UseSubduedForeground();
}

FLinearColor SAvaOutlinerColor::GetStateColorAndOpacity() const
{
	const TSharedPtr<IAvaOutlinerItem> Item = ItemWeak.Pin();
	if (!Item.IsValid())
	{
		return FLinearColor::White;
	}

	const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin();
	if (!OutlinerView.IsValid())
	{
		return FLinearColor::White;
	}

	const bool bIsSelected = OutlinerView->IsItemSelected(Item.ToSharedRef());

	const TSharedPtr<SAvaOutlinerTreeRow> Row = RowWeak.Pin();

	const bool IsRowNotHovered      = Row.IsValid() && !Row->IsHovered();
	const bool IsColorOptionsClosed = ColorOptions.IsValid() && !ColorOptions->IsOpen();

	// make the foreground brush transparent if it is not selected and it doesn't have an Item Color
	if (ItemColor == NAME_None && IsRowNotHovered && !bIsSelected && IsColorOptionsClosed)
	{
		return FLinearColor::Transparent;
	}

	return FLinearColor::White;
}

TSharedRef<SWidget> SAvaOutlinerColor::GetOutlinerColorOptions()
{
	FMenuBuilder Builder(true, nullptr);

	if (const TSharedPtr<FAvaOutliner> Outliner = OutlinerWeak.Pin())
	{
		//Only add a Remove Color entry if the Item has a color by itself (i.e. not inherited by parent)
		if (Outliner->FindItemColor(ItemWeak.Pin(), false).IsSet())
		{
			Builder.AddMenuEntry(LOCTEXT("RemoveColor", "Remove Color")
				, FText()
				, FSlateIcon()
				, FUIAction(FExecuteAction::CreateSP(this, &SAvaOutlinerColor::RemoveItemColor)));
		}

		Builder.BeginSection("Colors", LOCTEXT("Colors", "Colors"));
		for (const FAvaOutlinerColorPair& ColorPair : Outliner->GetColorMap())
		{
			TSharedRef<SAvaOutlinerColorEntry> ColorEntry = SNew(SAvaOutlinerColorEntry, ColorPair.Key, ColorPair.Value)
				.OnColorEntrySelected(this, &SAvaOutlinerColor::OnColorEntrySelected);

			Builder.AddWidget(ColorEntry, FText::GetEmpty());
		}
		Builder.EndSection();
	}
	return Builder.MakeWidget();
}

FReply SAvaOutlinerColor::OnColorEntrySelected(FName InColorEntry) const
{
	if (ColorOptions.IsValid())
	{
		ColorOptions->SetIsOpen(false);
	}

	if (const TSharedPtr<FAvaOutliner> Outliner = OutlinerWeak.Pin())
	{
		Outliner->SetItemColor(ItemWeak.Pin(), InColorEntry);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SAvaOutlinerColor::Press()
{
	bIsPressed = true;
}

void SAvaOutlinerColor::Release()
{
	bIsPressed = false;
}

FReply SAvaOutlinerColor::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		Press();
		const TSharedRef<SAvaOutlinerColor> This = SharedThis(this);
		return FReply::Handled().CaptureMouse(This).DetectDrag(This, EKeys::LeftMouseButton);
	}
	return SCompoundWidget::OnMouseButtonDown(MyGeometry, MouseEvent);
}

FReply SAvaOutlinerColor::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply Reply = FReply::Unhandled();
	if (bIsPressed && IsHovered() && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		Release();
		if (ColorOptions.IsValid())
		{
			ColorOptions->SetIsOpen(true);
		}
		Reply = FReply::Handled();
	}

	if (Reply.GetMouseCaptor().IsValid() == false && HasMouseCapture())
	{
		Reply.ReleaseMouseCapture();
	}
	return Reply;
}

void SAvaOutlinerColor::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	Release();
	SCompoundWidget::OnFocusLost(InFocusEvent);
}

FReply SAvaOutlinerColor::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		return FReply::Handled().BeginDragDrop(FColorDragDropOp::New(ItemColor));
	}
	return SCompoundWidget::OnDragDetected(MyGeometry, MouseEvent);
}

void SAvaOutlinerColor::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (const TSharedPtr<FColorDragDropOp> ColorOp = DragDropEvent.GetOperationAs<FColorDragDropOp>())
	{
		OnColorEntrySelected(ColorOp->Color);
	}
	SCompoundWidget::OnDragEnter(MyGeometry, DragDropEvent);
}

#undef LOCTEXT_NAMESPACE
