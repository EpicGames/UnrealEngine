// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Widgets/SCompoundWidget.h"

class FAvaOutliner;
class FAvaOutlinerView;
class SAvaOutlinerTreeRow;
class SMenuAnchor;

class SAvaOutlinerColor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaOutlinerColor) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const FAvaOutlinerItemPtr& InItem
		, const TSharedRef<FAvaOutlinerView>& InOutlinerView
		, const TSharedRef<SAvaOutlinerTreeRow>& InRow);

	FLinearColor FindItemColor() const;
	void RemoveItemColor() const;

	FSlateColor GetBorderColor() const;
	FLinearColor GetStateColorAndOpacity() const;

	TSharedRef<SWidget> GetOutlinerColorOptions();

	FReply OnColorEntrySelected(FName InColorEntry) const;

	void Press();
	void Release();

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;

	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

private:
	TSharedPtr<SMenuAnchor> ColorOptions;

	TWeakPtr<IAvaOutlinerItem> ItemWeak;

	TWeakPtr<SAvaOutlinerTreeRow> RowWeak;

	TWeakPtr<FAvaOutliner> OutlinerWeak;

	TWeakPtr<FAvaOutlinerView> OutlinerViewWeak;

	mutable FName ItemColor = NAME_None;

	bool bIsPressed = false;
};
