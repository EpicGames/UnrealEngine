// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaOutlinerItemChip.h"
#include "AvaOutlinerView.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Item/IAvaOutlinerItem.h"
#include "Slate/Styling/AvaOutlinerStyleUtils.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Views/STableRow.h"

void SAvaOutlinerItemChip::Construct(const FArguments& InArgs
	, const TSharedRef<IAvaOutlinerItem>& InItem
	, const TSharedPtr<FAvaOutlinerView>& InOutlinerView)
{
	ItemWeak          = InItem;
	OutlinerViewWeak  = InOutlinerView;
	ChipStyle         = InArgs._ChipStyle;
	OnItemChipClicked = InArgs._OnItemChipClicked;
	OnValidDragOver   = InArgs._OnValidDragOver;

	constexpr float ChipSize = 14.f;

	SBorder::Construct(SBorder::FArguments()
		.Padding(2.f, 1.f)
		.Visibility(EVisibility::Visible)
		[
			SNew(SBox)
			.WidthOverride(ChipSize)
			.HeightOverride(ChipSize)
			[
				SNew(SScaleBox)
				.Stretch(EStretch::ScaleToFit)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(InItem, &IAvaOutlinerItem::GetIconBrush)
				]
			]
		]);
	SetToolTipText(TAttribute<FText>(InItem, &IAvaOutlinerItem::GetDisplayName));
	SetBorderImage(TAttribute<const FSlateBrush*>(this, &SAvaOutlinerItemChip::GetItemBackgroundBrush));
}

int32 SAvaOutlinerItemChip::OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InCullingRect
	, FSlateWindowElementList& OutDrawElements, int32 LayerId
	, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	LayerId = SBorder::OnPaint(InArgs, InAllottedGeometry, InCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	if (ItemDropZone.IsSet())
	{
		const FSlateBrush* const DropIndicatorBrush = GetDropIndicatorBrush(*ItemDropZone);

		// Reuse the drop indicator asset for horizontal, by rotating the drawn box 90 degrees.
		const FVector2f LocalSize(InAllottedGeometry.GetLocalSize());
		const FVector2f Pivot(LocalSize * 0.5f);
		const FVector2f RotatedLocalSize(LocalSize.Y, LocalSize.X);

		// Make the box centered to the allotted geometry, so that it can be rotated around the center.
		FSlateLayoutTransform RotatedTransform(Pivot - RotatedLocalSize * 0.5f);

		FSlateDrawElement::MakeRotatedBox(OutDrawElements
			, LayerId++
			, InAllottedGeometry.ToPaintGeometry(RotatedLocalSize, RotatedTransform)
			, DropIndicatorBrush
			, ESlateDrawEffect::None
			, -UE_HALF_PI // 90 deg CCW
			, RotatedLocalSize * 0.5f // Relative center to the flipped
			, FSlateDrawElement::RelativeToElement
			, DropIndicatorBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint());
	}
	return LayerId;
}

FReply SAvaOutlinerItemChip::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		Press();

		TSharedRef<SAvaOutlinerItemChip> This = SharedThis(this);

		return FReply::Handled()
			.DetectDrag(This, InMouseEvent.GetEffectingButton())
			.CaptureMouse(This)
			.SetUserFocus(This, EFocusCause::Mouse);
	}
	return FReply::Unhandled();
}

FReply SAvaOutlinerItemChip::OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	FReply Reply = FReply::Unhandled();

	if (IsPressed() && InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		Release();

		if (OnItemChipClicked.IsBound())
		{
			bool bEventOverButton = IsHovered();
			if (!bEventOverButton && InMouseEvent.IsTouchEvent())
			{
				bEventOverButton = InGeometry.IsUnderLocation(InMouseEvent.GetScreenSpacePosition());
			}

			if (bEventOverButton && HasMouseCapture())
			{
				Reply = OnItemChipClicked.Execute(ItemWeak.Pin(), InMouseEvent);
			}
		}

		// If the user of the button didn't handle this click, then the button's default behavior handles it.
		if (!Reply.IsEventHandled())
		{
			Reply = FReply::Handled();
		}
	}

	// If the user hasn't requested a new mouse captor and the button still has mouse capture,
	// then the default behavior of the button is to release mouse capture.
	if (Reply.GetMouseCaptor().IsValid() == false && HasMouseCapture())
	{
		Reply.ReleaseMouseCapture();
	}

	return Reply;
}

void SAvaOutlinerItemChip::OnMouseCaptureLost(const FCaptureLostEvent& InCaptureLostEvent)
{
	Release();
}

FReply SAvaOutlinerItemChip::OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	const FAvaOutlinerItemPtr Item = ItemWeak.Pin();
	const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin();

	if (!Item.IsValid() || !OutlinerView.IsValid())
	{
		ItemDropZone = TOptional<EItemDropZone>();
		return FReply::Unhandled();
	}

	return OutlinerView->OnDragDetected(InGeometry, InMouseEvent, Item);
}

FReply SAvaOutlinerItemChip::OnDragOver(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	const FAvaOutlinerItemPtr Item = ItemWeak.Pin();
	const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin();

	if (!Item.IsValid() || !OutlinerView.IsValid())
	{
		ItemDropZone = TOptional<EItemDropZone>();
		return FReply::Unhandled();
	}

	const FVector2f LocalPointerPosition = InGeometry.AbsoluteToLocal(InDragDropEvent.GetScreenSpacePosition());

	const EItemDropZone ItemHoverZone = SAvaOutlinerItemChip::GetHoverZone(LocalPointerPosition
		, InGeometry.GetLocalSize());

	ItemDropZone = OutlinerView->OnCanDrop(InDragDropEvent, ItemHoverZone, Item);

	if (ItemDropZone.IsSet() && OnValidDragOver.IsBound())
	{
		OnValidDragOver.Execute(InGeometry, InDragDropEvent);
	}

	return FReply::Handled();
}

FReply SAvaOutlinerItemChip::OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	ItemDropZone = TOptional<EItemDropZone>();

	const FAvaOutlinerItemPtr Item = ItemWeak.Pin();
	const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin();

	if (!Item.IsValid() || !OutlinerView.IsValid())
	{
		return FReply::Unhandled();
	}

	const FVector2f LocalPointerPosition = InGeometry.AbsoluteToLocal(InDragDropEvent.GetScreenSpacePosition());
	const EItemDropZone DropZone = SAvaOutlinerItemChip::GetHoverZone(LocalPointerPosition, InGeometry.GetLocalSize());
	if (OutlinerView->OnCanDrop(InDragDropEvent, DropZone, Item).IsSet())
	{
		return OutlinerView->OnDrop(InDragDropEvent, DropZone, Item);
	}
	return FReply::Unhandled();
}

void SAvaOutlinerItemChip::OnDragLeave(const FDragDropEvent& InDragDropEvent)
{
	ItemDropZone = TOptional<EItemDropZone>();
}

bool SAvaOutlinerItemChip::IsSelected() const
{
	if (OutlinerViewWeak.IsValid() && ItemWeak.IsValid())
	{
		return OutlinerViewWeak.Pin()->IsItemSelected(ItemWeak.Pin());
	}
	return false;
}

const FSlateBrush* SAvaOutlinerItemChip::GetItemBackgroundBrush() const
{
	using namespace UE::AvaOutliner::Private;

	const EStyleType StyleType = IsHovered()
		? EStyleType::Hovered
		: EStyleType::Normal;

	return &FStyleUtils::GetBrush(StyleType, IsSelected());
}

const FSlateBrush* SAvaOutlinerItemChip::GetDropIndicatorBrush(EItemDropZone InItemDropZone) const
{
	switch (InItemDropZone)
	{
	case EItemDropZone::AboveItem:
		return &ChipStyle->DropIndicator_Above;

	case EItemDropZone::OntoItem:
		return &ChipStyle->DropIndicator_Onto;

	case EItemDropZone::BelowItem:
		return &ChipStyle->DropIndicator_Below;

	default:
		return nullptr;
	};
}

void SAvaOutlinerItemChip::Press()
{
	bIsPressed = true;
}

void SAvaOutlinerItemChip::Release()
{
	bIsPressed = false;
}

EItemDropZone SAvaOutlinerItemChip::GetHoverZone(const FVector2f& InLocalPosition, const FVector2f& InLocalSize)
{
	// Clamping Values of the Edge Size so it's not too small nor too big
	// Referenced from the STableRow::ZoneFromPointerPosition
	constexpr float MinEdgeSize = 3.f;
	constexpr float MaxEdgeSize = 10.f;

	const float EdgeZoneSize = FMath::Clamp(InLocalSize.X * 0.25f, MinEdgeSize, MaxEdgeSize);

	if (InLocalPosition.X < EdgeZoneSize)
	{
		return EItemDropZone::AboveItem;
	}

	if (InLocalPosition.X > InLocalSize.X - EdgeZoneSize)
	{
		return EItemDropZone::BelowItem;
	}

	return EItemDropZone::OntoItem;
}
