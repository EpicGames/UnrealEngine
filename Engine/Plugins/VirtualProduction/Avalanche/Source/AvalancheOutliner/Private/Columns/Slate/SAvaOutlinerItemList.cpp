// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaOutlinerItemList.h"
#include "AvaOutlinerView.h"
#include "SAvaOutlinerItemChip.h"
#include "Slate/SAvaOutlinerTreeRow.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "SAvaOutlinerItemList"

void SAvaOutlinerItemList::Construct(const FArguments& InArgs
	, const FAvaOutlinerItemPtr& InParentItem
	, const TSharedRef<FAvaOutlinerView>& InOutlinerView
	, const TSharedRef<SAvaOutlinerTreeRow>& InRow)
{
	TreeRowWeak      = InRow;
	ParentItemWeak   = InParentItem;
	OutlinerViewWeak = InOutlinerView;

	InOutlinerView->GetOnOutlinerViewRefreshed().AddSP(this, &SAvaOutlinerItemList::Refresh);
	InParentItem->OnExpansionChanged().AddSP(this, &SAvaOutlinerItemList::OnItemExpansionChanged);

	ChildSlot
	.HAlign(EHorizontalAlignment::HAlign_Left)
	[
		SAssignNew(ItemListBox, SScrollBox)
		.ConsumeMouseWheel(EConsumeMouseWheel::Always)
		.Orientation(EOrientation::Orient_Horizontal)
		.ScrollBarThickness(FVector2D(2.f))
	];

	Refresh();
}

SAvaOutlinerItemList::~SAvaOutlinerItemList()
{
	if (OutlinerViewWeak.IsValid())
	{
		OutlinerViewWeak.Pin()->GetOnOutlinerViewRefreshed().RemoveAll(this);
	}
	if (ParentItemWeak.IsValid())
	{
		ParentItemWeak.Pin()->OnExpansionChanged().RemoveAll(this);
	}
}

void SAvaOutlinerItemList::OnItemExpansionChanged(const TSharedPtr<FAvaOutlinerView>& InOutlinerView, bool bInIsExpanded)
{
	Refresh();
}

void SAvaOutlinerItemList::Refresh()
{
	ItemListBox->ClearChildren();
	ChildItemListWeak.Reset();

	FAvaOutlinerItemPtr ParentItem            = ParentItemWeak.Pin();
	TSharedPtr<SAvaOutlinerTreeRow> TreeRow   = TreeRowWeak.Pin();
	TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin();

	if (!ParentItem.IsValid() || !OutlinerView.IsValid() || !TreeRow.IsValid())
	{
		return;
	}

	TSet<FAvaOutlinerItemPtr> DisallowedItems;
	{
		TArray<FAvaOutlinerItemPtr> ItemsToDisallow;

		// First get the Children in the Outliner View to see if we need to Disallow some of the Items
		// to avoid redundancy or unnecessarily showing other item's children (e.g. an actor's)
		static const TSet<FAvaOutlinerItemPtr> EmptySet;
		OutlinerView->GetChildrenOfItem(ParentItem, ItemsToDisallow, EAvaOutlinerItemViewMode::ItemTree, EmptySet);

		// If Parent item is Collapsed, only disallow items that are top levels (as these should deal with their own item list)
		// Items that can't be top level should be visualized by the Parent Item when Collapsed
		if (!EnumHasAllFlags(OutlinerView->GetViewItemFlags(ParentItem), EAvaOutlinerItemFlags::Expanded))
		{
			// keep only top level items
			ItemsToDisallow.RemoveAllSwap([](const FAvaOutlinerItemPtr& Child)
				{
					return !Child.IsValid() || !Child->CanBeTopLevel();
				}
				, EAllowShrinking::No);
		}

		DisallowedItems.Append(MoveTemp(ItemsToDisallow));
	}

	TArray<FAvaOutlinerItemPtr> Children;
	OutlinerView->GetChildrenOfItem(ParentItem, Children, EAvaOutlinerItemViewMode::HorizontalItemList, DisallowedItems);

	// TArray<>::Pop will be removing the item from the end which results in the Item List being in reverse
	// so instead of immediately adding it to the slot , we will add it to this array and reverse it
	TArray<FAvaOutlinerItemPtr> ItemsToAdd;

	const FAvaOutlinerView& OutlinerViewConstRef = *OutlinerView;

	while (!Children.IsEmpty())
	{
		FAvaOutlinerItemPtr Child = Children.Pop();

		if (!Child.IsValid() || DisallowedItems.Contains(Child))
		{
			continue;
		}

		if (Child->IsViewModeSupported(EAvaOutlinerItemViewMode::HorizontalItemList, OutlinerViewConstRef))
		{
			ItemsToAdd.Add(Child);
		}
		else
		{
			OutlinerView->GetChildrenOfItem(Child, Children, EAvaOutlinerItemViewMode::HorizontalItemList, DisallowedItems);
		}
	}

	ChildItemListWeak.Reserve(ItemsToAdd.Num());

	for (int32 ItemIndex = ItemsToAdd.Num() - 1; ItemIndex >= 0; --ItemIndex)
	{
		const FAvaOutlinerItemPtr& Item = ItemsToAdd[ItemIndex];
		ChildItemListWeak.Add(Item);
		ItemListBox->AddSlot()
			.Padding(0.f, 1.f)
			[
				SNew(SAvaOutlinerItemChip, Item.ToSharedRef(), OutlinerView)
				.OnItemChipClicked(this, &SAvaOutlinerItemList::OnItemChipSelected)
				.ChipStyle(TreeRow->GetStyle())
				.OnValidDragOver(this, &SAvaOutlinerItemList::OnItemChipValidDragOver)
			];
	}
}

FReply SAvaOutlinerItemList::OnItemChipSelected(const FAvaOutlinerItemPtr& InItem, const FPointerEvent& InMouseEvent)
{
	const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin();
	const FAvaOutlinerItemPtr ParentItem = ParentItemWeak.Pin();

	if (!InItem.IsValid() || !OutlinerView.IsValid() || !ParentItem.IsValid())
	{
		return FReply::Unhandled();
	}

	TArray<FAvaOutlinerItemPtr> SelectedItems = OutlinerView->GetViewSelectedItems();

	EAvaOutlinerItemSelectionFlags SelectionFlags = EAvaOutlinerItemSelectionFlags::SignalSelectionChange;

	if (InMouseEvent.IsControlDown())
	{
		// Deselect if the Item in question is already selected (i.e. CTRL also behaves like a Toggle)
		if (OutlinerView->IsItemSelected(InItem))
		{
			SelectedItems.Remove(InItem);
			OutlinerView->SelectItems(MoveTemp(SelectedItems), SelectionFlags);
			return FReply::Handled();
		}

		SelectionFlags |= EAvaOutlinerItemSelectionFlags::AppendToCurrentSelection;
	}

	// add the given item since it will be selected regardless if Shift (Range Selection) is pressed or not
	TArray<FAvaOutlinerItemPtr> ItemsToSelect { InItem };

	if (InMouseEvent.IsShiftDown() && !SelectedItems.IsEmpty())
	{
		// Similar to Control, Shift should always append Selections, never remove
		SelectionFlags |= EAvaOutlinerItemSelectionFlags::AppendToCurrentSelection;

		int32 StartIndex = INDEX_NONE;

		// Find whether there's a Child in the Selected Item List.
		// Iterate in Reverse order since we want more recent Selections to take precedence
		for (int32 ItemIndex = SelectedItems.Num() - 1; ItemIndex >= 0; --ItemIndex)
		{
			StartIndex = ChildItemListWeak.Find(SelectedItems[ItemIndex]);
			if (StartIndex != INDEX_NONE)
			{
				break;
			}
		}

		// If we manage to find an already selected child, start the range selection
		if (StartIndex != INDEX_NONE)
		{
			int32 TargetIndex = ChildItemListWeak.Find(InItem);

			StartIndex  = FMath::Clamp(StartIndex, 0, ChildItemListWeak.Num());
			TargetIndex = FMath::Clamp(TargetIndex, 0, ChildItemListWeak.Num());

			// If Target Index comes before, just iterate normally, and reverse later
			if (TargetIndex < StartIndex)
			{
				Swap(StartIndex, TargetIndex);
			}

			// Empty List as we're going to add the InItem anyway through the List Iteration
			ItemsToSelect.Empty(TargetIndex - StartIndex);

			for (int32 ItemIndex = StartIndex; ItemIndex <= TargetIndex; ++ItemIndex)
			{
				ItemsToSelect.Add(ChildItemListWeak[ItemIndex].Pin());
			}

			if (TargetIndex < StartIndex)
			{
				Algo::Reverse(ItemsToSelect);
			}
		}
	}

	OutlinerView->SelectItems(MoveTemp(ItemsToSelect), SelectionFlags);
	return FReply::Handled();
}

FReply SAvaOutlinerItemList::OnItemChipValidDragOver(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	TSharedPtr<SAvaOutlinerTreeRow> TreeRow = TreeRowWeak.Pin();
	if (!TreeRow.IsValid())
	{
		return FReply::Unhandled();
	}

	// When Item Chip has a Valid Drag Over (i.e. a Supported Drag/Drop), make sure the Tree Row Holding this Item Chips simulates a Drag Leave
	// Slate App won't do it as it will still find the Widget under the mouse
	TreeRow->OnDragLeave(InDragDropEvent);

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
