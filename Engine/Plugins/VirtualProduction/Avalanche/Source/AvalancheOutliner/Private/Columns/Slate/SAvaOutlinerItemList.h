// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Widgets/SCompoundWidget.h"

class SScrollBox;
class FAvaOutlinerView;
class SAvaOutlinerTreeRow;

/** Widget that visualizes the list of children an Item has when collapsed */
class SAvaOutlinerItemList : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaOutlinerItemList) {}
	SLATE_END_ARGS()

	virtual ~SAvaOutlinerItemList() override;

	void Construct(const FArguments& InArgs
		, const FAvaOutlinerItemPtr& InParentItem
		, const TSharedRef<FAvaOutlinerView>& InOutlinerView
		, const TSharedRef<SAvaOutlinerTreeRow>& InRow);

	void OnItemExpansionChanged(const TSharedPtr<FAvaOutlinerView>& InOutlinerView, bool bInIsExpanded);

	void Refresh();

	FReply OnItemChipSelected(const FAvaOutlinerItemPtr& InItem, const FPointerEvent& InMouseEvent);

	FReply OnItemChipValidDragOver(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent);

private:
	FAvaOutlinerItemWeakPtr ParentItemWeak;

	TArray<FAvaOutlinerItemWeakPtr> ChildItemListWeak;

	TWeakPtr<FAvaOutlinerView> OutlinerViewWeak;

	TWeakPtr<SAvaOutlinerTreeRow> TreeRowWeak;

	TSharedPtr<SScrollBox> ItemListBox;
};
