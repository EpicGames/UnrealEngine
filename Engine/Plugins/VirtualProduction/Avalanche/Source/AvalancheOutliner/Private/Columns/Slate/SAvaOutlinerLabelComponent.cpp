// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaOutlinerLabelComponent.h"
#include "Fonts/SlateFontInfo.h"
#include "Item/AvaOutlinerComponent.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/CoreStyle.h"

#define LOCTEXT_NAMESPACE "SAvaOutlinerLabelComponent"

void SAvaOutlinerLabelComponent::Construct(const FArguments& InArgs
	, const TSharedRef<FAvaOutlinerComponent>& InItem
	, const TSharedRef<SAvaOutlinerTreeRow>& InRow)
{
	SAvaOutlinerLabelItem::Construct(SAvaOutlinerLabelItem::FArguments(), InItem, InRow);
}

const FInlineEditableTextBlockStyle* SAvaOutlinerLabelComponent::GetTextBlockStyle() const
{
	static FInlineEditableTextBlockStyle Style = *SAvaOutlinerLabelItem::GetTextBlockStyle();
	static bool bFontSet = false;
	if (!bFontSet)
	{
		Style.TextStyle.Font = DEFAULT_FONT("Italic", 10);
		bFontSet = true;
	}
	return &Style;
}

#undef LOCTEXT_NAMESPACE
