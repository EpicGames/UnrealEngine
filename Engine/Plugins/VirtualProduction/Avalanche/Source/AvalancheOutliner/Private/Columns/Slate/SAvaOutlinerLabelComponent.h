// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SAvaOutlinerLabelItem.h"

class FAvaOutlinerComponent;

class SAvaOutlinerLabelComponent : public SAvaOutlinerLabelItem
{
public:
	SLATE_BEGIN_ARGS(SAvaOutlinerLabelComponent) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const TSharedRef<FAvaOutlinerComponent>& InItem
		, const TSharedRef<SAvaOutlinerTreeRow>& InRow);

	virtual const FInlineEditableTextBlockStyle* GetTextBlockStyle() const override;
};
