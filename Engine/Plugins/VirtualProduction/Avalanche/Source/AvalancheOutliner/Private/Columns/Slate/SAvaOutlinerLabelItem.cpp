// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaOutlinerLabelItem.h"
#include "ActorEditorUtils.h"
#include "AvaOutliner.h"
#include "AvaOutlinerView.h"
#include "Item/IAvaOutlinerItem.h"
#include "Slate/SAvaOutlinerExpanderArrow.h"
#include "Slate/SAvaOutlinerTreeRow.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaOutlinerLabelItem"

void SAvaOutlinerLabelItem::Construct(const FArguments& InArgs
	, const TSharedRef<IAvaOutlinerItem>& InItem
	, const TSharedRef<SAvaOutlinerTreeRow>& InRow)
{
	ItemWeak         = InItem;
	HighlightText    = InRow->GetHighlightText();
	OutlinerViewWeak = InRow->GetOutlinerView();

	InItem->OnRenameAction().AddSP(this, &SAvaOutlinerLabelItem::OnRenameAction);

	ChildSlot
	[
		SNew(SBox)
		.MinDesiredHeight(25)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(6.f, 0.f, 0.f, 0.f)
			[
				SNew(SAvaOutlinerExpanderArrow, InRow)
				.ExpanderArrowArgs(SExpanderArrow::FArguments()
				.IndentAmount(12)
				.ShouldDrawWires(true))
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.f, 1.f, 6.f, 1.f))
				[
					SNew(SBox)
					.WidthOverride(16.f)
					.HeightOverride(16.f)
					[
						SNew(SImage)
						.IsEnabled(this, &SAvaOutlinerLabelItem::IsItemEnabled)
						.Image(InItem, &IAvaOutlinerItem::GetIconBrush)
						.ToolTipText(InItem, &IAvaOutlinerItem::GetIconTooltipText)
					]
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f)
				[
					SAssignNew(InlineTextBlock, SInlineEditableTextBlock)
					.IsEnabled(this, &SAvaOutlinerLabelItem::IsItemEnabled)
					.Text(InItem, &IAvaOutlinerItem::GetDisplayName)
					.Style(GetTextBlockStyle())
					.HighlightText(HighlightText)
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
					.ColorAndOpacity(this, &SAvaOutlinerLabelItem::GetForegroundColor)
					.OnTextCommitted(this, &SAvaOutlinerLabelItem::OnLabelTextCommitted)
					.OnVerifyTextChanged(this, &SAvaOutlinerLabelItem::OnVerifyItemLabelChanged)
					.OnEnterEditingMode(this, &SAvaOutlinerLabelItem::OnEnterEditingMode)
					.OnExitEditingMode(this, &SAvaOutlinerLabelItem::OnExitEditingMode)
					.IsSelected(FIsSelected::CreateSP(InRow, &SAvaOutlinerTreeRow::IsSelectedExclusively))
					.IsReadOnly(this, &SAvaOutlinerLabelItem::IsReadOnly)
				]
			]
		]
	];
}

bool SAvaOutlinerLabelItem::IsReadOnly() const
{
	const FAvaOutlinerItemPtr Item = ItemWeak.Pin();

	return !Item.IsValid()
		|| !Item->CanRename()
		|| !OutlinerViewWeak.IsValid()
		|| OutlinerViewWeak.Pin()->IsItemReadOnly(Item);
}

bool SAvaOutlinerLabelItem::IsItemEnabled() const
{
	if (ItemWeak.IsValid() && OutlinerViewWeak.IsValid())
	{
		return !OutlinerViewWeak.Pin()->IsItemReadOnly(ItemWeak.Pin());
	}
	return false;
}

bool SAvaOutlinerLabelItem::OnVerifyItemLabelChanged(const FText& InLabel, FText& OutErrorMessage)
{
	return FActorEditorUtils::ValidateActorName(InLabel, OutErrorMessage);
}

void SAvaOutlinerLabelItem::OnLabelTextCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo)
{
	if (const TSharedPtr<IAvaOutlinerItem> Item = ItemWeak.Pin())
	{
		switch (InCommitInfo)
		{
			case ETextCommit::OnEnter:
			case ETextCommit::OnUserMovedFocus:
				RenameItem(InLabel);
				Item->OnRenameAction().Broadcast(EAvaOutlinerRenameAction::Completed, OutlinerViewWeak.Pin());
				break;

			case ETextCommit::Default:
			case ETextCommit::OnCleared:
			default:
				Item->OnRenameAction().Broadcast(EAvaOutlinerRenameAction::Cancelled, OutlinerViewWeak.Pin());
				break;
		}
	}
}

void SAvaOutlinerLabelItem::RenameItem(const FText& InLabel)
{
	if (TSharedPtr<IAvaOutlinerItem> Item = ItemWeak.Pin())
	{
		const bool bRenameSuccessful = Item->Rename(InLabel.ToString());
		if (bRenameSuccessful && OutlinerViewWeak.IsValid())
		{
			OutlinerViewWeak.Pin()->SetKeyboardFocus();
		}
	}
}

void SAvaOutlinerLabelItem::OnRenameAction(EAvaOutlinerRenameAction InRenameAction, const TSharedPtr<FAvaOutlinerView>& InOutlinerView) const
{
	if (InRenameAction == EAvaOutlinerRenameAction::Requested && InOutlinerView == OutlinerViewWeak)
	{
		InlineTextBlock->EnterEditingMode();
	}
}

void SAvaOutlinerLabelItem::OnEnterEditingMode()
{
	bInEditingMode = true;
}

void SAvaOutlinerLabelItem::OnExitEditingMode()
{
	bInEditingMode = false;
}

const FInlineEditableTextBlockStyle* SAvaOutlinerLabelItem::GetTextBlockStyle() const
{
	return &FCoreStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("InlineEditableTextBlockStyle");
}

#undef LOCTEXT_NAMESPACE
