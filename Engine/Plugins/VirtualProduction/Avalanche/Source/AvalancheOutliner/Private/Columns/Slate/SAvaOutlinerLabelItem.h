// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SCompoundWidget.h"

class FAvaOutlinerView;
class FAvaOutliner;
class FAvaOutlinerObject;
class SAvaOutliner;
class SAvaOutlinerTreeRow;
class SInlineEditableTextBlock;

class SAvaOutlinerLabelItem : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaOutlinerLabelItem) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const TSharedRef<IAvaOutlinerItem>& InItem
		, const TSharedRef<SAvaOutlinerTreeRow>& InRow);

	bool IsReadOnly() const;

	bool IsItemEnabled() const;

	bool OnVerifyItemLabelChanged(const FText& InLabel, FText& OutErrorMessage);

	void OnLabelTextCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo);
	void RenameItem(const FText& InLabel);

	void OnRenameAction(EAvaOutlinerRenameAction InRenameAction, const TSharedPtr<FAvaOutlinerView>& InOutlinerView) const;

	void OnEnterEditingMode();
	void OnExitEditingMode();

	virtual const FInlineEditableTextBlockStyle* GetTextBlockStyle() const;

private:
	TSharedPtr<SInlineEditableTextBlock> InlineTextBlock;

	FAvaOutlinerItemWeakPtr ItemWeak;

	TWeakPtr<FAvaOutlinerView> OutlinerViewWeak;

	TAttribute<FText> HighlightText;

	bool bInEditingMode = false;
};
