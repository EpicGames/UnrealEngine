// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaOutlinerLock.h"
#include "AvaOutlinerSettings.h"
#include "AvaOutlinerView.h"
#include "Item/IAvaOutlinerItem.h"
#include "Slate/SAvaOutlinerTreeRow.h"

#define LOCTEXT_NAMESPACE "SAvaOutlinerLock"

namespace UE::AvaOutliner::Private
{
	class FLockDragDropOp : public FDragDropOperation
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FLockDragDropOp, FDragDropOperation)

		/** Flag which defines whether to lock destination items or not */
		bool bShouldLock;

		/** Undo transaction stolen from the gutter which is kept alive for the duration of the drag */
		TUniquePtr<FScopedTransaction> UndoTransaction;

		/** The widget decorator to use */
		virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
		{
			return SNullWidget::NullWidget;
		}

		/** Create a new drag and drop operation out of the specified flag */
		static TSharedRef<FLockDragDropOp> New(bool bShouldLock, TUniquePtr<FScopedTransaction>& ScopedTransaction)
		{
			TSharedRef<FLockDragDropOp> Operation = MakeShared<FLockDragDropOp>();
			Operation->bShouldLock     = bShouldLock;
			Operation->UndoTransaction = MoveTemp(ScopedTransaction);
			Operation->Construct();
			return Operation;
		}
	};

	bool IsItemLocked(const FAvaOutlinerItemPtr& InItem)
	{
		return InItem.IsValid() && InItem->IsLocked();
	}

	void SetItemLocked(const FAvaOutlinerItemPtr& InItem, const bool bInLocked)
	{
		if (InItem.IsValid() && InItem->IsLocked() != bInLocked)
		{
			InItem->SetLocked(bInLocked);
		}
	}
}

void SAvaOutlinerLock::Construct(const FArguments& InArgs
	, const TSharedRef<FAvaOutlinerView>& InOutlinerView
	, const FAvaOutlinerItemPtr& InItem
	, const TSharedRef<SAvaOutlinerTreeRow>& InRow)
{
	ItemWeak         = InItem;
	RowWeak          = InRow;
	OutlinerViewWeak = InOutlinerView;
	LockedBrush      = FAppStyle::GetBrush(TEXT("Icons.Lock"));
	UnlockedBrush    = FAppStyle::GetBrush(TEXT("Icons.Unlock"));

	SImage::Construct(SImage::FArguments()
		.ColorAndOpacity(this, &SAvaOutlinerLock::GetForegroundColor)
		.Image(this, &SAvaOutlinerLock::GetBrush));
}

FSlateColor SAvaOutlinerLock::GetForegroundColor() const
{
	const TSharedPtr<IAvaOutlinerItem> Item = ItemWeak.Pin();

	if (!Item.IsValid())
	{
		return FSlateColor::UseForeground();
	}

	const bool bIsItemSelected = OutlinerViewWeak.IsValid()
		&& OutlinerViewWeak.Pin()->IsItemSelected(Item.ToSharedRef());

	const bool bIsItemHovered = RowWeak.IsValid()
		&& RowWeak.Pin()->IsHovered();

	const bool bAlwaysShowLock = UAvaOutlinerSettings::Get()->ShouldAlwaysShowLockState();

	// we can hide the brush if Settings for Always Showing State is OFF
	// and Item is not locked while also not being selected nor hovered
	if (!bAlwaysShowLock && !IsItemLocked() && !bIsItemSelected && !bIsItemHovered)
	{
		return FLinearColor::Transparent;
	}

	if (IsHovered() && !bIsItemSelected)
	{
		return FAppStyle::Get().GetSlateColor("Colors.ForegroundHover");
	}

	return FSlateColor::UseForeground();
}

void SAvaOutlinerLock::SetItemLocked(bool bInLocked)
{
	UE::AvaOutliner::Private::SetItemLocked(ItemWeak.Pin(), bInLocked);
}

bool SAvaOutlinerLock::IsItemLocked() const
{
	return UE::AvaOutliner::Private::IsItemLocked(ItemWeak.Pin());
}

const FSlateBrush* SAvaOutlinerLock::GetBrush() const
{
	return IsItemLocked() ? LockedBrush : UnlockedBrush;
}

FReply SAvaOutlinerLock::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		return FReply::Handled().BeginDragDrop(UE::AvaOutliner::Private::FLockDragDropOp::New(IsItemLocked(), UndoTransaction));
	}
	return FReply::Unhandled();
}

void SAvaOutlinerLock::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	using namespace UE::AvaOutliner::Private;
	if (const TSharedPtr<FLockDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FLockDragDropOp>())
	{
		SetItemLocked(DragDropOp->bShouldLock);
	}
}

FReply SAvaOutlinerLock::HandleClick()
{
	const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin();
	const TSharedPtr<IAvaOutlinerItem> Item = ItemWeak.Pin();

	if (!OutlinerView.IsValid() || !Item.IsValid())
	{
		return FReply::Unhandled();
	}

	// Open an undo transaction=
	UndoTransaction = MakeUnique<FScopedTransaction>(LOCTEXT("SetOutlinerItemLock", "Set Item Lock"));

	const bool bLockItem = !IsItemLocked();

	// We operate on all the selected items if the specified item is selected
	if (OutlinerView->IsItemSelected(Item.ToSharedRef()))
	{
		for (TSharedPtr<IAvaOutlinerItem>& SelectedItem : OutlinerView->GetViewSelectedItems())
		{
			if (UE::AvaOutliner::Private::IsItemLocked(SelectedItem) != bLockItem)
			{
				UE::AvaOutliner::Private::SetItemLocked(SelectedItem, bLockItem);
			}
		}
	}
	else
	{
		SetItemLocked(bLockItem);
	}

	return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
}

FReply SAvaOutlinerLock::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return HandleClick();
}

FReply SAvaOutlinerLock::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}
	return HandleClick();
}

FReply SAvaOutlinerLock::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		UndoTransaction.Reset();
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SAvaOutlinerLock::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	UndoTransaction.Reset();
}

#undef LOCTEXT_NAMESPACE
