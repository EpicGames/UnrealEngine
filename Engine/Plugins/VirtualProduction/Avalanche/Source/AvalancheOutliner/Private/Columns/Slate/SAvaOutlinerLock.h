// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "ScopedTransaction.h"
#include "Widgets/Images/SImage.h"

class FAvaOutlinerView;
struct FCaptureLostEvent;
class FDragDropEvent;
struct FGeometry;
struct FPointerEvent;
class FReply;
struct FSlateBrush;
struct FSlateColor;
class SAvaOutlinerTreeRow;

class SAvaOutlinerLock : public SImage
{
public:
	SLATE_BEGIN_ARGS(SAvaOutlinerLock) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const TSharedRef<FAvaOutlinerView>& InOutlinerView
		, const FAvaOutlinerItemPtr& InItem
		, const TSharedRef<SAvaOutlinerTreeRow>& InRow);

	virtual FSlateColor GetForegroundColor() const override;

	void SetItemLocked(bool bInLocked);

	bool IsItemLocked() const;

	const FSlateBrush* GetBrush() const;

	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** If a lock drag drop operation has entered this widget, set its item to the new lock state */
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	FReply HandleClick();

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;

private:
	TWeakPtr<IAvaOutlinerItem> ItemWeak;

	TWeakPtr<FAvaOutlinerView> OutlinerViewWeak;

	TWeakPtr<SAvaOutlinerTreeRow> RowWeak;

	TUniquePtr<FScopedTransaction> UndoTransaction;

	const FSlateBrush* LockedBrush   = nullptr;
	const FSlateBrush* UnlockedBrush = nullptr;
};
