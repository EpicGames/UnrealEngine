// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaOutlinerTag.h"
#include "Item/IAvaOutlinerItem.h"
#include "Slate/SAvaOutlinerTreeRow.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

void SAvaOutlinerTag::Construct(const FArguments& InArgs
	, const FAvaOutlinerItemPtr& InItem
	, const TSharedRef<SAvaOutlinerTreeRow>& InRow)
{
	Item = InItem;
	ChildSlot
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(this, &SAvaOutlinerTag::GetText)
			.HighlightText(InRow->GetHighlightText())
			.Justification(ETextJustify::Left)
		]

	];
}

FText SAvaOutlinerTag::GetText() const
{
	if (Item.IsValid())
	{
		TArray<FName> Tags = Item.Pin()->GetTags();
		FString Output;
		for (FName T : Tags)
		{
			if (!Output.IsEmpty())
			{
				Output += TEXT(", ");
			}
			Output += T.ToString();
		}
		return FText::FromString(Output);
	}
	return FText::GetEmpty();
}
