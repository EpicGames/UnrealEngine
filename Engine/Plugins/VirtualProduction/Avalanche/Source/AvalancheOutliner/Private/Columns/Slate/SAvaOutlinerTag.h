// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Widgets/SCompoundWidget.h"

class SAvaOutlinerTreeRow;

class SAvaOutlinerTag : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaOutlinerTag) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const FAvaOutlinerItemPtr& InItem
		, const TSharedRef<SAvaOutlinerTreeRow>& InRow);

	FText GetText() const;

private:
	TWeakPtr<IAvaOutlinerItem> Item;
};
