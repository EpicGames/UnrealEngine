// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaOutlinerVisibility.h"
#include "AvaOutlinerSettings.h"
#include "AvaOutlinerView.h"
#include "Columns/AvaOutlinerVisibilityColumn.h"
#include "Editor.h"
#include "Item/IAvaOutlinerItem.h"
#include "Slate/SAvaOutlinerTreeRow.h"

#define LOCTEXT_NAMESPACE "SAvaOutlinerVisibility"

namespace UE::AvaOutliner::Private
{
	class FVisibilityDragDropOp : public FDragDropOperation
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FVisibilityDragDropOp, FDragDropOperation)

		/** Flag which defines whether to hide destination items or not */
		bool bHidden;

		/** Undo transaction stolen from the gutter which is kept alive for the duration of the drag */
		TUniquePtr<FScopedTransaction> UndoTransaction;

		/** The widget decorator to use */
		virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
		{
			return SNullWidget::NullWidget;
		}

		/** Create a new drag and drop operation out of the specified flag */
		static TSharedRef<FVisibilityDragDropOp> New(const bool bHidden, TUniquePtr<FScopedTransaction>& ScopedTransaction)
		{
			TSharedRef<FVisibilityDragDropOp> Operation = MakeShared<FVisibilityDragDropOp>();
			Operation->bHidden = bHidden;
			Operation->UndoTransaction = MoveTemp(ScopedTransaction);
			Operation->Construct();
			return Operation;
		}
	};

	/** Check if the specified item is visible */
	bool IsItemVisible(const FAvaOutlinerItemPtr& Item, const TSharedPtr<FAvaOutlinerVisibilityColumn>& Column)
	{
		return Column.IsValid() && Item.IsValid()
			? Column->IsItemVisible(Item)
			: false;
	}
}

void SAvaOutlinerVisibility::Construct(const FArguments& InArgs
	, const TSharedRef<FAvaOutlinerVisibilityColumn>& InColumn
	, const TSharedRef<FAvaOutlinerView>& InOutlinerView
	, const FAvaOutlinerItemPtr& InItem
	, const TSharedRef<SAvaOutlinerTreeRow>& InRow)
{
	ItemWeak         = InItem;
	RowWeak          = InRow;
	ColumnWeak       = InColumn;
	OutlinerViewWeak = InOutlinerView;

	SImage::Construct(SImage::FArguments()
		.IsEnabled(this, &SAvaOutlinerVisibility::IsVisibilityWidgetEnabled)
		.ColorAndOpacity(this, &SAvaOutlinerVisibility::GetForegroundColor)
		.Image(this, &SAvaOutlinerVisibility::GetBrush));

	VisibleHoveredBrush       = FAppStyle::GetBrush(TEXT("Level.VisibleHighlightIcon16x"));
	VisibleNotHoveredBrush    = FAppStyle::GetBrush(TEXT("Level.VisibleIcon16x"));
	NotVisibleHoveredBrush    = FAppStyle::GetBrush(TEXT("Level.NotVisibleHighlightIcon16x"));
	NotVisibleNotHoveredBrush = FAppStyle::GetBrush(TEXT("Level.NotVisibleIcon16x"));
}

FReply SAvaOutlinerVisibility::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	using namespace UE::AvaOutliner::Private;
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		return FReply::Handled().BeginDragDrop(FVisibilityDragDropOp::New(!IsItemVisible(), UndoTransaction));
	}
	return FReply::Unhandled();
}

/** If a visibility drag drop operation has entered this widget, set its item to the new visibility state */
void SAvaOutlinerVisibility::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	using namespace UE::AvaOutliner::Private;
	const TSharedPtr<FVisibilityDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FVisibilityDragDropOp>();
	if (DragDropOp.IsValid())
	{
		SetIsVisible(!DragDropOp->bHidden);
	}
}

FReply SAvaOutlinerVisibility::HandleClick()
{
	if (!IsVisibilityWidgetEnabled())
	{
		return FReply::Unhandled();
	}

	const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin();
	const TSharedPtr<IAvaOutlinerItem> Item = ItemWeak.Pin();
	const TSharedPtr<FAvaOutlinerVisibilityColumn> Column = ColumnWeak.Pin();

	if (!OutlinerView.IsValid() || !Item.IsValid() || !Column.IsValid())
	{
		return FReply::Unhandled();
	}

	// Open an undo transaction
	UndoTransaction = MakeUnique<FScopedTransaction>(LOCTEXT("SetOutlinerItemVisibility", "Set Item Visibility"));

	const bool bVisible = !IsItemVisible();

	// We operate on all the selected items if the specified item is selected
	if (OutlinerView->IsItemSelected(Item.ToSharedRef()))
	{
		for (TSharedPtr<IAvaOutlinerItem>& SelectedItem : OutlinerView->GetViewSelectedItems())
		{
			if (UE::AvaOutliner::Private::IsItemVisible(SelectedItem, Column) != bVisible)
			{
				OnSetItemVisibility(SelectedItem, bVisible);
			}
		}
		GEditor->RedrawAllViewports();
	}
	else
	{
		SetIsVisible(bVisible);
	}

	return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
}

FReply SAvaOutlinerVisibility::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return HandleClick();
}

/** Called when the mouse button is pressed down on this widget */
FReply SAvaOutlinerVisibility::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	return HandleClick();
}

/** Process a mouse up message */
FReply SAvaOutlinerVisibility::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		UndoTransaction.Reset();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

/** Called when this widget had captured the mouse, but that capture has been revoked for some reason. */
void SAvaOutlinerVisibility::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	UndoTransaction.Reset();
}

EAvaOutlinerVisibilityType SAvaOutlinerVisibility::GetVisibilityType() const
{
	if (ColumnWeak.IsValid())
	{
		return ColumnWeak.Pin()->GetVisibilityType();
	}
	return EAvaOutlinerVisibilityType::None;
}

/** Get the brush for this widget */
const FSlateBrush* SAvaOutlinerVisibility::GetBrush() const
{
	if (IsItemVisible())
	{
		return IsHovered() ? VisibleHoveredBrush : VisibleNotHoveredBrush;
	}
	return IsHovered() ? NotVisibleHoveredBrush : NotVisibleNotHoveredBrush;
}

FSlateColor SAvaOutlinerVisibility::GetForegroundColor() const
{
	const TSharedPtr<IAvaOutlinerItem> Item = ItemWeak.Pin();

	if (!Item.IsValid())
	{
		return FSlateColor::UseForeground();
	}

	const bool bIsItemSelected = OutlinerViewWeak.IsValid()
		&& OutlinerViewWeak.Pin()->IsItemSelected(Item.ToSharedRef());

	const bool bIsItemHovered = RowWeak.IsValid()
		&& RowWeak.Pin()->IsHovered();

	const bool bAlwaysShowVisibility = UAvaOutlinerSettings::Get()->ShouldAlwaysShowVisibilityState();

	// we can hide the brush if Settings for Always Showing State is OFF
	// and Item is Visible while also not being selected nor hovered
	if (!bAlwaysShowVisibility && IsItemVisible() && !bIsItemSelected && !bIsItemHovered)
	{
		return FLinearColor::Transparent;
	}

	if (IsHovered() && !bIsItemSelected)
	{
		return FAppStyle::Get().GetSlateColor("Colors.ForegroundHover");
	}

	return FSlateColor::UseForeground();
}

/** Check if our wrapped tree item is visible */
bool SAvaOutlinerVisibility::IsItemVisible() const
{
	return UE::AvaOutliner::Private::IsItemVisible(ItemWeak.Pin(), ColumnWeak.Pin());
}

/** Set the item this widget is responsible for to be hidden or shown */
void SAvaOutlinerVisibility::SetIsVisible(const bool bVisible)
{
	const FAvaOutlinerItemPtr Item = ItemWeak.Pin();

	const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin();
	if (Item.IsValid() && OutlinerView.IsValid() && IsItemVisible() != bVisible)
	{
		OnSetItemVisibility(Item, bVisible);
		OutlinerView->Refresh();
		GEditor->RedrawAllViewports();
	}
}

void SAvaOutlinerVisibility::OnSetItemVisibility(const FAvaOutlinerItemPtr& Item, const bool bNewVisibility)
{
	// Apply the same visibility to the children
	Item->OnVisibilityChanged(GetVisibilityType(), bNewVisibility);
	for (const FAvaOutlinerItemPtr& Child : Item->GetChildren())
	{
		if (Child.IsValid() && Child->CanReceiveParentVisibilityPropagation())
		{
			OnSetItemVisibility(Child, bNewVisibility);
		}
	}
}

#undef LOCTEXT_NAMESPACE
