// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "ScopedTransaction.h"
#include "Widgets/Images/SImage.h"

class FAvaOutlinerVisibilityColumn;
class FAvaOutlinerView;
class SAvaOutlinerTreeRow;

/** Widget responsible for managing the visibility for a single item */
class SAvaOutlinerVisibility : public SImage
{
public:
	SLATE_BEGIN_ARGS(SAvaOutlinerVisibility) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const TSharedRef<FAvaOutlinerVisibilityColumn>& InColumn
		, const TSharedRef<FAvaOutlinerView>& InOutlinerView
		, const FAvaOutlinerItemPtr& InItem
		, const TSharedRef<SAvaOutlinerTreeRow>& InRow);

private:
	/** Returns whether the widget is enabled or not */
	virtual bool IsVisibilityWidgetEnabled() const { return true; }

	EAvaOutlinerVisibilityType GetVisibilityType() const;

	virtual const FSlateBrush* GetBrush() const;

	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** If a visibility drag drop operation has entered this widget, set its item to the new visibility state */
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	FReply HandleClick();

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;

	/** Called when visibility change happens on an Item. */
	virtual void OnSetItemVisibility(const FAvaOutlinerItemPtr& Item, const bool bNewVisibility);

	virtual FSlateColor GetForegroundColor() const override;

	/** Check if our wrapped tree item is visible */
	bool IsItemVisible() const;

	/** Set the item this widget is responsible for to be hidden or shown */
	void SetIsVisible(const bool bVisible);

	/** The tree item we relate to */
	TWeakPtr<IAvaOutlinerItem> ItemWeak;

	/** Reference back to the outliner so we can set visibility of a whole selection */
	TWeakPtr<FAvaOutlinerView> OutlinerViewWeak;

	TWeakPtr<FAvaOutlinerVisibilityColumn> ColumnWeak;

	TWeakPtr<SAvaOutlinerTreeRow> RowWeak;

	TUniquePtr<FScopedTransaction> UndoTransaction;

	/** Visibility brushes for the various states */
	const FSlateBrush* VisibleHoveredBrush       = nullptr;
	const FSlateBrush* VisibleNotHoveredBrush    = nullptr;
	const FSlateBrush* NotVisibleHoveredBrush    = nullptr;
	const FSlateBrush* NotVisibleNotHoveredBrush = nullptr;
};
