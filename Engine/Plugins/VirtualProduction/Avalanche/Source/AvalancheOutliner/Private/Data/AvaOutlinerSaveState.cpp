// Copyright Epic Games, Inc. All Rights Reserved.

#include "Data/AvaOutlinerSaveState.h"
#include "AvaOutliner.h"
#include "AvaOutlinerSettings.h"
#include "AvaOutlinerVersion.h"
#include "AvaOutlinerView.h"
#include "AvaSceneTree.h"
#include "Engine/World.h"
#include "Filters/IAvaOutlinerItemFilter.h"
#include "IAvaOutlinerProvider.h"
#include "Item/AvaOutlinerItemId.h"
#include "Item/AvaOutlinerItemUtils.h"
#include "Item/AvaOutlinerTreeRoot.h"
#include "Item/IAvaOutlinerItem.h"
#include "Misc/FileHelper.h"

FAvaOutlinerViewSaveState::FAvaOutlinerViewSaveState()
{
	const UAvaOutlinerSettings* const OutlinerSettings = UAvaOutlinerSettings::Get();
	check(OutlinerSettings);
	
	bUseMutedHierarchy     = OutlinerSettings->ShouldUseMutedHierarchy();
	bAutoExpandToSelection = OutlinerSettings->ShouldAutoExpandToSelection();
	ItemDefaultViewMode    = OutlinerSettings->GetItemDefaultViewMode();
	ItemProxyViewMode      = OutlinerSettings->GetItemProxyViewMode();
}

void operator<<(FArchive& Ar, FAvaOutlinerViewSaveState& InViewSaveState)
{
	Ar << InViewSaveState.ViewItemFlags;
	Ar << InViewSaveState.ActiveItemFilters;
	Ar << InViewSaveState.bUseMutedHierarchy;
	Ar << InViewSaveState.bAutoExpandToSelection;

	const int32 OutlinerVersion = Ar.CustomVer(FAvaOutlinerVersion::GUID);

	if (OutlinerVersion >= FAvaOutlinerVersion::HiddenItemTypes)
	{
		Ar << InViewSaveState.HiddenItemTypes;
	}

	if (OutlinerVersion >= FAvaOutlinerVersion::ViewModes)
	{
		Ar << InViewSaveState.ItemDefaultViewMode;
		Ar << InViewSaveState.ItemProxyViewMode;
	}

	if (OutlinerVersion >= FAvaOutlinerVersion::ColumnVisibility)
	{
		Ar << InViewSaveState.ColumnVisibility;
	}
}

void FAvaOutlinerSaveState::Serialize(FAvaOutliner& InOutliner, FArchive& Ar)
{
	UObject* const OutlinerWorld = InOutliner.GetWorld();

	if (Ar.IsSaving())
	{
		SaveSceneTree(InOutliner, /*bInResetTree*/true);
		InOutliner.ForEachOutlinerView([&InOutliner, this](const TSharedPtr<FAvaOutlinerView>& InOutlinerView)
		{
			SaveOutlinerViewState(InOutliner, *InOutlinerView);
		});

		TArray<FString> ItemIds;
		ItemColoring.GetKeys(ItemIds);
		for (const FString& ItemId : ItemIds)
		{
			if (!InOutliner.FindItem(ItemId).IsValid())
			{
				ItemColoring.Remove(ItemId);
			}
		}

		UpdateItemIdContexts(FSoftObjectPath(OutlinerWorld).ToString());
	}

	const int32 OutlinerVersion = Ar.CustomVer(FAvaOutlinerVersion::GUID);

	// No longer serialize if the Data is in Scene Tree
	if (OutlinerVersion < FAvaOutlinerVersion::SceneTree)
	{
		Ar << ItemSorting_DEPRECATED;
	}

	Ar << ItemColoring;
	Ar << OutlinerViewSaveStates;

	if (OutlinerVersion >= FAvaOutlinerVersion::ItemIdFullObjectPath)
	{
		Ar << ContextPath;	
	}

	if (Ar.IsLoading())
	{
		UpdateItemIdContexts(FSoftObjectPath(OutlinerWorld).ToString());

		// Use Scene Tree if Supported by the Outliner Version to Load the Item Ordering
		FAvaSceneTree* const SceneTree = (OutlinerVersion >= FAvaOutlinerVersion::SceneTree)
			? InOutliner.GetProvider().GetSceneTree()
			: nullptr;

		// Make sure either the Scene Tree is valid or we have an older outliner version prior to scene trees being supported
		ensure(SceneTree || OutlinerVersion < FAvaOutlinerVersion::SceneTree);
		LoadSceneTree(InOutliner.GetTreeRoot(), SceneTree, OutlinerWorld);

		InOutliner.ForEachOutlinerView([&InOutliner, this](const TSharedPtr<FAvaOutlinerView>& InOutlinerView)
		{
			LoadOutlinerViewState(InOutliner, *InOutlinerView);
		});
	}
}

void FAvaOutlinerSaveState::SaveOutlinerViewState(const FAvaOutliner& InOutliner
	, FAvaOutlinerView& InOutlinerView)
{
	const int32 OutlinerViewId = InOutlinerView.GetOutlinerViewId();
	EnsureOutlinerViewCount(OutlinerViewId);
	FAvaOutlinerViewSaveState& SaveState = OutlinerViewSaveStates[OutlinerViewId];

	SaveState.ActiveItemFilters.Empty(InOutlinerView.ActiveItemFilters.Num());
	for (const TSharedPtr<IAvaOutlinerItemFilter>& ActiveItemFilter : InOutlinerView.ActiveItemFilters)
	{
		if (ActiveItemFilter.IsValid())
		{
			SaveState.ActiveItemFilters.Add(ActiveItemFilter->GetFilterId());
		}
	}

	InOutlinerView.UpdateColumnVisibilityMap();

	SaveState.ColumnVisibility       = InOutlinerView.ColumnVisibility;
	SaveState.HiddenItemTypes        = InOutlinerView.HiddenItemTypes;
	SaveState.bUseMutedHierarchy     = InOutlinerView.bUseMutedHierarchy;
	SaveState.bAutoExpandToSelection = InOutlinerView.bAutoExpandToSelection;
	SaveState.ItemDefaultViewMode    = InOutlinerView.ItemDefaultViewMode;
	SaveState.ItemProxyViewMode      = InOutlinerView.ItemProxyViewMode;

	SaveOutlinerViewItems(InOutliner, InOutlinerView, SaveState);
}

void FAvaOutlinerSaveState::LoadOutlinerViewState(const FAvaOutliner& InOutliner
	, FAvaOutlinerView& InOutOutlinerView)
{
	const int32 OutlinerViewId = InOutOutlinerView.GetOutlinerViewId();
	EnsureOutlinerViewCount(OutlinerViewId);
	
	const FAvaOutlinerViewSaveState& SaveState = OutlinerViewSaveStates[OutlinerViewId];

	// Load Quick Type Filters
	for (const TSharedPtr<IAvaOutlinerItemFilter>& ItemFilter : InOutOutlinerView.GetItemFilters())
	{
		if (ItemFilter.IsValid() && SaveState.ActiveItemFilters.Contains(ItemFilter->GetFilterId()))
		{
			InOutOutlinerView.ActiveItemFilters.Add(ItemFilter);
		}
	}

	if (SaveState.HiddenItemTypes.IsSet())
	{
		InOutOutlinerView.HiddenItemTypes = *SaveState.HiddenItemTypes;
	}

	InOutOutlinerView.ColumnVisibility       = SaveState.ColumnVisibility;
	InOutOutlinerView.bUseMutedHierarchy     = SaveState.bUseMutedHierarchy;
	InOutOutlinerView.bAutoExpandToSelection = SaveState.bAutoExpandToSelection;
	InOutOutlinerView.ItemDefaultViewMode    = SaveState.ItemDefaultViewMode;
	InOutOutlinerView.ItemProxyViewMode      = SaveState.ItemProxyViewMode;

	LoadOutlinerViewItems(InOutliner, InOutOutlinerView, SaveState);
	InOutOutlinerView.PostLoad();
}

void FAvaOutlinerSaveState::SaveOutlinerViewItems(const FAvaOutliner& InOutliner
	, const FAvaOutlinerView& InOutlinerView
	, FAvaOutlinerViewSaveState& OutSaveState)
{
	const TSharedRef<FAvaOutlinerItem> TreeRoot = InOutliner.GetTreeRoot();

	TArray<FAvaOutlinerItemPtr> ItemsToSave = TreeRoot->GetChildren();

	const TMap<FAvaOutlinerItemId, EAvaOutlinerItemFlags>& ViewItemFlags = InOutlinerView.ViewItemFlags;
	
	TMap<FString, EAvaOutlinerItemFlags>& ItemFlagsToSave = OutSaveState.ViewItemFlags;

	ItemFlagsToSave.Reset();
	while (ItemsToSave.Num() > 0)
	{
		FAvaOutlinerItemPtr ItemToSave = ItemsToSave.Pop();
		if (ItemToSave.IsValid())
		{
			//Iteratively also Save children
			ItemsToSave.Append(ItemToSave->GetChildren());

			const FAvaOutlinerItemId ItemId = ItemToSave->GetItemId();

			//Save Item State Flags
			if (const EAvaOutlinerItemFlags* const ItemFlags = ViewItemFlags.Find(ItemId))
			{
				ItemFlagsToSave.Add(ItemId.GetStringId(), *ItemFlags);
			}
			else
			{
				ItemFlagsToSave.Remove(ItemId.GetStringId());
			}
		}
	}
}

void FAvaOutlinerSaveState::LoadOutlinerViewItems(const FAvaOutliner& InOutliner
	, FAvaOutlinerView& InOutOutlinerView
	, const FAvaOutlinerViewSaveState& InSaveState)
{
	const TSharedRef<FAvaOutlinerItem> TreeRoot = InOutliner.GetTreeRoot();

	TArray<FAvaOutlinerItemPtr> ItemsToLoad = TreeRoot->GetChildren();

	while (ItemsToLoad.Num() > 0)
	{
		FAvaOutlinerItemPtr ItemToLoad = ItemsToLoad.Pop();
		if (ItemToLoad.IsValid())
		{
			//Iteratively also Load Children
			ItemsToLoad.Append(ItemToLoad->GetChildren());

			const FAvaOutlinerItemId ItemId = ItemToLoad->GetItemId();

			//Load Item Flags
			{
				TMap<FAvaOutlinerItemId, EAvaOutlinerItemFlags>& ViewItemFlags = InOutOutlinerView.ViewItemFlags;
				
				const TMap<FString, EAvaOutlinerItemFlags>& SavedItemFlags = InSaveState.ViewItemFlags;

				if (const EAvaOutlinerItemFlags* const ItemFlags = SavedItemFlags.Find(ItemId.GetStringId()))
				{
					ViewItemFlags.Add(ItemId, *ItemFlags);
				}
				else
				{
					ViewItemFlags.Remove(ItemId);
				}
			}
		}
	}
}

void FAvaOutlinerSaveState::SaveSceneTree(const FAvaOutliner& InOutliner, bool bInResetTree)
{
	if (FAvaSceneTree* const SceneTree = InOutliner.GetProvider().GetSceneTree())
	{
		if (bInResetTree)
		{
			SceneTree->Reset();
		}
		SaveSceneTreeRecursive(InOutliner.GetTreeRoot(), *SceneTree, InOutliner.GetWorld());
		SceneTree->SortItems();
		SceneTree->ResolveObjects(InOutliner.GetWorld());
	}
}

void FAvaOutlinerSaveState::SaveSceneTreeRecursive(const FAvaOutlinerItemPtr& InParentItem, FAvaSceneTree& InSceneTree, UObject* InContext)
{
	const TArray<FAvaOutlinerItemPtr>& Children = InParentItem->GetChildren();

	const FAvaSceneItem ParentSceneItem = FAvaOutliner::MakeSceneItemFromOutlinerItem(InParentItem);

	for (int32 Index = 0; Index < Children.Num(); ++Index)
	{
		const FAvaOutlinerItemPtr& ChildItem = Children[Index];
		if (ChildItem.IsValid())
		{
			if (ChildItem->IsSortable())
			{
				const FAvaSceneItem SceneItem = FAvaOutliner::MakeSceneItemFromOutlinerItem(ChildItem);
				if (SceneItem.IsValid())
				{
					InSceneTree.GetOrAddTreeNode(SceneItem, ParentSceneItem);
				}
			}
			SaveSceneTreeRecursive(ChildItem, InSceneTree, InContext);
		}
	}
}

void FAvaOutlinerSaveState::LoadSceneTree(const FAvaOutlinerItemPtr& InParentItem, FAvaSceneTree* InSceneTree, UObject* InContext)
{
	TArray<FAvaOutlinerItemPtr>& Children = InParentItem->GetChildrenMutable();
	
	TArray<FAvaOutlinerItemPtr> Sortable;
	TArray<FAvaOutlinerItemPtr> Unsortable;
	UE::AvaOutliner::SplitItems(Children, Sortable, Unsortable);

	// If Scene Tree is valid, Item Sorting should be empty as this function only takes a valid Scene Tree if
	// loaded version supports Scene Trees (i.e. when Item Sorting stops being loaded in)
	if (InSceneTree && ensure(ItemSorting_DEPRECATED.IsEmpty()))
	{
		Sortable.Sort([InSceneTree](const FAvaOutlinerItemPtr& A, const FAvaOutlinerItemPtr& B)
		{
			const FAvaSceneTreeNode* const NodeA = InSceneTree->FindTreeNode(FAvaOutliner::MakeSceneItemFromOutlinerItem(A));
			const FAvaSceneTreeNode* const NodeB = InSceneTree->FindTreeNode(FAvaOutliner::MakeSceneItemFromOutlinerItem(B));
			return FAvaSceneTree::CompareTreeItemOrder(NodeA, NodeB);
		});
	}

	// Backwards Compat for older versions before Scene Tree existed
	if (!ItemSorting_DEPRECATED.IsEmpty())
	{
		Sortable.Sort([this](const FAvaOutlinerItemPtr& A, const FAvaOutlinerItemPtr& B)
		{
			if (A.IsValid() && B.IsValid())
			{
				const int32* const IndexA = ItemSorting_DEPRECATED.Find(A->GetItemId().GetStringId());
				const int32* const IndexB = ItemSorting_DEPRECATED.Find(B->GetItemId().GetStringId());
				if (IndexA && IndexB)
				{
					return *IndexA < *IndexB;
				}
			}
			return false;
		});
	}

	Children = MoveTemp(Unsortable);
	Children.Append(MoveTemp(Sortable));

	for (FAvaOutlinerItemPtr& Child : Children)
	{
		LoadSceneTree(Child, InSceneTree, InContext);
	}
}

void FAvaOutlinerSaveState::EnsureOutlinerViewCount(int32 InOutlinerViewId)
{
	const int32 CurrentCount     = OutlinerViewSaveStates.Num();
	const int32 MinOutlinerCount = InOutlinerViewId + 1;
	
	if (CurrentCount < MinOutlinerCount)
	{
		OutlinerViewSaveStates.AddDefaulted(MinOutlinerCount - CurrentCount);
	}
}

void FAvaOutlinerSaveState::UpdateItemIdContexts(FStringView InNewContext)
{
	// Already updated
	if (ContextPath == InNewContext)
	{
		return;
	}

	ON_SCOPE_EXIT
	{
		ContextPath = InNewContext;
	};

	auto FixItemIdMap = [&ContextPath = ContextPath, InNewContext]<typename InValueType>(TMap<FString, InValueType>& InItemIdMap)
		{
			TMap<FString, InValueType> ItemIdMapTemp = InItemIdMap;
			for (TPair<FString, InValueType>& Pair : ItemIdMapTemp)
			{
				FSoftObjectPath ObjectPath;
				ObjectPath.SetPath(ContextPath);

				FString AssetPath = ObjectPath.GetAssetPath().ToString();
				if (AssetPath.IsEmpty() || Pair.Key.StartsWith(AssetPath))
				{
					InItemIdMap.Remove(Pair.Key);

					Pair.Key.RemoveFromStart(AssetPath);
					Pair.Key.RemoveFromStart(TEXT(":"));

					FSoftObjectPath NewPath;
					NewPath.SetPath(InNewContext);
					NewPath.SetSubPathString(Pair.Key);

					InItemIdMap.Add(NewPath.ToString(), Pair.Value);
				}
			}
		};

	FixItemIdMap(ItemColoring);
	FixItemIdMap(ItemSorting_DEPRECATED);

	for (FAvaOutlinerViewSaveState& ViewState : OutlinerViewSaveStates)
	{
		FixItemIdMap(ViewState.ViewItemFlags);
	}
}
