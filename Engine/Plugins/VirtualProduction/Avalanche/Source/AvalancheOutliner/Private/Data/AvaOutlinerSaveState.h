// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"

class FAvaOutliner;
class FAvaOutlinerView;
struct FAvaSceneTree;

struct FAvaOutlinerViewSaveState
{
	FAvaOutlinerViewSaveState();

	TMap<FString, EAvaOutlinerItemFlags> ViewItemFlags;

	TMap<FName, bool> ColumnVisibility;

	TSet<FName> ActiveItemFilters;

	TOptional<TSet<FName>> HiddenItemTypes;

	EAvaOutlinerItemViewMode ItemDefaultViewMode;

	EAvaOutlinerItemViewMode ItemProxyViewMode;

	bool bUseMutedHierarchy;

	bool bAutoExpandToSelection;

	friend void operator<<(FArchive& Ar, FAvaOutlinerViewSaveState& InViewSaveState);
};

struct FAvaOutlinerSaveState
{
	friend FAvaOutliner;

	FAvaOutlinerSaveState() = default;

	void Serialize(FAvaOutliner& InOutliner, FArchive& Ar);

	/** Save the current state of the Outliner Widget (e.g. Quick Type Filters, Search Text, Scoped Item Flags, etc) */
	void SaveOutlinerViewState(const FAvaOutliner& InOutliner, FAvaOutlinerView& InOutlinerView);

	/** Load the current state of the Outliner Widget (e.g. Quick Type Filters, Search Text, Scoped Item Flags, etc) */
	void LoadOutlinerViewState(const FAvaOutliner& InOutliner, FAvaOutlinerView& InOutOutlinerView);

	const TMap<FString, FName>& GetItemColoring() const { return ItemColoring; }

private:
	/** Save the Item State in the Outliner Widget (e.g. Item Scoped Flags) */
	void SaveOutlinerViewItems(const FAvaOutliner& InOutliner
		, const FAvaOutlinerView& InOutlinerView
		, FAvaOutlinerViewSaveState& OutSaveState);

	/** Load the Item State in the Outliner Widget (e.g. Item Scoped Flags) */
	void LoadOutlinerViewItems(const FAvaOutliner& InOutliner
		, FAvaOutlinerView& InOutOutlinerView
		, const FAvaOutlinerViewSaveState& InSaveState);

	/** Save the Current Outliner Item */
	void SaveSceneTree(const FAvaOutliner& InOutliner, bool bInResetTree);

	/** Save the Current Outliner Item */
	void SaveSceneTreeRecursive(const FAvaOutlinerItemPtr& InParentItem, FAvaSceneTree& InSceneTree, UObject* InContext);

	/** Load the Current Outliner Item and the Outliner Widget View for that Item */
	void LoadSceneTree(const FAvaOutlinerItemPtr& InParentItem, FAvaSceneTree* InSceneTree, UObject* InContext);

	void EnsureOutlinerViewCount(int32 InOutlinerViewId);

	void UpdateItemIdContexts(FStringView InNewContext);

	/**
	 * Deprecated in favor of using Motion Design Scene Tree
	 * @see FAvaSceneTree
	 */
	TMap<FString, int32> ItemSorting_DEPRECATED;

	TMap<FString, FName> ItemColoring;
	
	TArray<FAvaOutlinerViewSaveState> OutlinerViewSaveStates;

	FString ContextPath;
};
