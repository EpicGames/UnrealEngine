// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FAvaOutlinerVersion::GUID(0x68D13BF9, 0x63C348C8, 0xB9E787CF, 0x22329F39);

FCustomVersionRegistration GRegisterAvalancheOutlinerVersion(FAvaOutlinerVersion::GUID
	, FAvaOutlinerVersion::LatestVersion
	, TEXT("AvalancheOutliner"));
