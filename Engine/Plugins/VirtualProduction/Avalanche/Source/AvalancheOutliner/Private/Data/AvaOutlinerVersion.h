// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"

/** Version used for serializing Motion Design Outliner Data */
struct FAvaOutlinerVersion
{
private:
	FAvaOutlinerVersion() = delete;

public:
	enum Type
	{
		/** Before file versioning was implemented */
		PreVersioning = 0,

		/** Outliner View: Serialization of Hidden Item Types */
		HiddenItemTypes,

		/** Outliner View: Serialization of View Mode Settings */
		ViewModes,

		/** Outliner View: Serialization of Outliner Column Visibility */
		ColumnVisibility,

		/** Outliner: Moved Item Ordering Serialization to using Motion Design Scene Tree */
		SceneTree,

		/** Outliner: Changed Item Id to be Full Object Path String rather than just the Sub String */
		ItemIdFullObjectPath,

		/** New versions are to be added above here */
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	const static FGuid GUID;
};
