// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragDropOps/AvaOutlinerItemDragDropOp.h"
#include "AvaOutliner.h"
#include "AvaOutlinerView.h"
#include "DragDropOps/Handlers/AvaOutlinerActorDropHandler.h"
#include "DragDropOps/Handlers/AvaOutlinerItemDropHandler.h"
#include "GameFramework/Actor.h"
#include "IAvaOutlinerProvider.h"
#include "Item/AvaOutlinerActor.h"
#include "Item/IAvaOutlinerItem.h"
#include "ScopedTransaction.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerItemDragDropOp"

namespace UE::AvaOutliner::Private
{
	FText GetActionName(EAvaOutlinerDragDropActionType InActionType)
	{
		switch (InActionType)
		{
			case EAvaOutlinerDragDropActionType::Move:
				return LOCTEXT("MoveAction", "Moving");

			case EAvaOutlinerDragDropActionType::Copy:
				return LOCTEXT("CopyAction", "Copying");
		}
		return FText::GetEmpty();
	}

	FText GetItemName(const TArray<FAvaOutlinerItemPtr>& InItems)
	{
		if (InItems.IsEmpty())
		{
			return LOCTEXT("NoItems", "0 Items");
		}

		if (InItems.Num() == 1)
		{
			return InItems[0]->GetDisplayName();
		}

		return FText::Format(LOCTEXT("ManyItems", "{0} and {1} other item(s)"), InItems[0]->GetDisplayName(), InItems.Num() - 1);
	}
}

TSharedRef<FAvaOutlinerItemDragDropOp> FAvaOutlinerItemDragDropOp::New(const TArray<FAvaOutlinerItemPtr>& InItems
	, const TSharedPtr<FAvaOutlinerView>& InOutlinerView
	, EAvaOutlinerDragDropActionType InActionType)
{
	TSharedRef<FAvaOutlinerItemDragDropOp> DragDropOp = MakeShared<FAvaOutlinerItemDragDropOp>();
	DragDropOp->Init(InItems, InOutlinerView, InActionType);
	return DragDropOp;
}

void FAvaOutlinerItemDragDropOp::GetDragDropOpActors(TArray<TWeakObjectPtr<AActor>>& OutActors) const
{
	for (const FAvaOutlinerItemPtr& Item : Items)
	{
		if (!Item.IsValid())
		{
			continue;
		}
		if (FAvaOutlinerActor* const ActorItem = Item->CastTo<FAvaOutlinerActor>())
		{
			if (AActor* const UnderlyingActor = ActorItem->GetActor())
			{
				OutActors.Add(UnderlyingActor);
			}
		}
	}
}

FAvaOutlinerItemDragDropOp::FOnItemDragDropOpInitialized& FAvaOutlinerItemDragDropOp::OnItemDragDropOpInitialized()
{
	static FOnItemDragDropOpInitialized Delegate;
	return Delegate;
}

FReply FAvaOutlinerItemDragDropOp::Drop(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem)
{
	FReply Reply = FReply::Unhandled();

	FScopedTransaction Transaction(LOCTEXT("OutlinerDropItems", "Outliner Drop Items"));

	for (const TSharedRef<FAvaOutlinerItemDropHandler>& DropHandler : DropHandlers)
	{
		if (DropHandler->GetItems().IsEmpty())
		{
			continue;
		}

		if (DropHandler->Drop(InDropZone, InTargetItem))
		{
			Reply = FReply::Handled();
		}
	}

	if (!Reply.IsEventHandled())
	{
		Transaction.Cancel();
	}

	return Reply;
}

TOptional<EItemDropZone> FAvaOutlinerItemDragDropOp::CanDrop(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem) const
{
	TSharedPtr<IAvaOutlinerView> OutlinerView = GetOutlinerView();

	// Only support Drag/Drop from Same Outliner
	if (!OutlinerView.IsValid() || InTargetItem->GetOwnerOutliner() != OutlinerView->GetOwnerOutliner())
	{
		return TOptional<EItemDropZone>();
	}

	for (const TSharedRef<FAvaOutlinerItemDropHandler>& DropHandler : DropHandlers)
	{
		if (DropHandler->GetItems().IsEmpty())
		{
			continue;
		}

		// Return the Item Drop Zone of the first Drop Handler that supports the Drop Zone and Target Item
		const TOptional<EItemDropZone> DropZone = DropHandler->CanDrop(InDropZone, InTargetItem);
		if (DropZone.IsSet())
		{
			return *DropZone;
		}
	}

	return TOptional<EItemDropZone>();
}

void FAvaOutlinerItemDragDropOp::Init(const TArray<FAvaOutlinerItemPtr>& InItems
	, const TSharedPtr<FAvaOutlinerView>& InOutlinerView
	, EAvaOutlinerDragDropActionType InActionType)
{
	Items            = InItems;
	OutlinerViewWeak = InOutlinerView;
	ActionType       = InActionType;
	MouseCursor      = EMouseCursor::GrabHandClosed;

	CurrentIconBrush = Items.Num() == 1
		? Items[0]->GetIconBrush()
		: FSlateIconFinder::FindIconForClass(AActor::StaticClass()).GetIcon();

	CurrentHoverText = FText::Format(LOCTEXT("HoverText", "{0} {1}")
		, UE::AvaOutliner::Private::GetActionName(InActionType)
		, UE::AvaOutliner::Private::GetItemName(InItems));

	CurrentIconColorAndOpacity = FSlateColor::UseForeground();

	SetupDefaults();
	Construct();

	// Add Default Drop Handlers 
	AddDropHandler<FAvaOutlinerActorDropHandler>();

	FAvaOutlinerItemDragDropOp::OnItemDragDropOpInitialized().Broadcast(*this);
}

#undef LOCTEXT_NAMESPACE
