// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragDropOps/Handlers/AvaOutlinerActorDropHandler.h"

#include "AvaOutliner.h"
#include "Framework/Application/SlateApplication.h"
#include "IAvaOutlinerProvider.h"
#include "Item/AvaOutlinerActor.h"
#include "ItemActions/AvaOutlinerAddItem.h"
#include "Widgets/Views/STableRow.h"

bool FAvaOutlinerActorDropHandler::IsDraggedItemSupported(const FAvaOutlinerItemPtr& InDraggedItem) const
{
	return InDraggedItem->IsA<FAvaOutlinerActor>();
}

TOptional<EItemDropZone> FAvaOutlinerActorDropHandler::CanDrop(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem) const
{
	switch (ActionType)
	{
		case EAvaOutlinerDragDropActionType::Move:
			// When moving, make sure the Destination is not one of the Items we're moving
			if (!Items.Contains(InTargetItem))
			{
				return InDropZone;
			}
			break;

		case EAvaOutlinerDragDropActionType::Copy:
			return InDropZone;
	}

	return TOptional<EItemDropZone>();
}

bool FAvaOutlinerActorDropHandler::Drop(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem)
{
	switch (ActionType)
	{
	case EAvaOutlinerDragDropActionType::Move:
		MoveItems(InDropZone, InTargetItem);
		break;

	case EAvaOutlinerDragDropActionType::Copy:
		StaticCastSharedRef<FAvaOutliner>(InTargetItem->GetOwnerOutliner())->DuplicateItems(Items, InTargetItem, InDropZone);
		break;

	default:
		return false;
	}

	return true;
}

void FAvaOutlinerActorDropHandler::MoveItems(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem)
{
	// Can't access this mouse event at this point, so use the Slate Application to check modifiers.
	const FAttachmentTransformRules& TransformRules = InTargetItem->GetOwnerOutliner()->GetProvider().GetTransformRule(!FSlateApplication::Get().GetModifierKeys().IsShiftDown());
	
	FAvaOutlinerAddItemParams Params;
	Params.RelativeItem             = InTargetItem;
	Params.RelativeDropZone         = InDropZone;
	Params.Flags                    = EAvaOutlinerAddItemFlags::Select | EAvaOutlinerAddItemFlags::Transact;
	Params.AttachmentTransformRules = TransformRules;
	Params.SelectionFlags           = EAvaOutlinerItemSelectionFlags::AppendToCurrentSelection;

	const TSet<FAvaOutlinerItemPtr> DraggedItemSet(Items);

	// Remove all Items whose Parent is in the Item Set 
	Items.RemoveAll([&DraggedItemSet](const FAvaOutlinerItemPtr& InItem)
		{
			return !InItem.IsValid() || !InItem->GetParent().IsValid() || DraggedItemSet.Contains(InItem->GetParent());
		});

	TSharedRef<FAvaOutliner> Outliner = StaticCastSharedRef<FAvaOutliner>(InTargetItem->GetOwnerOutliner());
	Params.Items = Items;
	Outliner->EnqueueItemAction<FAvaOutlinerAddItem>(Params);
}
