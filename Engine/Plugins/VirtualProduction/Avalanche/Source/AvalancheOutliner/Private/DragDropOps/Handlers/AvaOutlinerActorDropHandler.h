// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragDropOps/Handlers/AvaOutlinerItemDropHandler.h"

/** Class that handles Dropping Actor Items into a Target Item */
class FAvaOutlinerActorDropHandler : public FAvaOutlinerItemDropHandler
{
public:
	UE_AVA_INHERITS(FAvaOutlinerActorDropHandler, FAvaOutlinerItemDropHandler);

private:
	//~ Begin FAvaOutlinerItemDropHandler
	virtual bool IsDraggedItemSupported(const FAvaOutlinerItemPtr& InDraggedItem) const override;
	virtual TOptional<EItemDropZone> CanDrop(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem) const override;
	virtual bool Drop(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem) override;
	//~ End FAvaOutlinerItemDropHandler

	void MoveItems(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem);
};
