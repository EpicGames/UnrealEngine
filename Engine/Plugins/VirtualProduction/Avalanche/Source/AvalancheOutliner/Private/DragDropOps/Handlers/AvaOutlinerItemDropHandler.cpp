// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragDropOps/Handlers/AvaOutlinerItemDropHandler.h"
#include "DragDropOps/AvaOutlinerItemDragDropOp.h"

void FAvaOutlinerItemDropHandler::Initialize(const FAvaOutlinerItemDragDropOp& InDragDropOp)
{
	Items      = InDragDropOp.GetItems();
	ActionType = InDragDropOp.GetActionType();

	// Remove all Items that are Invalid or out of the Scope of this Handler
	Items.RemoveAll([this](const FAvaOutlinerItemPtr& InItem) { return !InItem.IsValid() || !IsDraggedItemSupported(InItem); });
}
