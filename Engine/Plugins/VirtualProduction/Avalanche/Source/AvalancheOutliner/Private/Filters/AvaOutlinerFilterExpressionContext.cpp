// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerFilterExpressionContext.h"
#include "AvaOutliner.h"
#include "AvaOutlinerModule.h"
#include "IAvaFilterExpressionFactory.h"
#include "Item/AvaOutlinerObject.h"
#include "Item/IAvaOutlinerItem.h"

void FAvaOutlinerFilterExpressionContext::SetItem(const IAvaOutlinerItem& InItem)
{
	Item = &InItem;

	FString NameInfo = Item->GetDisplayName().ToString();
	NameInfo.RemoveSpacesInline();

	ItemName = FName(NameInfo);

	FString ClassInfo = Item->GetClassName().ToString();
	ClassInfo.RemoveSpacesInline();

	ItemClass = FName(ClassInfo);
}

void FAvaOutlinerFilterExpressionContext::ClearItem()
{
	Item = nullptr;
	ItemName = TEXT("");
	ItemClass = TEXT("");
}

bool FAvaOutlinerFilterExpressionContext::TestBasicStringExpression(const FTextFilterString& InValue
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (InValue.CompareName(ItemName, InTextComparisonMode))
	{
		return true;
	}
	if (InValue.CompareName(ItemClass, InTextComparisonMode))
	{
		return true;
	}
	return false;
}

bool FAvaOutlinerFilterExpressionContext::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	const FAvaOutlinerModule& OutlinerModule = FAvaOutlinerModule::Get();

	if (OutlinerModule.CanFilterSupportComparisonOperation(InKey, InComparisonOperation))
	{
		FAvaTextFilterArgs FilterArgs;
		FilterArgs.ItemClass = ItemClass;
		FilterArgs.ItemDisplayName = ItemName;
		FilterArgs.ValueToCheck = InValue.AsName();
		FilterArgs.ComparisonOperation = InComparisonOperation;
		FilterArgs.ComparisonMode = InTextComparisonMode;

		return OutlinerModule.FilterExpression(InKey, *Item, FilterArgs);
	}

	return false;
}
