// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/TextFilterExpressionEvaluator.h"

class FName;
class FText;
class IAvaOutlinerItem;

/** Expression context to test the given asset data against the current text filter */
class FAvaOutlinerFilterExpressionContext : public ITextFilterExpressionContext
{
public:
	FAvaOutlinerFilterExpressionContext()
		: Item(nullptr) {}

	void SetItem(const IAvaOutlinerItem& InItem);

	void ClearItem();

	virtual bool TestBasicStringExpression(const FTextFilterString& InValue
		, const ETextFilterTextComparisonMode InTextComparisonMode) const override;

	virtual bool TestComplexExpression(const FName& InKey
		, const FTextFilterString& InValue
		, const ETextFilterComparisonOperation InComparisonOperation
		, const ETextFilterTextComparisonMode InTextComparisonMode) const override;

private:
	/** Pointer to the asset we're currently filtering */
	const IAvaOutlinerItem* Item;

	FName ItemName;

	FName ItemClass;
};
