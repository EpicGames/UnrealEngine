// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/AvaOutlinerItemTypeFilter.h"
#include "AvaOutlinerTextFilter.h"
#include "Item/AvaOutlinerActor.h"
#include "Item/AvaOutlinerObject.h"
#include "Item/IAvaOutlinerItem.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerItemTypeFilter"

FAvaOutlinerItemTypeFilterData::FAvaOutlinerItemTypeFilterData(const TArray<TSubclassOf<UObject>>& InFilterClasses
		, EAvaOutlinerTypeFilterMode InMode
		, const FSlateBrush* InIconBrush
		, const FText& InTooltipText
		, EClassFlags InRequiredClassFlags
		, EClassFlags InRestrictedClassFlags)
	: FilterClasses(InFilterClasses)
	, FilterMode(InMode)
	, TooltipText(InTooltipText)
	, bUseOverrideIcon(false)
	, IconBrush(InIconBrush)
	, RequiredClassFlags(InRequiredClassFlags)
	, RestrictedClassFlags(InRestrictedClassFlags)
{
}

bool FAvaOutlinerItemTypeFilterData::HasValidFilterData() const
{
	return !FilterClasses.IsEmpty() || !FilterText.IsEmptyOrWhitespace();
}

FText FAvaOutlinerItemTypeFilterData::GetTooltipText() const
{
	if (!TooltipText.IsEmpty())
	{
		return TooltipText;
	}

	TArray<FText> SupportedClasses;
	SupportedClasses.Reserve(FilterClasses.Num());

	for (TSubclassOf<UObject> FilterClass : FilterClasses)
	{
		if (FilterClass)
		{
			SupportedClasses.Add(FilterClass->GetDisplayNameText());
		}
	}

	const_cast<FAvaOutlinerItemTypeFilterData*>(this)->TooltipText = FText::Join(LOCTEXT("CommaDelimiter", ", "), SupportedClasses);
	return TooltipText;
}

const FSlateBrush* FAvaOutlinerItemTypeFilterData::GetIcon() const
{
	if (bUseOverrideIcon)
	{
		return &OverrideIcon;
	}

	if (IconBrush)
	{
		return IconBrush;
	}

	for (const TSubclassOf<UObject>& FilterClass : FilterClasses)
	{
		if (IsValid(FilterClass))
		{
			const_cast<FAvaOutlinerItemTypeFilterData*>(this)->IconBrush = FSlateIconFinder::FindIconForClass(FilterClass).GetIcon();
			return IconBrush;
		}
	}

	return nullptr;
}

EAvaOutlinerTypeFilterMode FAvaOutlinerItemTypeFilterData::GetFilterMode() const
{
	return FilterMode;
}

void FAvaOutlinerItemTypeFilterData::SetOverrideIconColor(FSlateColor InNewIconColor)
{
	OverrideIcon.TintColor = InNewIconColor;
}

bool FAvaOutlinerItemTypeFilterData::IsValidClass(const UClass* InClass) const
{
	if (!InClass)
	{
		return false;
	}

	if (InClass->HasAllClassFlags(RequiredClassFlags) && !InClass->HasAnyClassFlags(RestrictedClassFlags))
	{
		for (const TSubclassOf<UObject>& FilterClass : FilterClasses)
		{
			if (InClass->IsChildOf(FilterClass))
			{
				return true;
			}
		}
	}

	return false;
}

void FAvaOutlinerItemTypeFilterData::SetFilterText(const FText& InText)
{
	FilterText = InText;
}

bool FAvaOutlinerItemTypeFilterData::PassesFilterText(const FAvaOutlinerObject& InItem) const
{
	// Since it needs to be a const function we have to create the TextFilter here instead of being a variable
	if (FilterText.IsEmptyOrWhitespace())
	{
		return false;
	}
	FAvaOutlinerTextFilter TextFilterTest;
	TextFilterTest.SetFilterText(FilterText);
	return TextFilterTest.PassesFilter(InItem);
}

bool FAvaOutlinerItemTypeFilter::PassesFilter(FAvaOutlinerFilterType InItem) const
{
	if (FilterPassDelegate.IsBound())
	{
		return FilterPassDelegate.Execute(InItem);
	}

	auto ItemPassesFilter = [this](FAvaOutlinerFilterType InItem)
	{
		if (const FAvaOutlinerObject* ObjectItem = InItem.CastTo<FAvaOutlinerObject>())
		{
			const UObject* const Object = ObjectItem->GetObject();
			return Object && (FilterData.IsValidClass(Object->GetClass()) || FilterData.PassesFilterText(*ObjectItem));
		}
		return false;
	};

	const EAvaOutlinerTypeFilterMode FilterMode = FilterData.GetFilterMode();

	if (EnumHasAnyFlags(FilterMode, EAvaOutlinerTypeFilterMode::MatchesType))
	{
		if (ItemPassesFilter(InItem))
		{
			return true;
		}
	}

	if (EnumHasAnyFlags(FilterMode, EAvaOutlinerTypeFilterMode::ContainerOfType))
	{
		TArray<FAvaOutlinerItemPtr> RemainingItems = InItem.GetChildren();

		while (!RemainingItems.IsEmpty())
		{
			FAvaOutlinerItemPtr Item = RemainingItems.Pop();

			// Stop here if Item is invalid or if it's an Actor Item as any children of this item will be considered contained by the item itself, not the querying item
			// this could be improved if the Item Type filter as a whole ever moves past only checking UObject types, and moving past that Actors are the top level items in the outliner
			if (!Item.IsValid() || Item->IsA<FAvaOutlinerActor>())
			{
				continue;
			}

			if (ItemPassesFilter(*Item))
			{
				return true;
			}

			RemainingItems.Append(Item->GetChildren());
		}
	}

	return false;
}

const FSlateBrush* FAvaOutlinerItemTypeFilter::GetIconBrush() const
{
	return FilterData.GetIcon();
}

FText FAvaOutlinerItemTypeFilter::GetTooltipText() const
{
	return FilterData.GetTooltipText();
}

#undef LOCTEXT_NAMESPACE
