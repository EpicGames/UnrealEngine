// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/AvaOutlinerTextFilter.h"
#include "Filters/AvaOutlinerFilterExpressionContext.h"

FAvaOutlinerTextFilter::FAvaOutlinerTextFilter()
	: TextFilterExpressionContext(MakeShared<FAvaOutlinerFilterExpressionContext>())
	, TextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::Complex)
{
}

bool FAvaOutlinerTextFilter::PassesFilter(FAvaOutlinerFilterType InItem) const
{
	TextFilterExpressionContext->SetItem(InItem);
	const bool bMatched = TextFilterExpressionEvaluator.TestTextFilter(*TextFilterExpressionContext);
	TextFilterExpressionContext->ClearItem();
	return bMatched;
}

FText FAvaOutlinerTextFilter::GetFilterText() const
{
	return TextFilterExpressionEvaluator.GetFilterText();
}

void FAvaOutlinerTextFilter::SetFilterText(const FText& InFilterText)
{
	if (TextFilterExpressionEvaluator.SetFilterText(InFilterText))
	{
		OnChanged().Broadcast();
	}
}
