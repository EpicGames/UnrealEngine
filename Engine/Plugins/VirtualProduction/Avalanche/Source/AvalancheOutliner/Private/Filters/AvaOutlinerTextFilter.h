// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Misc/IFilter.h"
#include "Misc/TextFilterExpressionEvaluator.h"

class FAvaOutlinerFilterExpressionContext;
class FText;

class FAvaOutlinerTextFilter
	: public IFilter<FAvaOutlinerFilterType>
	, public TSharedFromThis<FAvaOutlinerTextFilter>
{
public:
	FAvaOutlinerTextFilter();

	//~ Begin IFilter
	virtual FChangedEvent& OnChanged() override { return ChangedEvent; }
	virtual bool PassesFilter(FAvaOutlinerFilterType InItem) const override;
	//~ End IFilter

	FText GetFilterText() const;
	
	void SetFilterText(const FText& InFilterText);

private:
	/** Transient context data, used when calling PassesFilter. Kept around to minimize re-allocations between multiple calls to PassesFilter */
	TSharedRef<FAvaOutlinerFilterExpressionContext> TextFilterExpressionContext;

	FTextFilterExpressionEvaluator TextFilterExpressionEvaluator;

	FChangedEvent ChangedEvent;
};
