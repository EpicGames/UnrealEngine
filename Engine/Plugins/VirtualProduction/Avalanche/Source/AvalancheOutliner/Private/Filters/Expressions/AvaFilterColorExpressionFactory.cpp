// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaFilterColorExpressionFactory.h"
#include "Item/AvaOutlinerItemProxy.h"
#include "Misc/TextFilterUtils.h"

const FName FAvaFilterColorExpressionFactory::KeyName = FName(TEXT("COLOR"));

FName FAvaFilterColorExpressionFactory::GetFilterIdentifier() const
{
	return KeyName;
}

bool FAvaFilterColorExpressionFactory::FilterExpression(const IAvaOutlinerItem& InItem, const FAvaTextFilterArgs& InArgs) const
{
	TOptional<FAvaOutlinerColorPair> ItemColorPair = InItem.GetColor();
	if (ItemColorPair.IsSet())
	{
		const bool bIsMatch = TextFilterUtils::TestBasicStringExpression(ItemColorPair->Key, InArgs.ValueToCheck, InArgs.ComparisonMode);
		return InArgs.ComparisonOperation == ETextFilterComparisonOperation::Equal ? bIsMatch : !bIsMatch;
	}
	return false;
}

bool FAvaFilterColorExpressionFactory::SupportsComparisonOperation(const ETextFilterComparisonOperation& InComparisonOperation) const
{
	return InComparisonOperation == ETextFilterComparisonOperation::Equal || InComparisonOperation == ETextFilterComparisonOperation::NotEqual;
}
