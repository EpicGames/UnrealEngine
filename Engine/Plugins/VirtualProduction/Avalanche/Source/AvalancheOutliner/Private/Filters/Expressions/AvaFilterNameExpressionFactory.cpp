// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaFilterNameExpressionFactory.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Item/AvaOutlinerActor.h"
#include "Item/AvaOutlinerItemProxy.h"
#include "Misc/TextFilterUtils.h"

namespace UE::AvaOutliner::Private
{
	static bool FilterActorComponentsByName(const FAvaOutlinerActor* InActorItem, FName InValueToCheck, ETextFilterComparisonOperation InComparisonOperation,ETextFilterTextComparisonMode InComparisonMode)
	{
		if (!InActorItem)
		{
			return false;
		}

		AActor* const Actor = InActorItem->GetActor();
		if (!Actor)
		{
			return false;
		}

		TArray<UActorComponent*> Components;
		Actor->GetComponents(Components);

		for (const UActorComponent* Component : Components)
		{
			FString ComponentName = Component->GetName();
			ComponentName.RemoveSpacesInline();
			ComponentName.ToUpperInline();
			const bool bIsComponentMatch = TextFilterUtils::TestBasicStringExpression(ComponentName, InValueToCheck, InComparisonMode);
			if (InComparisonOperation == ETextFilterComparisonOperation::Equal ? bIsComponentMatch : !bIsComponentMatch)
			{
				return true;
			}
		}
		return false;
	}
}

const FName FAvaFilterNameExpressionFactory::KeyName = FName(TEXT("NAME"));

FName FAvaFilterNameExpressionFactory::GetFilterIdentifier() const
{
	return KeyName;
}

bool FAvaFilterNameExpressionFactory::FilterExpression(const IAvaOutlinerItem& InItem, const FAvaTextFilterArgs& InArgs) const
{
	//If primary value match than not needing to check components if is an Actor
	const bool bIsMatch = TextFilterUtils::TestBasicStringExpression(InArgs.ItemDisplayName, InArgs.ValueToCheck, InArgs.ComparisonMode);
	if (InArgs.ComparisonOperation == ETextFilterComparisonOperation::Equal ? bIsMatch : !bIsMatch)
	{
		return true;
	}

	if (const FAvaOutlinerActor* Actor = InItem.CastTo<FAvaOutlinerActor>())
	{
		return UE::AvaOutliner::Private::FilterActorComponentsByName(Actor
			, InArgs.ValueToCheck
			, InArgs.ComparisonOperation
			, ETextFilterTextComparisonMode::Exact);
	}

	if (InArgs.ItemDisplayName == TEXT("MATERIALS"))
	{
		if (const FAvaOutlinerItemProxy* Proxy = InItem.CastTo<FAvaOutlinerItemProxy>())
		{
			for (const FAvaOutlinerItemPtr& ProxyChild : Proxy->GetChildren())
			{
				FString ItemName = ProxyChild->GetDisplayName().ToString();
				ItemName.RemoveSpacesInline();
				ItemName.ToUpperInline();
				const bool bIsProxyMatch = TextFilterUtils::TestBasicStringExpression(ItemName, InArgs.ValueToCheck, InArgs.ComparisonMode);
				if (InArgs.ComparisonOperation == ETextFilterComparisonOperation::Equal ? bIsProxyMatch : !bIsProxyMatch)
				{
					return true;
				}
			}
		}
	}
	return false;
}

bool FAvaFilterNameExpressionFactory::SupportsComparisonOperation(const ETextFilterComparisonOperation& InComparisonOperation) const
{
	return InComparisonOperation == ETextFilterComparisonOperation::Equal || InComparisonOperation == ETextFilterComparisonOperation::NotEqual;
}
