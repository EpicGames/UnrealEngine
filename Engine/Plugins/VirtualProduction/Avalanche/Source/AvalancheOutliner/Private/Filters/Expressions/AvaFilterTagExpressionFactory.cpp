// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaFilterTagExpressionFactory.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Item/AvaOutlinerActor.h"
#include "Item/AvaOutlinerItemProxy.h"
#include "Misc/TextFilterUtils.h"

const FName FAvaFilterTagExpressionFactory::KeyName = FName(TEXT("TAG"));

FName FAvaFilterTagExpressionFactory::GetFilterIdentifier() const
{
	return KeyName;
}

bool FAvaFilterTagExpressionFactory::FilterExpression(const IAvaOutlinerItem& InItem, const FAvaTextFilterArgs& InArgs) const
{
	//If primary value match than not needing to check components if is an Actor
	for (const FName& Tag : InItem.GetTags())
	{
		const bool bIsMatch = TextFilterUtils::TestBasicStringExpression(Tag, InArgs.ValueToCheck, InArgs.ComparisonMode);
		if (InArgs.ComparisonOperation == ETextFilterComparisonOperation::Equal ? bIsMatch : !bIsMatch)
		{
			return true;
		}
	}

	if (const FAvaOutlinerActor* Actor = InItem.CastTo<FAvaOutlinerActor>())
	{
		TArray<UActorComponent*> AComponents;
		Actor->GetActor()->GetComponents<UActorComponent>(AComponents);
		for (const UActorComponent* AComponent : AComponents)
		{
			for (const FName& Tag : AComponent->ComponentTags)
			{
				const bool bIsMatch = TextFilterUtils::TestBasicStringExpression(Tag, InArgs.ValueToCheck, InArgs.ComparisonMode);
				if (InArgs.ComparisonOperation == ETextFilterComparisonOperation::Equal ? bIsMatch : !bIsMatch)
				{
					return true;
				}
			}
		}
	}
	return false;
}

bool FAvaFilterTagExpressionFactory::SupportsComparisonOperation(const ETextFilterComparisonOperation& InComparisonOperation) const
{
	return InComparisonOperation == ETextFilterComparisonOperation::Equal || InComparisonOperation == ETextFilterComparisonOperation::NotEqual;
}
