// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaFilterTypeExpressionFactory.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Item/AvaOutlinerActor.h"
#include "Item/AvaOutlinerItemProxy.h"
#include "Misc/TextFilterUtils.h"

namespace UE::AvaOutliner::Private
{
	bool FilterActorComponentsByType(const FAvaOutlinerActor* InActorItem, FName InValueToCheck, ETextFilterComparisonOperation InComparisonOperation,ETextFilterTextComparisonMode InComparisonMode)
	{
		if (!InActorItem)
		{
			return false;
		}

		AActor* const Actor = InActorItem->GetActor();
		if (!Actor)
		{
			return false;
		}

		TArray<UActorComponent*> Components;
		Actor->GetComponents(Components);
		for (const UActorComponent* Component : Components)
		{
			UClass* const ComponentClass = Component->GetClass();
			if (!ComponentClass)
			{
				continue;
			}

			FString ComponentName = ComponentClass->GetName();
			ComponentName.RemoveSpacesInline();
			ComponentName.ToUpperInline();

			const bool bIsComponentMatch = TextFilterUtils::TestBasicStringExpression(ComponentName, InValueToCheck, InComparisonMode);
			if (InComparisonOperation == ETextFilterComparisonOperation::Equal ? bIsComponentMatch : !bIsComponentMatch)
			{
				return true;
			}
		}
		return false;
	}
}

const FName FAvaFilterTypeExpressionFactory::KeyName = FName(TEXT("TYPE"));

FName FAvaFilterTypeExpressionFactory::GetFilterIdentifier() const
{
	return KeyName;
}

bool FAvaFilterTypeExpressionFactory::FilterExpression(const IAvaOutlinerItem& InItem, const FAvaTextFilterArgs& InArgs) const
{
	//If primary value match than not needing to check components if is an Actor
	const bool bIsMatch = TextFilterUtils::TestBasicStringExpression(InArgs.ItemClass, InArgs.ValueToCheck, InArgs.ComparisonMode);
	if (InArgs.ComparisonOperation == ETextFilterComparisonOperation::Equal ? bIsMatch : !bIsMatch)
	{
		return true;
	}

	if (const FAvaOutlinerActor* Actor = InItem.CastTo<FAvaOutlinerActor>())
	{
		return UE::AvaOutliner::Private::FilterActorComponentsByType(Actor
			, InArgs.ValueToCheck
			, InArgs.ComparisonOperation
			, InArgs.ComparisonMode);
	}
	return false;
}

bool FAvaFilterTypeExpressionFactory::SupportsComparisonOperation(const ETextFilterComparisonOperation& InComparisonOperation) const
{
	return InComparisonOperation == ETextFilterComparisonOperation::Equal || InComparisonOperation == ETextFilterComparisonOperation::NotEqual;
}
