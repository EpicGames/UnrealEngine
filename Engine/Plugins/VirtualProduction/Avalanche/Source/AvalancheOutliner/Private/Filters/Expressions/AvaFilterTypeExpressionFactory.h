// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/IAvaFilterExpressionFactory.h"

class FAvaFilterTypeExpressionFactory final : public IAvaFilterExpressionFactory
{
public:
	static const FName KeyName;

	//~ Begin IAvaFilterExpressionFactory interface
	virtual FName GetFilterIdentifier() const override;
	virtual bool FilterExpression(const IAvaOutlinerItem& InItem, const FAvaTextFilterArgs& InArgs) const override;
	virtual bool SupportsComparisonOperation(const ETextFilterComparisonOperation& InComparisonOperation) const override;
	//~ End IAvaFilterExpressionFactory interface
};
