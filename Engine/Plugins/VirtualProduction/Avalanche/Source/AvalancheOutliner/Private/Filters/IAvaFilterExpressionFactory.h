// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"

class IAvaOutlinerItem;

/**
 * Holds the arguments required for the expression evaluation.
 */
struct FAvaTextFilterArgs
{
	/** item Class */
	FName ItemClass;

	/** Item Display Name */
	FName ItemDisplayName;

	/** Second Value */
	FName ValueToCheck;

	/** Comparison Operation requested (Equal/Greater/etc...) still needed for certain Filter */
	ETextFilterComparisonOperation ComparisonOperation;

	/** Comparison Mode requested see ETextFilterTextComparisonMode for more information */
	ETextFilterTextComparisonMode ComparisonMode;
};

class IAvaFilterExpressionFactory : public TSharedFromThis<IAvaFilterExpressionFactory>
{
public:
	virtual ~IAvaFilterExpressionFactory() {}

	/** Get the filter identifier for this factory */
	virtual FName GetFilterIdentifier() const = 0;

	/**
 	 * Create and return an Instance of the AvaFilterExpressionFactory Requested
 	 * @tparam InFilterExpressionFactoryType The type of the factory to instantiate
 	 * @param InArgs Additional Args for constructor of the AvaFilterExpressionFactory class if needed
 	 * @return The new filter factory
 	 */
	template<typename InFilterExpressionFactoryType, typename... InArgsType
			, typename = typename TEnableIf<TIsDerivedFrom<InFilterExpressionFactoryType, IAvaFilterExpressionFactory>::IsDerived>::Type>
	static TSharedRef<InFilterExpressionFactoryType> MakeInstance(InArgsType&&... InArgs)
	{
		return MakeShared<InFilterExpressionFactoryType>(Forward<InArgsType>(InArgs)...);
	}

	/**
	 * Evaluate the expression with the Value given
	 * @param InItem Current Item being checked
	 * @param InArgs The argument containing the data to evaluate the expression see FAvaTextFilterArgs for more information
	 * @return True if the expression evaluate to True, False otherwise
	 */
	virtual bool FilterExpression(const IAvaOutlinerItem& InItem, const FAvaTextFilterArgs& InArgs) const = 0;

	/**
	 * Whether the factory support a given Comparison Operation
	 * @param InComparisonOperation Comparison Operation to check
	 * @return True if Factory support passed Comparison Operation, False otherwise
	 */
	virtual bool SupportsComparisonOperation(const ETextFilterComparisonOperation& InComparisonOperation) const = 0;
};
