// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "AvaType.h"

struct FAssetSearchBoxSuggestion;

enum class EAvaFilterSuggestionType : uint8
{
	None      = 0x00, /** None */
	Generic   = 0x01, /** Executed without the need to cycle through outliner item (for example Colors in the Settings) */
	ItemBased = 0x10, /** Executed for each item in the outliner, the item is passed down to get suggestion from it */
	All       = 0x11  /** Executed for both ways */
};

ENUM_CLASS_FLAGS(EAvaFilterSuggestionType)

/**
 * Holds the arguments required for the suggestion factory to get the suggestion.
 */
struct FAvaFilterSuggestionPayload : public IAvaTypeCastable, public TSharedFromThis<FAvaFilterSuggestionPayload>
{
public:
	UE_AVA_INHERITS(FAvaFilterSuggestionPayload, IAvaTypeCastable)

	FAvaFilterSuggestionPayload(TArray<FAssetSearchBoxSuggestion>& OutPossibleSuggestions
		, const FString& InFilterValue)
		: OutPossibleSuggestions(OutPossibleSuggestions)
		, FilterValue(InFilterValue)
	{
	}

public:
	/** Holds all the suggestion that will be shown */
	TArray<FAssetSearchBoxSuggestion>& OutPossibleSuggestions;

	/** The current string value written in the searchbox */
	FString FilterValue;
};

/**
 * Specialization for the ItemBased suggestion factory that needs more arguments
 */
struct FAvaFilterSuggestionItemPayload : public FAvaFilterSuggestionPayload
{
public:
	UE_AVA_INHERITS(FAvaFilterSuggestionItemPayload, FAvaFilterSuggestionPayload)

	FAvaFilterSuggestionItemPayload(TArray<FAssetSearchBoxSuggestion>& OutPossibleSuggestions
		, const FString& InFilterValue
		, const FAvaOutlinerItemPtr& InItem
		, TSet<FString>& OutFilterCache)
		: FAvaFilterSuggestionPayload(OutPossibleSuggestions, InFilterValue)
		, Item(InItem)
		, OutFilterCache(OutFilterCache)
	{
	}

public:
	/** Current item to check */
	FAvaOutlinerItemPtr Item;

	/** Used to speed up the check on whether a suggestion is already added */
	TSet<FString>& OutFilterCache;
};

class IAvaFilterSuggestionFactory : public TSharedFromThis<IAvaFilterSuggestionFactory>
{
public:
	virtual ~IAvaFilterSuggestionFactory() {}

	/**
	  * Create and return an Instance of the AvaFilterSuggestionFactory Requested
	  * @tparam InSuggestionFactoryType The type of the factory to instantiate
	  * @param InArgs Additional Args for constructor of the AvaFilterSuggestionFactory class if needed
	  * @return The new suggestion factory
	  */
	template<typename InSuggestionFactoryType, typename... InArgsType
		, typename = typename TEnableIf<TIsDerivedFrom<InSuggestionFactoryType, IAvaFilterSuggestionFactory>::IsDerived>::Type>
	static TSharedRef<InSuggestionFactoryType> MakeInstance(InArgsType&&... InArgs)
	{
		return MakeShared<InSuggestionFactoryType>(Forward<InArgsType>(InArgs)...);
	}

	/**
	 * Get the type of the suggestion
	 * @return The type of the suggestion see EAvaFilterSuggestionType for more information
	 */
	virtual EAvaFilterSuggestionType GetSuggestionType() const = 0;

	/** Get the suggestion identifier for this factory */
	virtual FName GetSuggestionIdentifier() const = 0;

	/**
	 * Add suggestions entry to be shown
	 * @param InPayload Arguments to be passed to the factory to get the suggestion to show
	 */
	virtual void AddSuggestion(const TSharedRef<FAvaFilterSuggestionPayload> InPayload) = 0;
};
