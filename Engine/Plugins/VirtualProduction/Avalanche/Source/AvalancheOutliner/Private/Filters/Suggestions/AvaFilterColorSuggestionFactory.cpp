// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaFilterColorSuggestionFactory.h"
#include "AvaOutlinerSettings.h"
#include "SAssetSearchBox.h"

#define LOCTEXT_NAMESPACE "AvaFilterColorSuggestionFactory"

const FName FAvaFilterColorSuggestionFactory::KeyName = FName(TEXT("COLOR"));

void FAvaFilterColorSuggestionFactory::AddSuggestion(const TSharedRef<FAvaFilterSuggestionPayload> InPayload)
{
	TSet<FString> ColorCache;
	const FText ColorCategoryLabel = LOCTEXT("ColorCategoryLabel", "Ava-Outliner-Colors");

	if (const UAvaOutlinerSettings* Settings = UAvaOutlinerSettings::Get())
	{
		for (const TTuple<FName, FLinearColor> ColorPair : Settings->GetColorMap())
		{
			FString ColorName = ColorPair.Key.ToString();
			const FText DisplayName = FText::FromString(ColorName);
			ColorName.RemoveSpacesInline();
			FString ColorSuggestion = FString::Printf(TEXT("Color=%s"), *ColorName);
			if ((InPayload->FilterValue.IsEmpty() || ColorSuggestion.Contains(InPayload->FilterValue)) && !ColorCache.Contains(ColorName))
			{
				ColorCache.Add(ColorName);
				InPayload->OutPossibleSuggestions.Add(FAssetSearchBoxSuggestion{MoveTemp(ColorSuggestion), DisplayName, ColorCategoryLabel});
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
