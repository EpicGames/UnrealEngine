// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/IAvaFilterSuggestionFactory.h"

class FAvaFilterColorSuggestionFactory final : public IAvaFilterSuggestionFactory
{
public:
	static const FName KeyName;

	//~ Begin IAvaFilterSuggestionFactory interface
	virtual EAvaFilterSuggestionType GetSuggestionType() const override { return EAvaFilterSuggestionType::Generic; }
	virtual FName GetSuggestionIdentifier() const override { return KeyName; }
	virtual void AddSuggestion(const TSharedRef<FAvaFilterSuggestionPayload> InPayload) override;
	//~ End IAvaFilterSuggestionFactory interface
};
