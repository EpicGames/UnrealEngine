// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaFilterNameSuggestionFactory.h"
#include "Item/IAvaOutlinerItem.h"
#include "SAssetSearchBox.h"

#define LOCTEXT_NAMESPACE "AvaFilterNameSuggestionFactory"

const FName FAvaFilterNameSuggestionFactory::KeyName = FName(TEXT("NAME"));

void FAvaFilterNameSuggestionFactory::AddSuggestion(const TSharedRef<FAvaFilterSuggestionPayload> InPayload)
{
	if (const FAvaFilterSuggestionItemPayload* FilterPayload = InPayload->CastTo<FAvaFilterSuggestionItemPayload>())
	{
		if (FilterPayload->Item)
		{
			const FText NameCategoryLabel = LOCTEXT("NameCategoryLabel", "Ava-Outliner-Name");
			FString DisplayName = FilterPayload->Item->GetDisplayName().ToString();
			DisplayName.RemoveSpacesInline();
			FString NameSuggestion = FString::Printf(TEXT("Name=%s"), *DisplayName);

			if ((FilterPayload->FilterValue.IsEmpty() || NameSuggestion.Contains(FilterPayload->FilterValue)) && !FilterPayload->OutFilterCache.Contains(NameSuggestion))
			{
				FilterPayload->OutFilterCache.Add(NameSuggestion);
				FilterPayload->OutPossibleSuggestions.Add(FAssetSearchBoxSuggestion{MoveTemp(NameSuggestion), FilterPayload->Item->GetDisplayName(), NameCategoryLabel});
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
