// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaFilterTagSuggestionFactory.h"
#include "Item/IAvaOutlinerItem.h"
#include "SAssetSearchBox.h"

#define LOCTEXT_NAMESPACE "AvaFilterTagSuggestionFactory"

const FName FAvaFilterTagSuggestionFactory::KeyName = FName(TEXT("TAG"));

void FAvaFilterTagSuggestionFactory::AddSuggestion(const TSharedRef<FAvaFilterSuggestionPayload> InPayload)
{
	if (const FAvaFilterSuggestionItemPayload* FilterPayload = InPayload->CastTo<FAvaFilterSuggestionItemPayload>())
	{
		if (FilterPayload->Item)
		{
			const FText TagCategoryLabel = LOCTEXT("TagCategoryLabel", "Ava-Outliner-Tags");
			for (const FName& ItemTag : FilterPayload->Item->GetTags())
			{
				FString TagName = ItemTag.ToString();
				TagName.RemoveSpacesInline();
				const FText NameForDisplayRoot = FText::FromName(ItemTag);
				FString TagSuggestion = FString::Printf(TEXT("Tag=%s"), *TagName);
				if ((FilterPayload->FilterValue.IsEmpty() || TagSuggestion.Contains(FilterPayload->FilterValue)) && !FilterPayload->OutFilterCache.Contains(TagSuggestion))
				{
					FilterPayload->OutFilterCache.Add(TagSuggestion);
					FilterPayload->OutPossibleSuggestions.Add(FAssetSearchBoxSuggestion{MoveTemp(TagSuggestion), NameForDisplayRoot, TagCategoryLabel});
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
