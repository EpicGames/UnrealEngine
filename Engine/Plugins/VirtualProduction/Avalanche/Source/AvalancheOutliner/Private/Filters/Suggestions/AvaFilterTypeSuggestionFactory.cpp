// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaFilterTypeSuggestionFactory.h"
#include "Item/IAvaOutlinerItem.h"
#include "SAssetSearchBox.h"

#define LOCTEXT_NAMESPACE "AvaFilterTypeSuggestionFactory"

const FName FAvaFilterTypeSuggestionFactory::KeyName = FName(TEXT("TYPE"));

void FAvaFilterTypeSuggestionFactory::AddSuggestion(const TSharedRef<FAvaFilterSuggestionPayload> InPayload)
{
	if (const FAvaFilterSuggestionItemPayload* FilterPayload = InPayload->CastTo<FAvaFilterSuggestionItemPayload>())
	{
		if (FilterPayload->Item)
		{
			const FText TypeCategoryLabel = LOCTEXT("TypeCategoryLabel", "Ava-Outliner-Type");
			FString ClassName = FilterPayload->Item->GetClassName().ToString();
			ClassName.RemoveSpacesInline();
			const FText ClassNameWithSpaces = FilterPayload->Item->GetClassName();
			FString TypeSuggestion = FString::Printf(TEXT("Type=%s"), *ClassName);
			if ((FilterPayload->FilterValue.IsEmpty() || TypeSuggestion.Contains(FilterPayload->FilterValue)) && !FilterPayload->OutFilterCache.Contains(TypeSuggestion))
			{
				FilterPayload->OutFilterCache.Add(TypeSuggestion);
				FilterPayload->OutPossibleSuggestions.Add(FAssetSearchBoxSuggestion{MoveTemp(TypeSuggestion),ClassNameWithSpaces, TypeCategoryLabel});
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
