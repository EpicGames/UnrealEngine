// Copyright Epic Games, Inc. All Rights Reserved.

#include "Icon/AvaOutlinerObjectIconCustomization.h"
#include "Item/AvaOutlinerObject.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"

FAvaOutlinerObjectIconCustomization::FAvaOutlinerObjectIconCustomization(const UClass* InSupportedClass)
{
	SupportedClassName = FName(TEXT(""));

	if (IsValid(InSupportedClass))
	{
		SupportedClassName = InSupportedClass->GetFName();
	}
}

void FAvaOutlinerObjectIconCustomization::SetOverriddenIcon(const FOnGetOverriddenObjectIcon& InOverriddenIcon)
{
	OnGetOverriddenIcon = InOverriddenIcon;
}

FName FAvaOutlinerObjectIconCustomization::GetOutlinerItemIdentifier() const
{
	return SupportedClassName;
}

bool FAvaOutlinerObjectIconCustomization::HasOverrideIcon(TSharedPtr<const FAvaOutlinerItem> InOutlinerItem) const
{
	if (const FAvaOutlinerObject* ObjectItem = InOutlinerItem->CastTo<FAvaOutlinerObject>())
	{
		if (const UObject* Object = ObjectItem->GetObject())
		{
			if (const UObject* ObjectClass = Object->GetClass())
			{
				return ObjectClass->GetFName() == SupportedClassName;
			}
		}
	}

	return false;
}

FSlateIcon FAvaOutlinerObjectIconCustomization::GetOverrideIcon(TSharedPtr<const FAvaOutlinerItem> InOutlinerItem) const
{
	if (HasOverrideIcon(InOutlinerItem) && OnGetOverriddenIcon.IsBound())
	{
		return OnGetOverriddenIcon.Execute(InOutlinerItem);
	}

	return FSlateIcon();
}
