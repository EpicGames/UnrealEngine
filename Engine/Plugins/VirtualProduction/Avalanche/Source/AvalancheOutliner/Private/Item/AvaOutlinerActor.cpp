// Copyright Epic Games, Inc. All Rights Reserved.

#include "Item/AvaOutlinerActor.h"
#include "ActorFactories/ActorFactory.h"
#include "AssetSelection.h"
#include "AvaOutliner.h"
#include "AvaOutlinerSubsystem.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "DragDropOps/AvaOutlinerItemDragDropOp.h"
#include "Editor/UnrealEdEngine.h"
#include "GameFramework/Actor.h"
#include "Item/AvaOutlinerComponentProxy.h"
#include "ItemActions/AvaOutlinerAddItem.h"
#include "ItemActions/AvaOutlinerRemoveItem.h"
#include "ScopedTransaction.h"
#include "UnrealEdGlobals.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerActor"

FAvaOutlinerActor::FAvaOutlinerActor(IAvaOutliner& InOutliner, AActor* InActor)
	: Super(InOutliner, InActor)
	, Actor(InActor)
{
}

void FAvaOutlinerActor::FindChildren(TArray<FAvaOutlinerItemPtr>& OutChildren, bool bRecursive)
{
	Super::FindChildren(OutChildren, bRecursive);

	AActor* const UnderlyingActor = GetActor();
	if (!UnderlyingActor)
	{
		return;
	}

	FAvaOutliner& OutlinerPrivate = static_cast<FAvaOutliner&>(Outliner);
	const TArray<TWeakObjectPtr<AActor>> OutlinerChildren = OutlinerPrivate.GetActorSceneOutlinerChildren(UnderlyingActor);

	// Worst case and most likely case: all Outliner Children are valid.
	// Note: if recursive, re-allocations will still need to be done past this reserve count, as it's unknown at this point how many items are going to be added
	OutChildren.Reserve(OutChildren.Num() + OutlinerChildren.Num());

	for (const TWeakObjectPtr<AActor>& ChildWeak : OutlinerChildren)
	{
		if (AActor* const Child = ChildWeak.Get())
		{
			const FAvaOutlinerItemPtr ChildActorItem = Outliner.FindOrAdd<FAvaOutlinerActor>(Child);
			const FAvaOutlinerItemFlagGuard Guard(ChildActorItem, EAvaOutlinerItemFlags::IgnorePendingKill);
			OutChildren.Add(ChildActorItem);
			if (bRecursive)
			{
				ChildActorItem->FindChildren(OutChildren, bRecursive);
			}
		}
	}
}

void FAvaOutlinerActor::GetItemProxies(TArray<TSharedPtr<FAvaOutlinerItemProxy>>& OutItemProxies)
{
	Super::GetItemProxies(OutItemProxies);
	if (TSharedPtr<FAvaOutlinerItemProxy> ComponentItemProxy = Outliner.GetOrCreateItemProxy<FAvaOutlinerComponentProxy>(SharedThis(this)))
	{
		OutItemProxies.Add(ComponentItemProxy);
	}
}

bool FAvaOutlinerActor::CanAddChild(const FAvaOutlinerItemPtr& InChild) const
{
	// Super checks InChild.IsValid
	return Super::CanAddChild(InChild) && InChild->IsA<FAvaOutlinerActor>();
}

void FAvaOutlinerActor::OnChildrenAdded(TConstArrayView<FAvaOutlinerItemPtr> InAddedChildren, const FAvaOutlinerAddItemParams& InAddItemParams)
{
	Super::OnChildrenAdded(InAddedChildren, InAddItemParams);

	for (const FAvaOutlinerItemPtr& Item : InAddedChildren)
	{
		FAvaOutlinerActor* ActorItem = Item->CastTo<FAvaOutlinerActor>();
		if (!ActorItem)
		{
			continue;
		}

		if (AActor* const ChildActor = ActorItem->GetActor())
		{
			AActor* const ParentActor = GetActor();

			USceneComponent* const ChildRoot  = ChildActor->GetRootComponent();
			USceneComponent* const ParentRoot = ParentActor
				? ParentActor->GetDefaultAttachComponent()
				: nullptr;

			// Set Component Mobility to Moveable if ParentRoot isn't Static
			if (ChildRoot && ChildRoot->Mobility == EComponentMobility::Static
				&& ParentRoot && ParentRoot->Mobility != EComponentMobility::Static)
			{
				ChildRoot->SetMobility(ParentRoot->Mobility);
			}

			EAvaOutlinerHierarchyChangeType ChangeType;

			if (ParentActor != ChildActor->GetAttachParentActor() && GUnrealEd->CanParentActors(ParentActor, ChildActor))
			{
				// Attachment is persisted on the child so modify both actors for Undo/Redo but do not mark the Parent package dirty
				ChildActor->Modify();
				ParentActor->Modify(false);

				// Snap to socket if a valid socket name was provided, otherwise attach without changing the relative transform
				ChildRoot->AttachToComponent(ParentRoot
					, InAddItemParams.AttachmentTransformRules.Get(FAttachmentTransformRules::KeepWorldTransform)
					, NAME_None);

				ChangeType = EAvaOutlinerHierarchyChangeType::Attached;
			}
			else
			{
				ChangeType = EAvaOutlinerHierarchyChangeType::Rearranged;
			}

			FAvaOutliner& OutlinerPrivate = static_cast<FAvaOutliner&>(Outliner);
			OutlinerPrivate.QueueActorHierarchyChange(ChildActor, ParentActor, ChangeType);

		}
	}
}

bool FAvaOutlinerActor::RemoveChild(const FAvaOutlinerRemoveItemParams& InRemoveItemParams)
{
	//Check that we're removing a Child Item that is directly under us, and not a Grand child
	if (InRemoveItemParams.Item.IsValid() && Children.Contains(InRemoveItemParams.Item))
	{
		if (const FAvaOutlinerActor* const ActorItem = InRemoveItemParams.Item->CastTo<FAvaOutlinerActor>())
		{
			AActor* const ChildActor = ActorItem->GetActor();
			
			const AActor* const ParentActor = GetActor();

			if (ChildActor && ParentActor)
			{
				USceneComponent* const ChildRoot = ChildActor->GetRootComponent();
				if (ChildRoot && ChildRoot->GetAttachParent())
				{
					AActor* OldParentActor = ChildRoot->GetAttachParent()->GetOwner();
					//When Detaching, Make sure that this Child Actor is attached to us
					if (OldParentActor == ParentActor)
					{
						OldParentActor->Modify(false);
						ChildRoot->DetachFromComponent(InRemoveItemParams.DetachmentTransformRules.Get(FDetachmentTransformRules::KeepWorldTransform));

						FAvaOutliner& OutlinerPrivate = static_cast<FAvaOutliner&>(Outliner);
						OutlinerPrivate.QueueActorHierarchyChange(ChildActor, ParentActor, EAvaOutlinerHierarchyChangeType::Detached);
					}
				}
			}
		}
		return RemoveChildChecked(InRemoveItemParams.Item);
	}
	return false;
}

bool FAvaOutlinerActor::IsAllowedInOutliner() const
{
	FAvaOutliner& OutlinerPrivate = static_cast<FAvaOutliner&>(Outliner);
	return OutlinerPrivate.IsActorAllowedInOutliner(GetActor());
}

EAvaOutlinerItemViewMode FAvaOutlinerActor::GetSupportedViewModes(const FAvaOutlinerView& InOutlinerView) const
{
	// Actors should only be visualized in Outliner View and not appear in the Item Column List
	// Support any other type of View Mode
	return EAvaOutlinerItemViewMode::ItemTree | ~EAvaOutlinerItemViewMode::HorizontalItemList;
}

bool FAvaOutlinerActor::GetVisibility(EAvaOutlinerVisibilityType VisibilityType) const
{
	if (const AActor* const UnderlyingActor = GetActor())
	{
		switch (VisibilityType)
		{
			case EAvaOutlinerVisibilityType::Editor:
				return !UnderlyingActor->IsTemporarilyHiddenInEditor(true);

			case EAvaOutlinerVisibilityType::Runtime:
				return !UnderlyingActor->IsHidden();

			default:
				break;
		}
	}
	return false;
}

void FAvaOutlinerActor::OnVisibilityChanged(EAvaOutlinerVisibilityType Visibility, bool bNewVisibility)
{
	if (AActor* const UnderlyingActor = GetActor())
	{
		switch (Visibility)
		{
			case EAvaOutlinerVisibilityType::Editor: UnderlyingActor->Modify();
				UnderlyingActor->SetIsTemporarilyHiddenInEditor(!bNewVisibility);
				break;

			case EAvaOutlinerVisibilityType::Runtime: UnderlyingActor->Modify();
				UnderlyingActor->SetActorHiddenInGame(!bNewVisibility);
				break;

			default:
				break;
		}
	}
}

bool FAvaOutlinerActor::Rename(const FString& InName)
{
	AActor* const UnderlyingActor = GetActor();
	const bool bIsActorLabelEditable = UnderlyingActor && UnderlyingActor->IsActorLabelEditable();

	bool bRenamed = false;
	
	if (bIsActorLabelEditable && !InName.Equals(UnderlyingActor->GetActorLabel(), ESearchCase::CaseSensitive))
	{
		const FScopedTransaction Transaction(LOCTEXT("AvaOutlinerRenameActor", "Rename Actor"));
		FActorLabelUtilities::RenameExistingActor(UnderlyingActor, InName);
		bRenamed = true;
	}
	
	Super::Rename(InName);
	
	return bRenamed;
}

void FAvaOutlinerActor::SetLocked(bool bInIsLocked)
{
	if (AActor* const UnderlyingActor = GetActor())
	{
		UnderlyingActor->SetLockLocation(bInIsLocked);
	}
	Super::SetLocked(bInIsLocked);
}

bool FAvaOutlinerActor::IsLocked() const
{
	if (const AActor* const UnderlyingActor = GetActor())
	{
		return UnderlyingActor->IsLockLocation();
	}
	return false;
}

FText FAvaOutlinerActor::GetDisplayName() const
{
	if (const AActor* const UnderlyingActor = GetActor())
	{
		return FText::FromString(UnderlyingActor->GetActorLabel());
	}
	return FText::GetEmpty();
}

TArray<FName> FAvaOutlinerActor::GetTags() const
{
	if (AActor* const UnderlyingActor = GetActor())
	{
		return UnderlyingActor->Tags;
	}
	return Super::GetTags();
}

TOptional<EItemDropZone> FAvaOutlinerActor::CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone)
{
	if (const TSharedPtr<FAssetDragDropOp> AssetDragDropOp = InDragDropEvent.GetOperationAs<FAssetDragDropOp>())
	{
		//Can't drop if a Single of the Assets is an Invalid Asset
		for (const FAssetData& Asset : AssetDragDropOp->GetAssets())
		{
			UActorFactory* ActorFactory = AssetDragDropOp->GetActorFactory();
			if (!ActorFactory)
			{
				ActorFactory = FActorFactoryAssetProxy::GetFactoryForAsset(Asset);
			}
			if (!ActorFactory || !ActorFactory->CanPlaceElementsFromAssetData(Asset))
			{
				return TOptional<EItemDropZone>();
			}
		}
		return InDropZone;
	}

	return Super::CanAcceptDrop(InDragDropEvent, InDropZone);
}

FReply FAvaOutlinerActor::AcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone InDropZone)
{
	if (const TSharedPtr<FAssetDragDropOp> AssetDragDropOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>())
	{
		if (const AActor* const UnderlyingActor = GetActor())
		{
			return CreateItemsFromAssetDrop(AssetDragDropOp, InDropZone, UnderlyingActor->GetLevel());
		}
		return FReply::Unhandled();
	}

	return Super::AcceptDrop(DragDropEvent, InDropZone);
}

void FAvaOutlinerActor::SetObject_Impl(UObject* InObject)
{
	Super::SetObject_Impl(InObject);
	Actor = Cast<AActor>(InObject);
}

#undef LOCTEXT_NAMESPACE
