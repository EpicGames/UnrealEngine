// Copyright Epic Games, Inc. All Rights Reserved.

#include "Item/AvaOutlinerComponent.h"
#include "AvaOutliner.h"
#include "BlueprintEditorSettings.h"
#include "Columns/Slate/SAvaOutlinerLabelComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SceneComponent.h"
#include "Item/AvaOutlinerItemProxy.h"
#include "Item/AvaOutlinerMaterialProxy.h"
#include "ItemActions/AvaOutlinerAddItem.h"
#include "ItemActions/AvaOutlinerRemoveItem.h"
#include "Kismet2/ComponentEditorUtils.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerComponent"

FAvaOutlinerComponent::FAvaOutlinerComponent(IAvaOutliner& InOutliner, USceneComponent* InComponent)
	: Super(InOutliner, InComponent)
	, Component(InComponent)
{
}

void FAvaOutlinerComponent::FindChildren(TArray<FAvaOutlinerItemPtr>& OutChildren, bool bRecursive)
{
	Super::FindChildren(OutChildren, bRecursive);
	
	const USceneComponent* const UnderlyingComponent = GetComponent();	
	
	if (UnderlyingComponent)
	{
		for (USceneComponent* const Comp : UnderlyingComponent->GetAttachChildren())
		{
			//Only add Children that are owned by our owner (i.e. Components of other actors should not be added in our hierarchy)
			if (IsValid(Comp) && Comp->GetOwner() == UnderlyingComponent->GetOwner())
			{
				const FAvaOutlinerItemPtr ChildItem = Outliner.FindOrAdd<FAvaOutlinerComponent>(Comp);
				check(ChildItem.IsValid());
				OutChildren.Add(ChildItem);
				if (bRecursive)
				{
					ChildItem->FindChildren(OutChildren, bRecursive);
				}
			}
		}
	}
}

void FAvaOutlinerComponent::GetItemProxies(TArray<TSharedPtr<FAvaOutlinerItemProxy>>& OutItemProxies)
{
	Super::GetItemProxies(OutItemProxies);
	if (UPrimitiveComponent* const PrimitiveComponent = Cast<UPrimitiveComponent>(GetComponent()))
	{
		if (TSharedPtr<FAvaOutlinerItemProxy> MaterialItemProxy = Outliner.GetOrCreateItemProxy<FAvaOutlinerMaterialProxy>(SharedThis(this)))
		{
			OutItemProxies.Add(MaterialItemProxy);
		}
	}
}

bool FAvaOutlinerComponent::CanAddChild(const FAvaOutlinerItemPtr& InChild) const
{
	// Super checks InChild.IsValid
	return Super::CanAddChild(InChild) && InChild->IsA<FAvaOutlinerComponent>();
}

void FAvaOutlinerComponent::OnChildrenAdded(TConstArrayView<FAvaOutlinerItemPtr> InAddedChildren, const FAvaOutlinerAddItemParams& InAddItemParams)
{
	Super::OnChildrenAdded(InAddedChildren, InAddItemParams);

	for (const FAvaOutlinerItemPtr& Item : InAddedChildren)
	{
		FAvaOutlinerComponent* ComponentItem = Item->CastTo<FAvaOutlinerComponent>();
		if (!ComponentItem)
		{
			continue;
		}

		if (USceneComponent* const ChildComp = ComponentItem->GetComponent())
		{
			USceneComponent* const UnderlyingComponent = GetComponent();
			USceneComponent* const AttachParent = ChildComp->GetAttachParent();

			//Set Component Mobility to Moveable.
			if (ChildComp && ChildComp->Mobility == EComponentMobility::Static
				&& AttachParent && AttachParent->Mobility != EComponentMobility::Static)
			{
				ChildComp->SetMobility(AttachParent->Mobility);
			}
				
			if (UnderlyingComponent && AttachParent != UnderlyingComponent)
			{
				if (AttachParent)
				{
					AttachParent->Modify();
				}
				ChildComp->AttachToComponent(UnderlyingComponent, FAttachmentTransformRules::KeepRelativeTransform, NAME_None);
			}
		}
	}
}

bool FAvaOutlinerComponent::RemoveChild(const FAvaOutlinerRemoveItemParams& InRemoveItemParams)
{
	if (InRemoveItemParams.Item.IsValid())
	{
		if (const FAvaOutlinerComponent* const ComponentItem = InRemoveItemParams.Item->CastTo<FAvaOutlinerComponent>())
		{
			USceneComponent* const UnderlyingComponent = GetComponent();
			if (USceneComponent* const ChildComp = ComponentItem->GetComponent())
			{
				const USceneComponent* const AttachParent = ChildComp->GetAttachParent();
				if (UnderlyingComponent && AttachParent == UnderlyingComponent)
				{
					UnderlyingComponent->Modify();
					ChildComp->DetachFromComponent(InRemoveItemParams.DetachmentTransformRules.Get(FDetachmentTransformRules::KeepRelativeTransform));
				}
			}
		}
		return RemoveChildChecked(InRemoveItemParams.Item);
	}
	return true;
}

EAvaOutlinerItemViewMode FAvaOutlinerComponent::GetSupportedViewModes(const FAvaOutlinerView& InOutlinerView) const
{
	// Components should only be visualized in Outliner View and not appear in the Item Column List
	// Support any other type of View Mode
	return EAvaOutlinerItemViewMode::ItemTree | ~EAvaOutlinerItemViewMode::HorizontalItemList;
}

bool FAvaOutlinerComponent::IsAllowedInOutliner() const
{
	FAvaOutliner& OutlinerPrivate = static_cast<FAvaOutliner&>(Outliner);

	//Make sure Owner is Allowed (this also returns false if Owner is invalid)
	const USceneComponent* const UnderlyingComponent = GetComponent();
	if (!UnderlyingComponent || !OutlinerPrivate.IsActorAllowedInOutliner(UnderlyingComponent->GetOwner()))
	{
		return false;
	}
	return OutlinerPrivate.IsComponentAllowedInOutliner(UnderlyingComponent);
}

TSharedRef<SWidget> FAvaOutlinerComponent::GenerateLabelWidget(const TSharedRef<SAvaOutlinerTreeRow>& InRow)
{
	return SNew(SAvaOutlinerLabelComponent, SharedThis(this), InRow);
}

bool FAvaOutlinerComponent::GetVisibility(EAvaOutlinerVisibilityType VisibilityType) const
{
	const USceneComponent* const UnderlyingComponent = GetComponent();
	if (UnderlyingComponent)
	{
		switch (VisibilityType)
		{
			case EAvaOutlinerVisibilityType::Editor:
				return UnderlyingComponent->IsVisibleInEditor();

			case EAvaOutlinerVisibilityType::Runtime:
				return !UnderlyingComponent->bHiddenInGame;

			default:
				break;
		}
	}
	return false;
}

void FAvaOutlinerComponent::OnVisibilityChanged(EAvaOutlinerVisibilityType VisibilityType, bool bNewVisibility)
{
	USceneComponent* const UnderlyingComponent = GetComponent();
	if (UnderlyingComponent)
	{
		switch (VisibilityType)
		{
			case EAvaOutlinerVisibilityType::Editor:
				UnderlyingComponent->SetVisibility(bNewVisibility);
				break;

			case EAvaOutlinerVisibilityType::Runtime:
				UnderlyingComponent->SetHiddenInGame(!bNewVisibility);
				break;

			default: break;
		}
	}
}

FLinearColor FAvaOutlinerComponent::GetItemColor() const
{
	return FLinearColor(0.5f, 0.5f, 0.5f, 1.f);
}

TArray<FName> FAvaOutlinerComponent::GetTags() const
{
	if (USceneComponent* const UnderlyingComponent = GetComponent())
	{
		return UnderlyingComponent->ComponentTags;
	}
	return Super::GetTags();
}

void FAvaOutlinerComponent::SetObject_Impl(UObject* InObject)
{
	Super::SetObject_Impl(InObject);
	Component = Cast<USceneComponent>(InObject);
}

#undef LOCTEXT_NAMESPACE
