// Copyright Epic Games, Inc. All Rights Reserved.

#include "Item/AvaOutlinerComponentProxy.h"
#include "AvaOutliner.h"
#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "Item/AvaOutlinerActor.h"
#include "Item/AvaOutlinerComponent.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerComponentProxy"

FAvaOutlinerComponentProxy::FAvaOutlinerComponentProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem)
	: Super(InOutliner, InParentItem)
{
}

FText FAvaOutlinerComponentProxy::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Components");
}

FSlateIcon FAvaOutlinerComponentProxy::GetIcon() const
{
	return FSlateIconFinder::FindIconForClass(USceneComponent::StaticClass());
}

FText FAvaOutlinerComponentProxy::GetIconTooltipText() const
{
	return LOCTEXT("Tooltip", "Shows the Components in an Actor. Visualization, non-editable and UCS Components are excluded");
}

EAvaOutlinerItemViewMode FAvaOutlinerComponentProxy::GetSupportedViewModes(const FAvaOutlinerView& InOutlinerView) const
{
	// Components should only be visualized in Outliner View and not appear in the Item Column List
	// Support any other type of View Mode
	return EAvaOutlinerItemViewMode::ItemTree | ~EAvaOutlinerItemViewMode::HorizontalItemList;
}

void FAvaOutlinerComponentProxy::GetProxiedItems(const TSharedRef<IAvaOutlinerItem>& InParent, TArray<FAvaOutlinerItemPtr>& OutChildren, bool bInRecursive)
{
	if (const FAvaOutlinerActor* const ActorItem = InParent->CastTo<FAvaOutlinerActor>())
	{
		const AActor* const Actor = ActorItem->GetActor();

		if (!Actor)
		{
			return;
		}
		
		if (USceneComponent* const RootComponent = Actor->GetRootComponent())
		{
			const FAvaOutlinerItemPtr RootItem = Outliner.FindOrAdd<FAvaOutlinerComponent>(RootComponent);
			RootItem->SetParent(SharedThis(this));

			OutChildren.Add(RootItem);

			if (bInRecursive)
			{
				RootItem->FindChildren(OutChildren, bInRecursive);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
