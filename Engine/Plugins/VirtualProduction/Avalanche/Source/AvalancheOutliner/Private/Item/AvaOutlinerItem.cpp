// Copyright Epic Games, Inc. All Rights Reserved.

#include "Item/AvaOutlinerItem.h"
#include "ActorFactories/ActorFactory.h"
#include "AssetSelection.h"
#include "AvaOutliner.h"
#include "AvaOutlinerModule.h"
#include "AvaOutlinerView.h"
#include "AvaSceneTree.h"
#include "Columns/Slate/SAvaOutlinerLabelItem.h"
#include "Data/AvaOutlinerSaveState.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "DragDropOps/AvaOutlinerItemDragDropOp.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "IAvaOutlinerProvider.h"
#include "Item/AvaOutlinerActor.h"
#include "Item/AvaOutlinerItemUtils.h"
#include "ItemActions/AvaOutlinerAddItem.h"
#include "ItemActions/AvaOutlinerRemoveItem.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerItem"

namespace UE::AvaOutliner::Private
{
	EDetachmentRule GetDetachmentRuleFromAttachmentRule(EAttachmentRule AttachmentRule)
	{
		switch (AttachmentRule)
		{
			case EAttachmentRule::KeepRelative:
				return EDetachmentRule::KeepRelative;

			default:
				return EDetachmentRule::KeepWorld;
		}
	};

	FDetachmentTransformRules GetDetachmentRulesFromAttachmentRules(const TOptional<FAttachmentTransformRules>& AttachmentTransformRules)
	{
		if (AttachmentTransformRules.IsSet())
		{
			return FDetachmentTransformRules(
				GetDetachmentRuleFromAttachmentRule(AttachmentTransformRules->LocationRule),
				GetDetachmentRuleFromAttachmentRule(AttachmentTransformRules->RotationRule),
				GetDetachmentRuleFromAttachmentRule(AttachmentTransformRules->ScaleRule),
				true
			);
		}

		return FDetachmentTransformRules::KeepWorldTransform;
	};
}

FAvaOutlinerItem::FAvaOutlinerItem(IAvaOutliner& InOutliner)
	: Outliner(InOutliner)
{
}

FAvaOutlinerItem::~FAvaOutlinerItem()
{
}

TSharedRef<IAvaOutliner> FAvaOutlinerItem::GetOwnerOutliner() const
{
	return Outliner.AsShared();
}

bool FAvaOutlinerItem::IsItemValid() const
{
	return true;
}

void FAvaOutlinerItem::RefreshChildren()
{
	TArray<FAvaOutlinerItemPtr> FoundChildren;
	FindValidChildren(FoundChildren, /*bRecursiveFind=*/false);

	TArray<FAvaOutlinerItemPtr> Sortable;
	TArray<FAvaOutlinerItemPtr> Unsortable;
	UE::AvaOutliner::SplitItems(FoundChildren, Sortable, Unsortable);

	// Start with all Sortable/Unsortable Items, and remove every item seen by iterating Children
	TSet<FAvaOutlinerItemPtr> NewSortableChildren(Sortable);
	TSet<FAvaOutlinerItemPtr> NewUnsortableChildren(Unsortable);
	
	// Remove items from "Children" that were not present in the Sortable Found Children (we'll add non-sortable later)
	// Result is have Children only contain Items that existed previously
	for (TArray<FAvaOutlinerItemPtr>::TIterator ItemIter(Children); ItemIter; ++ItemIter)
	{
		FAvaOutlinerItemPtr Item(*ItemIter);
		if (!Item.IsValid() || NewUnsortableChildren.Contains(Item))
		{
			ItemIter.RemoveCurrent();
		}
		else if (!NewSortableChildren.Contains(Item) || !Item->IsItemValid())
		{
			Item->SetParent(nullptr);
			ItemIter.RemoveCurrent();
		}

		NewSortableChildren.Remove(Item);
		NewUnsortableChildren.Remove(Item);
	}

	// Find Children for New Children in case these new children have grand children
	// Note: This does not affect any of the current containers. It's just called for discovery
	auto FindGrandChildren = [](const TSet<FAvaOutlinerItemPtr>& InChildren)
		{
			for (const FAvaOutlinerItemPtr& Child : InChildren)
			{
				TArray<FAvaOutlinerItemPtr> GrandChildren;
				Child->FindValidChildren(GrandChildren, /*bRecursiveFind=*/true);
			}
		};

	FindGrandChildren(NewUnsortableChildren);
	FindGrandChildren(NewSortableChildren);

	// After removing Children not present in Sortable
	// Children should either be equal in size with Sortable (which means no new sortable children were added)
	// or Sortable has more entries which means there are new items to add
	if (Sortable.Num() > Children.Num())
	{
		check(!NewSortableChildren.IsEmpty());
		HandleNewSortableChildren(NewSortableChildren.Array());
	}

	// Rearrange so that Children are arranged like so:
	// [Unsortable Children][Sortable Children]
	Unsortable.Append(MoveTemp(Children));
	Children = MoveTemp(Unsortable);

	// Update the Parents of every Child in the List
	const FAvaOutlinerItemPtr This = SharedThis(this);
	for (const FAvaOutlinerItemPtr& Child : Children)
	{
		Child->SetParent(This);
	}
}

void FAvaOutlinerItem::FindChildren(TArray<FAvaOutlinerItemPtr>& OutChildren, bool bRecursive)
{
	FAvaOutliner& OutlinerPrivate = static_cast<FAvaOutliner&>(Outliner);

	TArray<TSharedPtr<FAvaOutlinerItemProxy>> ItemProxies;
	OutlinerPrivate.GetItemProxiesForItem(SharedThis(this), ItemProxies);
	OutChildren.Reserve(OutChildren.Num() + ItemProxies.Num());

	for (const TSharedPtr<FAvaOutlinerItemProxy>& ItemProxy : ItemProxies)
	{
		OutChildren.Add(ItemProxy);
		if (bRecursive)
		{
			ItemProxy->FindChildren(OutChildren, bRecursive);
		}
	}
}

bool FAvaOutlinerItem::CanAddChild(const FAvaOutlinerItemPtr& InChild) const
{
	return InChild.IsValid();
}

TArray<FAvaOutlinerItemPtr> FAvaOutlinerItem::AddChildren(const FAvaOutlinerAddItemParams& InAddItemParams)
{
	return AddChildrenInternal(InAddItemParams);
}

bool FAvaOutlinerItem::RemoveChild(const FAvaOutlinerRemoveItemParams& InRemoveItemParams)
{
	if (InRemoveItemParams.Item.IsValid())
	{
		return RemoveChildChecked(InRemoveItemParams);
	}
	return false;
}

void FAvaOutlinerItem::SetParent(FAvaOutlinerItemPtr InParent)
{
	//check that one of the parent's children is this
	ParentWeak = InParent;
}

EAvaOutlinerItemViewMode FAvaOutlinerItem::GetSupportedViewModes(const FAvaOutlinerView& InOutlinerView) const
{
	return InOutlinerView.GetItemDefaultViewMode();
}

FAvaOutlinerItemId FAvaOutlinerItem::GetItemId() const
{
	if (ItemId.IsValid())
	{
		return ItemId;
	}
	const_cast<FAvaOutlinerItem*>(this)->RecalculateItemId();
	return ItemId;
}

const FSlateBrush* FAvaOutlinerItem::GetIconBrush() const
{
	const FSlateIcon Icon = FAvaOutlinerModule::Get().FindOverrideIcon(SharedThis(this));
	if (Icon.IsSet())
	{
		return Icon.GetIcon();
	}

	return GetIcon().GetIcon();
}

TSharedRef<SWidget> FAvaOutlinerItem::GenerateLabelWidget(const TSharedRef<SAvaOutlinerTreeRow>& InRow)
{
	return SNew(SAvaOutlinerLabelItem, SharedThis(this), InRow);
}

bool FAvaOutlinerItem::Rename(const FString& InName)
{
	Outliner.GetProvider().NotifyOutlinerItemRenamed(SharedThis(this));
	return false;
}

bool FAvaOutlinerItem::Delete()
{
	Outliner.GetProvider().NotifyOutlinerItemDeleted(SharedThis(this));
	return false;
}

void FAvaOutlinerItem::SetLocked(bool bInIsLocked)
{
	Outliner.GetProvider().NotifyOutlinerItemLockChanged(SharedThis(this));
}

void FAvaOutlinerItem::AddFlags(EAvaOutlinerItemFlags Flags)
{
	EnumAddFlags(ItemFlags, Flags);
}

void FAvaOutlinerItem::RemoveFlags(EAvaOutlinerItemFlags Flags)
{
	EnumRemoveFlags(ItemFlags, Flags);
}

bool FAvaOutlinerItem::HasAnyFlags(EAvaOutlinerItemFlags Flags) const
{
	return EnumHasAnyFlags(ItemFlags, Flags);
}

bool FAvaOutlinerItem::HasAllFlags(EAvaOutlinerItemFlags Flags) const
{
	return EnumHasAllFlags(ItemFlags, Flags);
}

TOptional<FAvaOutlinerColorPair> FAvaOutlinerItem::GetColor(bool bRecurse) const
{
	FAvaOutliner& OutlinerPrivate = static_cast<FAvaOutliner&>(Outliner);
	return OutlinerPrivate.FindItemColor(SharedThis(const_cast<FAvaOutlinerItem*>(this)), bRecurse);
}

TOptional<EItemDropZone> FAvaOutlinerItem::CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone)
{
	if (const TSharedPtr<FAvaOutlinerItemDragDropOp> ItemDragDropOp = InDragDropEvent.GetOperationAs<FAvaOutlinerItemDragDropOp>())
	{
		const TOptional<EItemDropZone> DropZone = ItemDragDropOp->CanDrop(InDropZone, SharedThis(this));
		if (DropZone.IsSet())
		{
			return *DropZone;
		}
	}
	return Outliner.GetProvider().OnOutlinerItemCanAcceptDrop(InDragDropEvent, InDropZone, SharedThis(this));
}

FReply FAvaOutlinerItem::AcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone)
{
	if (const TSharedPtr<FAvaOutlinerItemDragDropOp> ItemDragDropOp = InDragDropEvent.GetOperationAs<FAvaOutlinerItemDragDropOp>())
	{
		const FReply Reply = ItemDragDropOp->Drop(InDropZone, SharedThis(this));
		if (Reply.IsEventHandled())
		{
			return Reply;
		}
	}
	return Outliner.GetProvider().OnOutlinerItemAcceptDrop(InDragDropEvent, InDropZone, SharedThis(this));
}

FLinearColor FAvaOutlinerItem::GetItemColor() const
{
	return FLinearColor::White;
}

void FAvaOutlinerItem::RecalculateItemId()
{
	const FAvaOutlinerItemId OldItemId = ItemId;
	ItemId = CalculateItemId();

	FAvaOutliner& OutlinerPrivate = static_cast<FAvaOutliner&>(Outliner);
	OutlinerPrivate.NotifyItemIdChanged(OldItemId, SharedThis(this));
}

TArray<FAvaOutlinerItemPtr> FAvaOutlinerItem::AddChildrenInternal(const FAvaOutlinerAddItemParams& InAddItemParams)
{
	// Remove all items that can't be added as children
	TArray<FAvaOutlinerItemPtr> ChildrenToAdd(InAddItemParams.Items);
	ChildrenToAdd.RemoveAll(
		[This=this](const FAvaOutlinerItemPtr& Item)
		{
			return !This->CanAddChild(Item);
		});

	for (const FAvaOutlinerItemPtr& Item : ChildrenToAdd)
	{
		if (const FAvaOutlinerItemPtr OldParent = Item->GetParent())
		{
			// if we are adding the child and the old parent is this,
			// then it means we're just rearranging, only remove from array
			if (OldParent.Get() == this)
			{
				Children.Remove(Item);
			}
			else
			{
				FAvaOutlinerRemoveItemParams RemoveParams(Item);
				RemoveParams.DetachmentTransformRules = UE::AvaOutliner::Private::GetDetachmentRulesFromAttachmentRules(InAddItemParams.AttachmentTransformRules);
				OldParent->RemoveChild(RemoveParams);
			}
		}
	}

	if (InAddItemParams.RelativeItem.IsValid() && InAddItemParams.RelativeDropZone.IsSet() && InAddItemParams.RelativeDropZone != EItemDropZone::OntoItem)
	{
		const int32 RelativeItemIndex = Children.Find(InAddItemParams.RelativeItem);
		if (RelativeItemIndex != INDEX_NONE)
		{
			const int32 TargetIndex = InAddItemParams.RelativeDropZone == EItemDropZone::BelowItem
				? RelativeItemIndex + 1
				: RelativeItemIndex;

			Children.Insert(ChildrenToAdd, TargetIndex);
		}
		else
		{
			Children.Append(ChildrenToAdd);
		}
	}
	else
	{
		Children.Append(ChildrenToAdd);
	}

	// Update Parents
	for (const FAvaOutlinerItemPtr& Item : ChildrenToAdd)
	{
		Item->SetParent(SharedThis(this));
	}

	OnChildrenAdded(ChildrenToAdd, InAddItemParams);
	return ChildrenToAdd;
}

bool FAvaOutlinerItem::RemoveChildChecked(const FAvaOutlinerRemoveItemParams& InRemoveItemParams)
{
	InRemoveItemParams.Item->SetParent(nullptr);
	return Children.Remove(InRemoveItemParams.Item) > 0;
}

void FAvaOutlinerItem::HandleNewSortableChildren(TArray<FAvaOutlinerItemPtr> InSortableChildren)
{
	FAvaSceneTree* const SceneTree = Outliner.GetProvider().GetSceneTree();

	if (SceneTree)
	{
		InSortableChildren.Sort([SceneTree](const FAvaOutlinerItemPtr& A, const FAvaOutlinerItemPtr& B)
		{
			const FAvaSceneTreeNode* const NodeA = SceneTree->FindTreeNode(FAvaOutliner::MakeSceneItemFromOutlinerItem(A));
			const FAvaSceneTreeNode* const NodeB = SceneTree->FindTreeNode(FAvaOutliner::MakeSceneItemFromOutlinerItem(B));
			return FAvaSceneTree::CompareTreeItemOrder(NodeA, NodeB);
		});
	}

	FAvaOutlinerAddItemParams AddItemParams;
	for (const FAvaOutlinerItemPtr& NewChild : InSortableChildren)
	{
		AddItemParams.Items = { NewChild };

		const FAvaSceneTreeNode* const TreeNode = SceneTree
			? SceneTree->FindTreeNode(FAvaOutliner::MakeSceneItemFromOutlinerItem(NewChild))
			: nullptr;

		if (TreeNode && Children.IsValidIndex(TreeNode->GetLocalIndex()))
		{
			// Add Before the Child at Index, so this Item is at the specific Index
			AddItemParams.RelativeItem = Children[TreeNode->GetLocalIndex()];
			AddItemParams.RelativeDropZone = EItemDropZone::AboveItem;
		}
		else
		{
			// Add After Last, so this Item is the last item in the List
			AddItemParams.RelativeItem = Children.IsEmpty() ? nullptr : Children.Last();
			AddItemParams.RelativeDropZone = EItemDropZone::BelowItem;
		}

		AddChildren(AddItemParams);
	}
}

FReply FAvaOutlinerItem::CreateItemsFromAssetDrop(const TSharedPtr<FAssetDragDropOp>& AssetDragDropOp
	, EItemDropZone DropZone
	, ULevel* Level)
{
	if (Level && AssetDragDropOp.IsValid())
	{
		// Force the Item Selection
		FAvaOutlinerAddItemParams AddChildParams;
		AddChildParams.RelativeItem     = SharedThis(this);
		AddChildParams.RelativeDropZone = DropZone;
		AddChildParams.Flags            = EAvaOutlinerAddItemFlags::Select | EAvaOutlinerAddItemFlags::AddChildren;

		bool bTransacting = false;
		int32 TransactionIndex = INDEX_NONE;

		const TArray<FAssetData>& Assets = AssetDragDropOp->GetAssets();

		// Iterate in Reverse so that the Items are added in the Correct Order
		for (int32 Index = Assets.Num() - 1; Index >= 0; --Index)
		{
			const FAssetData& Asset = Assets[Index];
			if (UActorFactory* ActorFactory = FActorFactoryAssetProxy::GetFactoryForAsset(Asset))
			{
				const AActor* const ActorTemplate = ActorFactory->GetDefaultActor(Asset);
				if (!ActorTemplate)
				{
					continue;
				}

				//Do once, prob can't use FScopedTransaction since we we need to make sure there's a valid Actor Factory 
				if (!bTransacting)
				{
					TransactionIndex = GEditor->BeginTransaction(LOCTEXT("OutlinerSpawnDroppedAsset", "Outliner Spawn Dropped Asset"));
					bTransacting = true;
					Level->Modify();
				}

				FActorSpawnParameters SpawnParams;
				SpawnParams.ObjectFlags = RF_Transactional;

				// Make the Outliner not automatically add the Actors we're adding here
				Outliner.SetIgnoreNotify(EAvaOutlinerIgnoreNotifyFlags::Spawn, true);

				if (AActor* const SpawnedActor = ActorFactory->CreateActor(Asset.GetAsset()
					, Level
					, Outliner.GetProvider().GetOutlinerDefaultActorSpawnTransform()
					, SpawnParams))
				{
					SpawnedActor->InvalidateLightingCache();
					SpawnedActor->PostEditChange();
					SpawnedActor->MarkPackageDirty();

					ULevel::LevelDirtiedEvent.Broadcast();

					AddChildParams.Items.Add(Outliner.FindOrAdd<FAvaOutlinerActor>(SpawnedActor));
					if (AddChildParams.Items.Num() > 1)
					{
						// When there is more than 1 item being added, start appending to Selection Instead
						AddChildParams.SelectionFlags |= EAvaOutlinerItemSelectionFlags::AppendToCurrentSelection;
					}
				}

				Outliner.SetIgnoreNotify(EAvaOutlinerIgnoreNotifyFlags::Spawn, false);
			}
		}

		if (!AddChildParams.Items.IsEmpty())
		{
			FAvaOutliner& OutlinerPrivate = static_cast<FAvaOutliner&>(Outliner);
			OutlinerPrivate.EnqueueItemAction<FAvaOutlinerAddItem>(AddChildParams);
		}

		if (bTransacting)
		{
			// Likely: Items have been added if Transaction Started is true
			if (!AddChildParams.Items.IsEmpty())
			{
				// Refresh here so that there's no delay between Spawn and Acknowledgement
				Outliner.Refresh();
				GEditor->EndTransaction();
			}
			// Unlikely: World failed to Spawn Actors, Cancel Transaction
			else if (TransactionIndex != INDEX_NONE)
			{
				GEditor->CancelTransaction(TransactionIndex);
			}

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
