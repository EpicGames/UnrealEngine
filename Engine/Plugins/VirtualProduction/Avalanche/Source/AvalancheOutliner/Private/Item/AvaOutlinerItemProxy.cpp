// Copyright Epic Games, Inc. All Rights Reserved.

#include "Item/AvaOutlinerItemProxy.h"
#include "AvaOutliner.h"
#include "AvaOutlinerView.h"
#include "Input/Reply.h"

FAvaOutlinerItemProxy::FAvaOutlinerItemProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem)
	: FAvaOutlinerItem(InOutliner)
{
	ParentWeak = InParentItem;
}

bool FAvaOutlinerItemProxy::IsItemValid() const
{
	return ParentWeak.IsValid() && Outliner.FindItem(ParentWeak.Pin()->GetItemId()).IsValid();
}

void FAvaOutlinerItemProxy::FindChildren(TArray<FAvaOutlinerItemPtr>& OutChildren, bool bRecursive)
{
	FAvaOutlinerItemPtr Parent = GetParent();
	if (!Parent.IsValid() || !Parent->IsAllowedInOutliner())
	{
		return;
	}
	
	Super::FindChildren(OutChildren, bRecursive);
	
	GetProxiedItems(Parent.ToSharedRef(), OutChildren, bRecursive);
}

void FAvaOutlinerItemProxy::SetParent(FAvaOutlinerItemPtr InParent)
{
	Super::SetParent(InParent);

	// Recalculate our Item Id because we rely on what our parent is for our id
	RecalculateItemId();
}

EAvaOutlinerItemViewMode FAvaOutlinerItemProxy::GetSupportedViewModes(const FAvaOutlinerView& InOutlinerView) const
{
	// Hide Proxies if it has no Children
	if (Children.IsEmpty())
	{
		return EAvaOutlinerItemViewMode::None;
	}
	return InOutlinerView.GetItemProxyViewMode();
}

bool FAvaOutlinerItemProxy::CanAutoExpand() const
{
	return false;
}

FAvaOutlinerItemId FAvaOutlinerItemProxy::CalculateItemId() const
{
	if (ParentWeak.IsValid())
	{
		return FAvaOutlinerItemId(ParentWeak.Pin(), *this);	
	}
	return FAvaOutlinerItemId();
}
