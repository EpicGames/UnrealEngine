// Copyright Epic Games, Inc. All Rights Reserved.

#include "Item/AvaOutlinerItemUtils.h"
#include "AvaOutliner.h"
#include "Containers/Array.h"
#include "Item/IAvaOutlinerItem.h"

bool UE::AvaOutliner::CompareOutlinerItemOrder(const FAvaOutlinerItemPtr& A, const FAvaOutlinerItemPtr& B)
{
	if (!A.IsValid())
	{
		return false;
	}
	if (!B.IsValid())
	{
		return false;
	}
	if (const FAvaOutlinerItemPtr LowestCommonAncestor = FAvaOutliner::FindLowestCommonAncestor({A, B}))
	{
		const TArray<FAvaOutlinerItemPtr> PathToA = LowestCommonAncestor->FindPath({A});
		const TArray<FAvaOutlinerItemPtr> PathToB = LowestCommonAncestor->FindPath({B});

		int32 Index = 0;
		
		int32 PathAIndex = -1;
		int32 PathBIndex = -1;

		while (PathAIndex == PathBIndex)
		{
			if (!PathToA.IsValidIndex(Index))
			{
				return true;
			}
			if (!PathToB.IsValidIndex(Index))
			{
				return false;
			}

			PathAIndex = LowestCommonAncestor->GetChildIndex(PathToA[Index]);
			PathBIndex = LowestCommonAncestor->GetChildIndex(PathToB[Index]);
			++Index;
		}
		return PathAIndex < PathBIndex;
	}
	return false;
}

void UE::AvaOutliner::SplitItems(const TArray<FAvaOutlinerItemPtr>& InItems
	, TArray<FAvaOutlinerItemPtr>& OutSortable
	, TArray<FAvaOutlinerItemPtr>& OutUnsortable)
{
	// Allocate both for worst case scenarios
	OutSortable.Reserve(InItems.Num());
	OutUnsortable.Reserve(InItems.Num());

	for (const FAvaOutlinerItemPtr& Item : InItems)
	{
		if (!Item.IsValid())
		{
			continue;
		}

		if (Item->IsSortable())
		{
			OutSortable.Add(Item);
		}
		else
		{
			OutUnsortable.Add(Item);
		}
	}
}
