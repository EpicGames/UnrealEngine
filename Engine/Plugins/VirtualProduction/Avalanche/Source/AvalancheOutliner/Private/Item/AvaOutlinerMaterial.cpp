// Copyright Epic Games, Inc. All Rights Reserved.

#include "Item/AvaOutlinerMaterial.h"
#include "Components/PrimitiveComponent.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Item/AvaOutlinerComponent.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "ScopedTransaction.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerMaterial"

FAvaOutlinerMaterial::FAvaOutlinerMaterial(IAvaOutliner& InOutliner
		, UMaterialInterface* InMaterial
		, const FAvaOutlinerItemPtr& InReferencingItem
		, int32 InMaterialIndex)
	: Super(InOutliner, InMaterial, InReferencingItem, TEXT("[Slot ") + FString::FromInt(InMaterialIndex) + TEXT("]"))
	, Material(InMaterial)
	, MaterialIndex(InMaterialIndex)
{
	OnMaterialChanged();
}

TOptional<EItemDropZone> FAvaOutlinerMaterial::CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone)
{
	// When the Referencing Item is a Primitive Component
	// only process if we are dragging a single material asset into it
	if (UPrimitiveComponent* const ReferencingPrimitive = GetReferencingPrimitive())
	{
		if (const TSharedPtr<FAssetDragDropOp> AssetDragDropOp = InDragDropEvent.GetOperationAs<FAssetDragDropOp>())
		{ 
			if (AssetDragDropOp->GetAssets().Num() == 1)
			{
				UClass* const AssetClass = AssetDragDropOp->GetAssets()[0].GetClass();
				if (AssetClass && AssetClass->IsChildOf(UMaterialInterface::StaticClass()))
				{
					// Only supports Onto (above or below doesn't make sense for items as they're not hierarchical)
					return EItemDropZone::OntoItem;
				}
			}
		}
	}
	return Super::CanAcceptDrop(InDragDropEvent, InDropZone);
}

FReply FAvaOutlinerMaterial::AcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone)
{
	if (UPrimitiveComponent* const ReferencingPrimitive = GetReferencingPrimitive())
	{
		UMaterialInterface* DraggedMaterial = nullptr;
		
		// Asset being dragged is a single material
		if (const TSharedPtr<FAssetDragDropOp> AssetDragDropOp = InDragDropEvent.GetOperationAs<FAssetDragDropOp>())
		{
			if (AssetDragDropOp->GetAssets().Num() == 1)
			{
				DraggedMaterial = Cast<UMaterialInterface>(AssetDragDropOp->GetAssets()[0].GetAsset());
			}
		}
		
		if (DraggedMaterial)
		{
			const UMaterialInterface* const CurrentMaterial = ReferencingPrimitive->GetMaterial(MaterialIndex);
			
			// Only set if they differ. Don't want to Modify or Mark Render State Dirty if nothing changes 
			if (CurrentMaterial != DraggedMaterial)
			{
				FScopedTransaction Transaction(LOCTEXT("SetMaterial", "Set Material"));
				
				ReferencingPrimitive->Modify();
				ReferencingPrimitive->SetMaterial(MaterialIndex, DraggedMaterial);
				
				// Base Dynamic Mesh Component does not mark render state dirty in its SetMaterial Implementation
				ReferencingPrimitive->MarkRenderStateDirty();
				
				SetObject(DraggedMaterial);
				return FReply::Handled();
			}
		}
	}
	
	return Super::AcceptDrop(InDragDropEvent, InDropZone);
}

FSlateIcon FAvaOutlinerMaterial::GetIcon() const
{
	return MaterialIcon;
}

UPrimitiveComponent* FAvaOutlinerMaterial::GetReferencingPrimitive() const
{
	const FAvaOutlinerItemPtr ReferencingItem = GetReferencingItem();
	if (!ReferencingItem.IsValid())
	{
		return nullptr;
	}
	
	const FAvaOutlinerComponent* const ComponentItem = ReferencingItem->CastTo<FAvaOutlinerComponent>();
	if (!ComponentItem)
	{
		return nullptr;
	}
	
	return Cast<UPrimitiveComponent>(ComponentItem->GetComponent());
}

void FAvaOutlinerMaterial::OnMaterialChanged()
{
	static const FSlateIcon ObjectIcon = FSlateIconFinder::FindIconForClass(UObject::StaticClass());
	MaterialIcon = Super::GetIcon();

	if (MaterialIcon == ObjectIcon)
	{
		MaterialIcon = FSlateIconFinder::FindIconForClass(UMaterial::StaticClass());
	}
}

void FAvaOutlinerMaterial::SetObject_Impl(UObject* InObject)
{
	Super::SetObject_Impl(InObject);
	Material = Cast<UMaterialInterface>(InObject);
	OnMaterialChanged();
}

#undef LOCTEXT_NAMESPACE
