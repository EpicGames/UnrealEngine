// Copyright Epic Games, Inc. All Rights Reserved.

#include "Item/AvaOutlinerMaterialProxy.h"
#include "AvaOutliner.h"
#include "Components/PrimitiveComponent.h"
#include "Item/AvaOutlinerComponent.h"
#include "Item/AvaOutlinerMaterial.h"
#include "Materials/Material.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerMaterialProxy"

FAvaOutlinerMaterialProxy::FAvaOutlinerMaterialProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem)
	: FAvaOutlinerItemProxy(InOutliner, InParentItem)
{
}

FAvaOutlinerMaterialProxy::~FAvaOutlinerMaterialProxy()
{
	UnbindDelegates();
}

void FAvaOutlinerMaterialProxy::BindDelegates()
{
	UnbindDelegates();
	OnObjectPropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this
		, &FAvaOutlinerMaterialProxy::OnObjectPropertyChanged);
}

void FAvaOutlinerMaterialProxy::UnbindDelegates()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnObjectPropertyChangedHandle);
	OnObjectPropertyChangedHandle.Reset();
}

void FAvaOutlinerMaterialProxy::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	// Since we only deal with Components, if the object is not a Primitive Component do early return
	UPrimitiveComponent* const ChangedPrimitive = Cast<UPrimitiveComponent>(InObject);
	if (!ChangedPrimitive)
	{
		return;
	}
	
	const FAvaOutlinerItemPtr Parent = GetParent();
	if (!Parent.IsValid())
	{
		return;
	}
	
	const FAvaOutlinerComponent* const ComponentItem = Parent->CastTo<FAvaOutlinerComponent>();
	if (ComponentItem && ComponentItem->GetComponent() == ChangedPrimitive)
	{
		RefreshChildren();
	}
}

void FAvaOutlinerMaterialProxy::OnItemRegistered()
{
	Super::OnItemRegistered();
	BindDelegates();
}

void FAvaOutlinerMaterialProxy::OnItemUnregistered()
{
	Super::OnItemUnregistered();
	UnbindDelegates();
}

FText FAvaOutlinerMaterialProxy::GetDisplayName() const
{
	return LOCTEXT("MaterialProxy_Name", "Materials");
}

FSlateIcon FAvaOutlinerMaterialProxy::GetIcon() const
{
	return FSlateIconFinder::FindIconForClass(UMaterial::StaticClass());
}

FText FAvaOutlinerMaterialProxy::GetIconTooltipText() const
{
	return LOCTEXT("Tooltip", "Shows all the Materials used in a Primitive Component");
}

void FAvaOutlinerMaterialProxy::GetProxiedItems(const TSharedRef<IAvaOutlinerItem>& InParent, TArray<FAvaOutlinerItemPtr>& OutChildren, bool bInRecursive)
{
	if (const FAvaOutlinerComponent* const ComponentItem = InParent->CastTo<FAvaOutlinerComponent>())
	{
		if (UPrimitiveComponent* const PrimitiveComponent = Cast<UPrimitiveComponent>(ComponentItem->GetComponent()))
		{
			TArray<UMaterialInterface*> Materials;
			PrimitiveComponent->GetUsedMaterials(Materials);

			for (int32 Index = 0; Index < Materials.Num(); ++Index)
			{
				UMaterialInterface* const Material = Materials[Index];

				// Note that we pass in the parent of this proxy rather than the proxy itself because
				// the parent (component item) is what's referencing this material
				// Also, the Material Item already has a way to get the proxy (via GetParent())
				FAvaOutlinerItemPtr MaterialItem = Outliner.FindOrAdd<FAvaOutlinerMaterial>(Material, InParent, Index);

				if (MaterialItem->IsExactlyA<FAvaOutlinerMaterial>())
				{
					MaterialItem->SetParent(SharedThis(this));
					OutChildren.Add(MaterialItem);
				}

				if (bInRecursive)
				{
					MaterialItem->FindChildren(OutChildren, bInRecursive);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
