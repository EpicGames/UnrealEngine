// Copyright Epic Games, Inc. All Rights Reserved.

#include "Item/AvaOutlinerObject.h"
#include "AvaOutliner.h"
#include "Item/AvaOutlinerItemId.h"
#include "Selection/AvaOutlinerScopedSelection.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerObject"

FAvaOutlinerObject::FAvaOutlinerObject(IAvaOutliner& InOutliner, UObject* InObject)
	: FAvaOutlinerItem(InOutliner)
	, Object(InObject)
{
}

bool FAvaOutlinerObject::IsItemValid() const
{
	return FAvaOutlinerItem::IsItemValid() && IsValid(Object.Get());
}

void FAvaOutlinerObject::Select(FAvaOutlinerScopedSelection& InSelection) const
{
	InSelection.Select(GetObject());
}

bool FAvaOutlinerObject::IsSelected(const FAvaOutlinerScopedSelection& InSelection) const
{
	return InSelection.IsSelected(GetObject());
}

FText FAvaOutlinerObject::GetDisplayName() const
{
	if (const UObject* const UnderlyingObject = GetObject())
	{
		return FText::FromString(UnderlyingObject->GetName());
	}
	return FText::GetEmpty();
}

FText FAvaOutlinerObject::GetClassName() const
{
	if (const UObject* const UnderlyingObject = GetObject())
	{
		return UnderlyingObject->GetClass()->GetDisplayNameText();
	}
	return FText::GetEmpty();
}

FSlateIcon FAvaOutlinerObject::GetIcon() const
{
	if (const UObject* const UnderlyingObject = GetObject())
	{
		return FSlateIconFinder::FindIconForClass(UnderlyingObject->GetClass());
	}
	return FSlateIcon();
}

FText FAvaOutlinerObject::GetIconTooltipText() const
{
	if (const UObject* const UnderlyingObject = GetObject())
	{
		return UnderlyingObject->GetClass()->GetDisplayNameText();
	}
	return FText::GetEmpty();
}

bool FAvaOutlinerObject::IsAllowedInOutliner() const
{
	UObject* UnderlyingObject = GetObject();
	return !!UnderlyingObject;
}

void FAvaOutlinerObject::OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, bool bRecursive)
{
	//Get the Object even if it's Pending Kill (most likely it is)
	const UObject* ObjectPendingKill = Object.Get(true);
	if (ObjectPendingKill && InReplacementMap.Contains(ObjectPendingKill))
	{
		SetObject(InReplacementMap[ObjectPendingKill]);
	}

	//This handles calling OnObjectsReplaced for every child item
	Super::OnObjectsReplaced(InReplacementMap, bRecursive);
}

bool FAvaOutlinerObject::Rename(const FString& InName)
{
	return Super::Rename(InName);
}

void FAvaOutlinerObject::SetObject(UObject* InObject)
{
	SetObject_Impl(InObject);
	RecalculateItemId();
}

FAvaOutlinerItemId FAvaOutlinerObject::CalculateItemId() const
{
	return FAvaOutlinerItemId(GetObject());
}

void FAvaOutlinerObject::SetObject_Impl(UObject* InObject)
{
	Object = InObject;
}

#undef LOCTEXT_NAMESPACE
