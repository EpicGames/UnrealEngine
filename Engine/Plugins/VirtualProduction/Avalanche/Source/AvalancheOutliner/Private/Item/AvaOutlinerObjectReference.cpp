// Copyright Epic Games, Inc. All Rights Reserved.

#include "Item/AvaOutlinerObjectReference.h"
#include "Algo/Transform.h"
#include "AvaOutliner.h"
#include "Item/AvaOutlinerItemId.h"
#include "ItemActions/AvaOutlinerRemoveItem.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerObjectReference"

FAvaOutlinerSharedObject::FAvaOutlinerSharedObject(IAvaOutliner& InOutliner, UObject* InObject)
	: Super(InOutliner, InObject)
{
}

void FAvaOutlinerSharedObject::AddReference(const TSharedRef<FAvaOutlinerObjectReference>& InObjectReference)
{
	ObjectReferences.Add(InObjectReference);
}

void FAvaOutlinerSharedObject::RemoveReference(const TSharedRef<FAvaOutlinerObjectReference>& InObjectReference)
{
	ObjectReferences.Remove(InObjectReference);
	ObjectReferences.RemoveAll([](const TWeakPtr<FAvaOutlinerObjectReference>& InItemWeak){ return !InItemWeak.IsValid(); });
	
	if (ObjectReferences.IsEmpty())
	{
		FAvaOutlinerRemoveItemParams Params;
		Params.Item = SharedThis(this);
		Outliner.UnregisterItem(GetItemId());
	}
}

TArray<FAvaOutlinerItemPtr> FAvaOutlinerSharedObject::GetObjectReferences() const
{
	TArray<FAvaOutlinerItemPtr> OutObjectReferences;
	
	Algo::TransformIf(ObjectReferences, OutObjectReferences
		, [](const TWeakPtr<FAvaOutlinerObjectReference>& InItem) { return InItem.IsValid(); }
		, [](const TWeakPtr<FAvaOutlinerObjectReference>& InItem) { return InItem.Pin(); });
	
	return OutObjectReferences;
}

FAvaOutlinerObjectReference::FAvaOutlinerObjectReference(IAvaOutliner& InOutliner
		, UObject* InObject
		, const FAvaOutlinerItemPtr& InReferencingItem
		, const FString& InReferenceId)
	: Super(InOutliner, InObject)
	, ReferencingItemWeak(InReferencingItem)
	, ReferenceId(InReferenceId)
{
}

void FAvaOutlinerObjectReference::OnItemRegistered()
{
	FAvaOutlinerItemPtr SharedObject = Outliner.FindOrAdd<FAvaOutlinerSharedObject>(GetObject());
	if (FAvaOutlinerSharedObject* const SharedObjectItem = SharedObject->CastTo<FAvaOutlinerSharedObject>())
	{
		SharedObjectItem->AddReference(SharedThis(this));
	}
}

void FAvaOutlinerObjectReference::OnItemUnregistered()
{
	FAvaOutlinerItemPtr SharedObject = Outliner.FindOrAdd<FAvaOutlinerSharedObject>(GetObject());
	if (FAvaOutlinerSharedObject* const SharedObjectItem = SharedObject->CastTo<FAvaOutlinerSharedObject>())
	{
		SharedObjectItem->RemoveReference(SharedThis(this));
	}
}

bool FAvaOutlinerObjectReference::IsItemValid() const
{
	return FAvaOutlinerObject::IsItemValid()
		&& ReferencingItemWeak.IsValid()
		&& ReferencingItemWeak.Pin()->IsItemValid();
}

FText FAvaOutlinerObjectReference::GetDisplayName() const
{
	return FText::Format(LOCTEXT("DisplayName", "{0} {1}")
		, FText::FromString(ReferenceId)
		, FAvaOutlinerObject::GetDisplayName());
}

FAvaOutlinerItemId FAvaOutlinerObjectReference::CalculateItemId() const
{
	if (ReferencingItemWeak.IsValid())
	{
		return FAvaOutlinerItemId(GetObject(), ReferencingItemWeak.Pin(), ReferenceId);	
	}
	return FAvaOutlinerItemId();
}

#undef LOCTEXT_NAMESPACE
