// Copyright Epic Games, Inc. All Rights Reserved.

#include "Item/AvaOutlinerTreeRoot.h"
#include "ActorFactories/ActorFactory.h"
#include "AssetSelection.h"
#include "AvaOutliner.h"
#include "AvaOutlinerSubsystem.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "DragDropOps/AvaOutlinerItemDragDropOp.h"
#include "EngineUtils.h"
#include "Item/AvaOutlinerActor.h"
#include "Item/AvaOutlinerLevel.h"
#include "ItemActions/AvaOutlinerAddItem.h"
#include "Textures/SlateIcon.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/STableRow.h"

void FAvaOutlinerTreeRoot::FindChildren(TArray<FAvaOutlinerItemPtr>& OutChildren, bool bRecursive)
{
	Super::FindChildren(OutChildren, bRecursive);

	UWorld* const World = Outliner.GetWorld();
	if (!World)
	{
		return;
	}

	const TArray<ULevel*>& Levels = World->GetLevels();

	OutChildren.Reserve(OutChildren.Num() + Levels.Num());

	for (ULevel* Level : Levels)
	{
		if (!Level)
		{
			continue;
		}

		const FAvaOutlinerItemPtr LevelItem = Outliner.FindOrAdd<FAvaOutlinerLevel>(Level);

		const FAvaOutlinerItemFlagGuard Guard(LevelItem, EAvaOutlinerItemFlags::IgnorePendingKill);
		OutChildren.Add(LevelItem);
		if (bRecursive)
		{
			LevelItem->FindChildren(OutChildren, bRecursive);
		}
	}
}

bool FAvaOutlinerTreeRoot::CanAddChild(const FAvaOutlinerItemPtr& InChild) const
{
	return Super::CanAddChild(InChild) && InChild->CanBeTopLevel();
}

TArray<FAvaOutlinerItemPtr> FAvaOutlinerTreeRoot::AddChildren(const FAvaOutlinerAddItemParams& InAddItemParams)
{
	/** Holds whether an item is spawned (true) or rearranged (false) */
	TMap<FAvaOutlinerItemPtr, EAvaOutlinerHierarchyChangeType> HierarchyChangeTypeMap;
	HierarchyChangeTypeMap.Reserve(InAddItemParams.Items.Num());

	// Store whether the item is being spawned or rearranged
	for (const FAvaOutlinerItemPtr& Item : InAddItemParams.Items)
	{
		if (!Item.IsValid())
		{
			continue;
		}

		// if current parent is the root, it means it's just rearranging
		const bool bRearranging = Item->GetParent().Get() == this;

		// Is it a new actor that we just spawned
		const bool bSpawning = !Children.Contains(Item);

		if (bRearranging || bSpawning)
		{
			HierarchyChangeTypeMap.Add(Item, bSpawning ? EAvaOutlinerHierarchyChangeType::Attached : EAvaOutlinerHierarchyChangeType::Rearranged);
		}
	}

	const TArray<FAvaOutlinerItemPtr> AddedChildren = FAvaOutlinerItem::AddChildren(InAddItemParams);

	FAvaOutliner& OutlinerPrivate = static_cast<FAvaOutliner&>(Outliner);

	if (UAvaOutlinerSubsystem* const OutlinerSubsystem = OutlinerPrivate.GetOutlinerSubsystem())
	{
		for (const FAvaOutlinerItemPtr& AddedChild : AddedChildren)
		{
			if (const EAvaOutlinerHierarchyChangeType* HierarchyChangeType = HierarchyChangeTypeMap.Find(AddedChild))
			{
				if (const FAvaOutlinerActor* ActorItem = AddedChild->CastTo<FAvaOutlinerActor>())
				{
					if (AActor* const Actor = ActorItem->GetActor())
					{
						OutlinerPrivate.QueueActorHierarchyChange(Actor, /*ParentActor*/nullptr, *HierarchyChangeType);
					}
				}
			}
		}
	}

	return AddedChildren;
}

bool FAvaOutlinerTreeRoot::IsAllowedInOutliner() const
{
	checkNoEntry();
	return false;
}

FText FAvaOutlinerTreeRoot::GetDisplayName() const
{
	checkNoEntry();
	return FText::GetEmpty();
}

FText FAvaOutlinerTreeRoot::GetClassName() const
{
	checkNoEntry();
	return FText::GetEmpty();
}

FText FAvaOutlinerTreeRoot::GetIconTooltipText() const
{
	checkNoEntry();
	return FText::GetEmpty();
}

FSlateIcon FAvaOutlinerTreeRoot::GetIcon() const
{
	checkNoEntry();
	return FSlateIcon();
}

TSharedRef<SWidget> FAvaOutlinerTreeRoot::GenerateLabelWidget(const TSharedRef<SAvaOutlinerTreeRow>& InRow)
{
	checkNoEntry();
	return SNullWidget::NullWidget;
}

bool FAvaOutlinerTreeRoot::CanRename() const
{
	checkNoEntry();
	return false;
}

bool FAvaOutlinerTreeRoot::Rename(const FString& InName)
{
	checkNoEntry();
	return false;
}

TOptional<EItemDropZone> FAvaOutlinerTreeRoot::CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone)
{
	if (const TSharedPtr<FAssetDragDropOp> AssetDragDropOp = InDragDropEvent.GetOperationAs<FAssetDragDropOp>())
	{
		//Can't drop if a Single of the Assets is an Invalid Asset
		for (const FAssetData& Asset : AssetDragDropOp->GetAssets())
		{
			UActorFactory* ActorFactory = AssetDragDropOp->GetActorFactory();
			if (!ActorFactory)
			{
				ActorFactory = FActorFactoryAssetProxy::GetFactoryForAsset(Asset);
			}
			if (!ActorFactory || !ActorFactory->CanPlaceElementsFromAssetData(Asset))
			{
				return TOptional<EItemDropZone>();
			}
		}
		return InDropZone;
	}

	return Super::CanAcceptDrop(InDragDropEvent, InDropZone);
}

FReply FAvaOutlinerTreeRoot::AcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone)
{
	// Force Tree Root to always be Onto Item so we always create new Items as Children of Root
	const EItemDropZone ForcedDropZone = EItemDropZone::OntoItem;

	if (const TSharedPtr<FAssetDragDropOp> AssetDragDropOp = InDragDropEvent.GetOperationAs<FAssetDragDropOp>())
	{
		const UWorld* const World = Outliner.GetWorld();
		check(World);
		return CreateItemsFromAssetDrop(AssetDragDropOp, ForcedDropZone, World->GetCurrentLevel());
	}

	return Super::AcceptDrop(InDragDropEvent, ForcedDropZone);
}

FAvaOutlinerItemId FAvaOutlinerTreeRoot::CalculateItemId() const
{
	return FAvaOutlinerItemId(TEXT("OutlinerRoot"));
}
