// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Item/AvaOutlinerItem.h"

/*
 * The Item that ensures that every item (except self) has a parent in the hierarchy to make it easier
 * to handle. This is not really a visual item, so it can't appear in the Outliner View
 */
class FAvaOutlinerTreeRoot final : public FAvaOutlinerItem
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerTreeRoot, FAvaOutlinerItem)

	FAvaOutlinerTreeRoot(IAvaOutliner& InOutliner)
		: FAvaOutlinerItem(InOutliner)
	{
	}

	virtual ~FAvaOutlinerTreeRoot() override = default;

	//~ Begin IAvaOutlinerItem
	virtual void FindChildren(TArray<FAvaOutlinerItemPtr>& OutChildren, bool bRecursive) override;
	virtual bool CanAddChild(const FAvaOutlinerItemPtr& InChild) const override;
	virtual TArray<FAvaOutlinerItemPtr> AddChildren(const FAvaOutlinerAddItemParams& InAddItemParams) override;
	virtual bool IsAllowedInOutliner() const override;
	virtual FText GetDisplayName() const override;
	virtual FText GetClassName() const override;
	virtual FText GetIconTooltipText() const override;
	virtual FSlateIcon GetIcon() const override;
	virtual TSharedRef<SWidget> GenerateLabelWidget(const TSharedRef<SAvaOutlinerTreeRow>& InRow) override;
	virtual bool CanRename() const override;
	virtual bool Rename(const FString& InName) override;
	virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) override;	
	virtual FReply AcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) override;
	//~ End IAvaOutlinerItem
	
protected:
	//~ Begin FAvaOutlinerItem
	virtual FAvaOutlinerItemId CalculateItemId() const override;
	//~ End FAvaOutlinerItem
};
