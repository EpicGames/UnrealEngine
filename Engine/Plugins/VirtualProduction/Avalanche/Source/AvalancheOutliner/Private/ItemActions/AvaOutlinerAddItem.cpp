// Copyright Epic Games, Inc. All Rights Reserved.

#include "ItemActions/AvaOutlinerAddItem.h"
#include "Algo/AllOf.h"
#include "AvaOutliner.h"
#include "Item/AvaOutlinerObject.h"
#include "Item/AvaOutlinerTreeRoot.h"
#include "Item/IAvaOutlinerItem.h"
#include "Selection/AvaOutlinerScopedSelection.h"
#include "Widgets/Views/STableRow.h"

FAvaOutlinerAddItem::FAvaOutlinerAddItem(const FAvaOutlinerAddItemParams& InAddItemParams)
	: AddParams(InAddItemParams)
{
}

bool FAvaOutlinerAddItem::ShouldTransact() const
{
	return EnumHasAnyFlags(AddParams.Flags, EAvaOutlinerAddItemFlags::Transact);
}

void FAvaOutlinerAddItem::Execute(FAvaOutliner& InOutliner)
{
	// Pre-pass on all items
	for (const FAvaOutlinerItemPtr& ItemToAdd : AddParams.Items)
	{
		if (!ItemToAdd.IsValid())
		{
			continue;
		}

		//Try to Create Children on Find
		if (EnumHasAnyFlags(AddParams.Flags, EAvaOutlinerAddItemFlags::AddChildren))
		{
			constexpr bool bRecursiveFind = true;
			TArray<FAvaOutlinerItemPtr> Children;
			ItemToAdd->FindValidChildren(Children, bRecursiveFind);
		}

		const FAvaOutlinerItemPtr ParentItem = ItemToAdd->GetParent();

		//If this Array has elements in it, then we need to stop a Circular Dependency from forming
		TArray<FAvaOutlinerItemPtr> PathToRelativeItem = ItemToAdd->FindPath({AddParams.RelativeItem});
		if (!PathToRelativeItem.IsEmpty() && ParentItem.IsValid())
		{
			FAvaOutlinerAddItemParams CircularSolverParams;
			CircularSolverParams.Items                    = { PathToRelativeItem[0] };
			CircularSolverParams.RelativeItem             = ItemToAdd;
			CircularSolverParams.RelativeDropZone         = EItemDropZone::AboveItem;
			CircularSolverParams.Flags                    = AddParams.Flags;
			CircularSolverParams.AttachmentTransformRules = AddParams.AttachmentTransformRules;

			ParentItem->AddChildren(CircularSolverParams);
		}
	}

	// Item parent where the items will be added to. Defaults to tree root
	TSharedPtr<IAvaOutlinerItem> ItemParent = InOutliner.GetTreeRoot();

	if (AddParams.RelativeItem.IsValid())
	{
		const TSharedPtr<IAvaOutlinerItem> RelativeItemParent = AddParams.RelativeItem->GetParent();

		// If it's onto item, the Relative Item is going to be the Parent
		if (!AddParams.RelativeDropZone.IsSet() || AddParams.RelativeDropZone == EItemDropZone::OntoItem)
		{
			auto IsParentRelativeItem =
				[&RelativeItem = AddParams.RelativeItem](const FAvaOutlinerItemPtr& InItem)
				{
					return InItem.IsValid() && InItem->GetParent() == RelativeItem;
				};

			// If adding onto the Relative Item is and relative item is the Current Parent of ALL the dragged items,
			// detach item from item by adding these items below the parent
			if (RelativeItemParent && Algo::AllOf(AddParams.Items, IsParentRelativeItem))
			{
				ItemParent = RelativeItemParent;
				AddParams.RelativeDropZone = EItemDropZone::BelowItem;
			}
			else
			{
				ItemParent = AddParams.RelativeItem;
			}
		}
		//else we place it as a Sibling to the Relative Item
		else if (RelativeItemParent)
		{
			ItemParent = RelativeItemParent;
		}
	}

	// Item parent is defaulted to tree root which is guaranteed to be valid.
	// And item parent can only be set to a valid item thereafter
	check(ItemParent.IsValid());
	const TArray<FAvaOutlinerItemPtr> AddedChildren = ItemParent->AddChildren(AddParams);

	if (AddedChildren.IsEmpty())
	{
		return;
	}

	// Sync selection from mode tools and outliner
	if (const FEditorModeTools* const ModeTools = InOutliner.GetModeTools())
	{
		const FAvaOutlinerScopedSelection ScopedSelection(*ModeTools, EAvaOutlinerScopedSelectionPurpose::Read);

		for (const FAvaOutlinerItemPtr& AddedChild : AddedChildren)
		{
			const bool bSelectedInModeTools = AddedChild->IsSelected(ScopedSelection);
			const bool bSelectedInOutliner  = InOutliner.GetSelectedItems().Contains(AddedChild);

			// Update Selection if there's a discrepancy between the Mode Tools Selection & the Outliner View Selected Items
			if (bSelectedInModeTools != bSelectedInOutliner)
			{
				// Automatically Select Item if it's Selected in Mode Tools and not yet in the Outliner
				if (bSelectedInModeTools)
				{
					// Select in Outliner but don't signal selection as we already have it selected in Mode Tools
					AddParams.Flags |= EAvaOutlinerAddItemFlags::Select;
					AddParams.SelectionFlags &= ~EAvaOutlinerItemSelectionFlags::SignalSelectionChange;
				}
				// Signal Selection Change when we attempt to select this item in the Outliner but it isn't selected in Mode Tools
				else if (EnumHasAnyFlags(AddParams.Flags, EAvaOutlinerAddItemFlags::Select))
				{
					AddParams.SelectionFlags |= EAvaOutlinerItemSelectionFlags::SignalSelectionChange;
				}
			}
		}
	}

	if (EnumHasAnyFlags(AddParams.Flags, EAvaOutlinerAddItemFlags::Select))
	{
		InOutliner.SelectItems(AddedChildren, AddParams.SelectionFlags);
	}

	InOutliner.SetOutlinerModified();
}

void FAvaOutlinerAddItem::OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, bool bRecursive)
{
	for (const FAvaOutlinerItemPtr& Item : AddParams.Items)
	{
		if (Item.IsValid())
		{
			Item->OnObjectsReplaced(InReplacementMap, bRecursive);
		}
	}
	if (AddParams.RelativeItem.IsValid())
	{
		AddParams.RelativeItem->OnObjectsReplaced(InReplacementMap, bRecursive);
	}
}
