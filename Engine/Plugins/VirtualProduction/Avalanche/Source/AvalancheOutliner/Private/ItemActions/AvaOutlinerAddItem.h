// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerItemAction.h"
#include "Item/AvaOutlinerItemParameters.h"

/**
 * Item Action responsible of adding an item to the Tree under a given optional Parent.
 * If Parent is null, it is added as a Top Level Item.
 */
class FAvaOutlinerAddItem : public IAvaOutlinerAction
{
public:
	UE_AVA_INHERITS(FAvaOutlinerAddItem, IAvaOutlinerAction);

	FAvaOutlinerAddItem(const FAvaOutlinerAddItemParams& InAddItemParams);

	//~ Begin IAvaOutlinerAction
	virtual bool ShouldTransact() const override;
	virtual void Execute(FAvaOutliner& InOutliner) override;
	virtual void OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, bool bRecursive) override;
	//~ End IAvaOutlinerAction

protected:
	FAvaOutlinerAddItemParams AddParams;
};
