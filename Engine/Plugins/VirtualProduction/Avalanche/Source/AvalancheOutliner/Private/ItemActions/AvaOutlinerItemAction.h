// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "AvaType.h"

class FAvaOutliner;
class UObject;

/**
 * Interface class for an Action in the Outliner (e.g. Add/Delete/Move Tree Item)
 */ 
class IAvaOutlinerAction : public IAvaTypeCastable
{
public:
	UE_AVA_INHERITS(IAvaOutlinerAction, IAvaTypeCastable);

	/** Determines whether the given action modifies its objects and should transact */
	virtual bool ShouldTransact() const { return false; }

	/** The Action to execute on the given Outliner */
	virtual void Execute(FAvaOutliner& InOutliner) = 0;

	/** Replace any Objects that might be held in this Action that has been killed and replaced by a new object (e.g. BP Components) */
	virtual void OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, bool bRecursive) = 0;
};
