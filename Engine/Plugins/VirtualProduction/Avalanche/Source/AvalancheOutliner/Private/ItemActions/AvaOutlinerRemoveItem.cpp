// Copyright Epic Games, Inc. All Rights Reserved.

#include "ItemActions/AvaOutlinerRemoveItem.h"
#include "AvaOutliner.h"
#include "Item/IAvaOutlinerItem.h"
#include "ItemActions/AvaOutlinerAddItem.h"
#include "Widgets/Views/STableRow.h"

FAvaOutlinerRemoveItem::FAvaOutlinerRemoveItem(const FAvaOutlinerRemoveItemParams& InRemoveItemParams)
	: RemoveParams(InRemoveItemParams)
{
	if (RemoveParams.Item.IsValid())
	{
		RemoveParams.Item->AddFlags(EAvaOutlinerItemFlags::PendingRemoval);
	}
}

void FAvaOutlinerRemoveItem::Execute(FAvaOutliner& InOutliner)
{
	if (RemoveParams.Item.IsValid())
	{
		FAvaOutlinerItemFlagGuard Guard(RemoveParams.Item, EAvaOutlinerItemFlags::IgnorePendingKill);

		//Copy the Array since we may modify it below on Add/Remove Child
		TArray<FAvaOutlinerItemPtr> Children = RemoveParams.Item->GetChildren();

		FAvaOutlinerItemPtr Parent       = RemoveParams.Item->GetParent();
		FAvaOutlinerItemPtr RelativeItem = RemoveParams.Item;

		//Search the lowest parent that is not pending removal
		while (Parent.IsValid() && Parent->HasAnyFlags(EAvaOutlinerItemFlags::PendingRemoval))
		{
			RelativeItem = Parent;
			Parent       = Parent->GetParent();
		}

		//Reparent the Item's Children to the Valid Parent found above
		if (Parent.IsValid())
		{
			FAvaOutlinerAddItemParams ReparentParams;
			ReparentParams.RelativeItem             = RelativeItem;
			ReparentParams.RelativeDropZone         = EItemDropZone::BelowItem;
			ReparentParams.Flags                    = EAvaOutlinerAddItemFlags::Select;
			ReparentParams.AttachmentTransformRules = FAttachmentTransformRules::KeepWorldTransform;

			TArray<FAvaOutlinerItemPtr> ItemsToReparent;

			ItemsToReparent.Append(Children);
			while (ItemsToReparent.Num() > 0)
			{
				ReparentParams.Items = { ItemsToReparent.Pop() };

				//Then Parent it to the Item's Parent.
				//If we couldn't add the Child, then it means either the Child Item itself is invalid
				//or the underlying info (e.g. object) is invalid (e.g. actor pending kill)
				if (Parent->AddChildren(ReparentParams).IsEmpty())
				{
					//so try reparenting the children of this invalid child (since this invalid child will be removed)
					if (ReparentParams.Items[0].IsValid())
					{
						ItemsToReparent.Append(ReparentParams.Items[0]->GetChildren());
					}
				}
			}

			//In case the Parent is still the same, Remove Child Item, else the Parent is going to be removed anyways
			if (Parent == RemoveParams.Item->GetParent())
			{
				Parent->RemoveChild(RemoveParams.Item);
			}
		}
		else
		{
			for (const FAvaOutlinerItemPtr& Child : Children)
			{
				RemoveParams.Item->RemoveChild(Child);
			}
		}
		
		InOutliner.UnregisterItem(RemoveParams.Item->GetItemId());
		InOutliner.SetOutlinerModified();
	}
}

void FAvaOutlinerRemoveItem::OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, bool bRecursive)
{
	if (RemoveParams.Item.IsValid())
	{
		RemoveParams.Item->OnObjectsReplaced(InReplacementMap, bRecursive);
	}
}
