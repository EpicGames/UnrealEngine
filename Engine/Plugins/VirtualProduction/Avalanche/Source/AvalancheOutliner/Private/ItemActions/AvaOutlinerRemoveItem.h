// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerItemAction.h"
#include "Item/AvaOutlinerItemParameters.h"

/**
 * Item Action responsible of removing/unregistering Items from the Tree
 */
class FAvaOutlinerRemoveItem : public IAvaOutlinerAction
{
public:
	UE_AVA_INHERITS(FAvaOutlinerRemoveItem, IAvaOutlinerAction);

	FAvaOutlinerRemoveItem(const FAvaOutlinerRemoveItemParams& InRemoveItemParams);

	//~ Begin IAvaOutlinerAction
	virtual void Execute(FAvaOutliner& InOutliner) override;
	virtual void OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, bool bRecursive) override;
	//~ End IAvaOutlinerAction

protected:
	FAvaOutlinerRemoveItemParams RemoveParams;
};
