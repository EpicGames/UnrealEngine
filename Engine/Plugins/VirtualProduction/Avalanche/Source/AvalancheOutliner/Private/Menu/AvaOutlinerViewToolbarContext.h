// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "AvaOutlinerViewToolbarContext.generated.h"

class FAvaOutlinerView;

UCLASS()
class UAvaOutlinerViewToolbarContext : public UObject
{
	GENERATED_BODY()

	friend FAvaOutlinerView;

public:
	UAvaOutlinerViewToolbarContext() = default;

	TSharedPtr<FAvaOutlinerView> GetOutlinerView() const { return OutlinerViewWeak.Pin(); }

private:
	TWeakPtr<FAvaOutlinerView> OutlinerViewWeak;
};
