// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/AvaOutlinerScopedSelection.h"
#include "Components/ActorComponent.h"
#include "EditorModeManager.h"
#include "GameFramework/Actor.h"
#include "Selection.h"

namespace UE::AvaOutliner::Private
{
	USelection* GetSelection(const FEditorModeTools& InEditorModeTools, const UObject* InObject)
	{
		if (Cast<AActor>(InObject))
		{
			return InEditorModeTools.GetSelectedActors();
		}

		if (Cast<UActorComponent>(InObject))
		{
			return InEditorModeTools.GetSelectedComponents();
		}

		return InEditorModeTools.GetSelectedObjects();
	}

	void SyncSelection(USelection* InSelection, const TArray<UObject*>& InObjects)
	{
		// Check if we even need to update selection, by comparing Old Selections vs new Selections
		TArray<UObject*> SelectedObjects;
		InSelection->GetSelectedObjects(SelectedObjects);

		// If Element Count is the same, make sure there is no new element to add
		if (SelectedObjects.Num() == InObjects.Num())
		{
			TSet<UObject*> SelectedObjectSet = TSet<UObject*>(MoveTemp(SelectedObjects));

			// Assume that Containers are the same at the start
			bool bSelectionUpdateRequired = false;

			for (UObject* const Object : InObjects)
			{
				// if there's at least one object not in the current set, then Selection Change is Required
				if (!SelectedObjectSet.Contains(Object))
				{
					bSelectionUpdateRequired = true;
					break;
				}
			}

			if (!bSelectionUpdateRequired)
			{
				return;
			}
		}

		InSelection->Modify();
		InSelection->BeginBatchSelectOperation();
		InSelection->DeselectAll();

		for (UObject* const Object : InObjects)
		{
			InSelection->Select(Object);
		}

		InSelection->EndBatchSelectOperation();
	}
}

FAvaOutlinerScopedSelection::FAvaOutlinerScopedSelection(const FEditorModeTools& InEditorModeTools, EAvaOutlinerScopedSelectionPurpose InPurpose)
	: EditorModeTools(InEditorModeTools)
	, Purpose(InPurpose)
{
}

FAvaOutlinerScopedSelection::~FAvaOutlinerScopedSelection()
{
	if (Purpose == EAvaOutlinerScopedSelectionPurpose::Sync)
	{
		SyncSelections();
	}
}

void FAvaOutlinerScopedSelection::Select(UObject* InObject)
{
	if (!ensureMsgf(Purpose == EAvaOutlinerScopedSelectionPurpose::Sync
		, TEXT("Scope is trying to Select, but it's not a Sync Scope.")))
	{
		return;
	}

	if (!InObject)
	{
		return;
	}

	bool bIsAlreadyInSet = false;
	ObjectsSet.Add(InObject, &bIsAlreadyInSet);
	if (bIsAlreadyInSet)
	{
		return;
	}

	if (AActor* const Actor = Cast<AActor>(InObject))
	{
		SelectedActors.Add(Actor);
	}
	else if (UActorComponent* const Component = Cast<UActorComponent>(InObject))
	{
		SelectedComponents.Add(Component);
	}
	else
	{
		SelectedObjects.Add(InObject);
	}
}

bool FAvaOutlinerScopedSelection::IsSelected(const UObject* InObject) const
{
	// Return true if it's Selected in Objects Set (i.e. Pending to Select, but not yet propagated to Editor Mode Tools)
	if (Purpose == EAvaOutlinerScopedSelectionPurpose::Sync && ObjectsSet.Contains(InObject))
	{
		return true;
	}

	USelection* const Selection = UE::AvaOutliner::Private::GetSelection(EditorModeTools, InObject);
	if (ensure(Selection))
	{
		return Selection->IsSelected(InObject);
	}
	return false;
}

void FAvaOutlinerScopedSelection::SyncSelections()
{
	UE::AvaOutliner::Private::SyncSelection(EditorModeTools.GetSelectedActors(), SelectedActors);
	UE::AvaOutliner::Private::SyncSelection(EditorModeTools.GetSelectedComponents(), SelectedComponents);
	UE::AvaOutliner::Private::SyncSelection(EditorModeTools.GetSelectedObjects(), SelectedObjects);
}

const FEditorModeTools& FAvaOutlinerScopedSelection::GetEditorModeTools() const
{
	return EditorModeTools;
}
