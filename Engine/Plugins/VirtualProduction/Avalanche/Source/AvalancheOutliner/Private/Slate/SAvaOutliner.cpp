// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SAvaOutliner.h"
#include "AvaOutliner.h"
#include "AvaOutlinerModule.h"
#include "AvaOutlinerSettings.h"
#include "AvaOutlinerView.h"
#include "Columns/IAvaOutlinerColumn.h"
#include "DragDropOps/AvaOutlinerItemDragDropOp.h"
#include "Filters/AvaOutlinerTextFilter.h"
#include "Filters/SCustomTextFilterDialog.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "Item/AvaOutlinerTreeRoot.h"
#include "Misc/MessageDialog.h"
#include "SAssetSearchBox.h"
#include "Slate/SAvaOutlinerItemFilters.h"
#include "Slate/SAvaOutlinerTreeRow.h"
#include "Slate/SAvaOutlinerTreeView.h"
#include "Stats/Slate/SAvaOutlinerStats.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SAvaOutliner"

namespace UE::AvaOutliner::Private
{
	void ExtractAssetSearchFilterTerms(const FText& SearchText, FString* OutFilterKey, FString* OutFilterValue, int32* OutSuggestionInsertionIndex)
	{
		const FString SearchString = SearchText.ToString();

		if (OutFilterKey)
		{
			OutFilterKey->Reset();
		}
		if (OutFilterValue)
		{
			OutFilterValue->Reset();
		}
		if (OutSuggestionInsertionIndex)
		{
			*OutSuggestionInsertionIndex = SearchString.Len();
		}

		// Build the search filter terms so that we can inspect the tokens
		FTextFilterExpressionEvaluator LocalFilter(ETextFilterExpressionEvaluatorMode::Complex);
		LocalFilter.SetFilterText(SearchText);

		// Inspect the tokens to see what the last part of the search term was
		// If it was a key->value pair then we'll use that to control what kinds of results we show
		// For anything else we just use the text from the last token as our filter term to allow incremental auto-complete
		const TArray<FExpressionToken>& FilterTokens = LocalFilter.GetFilterExpressionTokens();
		if (FilterTokens.Num() > 0)
		{
			const FExpressionToken& LastToken = FilterTokens.Last();

			// If the last token is a text token, then consider it as a value and walk back to see if we also have a key
			if (LastToken.Node.Cast<TextFilterExpressionParser::FTextToken>())
			{
				if (OutFilterValue)
				{
					*OutFilterValue = LastToken.Context.GetString();
				}
				if (OutSuggestionInsertionIndex)
				{
					*OutSuggestionInsertionIndex = FMath::Min(*OutSuggestionInsertionIndex, LastToken.Context.GetCharacterIndex());
				}

				if (FilterTokens.IsValidIndex(FilterTokens.Num() - 2))
				{
					const FExpressionToken& ComparisonToken = FilterTokens[FilterTokens.Num() - 2];
					if (ComparisonToken.Node.Cast<TextFilterExpressionParser::FEqual>())
					{
						if (FilterTokens.IsValidIndex(FilterTokens.Num() - 3))
						{
							const FExpressionToken& KeyToken = FilterTokens[FilterTokens.Num() - 3];
							if (KeyToken.Node.Cast<TextFilterExpressionParser::FTextToken>())
							{
								if (OutFilterKey)
								{
									*OutFilterKey = KeyToken.Context.GetString();
								}
								if (OutSuggestionInsertionIndex)
								{
									*OutSuggestionInsertionIndex = FMath::Min(*OutSuggestionInsertionIndex, KeyToken.Context.GetCharacterIndex());
								}
							}
						}
					}
				}
			}
			// If the last token is a comparison operator, then walk back and see if we have a key
			else if (LastToken.Node.Cast<TextFilterExpressionParser::FEqual>())
			{
				if (FilterTokens.IsValidIndex(FilterTokens.Num() - 2))
				{
					const FExpressionToken& KeyToken = FilterTokens[FilterTokens.Num() - 2];
					if (KeyToken.Node.Cast<TextFilterExpressionParser::FTextToken>())
					{
						if (OutFilterKey)
						{
							*OutFilterKey = KeyToken.Context.GetString();
						}
						if (OutSuggestionInsertionIndex)
						{
							*OutSuggestionInsertionIndex = FMath::Min(*OutSuggestionInsertionIndex, KeyToken.Context.GetCharacterIndex());
						}
					}
				}
			}
		}
	}
}


void SAvaOutliner::Construct(const FArguments& InArgs, const TSharedRef<FAvaOutlinerView>& InOutlinerView)
{
	CustomTextFilterWindow = nullptr;
	OutlinerViewWeak = InOutlinerView;
	bIsEnterLastKeyPressed = false;
	HeaderRowWidget = SNew(SHeaderRow)
		.Visibility(EVisibility::Visible)
		.CanSelectGeneratedColumn(true);

	ReconstructColumns();

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.Padding(4.f, 4.f, 0.f, 4.f)
			[
				SAssignNew(SearchBoxPtr, SAssetSearchBox)
				.HintText(LOCTEXT("FilterSearch", "Search..."))
				.ToolTipText(LOCTEXT("FilterSearchHint", "Type here to search..."))
				.ShowSearchHistory(true)
				.OnTextChanged(this, &SAvaOutliner::OnSearchTextChanged)
				.OnTextCommitted(this, &SAvaOutliner::OnSearchTextCommitted)
				.OnSaveSearchClicked(this, &SAvaOutliner::OnSaveSearchButtonClicked)
				.OnAssetSearchBoxSuggestionFilter(this, &SAvaOutliner::OnAssetSearchSuggestionFilter)
				.OnAssetSearchBoxSuggestionChosen(this, &SAvaOutliner::OnAssetSearchSuggestionChosen)
				.DelayChangeNotificationsWhileTyping(true)
				.Visibility(EVisibility::Visible)
				.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("SAvaOutlinerSearchBox")))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f, 4.f, 0.f)
			[
				SAssignNew(ToolBarBox, SBox)
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SAvaOutlinerItemFilters, InOutlinerView)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(TreeBorder, SBorder)
					.BorderImage(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>(TEXT("SceneOutliner.TableViewRow")).DropIndicator_Onto)
					.Visibility(EVisibility::Hidden)
				]
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.Padding(2.0f, 2.0f, 2.0f, 2.0f)
				[
					SAssignNew(TreeView, SAvaOutlinerTreeView, InOutlinerView)
					.TreeViewArgs(STreeView<FAvaOutlinerItemPtr>::FArguments()
					.HeaderRow(HeaderRowWidget)
					.TreeItemsSource(&InOutlinerView->GetRootVisibleItems())
					.OnGetChildren(InOutlinerView, &FAvaOutlinerView::GetChildrenOfItem)
					.OnGenerateRow(this, &SAvaOutliner::OnItemGenerateRow)
					.OnSelectionChanged(this, &SAvaOutliner::OnItemSelectionChanged)
					.OnExpansionChanged(InOutlinerView, &FAvaOutlinerView::OnItemExpansionChanged)
					.OnContextMenuOpening(InOutlinerView, &FAvaOutlinerView::CreateItemContextMenu)
					.OnSetExpansionRecursive(InOutlinerView, &FAvaOutlinerView::SetItemExpansionRecursive)
					.HighlightParentNodesForSelection(true)
					.AllowInvisibleItemSelection(true) //To select items that are still collapsed
					.SelectionMode(ESelectionMode::Multi))
				]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SAvaOutlinerStats, InOutlinerView)
		]
	];

	Refresh();
}

SAvaOutliner::~SAvaOutliner()
{
	if (OutlinerViewWeak.IsValid())
	{
		OutlinerViewWeak.Pin()->SaveState();
	}
}

void SAvaOutliner::SetToolBarWidget(TSharedRef<SWidget> InToolBarWidget)
{
	ToolBarBox->SetContent(InToolBarWidget);
}

void SAvaOutliner::ReconstructColumns()
{
	const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin();

	if (!OutlinerView.IsValid())
	{
		return;
	}

	HeaderRowWidget->ClearColumns();

	// Add the Columns allocated by the Owning Instance
	for (const TPair<FName, TSharedPtr<IAvaOutlinerColumn>>& Pair : OutlinerView->GetColumns())
	{
		const TSharedPtr<IAvaOutlinerColumn>& Column = Pair.Value;

		if (Column.IsValid())
		{
			HeaderRowWidget->AddColumn(Column->ConstructHeaderRowColumn());
			HeaderRowWidget->SetShowGeneratedColumn(Column->GetColumnId(), OutlinerView->ShouldShowColumnByDefault(Column));
		}
	}
}

void SAvaOutliner::Refresh()
{
	TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin();
	check(OutlinerView.IsValid());

	check(TreeView.IsValid());
	TreeView->RequestTreeRefresh();
}

void SAvaOutliner::SetItemSelection(const TArray<FAvaOutlinerItemPtr>& InItems, bool bSignalSelectionChange)
{
	if (bSelectingItems)
	{
		return;
	}

	TGuardValue<bool> Guard(bSelectingItems, true);

	TreeView->Private_ClearSelection();

	if (!InItems.IsEmpty())
	{
		TreeView->SetItemSelection(InItems, true, ESelectInfo::Type::Direct);
	}

	if (bSignalSelectionChange)
	{
		TreeView->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);
	}
}

void SAvaOutliner::OnItemSelectionChanged(FAvaOutlinerItemPtr InItem, ESelectInfo::Type InSelectionType)
{
	if (TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		TArray<FAvaOutlinerItemPtr> SelectedItems = TreeView->GetSelectedItems();
		const bool bUpdateModeTools = InSelectionType != ESelectInfo::Type::Direct;
		OutlinerView->NotifyItemSelectionChanged(SelectedItems, InItem, bUpdateModeTools);
	}
}

void SAvaOutliner::ScrollItemIntoView(const FAvaOutlinerItemPtr& InItem) const
{
	TreeView->ScrollItemIntoView(InItem);
}

void SAvaOutliner::SetItemExpansion(FAvaOutlinerItemPtr Item, bool bShouldExpand) const
{
	TreeView->SetItemExpansion(Item, bShouldExpand);
}

void SAvaOutliner::UpdateItemExpansions(FAvaOutlinerItemPtr Item) const
{
	TreeView->UpdateItemExpansions(Item);
}

TSharedRef<ITableRow> SAvaOutliner::OnItemGenerateRow(FAvaOutlinerItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin();
	check(Item.IsValid() && OutlinerView.IsValid());

	return SNew(SAvaOutlinerTreeRow, OutlinerView.ToSharedRef(), TreeView, Item)
		.HighlightText(OutlinerView->GetTextFilter(), &FAvaOutlinerTextFilter::GetFilterText);
}

void SAvaOutliner::SetKeyboardFocus() const
{
	if (SupportsKeyboardFocus())
	{
		FWidgetPath OutlinerTreeViewWidgetPath;
		// NOTE: Careful, GeneratePathToWidget can be reentrant in that it can call visibility delegates and such
		FSlateApplication::Get().GeneratePathToWidgetUnchecked(TreeView.ToSharedRef(), OutlinerTreeViewWidgetPath);
		FSlateApplication::Get().SetKeyboardFocus(OutlinerTreeViewWidgetPath, EFocusCause::SetDirectly);
	}
}

FText SAvaOutliner::GetSearchText() const
{
	if (TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		return OutlinerView->GetSearchText();
	}
	return FText::GetEmpty();
}

void SAvaOutliner::OnSearchTextChanged(const FText& FilterText) const
{
	if (TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		OutlinerView->SetSearchText(FilterText);
	}
}

void SAvaOutliner::OnSearchTextCommitted(const FText& FilterText, ETextCommit::Type CommitType) const
{
	if (TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		OutlinerView->SetSearchText(FilterText);
	}
}

void SAvaOutliner::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		OutlinerView->UpdateRecentOutlinerViews();
	}
	SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
}

FReply SAvaOutliner::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	bIsEnterLastKeyPressed = InKeyEvent.GetKey() == EKeys::Enter;
	if (TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		OutlinerView->UpdateRecentOutlinerViews();

		TSharedPtr<FUICommandList> CommandList = OutlinerView->GetBaseCommandList();
		if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}
	}
	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

FReply SAvaOutliner::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin();

	if (!OutlinerView.IsValid())
	{
		return FReply::Unhandled();
	}

	return OutlinerView->OnDrop(DragDropEvent, EItemDropZone::OntoItem, nullptr);
}

void SAvaOutliner::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FAvaOutlinerItemDragDropOp> ItemDragDropOp = DragDropEvent.GetOperationAs<FAvaOutlinerItemDragDropOp>())
	{
		bool bIsDragFromThisOutlinerView = ItemDragDropOp->GetOutlinerView() == OutlinerViewWeak;
		bool bContainedItemDragDropOp = ItemDragDropOps.Remove(ItemDragDropOp) > 0;

		// Don't process Drag Enter unless it has already left before.
		// Only applicable if the DragDrop starts from is from within (i.e. same outliner view and it's an FAvaOutlinerItemDragDropOp)
		// This is because DragEnter doesn't have an FReply to stop SAvaOutliner from receiving the DragEnter event as soon as a drag starts
		if (bIsDragFromThisOutlinerView && !bContainedItemDragDropOp)
		{
			return;
		}
	}

	if (TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		OutlinerView->OnDragEnter(DragDropEvent, nullptr);
	}
}

void SAvaOutliner::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FAvaOutlinerItemDragDropOp> ItemDragDropOp = DragDropEvent.GetOperationAs<FAvaOutlinerItemDragDropOp>())
	{
		ItemDragDropOps.Add(ItemDragDropOp);
	}

	if (TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		OutlinerView->OnDragLeave(DragDropEvent, nullptr);
	}
}

void SAvaOutliner::SetTreeBorderVisibility(bool bVisible)
{
	TreeBorder->SetVisibility(bVisible ? EVisibility::SelfHitTestInvisible : EVisibility::Hidden);
}

void SAvaOutliner::GenerateColumnVisibilityMap(TMap<FName, bool>& OutVisibilityMap)
{
	const TIndirectArray<SHeaderRow::FColumn>& Columns = HeaderRowWidget->GetColumns();
	OutVisibilityMap.Empty(Columns.Num());

	for (const SHeaderRow::FColumn& Column : Columns)
	{
		OutVisibilityMap.Add(Column.ColumnId, Column.bIsVisible);
	}
}

void SAvaOutliner::OnAssetSearchSuggestionFilter(const FText& InSearchText, TArray<FAssetSearchBoxSuggestion>& OutPossibleSuggestions, FText& OutSuggestionHighlightText) const
{
	// We don't bind the suggestion list, so this list should be empty as we populate it here based on the search term
	check(OutPossibleSuggestions.IsEmpty());

	if (bIsEnterLastKeyPressed)
	{
		return;
	}

	FString FilterKey;
	FString FilterValue;
	UE::AvaOutliner::Private::ExtractAssetSearchFilterTerms(InSearchText, &FilterKey, &FilterValue, nullptr);

	const FAvaOutlinerModule& OutlinerModule = FAvaOutlinerModule::Get();
	const TSharedRef<FAvaFilterSuggestionPayload> GenericPayload(MakeShared<FAvaFilterSuggestionPayload>(OutPossibleSuggestions,FilterValue));

	for (const TSharedPtr<IAvaFilterSuggestionFactory>& GenericSuggestion : OutlinerModule.GetSuggestions(EAvaFilterSuggestionType::Generic))
	{
		GenericSuggestion->AddSuggestion(GenericPayload);
	}

	if (const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		if (const TSharedPtr<FAvaOutliner> OutlinerPtr = OutlinerView->GetOutliner())
		{
			//Used to avoid adding the same suggestion twice (different Obj but with same name/same class/same tag)
			TSet<FString> FilterCache;
			const TSharedRef<FAvaFilterSuggestionItemPayload> ItemBasedPayload(MakeShared<FAvaFilterSuggestionItemPayload>(OutPossibleSuggestions, FilterValue, nullptr, FilterCache));

			for (const FAvaOutlinerItemPtr& RootChildFromRoot : OutlinerPtr->GetTreeRoot()->GetChildren())
			{
				for (const TSharedPtr<IAvaFilterSuggestionFactory>& ItemSuggestion : OutlinerModule.GetSuggestions(EAvaFilterSuggestionType::ItemBased))
				{
					ItemBasedPayload->Item = RootChildFromRoot;
					ItemSuggestion->AddSuggestion(ItemBasedPayload);
				}

				TArray<FAvaOutlinerItemPtr> Children;
				RootChildFromRoot->FindValidChildren(Children, true);

				for (const FAvaOutlinerItemPtr& Child : Children)
				{
					for (const TSharedPtr<IAvaFilterSuggestionFactory>& ItemSuggestion : OutlinerModule.GetSuggestions(EAvaFilterSuggestionType::ItemBased))
					{
						ItemBasedPayload->Item = Child;
						ItemSuggestion->AddSuggestion(ItemBasedPayload);
					}
				}
			}
		}
	}
	OutSuggestionHighlightText = FText::FromString(FilterValue);
}

FText SAvaOutliner::OnAssetSearchSuggestionChosen(const FText& InSearchText, const FString& InSuggestion) const
{
	int32 SuggestionInsertionIndex = 0;
	UE::AvaOutliner::Private::ExtractAssetSearchFilterTerms(InSearchText, nullptr, nullptr, &SuggestionInsertionIndex);

	FString SearchString = InSearchText.ToString();
	SearchString.RemoveAt(SuggestionInsertionIndex, SearchString.Len() - SuggestionInsertionIndex, EAllowShrinking::No);
	SearchString.Append(InSuggestion);
	return FText::FromString(SearchString);
}

void SAvaOutliner::OnSaveSearchButtonClicked(const FText& InText)
{
	/** If we already have a window, delete it */
	OnCancelCustomTextFilterDialog();

	CreateCustomTextFilterWindow(InText);
}

void SAvaOutliner::OnCreateCustomTextFilter(const FCustomTextFilterData& InFilterData, bool ApplyFilter) const
{
	if (UAvaOutlinerSettings* OutlinerSettings = UAvaOutlinerSettings::Get())
	{
		FAvaOutlinerItemTypeFilterData NewFilter = FAvaOutlinerItemTypeFilterData();
		NewFilter.SetFilterText(InFilterData.FilterString);
		NewFilter.SetOverrideIconColor(InFilterData.FilterColor);

		const FName FilterKey = FName(InFilterData.FilterLabel.ToString() + TEXT("_") + FGuid::NewGuid().ToString());
		if (!OutlinerSettings->AddCustomItemTypeFilter(FilterKey, NewFilter))
		{
			FMessageDialog::Open(EAppMsgType::Ok,
				LOCTEXT("InvalidKeyError_Message", "An error has occured when creating the key, try change the Filter Label text and retry"),
				LOCTEXT("InvalidKeyError_Title", "Invalid Key"));
		}
		else
		{
			if (const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
			{
				OutlinerView->UpdateCustomFilters();

				if (ApplyFilter)
				{
					OutlinerView->EnableItemFilterById(FilterKey, true);
				}

				OutlinerSettings->SaveConfig();
				OnCancelCustomTextFilterDialog();
			}
		}
	}
}

void SAvaOutliner::OnCancelCustomTextFilterDialog() const
{
	if(CustomTextFilterWindow.IsValid())
	{
		CustomTextFilterWindow.Pin()->RequestDestroyWindow();
	}
}

void SAvaOutliner::CreateCustomTextFilterWindow(const FText& InText)
{
	FCustomTextFilterData CustomTextFilterData;
	CustomTextFilterData.FilterLabel = InText;
	CustomTextFilterData.FilterString = InText;

	const TSharedPtr<SWindow> NewTextFilterWindow = SNew(SWindow)
		.Title(LOCTEXT("CreateCustomTextAvaFilterWindow", "Create Custom Motion Design Filter"))
		.HasCloseButton(true)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.SizingRule(ESizingRule::FixedSize)
		.ClientSize(FVector2D(724, 183));

	const TSharedPtr<SCustomTextFilterDialog> CustomTextFilterDialog =
		SNew(SCustomTextFilterDialog)
		.FilterData(CustomTextFilterData)
		.InEditMode(false)
		.OnCreateFilter(this, &SAvaOutliner::OnCreateCustomTextFilter)
		.OnCancelClicked(this, &SAvaOutliner::OnCancelCustomTextFilterDialog);

	NewTextFilterWindow->SetContent(CustomTextFilterDialog.ToSharedRef());
	FSlateApplication::Get().AddWindow(NewTextFilterWindow.ToSharedRef());

	CustomTextFilterWindow = NewTextFilterWindow;
}

#undef LOCTEXT_NAMESPACE
