// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Item/IAvaOutlinerItem.h"
#include "Widgets/SCompoundWidget.h"

class FAvaOutlinerItemDragDropOp;
class FAvaOutlinerView;
class ITableRow;
class SAssetFilterBar;
class SAssetSearchBox;
class SAvaOutlinerItemFilters;
class SAvaOutlinerTreeView;
class SBorder;
class SBox;
class SComboButton;
class SHeaderRow;
class SHorizontalBox;
class STableViewBase;
class UToolMenu;

struct FAssetSearchBoxSuggestion;
struct FCustomTextFilterData;

class SAvaOutliner : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaOutliner) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FAvaOutlinerView>& InOutlinerView);

	virtual ~SAvaOutliner() override;

	TSharedPtr<SAvaOutlinerTreeView> GetTreeView() const { return TreeView; }

	void SetToolBarWidget(TSharedRef<SWidget> InToolBarWidget);

	void ReconstructColumns();

	void Refresh();

	void SetItemSelection(const TArray<FAvaOutlinerItemPtr>& InItems, bool bSignalSelectionChange);

	void OnItemSelectionChanged(FAvaOutlinerItemPtr InItem, ESelectInfo::Type InSelectionType);

	void ScrollItemIntoView(const FAvaOutlinerItemPtr& InItem) const;

	void SetItemExpansion(FAvaOutlinerItemPtr Item, bool bShouldExpand) const;

	void UpdateItemExpansions(FAvaOutlinerItemPtr Item) const;

	TSharedRef<ITableRow> OnItemGenerateRow(FAvaOutlinerItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable);

	void SetKeyboardFocus() const;

	FText GetSearchText() const;
	void OnSearchTextChanged(const FText& FilterText) const;
	void OnSearchTextCommitted(const FText& FilterText, ETextCommit::Type CommitType) const;

	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;

	void SetTreeBorderVisibility(bool bVisible);

	void GenerateColumnVisibilityMap(TMap<FName, bool>& OutVisibilityMap);

	void OnAssetSearchSuggestionFilter(const FText& InSearchText, TArray<FAssetSearchBoxSuggestion>& OutPossibleSuggestions, FText& OutSuggestionHighlightText) const;

	FText OnAssetSearchSuggestionChosen(const FText& InSearchText, const FString& InSuggestion) const;

	void OnSaveSearchButtonClicked(const FText& InText);

	void OnCreateCustomTextFilter(const FCustomTextFilterData& InFilterData, bool ApplyFilter) const;

	void OnCancelCustomTextFilterDialog() const;

	void CreateCustomTextFilterWindow(const FText& InText);

private:
	/** The Outliner Instance that created this Widget */
	TWeakPtr<FAvaOutlinerView> OutlinerViewWeak;

	/** The toolbar container */
	TSharedPtr<SBox> ToolBarBox;

	/** The header row of the Outliner */
	TSharedPtr<SHeaderRow> HeaderRowWidget;

	TSharedPtr<SBorder> TreeBorder;

	TSharedPtr<SAvaOutlinerTreeView> TreeView;

	TSharedPtr<SComboButton> SettingsMenu;

	TSharedPtr<SAssetSearchBox> SearchBoxPtr;

	/** The window containing the custom text filter dialog */
	TWeakPtr<SWindow> CustomTextFilterWindow;

	/** If true the SuggestionList shouldn't appear since we selected what we wanted */
	bool bIsEnterLastKeyPressed = false;

	bool bSelectingItems = false;

	TSet<TWeakPtr<FAvaOutlinerItemDragDropOp>> ItemDragDropOps;
};
