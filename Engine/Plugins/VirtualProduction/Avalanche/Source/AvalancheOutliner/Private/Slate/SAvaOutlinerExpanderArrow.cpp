// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaOutlinerExpanderArrow.h"
#include "Widgets/Views/ITableRow.h"

void SAvaOutlinerExpanderArrow::Construct(const FArguments& InArgs, const TSharedPtr<ITableRow>& TableRow)
{
	WireTint = InArgs._WireTint;
	SExpanderArrow::Construct(InArgs._ExpanderArrowArgs, TableRow);
}

int32 SAvaOutlinerExpanderArrow::OnPaint(const FPaintArgs& Args
	, const FGeometry& AllottedGeometry
	, const FSlateRect& MyCullingRect
	, FSlateWindowElementList& OutDrawElements
	, int32 LayerId
	, const FWidgetStyle& InWidgetStyle
	, bool bParentEnabled) const
{
	static constexpr float WireThickness = 2.0f;
	
	static constexpr float HalfWireThickness = WireThickness * 0.5f;
	
	// We want to support drawing wires for the tree
	//                 Needs Wire Array
	//   v-[A]         {}
	//   |-v[B]        {1}
	//   | '-v[B]      {1,1}
	//   |   |--[C]    {1,0,1}
	//   |   |--[D]    {1,0,1}
	//   |   '--[E]    {1,0,1}
	//   |>-[F]        {}
	//   '--[G]        {}
	//
	//
	static const FName NAME_VerticalBarBrush = TEXT("WhiteBrush");
	
	const float Indent = GetIndentAmount();
	
	const FSlateBrush* const VerticalBarBrush = StyleSet == nullptr
		? nullptr
		: StyleSet->GetBrush(NAME_VerticalBarBrush);

	if (ShouldDrawWires.Get() == true && VerticalBarBrush != nullptr)
	{
		const FLinearColor WireColor = WireTint.Get(FLinearColor(0.1f, 0.1f, 0.1f, 0.25f));
		
		const TSharedPtr<ITableRow> OwnerRow = OwnerRowPtr.Pin();

		// Draw vertical wires to indicate paths to parent nodes.
		const TBitArray<>& NeedsWireByLevel = OwnerRow->GetWiresNeededByDepth();
		
		const int32 NumLevels = NeedsWireByLevel.Num();
		
		for (int32 Level = 0; Level < NumLevels; ++Level)
		{
			const float CurrentIndent = Indent * Level;

			if (NeedsWireByLevel[Level])
			{
				FSlateDrawElement::MakeBox(OutDrawElements
					, LayerId
					, AllottedGeometry.ToPaintGeometry(FVector2D(WireThickness, AllottedGeometry.Size.Y)
						, FSlateLayoutTransform(FVector2D(CurrentIndent - 3.f, 0)))
					, VerticalBarBrush
					, ESlateDrawEffect::None
					, WireColor);
			}
		}

		const float HalfCellHeight = 0.5f * AllottedGeometry.Size.Y;

		// For items that are the last expanded child in a list, we need to draw a special angle connector wire.
		if (const bool bIsLastChild = OwnerRow->IsLastChild())
		{
			const float CurrentIndent = Indent * (NumLevels - 1);
			
			FSlateDrawElement::MakeBox(OutDrawElements
				, LayerId
				, AllottedGeometry.ToPaintGeometry(FVector2D(WireThickness, HalfCellHeight + HalfWireThickness)
					, FSlateLayoutTransform(FVector2D(CurrentIndent - 3.f, 0)))
				, VerticalBarBrush
				, ESlateDrawEffect::None
				, WireColor);
		}

		// If this item is expanded, we need to draw a 1/2-height the line down to its first child cell.
		if (const bool bItemAppearsExpanded = OwnerRow->IsItemExpanded() && OwnerRow->DoesItemHaveChildren())
		{
			const float CurrentIndent = Indent * NumLevels;
			
			FSlateDrawElement::MakeBox(OutDrawElements
				, LayerId
				, AllottedGeometry.ToPaintGeometry(FVector2D(WireThickness, HalfCellHeight + HalfWireThickness)
					, FSlateLayoutTransform(FVector2D(CurrentIndent - 3.f, HalfCellHeight - HalfWireThickness)))
				, VerticalBarBrush
				, ESlateDrawEffect::None
				, WireColor);
		}

		// Draw horizontal connector from parent wire to child.
		{
			const float LeafDepth = OwnerRow->DoesItemHaveChildren()
				? 10.f
				: 0.0f;
			
			const float HorizontalWireStart = (NumLevels - 1) * Indent;
			
			FSlateDrawElement::MakeBox(OutDrawElements
				, LayerId
				, AllottedGeometry.ToPaintGeometry(FVector2D(AllottedGeometry.Size.X - HorizontalWireStart - WireThickness - LeafDepth, WireThickness)
					, FSlateLayoutTransform(FVector2D(HorizontalWireStart + WireThickness - 3.f, 0.5f * (AllottedGeometry.Size.Y - WireThickness))))
				, VerticalBarBrush
				, ESlateDrawEffect::None
				, WireColor);
		}
	}

	LayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	return LayerId;
}
