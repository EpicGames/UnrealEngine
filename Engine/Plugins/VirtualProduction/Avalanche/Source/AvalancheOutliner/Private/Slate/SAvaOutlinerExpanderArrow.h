// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SExpanderArrow.h"

class ITableRow;

class SAvaOutlinerExpanderArrow : public SExpanderArrow
{
public:
	SLATE_BEGIN_ARGS(SAvaOutlinerExpanderArrow) {}
		SLATE_ARGUMENT(SExpanderArrow::FArguments, ExpanderArrowArgs)
		SLATE_ATTRIBUTE(FLinearColor, WireTint)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<ITableRow>& TableRow);

protected:
	//~ Begin SWidget
	virtual int32 OnPaint(const FPaintArgs& Args
		, const FGeometry& AllottedGeometry
		, const FSlateRect& MyCullingRect
		, FSlateWindowElementList& OutDrawElements
		, int32 LayerId
		, const FWidgetStyle& InWidgetStyle
		, bool bParentEnabled) const override;
	//~ End SWidget
	
	TAttribute<FLinearColor> WireTint;
};
