// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaOutlinerItemFilters.h"
#include "AvaOutliner.h"
#include "AvaOutlinerStyle.h"
#include "AvaOutlinerView.h"
#include "Filters/IAvaOutlinerItemFilter.h"
#include "Framework/Application/SlateApplication.h"
#include "Internationalization/Text.h"
#include "Slate/SAvaOutliner.h"
#include "Styling/AvaOutlinerStyleUtils.h"
#include "Styling/SlateBrush.h"
#include "Styling/StyleColors.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SOverlay.h"

#define LOCTEXT_NAMESPACE "SAvaOutlinerItemFilters"

namespace UE::AvaOutliner::Private
{
	const FButtonStyle& GetFilterItemMenuButtonStyle()
	{
		static const FButtonStyle FilterItemMenuButtonStyle = FButtonStyle()
			   .SetNormal(FStyleUtils::GetColorBrush(EStyleType::Normal, true))
			   .SetHovered(FStyleUtils::GetColorBrush(EStyleType::Hovered, true))
			   .SetPressed(FStyleUtils::GetColorBrush(EStyleType::Pressed, true));
		return FilterItemMenuButtonStyle;
	}

	const FCheckBoxStyle& GetItemFilterCheckboxStyle()
	{
		static const FCheckBoxStyle ItemFilterCheckboxStyle = FCheckBoxStyle(FStyleUtils::GetSlimToolBarStyle().ToggleButton)
			   .SetPadding(FMargin(8.f, 4.f, 8.f, 4.f))
			   .SetCheckedImage(FStyleUtils::GetColorBrush(EStyleType::Normal, true))
			   .SetCheckedHoveredImage(FStyleUtils::GetColorBrush(EStyleType::Hovered, true))
			   .SetUncheckedHoveredImage(FStyleUtils::GetColorBrush(EStyleType::Hovered, false))
			   .SetCheckedPressedImage(FStyleUtils::GetColorBrush(EStyleType::Pressed, true))
			   .SetUncheckedPressedImage(FStyleUtils::GetColorBrush(EStyleType::Pressed, false));
		return ItemFilterCheckboxStyle;
	}
}

void SAvaOutlinerItemFilters::Construct(const FArguments& InArgs, const TSharedRef<FAvaOutlinerView>& InOutlinerView)
{
	using namespace UE::AvaOutliner::Private;

	OutlinerViewWeak = InOutlinerView;

	constexpr float SequenceDuration = 0.125f;
	ExpandFiltersSequence.AddCurve(0.f, SequenceDuration, ECurveEaseFunction::CubicInOut);

	if (const TSharedPtr<FAvaOutliner> Outliner = InOutlinerView->GetOutliner())
	{
		Outliner->OnOutlinerLoaded.AddSP(this, &SAvaOutlinerItemFilters::OnOutlinerLoaded);
	}

	TSharedRef<SVerticalBox> FilterVerticalPanel = SNew(SVerticalBox);

	ItemFilterScrollBox = SNew(SScrollBox)
		.Orientation(Orient_Vertical)
		.ScrollBarVisibility(EVisibility::Collapsed);

	FilterVerticalPanel->AddSlot()
		.AutoHeight()
		[
			SNew(SButton)
			.ButtonStyle(&GetFilterItemMenuButtonStyle())
			.ToolTipText(this, &SAvaOutlinerItemFilters::GetFilterItemMenuButtonToolTip)
			.OnClicked(this, &SAvaOutlinerItemFilters::ToggleShowItemFilters)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAvaOutlinerStyle::Get().GetBrush(TEXT("AvaOutliner.FilterIcon")))
					.DesiredSizeOverride(FVector2D(24.f))
				]
				+ SOverlay::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(this, &SAvaOutlinerItemFilters::GetFilterItemMenuButtonIcon)
					.DesiredSizeOverride(FVector2D(12.f))
					.RenderTransform(FSlateRenderTransform(FVector2f(2.f, 0.f)))
				]
			]
		];

	FilterVerticalPanel->AddSlot()
	   .AutoHeight()
	   .MaxHeight(3.f)
		[
			SNew(SColorBlock)
			.Color(FStyleUtils::GetColor(EStyleType::Normal, true).GetSpecifiedColor() * 0.75f)
		];

	//Add Item Filters
	for (const TSharedPtr<IAvaOutlinerItemFilter>& Filter : InOutlinerView->GetItemFilters())
	{
		AddItemFilterSlot(Filter, false);
	}

	InOutlinerView->GetOnCustomFiltersChanged().AddSP(this, &SAvaOutlinerItemFilters::UpdateCustomItemFilters);
	UpdateCustomItemFilters();

	FilterVerticalPanel->AddSlot()
		.FillHeight(1.f)
		[
			SAssignNew(ItemFilterBox, SBox)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.MaxWidth(4.f)
				[
					SNew(SColorBlock)
					.Color(FStyleUtils::GetColor(EStyleType::Normal, true).GetSpecifiedColor())
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					ItemFilterScrollBox.ToSharedRef()
				]
			]
		];

	FilterVerticalPanel->AddSlot()
		.AutoHeight()
		.Padding(1.f, 5.f, 1.f, 5.f)
		[
			SNew(SSeparator)
			.Orientation(Orient_Horizontal)
			.Thickness(FStyleUtils::GetSlimToolBarStyle().SeparatorThickness)
			.SeparatorImage(&FStyleUtils::GetSlimToolBarStyle().SeparatorBrush)
		];

	auto AddShortcut = [&FilterVerticalPanel, this](const FSlateBrush* Brush
		, FOnClicked&& OnClicked
		, FText&& Tooltip)
	{
		FilterVerticalPanel->AddSlot()
	       .AutoHeight()
	       .Padding(4.f, 2.f, 4.f, 2.f)
	       .HAlign(HAlign_Left)
	       .VAlign(VAlign_Top)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
				.OnClicked(MoveTemp(OnClicked))
				.ToolTipText(MoveTemp(Tooltip))
				[
					SNew(SImage)
					.Image(Brush)
					.DesiredSizeOverride(FVector2D(16.f))
				]
			];
	};

	AddShortcut(FAppStyle::GetBrush(TEXT("FoliageEditMode.SelectAll"))
		, FOnClicked::CreateSP(this, &SAvaOutlinerItemFilters::SelectAll)
		, LOCTEXT("SelectAll", "Selects All the Quick Type Filters"));

	AddShortcut(FAppStyle::GetBrush(TEXT("FoliageEditMode.DeselectAll"))
		, FOnClicked::CreateSP(this, &SAvaOutlinerItemFilters::DeselectAll)
		, LOCTEXT("DeselectAll", "Deselects All the Quick Type Filters"));

	ChildSlot
	.VAlign(VAlign_Top)
	.HAlign(HAlign_Fill)
	.Padding(0.f, 2.f, 0.f, 0.f)
	[
		FilterVerticalPanel
	];
}

SAvaOutlinerItemFilters::~SAvaOutlinerItemFilters()
{
	if (TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		OutlinerView->GetOnCustomFiltersChanged().RemoveAll(this);
	}
}

void SAvaOutlinerItemFilters::AddItemFilterSlot(const TSharedPtr<IAvaOutlinerItemFilter>& InFilter, bool bIsCustomSlot)
{
	if (!InFilter.IsValid())
	{
		return;
	}

	const FName FilterId = InFilter->GetFilterId();
	TMap<FName, TSharedPtr<SWidget>>& Slots = bIsCustomSlot ? CustomItemFilterSlots : ItemFilterSlots;
	
	if (TSharedPtr<SWidget>* const FoundExistingSlot = Slots.Find(FilterId))
	{
		if (FoundExistingSlot->IsValid())
		{
			ItemFilterScrollBox->RemoveSlot(FoundExistingSlot->ToSharedRef());	
		}
	}

	TSharedRef<SWidget> Slot = SNew(SCheckBox)
		.Style(&UE::AvaOutliner::Private::GetItemFilterCheckboxStyle())
		.ToolTipText(InFilter->GetTooltipText())
		.OnCheckStateChanged(this, &SAvaOutlinerItemFilters::OnCheckBoxStateChanged, InFilter)
		.IsChecked(this, &SAvaOutlinerItemFilters::IsChecked, InFilter)
		[
			SNew(SScaleBox)
			[
				SNew(SImage)
				.Image(InFilter->GetIconBrush())
				.DesiredSizeOverride(FVector2D(16.f))
			]
		];

	ItemFilterScrollBox->AddSlot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		[
			Slot
		];

	Slots.Add(FilterId, Slot);
}

void SAvaOutlinerItemFilters::UpdateCustomItemFilters()
{
	if (TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		check(ItemFilterScrollBox.IsValid());

		//Remove the existing custom filter slots
		for (const TPair<FName, TSharedPtr<SWidget>>& Pair : CustomItemFilterSlots)
		{
			if (Pair.Value.IsValid())
			{
				ItemFilterScrollBox->RemoveSlot(Pair.Value.ToSharedRef());
			}
		}

		const TArray<TSharedPtr<IAvaOutlinerItemFilter>>& CustomItemFilters = OutlinerView->GetCustomItemFilters();
		CustomItemFilterSlots.Empty(CustomItemFilters.Num());

		//Add the new slots for the updated custom filter list
		for (const TSharedPtr<IAvaOutlinerItemFilter>& CustomItemFilter : CustomItemFilters)
		{
			AddItemFilterSlot(CustomItemFilter, true);
		}
	}
}

void SAvaOutlinerItemFilters::OnOutlinerLoaded()
{
	if (const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		OnShowItemFiltersChanged(*OutlinerView);
	}
}

void SAvaOutlinerItemFilters::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	const bool bPlayingSequence = ExpandFiltersSequence.IsPlaying();

	if (bPlayingSequence)
	{
		const float Alpha = GetShowItemFiltersLerp();
		ItemFilterBox->SetRenderOpacity(Alpha);
		ItemFilterBox->SetHeightOverride(Alpha * ItemFilterBoxTargetHeight);
	}
	else if (bPlayedSequenceLastTick)
	{
		ItemFilterBox->SetRenderOpacity(static_cast<float>(bShowItemFilters));
		ItemFilterBox->SetHeightOverride(bShowItemFilters ? TAttribute<FOptionalSize>() : 0.f);
	}

	bPlayedSequenceLastTick = bPlayingSequence;
}

FSlateColor SAvaOutlinerItemFilters::GetFilterStateColor(TSharedPtr<IAvaOutlinerItemFilter> Filter) const
{
	if (OutlinerViewWeak.IsValid() && OutlinerViewWeak.Pin()->IsItemFilterEnabled(Filter))
	{
		return FSlateColor(FLinearColor(0.701f, 0.225f, 0.003f));
	}
	return FSlateColor::UseForeground();
}

ECheckBoxState SAvaOutlinerItemFilters::IsChecked(TSharedPtr<IAvaOutlinerItemFilter> Filter) const
{
	if (OutlinerViewWeak.IsValid() && OutlinerViewWeak.Pin()->IsItemFilterEnabled(Filter))
	{
		return ECheckBoxState::Checked;
	}
	return ECheckBoxState::Unchecked;
}

void SAvaOutlinerItemFilters::OnCheckBoxStateChanged(ECheckBoxState CheckBoxState, TSharedPtr<IAvaOutlinerItemFilter> Filter) const
{
	if (const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		if (CheckBoxState == ECheckBoxState::Checked)
		{
			OutlinerView->EnableItemFilter(Filter);
		}
		else
		{
			OutlinerView->DisableItemFilter(Filter);
		}
	}
}

float SAvaOutlinerItemFilters::GetShowItemFiltersLerp() const
{
	if (ExpandFiltersSequence.IsPlaying())
	{
		return ExpandFiltersSequence.GetLerp();
	}
	return static_cast<float>(bShowItemFilters);
}

void SAvaOutlinerItemFilters::OnShowItemFiltersChanged(const FAvaOutlinerView& InOutlinerView)
{
	bShowItemFilters = InOutlinerView.ShouldShowItemFilters();
}

FReply SAvaOutlinerItemFilters::ToggleShowItemFilters()
{
	if (const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
		{
			for (const TSharedPtr<IAvaOutlinerItemFilter>& Filter : OutlinerView->GetItemFilters())
			{
				OutlinerView->DisableItemFilter(Filter, false);
			}
			for (const TSharedPtr<IAvaOutlinerItemFilter>& Filter : OutlinerView->GetCustomItemFilters())
			{
				OutlinerView->DisableItemFilter(Filter, false);
			}

			OutlinerView->Refresh();

			return FReply::Handled();
		}

		OutlinerView->ToggleShowItemFilters();
		OnShowItemFiltersChanged(*OutlinerView);

		const float ViewFraction = ItemFilterScrollBox->GetViewFraction();

		if (ViewFraction > 0.f)
		{
			ItemFilterBoxTargetHeight = ItemFilterScrollBox->GetDesiredSize().Y / ViewFraction;
		}
		else
		{
			ItemFilterBoxTargetHeight = 0.f;
		}

		if (OutlinerView->ShouldShowItemFilters())
		{
			ExpandFiltersSequence.Play(SharedThis(this));
		}
		else
		{
			ExpandFiltersSequence.PlayReverse(SharedThis(this));
		}

		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SAvaOutlinerItemFilters::SelectAll()
{
	if (const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		for (const TSharedPtr<IAvaOutlinerItemFilter>& Filter : OutlinerView->GetItemFilters())
		{
			OutlinerView->EnableItemFilter(Filter);
		}
		for (const TSharedPtr<IAvaOutlinerItemFilter>& Filter : OutlinerView->GetCustomItemFilters())
		{
			OutlinerView->EnableItemFilter(Filter);
		}
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SAvaOutlinerItemFilters::DeselectAll()
{
	if (const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		for (const TSharedPtr<IAvaOutlinerItemFilter>& Filter : OutlinerView->GetItemFilters())
		{
			OutlinerView->DisableItemFilter(Filter);
		}
		for (const TSharedPtr<IAvaOutlinerItemFilter>& Filter : OutlinerView->GetCustomItemFilters())
		{
			OutlinerView->DisableItemFilter(Filter);
		}
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FText SAvaOutlinerItemFilters::GetFilterItemMenuButtonToolTip() const
{
	if (const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		FText FilterText;

		for (const TSharedPtr<IAvaOutlinerItemFilter>& Filter : OutlinerView->GetActiveItemFilters())
		{
			if (OutlinerView->IsItemFilterEnabled(Filter))
			{
				FilterText = FText::Format(LOCTEXT("ItemFilterText", "{0}\n  {1}"), FilterText, Filter->GetTooltipText());
			}
		}

		return FilterText.IsEmpty()
			? LOCTEXT("NoActiveFiltersTooltip", "No active filters")
			: FText::Format(LOCTEXT("CustomItemFilterText", "Active Filters:{0}\n\n"
				"Shift + Click to clear all active filters"), FilterText);
	}

	return FText();
}

const FSlateBrush* SAvaOutlinerItemFilters::GetFilterItemMenuButtonIcon() const
{
	if (const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		const TSet<TSharedPtr<IAvaOutlinerItemFilter>>& ActiveFilters = OutlinerView->GetActiveItemFilters();
		const int32 ActiveFilterCount = ActiveFilters.Num();

		const FSlateBrush* OutBrush = nullptr;

		for (const TSharedPtr<IAvaOutlinerItemFilter>& ItemFilter : ActiveFilters)
		{
			if (OutBrush)
			{
				OutBrush = FAppStyle::GetBrush(TEXT("MessageLog.Warning"));
				break;
			}
			else
			{
				OutBrush = ItemFilter->GetIconBrush();
			}
		}

		return OutBrush;
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
