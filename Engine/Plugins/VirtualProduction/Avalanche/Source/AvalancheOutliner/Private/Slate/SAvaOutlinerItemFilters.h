// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/CurveSequence.h"
#include "SCheckBoxList.h"
#include "Widgets/SCompoundWidget.h"

class FAvaOutlinerView;
class FText;
class IAvaOutlinerItemFilter;
class SAvaOutlinerItemFilters;
class SBox;
class SScrollBox;
struct FSlateBrush;

class SAvaOutlinerItemFilters : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaOutlinerItemFilters) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FAvaOutlinerView>& InOutlinerView);

	virtual ~SAvaOutlinerItemFilters() override;

	/** Adds the Filter */
	void AddItemFilterSlot(const TSharedPtr<IAvaOutlinerItemFilter>& InFilter, bool bIsCustomSlot);

	void UpdateCustomItemFilters();

	void OnOutlinerLoaded();

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	FSlateColor GetFilterStateColor(TSharedPtr<IAvaOutlinerItemFilter> Filter) const;

	ECheckBoxState IsChecked(TSharedPtr<IAvaOutlinerItemFilter> Filter) const;
	void OnCheckBoxStateChanged(ECheckBoxState CheckBoxState, TSharedPtr<IAvaOutlinerItemFilter> Filter) const;

	float GetShowItemFiltersLerp() const;
	void OnShowItemFiltersChanged(const FAvaOutlinerView& InOutlinerView);

	FReply ToggleShowItemFilters();

	FReply SelectAll();
	FReply DeselectAll();

private:
	FText GetFilterItemMenuButtonToolTip() const;
	const FSlateBrush* GetFilterItemMenuButtonIcon() const;

	TWeakPtr<FAvaOutlinerView> OutlinerViewWeak;

	TSharedPtr<SBox> ItemFilterBox;

	TSharedPtr<SScrollBox> ItemFilterScrollBox;

	TMap<FName, TSharedPtr<SWidget>> ItemFilterSlots;

	/**
	 * Map of Filter Ids and their Slot.
	 * This is a separate map purely to allow users to use filter-ids of natively constructed item filters
	 * as they are not exposed
	 */
	TMap<FName, TSharedPtr<SWidget>> CustomItemFilterSlots;

	FCurveSequence ExpandFiltersSequence;

	/** Target Height of the Item Filter Box when playing the sequence */
	float ItemFilterBoxTargetHeight = 0.f;

	/** Whether to expand and show the Item Filter List */
	bool bShowItemFilters = false;

	/** The cached state of Expand Filters Sequence, to know when states have changed. Default to true so that we run it at the start  */
	bool bPlayedSequenceLastTick = true;
};
