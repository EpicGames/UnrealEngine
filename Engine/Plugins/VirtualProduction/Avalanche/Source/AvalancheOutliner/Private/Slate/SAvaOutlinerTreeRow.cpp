// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SAvaOutlinerTreeRow.h"
#include "AvaOutlinerStyle.h"
#include "AvaOutlinerView.h"
#include "Columns/IAvaOutlinerColumn.h"
#include "Item/IAvaOutlinerItem.h"
#include "Slate/SAvaOutliner.h"
#include "Slate/SAvaOutlinerTreeView.h"

#define LOCTEXT_NAMESPACE "SAvaOutlinerTreeRow"

void SAvaOutlinerTreeRow::Construct(const FArguments& InArgs
	, const TSharedRef<FAvaOutlinerView>& InOutlinerView
	, const TSharedPtr<SAvaOutlinerTreeView>& InTreeView
	, const FAvaOutlinerItemPtr& InItem)
{
	OutlinerViewWeak = InOutlinerView;
	TreeViewWeak     = InTreeView;
	Item             = InItem;
	HighlightText    = InArgs._HighlightText;

	SetColorAndOpacity(TAttribute<FLinearColor>::CreateSP(&*InOutlinerView, &FAvaOutlinerView::GetItemBrushColor, Item));

	SMultiColumnTableRow::Construct(FSuperRowType::FArguments()
			.Style(&FAvaOutlinerStyle::Get().GetWidgetStyle<FTableRowStyle>("AvaOutliner.TableViewRow"))
			.OnCanAcceptDrop(InOutlinerView, &FAvaOutlinerView::OnCanDrop)
			.OnDragDetected(InOutlinerView, &FAvaOutlinerView::OnDragDetected, Item)
			.OnDragEnter(InOutlinerView, &FAvaOutlinerView::OnDragEnter, Item)
			.OnDragLeave(InOutlinerView, &FAvaOutlinerView::OnDragLeave, Item)
			.OnAcceptDrop(InOutlinerView, &FAvaOutlinerView::OnDrop)
			.OnDrop(this, &SAvaOutlinerTreeRow::OnDefaultDrop)
		, InTreeView.ToSharedRef());
}

TSharedRef<SWidget> SAvaOutlinerTreeRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	if (Item.IsValid())
	{
		const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin();
		check(OutlinerView.IsValid());
		
		if (const TSharedPtr<IAvaOutlinerColumn> Column = OutlinerView->GetColumns().FindRef(InColumnName))
		{
			return Column->ConstructRowWidget(Item.ToSharedRef(), OutlinerView.ToSharedRef(), SharedThis(this));
		}
	}
	return SNullWidget::NullWidget;
}

FReply SAvaOutlinerTreeRow::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin();

	if (!OutlinerView.IsValid())
	{
		return FReply::Unhandled();
	}

	//Select Item and the Tree of Children it contains
	if (MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton)
	{
		EAvaOutlinerItemSelectionFlags Flags = EAvaOutlinerItemSelectionFlags::IncludeChildren
			| EAvaOutlinerItemSelectionFlags::SignalSelectionChange;

		if (MouseEvent.IsControlDown())
		{
			Flags |= EAvaOutlinerItemSelectionFlags::AppendToCurrentSelection;
		}
		
		OutlinerView->SelectItems({Item}, Flags);

		return FReply::Handled();
	}
	
	return SMultiColumnTableRow::OnMouseButtonUp(MyGeometry, MouseEvent);
}

TSharedPtr<FAvaOutlinerView> SAvaOutlinerTreeRow::GetOutlinerView() const
{
	return OutlinerViewWeak.Pin();
}

FReply SAvaOutlinerTreeRow::OnDefaultDrop(const FDragDropEvent& InDragDropEvent) const
{
	if (TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		OutlinerView->SetDragIntoTreeRoot(false);
	}

	// Always return handled as no action should take place if the Drop wasn't accepted
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
