// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/STableRow.h"

class SAvaOutliner;
class FAvaOutlinerView;
class SAvaOutlinerTreeView;

class SAvaOutlinerTreeRow : public SMultiColumnTableRow<FAvaOutlinerItemPtr>
{
public:
	SLATE_BEGIN_ARGS(SAvaOutlinerTreeRow) {}
		SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const TSharedRef<FAvaOutlinerView>& InOutlinerView
		, const TSharedPtr<SAvaOutlinerTreeView>& InTreeView
		, const FAvaOutlinerItemPtr& InItem);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

	TAttribute<FText> GetHighlightText() const { return HighlightText; }
	
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	
	TSharedPtr<FAvaOutlinerView> GetOutlinerView() const;
	
	const FTableRowStyle*  GetStyle() const { return Style; }

	/** The default reply if a row did not handle AcceptDrop */
	FReply OnDefaultDrop(const FDragDropEvent& InDragDropEvent) const;

private:
	FAvaOutlinerItemPtr Item;
	
	TWeakPtr<SAvaOutlinerTreeView> TreeViewWeak;
	
	TWeakPtr<FAvaOutlinerView> OutlinerViewWeak;
	
	TAttribute<FText> HighlightText;
};
