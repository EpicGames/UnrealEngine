// Copyright Epic Games, Inc. All Rights Reserved.

#include "Slate/SAvaOutlinerTreeView.h"
#include "AvaOutlinerView.h"
#include "Item/IAvaOutlinerItem.h"
#include "Framework/Commands/UICommandList.h"

void SAvaOutlinerTreeView::Construct(const FArguments& InArgs, const TSharedPtr<FAvaOutlinerView>& InOutlinerView)
{
	OutlinerViewWeak = InOutlinerView;
	STreeView::Construct(InArgs._TreeViewArgs);
}

int32 SAvaOutlinerTreeView::GetItemIndex(FAvaOutlinerItemPtr Item) const
{
	if (SListView<FAvaOutlinerItemPtr>::HasValidItemsSource())
	{
		return SListView<FAvaOutlinerItemPtr>::GetItems().Find(TListTypeTraits<FAvaOutlinerItemPtr>::NullableItemTypeConvertToItemType(Item));
	}
	return INDEX_NONE;
}

void SAvaOutlinerTreeView::FocusOnItem(const FAvaOutlinerItemPtr& InItem)
{
	SelectorItem = InItem;
	RangeSelectionStart = InItem;
}

void SAvaOutlinerTreeView::ScrollItemIntoView(const FAvaOutlinerItemPtr& InItem)
{
	RequestScrollIntoView(InItem, 0);
}

void SAvaOutlinerTreeView::UpdateItemExpansions(FAvaOutlinerItemPtr InItem)
{
	if (InItem.IsValid())
	{
		const FSparseItemInfo* const SparseItemInfo = SparseItemInfos.Find(InItem);

		bool bIsExpanded = false;
		bool bHasExpandedChildren = false;

		if (SparseItemInfo)
		{
			bIsExpanded = SparseItemInfo->bIsExpanded;
			bHasExpandedChildren = SparseItemInfo->bHasExpandedChildren;
		}

		//Skip to avoid redundancy
		if (bIsExpanded || bHasExpandedChildren)
		{
			return;
		}

		TArray<FAvaOutlinerItemPtr> ItemsToCheck = InItem->GetChildren();
		while (ItemsToCheck.Num() > 0)
		{
			FAvaOutlinerItemPtr ItemToCheck = ItemsToCheck.Pop();

			if (ItemToCheck.IsValid())
			{
				if (IsItemExpanded(ItemToCheck))
				{
					bHasExpandedChildren = true;
					break;
				}
				ItemsToCheck.Append(ItemToCheck->GetChildren());
			}
		}

		if (bIsExpanded || bHasExpandedChildren)
		{
			SparseItemInfos.Add(InItem, FSparseItemInfo(bIsExpanded, bHasExpandedChildren));
		}
	}
}

void SAvaOutlinerTreeView::Private_SetItemSelection(FAvaOutlinerItemPtr InItem, bool bShouldBeSelected, bool bWasUserDirected)
{
	const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin();
	if (OutlinerView.IsValid() && !OutlinerView->IsOutlinerLocked() && OutlinerView->CanSelectItem(InItem))
	{
		STreeView::Private_SetItemSelection(InItem, bShouldBeSelected, bWasUserDirected);
	}
}

void SAvaOutlinerTreeView::Private_ClearSelection()
{
	const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin();
	if (OutlinerView.IsValid() && !OutlinerView->IsOutlinerLocked())
	{
		STreeView::Private_ClearSelection();
	}
}

void SAvaOutlinerTreeView::Private_SignalSelectionChanged(ESelectInfo::Type SelectInfo)
{
	STreeView::Private_SignalSelectionChanged(SelectInfo);

	const TItemSet AddedItems = SelectedItems.Difference(PreviousSelectedItems);
	for (const FAvaOutlinerItemPtr& AddedItem : AddedItems)
	{
		if (AddedItem.IsValid())
		{
			AddedItem->OnItemSelectionChanged(true);
		}
	}

	const TItemSet RemovedItems = PreviousSelectedItems.Difference(SelectedItems);
	for (const FAvaOutlinerItemPtr& RemovedItem : RemovedItems)
	{
		if (RemovedItem.IsValid())
		{
			RemovedItem->OnItemSelectionChanged(false);
		}
	}
	PreviousSelectedItems = SelectedItems;
}

void SAvaOutlinerTreeView::Private_UpdateParentHighlights()
{
	this->Private_ClearHighlightedItems();
	for (TItemSet::TConstIterator SelectedItemIt(this->SelectedItems); SelectedItemIt; ++SelectedItemIt)
	{
		FAvaOutlinerItemPtr SelectedItem(*SelectedItemIt);

		// Sometimes selection events can come through before the Linearized List is built, so the item may not exist yet.
		const int32 ItemIndex = LinearizedItems.Find(SelectedItem);
		if (ItemIndex == INDEX_NONE)
		{
			if (SelectedItem.IsValid())
			{
				FAvaOutlinerItemPtr ParentItem = SelectedItem->GetParent();
				while (ParentItem.IsValid())
				{
					if (LinearizedItems.Contains(ParentItem))
					{
						this->SetItemHighlighted(ParentItem, true);
					}
					ParentItem = ParentItem->GetParent();
				}
			}
			continue;
		}

		if (DenseItemInfos.IsValidIndex(ItemIndex))
		{
			const FItemInfo& ItemInfo = DenseItemInfos[ItemIndex];
			
			int32 ParentIndex = ItemInfo.ParentIndex;
			
			while (ParentIndex != INDEX_NONE)
			{
				const FAvaOutlinerItemPtr& ParentItem = this->LinearizedItems[ParentIndex];
				this->Private_SetItemHighlighted(ParentItem, true);

				const FItemInfo& ParentItemInfo = DenseItemInfos[ParentIndex];
				
				ParentIndex = ParentItemInfo.ParentIndex;
			}
		}
	}
}

FCursorReply SAvaOutlinerTreeView::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	if (IsRightClickScrolling() && CursorEvent.IsMouseButtonDown(EKeys::MiddleMouseButton))
	{
		// We hide the native cursor as we'll be drawing the software EMouseCursor::GrabHandClosed cursor
		return FCursorReply::Cursor(EMouseCursor::None);
	}
	return STreeView::OnCursorQuery(MyGeometry, CursorEvent);
}

FReply SAvaOutlinerTreeView::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton) && !MouseEvent.IsTouchEvent())
	{
		// We only care about deltas along the scroll axis
		FTableViewDimensions CursorDeltaDimensions(Orientation, MouseEvent.GetCursorDelta());
		CursorDeltaDimensions.LineAxis = 0.f;

		const float ScrollByAmount = CursorDeltaDimensions.ScrollAxis / MyGeometry.Scale;

		// If scrolling with the right mouse button, we need to remember how much we scrolled.
		// If we did not scroll at all, we will bring up the context menu when the mouse is released.
		AmountScrolledWhileRightMouseDown += FMath::Abs(ScrollByAmount);

		// Has the mouse moved far enough with the right mouse button held down to start capturing
		// the mouse and dragging the view?
		if (IsRightClickScrolling())
		{
			// Make sure the active timer is registered to update the inertial scroll
			if (!bIsScrollingActiveTimerRegistered)
			{
				bIsScrollingActiveTimerRegistered = true;
				RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SAvaOutlinerTreeView::UpdateInertialScroll));
			}

			TickScrollDelta -= ScrollByAmount;

			const float AmountScrolled = this->ScrollBy(MyGeometry, -ScrollByAmount, AllowOverscroll);

			FReply Reply = FReply::Handled();

			// The mouse moved enough that we're now dragging the view. Capture the mouse
			// so the user does not have to stay within the bounds of the list while dragging.
			if (this->HasMouseCapture() == false)
			{
				Reply.CaptureMouse(AsShared()).UseHighPrecisionMouseMovement(AsShared());
				SoftwareCursorPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
				bShowSoftwareCursor    = true;
			}

			// Check if the mouse has moved.
			if (AmountScrolled != 0)
			{
				SoftwareCursorPosition += CursorDeltaDimensions.ToVector2D();
			}

			return Reply;
		}
	}

	return STreeView::OnMouseMove(MyGeometry, MouseEvent);
}

FReply SAvaOutlinerTreeView::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton)
	{
		FReply Reply = FReply::Handled().ReleaseMouseCapture();
		AmountScrolledWhileRightMouseDown = 0;
		bShowSoftwareCursor = false;

		// If we have mouse capture, snap the mouse back to the closest location that is within the list's bounds
		if (HasMouseCapture())
		{
			const FSlateRect ListScreenSpaceRect = MyGeometry.GetLayoutBoundingRect();
			const FVector2D CursorPosition = MyGeometry.LocalToAbsolute(SoftwareCursorPosition);

			const FIntPoint BestPositionInList(
					FMath::RoundToInt(FMath::Clamp(CursorPosition.X, ListScreenSpaceRect.Left, ListScreenSpaceRect.Right)),
					FMath::RoundToInt(FMath::Clamp(CursorPosition.Y, ListScreenSpaceRect.Top, ListScreenSpaceRect.Bottom))
				);

			Reply.SetMousePos(BestPositionInList);
		}

		return Reply;
	}
	return STreeView::OnMouseButtonUp(MyGeometry, MouseEvent);
}

FReply SAvaOutlinerTreeView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		OutlinerView->UpdateRecentOutlinerViews();

		TSharedPtr<FUICommandList> CommandList = OutlinerView->GetViewCommandList();
		if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();	
		}
	}
	return STreeView::OnKeyDown(MyGeometry, InKeyEvent);
}
