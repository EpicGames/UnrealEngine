// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Widgets/Views/STreeView.h"

class FAvaOutlinerView;

class SAvaOutlinerTreeView : public STreeView<FAvaOutlinerItemPtr>
{
public:
	SLATE_BEGIN_ARGS(SAvaOutlinerTreeView) {}
		SLATE_ARGUMENT(STreeView<FAvaOutlinerItemPtr>::FArguments, TreeViewArgs)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<FAvaOutlinerView>& InOutlinerView);

	int32 GetItemIndex(FAvaOutlinerItemPtr Item) const;

	void FocusOnItem(const FAvaOutlinerItemPtr& InItem);

	void ScrollItemIntoView(const FAvaOutlinerItemPtr& InItem);

	void UpdateItemExpansions(FAvaOutlinerItemPtr InItem);

	//~ Begin STreeView
	virtual void Private_UpdateParentHighlights() override;
	//~ End STreeView

	//~ Begin ITypedTableView
	virtual void Private_SetItemSelection(FAvaOutlinerItemPtr InItem, bool bShouldBeSelected, bool bWasUserDirected = false) override;
	virtual void Private_SignalSelectionChanged(ESelectInfo::Type SelectInfo) override;
	virtual void Private_ClearSelection() override;
	//~ End SListView

	//~ Begin SWidget
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget

private:
	TWeakPtr<FAvaOutlinerView> OutlinerViewWeak;
	
	TItemSet PreviousSelectedItems;
};
