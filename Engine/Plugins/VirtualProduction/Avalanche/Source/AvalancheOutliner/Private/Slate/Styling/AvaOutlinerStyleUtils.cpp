// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerStyleUtils.h"
#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateNoResource.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/ToolBarStyle.h"

namespace UE::AvaOutliner::Private
{
	const FToolBarStyle& FStyleUtils::GetSlimToolBarStyle()
	{
		static const FToolBarStyle& SlimToolBarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>(TEXT("SlimToolBar"));
		return SlimToolBarStyle;
	}
	
	const FSlateBrush& FStyleUtils::GetBrush(EStyleType InStyleType, bool bIsSelected)
	{
		const FCheckBoxStyle& Style = GetSlimToolBarStyle().ToggleButton;
		
		switch (InStyleType)
		{
		case EStyleType::Normal:
			return bIsSelected ? Style.CheckedImage : Style.UncheckedImage;
		
		case EStyleType::Hovered:
			return bIsSelected ? Style.CheckedHoveredImage : Style.UncheckedHoveredImage;
		
		case EStyleType::Pressed:
			return bIsSelected ? Style.CheckedPressedImage : Style.UncheckedPressedImage;
		
		default:
			break;
		}
		
		static const FSlateNoResource NullBrush;
		return NullBrush;
	}
	
	FSlateColor FStyleUtils::GetColor(EStyleType InStyleType, bool bIsSelected)
	{
		return GetBrush(InStyleType, bIsSelected).TintColor;
	}
	
	FSlateColorBrush FStyleUtils::GetColorBrush(EStyleType InStyleType, bool bIsSelected)
	{
		return FSlateColorBrush(GetColor(InStyleType, bIsSelected));
	}
}
