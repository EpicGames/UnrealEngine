// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

struct FButtonStyle;
struct FCheckBoxStyle;
struct FSlateBrush;
struct FSlateColor;
struct FSlateColorBrush;
struct FToolBarStyle;

namespace UE::AvaOutliner::Private
{
	enum class EStyleType
	{
		Normal,
		Hovered,
		Pressed,
	};
	
	struct FStyleUtils
	{
		static const FToolBarStyle& GetSlimToolBarStyle();
		
		static const FSlateBrush& GetBrush(EStyleType InStyleType, bool bIsSelected);
		
		static FSlateColor GetColor(EStyleType InStyleType, bool bIsSelected);
		
		static FSlateColorBrush GetColorBrush(EStyleType InStyleType, bool bIsSelected);
	};
}
