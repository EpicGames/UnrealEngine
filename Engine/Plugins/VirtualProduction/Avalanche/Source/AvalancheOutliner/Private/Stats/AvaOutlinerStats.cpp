// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerStats.h"
#include "AvaOutlinerView.h"

void FAvaOutlinerStats::Tick(FAvaOutlinerView& InOutlinerView)
{
	if (DirtyStatTypes.Num() > 0)
	{
		//Selected Items
		if (DirtyStatTypes.Contains(EAvaOutlinerStatCountType::SelectedItemCount))
		{
			SetStatCount(EAvaOutlinerStatCountType::SelectedItemCount, InOutlinerView.GetViewSelectedItemCount());
		}

		//Visible Items
		if (DirtyStatTypes.Contains(EAvaOutlinerStatCountType::VisibleItemCount))
		{
			SetStatCount(EAvaOutlinerStatCountType::VisibleItemCount, InOutlinerView.CalculateVisibleItemCount());
		}

		DirtyStatTypes.Reset();
	}
}

void FAvaOutlinerStats::MarkCountTypeDirty(EAvaOutlinerStatCountType CountType)
{
	DirtyStatTypes.Add(CountType);
}

void FAvaOutlinerStats::SetStatCount(EAvaOutlinerStatCountType CountType, int32 Count)
{
	StatCount.Add(CountType, Count);
}

int32 FAvaOutlinerStats::GetStatCount(EAvaOutlinerStatCountType CountType) const
{
	if (const int32* const Count = StatCount.Find(CountType))
	{
		return *Count;
	}
	return 0;
}
