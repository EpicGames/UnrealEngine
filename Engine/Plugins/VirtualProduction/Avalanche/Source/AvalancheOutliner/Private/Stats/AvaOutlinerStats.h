// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Item/IAvaOutlinerItem.h"

class FAvaOutlinerView;

enum class EAvaOutlinerStatCountType : uint8
{
	None,
	/** Number of items that are present in the Outliner View (not to be confused with Actor/Component Visibility) */
	VisibleItemCount,
	/** Number of items that are selected in the Outliner View */
	SelectedItemCount,
};

/**
 * Responsible for holding Stat information for an Outliner View
 */
struct FAvaOutlinerStats
{
	/**
	 * Updates to check whether a given stat type is dirty.
	 * Even though this is Ticked every frame it will only do work for the dirty stats
	 */
	void Tick(FAvaOutlinerView& InOutlinerView);

	/** Dirties a given count type */
	void MarkCountTypeDirty(EAvaOutlinerStatCountType CountType);

	/** Sets the count for a given countable stat type */
	void SetStatCount(EAvaOutlinerStatCountType CountType, int32 Count);

	/** Gets the count for a given countable stat type */
	int32 GetStatCount(EAvaOutlinerStatCountType CountType) const;

private:
	/** Map of a Countable Stat Type and the current count it has in this Instance */
	TMap<EAvaOutlinerStatCountType, int32> StatCount;

	/** The Count Types that have changed and must be modified on next Tick */
	TSet<EAvaOutlinerStatCountType> DirtyStatTypes;
};
