// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaOutlinerStats.h"
#include "AvaOutlinerView.h"
#include "Stats/AvaOutlinerStats.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaOutlinerStats"

void SAvaOutlinerStats::Construct(const FArguments& InArgs, const TSharedRef<FAvaOutlinerView>& InOutlinerView)
{
	OutlinerViewWeak = InOutlinerView;
	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

	auto AddStatBlock = [&HorizontalBox, this](const FText& Title, EAvaOutlinerStatCountType CountType)
	{
		HorizontalBox->AddSlot()
			.AutoWidth()
			.Padding(3.f, 0.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f, 0.f)
				[
					SNew(STextBlock)
					.Text(this, &SAvaOutlinerStats::GetStatsCount, CountType)
					.Justification(ETextJustify::Center)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5.f, 0.f)
				[
					SNew(STextBlock)
					.Text(Title)
					.Justification(ETextJustify::Left)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5.f, 0.f)
				[
					SNew(SSeparator)
					.Orientation(EOrientation::Orient_Vertical)
					.Thickness(1.f)
				]
			];
	};

	AddStatBlock(LOCTEXT("VisibleItemCount", "Visible Items")
		, EAvaOutlinerStatCountType::VisibleItemCount);

	AddStatBlock(LOCTEXT("SelectedItemCount", "Selected Items")
		, EAvaOutlinerStatCountType::SelectedItemCount);

	HorizontalBox->AddSlot()
		.FillWidth(1.f)
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Right)
		.Padding(0.f, 0.f, 2.f, 0.f)
		[
			SNew(SHorizontalBox)
			.Visibility(this, &SAvaOutlinerStats::GetSelectionNavigationVisibility)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Fill)
			[
				SNew(SButton)
				.Text(LOCTEXT("Previous", "Previous"))
				.ButtonStyle(FAppStyle::Get(), "RoundButton")
				.VAlign(VAlign_Center)
				.ContentPadding(0.f)
				.OnClicked(this, &SAvaOutlinerStats::ScrollToPreviousSelection)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(3.f, 0.f)
			[
				SNew(SSeparator)
				.Orientation(EOrientation::Orient_Vertical)
				.Thickness(1.f)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Fill)
			[
				SNew(SButton)
				.Text(LOCTEXT("Next", "Next"))
				.ButtonStyle(FAppStyle::Get(), "RoundButton")
				.VAlign(VAlign_Center)
				.ContentPadding(0.f)
				.OnClicked(this, &SAvaOutlinerStats::ScrollToNextSelection)
			]
		];

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Brushes.Header"))
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill)
		.Padding(2.f)
		[
			HorizontalBox
		]
	];
}

FText SAvaOutlinerStats::GetStatsCount(EAvaOutlinerStatCountType CountType) const
{
	if (const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		return FText::AsNumber(OutlinerView->GetOutlinerStats()->GetStatCount(CountType));
	}
	return FText::AsNumber(0);
}

FReply SAvaOutlinerStats::ScrollToPreviousSelection() const
{
	if (const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		OutlinerView->ScrollPrevIntoView();
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SAvaOutlinerStats::ScrollToNextSelection() const
{
	if (const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin())
	{
		OutlinerView->ScrollNextIntoView();
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

EVisibility SAvaOutlinerStats::GetSelectionNavigationVisibility() const
{
	const TSharedPtr<FAvaOutlinerView> OutlinerView = OutlinerViewWeak.Pin();
	if (OutlinerView.IsValid() && OutlinerView->GetViewSelectedItemCount() > 0)
	{
		return EVisibility::SelfHitTestInvisible;
	}
	return EVisibility::Hidden;
}

#undef LOCTEXT_NAMESPACE
