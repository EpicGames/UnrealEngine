// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/AvaOutlinerStats.h"
#include "Widgets/SCompoundWidget.h"

class FAvaOutlinerView;

class SAvaOutlinerStats : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaOutlinerStats) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FAvaOutlinerView>& InOutlinerView);

	FText GetStatsCount(EAvaOutlinerStatCountType CountType) const;

	FReply ScrollToPreviousSelection() const;
	
	FReply ScrollToNextSelection() const;

	EVisibility GetSelectionNavigationVisibility() const;

private:
	TWeakPtr<FAvaOutlinerView> OutlinerViewWeak;
};
