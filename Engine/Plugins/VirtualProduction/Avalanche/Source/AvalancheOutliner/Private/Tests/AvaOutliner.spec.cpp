// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutliner.h"
#include "AvaOutlinerView.h"
#include "AvaOutlinerTestUtils.h"
#include "Engine/DirectionalLight.h"
#include "Engine/StaticMeshActor.h"
#include "EngineUtils.h"
#include "Item/AvaOutlinerActor.h"
#include "Misc/AutomationTest.h"

#if WITH_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FAvaOutlinerSpec, "Avalanche.Outliner",
	EAutomationTestFlags::EngineFilter
	| EAutomationTestFlags::EditorContext
	| EAutomationTestFlags_ApplicationContextMask)

	enum class EWhenType
{
	NoItemsSelected,
	ItemsSelected
};

void When(const FString& InDescription, TFunction<void()>&& DoWork)
{
	Describe("When " + InDescription, MoveTemp(DoWork));
}

void When(EWhenType InWhenType, TFunction<void()>&& DoWork)
{
	switch (InWhenType)
	{
	case EWhenType::NoItemsSelected:
		When("no items are selected", MoveTemp(DoWork));
		break;

	case EWhenType::ItemsSelected:
		When("multiple items are selected", [this, DoWork = MoveTemp(DoWork)]()
			{
				BeforeEach([this]()
					{
						const bool bAllValid = !TestItems.ContainsByPredicate([](const FAvaOutlinerItemPtr& Item)
							{
								return !Item.IsValid();
							});

						if (TestItems.Num() > 0 && bAllValid)
						{
							OutlinerView->SelectItems(TestItems);
						}
					});
				DoWork();
			});
		break;
	}
}

TSharedPtr<FAvaOutliner> Outliner;
TSharedPtr<FAvaOutlinerView> OutlinerView;
TSharedPtr<UE::AvaOutliner::Private::FAvaOutlinerProviderTest> OutlinerProvider;
TArray<FAvaOutlinerItemPtr> TestItems;

END_DEFINE_SPEC(FAvaOutlinerSpec)

void FAvaOutlinerSpec::Define()
{
	BeforeEach([this]()
		{
			TestItems.Reset();
			OutlinerView.Reset();
			Outliner.Reset();
			OutlinerProvider.Reset();

			OutlinerProvider = MakeShared<UE::AvaOutliner::Private::FAvaOutlinerProviderTest>();

			constexpr int32 OutlinerViewId = 0;
			Outliner = OutlinerProvider->GetOutliner();
			check(Outliner.IsValid());

			Outliner->RegisterOutlinerView(OutlinerViewId);
			Outliner->Refresh();

			OutlinerView = Outliner->GetMostRecentOutlinerView();
			check(OutlinerView.IsValid());

			const TArray<UObject*> TestObjects = {
				OutlinerProvider->GetDirectionalLight(),
				OutlinerProvider->GetFloor(),
				OutlinerProvider->GetSkySphere()
			};

			TestItems = OutlinerProvider->GetOutlinerItems(TestObjects);
		});

#if 0 // DISABLED: Flaky in CI — temporarily disable
	Describe("Spawning an Actor", [this]()
		{
			BeforeEach([this]()
				{
					OutlinerProvider->TestSpawnActor();
					Outliner->Refresh();
				});

			It("should register an item for the actor", [this]()
				{
					const FAvaOutlinerItemPtr Item = Outliner->FindItem(OutlinerProvider->GetTestSpawnActor());
					TestNotNull(TEXT("Item found for the actor"), Item.Get());

					const TSharedPtr<FAvaOutlinerActor> ActorItem = StaticCastSharedPtr<FAvaOutlinerActor>(Item);
					TestNotNull(TEXT("Item cast to FAvaOutlinerActor is valid"), ActorItem.Get());

					const AActor* Actor = ActorItem->GetActor();
					TestTrue(TEXT("Actor item matches the spawned actor"), Actor == OutlinerProvider->GetTestSpawnActor());
				});
		});
#endif

#if 0 // DISABLED: Flaky in CI — temporarily disable
	Describe("Duplicating Selected Items", [this]()
		{
			When(EWhenType::NoItemsSelected, [this]()
				{
					It("should keep the outliner with no selections", [this]()
						{
							OutlinerView->DuplicateSelected();
							Outliner->Refresh();
							TestEqual(TEXT("Selected Item count is 0"), Outliner->GetSelectedItemCount(), 0);
						});
				});

			When(EWhenType::ItemsSelected, [this]()
				{
					Describe("When selected objects have duplicated", [this]()
						{
							BeforeEach([this]()
								{
									OutlinerView->DuplicateSelected();
								});

							It("should have a matching pending item action count", [this]()
								{
									TestEqual(TEXT("Number of items enqueued match the requested number"),
										Outliner->GetPendingItemActionCount(),
										TestItems.Num());
								});

							Describe("When outliner has refreshed", [this]()
								{
									BeforeEach([this]()
										{
											Outliner->Refresh();
										});

									It("should have a matching selected item count", [this]()
										{
											TestEqual(TEXT("The template item count and the number of copies match"),
												Outliner->GetSelectedItemCount(),
												TestItems.Num());
										});

									It("should have the duplicate copies selected instead of the template", [this]()
										{
											const TArray<FAvaOutlinerItemPtr> DuplicateItems = Outliner->GetSelectedItems();
											TestFalse(TEXT("The newly selected item count is not zero"), DuplicateItems.IsEmpty());

											constexpr int32 Index = 0;
											FAvaOutliner::SortItems(TestItems);

											if (DuplicateItems.IsValidIndex(Index) && TestItems.IsValidIndex(Index) &&
												DuplicateItems[Index].IsValid() && TestItems[Index].IsValid())
											{
												TestNotEqual(TEXT("The newly selected items are different from the old selected items"),
													DuplicateItems[Index],
													TestItems[Index]);
											}
										});

									It("should have each duplicate item right below its respective template item", [this]()
										{
											constexpr int32 Index = 0;
											FAvaOutliner::SortItems(TestItems);

											if (TestItems.IsValidIndex(Index) && TestItems[Index].IsValid())
											{
												const FAvaOutlinerItemPtr TemplateItem = TestItems[Index];
												const TArray<FAvaOutlinerItemPtr> SelectedItems = Outliner->GetSelectedItems();

												if (SelectedItems.IsValidIndex(Index) && SelectedItems[Index].IsValid())
												{
													const FAvaOutlinerItemPtr DuplicateItem = SelectedItems[Index];

													TestEqual(TEXT("The template item and duplicate item share the same parent item"),
														TemplateItem->GetParent(),
														DuplicateItem->GetParent());

													const FAvaOutlinerItemPtr ParentItem = TemplateItem->GetParent();

													if (ParentItem.IsValid())
													{
														const int32 TemplateIdx = ParentItem->GetChildIndex(TemplateItem);
														const int32 DuplicateIdx = ParentItem->GetChildIndex(DuplicateItem);

														TestEqual(TEXT("The duplicate item is right below template item"),
															TemplateIdx + 1,
															DuplicateIdx);
													}
												}
											}
										});
								});
						});
				});
		});
#endif

#if 0 // DISABLED: Flaky in CI — temporarily disable
	Describe("Grouping Selected Items", [this]()
		{
			When("grouping with null", [this]()
				{
					When(EWhenType::NoItemsSelected, [this]()
						{
							It("should keep the outliner with no selections", [this]()
								{
									Outliner->GroupSelection(nullptr);
									Outliner->Refresh();
									TestEqual(TEXT("Selected Item count is 0"), Outliner->GetSelectedItemCount(), 0);
								});
						});

					When(EWhenType::ItemsSelected, [this]()
						{
							It("should keep the outliner with the same selected items", [this]()
								{
									const TArray<FAvaOutlinerItemPtr> PrevItems = Outliner->GetSelectedItems();

									Outliner->GroupSelection(nullptr);
									Outliner->Refresh();

									const TArray<FAvaOutlinerItemPtr> CurrItems = Outliner->GetSelectedItems();

									TestEqual(TEXT("Selected Item count remains the same"),
										PrevItems.Num(), CurrItems.Num());

									if (PrevItems.Num() > 0 && CurrItems.Num() > 0 &&
										PrevItems[0].IsValid() && CurrItems[0].IsValid())
									{
										TestEqual(TEXT("Selected Items remain the same"),
											PrevItems[0], CurrItems[0]);
									}
								});
						});
				});

			When("grouping with an already existing actor", [this]()
				{
					When(EWhenType::NoItemsSelected, [this]()
						{
							BeforeEach([this]()
								{
									AActor* GroupActor = OutlinerProvider->GetVolumeActor();
									TestNotNull(TEXT("Grouping actor exists"), GroupActor);
									Outliner->GroupSelection(GroupActor);
									Outliner->Refresh();
								});

							It("should keep the outliner with no selections", [this]()
								{
									TestEqual(TEXT("Selected item count is zero"), Outliner->GetSelectedItemCount(), 0);
								});
						});

					When(EWhenType::ItemsSelected, [this]()
						{
							BeforeEach([this]()
								{
									AActor* GroupActor = OutlinerProvider->GetVolumeActor();
									TestNotNull(TEXT("Grouping actor exists"), GroupActor);
									Outliner->GroupSelection(GroupActor);
									Outliner->Refresh();
								});

							It("should keep the outliner with the selected items", [this]()
								{
									const TArray<FAvaOutlinerItemPtr> Selected = Outliner->GetSelectedItems();
									TestEqual(TEXT("Selected item count is consistent"), Selected.Num(), TestItems.Num());

									if (Selected.Num() > 0 && TestItems.Num() > 0 &&
										Selected[0].IsValid() && TestItems[0].IsValid())
									{
										TestEqual(TEXT("The selected items are consistent"), TestItems[0], Selected[0]);
									}
								});

							It("should have placed the selected items under the grouping actor", [this]()
								{
									const AActor* GroupActor = OutlinerProvider->GetVolumeActor();
									const FAvaOutlinerItemPtr GroupItem = Outliner->FindItem(GroupActor);

									TestNotNull(TEXT("Grouping item exists in outliner"), GroupItem.Get());

									if (TestItems.Num() > 0 && TestItems[0].IsValid())
									{
										const int32 ChildIdx = GroupItem->GetChildIndex(TestItems[0]);
										TestTrue(TEXT("The selected items are under the grouping item"), ChildIdx != INDEX_NONE);
									}
								});
						});
				});
		});
#endif

	AfterEach([this]()
		{
			TestItems.Reset();
			OutlinerView.Reset();
			Outliner.Reset();
			OutlinerProvider.Reset();

			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		});
}

#endif // WITH_AUTOMATION_TESTS
