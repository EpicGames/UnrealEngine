// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerTestUtils.h"
#include "AvaOutliner.h"
#include "AvaOutlinerSubsystem.h"
#include "Components/LightComponent.h"
#include "Components/SkyLightComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/DirectionalLight.h"
#include "Engine/PostProcessVolume.h"
#include "Engine/SkyLight.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/TriggerVolume.h"
#include "Editor/UnrealEdEngine.h"
#include "PackageTools.h"
#include "Selection.h"
#include "UnrealEdGlobals.h"

void UE::AvaOutliner::Private::FAvaOutlinerEditorModeTools::Init(UWorld* InWorld)
{
	World = InWorld;

	SelectedElements = NewObject<UTypedElementSelectionSet>(GetTransientPackage(), NAME_None, RF_Transactional);
	check(SelectedElements);

	ActorSelection = USelection::CreateActorSelection(GetTransientPackage(), NAME_None, RF_Transactional);
	check(ActorSelection);
	ActorSelection->SetElementSelectionSet(SelectedElements);
	ActorSelection->AddToRoot();

	ComponentSelection = USelection::CreateComponentSelection(GetTransientPackage(), NAME_None, RF_Transactional);
	check(ComponentSelection);
	ComponentSelection->SetElementSelectionSet(SelectedElements);
	ComponentSelection->AddToRoot();

	ObjectSelection = USelection::CreateObjectSelection(GetTransientPackage(), NAME_None, RF_Transactional);
	check(ObjectSelection);
	ObjectSelection->SetElementSelectionSet(NewObject<UTypedElementSelectionSet>(ObjectSelection, NAME_None, RF_Transactional));
	ObjectSelection->AddToRoot();
}

void UE::AvaOutliner::Private::FAvaOutlinerEditorModeTools::Cleanup()
{
	if (!UObjectInitialized())
	{
		return;
	}

	auto CleanupSelection = [](USelection*& InSelection)
		{
			if (!InSelection)
			{
				return;	
			}

			if (!InSelection->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed))
			{
				if (UTypedElementSelectionSet* const SelectionSet = InSelection->GetElementSelectionSet())
				{
					SelectionSet->ClearSelection(FTypedElementSelectionOptions());
				}
			}

			InSelection->RemoveFromRoot();
			InSelection = nullptr;
		};

	CleanupSelection(ActorSelection);
	CleanupSelection(ComponentSelection);
	CleanupSelection(ObjectSelection);

	if (SelectedElements)
	{
		SelectedElements->RemoveFromRoot();
		SelectedElements = nullptr;
	}

	World = nullptr;
}

UE::AvaOutliner::Private::FAvaOutlinerEditorModeTools::~FAvaOutlinerEditorModeTools()
{
	Cleanup();
}

UE::AvaOutliner::Private::FAvaOutlinerProviderTest::FAvaOutlinerProviderTest()
	: Outliner(MakeShared<FAvaOutliner>(*this))
	, ModeTools(MakeShared<FAvaOutlinerEditorModeTools>())
{
	constexpr EWorldType::Type WorldType = EWorldType::Editor;
	World = UWorld::CreateWorld(WorldType, false, FName(TEXT("MotionDesignOutlinerTestWorld")));

	UPackage* const Package = World->GetPackage();
	Package->SetFlags(RF_Transient | RF_Public);
	Package->AddToRoot();

	FWorldContext& WorldContext = GEngine->CreateNewWorldContext(WorldType);
	WorldContext.SetCurrentWorld(World);

	ModeTools->Init(World);

	FillWorld();
}

UE::AvaOutliner::Private::FAvaOutlinerProviderTest::~FAvaOutlinerProviderTest()
{
	CleanupWorld();
}

void UE::AvaOutliner::Private::FAvaOutlinerProviderTest::FillWorld()
{
	if (!World)
	{
		return;
	}

	// Volume Actor
	{
		VolumeActor = World->SpawnActorDeferred<AVolume>(ATriggerVolume::StaticClass()
		   , FTransform::Identity
		   , nullptr
		   , nullptr
		   , ESpawnActorCollisionHandlingMethod::AlwaysSpawn);
		VolumeActor->FinishSpawning(FTransform::Identity, true);
	}

	// Directional Light
	{
		DirectionalLight = World->SpawnActorDeferred<ADirectionalLight>(ADirectionalLight::StaticClass()
			, FTransform::Identity
			, nullptr
			, nullptr
			, ESpawnActorCollisionHandlingMethod::AlwaysSpawn);
		DirectionalLight->FinishSpawning(FTransform::Identity, true);
	}

	// Sky Light
	{
		SkyLight = World->SpawnActorDeferred<ASkyLight>(ASkyLight::StaticClass()
			, FTransform::Identity
			, nullptr
			, nullptr
			, ESpawnActorCollisionHandlingMethod::AlwaysSpawn);
		
		USkyLightComponent* const SkyLightComponent = SkyLight->GetLightComponent();
		SkyLightComponent->bLowerHemisphereIsBlack = false;
		SkyLightComponent->Mobility = EComponentMobility::Movable;
		
		SkyLight->FinishSpawning(FTransform::Identity);
	}

	// Sky Sphere
	{
		const FTransform SphereTransform(FRotator(0, 0, 0), FVector(0, 0, 0), FVector(2000));
		
		SkySphere = World->SpawnActorDeferred<AStaticMeshActor>(AStaticMeshActor::StaticClass()
			, SphereTransform
			, nullptr
			, nullptr
			, ESpawnActorCollisionHandlingMethod::AlwaysSpawn);

		// Large scale to prevent sphere from clipping
		// Set up sky sphere showing the same cube map as used by the sky light
		UStaticMesh* const SkySphereMesh = LoadObject<UStaticMesh>(nullptr
			, TEXT("/Engine/EditorMeshes/AssetViewer/Sphere_inversenormals.Sphere_inversenormals"));

		check(SkySphereMesh);

		UStaticMeshComponent* const MeshComponent = SkySphere->GetStaticMeshComponent();
		check(MeshComponent);
		MeshComponent->SetStaticMesh(SkySphereMesh);
		MeshComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);

		SkySphere->FinishSpawning(SphereTransform, true);
	}

	// Post Process Volume
	{
		PostProcessVolume = World->SpawnActorDeferred<APostProcessVolume>(APostProcessVolume::StaticClass()
			, FTransform::Identity
			, nullptr
			, nullptr
			, ESpawnActorCollisionHandlingMethod::AlwaysSpawn);

		PostProcessVolume->BlendWeight = 1.f;
		PostProcessVolume->bUnbound = true;
		PostProcessVolume->FinishSpawning(FTransform::Identity, true);
	}

	// Floor Mesh
	{
		Floor = World->SpawnActorDeferred<AStaticMeshActor>(AStaticMeshActor::StaticClass()
			, FTransform::Identity
			, nullptr
			, nullptr
			, ESpawnActorCollisionHandlingMethod::AlwaysSpawn);

		// Large scale to prevent sphere from clipping
		// Set up sky sphere showing the same cube map as used by the sky light
		UStaticMesh* const FloorMesh = LoadObject<UStaticMesh>(nullptr
			, TEXT("/Engine/EditorMeshes/AssetViewer/Floor_Mesh.Floor_Mesh"));
		
		check(FloorMesh);

		UStaticMeshComponent* const MeshComponent = Floor->GetStaticMeshComponent();
		check(MeshComponent);
		MeshComponent->SetStaticMesh(FloorMesh);
		Floor->FinishSpawning(FTransform::Identity, true);
	}
}

void UE::AvaOutliner::Private::FAvaOutlinerProviderTest::CleanupWorld()
{
	ModeTools->Cleanup();

	if (World)
	{
		UPackage* const Package = World->GetPackage();

		World->RemoveFromRoot();
		GEngine->DestroyWorldContext(World);
		World->DestroyWorld(false);
		World->MarkAsGarbage();
		World = nullptr;
		
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		if (Package)
		{
			Package->RemoveFromRoot();
			const TArray<UPackage*> PackagesToUnload { Package };
			UPackageTools::UnloadPackages(PackagesToUnload);
		}
	}
}

TArray<FAvaOutlinerItemPtr> UE::AvaOutliner::Private::FAvaOutlinerProviderTest::GetOutlinerItems(const TArray<UObject*>& InObjects) const
{
	TArray<FAvaOutlinerItemPtr> OutlinerItems;
	OutlinerItems.Reserve(InObjects.Num());

	for (UObject* Object : InObjects)
	{
		if (FAvaOutlinerItemPtr Item = Outliner->FindItem(Object))
		{
			OutlinerItems.Add(Item);
		}
	}

	return OutlinerItems;
}

void UE::AvaOutliner::Private::FAvaOutlinerProviderTest::TestSpawnActor()
{
	TestSpawnedActor = World->SpawnActorDeferred<AStaticMeshActor>(AStaticMeshActor::StaticClass()
			, FTransform::Identity
			, nullptr
			, nullptr
			, ESpawnActorCollisionHandlingMethod::AlwaysSpawn);
	TestSpawnedActor->FinishSpawning(FTransform::Identity, true);
}

bool UE::AvaOutliner::Private::FAvaOutlinerProviderTest::CanOutlinerProcessActorSpawn(AActor* InActor) const
{
	return !InActor->bIsEditorPreviewActor;
}

void UE::AvaOutliner::Private::FAvaOutlinerProviderTest::OutlinerDuplicateActors(const TArray<AActor*>& InTemplateActors)
{
	constexpr EAvaOutlinerIgnoreNotifyFlags NotifiesToIgnore = EAvaOutlinerIgnoreNotifyFlags::Spawn
		| EAvaOutlinerIgnoreNotifyFlags::Duplication;

	//Ignore Listening to duplication automatically as we are going to add in Relative Item and Drop Zone
	Outliner->SetIgnoreNotify(NotifiesToIgnore, true);

	TArray<AActor*> TemplateActors(InTemplateActors);

	TMap<AActor*, AActor*> DuplicateActorMap;
	DuplicateActorMap.Reserve(InTemplateActors.Num());

	// Template to Duplicate Map
	TMap<AActor*, AActor*> TemplateActorMap;
	TemplateActorMap.Reserve(InTemplateActors.Num());

	// Duplicated Actor to the SubPath to the Object the original actor  is Attached To
	TMap<AActor*, FSoftObjectPath> AttachmentInfo;
	AttachmentInfo.Reserve(InTemplateActors.Num());
	
	for (TArray<AActor*>::TIterator Iter = TemplateActors.CreateIterator(); Iter; ++Iter)
	{
		if (!IsValid(*Iter))
		{
			Iter.RemoveCurrent();
		}
		TemplateActorMap.Add(*Iter);
	}
	
	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
	
	for (AActor* const TemplateActor : InTemplateActors)
	{
		SpawnInfo.Name = *TemplateActor->GetFName().GetPlainNameString();
		SpawnInfo.Template = TemplateActor;
		
		const FString OriginalLabel = TemplateActor->GetActorLabel();
			
		if (AActor* const DuplicateActor = World->SpawnActor(SpawnInfo.Template->GetClass(), nullptr, nullptr, SpawnInfo))
		{
			FActorLabelUtilities::RenameExistingActor(DuplicateActor, OriginalLabel, true);
				
			DuplicateActorMap.Add(DuplicateActor, TemplateActor);
			TemplateActorMap[TemplateActor] = DuplicateActor;
				
			if (const AActor* const AttachParent = DuplicateActor->GetAttachParentActor())
			{
				DuplicateActor->SetActorRelativeTransform(SpawnInfo.Template->GetRootComponent()->GetRelativeTransform());
				if (TemplateActorMap.Contains(AttachParent))
				{
					AttachmentInfo.Add(DuplicateActor, DuplicateActor->GetRootComponent()->GetAttachParent());
				}
			}
		}
	}

	// Attach the Duplicated Actors to the Template Parents (only for those that do have a duplicated immediate parent)
	for (const TPair<AActor*, FSoftObjectPath>& Pair : AttachmentInfo)
	{
		FString AttachParentName;
		{
			const FString& SubPathString = Pair.Value.GetSubPathString();
			const int32 LastDotIndex = SubPathString.Find(TEXT("."), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
			if (LastDotIndex == INDEX_NONE)
			{
				AttachParentName = SubPathString;
			}
			else
			{
				AttachParentName = SubPathString.RightChop(LastDotIndex + 1);
			}
		}

		const AActor* const DuplicatedParentActor = TemplateActorMap[Pair.Key->GetAttachParentActor()];
		const FString DuplicatedParentComponentPath = DuplicatedParentActor->GetPathName() + TEXT(".") + AttachParentName;
		
		if (USceneComponent* const DuplicatedParentComponent = FindObject<USceneComponent>(nullptr, *DuplicatedParentComponentPath))
		{
			Pair.Key->AttachToComponent(DuplicatedParentComponent
				, FAttachmentTransformRules::KeepWorldTransform
				, Pair.Key->GetAttachParentSocketName());	
		}
	}

	// Dirty / Invalidate new duplicate actor
	for (const TPair<AActor*, AActor*>& Pair : DuplicateActorMap)
	{
		AActor* const DuplicateActor = Pair.Key;
		
		DuplicateActor->CheckDefaultSubobjects();
		DuplicateActor->InvalidateLightingCache();
		
		// Call PostEditMove to update components, etc.
		DuplicateActor->PostEditMove(true);
		DuplicateActor->PostDuplicate(EDuplicateMode::Normal);
		DuplicateActor->CheckDefaultSubobjects();

		// Request saves/refreshes.
		DuplicateActor->MarkPackageDirty();
	}

	// Select duplicate actors
	if (DuplicateActorMap.Num() > 0)
	{
		USelection* const ActorSelection = ModeTools->GetSelectedActors();
		ActorSelection->Modify();
		ActorSelection->BeginBatchSelectOperation();
		ActorSelection->DeselectAll();
		
		for (const TPair<AActor*, AActor*>& Pair : DuplicateActorMap)
		{
			AActor* const DuplicateActor = Pair.Key;
			ActorSelection->Select(DuplicateActor);
		}
		
		ActorSelection->EndBatchSelectOperation(true);
	}

	Outliner->SetIgnoreNotify(NotifiesToIgnore, false);
	Outliner->OnActorsDuplicated(DuplicateActorMap);
}

void UE::AvaOutliner::Private::FAvaOutlinerProviderTest::OutlinerDeleteActors(const TArray<AActor*>& InDeleteActors)
{
	if (InDeleteActors.IsEmpty())
	{
		return;
	}

	auto DestroyActor = [](AActor* InActor)
	{
		if (UWorld* ActorWorld = InActor->GetWorld())
		{
			if (GIsEditor && GUnrealEd)
			{
				GUnrealEd->DeleteActors({InActor}, ActorWorld, GUnrealEd->GetSelectedActors()->GetElementSelectionSet());
			}
			else
			{
				ActorWorld->DestroyActor(InActor);
			}
		}
	};

	for (AActor* Actor : InDeleteActors)
	{
		if (Actor)
		{
			Actor->Modify();
			DestroyActor(Actor);
		}
	}
}

TOptional<EItemDropZone> UE::AvaOutliner::Private::FAvaOutlinerProviderTest::OnOutlinerItemCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, FAvaOutlinerItemPtr TargetItem) const
{
	return TOptional<EItemDropZone>();
}

FReply UE::AvaOutliner::Private::FAvaOutlinerProviderTest::OnOutlinerItemAcceptDrop(const FDragDropEvent& DragDropEvent
	, EItemDropZone DropZone
	, FAvaOutlinerItemPtr TargetItem)
{
	return FReply::Unhandled();
}

const FAttachmentTransformRules& UE::AvaOutliner::Private::FAvaOutlinerProviderTest::GetTransformRule(bool bIsPrimaryTransformRule) const
{
	return bIsPrimaryTransformRule ? FAttachmentTransformRules::KeepWorldTransform: FAttachmentTransformRules::KeepRelativeTransform;
}
