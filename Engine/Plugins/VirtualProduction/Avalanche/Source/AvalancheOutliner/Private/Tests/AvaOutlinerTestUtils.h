// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorModeManager.h"
#include "IAvaOutlinerProvider.h"

class ADirectionalLight;
class APostProcessVolume;
class ASkyLight;
class AStaticMeshActor;
class AVolume;
class FAvaOutliner;

namespace UE::AvaOutliner::Private
{
	/** Mode Tools used to test the Outliner */
	class FAvaOutlinerEditorModeTools : public FEditorModeTools
	{
	public:
		FAvaOutlinerEditorModeTools() = default;
	
		void Init(UWorld* InWorld);

		void Cleanup();
		
		virtual ~FAvaOutlinerEditorModeTools() override;

		//~ Begin FEditorModeTools
		virtual USelection* GetSelectedActors() const override { return ActorSelection; }
		virtual USelection* GetSelectedComponents() const override { return ComponentSelection; }
		virtual USelection* GetSelectedObjects() const override { return ObjectSelection; }
		virtual UWorld* GetWorld() const override { return World; }
		//~ End FEditorModeTools
	
	private:
		UTypedElementSelectionSet* SelectedElements = nullptr;
	
		UWorld* World = nullptr;
		
		USelection* ActorSelection     = nullptr;
		USelection* ComponentSelection = nullptr;
		USelection* ObjectSelection    = nullptr;
	};

	/** Provider used to test the Outliner */
	class FAvaOutlinerProviderTest : public IAvaOutlinerProvider
	{
	public:
		FAvaOutlinerProviderTest();
		
		virtual ~FAvaOutlinerProviderTest() override;

		/** Spawns the Default Actors in the Scene. Requires World to be Valid */
		void FillWorld();

		/** Destroys the World and cleans it up along the package this world belongs to */
		void CleanupWorld();

		/** Gets a List of Valid Outliner Items for the given Objects. If an Object is not found in the Outliner Item List, it will not be present in the returning array */
		TArray<FAvaOutlinerItemPtr> GetOutlinerItems(const TArray<UObject*>& InObjects) const;

		AVolume* GetVolumeActor() const { return VolumeActor.Get(); }
		ADirectionalLight* GetDirectionalLight() const { return DirectionalLight.Get(); }
		ASkyLight* GetSkyLight() const { return SkyLight.Get(); }
		AStaticMeshActor* GetSkySphere() const { return SkySphere.Get(); }
		APostProcessVolume* GetPostProcessVolume() const { return PostProcessVolume.Get(); }
		AStaticMeshActor* GetFloor() const { return Floor.Get(); }
		AStaticMeshActor* GetTestSpawnActor() const { return TestSpawnedActor.Get(); }

		/**
		 * Spawns a Test Actor and assigns it to the member variable TestSpawnedActor
		 * @see FAvaOutlinerProviderTest::GetTestSpawnActor
		 */
		void TestSpawnActor();
		
		virtual TSharedRef<FAvaOutliner> GetOutliner() const { return Outliner; }
		
		//~ Begin IAvaOutlinerProvider
		virtual bool ShouldCreateWidget() const override { return false; }
		virtual bool CanOutlinerProcessActorSpawn(AActor* InActor) const override;
		virtual bool ShouldLockOutliner() const override { return false; }
		virtual bool ShouldHideItem(const FAvaOutlinerItemPtr& Item) const override { return false; }
		virtual void OutlinerDuplicateActors(const TArray<AActor*>& InTemplateActors) override;
		virtual void OutlinerDeleteActors(const TArray<AActor*>& InDeleteActors) override;
		virtual FEditorModeTools* GetOutlinerModeTools() const override { return &ModeTools.Get(); }
		virtual FAvaSceneTree* GetSceneTree() const override { return nullptr; }
		virtual UWorld* GetOutlinerWorld() const override { return World; }
		virtual FTransform GetOutlinerDefaultActorSpawnTransform() const override { return FTransform::Identity; }
		virtual TOptional<EItemDropZone> OnOutlinerItemCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, FAvaOutlinerItemPtr TargetItem) const override;
		virtual FReply OnOutlinerItemAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, FAvaOutlinerItemPtr TargetItem) override;
		virtual const FAttachmentTransformRules& GetTransformRule(bool bIsPrimaryTransformRule) const override;
		//~ End IAvaOutlinerProvider

	private:
		/** The World used by this Provider and where the actors below are spawned into */
		UWorld* World = nullptr;

		/** Actors Present in World, Spawned in this Order */
		TWeakObjectPtr<AVolume> VolumeActor;
		TWeakObjectPtr<ADirectionalLight> DirectionalLight;
		TWeakObjectPtr<ASkyLight> SkyLight;
		TWeakObjectPtr<AStaticMeshActor> SkySphere;
		TWeakObjectPtr<APostProcessVolume> PostProcessVolume;
		TWeakObjectPtr<AStaticMeshActor> Floor;

		/** Actor that is spawned when calling FAvaOutlinerProviderTest::TestSpawnActor */
		TWeakObjectPtr<AStaticMeshActor> TestSpawnedActor;

		TSharedRef<FAvaOutliner> Outliner;
		TSharedRef<FAvaOutlinerEditorModeTools> ModeTools;
	};
}
