// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "AvaOutlinerDefines.generated.h"

class AActor;
class IAvaOutlinerItem;
class USceneComponent;

using FAvaOutlinerColorPair = TPair<FName, FLinearColor>;
using FAvaOutlinerFilterType = const IAvaOutlinerItem&;
using FAvaOutlinerItemPtr = TSharedPtr<IAvaOutlinerItem>;
using FAvaOutlinerItemWeakPtr = TWeakPtr<IAvaOutlinerItem>;

enum class EAvaOutlinerItemFlags : uint8
{
	None = 0,
	/** Whether Item should get the underlying UObject ignoring if it's pending kill */
	IgnorePendingKill = 1 << 0,
	/** Item pending Removal from the Outliner */
	PendingRemoval = 1 << 1,
	/** Whether the Item is in expanded state to show its child items */
	Expanded = 1 << 2,
};
ENUM_CLASS_FLAGS(EAvaOutlinerItemFlags);

/** Flags specifying how an Item should be Selected */
enum class EAvaOutlinerItemSelectionFlags : uint8
{
	None = 0,
	/** Append to the Current Selection*/
	AppendToCurrentSelection = 1 << 0,
	/** Signal Selection Change (e.g. to trigger on Selection Change Delegate) */
	SignalSelectionChange = 1 << 1,
	/** Auto-include the Items' Children in the Selection */
	IncludeChildren = 1 << 2,
	/** Whether to Scroll first Item into View */
	ScrollIntoView = 1 << 3,
};
ENUM_CLASS_FLAGS(EAvaOutlinerItemSelectionFlags);

enum class EAvaOutlinerIgnoreNotifyFlags : uint8
{
	None = 0,
	/** Ignores automatically handling actor spawns, usually so that it is manually handled */
	Spawn = 1 << 0,
	/** Ignores automatically handling actor duplications, usually so that it is manually handled */
	Duplication = 1 << 1,
	All = 0xFF,
};
ENUM_CLASS_FLAGS(EAvaOutlinerIgnoreNotifyFlags);

enum class EAvaOutlinerVisibilityType : uint8
{
	None,
	/** Visible in Editor */
	Editor,
	/** Visible in Game */
	Runtime,
};

enum class EAvaOutlinerRenameAction : uint8
{
	None,
	Requested,
	Completed,
	Cancelled,
};

enum class EAvaOutlinerExtensionPosition
{
	Before,
	After,
};

/** The type of visualization being done to the item */
UENUM(BlueprintType, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EAvaOutlinerItemViewMode : uint8
{
	None = 0 UMETA(Hidden),

	/** Outliner Tree Hierarchy View of the Items */
	ItemTree = 1 << 0,
	/** Flattened Horizontal List of Nested Items shown in the "Items" column */
	HorizontalItemList = 1<< 1,

	/** All the Views */
	All = ItemTree | HorizontalItemList
};
ENUM_CLASS_FLAGS(EAvaOutlinerItemViewMode);

enum class EAvaOutlinerDragDropActionType : uint8
{
	Move,
	Copy,
};
