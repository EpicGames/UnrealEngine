// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Subsystems/WorldSubsystem.h"
#include "Templates/SharedPointer.h"
#include "AvaOutlinerSubsystem.generated.h"

class IAvaOutliner;
class IAvaOutlinerProvider;

UENUM()
enum class EAvaOutlinerHierarchyChangeType : uint8
{
	/** When the actor has detached from the Parent */
	Detached,
	/** When the actor has attached to the Parent */
	Attached,
	/** When the actor was rearranged and kept its attachment to the Parent */
	Rearranged,
};

/** Subsystem in charge of instancing and keeping reference of the World's Outliner */
UCLASS(MinimalAPI)
class UAvaOutlinerSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	/**
	 * Instantiates the World's Outliner
	 * @param InProvider the outliner provider
	 * @param bInForceCreate whether to create a new IAvaOutliner even if one already exists
	 * @returns an existing or new valid IAvaOutliner instance
	 * @note the subsystem holds a non-owning reference of the outliner, the provider is what ideally should be holding the owning reference
	 */
	AVALANCHEOUTLINER_API TSharedRef<IAvaOutliner> GetOrCreateOutliner(IAvaOutlinerProvider& InProvider, bool bInForceCreate = false);

	TSharedPtr<IAvaOutliner> GetOutliner() const
	{
		return OutlinerWeak.Pin();
	}

	DECLARE_EVENT_ThreeParams(UAvaOutlinerSubsystem, FActorHierarchyChanged, AActor* /*InActor*/, const AActor* /*InParentActor*/, EAvaOutlinerHierarchyChangeType);
	FActorHierarchyChanged& OnActorHierarchyChanged()
	{
		return ActorHierarchyChangedEvent;
	}

	/**
	 * Called when the Actor has changed in the Outliner Hierarchy
	 * @param InActor the actor that changed
	 * @param InParentActor the parent actor involved (old if detached, new if attached, null if the root or invalid actor)
	 * @param InChangeType the type of hierarchical change notified
	 */
	void BroadcastActorHierarchyChanged(AActor* InActor, const AActor* InParentActor, EAvaOutlinerHierarchyChangeType InChangeType) const;

protected:
	//~ Begin UWorldSubsystem
	virtual bool DoesSupportWorldType(const EWorldType::Type InWorldType) const override;
	//~ End UWorldSubsystem

private:
	TWeakPtr<IAvaOutliner> OutlinerWeak;

	FActorHierarchyChanged ActorHierarchyChangedEvent;
};
