// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Handlers/AvaOutlinerItemDropHandler.h"

class IAvaOutlinerView;
class FAvaOutlinerItemDropHandler;

enum class EItemDropZone;

/** Drag Drop Operation for Ava Outliner Items. Customized behavior can be added in via the AddDropHandler function */
class FAvaOutlinerItemDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FAvaOutlinerItemDragDropOp, FDecoratedDragDropOp)

	static TSharedRef<FAvaOutlinerItemDragDropOp> New(const TArray<FAvaOutlinerItemPtr>& InItems
		, const TSharedPtr<FAvaOutlinerView>& InOutlinerView
		, EAvaOutlinerDragDropActionType InActionType);

	TSharedPtr<IAvaOutlinerView> GetOutlinerView() const
	{
		return OutlinerViewWeak.Pin();
	}

	TConstArrayView<FAvaOutlinerItemPtr> GetItems() const
	{
		return Items;
	}

	EAvaOutlinerDragDropActionType GetActionType() const
	{
		return ActionType;
	}

	/** Retrieves the Actors found in the DragDropOp, assuming this is of type FAvaOutlinerItemDragDropOp */
	AVALANCHEOUTLINER_API void GetDragDropOpActors(TArray<TWeakObjectPtr<AActor>>& OutActors) const;

	/** Called when the FAvaOutlinerItemDragDropOp has been created and Initialized in FAvaOutlinerItemDragDropOp::Init */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnItemDragDropOpInitialized, FAvaOutlinerItemDragDropOp&)
	AVALANCHEOUTLINER_API static FOnItemDragDropOpInitialized& OnItemDragDropOpInitialized();

	template<typename InDropHandlerType
		, typename = typename TEnableIf<TIsDerivedFrom<InDropHandlerType, FAvaOutlinerItemDropHandler>::Value>::Type
		, typename... InArgTypes>
	void AddDropHandler(InArgTypes&&... InArgs)
	{
		TSharedRef<FAvaOutlinerItemDropHandler> DropHandler = MakeShared<InDropHandlerType>(Forward<InArgTypes>(InArgs)...);
		DropHandler->Initialize(*this);
		DropHandlers.Add(DropHandler);
	}

	FReply Drop(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem);

	TOptional<EItemDropZone> CanDrop(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem) const;

protected:
	void Init(const TArray<FAvaOutlinerItemPtr>& InItems
		, const TSharedPtr<FAvaOutlinerView>& InOutlinerView
		, EAvaOutlinerDragDropActionType InActionType);

	TArray<FAvaOutlinerItemPtr> Items;

	TArray<TSharedRef<FAvaOutlinerItemDropHandler>> DropHandlers;

	TWeakPtr<IAvaOutlinerView> OutlinerViewWeak;

	EAvaOutlinerDragDropActionType ActionType = EAvaOutlinerDragDropActionType::Move;
};
