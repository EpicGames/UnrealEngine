// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Containers/Array.h"
#include "Item/IAvaOutlinerItem.h"
#include "Templates/EnableIf.h"
#include "Templates/SharedPointer.h"

class FAvaOutlinerItemDragDropOp;
class FReply;
enum class EItemDropZone;
template<typename OptionalType> struct TOptional;

/**
 * Base Class to Handle Dropping Outliner Items into a Target Outliner Item
 * @see built-in example FAvaOutlinerActorDropHandler
 */
class FAvaOutlinerItemDropHandler : public IAvaTypeCastable, public TSharedFromThis<FAvaOutlinerItemDropHandler>
{
	friend FAvaOutlinerItemDragDropOp;

	AVALANCHEOUTLINER_API void Initialize(const FAvaOutlinerItemDragDropOp& InDragDropOp);

public:
	UE_AVA_INHERITS(FAvaOutlinerItemDropHandler, IAvaTypeCastable);

	TConstArrayView<FAvaOutlinerItemPtr> GetItems() const
	{
		return Items;
	}	

protected:
	virtual bool IsDraggedItemSupported(const FAvaOutlinerItemPtr& InDraggedItem) const = 0;

	virtual TOptional<EItemDropZone> CanDrop(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem) const = 0;

	virtual bool Drop(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem) = 0;

	enum class EIterationResult
	{
		Continue,
		Break,
	};
	template<typename InItemType, typename = typename TEnableIf<TIsDerivedFrom<InItemType, IAvaOutlinerItem>::Value>::Type>
	void ForEachItem(const TFunctionRef<EIterationResult(InItemType&)>& InFunc) const
	{
		for (const FAvaOutlinerItemPtr& Item : Items)
		{
			if (!Item.IsValid())
			{
				continue;
			}

			if (InItemType* const CastedItem = Item->CastTo<InItemType>())
			{
				EIterationResult IterationResult = InFunc(*CastedItem);
				if (IterationResult == EIterationResult::Break)
				{
					break;
				}
			}
		}
	}

	TArray<FAvaOutlinerItemPtr> Items;

	EAvaOutlinerDragDropActionType ActionType = EAvaOutlinerDragDropActionType::Move;
};
