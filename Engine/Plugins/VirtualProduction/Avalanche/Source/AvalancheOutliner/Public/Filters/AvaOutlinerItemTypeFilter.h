// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "IAvaOutlinerItemFilter.h"
#include "Styling/SlateBrush.h"
#include "Templates/SubclassOf.h"
#include "AvaOutlinerItemTypeFilter.generated.h"

class FAvaOutlinerObject;
class UObject;

UENUM()
enum class EAvaOutlinerTypeFilterMode : uint8
{
	None = 0,

	/** Outliner Item type matches the Filter Type */
	MatchesType = 1 << 0,

	/** Outliner Item contains an item (as a descendant/child) of that type*/
	ContainerOfType = 1 << 1,
};
ENUM_CLASS_FLAGS(EAvaOutlinerTypeFilterMode);

USTRUCT()
struct FAvaOutlinerItemTypeFilterData
{
	GENERATED_BODY()

	FAvaOutlinerItemTypeFilterData() = default;

	AVALANCHEOUTLINER_API FAvaOutlinerItemTypeFilterData(const TArray<TSubclassOf<UObject>>& InFilterClasses
		, EAvaOutlinerTypeFilterMode InMode  = EAvaOutlinerTypeFilterMode::MatchesType
		, const FSlateBrush* InIconBrush     = nullptr
		, const FText& InTooltipText         = FText::GetEmpty()
		, EClassFlags InRequiredClassFlags   = CLASS_None
		, EClassFlags InRestrictedClassFlags = CLASS_None);

	bool HasValidFilterData() const;

	FText GetTooltipText() const;

	const FSlateBrush* GetIcon() const;

	EAvaOutlinerTypeFilterMode GetFilterMode() const;

	void SetOverrideIconColor(FSlateColor InNewIconColor);

	bool IsValidClass(const UClass* InClass) const;

	void SetFilterText(const FText& InText);

	bool PassesFilterText(const FAvaOutlinerObject& InItem) const;

private:
	UPROPERTY(EditAnywhere, Category = "Filter")
	TArray<TSubclassOf<UObject>> FilterClasses;

	UPROPERTY(EditAnywhere, Category = "Filter")
	EAvaOutlinerTypeFilterMode FilterMode = EAvaOutlinerTypeFilterMode::MatchesType;

	UPROPERTY(EditAnywhere, Category = "Filter|Advanced")
	FText FilterText;

	UPROPERTY(EditAnywhere, Category = "Filter")
	FText TooltipText;

	UPROPERTY(EditAnywhere, Category = "Filter", meta=(EditCondition="bUseOverrideIcon"))
	FSlateBrush OverrideIcon;

	UPROPERTY(EditAnywhere, Category = "Filter", meta=(InlineEditConditionToggle))
	bool bUseOverrideIcon = true;

	const FSlateBrush* IconBrush     = nullptr;

	EClassFlags RequiredClassFlags   = CLASS_None;

	EClassFlags RestrictedClassFlags = CLASS_None;
};

DECLARE_DELEGATE_RetVal_OneParam(bool, FAvaOutlinerItemTypeFilterPassDelegate, FAvaOutlinerFilterType);

class AVALANCHEOUTLINER_API FAvaOutlinerItemTypeFilter : public IAvaOutlinerItemFilter
{
public:
	explicit FAvaOutlinerItemTypeFilter(FName InFilterId
			, const TArray<TSubclassOf<UObject>>& InFilterClasses
			, EAvaOutlinerTypeFilterMode InMode  = EAvaOutlinerTypeFilterMode::MatchesType
			, const FSlateBrush* InIconBrush     = nullptr
			, const FText& InTooltipText         = FText::GetEmpty()
			, EClassFlags InRequiredClassFlags   = CLASS_None
			, EClassFlags InRestrictedClassFlags = CLASS_None)
		: FilterId(InFilterId)
		, FilterData(InFilterClasses, InMode, InIconBrush, InTooltipText, InRequiredClassFlags, InRestrictedClassFlags)
	{
	}

	explicit FAvaOutlinerItemTypeFilter(FName InFilterId, const FAvaOutlinerItemTypeFilterData& InFilterData)
		: FilterId(InFilterId)
		, FilterData(InFilterData)
	{
	}

	explicit FAvaOutlinerItemTypeFilter(FName InFilterId
			, const FAvaOutlinerItemTypeFilterPassDelegate& InFilterPassDelegate
			, const FSlateBrush* InIconBrush = nullptr
			, const FText& InTooltipText = FText::GetEmpty())
		: FilterId(InFilterId)
		, FilterData({}, EAvaOutlinerTypeFilterMode::MatchesType, InIconBrush, InTooltipText, CLASS_None, CLASS_None)
		, FilterPassDelegate(InFilterPassDelegate)
	{
	}

protected:
	//~ Begin IFilter
	virtual FChangedEvent& OnChanged() override { return ChangedEvent; }
	virtual bool PassesFilter(FAvaOutlinerFilterType InItem) const override;
	//~ End IFilter

	//~ Begin IAvaOutlinerItemFilter
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual FText GetTooltipText() const override;
	virtual FName GetFilterId() const override { return FilterId; }
	//~ End IAvaOutlinerItemFilter

	FName FilterId;

	FAvaOutlinerItemTypeFilterData FilterData;

	FChangedEvent ChangedEvent;

	FAvaOutlinerItemTypeFilterPassDelegate FilterPassDelegate;
};
