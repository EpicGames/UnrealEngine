// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Misc/IFilter.h"
#include "Templates/SharedPointer.h"

class FName;
class FText;
struct FSlateBrush;

class IAvaOutlinerItemFilter
	: public IFilter<FAvaOutlinerFilterType>
	, public TSharedFromThis<IAvaOutlinerItemFilter>
{
public:
	/** Gets a Unique Identifier for the Filter*/
	virtual FName GetFilterId() const = 0;

	/** Gets a the Filter Icon Brush to represent in the Outliner */
	virtual const FSlateBrush* GetIconBrush() const = 0;

	/**Gets the Filter Tooltip to show in the Outliner */
	virtual FText GetTooltipText() const = 0;
};
