// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "Delegates/Delegate.h"
#include "IAvaOutlinerModule.h"
#include "Item/AvaOutlinerItemId.h"
#include "Item/AvaOutlinerItemProxy.h"
#include "Item/IAvaOutlinerItem.h"
#include "ItemProxies/AvaOutlinerItemProxyRegistry.h"
#include "ItemProxies/IAvaOutlinerItemProxyFactory.h"
#include "Templates/SharedPointer.h"

class FAvaOutlinerTreeRoot;
class AActor;
class FArchive;
class FAvaEditorSelection;
class FUICommandList;
class IAvaOutlinerProvider;
class IAvaOutlinerView;
class UWorld;
enum class EItemDropZone;
struct FAttachmentTransformRules;
struct FAvaOutlinerItemId;
template<typename OptionalType> struct TOptional;

/** 
 * The Outliner Object that is commonly instanced once per World
 * (unless for advanced use where there are different outliner instances with different item ordering and behaviors).
 * This is the object that dictates core outliner behavior like how items are sorted, which items are allowed, etc.
 * Views are the objects that take this core behavior and show a part of it (e.g. through filters).
 */
class IAvaOutliner : public TSharedFromThis<IAvaOutliner>
{
public:
	virtual ~IAvaOutliner() = default;

	DECLARE_MULTICAST_DELEGATE(FOnOutlinerLoaded);
	virtual FOnOutlinerLoaded& GetOnOutlinerLoaded() = 0;

	virtual IAvaOutlinerProvider& GetProvider() const = 0;

	/** Sets the Command List that the Outliner Views will use to append their Command Lists to */
	virtual void SetBaseCommandList(const TSharedPtr<FUICommandList>& InBaseCommandList) = 0;

	/** Serializes the Outliner Save State to the given archive */
	virtual void Serialize(FArchive& Ar) = 0;

	/** Register a new Outliner View to the Outliner to the given id, replacing the old view that was bound to the given id */
	virtual TSharedPtr<IAvaOutlinerView> RegisterOutlinerView(int32 InOutlinerViewId) = 0;

	/** Gets the Outliner View bound to the given id */
	virtual TSharedPtr<IAvaOutlinerView> GetOutlinerView(int32 InOutlinerViewId) const = 0;

	/** Instantiates a new Item and automatically registers it to the Outliner */
	template<typename InItemType, typename = typename TEnableIf<TIsDerivedFrom<InItemType, IAvaOutlinerItem>::IsDerived>::Type, typename ...InArgTypes>
	TSharedRef<InItemType> FindOrAdd(InArgTypes&&... InArgs)
	{
		TSharedRef<InItemType> Item = MakeShared<InItemType>(*this, Forward<InArgTypes>(InArgs)...);

		// If an existing item already exists and has a valid state, use that and forget about the newly created
		FAvaOutlinerItemPtr ExistingItem = FindItem(Item->GetItemId());
		if (ExistingItem.IsValid() && ExistingItem->IsItemValid() && ExistingItem->IsA<InItemType>())
		{
			return StaticCastSharedPtr<InItemType>(ExistingItem).ToSharedRef();
		}

		if (Item->IsAllowedInOutliner())
		{
			RegisterItem(Item);
		}

		return Item;
	}

	/** Tries to find the Item Proxy Factory for the given Item Proxy Type Name */
	template<typename InItemProxyType, typename = typename TEnableIf<TIsDerivedFrom<InItemProxyType, FAvaOutlinerItemProxy>::IsDerived>::Type>
	IAvaOutlinerItemProxyFactory* GetItemProxyFactory() const
	{
		// First look for the Registry in Outliner
		if (IAvaOutlinerItemProxyFactory* Factory = GetItemProxyRegistry().GetItemProxyFactory<InItemProxyType>())
		{
			return Factory;
		}
		// Fallback to finding the Factory in the Module if the Outliner did not find it
		return IAvaOutlinerModule::Get().GetItemProxyRegistry().GetItemProxyFactory<InItemProxyType>();
	}

	/**
	 * Tries to get the Item Proxy Factory for the given Item Proxy type, first trying the Outliner Registry then the Module's
	 * then returns an existing item proxy created via the factory, or creates one if there's no existing item proxy
	 * @returns the Item Proxy created by the Factory. Can be null if no factory was found or if the factory intentionally returns null
	 */
	template<typename InItemProxyType, typename = typename TEnableIf<TIsDerivedFrom<InItemProxyType, FAvaOutlinerItemProxy>::IsDerived>::Type>
	TSharedPtr<FAvaOutlinerItemProxy> GetOrCreateItemProxy(const FAvaOutlinerItemPtr& InParentItem)
	{
		if (!InParentItem.IsValid() || !InParentItem->IsAllowedInOutliner())
		{
			return nullptr;
		}
		
		IAvaOutlinerItemProxyFactory* const Factory = GetItemProxyFactory<InItemProxyType>();
		if (!Factory)
		{
			return nullptr;
		}
		
		TSharedPtr<FAvaOutlinerItemProxy> OutItemProxy;
		if (FAvaOutlinerItemPtr ExistingItemProxy = FindItem(FAvaOutlinerItemId(InParentItem, *Factory)))
		{
			check(ExistingItemProxy->IsA<FAvaOutlinerItemProxy>());
			ExistingItemProxy->SetParent(InParentItem);
			OutItemProxy = StaticCastSharedPtr<FAvaOutlinerItemProxy>(ExistingItemProxy);
		}
		else
		{
			OutItemProxy = Factory->CreateItemProxy(*this, InParentItem);
		}
		RegisterItem(OutItemProxy);
		return OutItemProxy;
	}

	/** Registers the given Item, replacing the old one. */
	virtual void RegisterItem(const FAvaOutlinerItemPtr& InItem) = 0;

	/** Unregisters the Item having the given ItemId */
	virtual void UnregisterItem(const FAvaOutlinerItemId& InItemId) = 0;

	/** Ensures that the next time Refresh is called in tick, Refresh will be called */
	virtual void RequestRefresh() = 0;

	/**
	 * Flushes the Pending Actions from the Queue while also updating the state of the Outliner.
	 * Calling it directly is forcing it to happen.
	 * If a refresh is needed it will be called on the next tick automatically.
	 */
	virtual void Refresh() = 0;

	/** Gets the Tree Root Item of the Outliner */
	virtual TSharedRef<FAvaOutlinerItem> GetTreeRoot() const = 0;

	/**
	 * Finds the Registered Item that has the given Id
	 * @returns the Item with the given Id, or null if the item does not exist or was not registered to the Outliner
	 */
	virtual FAvaOutlinerItemPtr FindItem(const FAvaOutlinerItemId& InItemId) const = 0;

	/**
 	 * Adds or Removes the Ignore Notify Flags to prevent certain actions from automatically happening when they're triggered
 	 * @param InFlag the ignore flag to add or remove
 	 * @param bIgnore whether to add (true) or remove (false) the flag
 	 */
	virtual void SetIgnoreNotify(EAvaOutlinerIgnoreNotifyFlags InFlag, bool bIgnore) = 0;

	/**
 	 * Should be called when Actors have been copied and give Ava Outliner opportunity to add to the Buffer to copy the Outliner data for those Actors
 	 * @param InOutCopiedData the data to process / append to for copy
 	 * @param InCopiedActors the actors to copy
 	 */
	virtual void OnActorsCopied(FString& InOutCopiedData, TConstArrayView<AActor*> InCopiedActors) = 0;

	/**
	 * Should be called when Actors have been pasted to parse the data that was filled in by IAvaOutliner::OnActorsCopied
	 * @param InPastedData pasted string data
	 * @param InPastedActors map of the original actor name to its created actor on paste
	 */
	virtual void OnActorsPasted(FStringView InPastedData, const TMap<FName, AActor*>& InPastedActors) = 0;

	/**
	 * Handles when Actors have been Duplicated
	 * @param InDuplicateActorMap the map of the Duplicate Actors to their Templates
	 * @param InRelativeItem he item to use as positional reference to where to place the duplicate items in the outliner
	 * @param InRelativeDropZone where to put the duplicate items relative to the InRelativeItem (above, below, onto)
	 */
	virtual void OnActorsDuplicated(const TMap<AActor*, AActor*>& InDuplicateActorMap
		, FAvaOutlinerItemPtr InRelativeItem = nullptr
		, TOptional<EItemDropZone> InRelativeDropZone = TOptional<EItemDropZone>()) = 0;

	/**
 	 * Gets all the Selected Items and puts/attaches them under the given Grouping Actor.
 	 * Requires that the Grouping Actor is valid and spawned in the World.
 	 */
	virtual void GroupSelection(AActor* InGroupingActor, const TOptional<FAttachmentTransformRules>& InTransformRules) = 0;

	/** Called when the objects have been selected and notified through USelection Instances in Mode Tools */
	virtual void OnObjectSelectionChanged(const FAvaEditorSelection& InEditorSelection) = 0;

	/** Gets the World the Outliner is working with */
	virtual UWorld* GetWorld() const = 0;

	/** Returns true if the given actor is a default actor of the given world that will be hidden from the outliner. */
	virtual bool IsDefaultWorldActorToHide(const UWorld* const InWorld, const AActor* const InActor) const = 0;

protected:
	virtual const FAvaOutlinerItemProxyRegistry& GetItemProxyRegistry() const = 0;
};
