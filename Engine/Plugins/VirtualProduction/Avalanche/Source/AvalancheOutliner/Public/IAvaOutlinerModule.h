// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Containers/ContainersFwd.h"
#include "Delegates/Delegate.h"
#include "Icon/AvaOutlinerIconCustomization.h"
#include "Item/AvaOutlinerItem.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

class FAvaOutlinerItem;
class FAvaOutlinerItemProxy;
class FAvaOutlinerItemProxyRegistry;
class UToolMenu;
struct FAvaTextFilterArgs;
struct FSlateIcon;
struct FToolMenuContext;

class IAvaOutlinerModule : public IModuleInterface
{
public:
	/** Returns the instance of this module, assuming it's loaded */
	static IAvaOutlinerModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IAvaOutlinerModule>(TEXT("AvalancheOutliner"));
	};
	/** Returns whether this module has been loaded  */
	static bool IsLoaded()
	{
		return FModuleManager::Get().IsModuleLoaded(TEXT("AvalancheOutliner"));
	};

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnExtendOutlinerToolMenu, UToolMenu*);

	/** Option to extend the Outliner Item Context Menu */
	virtual FOnExtendOutlinerToolMenu& GetOnExtendOutlinerItemContextMenu() = 0;

	/** Get the Module's Item Proxy Registry. Used as back up in case the Outliner's Instance does not have the Item type registered */
	virtual FAvaOutlinerItemProxyRegistry& GetItemProxyRegistry() = 0;

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnExtendItemProxiesForItem, IAvaOutliner&, const FAvaOutlinerItemPtr&, TArray<TSharedPtr<FAvaOutlinerItemProxy>>&);

	/** Gets the Delegate to call for adding in the Item Proxies to put under the given Item */
	virtual FOnExtendItemProxiesForItem& GetOnExtendItemProxiesForItem() = 0;

	/**
	 * Register an icon customization for the given OutlinerItemClass
	 * @tparam InOutlinerItemClass Outliner class to add a customization for
	 * @tparam InIconCustomization Icon customization to add
	 * @tparam InArgsType Additional constructor parameter type for the customization class (ex. UClass* for Object/Actor customization)
	 * @param InArgs Additional constructor parameter for the customization class
	 * @return The customization created
	 */
	template<typename InOutlinerItemClass, typename InIconCustomization, typename... InArgsType
		UE_REQUIRES(TIsDerivedFrom<InOutlinerItemClass, FAvaOutlinerItem>::Value && TIsDerivedFrom<InIconCustomization, IAvaOutlinerIconCustomization>::Value)>
	InIconCustomization& RegisterOverriddenIcon(InArgsType&&... InArgs)
	{
		TSharedRef<InIconCustomization> OutlinerIconCustomization = MakeShared<InIconCustomization>(Forward<InArgsType>(InArgs)...);
		RegisterOverridenIcon_Internal(TAvaType<InOutlinerItemClass>::GetTypeId(), OutlinerIconCustomization);
		return OutlinerIconCustomization.Get();
	}

	/**
	 * Unregister an icon customization
	 * @tparam InOutlinerItemClass Outliner item class (ex. FAvaOutlinerItem)
	 * @param InAdditionalKeyParameter The additional key parameter to found the customization (ex. ObjectClass FName for ObjectIcon customization)
	 */
	template<typename InOutlinerItemClass
		UE_REQUIRES(TIsDerivedFrom<InOutlinerItemClass, FAvaOutlinerItem>::Value)>
	void UnregisterOverriddenIcon(FName InAdditionalKeyParameter)
	{
		UnregisterOverriddenIcon_Internal(TAvaType<InOutlinerItemClass>::GetTypeId(), InAdditionalKeyParameter);
	}

protected:
	virtual void RegisterOverridenIcon_Internal(const FAvaTypeId& InItemTypeId, const TSharedRef<IAvaOutlinerIconCustomization>& InAvaOutlinerIconCustomization) = 0;
	virtual void UnregisterOverriddenIcon_Internal(const FAvaTypeId& InItemTypeId, const FName& InSpecializationIdentifier) = 0;
};
