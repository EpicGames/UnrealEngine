// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"

class FAvaOutlinerItemDragDropOp;
class FAvaOutlinerItemProxy;
class FDragDropEvent;
class FEditorModeTools;
class FReply;
class FUICommandInfo;
class SHorizontalBox;
class UToolMenu;
class UWorld;
enum class EItemDropZone;
struct FAttachmentTransformRules;
struct FAvaSceneTree;

/**
 * Provides the Outliner with functionality it is not responsible for
 * Some examples include the World to use, Mode Tools, Duplicating Actors, etc
 * It also allows custom extensibility options for the Outliner
 */
class IAvaOutlinerProvider
{
public:
	virtual ~IAvaOutlinerProvider() = default;

	/** Whether an Outliner Widget should be created */
	virtual bool ShouldCreateWidget() const { return true; }

	/** Determines whether an Actor can be added and be visible in the Outliner */
	virtual bool CanOutlinerProcessActorSpawn(AActor* InActor) const = 0;

	/** Determines whether the Outliner should be read-only */
	virtual bool ShouldLockOutliner() const = 0;

	/** An extended check to determine whether Item should be hidden in the Outliner */
	virtual bool ShouldHideItem(const FAvaOutlinerItemPtr& Item) const = 0;

	/**
	 * Called when the Outliner requests the duplicate of the given template actors
	 * Implementation must return a map of the Duplicate Actor to the Template Actor
	 */
	virtual void OutlinerDuplicateActors(const TArray<AActor*>& InTemplateActors) = 0;

	/**
	 * Called when the Outliner requests the deletion of the given actors	
	 */
	virtual void OutlinerDeleteActors(const TArray<AActor*>& InDeleteActors) = 0;

	/** Returns the Mode Tools used (e.g. for Selections) */
	virtual FEditorModeTools* GetOutlinerModeTools() const = 0;

	/** Returns the Scene Tree to use for the Outliner Tree Ordering */
	virtual FAvaSceneTree* GetSceneTree() const = 0;

	/** Returns the World to use for the Outliner */
	virtual UWorld* GetOutlinerWorld() const = 0;

	/** Gets the Default Transform to use when Spawning an Actor from the Outliner (used when there's no info related to Transform) */
	virtual FTransform GetOutlinerDefaultActorSpawnTransform() const = 0;

	/** Option to quickly extend the Outliner ToolBar without having to implement UToolMenus::Get()->ExtendMenu */
	virtual void ExtendOutlinerToolBar(UToolMenu* InToolBarMenu) {}

	/** Option to quickly extend the Outliner Item Context Menu without having to implement UToolMenus::Get()->ExtendMenu */
	virtual void ExtendOutlinerItemContextMenu(UToolMenu* InItemContextMenu) {}
	
	/** Extension to add Item Filters to the Outliner */
	virtual void ExtendOutlinerItemFilters(TArray<TSharedPtr<class IAvaOutlinerItemFilter>>& InItemFilters) {}

	/** Determines whether an external Drag Drop event (i.e. not an Outliner one) can be accepted by the Outliner for a given Target Item */
	virtual TOptional<EItemDropZone> OnOutlinerItemCanAcceptDrop(const FDragDropEvent& DragDropEvent
		, EItemDropZone DropZone
		, FAvaOutlinerItemPtr TargetItem) const = 0;

	/** Processes an external Drag Drop event (i.e. not an Outliner one) for a given Target item */
	virtual FReply OnOutlinerItemAcceptDrop(const FDragDropEvent& DragDropEvent
		, EItemDropZone DropZone
		, FAvaOutlinerItemPtr TargetItem) = 0;

	/** Called when an Item has been Renamed */
	virtual void NotifyOutlinerItemRenamed(const FAvaOutlinerItemPtr& InItem) {}

	/** Called when an Item has been deleted */
	virtual void NotifyOutlinerItemDeleted(const FAvaOutlinerItemPtr& InItem) {}

	/** Called when an Item's Lock has Changed */
	virtual void NotifyOutlinerItemLockChanged(const FAvaOutlinerItemPtr& InItem) {}

	/** Returns the transform rule to be applied when nesting items. It can be selected between primary or secondary. */
	virtual const FAttachmentTransformRules& GetTransformRule(bool bIsPrimaryTransformRule) const = 0;
};
