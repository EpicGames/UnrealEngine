// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IAvaOutliner;
class SWidget;

class IAvaOutlinerView : public TSharedFromThis<IAvaOutlinerView>
{
public:
	virtual ~IAvaOutlinerView() = default;

	/** Returns the Outliner Widget. Can be null widget */
	virtual TSharedRef<SWidget> GetOutlinerWidget() const = 0;

	virtual TSharedPtr<IAvaOutliner> GetOwnerOutliner() const  = 0;
};
