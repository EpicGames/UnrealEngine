// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FAvaOutlinerItem;
struct FSlateIcon;

class IAvaOutlinerIconCustomization
{
public:
	virtual ~IAvaOutlinerIconCustomization() = default;

	virtual FName GetOutlinerItemIdentifier() const = 0;

	virtual bool HasOverrideIcon(TSharedPtr<const FAvaOutlinerItem> InOutlinerItem) const = 0;

	virtual FSlateIcon GetOverrideIcon(TSharedPtr<const FAvaOutlinerItem> InOutlinerItem) const = 0;
};
