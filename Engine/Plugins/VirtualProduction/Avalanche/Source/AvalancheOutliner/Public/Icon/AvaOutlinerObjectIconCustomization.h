// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerIconCustomization.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Templates/SharedPointer.h"

class UClass;

DECLARE_DELEGATE_RetVal_OneParam(FSlateIcon /** OutIconForItem */, FOnGetOverriddenObjectIcon, TSharedPtr<const FAvaOutlinerItem> /** InItemToCustomize */)

class FAvaOutlinerObjectIconCustomization : public IAvaOutlinerIconCustomization
{
public:
	AVALANCHEOUTLINER_API FAvaOutlinerObjectIconCustomization(const UClass* InSupportedClass);

	AVALANCHEOUTLINER_API void SetOverriddenIcon(const FOnGetOverriddenObjectIcon& InOverriddenIcon);

private:
	//~ Begin IAvaOutlinerIconCustomization
	AVALANCHEOUTLINER_API virtual FName GetOutlinerItemIdentifier() const override;
	AVALANCHEOUTLINER_API virtual bool HasOverrideIcon(TSharedPtr<const FAvaOutlinerItem> InOutlinerItem) const override;
	AVALANCHEOUTLINER_API virtual FSlateIcon GetOverrideIcon(TSharedPtr<const FAvaOutlinerItem> InOutlinerItem) const override;
	//~ End IAvaOutlinerIconCustomization

	FName SupportedClassName;

	FOnGetOverriddenObjectIcon OnGetOverriddenIcon;
};
