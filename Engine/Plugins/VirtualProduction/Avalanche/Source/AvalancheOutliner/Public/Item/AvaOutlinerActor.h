// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerObject.h"

class AActor;

/**
 * Item in Outliner representing an AActor.
 */
class AVALANCHEOUTLINER_API FAvaOutlinerActor : public FAvaOutlinerObject
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerActor, FAvaOutlinerObject);

	FAvaOutlinerActor(IAvaOutliner& InOutliner, AActor* InActor);

	//~ Begin IAvaOutlinerItem
	virtual void FindChildren(TArray<FAvaOutlinerItemPtr>& OutChildren, bool bRecursive) override;
	virtual void GetItemProxies(TArray<TSharedPtr<FAvaOutlinerItemProxy>>& OutItemProxies) override;
	virtual bool IsSortable() const override { return true; }
	virtual bool CanAddChild(const FAvaOutlinerItemPtr& InChild) const override;
	virtual void OnChildrenAdded(TConstArrayView<FAvaOutlinerItemPtr> InAddedChildren, const FAvaOutlinerAddItemParams& InAddItemParams) override;
	virtual bool RemoveChild(const FAvaOutlinerRemoveItemParams& InRemoveItemParams) override;
	virtual bool CanBeTopLevel() const override { return true; }
	virtual bool IsAllowedInOutliner() const override;
	virtual EAvaOutlinerItemViewMode GetSupportedViewModes(const FAvaOutlinerView& InOutlinerView) const override;
	virtual bool CanReceiveParentVisibilityPropagation() const override { return true; }
	virtual bool ShowVisibility(EAvaOutlinerVisibilityType VisibilityType) const override { return true; }
	virtual bool GetVisibility(EAvaOutlinerVisibilityType VisibilityType) const override;
	virtual void OnVisibilityChanged(EAvaOutlinerVisibilityType VisibilityType, bool bNewVisibility) override;
	virtual bool CanRename() const override { return Actor.IsValid(); }
	virtual bool Rename(const FString& InName) override;
	virtual bool CanDelete() const override { return Actor.IsValid(); }
	virtual bool Delete() override { return !Actor.IsValid(); }
	virtual bool CanLock() const override { return true; }
	virtual void SetLocked(bool bInIsLocked) override;
	virtual bool IsLocked() const override;
	virtual FText GetDisplayName() const override;
	virtual TArray<FName> GetTags() const override;
	virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) override;
	virtual FReply AcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) override;
	//~ End IAvaOutlinerItem

	AActor* GetActor() const { return Actor.Get(IsIgnoringPendingKill()); }

protected:
	//~ Begin FAvaOutlinerObjectItem
	virtual void SetObject_Impl(UObject* InObject) override;
	//~ End FAvaOutlinerObjectItem
	
	TWeakObjectPtr<AActor> Actor;
};
