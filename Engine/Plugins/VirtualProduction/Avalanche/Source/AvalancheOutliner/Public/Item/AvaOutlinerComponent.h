// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerObject.h"

class USceneComponent;

/**
 * Item in Outliner representing a USceneComponent.
 */
class AVALANCHEOUTLINER_API FAvaOutlinerComponent : public FAvaOutlinerObject
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerComponent, FAvaOutlinerObject);

	FAvaOutlinerComponent(IAvaOutliner& InOutliner, USceneComponent* InComponent);

	//~ Begin IAvaOutlinerItem
	virtual void FindChildren(TArray<FAvaOutlinerItemPtr>& OutChildren, bool bRecursive) override;
	virtual void GetItemProxies(TArray<TSharedPtr<FAvaOutlinerItemProxy>>& OutItemProxies) override;
	virtual bool CanAddChild(const FAvaOutlinerItemPtr& InChild) const override;
	virtual void OnChildrenAdded(TConstArrayView<FAvaOutlinerItemPtr> InAddedChildren, const FAvaOutlinerAddItemParams& InAddItemParams) override;
	virtual bool RemoveChild(const FAvaOutlinerRemoveItemParams& InRemoveItemParams) override;
	virtual EAvaOutlinerItemViewMode GetSupportedViewModes(const FAvaOutlinerView& InOutlinerView) const override;
	virtual bool IsAllowedInOutliner() const override;
	virtual TSharedRef<SWidget> GenerateLabelWidget(const TSharedRef<SAvaOutlinerTreeRow>& InRow) override;
	virtual bool CanReceiveParentVisibilityPropagation() const override { return false; }
	virtual bool ShowVisibility(EAvaOutlinerVisibilityType VisibilityType) const override { return true; }
	virtual bool GetVisibility(EAvaOutlinerVisibilityType VisibilityType) const override;
	virtual void OnVisibilityChanged(EAvaOutlinerVisibilityType VisibilityType, bool bNewVisibility) override;
	virtual bool CanRename() const override { return false; }
	virtual FLinearColor GetItemColor() const override;
	virtual TArray<FName> GetTags() const override;
	//~ End IAvaOutlinerItem 

	USceneComponent* GetComponent() const { return Component.Get(IsIgnoringPendingKill()); }

protected:
	//~ Begin FAvaOutlinerObjectItem
	virtual void SetObject_Impl(UObject* InObject) override;
	//~ End FAvaOutlinerObjectItem
	
	TWeakObjectPtr<USceneComponent> Component;
};
