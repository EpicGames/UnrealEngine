// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerItemProxy.h"

class AVALANCHEOUTLINER_API FAvaOutlinerComponentProxy : public FAvaOutlinerItemProxy
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerComponentProxy, FAvaOutlinerItemProxy)

	FAvaOutlinerComponentProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem);
	
	//~ Begin IAvaOutlinerItem
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	virtual FText GetIconTooltipText() const override;
	virtual EAvaOutlinerItemViewMode GetSupportedViewModes(const FAvaOutlinerView& InOutlinerView) const override;
	//~ End IAvaOutlinerItem
	
	//~ Begin FAvaOutlinerItemProxy
	virtual void GetProxiedItems(const TSharedRef<IAvaOutlinerItem>& InParent, TArray<FAvaOutlinerItemPtr>& OutChildren, bool bInRecursive) override;
	//~ End FAvaOutlinerItemProxy
};
