// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerItemId.h"
#include "IAvaOutlinerItem.h"

class FAssetDragDropOp;
class FAvaOutlinerItemDragDropOp;
class ULevel;

/** Base Implementation of IAvaOutlinerItem */
class AVALANCHEOUTLINER_API FAvaOutlinerItem : public IAvaOutlinerItem
{
	friend class FAvaOutliner;

public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerItem, IAvaOutlinerItem);

	FAvaOutlinerItem(IAvaOutliner& InOutliner);

	virtual ~FAvaOutlinerItem() override;

	//~ Begin IAvaOutlinerItem
	virtual TSharedRef<IAvaOutliner> GetOwnerOutliner() const override;
	virtual bool IsItemValid() const override;
	virtual void RefreshChildren() override;
	virtual const TArray<FAvaOutlinerItemPtr>& GetChildren() const override { return Children; }
	virtual void FindChildren(TArray<FAvaOutlinerItemPtr>& OutChildren, bool bRecursive) override;
	virtual TArray<FAvaOutlinerItemPtr>& GetChildrenMutable() override { return Children; }
	virtual bool IsSortable() const override { return false; }
	virtual bool CanAddChild(const FAvaOutlinerItemPtr& InChild) const override;
	virtual TArray<FAvaOutlinerItemPtr> AddChildren(const FAvaOutlinerAddItemParams& InAddItemParams) override;
	virtual bool RemoveChild(const FAvaOutlinerRemoveItemParams& InRemoveItemParams) override;
	virtual FAvaOutlinerItemPtr GetParent() const override { return ParentWeak.Pin(); }
	virtual bool CanBeTopLevel() const override { return false; }
	virtual void SetParent(FAvaOutlinerItemPtr InParent) override;
	virtual bool IsAllowedInOutliner() const override { return true; }
	virtual EAvaOutlinerItemViewMode GetSupportedViewModes(const FAvaOutlinerView& InOutlinerView) const override;
	virtual FAvaOutlinerItemId GetItemId() const override final;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual TSharedRef<SWidget> GenerateLabelWidget(const TSharedRef<SAvaOutlinerTreeRow>& InRow) override;
	virtual bool ShowVisibility(EAvaOutlinerVisibilityType VisibilityType) const override { return false; }
	virtual bool CanReceiveParentVisibilityPropagation() const override { return false; }
	virtual bool GetVisibility(EAvaOutlinerVisibilityType VisibilityType) const override { return false; }
	virtual bool CanAutoExpand() const override { return true; }
	virtual bool CanRename() const override { return false; }
	virtual bool Rename(const FString& InName) override;
	virtual bool CanDelete() const override { return false; }
	virtual bool Delete() override;
	virtual bool CanLock() const override { return false; }
	virtual void SetLocked(bool bInIsLocked) override;
	virtual bool IsLocked() const override { return false; }
	virtual void AddFlags(EAvaOutlinerItemFlags Flags) override;
	virtual void RemoveFlags(EAvaOutlinerItemFlags Flags) override;
	virtual bool HasAnyFlags(EAvaOutlinerItemFlags Flags) const override;
	virtual bool HasAllFlags(EAvaOutlinerItemFlags Flags) const override;
	virtual void SetFlags(EAvaOutlinerItemFlags InFlags) override { ItemFlags = InFlags; }
	virtual EAvaOutlinerItemFlags GetFlags() const override { return ItemFlags; }
	virtual TOptional<FAvaOutlinerColorPair> GetColor(bool bRecurse = true) const override;
	virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) override;
	virtual FReply AcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) override;
	virtual FLinearColor GetItemColor() const override;
	virtual FOnRenameAction& OnRenameAction() override { return OnRenameActionDelegate; }
	virtual FOnExpansionChanged& OnExpansionChanged() override { return OnExpansionChangedDelegate; }
	//~ End IAvaOutlinerItem
	
protected:
	/** Gets the Item Id with the latest information (e.g. parent, object, etc)*/
	virtual FAvaOutlinerItemId CalculateItemId() const = 0;

	/** Sets the ItemId member var to what CalculateItemId returns */
	void RecalculateItemId();
	
	/** The actual implementation of putting the given Item under the Children array */
	UE_DEPRECATED(5.6, "Use AddChildrenInternal instead")
	void AddChildChecked(const FAvaOutlinerAddItemParams& InAddItemParams)
	{
		AddChildrenInternal(InAddItemParams);
	}

	/** The actual implementation of putting the given Item under the Children array */
	TArray<FAvaOutlinerItemPtr> AddChildrenInternal(const FAvaOutlinerAddItemParams& InAddItemParams);

	/** The actual implementation of removing the given item from the Children array */
	bool RemoveChildChecked(const FAvaOutlinerRemoveItemParams& InRemoveItemParams);
	
	/** Careful handling of multiple children being detected and added to this item children array */
	void HandleNewSortableChildren(TArray<FAvaOutlinerItemPtr> InSortableChildren);

	/** Handling when an Asset from Content Browser has been dragged on to this item */
	FReply CreateItemsFromAssetDrop(const TSharedPtr<FAssetDragDropOp>& AssetDragDropOp, EItemDropZone DropZone, ULevel* Level);

	/** Reference to the Owning Outliner */
	IAvaOutliner& Outliner;

	/** Weak pointer to the Parent Item. Can be null, but if valid, the Parent should have this item in the Children Array */
	TWeakPtr<IAvaOutlinerItem> ParentWeak;

	/** Array of Shared pointers to the Child Items. These Items should have their ParentWeak pointing to this item */
	TArray<FAvaOutlinerItemPtr> Children;

	/** Delegate for when Expansion changes in the Item */
	FOnExpansionChanged OnExpansionChangedDelegate;
	
	/** The delegate for renaming */
	FOnRenameAction OnRenameActionDelegate;

	/** The current flags set for this item */
	EAvaOutlinerItemFlags ItemFlags = EAvaOutlinerItemFlags::None;

	FAvaOutlinerItemId ItemId;
};
