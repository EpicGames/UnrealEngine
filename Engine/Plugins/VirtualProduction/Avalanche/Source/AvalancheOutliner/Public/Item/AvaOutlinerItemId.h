// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/UnrealMemory.h"
#include "Item/IAvaOutlinerItem.h"
#include "ItemProxies/IAvaOutlinerItemProxyFactory.h"
#include "UObject/SoftObjectPath.h"

class UObject;

/** Struct to identify an item in the Outliner */
struct FAvaOutlinerItemId
{
	/** Default Ctor. Does not run any CalculateTypeHash so this instance remains invalid */
	FAvaOutlinerItemId() = default;

	/**
	 * Ctor used for Objects that appear ONCE in the Outliner (e.g. an Actor or Component), so only the Object itself is needed for the Id
	 * for objects that can appear multiple times because they're referenced multiple times, see the ctor below that takes in a Referencing Item and Referencing Id
	 * @param InObject the object that the item represents
	 */
	FAvaOutlinerItemId(const UObject* InObject)
	{
		Id = GetObjectPath(InObject);
		CalculateTypeHash();
	}

	/**
	 * Ctor used for Objects that are expected to appear multiple times in the Outliner (e.g. a Material Ref)
	 *
	 * Example Id #1:
	 * [Component Path], [Full Path of Material Asset], [Slot Index]
	 * "/Game/World.World:PersistentLevel.StaticMeshActor_0.StaticMeshComponent,/Game/Materials/M_TestMaterial.M_TestMaterial,[Slot 0]"
	 *
	 * Example Id #2:
	 * [Component Path], [Material Instance Dynamic Path], [Slot Index]
	 * "/Game/World.World:PersistentLevel.StaticMeshActor_0.StaticMeshComponent,/Game/World.World:PersistentLevel.StaticMeshActor_0.StaticMeshComponent.MaterialInstanceDynamic_0,[Slot 0]"
	 * 
	 * @param InObject the Object being referenced multiple times (e.g. a Material)
	 * @param InReferencingItem the Item holding a reference to the object (e.g. a Primitive Comp referencing a Material)
	 * @param InReferencingId the Id to differentiate this reference from other references within the same object (e.g. can be slot or property name)
	 */
	FAvaOutlinerItemId(const UObject* InObject, const FAvaOutlinerItemPtr& InReferencingItem, const FString& InReferencingId)
		: FAvaOutlinerItemId(InReferencingItem->GetItemId())
	{
		Id += TEXT(",") + GetObjectPath(InObject)
			+ TEXT(",") + InReferencingId;
		CalculateTypeHash();
	}

	/**
	 * Ctor used for making the Item Id for a item proxy that will be under the given Parent Item.
	 * Used when the actual Item Proxy is not created yet, but know it's factory and want to know whether the item proxy already exists
	 * @param InParentItem the parent item holding the item proxy
	 * @param InItemProxyFactory the factory responsible of creating the item proxy
	 */
	FAvaOutlinerItemId(const FAvaOutlinerItemPtr& InParentItem, const IAvaOutlinerItemProxyFactory& InItemProxyFactory)
		: FAvaOutlinerItemId(InParentItem->GetItemId())
	{
		Id += TEXT(",") + InItemProxyFactory.GetItemProxyTypeName().ToString();
		CalculateTypeHash();
	}

	/**
	 * Ctor used for making the Item Id for a Item Proxy under the given Parent Item.
	 * @param InParentItem the parent item holding the item proxy
	 * @param InItemProxy the item proxy under the parent item
	 */
	FAvaOutlinerItemId(const FAvaOutlinerItemPtr& InParentItem, const IAvaOutlinerItem& InItemProxy)
		: FAvaOutlinerItemId(InParentItem->GetItemId())
	{
		Id += TEXT(",") + InItemProxy.GetTypeId().ToString();
		CalculateTypeHash();
	}

	/** More flexible option to just specify the string directly. Could be used for folders (e.g. for a nested folder C could be "A/B/C" */
	FAvaOutlinerItemId(const FString& InUniqueId)
	{
		Id = InUniqueId;
		CalculateTypeHash();
	}

	FAvaOutlinerItemId(const FAvaOutlinerItemId& Other)
	{
		*this = Other;
	}

	FAvaOutlinerItemId(FAvaOutlinerItemId&& Other) noexcept
	{
		*this = MoveTemp(Other);
	}

	FAvaOutlinerItemId& operator=(const FAvaOutlinerItemId& Other)
	{
		Id = Other.Id;
		CalculateTypeHash();
		return *this;
	}

	FAvaOutlinerItemId& operator=(FAvaOutlinerItemId&& Other) noexcept
	{
		Swap(*this, Other);
		CalculateTypeHash();
		return *this;
	}

	bool operator==(const FAvaOutlinerItemId& Other) const
	{
		return Id == Other.Id && CachedHash == Other.CachedHash;
	}

	bool operator!=(const FAvaOutlinerItemId& Other) const
	{
		return !(*this == Other);
	}

	/** Returns whether this Id has a cached hash (i.e. ran any ctor except the default one) */
	bool IsValid() const { return bHasCachedHash; }

	friend uint32 GetTypeHash(const FAvaOutlinerItemId& ItemId)
	{
		return ItemId.CachedHash;
	}

	FString GetStringId() const { return Id; }

private:
	FString GetObjectPath(const UObject* InObject) const
	{
		return FSoftObjectPath(InObject).ToString();
	}

	void CalculateTypeHash()
	{
		CachedHash = GetTypeHash(Id);
		bHasCachedHash = true;
	}

	FString Id;
	uint32 CachedHash = 0;
	bool bHasCachedHash = false;
};
