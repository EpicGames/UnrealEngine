// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Engine/EngineTypes.h"
#include "Misc/Optional.h"

enum class EItemDropZone;

enum class EAvaOutlinerAddItemFlags : uint8
{
	None        = 0,
	/** Also add the children of the given item even if they were not made into their own Add Item Action */
	AddChildren = 1 << 0,
	/** Select this Item on Add */
	Select      = 1 << 1,
	/** Make a Transaction for this Action */
	Transact    = 1 << 2,
};
ENUM_CLASS_FLAGS(EAvaOutlinerAddItemFlags);

struct FAvaOutlinerAddItemParams
{
	/** *Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAvaOutlinerAddItemParams() = default;
	~FAvaOutlinerAddItemParams() = default;
	FAvaOutlinerAddItemParams(const FAvaOutlinerAddItemParams&) = default;
	FAvaOutlinerAddItemParams(FAvaOutlinerAddItemParams&&) = default;
	FAvaOutlinerAddItemParams& operator=(const FAvaOutlinerAddItemParams&) = default;
	FAvaOutlinerAddItemParams& operator=(FAvaOutlinerAddItemParams&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	explicit FAvaOutlinerAddItemParams(const FAvaOutlinerItemPtr& InItem
			, EAvaOutlinerAddItemFlags InFlags            = EAvaOutlinerAddItemFlags::None
			, const FAvaOutlinerItemPtr& InRelativeItem   = nullptr
			, TOptional<EItemDropZone> InRelativeDropZone = TOptional<EItemDropZone>())
		: RelativeItem(InRelativeItem)
		, RelativeDropZone(InRelativeDropZone)
		, Flags(InFlags)
	{
		if (InItem.IsValid())
		{
			Items.Add(InItem);
		}
	}

	UE_DEPRECATED(5.6, "Use 'Items' instead")
	FAvaOutlinerItemPtr Item;

	/** The items to add */
	TArray<FAvaOutlinerItemPtr, TInlineAllocator<1>> Items;

	/** The Item to use as base in where to place the Item */
	FAvaOutlinerItemPtr RelativeItem;

	/** The Placement Order from the Relative Item (Onto/Inside, Above, Below) */
	TOptional<EItemDropZone> RelativeDropZone;

	/** Some Extra Flags for what to do when Adding or After Adding the Items */
	EAvaOutlinerAddItemFlags Flags = EAvaOutlinerAddItemFlags::None;

	/** Flags to Indicate how we should Select the Item. This only applies if bool bSelectItem is true */
	EAvaOutlinerItemSelectionFlags SelectionFlags = EAvaOutlinerItemSelectionFlags::None;

	/** Optional Transform override Rule when Attaching Items */
	TOptional<FAttachmentTransformRules> AttachmentTransformRules;
};

struct FAvaOutlinerRemoveItemParams
{
	FAvaOutlinerRemoveItemParams(const FAvaOutlinerItemPtr& InItem = nullptr)
		: Item(InItem)
	{
	}

	FAvaOutlinerItemPtr Item;

	/** Optional Transform override Rule when Detaching Items */
	TOptional<FDetachmentTransformRules> DetachmentTransformRules;
};
