// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerItem.h"

/**
 * Item Proxies are Outliner Items that have as only purpose to group and hold common items together.
 * The description or name of such commonality between these Items should be the name of the Proxy that holds them.
 * 
 * NOTE: Although Item Proxies by default require to have a Parent to Show in Outliner,
 * they can be created without parent as means to get overriden behavior (e.g. DisplayName, Icon, etc)
 */
class AVALANCHEOUTLINER_API FAvaOutlinerItemProxy : public FAvaOutlinerItem
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerItemProxy, FAvaOutlinerItem);

	FAvaOutlinerItemProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem);

	//~ Begin IAvaOutlinerItem
	virtual bool IsItemValid() const override;
	virtual void FindChildren(TArray<FAvaOutlinerItemPtr>& OutChildren, bool bRecursive) override final;
	virtual void SetParent(FAvaOutlinerItemPtr InParent) override;
	virtual EAvaOutlinerItemViewMode GetSupportedViewModes(const FAvaOutlinerView& InOutlinerView) const override;
	virtual bool CanAutoExpand() const override;
	virtual FText GetClassName() const override { return FText::GetEmpty(); }
	//~ End IAvaOutlinerItem

	void SetPriority(uint32 InPriority) { Priority = InPriority; }

	uint32 GetPriority() const { return Priority; }

	/** Gets the Items that this Item Proxy is representing / holding (i.e. children) */
	virtual void GetProxiedItems(const TSharedRef<IAvaOutlinerItem>& InParent, TArray<FAvaOutlinerItemPtr>& OutChildren, bool bInRecursive) = 0;

protected:
	//~ Begin FAvaOutlinerItem
	virtual FAvaOutlinerItemId CalculateItemId() const override;
	//~ End FAvaOutlinerItem

private:
	/** This Item Proxy's Order Priority (i.e. Highest priority is placed topmost or leftmost (depending on Orientation). Priority 0 is lowest priority */
	uint32 Priority = 0;
};
