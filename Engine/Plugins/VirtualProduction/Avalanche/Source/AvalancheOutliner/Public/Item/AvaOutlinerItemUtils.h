// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Containers/ContainersFwd.h"

namespace UE::AvaOutliner
{
	/**
	 * Compares the absolute order of the items in the Outliner and returns true if A comes before B in the outliner.
	 * Useful to use when sorting Items.
	 */
	AVALANCHEOUTLINER_API bool CompareOutlinerItemOrder(const FAvaOutlinerItemPtr& A, const FAvaOutlinerItemPtr& B);

	/** Returns two subset arrays of Items: one is containing only Sortable Items and the other Non Sortable Items */
	AVALANCHEOUTLINER_API void SplitItems(const TArray<FAvaOutlinerItemPtr>& InItems
		, TArray<FAvaOutlinerItemPtr>& OutSortable
		, TArray<FAvaOutlinerItemPtr>& OutUnsortable);
}
