// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerLevel.h"
#include "AvaOutliner.h"
#include "AvaOutlinerActor.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Input/Reply.h"
#include "Styling/SlateIconFinder.h"

FAvaOutlinerLevel::FAvaOutlinerLevel(IAvaOutliner& InOutliner, ULevel* InLevel)
	: Super(InOutliner, InLevel)
	, LevelWeak(InLevel)
{
}

void FAvaOutlinerLevel::FindChildren(TArray<FAvaOutlinerItemPtr>& OutChildren, bool bRecursive)
{
	Super::FindChildren(OutChildren, bRecursive);

	ULevel* const Level = GetLevel();
	if (!Level)
	{
		return;
	}

	OutChildren.Reserve(OutChildren.Num() + Level->Actors.Num());

	for (AActor* Actor : Level->Actors)
	{
		// Only consider actors that are not attached to any other actor
		if (!Actor || Actor->GetSceneOutlinerParent())
		{
			continue;
		}

		const FAvaOutlinerItemPtr ActorItem = Outliner.FindOrAdd<FAvaOutlinerActor>(Actor);

		const FAvaOutlinerItemFlagGuard Guard(ActorItem, EAvaOutlinerItemFlags::IgnorePendingKill);
		OutChildren.Add(ActorItem);
		if (bRecursive)
		{
			ActorItem->FindChildren(OutChildren, bRecursive);
		}
	}
}

bool FAvaOutlinerLevel::RemoveChild(const FAvaOutlinerRemoveItemParams& InRemoveItemParams)
{
	// todo: handle actor being moved from this level item
	return Super::RemoveChild(InRemoveItemParams);
}

EAvaOutlinerItemViewMode FAvaOutlinerLevel::GetSupportedViewModes(const FAvaOutlinerView& InOutlinerView) const
{
	// Actors should only be visualized in Outliner View and not appear in the Item Column List
	// Support any other type of View Mode
	return EAvaOutlinerItemViewMode::ItemTree | ~EAvaOutlinerItemViewMode::HorizontalItemList;
}

FText FAvaOutlinerLevel::GetDisplayName() const
{
	ULevel* Level = GetLevel();
	if (!Level)
	{
		return FText::GetEmpty();
	}

	UWorld* OwningWorld = Level->GetWorld();
	UWorld* OuterWorld = Level->GetTypedOuter<UWorld>();
	if (!OwningWorld || !OuterWorld || OwningWorld == OuterWorld)
	{
		return FText::FromString(Level->GetName());
	}

	return FText::FromString(OuterWorld->GetName());	
}

FSlateIcon FAvaOutlinerLevel::GetIcon() const
{
	return FSlateIconFinder::FindIconForClass(UWorld::StaticClass());
}

TOptional<EItemDropZone> FAvaOutlinerLevel::CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone)
{
	return TOptional<EItemDropZone>();
}

FReply FAvaOutlinerLevel::AcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone)
{
	return FReply::Unhandled();
}

ULevel* FAvaOutlinerLevel::GetLevel() const
{
	return LevelWeak.Get(IsIgnoringPendingKill());
}

void FAvaOutlinerLevel::SetObject_Impl(UObject* InObject)
{
	Super::SetObject_Impl(InObject);
	LevelWeak = Cast<ULevel>(InObject);
}
