// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerObject.h"

class ULevel;

/**
 * Item in Outliner representing a Level.
 */
class FAvaOutlinerLevel : public FAvaOutlinerObject
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerLevel, FAvaOutlinerObject);

	AVALANCHEOUTLINER_API FAvaOutlinerLevel(IAvaOutliner& InOutliner, ULevel* InLevel);

	//~ Begin IAvaOutlinerItem
	AVALANCHEOUTLINER_API virtual void FindChildren(TArray<FAvaOutlinerItemPtr>& OutChildren, bool bRecursive) override;
	AVALANCHEOUTLINER_API virtual bool RemoveChild(const FAvaOutlinerRemoveItemParams& InRemoveItemParams) override;
	AVALANCHEOUTLINER_API virtual EAvaOutlinerItemViewMode GetSupportedViewModes(const FAvaOutlinerView& InOutlinerView) const override;
	virtual bool CanBeTopLevel() const override { return true; }
	virtual bool CanRename() const override { return false; }
	virtual bool CanLock() const override { return false; }
	AVALANCHEOUTLINER_API virtual FText GetDisplayName() const override;
	AVALANCHEOUTLINER_API virtual FSlateIcon GetIcon() const override;
	AVALANCHEOUTLINER_API virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) override;
	AVALANCHEOUTLINER_API virtual FReply AcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) override;
	//~ End IAvaOutlinerItem

	AVALANCHEOUTLINER_API ULevel* GetLevel() const;

protected:
	//~ Begin FAvaOutlinerObjectItem
	virtual void SetObject_Impl(UObject* InObject) override;
	//~ End FAvaOutlinerObjectItem

	TWeakObjectPtr<ULevel> LevelWeak;
};
