// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerObjectReference.h"
#include "Textures/SlateIcon.h"

class UPrimitiveComponent;
class UMaterialInterface;

/**
 * Item in Outliner representing a Material. Inherits from FAvaOutlinerObjectReference as multiple objects can have the same material
 */
class AVALANCHEOUTLINER_API FAvaOutlinerMaterial : public FAvaOutlinerObjectReference
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerMaterial, FAvaOutlinerObjectReference)

	FAvaOutlinerMaterial(IAvaOutliner& InOutliner
		, UMaterialInterface* InMaterial
		, const FAvaOutlinerItemPtr& InReferencingItem
		, int32 InMaterialIndex);

	UMaterialInterface* GetMaterial() const { return Material.Get(IsIgnoringPendingKill()); }

	//~ Begin IAvaOutlinerItem
	virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) override;
	virtual FReply AcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) override;
	virtual FSlateIcon GetIcon() const override;
	//~ End IAvaOutlinerItem
	
protected:
	UPrimitiveComponent* GetReferencingPrimitive() const;

	void OnMaterialChanged();

	//~ Begin FAvaOutlinerObjectItem
	virtual void SetObject_Impl(UObject* InObject) override;
	//~ End FAvaOutlinerObjectItem
	
	TWeakObjectPtr<UMaterialInterface> Material;

	FSlateIcon MaterialIcon;

	int32 MaterialIndex = 0;
};
