// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerItemProxy.h"
#include "ItemProxies/IAvaOutlinerItemProxyFactory.h"

class AVALANCHEOUTLINER_API FAvaOutlinerMaterialProxy : public FAvaOutlinerItemProxy
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerMaterialProxy, FAvaOutlinerItemProxy);

	FAvaOutlinerMaterialProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem);

	virtual ~FAvaOutlinerMaterialProxy() override;
	
	//~ Begin IAvaOutlinerItem
	virtual void OnItemRegistered() override;
	virtual void OnItemUnregistered() override;
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	virtual FText GetIconTooltipText() const override;
	//~ End IAvaOutlinerItem

	//~ Begin FAvaOutlinerItemProxy
	virtual void GetProxiedItems(const TSharedRef<IAvaOutlinerItem>& InParent, TArray<FAvaOutlinerItemPtr>& OutChildren, bool bInRecursive) override;
	//~ End FAvaOutlinerItemProxy
	
private:
	void BindDelegates();
	
	void UnbindDelegates();

	void OnObjectPropertyChanged(UObject* InObject, struct FPropertyChangedEvent& InPropertyChangedEvent);
	
	FDelegateHandle OnObjectPropertyChangedHandle;
};
