// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerItem.h"
#include "UObject/WeakObjectPtr.h"

/** Item Class that represents a UObject */
class AVALANCHEOUTLINER_API FAvaOutlinerObject : public FAvaOutlinerItem
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerObject, FAvaOutlinerItem);

	FAvaOutlinerObject(IAvaOutliner& InOutliner, UObject* InObject);

	//~ Begin IAvaOutlinerItem
	virtual bool IsItemValid() const override;
	virtual void Select(FAvaOutlinerScopedSelection& InSelection) const override;
	virtual bool IsSelected(const FAvaOutlinerScopedSelection& InSelection) const override;
	virtual FText GetDisplayName() const override;
	virtual FText GetClassName() const override;
	virtual FSlateIcon GetIcon() const override;
	virtual FText GetIconTooltipText() const override;
	virtual bool IsAllowedInOutliner() const override;
	virtual void OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, bool bRecursive) override;
	virtual bool CanRename() const override { return false; }
	virtual bool Rename(const FString& InName) override;
	//~ End IAvaOutlinerItem

	UObject* GetObject() const { return Object.Get(IsIgnoringPendingKill()); }
	
	void SetObject(UObject* InObject);

protected:
	//~Begin FAvaOutlinerItem
	virtual FAvaOutlinerItemId CalculateItemId() const override;
	//~End FAvaOutlinerItem
	
	virtual void SetObject_Impl(UObject* InObject);

	TWeakObjectPtr<UObject> Object;
};
