// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerObject.h"

class FAvaOutlinerObjectReference;

/**
 * Object Item that will be shared across multiple objects (e.g. Materials).
 * This is not an Item added directly to the Outliner directly but rather shown through its Reference Items.
 * 
 * For Example, for a given material that is used across multiple Items,
 * the Material Object itself would be considered a FAvaOutlinerSharedObject
 * but the actual Item in the Outliner shown will be a FAvaOutlinerObjectReference.
 * 
 * @see FAvaOutlinerObjectReference
 */
class AVALANCHEOUTLINER_API FAvaOutlinerSharedObject : public FAvaOutlinerObject
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerSharedObject, FAvaOutlinerObject);

	FAvaOutlinerSharedObject(IAvaOutliner& InOutliner, UObject* InObject);

	//~ Begin IAvaOutlinerItem
	virtual bool IsAllowedInOutliner() const override { return false; }
	virtual bool CanBeTopLevel() const override { return true; }
	//~ End IAvaOutlinerItem
	
	void AddReference(const TSharedRef<FAvaOutlinerObjectReference>& InObjectReference);
	
	void RemoveReference(const TSharedRef<FAvaOutlinerObjectReference>& InObjectReference);

	TArray<FAvaOutlinerItemPtr> GetObjectReferences() const;

private:
	TArray<TWeakPtr<FAvaOutlinerObjectReference>> ObjectReferences;
};

/**
 * Reference Item of FAvaOutlinerSharedObject that holds an Object that is likely used across other items in the Outliner.
 * @see FAvaOutlinerSharedObject
 */
class AVALANCHEOUTLINER_API FAvaOutlinerObjectReference : public FAvaOutlinerObject
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerObjectReference, FAvaOutlinerObject);

	FAvaOutlinerObjectReference(IAvaOutliner& InOutliner
		, UObject* InObject
		, const FAvaOutlinerItemPtr& InReferencingItem
		, const FString& InReferenceId);
	
	//~ Begin IAvaOutlinerItem
	virtual void OnItemRegistered() override;
	virtual void OnItemUnregistered() override;
	virtual bool IsItemValid() const override;
	virtual FText GetDisplayName() const override;
	//~ End IAvaOutlinerItem

	FAvaOutlinerItemPtr GetReferencingItem() const { return ReferencingItemWeak.Pin(); }
	
protected:
	//~Begin FAvaOutlinerItem
	virtual FAvaOutlinerItemId CalculateItemId() const override;
	//~End FAvaOutlinerItem
	
	/** The item referencing the object (e.g. a Component item holding reference of a Material) */
	FAvaOutlinerItemWeakPtr ReferencingItemWeak;
	
	/**
	 * Id to distinguish the reference from other references held by the same Referencing Item.
	 * This reference Id can be the for example a slot index of the material in a primitive component
	 * or can be the uproperty name of the variable holding the reference
	 */
	FString ReferenceId;
};
