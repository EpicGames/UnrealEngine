// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "AvaType.h"
#include "Delegates/Delegate.h"

class FAvaOutlinerItemProxy;
class FAvaOutlinerScopedSelection;
class FAvaOutlinerView;
class FDragDropEvent;
class FEditorModeTools;
class FReply;
class IAvaOutliner;
class SAvaOutlinerTreeRow;
class SWidget;
class UObject;
enum class EItemDropZone;
struct FAvaOutlinerAddItemParams;
struct FAvaOutlinerItemId;
struct FAvaOutlinerRemoveItemParams;
struct FLinearColor;
struct FSlateBrush;
struct FSlateIcon;

/*
 * An Outliner Item is the class that represents a Single Element (i.e. Node) in the Outliner Tree.
 * This can be an Item that represents an Object (e.g. Actor,Component) or a Folder, or something else.
*/
class IAvaOutlinerItem : public IAvaTypeCastable, public TSharedFromThis<IAvaOutlinerItem>
{
protected:
	using IndexType = TArray<IAvaOutlinerItem>::SizeType;

public:
	UE_AVA_INHERITS(IAvaOutlinerItem, IAvaTypeCastable);

	/** Used to signal the Scoped Selection that this Item should be Selected */
	virtual void Select(FAvaOutlinerScopedSelection& InSelection) const {}

	/** Determines whether the given Item is selected in the given Scoped Selection */
	virtual bool IsSelected(const FAvaOutlinerScopedSelection& InSelection) const { return false; }

	/** Whether the Item can be selected in Outliner at all */
	virtual bool IsSelectable() const { return true; }
	
	/** Gets the Outliner that owns this Item */
	virtual TSharedRef<IAvaOutliner> GetOwnerOutliner() const = 0;

	/** Determines whether the Item properties are in a valid state */
	virtual bool IsItemValid() const = 0;

	/** Called when the Item has been registered into the Outliner */
	virtual void OnItemRegistered() {}

	/** Called when the Item has been unregistered from the Outliner */
	virtual void OnItemUnregistered() {}

	/** Called when the Item been selected/deselected from the Tree View */
	virtual void OnItemSelectionChanged(bool bInIsSelected) {}

	/** Refreshes what the Parent and Children are of this Item. (not recursive!) */
	virtual void RefreshChildren() = 0;

	/** Resets both the Cached Visible Children and Children (before doing so, sets all child's parents to null) */
	virtual void ResetChildren()
	{
		for (const FAvaOutlinerItemPtr& Item : GetChildren())
		{
			if (Item.IsValid())
			{
				Item->SetParent(nullptr);
			}
		}
		GetChildrenMutable().Reset();	
	};

	/**
	 * Determines whether this item can be sorted by the Outliner or not.
	 * Unsorted Items usually mean that they have their own way of sorting that Outliner's Item Sorting Data should not interfere with.
	 * Note: Unsorted Children go before Sorted Items (e.g. Item Proxies go first before Actors below a Parent)
	 */
	virtual bool IsSortable() const = 0;

	/** Determines whether the given Child is supported and can be added under this Item */
	virtual bool CanAddChild(const FAvaOutlinerItemPtr& InChild) const = 0;

	/**
	 * Adds another Child under this Item if such Item is supported.
	 * Returns true if it did, false if it could not add it (e.g. item not supported).
	 */
	UE_DEPRECATED(5.6, "Use AddChildren instead")
	virtual bool AddChild(const FAvaOutlinerAddItemParams& InAddItemParams) final
	{
		return AddChildren(InAddItemParams).Num() > 0;
	}

	/**
	 * Adds the items in the add items struct under this item if such items are supported.
	 * Returns the children that were successfully added
	 */
	virtual TArray<FAvaOutlinerItemPtr> AddChildren(const FAvaOutlinerAddItemParams& InAddItemParams) = 0;

	/** Called when children have been added to this item */
	virtual void OnChildrenAdded(TConstArrayView<FAvaOutlinerItemPtr> InAddedChildren, const FAvaOutlinerAddItemParams& InAddItemParams) {}

	/**
	 * Removes the given child from this Item if it was ever indeed a child.
	 * Returns true if the removal did happen.
	 */
	virtual bool RemoveChild(const FAvaOutlinerRemoveItemParams& InRemoveItemParams) = 0;

	/** Figures out the Children of this Item. This is only relevant for items that do have that functionality (e.g. Components or Actors) */
	virtual void FindChildren(TArray<FAvaOutlinerItemPtr>& OutChildren, bool bRecursive) = 0;

	void FindValidChildren(TArray<FAvaOutlinerItemPtr>& OutChildren, bool bRecursive)
	{
		FindChildren(OutChildren, bRecursive);

		OutChildren.RemoveAll(
			[](const FAvaOutlinerItemPtr& InItem)
			{
				return !InItem.IsValid() || !InItem->IsAllowedInOutliner();
			});
	}

	/** Gets the Item Proxies for this Item (e.g. Component Item that represent Primitives add in a Material Proxy to display) */
	virtual void GetItemProxies(TArray<TSharedPtr<FAvaOutlinerItemProxy>>& OutItemProxies) {}
	
	/*
	 * Tries to Find the First Path of Descendants (not including self) that lead to a given Item in the Set.
	 * The last item is the Item in the Set that was found so path might be A/B/C/.../ItemInSet  where A is Child of This.
	 * Returns Empty array if failed.
	*/
	virtual TArray<FAvaOutlinerItemPtr> FindPath(const TArray<FAvaOutlinerItemPtr>& Items) const
	{
		const TSharedPtr<const IAvaOutlinerItem> This = SharedThis(this);
		TArray<FAvaOutlinerItemPtr> Path;
		for (const FAvaOutlinerItemPtr& Item : Items)
		{
			Path.Reset();
			FAvaOutlinerItemPtr CurrentItem = Item;
			while (CurrentItem.IsValid())
			{
				if (This == CurrentItem)
				{
					Algo::Reverse(Path);
					return Path;
				}
				Path.Add(CurrentItem);
				CurrentItem = CurrentItem->GetParent();
			}
		}
		return TArray<FAvaOutlinerItemPtr>();
	}

	/** Gets the current Child Items of this Item */
	virtual const TArray<FAvaOutlinerItemPtr>& GetChildren() const = 0;
	
	/** Gets the current Child Items of this Item */
	virtual TArray<FAvaOutlinerItemPtr>& GetChildrenMutable() = 0;

	/**
	 * Gets the Index that the given Child Item is at.
	 * NOTE: This includes HIDDEN items as Item Visibility is relative to each Outliner View.
	 * To consider only Visible Items, use FAvaOutlinerView::GetVisibleChildAt.
	 */
	virtual IndexType GetChildIndex(const FAvaOutlinerItemPtr& ChildItem) const
	{
		return GetChildren().Find(ChildItem);
	}

	/*
	 * Gets the Child Item at the given Index.
	 * NOTE: This includes HIDDEN items as Item Visibility is relative to each Outliner View.
	 * To consider only Visible Items, use FAvaOutlinerView::GetVisibleChildAt.
	 */
	virtual FAvaOutlinerItemPtr GetChildAt(IndexType Index) const
	{
		const TArray<FAvaOutlinerItemPtr>& ChildItems = GetChildren();
		if (ChildItems.IsValidIndex(Index))
		{
			return ChildItems[Index];
		}
		return nullptr;
	};

	/** Gets the Parent of this Item. Should only be null prior to registering it in Outliner or if its Root Item */
	virtual FAvaOutlinerItemPtr GetParent() const = 0;

	/**
	 * Whether this Item can be at the Top Level just beneath the Root, or it needs to always be under some other Item
	 * E.g. Actors can be Top Level, but Components or Materials can't
	 */
	virtual bool CanBeTopLevel() const = 0;
	
	/** Sets the Parent. Note that the Parent must've already have this instance as a child (check is done) */
	virtual void SetParent(FAvaOutlinerItemPtr InParent) = 0;

	/** Gets the Id of this Item */
	virtual FAvaOutlinerItemId GetItemId() const = 0;
	
	/** Returns whether this Item (and what it represents) should be allowed to be registered in Outliner */
	virtual bool IsAllowedInOutliner() const = 0;
	
	/** Gets the Display Name Text of the Item */
	virtual FText GetDisplayName() const = 0;

	/** Gets the Class/Type of this Item (e.g. for Items that represent UObjects, it will be the UObject class) */
	virtual FText GetClassName() const = 0;

	virtual FSlateIcon GetIcon() const = 0;
	
	virtual const FSlateBrush* GetIconBrush() const = 0;
	
	virtual FText GetIconTooltipText() const = 0;

	/** Gets the View Modes that this Item Supports */
	virtual EAvaOutlinerItemViewMode GetSupportedViewModes(const FAvaOutlinerView& InOutlinerView) const = 0;
	
	/** Whether this Item should be visualized in the given View Mode, for the given Outliner View */
	virtual bool IsViewModeSupported(EAvaOutlinerItemViewMode InViewMode, const FAvaOutlinerView& InOutlinerView) const
	{
		return EnumHasAnyFlags(InViewMode, GetSupportedViewModes(InOutlinerView));
	};
	
	/**
	 * Called when objects have been replaced on the Engine side. Used to replace any UObjects used by this item
	 * @param InReplacementMap the map of old object that is garbage to the new object that replaced it
	 * @param bRecursive whether to recurse this same function to children items
	 */
	virtual void OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, bool bRecursive)
	{
		if (bRecursive)
		{
			for (const FAvaOutlinerItemPtr& ChildItem : GetChildren())
			{
				if (ChildItem.IsValid())
				{
					ChildItem->OnObjectsReplaced(InReplacementMap, bRecursive);
				}
			}
		}
	};

	/** Function responsible of Generating the Label Widget for this Item (i.e. the column containing the Icon and the Name) */
	virtual TSharedRef<SWidget> GenerateLabelWidget(const TSharedRef<SAvaOutlinerTreeRow>& InRow) = 0;

	/** Whether this Item supports Visibility for the Given Type */
	virtual bool ShowVisibility(EAvaOutlinerVisibilityType VisibilityType) const = 0;

	/** Whether a change in Parent Visibility should also affect this Item's Visibility */
	virtual bool CanReceiveParentVisibilityPropagation() const = 0;

	/** Whether this Item is currently visible or not for the Given Type */
	virtual bool GetVisibility(EAvaOutlinerVisibilityType VisibilityType) const = 0;

	/** Called when the Visibility on Item has been changed on the Outliner side */
	virtual void OnVisibilityChanged(EAvaOutlinerVisibilityType VisibilityType, bool bNewVisibility) {}

	/**
	 * Delegate signature for when the Item Expansion Changes
	 * const TSharedPtr<FAvaOutlinerView>& - The Outliner View where this Expansion happened
	 * bool - Whether the Item is expanded
	 */
	using FOnExpansionChanged = TMulticastDelegate<void(const TSharedPtr<FAvaOutlinerView>&, bool)>;
	
	/** Called when Expansion state (Expanded/Collapsed) has been changed */
	virtual FOnExpansionChanged& OnExpansionChanged() = 0;

	/** Whether the Item is able to expand when AutoExpand functionality is enabled */
	virtual bool CanAutoExpand() const = 0;
	
	/** Whether this Item can be renamed or not */
	virtual bool CanRename() const = 0;
	
	/** The implementation to rename the item (e.g. for AActor it will be the Actor Label that is changed) */
	virtual bool Rename(const FString& InName) = 0;

	/** Whether this Item can be deleted or not */
	virtual bool CanDelete() const = 0;

	/** The implementation to delete the item */
	virtual bool Delete() = 0;

	/** Whether an Item can be Locked in any kind. E.g. Actors can be location-locked */
	virtual bool CanLock() const = 0;

	/** Lock/Unlock the Item */
	virtual void SetLocked(bool bInIsLocked) = 0;
	
	/** Whether the Item is currently Locked */
	virtual bool IsLocked() const = 0;
	
	virtual void AddFlags(EAvaOutlinerItemFlags Flag) = 0;
	
	virtual void RemoveFlags(EAvaOutlinerItemFlags Flag) = 0;
	
	virtual bool HasAnyFlags(EAvaOutlinerItemFlags Flag) const = 0;
	
	virtual bool HasAllFlags(EAvaOutlinerItemFlags Flag) const = 0;
	
	virtual void SetFlags(EAvaOutlinerItemFlags InFlags) = 0;
	
	virtual EAvaOutlinerItemFlags GetFlags() const = 0;

	/** Gets the Tags found for this Item (e.g. for Actors, actor tags and for Components Component Tags)*/
	virtual TArray<FName> GetTags() const { return TArray<FName>(); };
	
	/** returns the Item's Color Name and Color (either inherited if bRecurse is true, or explicit if false)*/
	virtual TOptional<FAvaOutlinerColorPair> GetColor(bool bRecurse = true) const = 0;

	/** Returns the item's height in tree, Root Item should return 0 as it has no Parent */
	int32 GetItemTreeHeight() const
	{
		int32 Height = 0;
		FAvaOutlinerItemPtr TopParent = GetParent();
		while (TopParent.IsValid())
		{
			TopParent = TopParent->GetParent();
			++Height;
		}
		return Height;
	}

	/**
	 * Delegate signature for relaying an Item Rename action
	 * EAvaOutlinerRenameAction the type of action being relayed (e.g. request a rename, or notify rename complete, etc)
	 * const TSharedPtr<FAvaOutlinerView>& - The Outliner View where the rename action is taking place
	 */
	using FOnRenameAction = TMulticastDelegate<void(EAvaOutlinerRenameAction, const TSharedPtr<FAvaOutlinerView>&)>;
	
	/** Broadcasts whenever a rename action takes place from a given view (e.g. when pressing "F2" to rename, or committing the rename text) */
	virtual FOnRenameAction& OnRenameAction() = 0;

	/** Determines if and where the incoming Drag Drop Event can be processed by this item */
	virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) = 0;

	/** Processes the Drag and Drop Event for this Item */
	virtual FReply AcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) = 0;

	/** Gets the Brush Color to use for the Item in the Outliner (e.g. Components should be grayed out a bit) */
	virtual FLinearColor GetItemColor() const = 0;

	/** Whether Ignoring Pending Kill. Useful to get underlying UObjects that are pending kill and get the pointer to it and not a null value */
	bool IsIgnoringPendingKill() const { return HasAllFlags(EAvaOutlinerItemFlags::IgnorePendingKill); }
};

/** Adds Scoped Item Flags, removes them when out of scope. Useful for temp checks like IgnorePendingKill */
struct FAvaOutlinerItemFlagGuard
{
	FAvaOutlinerItemFlagGuard(FAvaOutlinerItemPtr InItem, EAvaOutlinerItemFlags ItemFlags)
	{
		if (InItem.IsValid())
		{
			Item         = InItem;
			OldItemFlags = InItem->GetFlags();
			InItem->SetFlags(ItemFlags);
		}
	}

	~FAvaOutlinerItemFlagGuard()
	{
		if (Item.IsValid())
		{
			Item->SetFlags(OldItemFlags);
		}
	}

protected:
	FAvaOutlinerItemPtr Item;
	
	EAvaOutlinerItemFlags OldItemFlags;
};
