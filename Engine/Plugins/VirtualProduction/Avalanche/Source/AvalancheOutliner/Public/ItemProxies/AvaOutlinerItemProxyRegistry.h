// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaOutlinerItemProxyFactory.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Factories/AvaOutlinerItemProxyDefaultFactory.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

/**
 * Handles registering an Outliner Item Type with an Outliner Item Proxy Factory that creates the respective IAvaOutlinerItemProxy
 * @see IAvaOutlinerItemProxy
 */
class FAvaOutlinerItemProxyRegistry
{
public:
	template<typename InItemProxyFactoryType
		, typename = typename TEnableIf<TIsDerivedFrom<InItemProxyFactoryType, IAvaOutlinerItemProxyFactory>::IsDerived>::Type
		, typename... InArgTypes>
	void RegisterItemProxyFactory(InArgTypes&&... InArgs)
	{
		const TSharedRef<IAvaOutlinerItemProxyFactory> Factory = MakeShared<InItemProxyFactoryType>(Forward<InArgTypes>(InArgs)...);
		const FName ItemProxyTypeName = Factory->GetItemProxyTypeName();
		ItemProxyFactories.Add(ItemProxyTypeName, Factory);
	}

	/** Registers an Item Proxy Type with the Default Factory */
	template<typename InItemProxyType
		, uint32 InItemProxyPriority = 0
		, typename InType = typename TEnableIf<TIsDerivedFrom<InItemProxyType, FAvaOutlinerItemProxy>::IsDerived>::Type
		, typename... InArgTypes>
	void RegisterItemProxyWithDefaultFactory()
	{
		RegisterItemProxyFactory<TAvaOutlinerItemProxyDefaultFactory<InItemProxyType, InItemProxyPriority>>();
	}

	/** Unregisters the given Item Type from having an Item Proxy Factory */
	template<typename InItemProxyType, typename = typename TEnableIf<TIsDerivedFrom<InItemProxyType, FAvaOutlinerItemProxy>::IsDerived>::Type>
	void UnregisterItemProxyFactory()
	{
		ItemProxyFactories.Remove(TAvaType<InItemProxyType>::GetTypeId().ToName());
	}

	void UnregisterItemProxyFactory(FName InItemProxyTypeName)
	{
		ItemProxyFactories.Remove(InItemProxyTypeName);
	}

	/** Unregisters all the Item Proxy Factories for this Instance */
	void UnregisterAllItemProxyFactories()
	{
		ItemProxyFactories.Empty();
	}

	/** Gets the Item Proxy Factory for the given Item Proxy Type Name. Returns nullptr if not found */
	IAvaOutlinerItemProxyFactory* GetItemProxyFactory(FName InItemProxyTypeName) const
	{
		if (const TSharedRef<IAvaOutlinerItemProxyFactory>* const FoundFactory = ItemProxyFactories.Find(InItemProxyTypeName))
		{
			return &(FoundFactory->Get());
		}
		return nullptr;
	}

	/** Gets the Item Proxy Factory if it was registered with the Item Proxy Type Name. Returns nullptr if not found */
	template<typename InItemProxyType, typename = typename TEnableIf<TIsDerivedFrom<InItemProxyType, FAvaOutlinerItemProxy>::IsDerived>::Type>
	IAvaOutlinerItemProxyFactory* GetItemProxyFactory() const
	{
		return GetItemProxyFactory(TAvaType<InItemProxyType>::GetTypeId().ToName());
	}

	/** Gets all the Item Proxy Type Names that exist in this registry */
	void GetRegisteredItemProxyTypeNames(TSet<FName>& OutItemProxyTypeNames) const { ItemProxyFactories.GetKeys(OutItemProxyTypeNames); }

private:
	/** Map of the Item Proxy Type Name and its Item Proxy Factory */
	TMap<FName, TSharedRef<IAvaOutlinerItemProxyFactory>> ItemProxyFactories;
};
