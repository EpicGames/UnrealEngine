// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Item/AvaOutlinerItemProxy.h"
#include "ItemProxies/IAvaOutlinerItemProxyFactory.h"

/**
 * Default Template Item Proxy Factory classes to create the Item Proxy without having to write it out for all classes 
 * that don't need special behavior or custom constructors
 */
template<typename InItemProxyType, uint32 InItemProxyPriority = 0
	, typename = typename TEnableIf<TIsDerivedFrom<InItemProxyType, FAvaOutlinerItemProxy>::IsDerived>::Type>
class TAvaOutlinerItemProxyDefaultFactoryBase : public IAvaOutlinerItemProxyFactory
{
public:
	virtual FName GetItemProxyTypeName() const override
	{
		return TAvaType<InItemProxyType>::GetTypeId().ToName();
	}

protected:
	template<typename... InArgTypes>
	TSharedRef<FAvaOutlinerItemProxy> DefaultCreateItemProxy(InArgTypes&&... InArgs)
	{
		TSharedRef<FAvaOutlinerItemProxy> ItemProxy = MakeShared<InItemProxyType>(Forward<InArgTypes>(InArgs)...);
		ItemProxy->SetPriority(InItemProxyPriority);
		return ItemProxy;
	}
};

template<typename InItemProxyType, uint32 InItemProxyPriority = 0
	, typename = typename TEnableIf<TIsDerivedFrom<InItemProxyType, FAvaOutlinerItemProxy>::IsDerived>::Type>
class TAvaOutlinerItemProxyDefaultFactory : public TAvaOutlinerItemProxyDefaultFactoryBase<InItemProxyType, InItemProxyPriority>
{
public:
	virtual TSharedPtr<FAvaOutlinerItemProxy> CreateItemProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem) override
	{
		return this->DefaultCreateItemProxy(InOutliner, InParentItem);
	}
};
