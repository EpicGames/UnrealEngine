// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Templates/SharedPointer.h"

class FAvaOutlinerItemProxy;
class FName;
class IAvaOutliner;

/**
 * Item Proxy Factories are the classes that instance or get the existing Outliner Item Proxies for a given Item
 * @see IAvaOutlinerItemProxy
 */
class IAvaOutlinerItemProxyFactory
{
public:
	virtual ~IAvaOutlinerItemProxyFactory() = default;

	/** Gets the Type Name of the Item Proxy the Factory creates */
	virtual FName GetItemProxyTypeName() const = 0;

	/** Returns a newly created instance of the Relevant Item Proxy if successful */
	virtual TSharedPtr<FAvaOutlinerItemProxy> CreateItemProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem) = 0;
};
