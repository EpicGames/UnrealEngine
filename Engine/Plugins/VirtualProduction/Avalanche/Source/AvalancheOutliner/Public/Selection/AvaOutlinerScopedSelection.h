// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"

class FEditorModeTools;
class UObject;

enum class EAvaOutlinerScopedSelectionPurpose
{
	/** At the end of the Scope, it will set whatever has been added to the Selected List to be the new Selection */
	Sync,
	/** Used only to check for whether an Object is Selected or not. Cannot execute "Select" */
	Read,
};

/** Handler to Sync Selection from Outliner to the Editor Mode Tools */
class FAvaOutlinerScopedSelection
{
public:
	explicit FAvaOutlinerScopedSelection(const FEditorModeTools& InEditorModeTools
		, EAvaOutlinerScopedSelectionPurpose InPurpose = EAvaOutlinerScopedSelectionPurpose::Read);

	~FAvaOutlinerScopedSelection();

	AVALANCHEOUTLINER_API void Select(UObject* InObject);

	AVALANCHEOUTLINER_API bool IsSelected(const UObject* InObject) const;

	AVALANCHEOUTLINER_API const FEditorModeTools& GetEditorModeTools() const;

private:
	void SyncSelections();

	const FEditorModeTools& EditorModeTools;

	/** All Objects Selected (Actors, Components, Objects) */
	TSet<const UObject*> ObjectsSet;

	TArray<UObject*> SelectedActors;
	TArray<UObject*> SelectedComponents;
	TArray<UObject*> SelectedObjects;

	EAvaOutlinerScopedSelectionPurpose Purpose;
};
