// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaOutlinerDefines.h"
#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "AvaOutlinerItemsContext.generated.h"

class IAvaOutliner;

UCLASS(MinimalAPI)
class UAvaOutlinerItemsContext : public UObject
{
	GENERATED_BODY()

	friend class FAvaOutlinerView;

public:
	TSharedPtr<IAvaOutliner> GetOutliner() const
	{
		return OutlinerWeak.Pin();
	}

	TConstArrayView<FAvaOutlinerItemWeakPtr> GetItems() const
	{
		return ItemListWeak;
	}

private:
	TWeakPtr<IAvaOutliner> OutlinerWeak;

	TArray<FAvaOutlinerItemWeakPtr> ItemListWeak;
};
