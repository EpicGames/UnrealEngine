// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalanchePropertyAnimator : ModuleRules
{
    public AvalanchePropertyAnimator(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
	            "Avalanche",
	            "AvalancheSequence",
	            "AvalancheText",
	            "Core",
                "CoreUObject",
                "Engine",
                "MovieScene",
                "PropertyAnimator",
                "PropertyAnimatorCore",
                "Slate",
                "SlateCore",
                "Text3D"
            }
        );

        if (Target.Type == TargetRules.TargetType.Editor)
        {
	        PrivateDependencyModuleNames.Add("Sequencer");
        }
    }
}