// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalanchePropertyAnimatorEditor : ModuleRules
{
    public AvalanchePropertyAnimatorEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
	            "AvalancheOutliner",
	            "Core",
                "CoreUObject",
                "Engine",
                "PropertyAnimator",
                "PropertyAnimatorCore",
                "PropertyAnimatorCoreEditor",
                "Slate",
                "SlateCore" 
            }
        );
    }
}