// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaPropertyAnimatorEditorModule.h"

#include "DragDropOps/AvaOutlinerItemDragDropOp.h"
#include "IAvaOutliner.h"
#include "IAvaOutlinerModule.h"
#include "Item/AvaOutlinerActor.h"
#include "ItemProxies/AvaOutlinerItemProxyRegistry.h"
#include "Outliner/AvaPropertyAnimatorEditorOutlinerContextMenu.h"
#include "Outliner/AvaPropertyAnimatorEditorOutlinerDropHandler.h"
#include "Outliner/AvaPropertyAnimatorEditorOutlinerProxy.h"

#define LOCTEXT_NAMESPACE "AvaPropertyAnimatorEditorModule"

void FAvaPropertyAnimatorEditorModule::StartupModule()
{
    RegisterOutlinerItems();
}

void FAvaPropertyAnimatorEditorModule::ShutdownModule()
{
    UnregisterOutlinerItems();
}

void FAvaPropertyAnimatorEditorModule::RegisterOutlinerItems()
{
	FAvaOutlinerItemProxyRegistry& ItemProxyRegistry = IAvaOutlinerModule::Get().GetItemProxyRegistry();

	ItemProxyRegistry.RegisterItemProxyWithDefaultFactory<FAvaPropertyAnimatorEditorOutlinerProxy, 50>();

	OutlinerProxiesExtensionDelegateHandle = IAvaOutlinerModule::Get().GetOnExtendItemProxiesForItem().AddLambda(
		[](IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InItem, TArray<TSharedPtr<FAvaOutlinerItemProxy>>& OutItemProxies)
		{
			if (InItem->IsA<FAvaOutlinerActor>())
			{
				if (const TSharedPtr<FAvaOutlinerItemProxy> ControllerProxy = InOutliner.GetOrCreateItemProxy<FAvaPropertyAnimatorEditorOutlinerProxy>(InItem))
				{
					OutItemProxies.Add(ControllerProxy);
				}
			}
		});

	OutlinerContextDelegateHandle = IAvaOutlinerModule::Get().GetOnExtendOutlinerItemContextMenu()
		.AddStatic(&FAvaPropertyAnimatorEditorOutlinerContextMenu::OnExtendOutlinerContextMenu);

	OutlinerDropHandlerDelegateHandle = FAvaOutlinerItemDragDropOp::OnItemDragDropOpInitialized().AddStatic(
		[](FAvaOutlinerItemDragDropOp& InDragDropOp)
		{
			InDragDropOp.AddDropHandler<FAvaPropertyAnimatorEditorOutlinerDropHandler>();
		});
}

void FAvaPropertyAnimatorEditorModule::UnregisterOutlinerItems()
{
	if (IAvaOutlinerModule::IsLoaded())
	{
		IAvaOutlinerModule& OutlinerModule = IAvaOutlinerModule::Get();

		FAvaOutlinerItemProxyRegistry& ItemProxyRegistry = OutlinerModule.GetItemProxyRegistry();
		ItemProxyRegistry.UnregisterItemProxyFactory<FAvaPropertyAnimatorEditorOutlinerProxy>();

		OutlinerModule.GetOnExtendItemProxiesForItem().Remove(OutlinerProxiesExtensionDelegateHandle);
		OutlinerProxiesExtensionDelegateHandle.Reset();

		OutlinerModule.GetOnExtendOutlinerItemContextMenu().Remove(OutlinerContextDelegateHandle);
		OutlinerContextDelegateHandle.Reset();

		FAvaOutlinerItemDragDropOp::OnItemDragDropOpInitialized().Remove(OutlinerDropHandlerDelegateHandle);
		OutlinerDropHandlerDelegateHandle.Reset();
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FAvaPropertyAnimatorEditorModule, AvalanchePropertyAnimatorEditor)