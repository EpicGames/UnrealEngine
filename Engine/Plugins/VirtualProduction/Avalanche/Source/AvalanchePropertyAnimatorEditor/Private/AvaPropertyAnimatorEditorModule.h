// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

class FAvaPropertyAnimatorEditorModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;
	//~ End IModuleInterface

protected:
	void RegisterOutlinerItems();
	void UnregisterOutlinerItems();

	FDelegateHandle OutlinerProxiesExtensionDelegateHandle;
	FDelegateHandle OutlinerContextDelegateHandle;
	FDelegateHandle OutlinerDropHandlerDelegateHandle;
};
