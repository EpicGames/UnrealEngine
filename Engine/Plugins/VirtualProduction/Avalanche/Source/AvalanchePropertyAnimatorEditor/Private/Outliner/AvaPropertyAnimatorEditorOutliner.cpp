// Copyright Epic Games, Inc. All Rights Reserved.

#include "Outliner/AvaPropertyAnimatorEditorOutliner.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Components/PropertyAnimatorCoreComponent.h"
#include "IAvaOutliner.h"
#include "Selection/AvaOutlinerScopedSelection.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/PropertyAnimatorCoreSubsystem.h"

FAvaPropertyAnimatorEditorOutliner::FAvaPropertyAnimatorEditorOutliner(IAvaOutliner& InOutliner, UPropertyAnimatorCoreBase* InAnimator)
	: FAvaOutlinerObject(InOutliner, InAnimator)
	, PropertyAnimator(InAnimator)
{
	const TSharedPtr<const FPropertyAnimatorCoreMetadata> AnimatorMetadata = PropertyAnimator->GetAnimatorMetadata();
	ItemName = FText::FromName(PropertyAnimator->GetAnimatorDisplayName());
	ItemIcon = FSlateIconFinder::FindIconForClass(UPropertyAnimatorCoreComponent::StaticClass());
	ItemTooltip = AnimatorMetadata->DisplayName;

	UPropertyAnimatorCoreBase::OnPropertyAnimatorRemoved().AddRaw(this, &FAvaPropertyAnimatorEditorOutliner::OnAnimatorRemoved);
}

FAvaPropertyAnimatorEditorOutliner::~FAvaPropertyAnimatorEditorOutliner()
{
	if (UObjectInitialized())
	{
		UPropertyAnimatorCoreBase::OnPropertyAnimatorRemoved().RemoveAll(this);
	}
}

void FAvaPropertyAnimatorEditorOutliner::Select(FAvaOutlinerScopedSelection& InSelection) const
{
	UPropertyAnimatorCoreBase* const UnderlyingAnimator = GetPropertyAnimator();

	if (!UnderlyingAnimator)
	{
		return;
	}

	const AActor* const OwningActor = UnderlyingAnimator->GetAnimatorActor();

	if (!InSelection.IsSelected(OwningActor))
	{
		InSelection.Select(UnderlyingAnimator);
	}
}

FText FAvaPropertyAnimatorEditorOutliner::GetDisplayName() const
{
	return ItemName;
}

FText FAvaPropertyAnimatorEditorOutliner::GetIconTooltipText() const
{
	return ItemTooltip;
}

FSlateIcon FAvaPropertyAnimatorEditorOutliner::GetIcon() const
{
	return ItemIcon;
}

bool FAvaPropertyAnimatorEditorOutliner::ShowVisibility(EAvaOutlinerVisibilityType InVisibilityType) const
{
	return InVisibilityType == EAvaOutlinerVisibilityType::Runtime;
}

bool FAvaPropertyAnimatorEditorOutliner::GetVisibility(EAvaOutlinerVisibilityType InVisibilityType) const
{
	return InVisibilityType == EAvaOutlinerVisibilityType::Runtime
		&& PropertyAnimator.IsValid()
		&& PropertyAnimator->GetAnimatorEnabled();
}

void FAvaPropertyAnimatorEditorOutliner::OnVisibilityChanged(EAvaOutlinerVisibilityType InVisibilityType, bool bInNewVisibility)
{
	if (InVisibilityType == EAvaOutlinerVisibilityType::Runtime && PropertyAnimator.IsValid())
	{
		PropertyAnimator->SetAnimatorEnabled(bInNewVisibility);
	}
}

bool FAvaPropertyAnimatorEditorOutliner::CanDelete() const
{
	return PropertyAnimator.IsValid();
}

bool FAvaPropertyAnimatorEditorOutliner::Delete()
{
	if (const UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get())
	{
		return AnimatorSubsystem->RemoveAnimator(PropertyAnimator.Get(), /** Transact */false);
	}

	return false;
}

void FAvaPropertyAnimatorEditorOutliner::SetObject_Impl(UObject* InObject)
{
	FAvaOutlinerObject::SetObject_Impl(InObject);
	PropertyAnimator = Cast<UPropertyAnimatorCoreBase>(InObject);
}

void FAvaPropertyAnimatorEditorOutliner::OnAnimatorRemoved(UPropertyAnimatorCoreComponent* InComponent, UPropertyAnimatorCoreBase* InAnimator, EPropertyAnimatorCoreUpdateEvent InReason) const
{
	if (InAnimator && PropertyAnimator.Get(/** EvenIfPendingKill */true) == InAnimator)
	{
		if (const TSharedPtr<IAvaOutliner> OwnerOutliner = GetOwnerOutliner())
		{
			OwnerOutliner->RequestRefresh();
		}
	}
}
