// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Item/AvaOutlinerObject.h"
#include "Textures/SlateIcon.h"
#include "UObject/WeakObjectPtr.h"

class UPropertyAnimatorCoreBase;
class UPropertyAnimatorCoreComponent;
enum class EPropertyAnimatorCoreUpdateEvent : uint8;

/** Item representing a property animator item */
class FAvaPropertyAnimatorEditorOutliner : public FAvaOutlinerObject
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaPropertyAnimatorEditorOutliner, FAvaOutlinerObject);

	FAvaPropertyAnimatorEditorOutliner(IAvaOutliner& InOutliner, UPropertyAnimatorCoreBase* InObject);
	virtual ~FAvaPropertyAnimatorEditorOutliner() override;

	UPropertyAnimatorCoreBase* GetPropertyAnimator() const
	{
		return PropertyAnimator.Get();
	}

	//~ Begin IAvaOutlinerItem
	virtual void Select(FAvaOutlinerScopedSelection& InSelection) const override;
	virtual FText GetDisplayName() const override;
	virtual FText GetIconTooltipText() const override;
	virtual FSlateIcon GetIcon() const override;
	virtual bool ShowVisibility(EAvaOutlinerVisibilityType InVisibilityType) const override;
	virtual bool GetVisibility(EAvaOutlinerVisibilityType InVisibilityType) const override;
	virtual void OnVisibilityChanged(EAvaOutlinerVisibilityType InVisibilityType, bool bInNewVisibility) override;
	virtual bool CanDelete() const override;
	virtual bool Delete() override;
	//~ End IAvaOutlinerItem

protected:
	//~ Begin FAvaOutlinerObjectItem
	virtual void SetObject_Impl(UObject* InObject) override;
	//~ End FAvaOutlinerObjectItem

	void OnAnimatorRemoved(UPropertyAnimatorCoreComponent* InComponent, UPropertyAnimatorCoreBase* InAnimator, EPropertyAnimatorCoreUpdateEvent InReason) const;

	TWeakObjectPtr<UPropertyAnimatorCoreBase> PropertyAnimator;

	FText ItemName;
	FSlateIcon ItemIcon;
	FText ItemTooltip;
};