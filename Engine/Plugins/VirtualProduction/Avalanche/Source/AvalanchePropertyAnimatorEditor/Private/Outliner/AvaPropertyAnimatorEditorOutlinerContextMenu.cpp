// Copyright Epic Games, Inc. All Rights Reserved.

#include "Outliner/AvaPropertyAnimatorEditorOutlinerContextMenu.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "AvaPropertyAnimatorEditorOutliner.h"
#include "AvaPropertyAnimatorEditorOutlinerProxy.h"
#include "Components/PropertyAnimatorCoreComponent.h"
#include "Item/AvaOutlinerActor.h"
#include "Menus/PropertyAnimatorCoreEditorMenuDefs.h"
#include "Subsystems/PropertyAnimatorCoreEditorSubsystem.h"
#include "ToolMenuContext/AvaOutlinerItemsContext.h"

void FAvaPropertyAnimatorEditorOutlinerContextMenu::OnExtendOutlinerContextMenu(UToolMenu* InToolMenu)
{
	if (!InToolMenu)
	{
		return;
	}

	const UAvaOutlinerItemsContext* OutlinerItemsContext = InToolMenu->Context.FindContext<UAvaOutlinerItemsContext>();

	if (!IsValid(OutlinerItemsContext))
	{
		return;
	}

	UPropertyAnimatorCoreEditorSubsystem* EditorSubsystem = UPropertyAnimatorCoreEditorSubsystem::Get();

	if (!IsValid(EditorSubsystem))
	{
		return;
	}

	TSet<UObject*> ContextObjects;
	GetContextObjects(OutlinerItemsContext, ContextObjects);

	const FPropertyAnimatorCoreEditorMenuContext MenuContext(ContextObjects, {});
	FPropertyAnimatorCoreEditorMenuOptions MenuOptions(
		{
			EPropertyAnimatorCoreEditorMenuType::NewAdvanced
			, EPropertyAnimatorCoreEditorMenuType::Enable
			, EPropertyAnimatorCoreEditorMenuType::Disable
			, EPropertyAnimatorCoreEditorMenuType::Delete
		}
	);
	MenuOptions.CreateSubMenu(true);

	EditorSubsystem->FillAnimatorMenu(InToolMenu, MenuContext, MenuOptions);
}

void FAvaPropertyAnimatorEditorOutlinerContextMenu::GetContextObjects(const UAvaOutlinerItemsContext* InContext, TSet<UObject*>& OutObjects)
{
	OutObjects.Empty();

	if (!IsValid(InContext) || InContext->GetItems().IsEmpty())
	{
		return;
	}

	for (FAvaOutlinerItemWeakPtr ItemWeak : InContext->GetItems())
	{
		const TSharedPtr<IAvaOutlinerItem> Item = ItemWeak.Pin();

		// is it an actor
		if (const FAvaOutlinerActor* ActorItem = Item->CastTo<FAvaOutlinerActor>())
		{
			if (AActor* Actor = ActorItem->GetActor())
			{
				OutObjects.Add(Actor);
			}
		}
		// is it a component
		else if (const FAvaPropertyAnimatorEditorOutlinerProxy* AnimatorProxy = Item->CastTo<FAvaPropertyAnimatorEditorOutlinerProxy>())
		{
			if (const UPropertyAnimatorCoreComponent* AnimatorComponent = AnimatorProxy->GetPropertyAnimatorComponent())
			{
				OutObjects.Add(AnimatorComponent->GetOwner());
			}
		}
		// is it an animator
		else if (const FAvaPropertyAnimatorEditorOutliner* AnimatorItem = Item->CastTo<FAvaPropertyAnimatorEditorOutliner>())
		{
			if (UPropertyAnimatorCoreBase* Animator = AnimatorItem->GetPropertyAnimator())
			{
				OutObjects.Add(Animator);
			}
		}
	}
}
