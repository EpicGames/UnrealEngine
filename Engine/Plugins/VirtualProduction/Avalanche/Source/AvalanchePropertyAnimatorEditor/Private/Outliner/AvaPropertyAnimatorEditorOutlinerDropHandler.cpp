// Copyright Epic Games, Inc. All Rights Reserved.

#include "Outliner/AvaPropertyAnimatorEditorOutlinerDropHandler.h"

#include "Components/PropertyAnimatorCoreComponent.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Item/AvaOutlinerActor.h"
#include "Outliner/AvaPropertyAnimatorEditorOutliner.h"
#include "Outliner/AvaPropertyAnimatorEditorOutlinerProxy.h"
#include "Subsystems/PropertyAnimatorCoreSubsystem.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "AvaPropertyAnimatorEditorOutlinerDropHandler"

DEFINE_LOG_CATEGORY_STATIC(LogAvaPropertyAnimatorEditorOutlinerDropHandler, Log, All);

bool FAvaPropertyAnimatorEditorOutlinerDropHandler::IsDraggedItemSupported(const FAvaOutlinerItemPtr& InDraggedItem) const
{
	return InDraggedItem->IsA<FAvaPropertyAnimatorEditorOutliner>() || InDraggedItem->IsA<FAvaPropertyAnimatorEditorOutlinerProxy>();
}

TOptional<EItemDropZone> FAvaPropertyAnimatorEditorOutlinerDropHandler::CanDrop(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem) const
{
	// Dropping on actor directly
	if (const FAvaOutlinerActor* const TargetActorItem = InTargetItem->CastTo<FAvaOutlinerActor>())
	{
		return CanDropOnActor(TargetActorItem->GetActor(), InDropZone);
	}

	// Dropping onto the animator proxy itself has the same effects as dropping on actor
	if (const FAvaPropertyAnimatorEditorOutlinerProxy* const TargetAnimatorProxy = InTargetItem->CastTo<FAvaPropertyAnimatorEditorOutlinerProxy>())
	{
		if (const UPropertyAnimatorCoreComponent* const AnimatorComponent = TargetAnimatorProxy->GetPropertyAnimatorComponent())
		{
			return CanDropOnActor(AnimatorComponent->GetOwner(), InDropZone);
		}

		return TOptional<EItemDropZone>();
	}

	// If target item is none of the above, nor an animator item, then it's not a supported target
	const FAvaPropertyAnimatorEditorOutliner* const TargetAnimatorItem = InTargetItem->CastTo<FAvaPropertyAnimatorEditorOutliner>();
	if (!TargetAnimatorItem || !TargetAnimatorItem->GetPropertyAnimator())
	{
		return TOptional<EItemDropZone>();
	}

	// Only clone animator on target, move not supported
	const UPropertyAnimatorCoreBase* TargetAnimator = TargetAnimatorItem->GetPropertyAnimator();
	TSet<UPropertyAnimatorCoreBase*> DraggedAnimators = GetDraggedAnimators();

	for (TSet<UPropertyAnimatorCoreBase*>::TIterator It(DraggedAnimators); It; ++It)
	{
		const UPropertyAnimatorCoreBase* Animator = *It;
		if (!Animator || Animator->GetAnimatorActor() == TargetAnimator->GetAnimatorActor())
		{
			It.RemoveCurrent();
		}
	}

	// If the animators are empty, return fail early
	if (DraggedAnimators.IsEmpty())
	{
		return TOptional<EItemDropZone>();
	}

	return InDropZone;
}

bool FAvaPropertyAnimatorEditorOutlinerDropHandler::Drop(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem)
{
	if (const FAvaOutlinerActor* const TargetActorItem = InTargetItem->CastTo<FAvaOutlinerActor>())
	{
		return DropAnimatorsOnActor(TargetActorItem->GetActor(), InDropZone);
	}

	// Dropping onto the animator proxy itself has the same effects as dropping on actor
	if (const FAvaPropertyAnimatorEditorOutlinerProxy* const TargetAnimatorProxy = InTargetItem->CastTo<FAvaPropertyAnimatorEditorOutlinerProxy>())
	{
		if (const UPropertyAnimatorCoreComponent* const AnimatorComponent = TargetAnimatorProxy->GetPropertyAnimatorComponent())
		{
			return DropAnimatorsOnActor(AnimatorComponent->GetOwner(), InDropZone);
		}

		return false;
	}

	// If target item is none of the above, nor an animator item, then it's not a supported target
	const FAvaPropertyAnimatorEditorOutliner* const TargetAnimatorItem = InTargetItem->CastTo<FAvaPropertyAnimatorEditorOutliner>();
	if (!TargetAnimatorItem || !TargetAnimatorItem->GetPropertyAnimator())
	{
		return false;
	}

	UPropertyAnimatorCoreBase* TargetAnimator = TargetAnimatorItem->GetPropertyAnimator();
	return DropAnimatorsOnAnimator(TargetAnimator, InDropZone);
}

TSet<UPropertyAnimatorCoreBase*> FAvaPropertyAnimatorEditorOutlinerDropHandler::GetDraggedAnimators() const
{
	TSet<UPropertyAnimatorCoreBase*> DraggedAnimators;

	// Rather than iterating separately, keep order of how they were dragged
	ForEachItem<IAvaOutlinerItem>([&DraggedAnimators](IAvaOutlinerItem& InItem)->EIterationResult
	{
		if (const FAvaPropertyAnimatorEditorOutlinerProxy* const AnimatorItemProxy = InItem.CastTo<FAvaPropertyAnimatorEditorOutlinerProxy>())
		{
			if (const UPropertyAnimatorCoreComponent* const AnimatorComponent = AnimatorItemProxy->GetPropertyAnimatorComponent())
			{
				AnimatorComponent->ForEachAnimator([&DraggedAnimators](UPropertyAnimatorCoreBase* InAnimator)
				{
					DraggedAnimators.Add(InAnimator);
					return true;
				});
			}
		}
		else if (const FAvaPropertyAnimatorEditorOutliner* const AnimatorItem = InItem.CastTo<FAvaPropertyAnimatorEditorOutliner>())
		{
			if (UPropertyAnimatorCoreBase* const Animator = AnimatorItem->GetPropertyAnimator())
			{
				DraggedAnimators.Add(Animator);
			}
		}

		return EIterationResult::Continue;
	});

	return DraggedAnimators;
}

TOptional<EItemDropZone> FAvaPropertyAnimatorEditorOutlinerDropHandler::CanDropOnActor(AActor* InActor, EItemDropZone InDropZone) const
{
	if (!IsValid(InActor))
	{
		return TOptional<EItemDropZone>();
	}

	const UPropertyAnimatorCoreSubsystem* const AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get();
	if (!AnimatorSubsystem)
	{
		return TOptional<EItemDropZone>();
	}

	TSet<UPropertyAnimatorCoreBase*> DraggedAnimators = GetDraggedAnimators();

	for (TSet<UPropertyAnimatorCoreBase*>::TIterator It(DraggedAnimators); It; ++It)
	{
		const UPropertyAnimatorCoreBase* Animator = *It;
		if (!Animator || Animator->GetAnimatorActor() == InActor)
		{
			It.RemoveCurrent();
		}
	}

	// For actor items, drop zone can only be onto the actor
	return !DraggedAnimators.IsEmpty()
		? EItemDropZone::OntoItem
		: TOptional<EItemDropZone>();
}

bool FAvaPropertyAnimatorEditorOutlinerDropHandler::DropAnimatorsOnActor(AActor* InActor, EItemDropZone InDropZone) const
{
	const UPropertyAnimatorCoreSubsystem* const AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get();

	if (!AnimatorSubsystem)
	{
		return false;
	}

	TSet<UPropertyAnimatorCoreBase*> CloneAnimators = GetDraggedAnimators();

	// Remove animators that are already on the target actor
	for (TSet<UPropertyAnimatorCoreBase*>::TIterator It(CloneAnimators); It; ++It)
	{
		const UPropertyAnimatorCoreBase* Animator = *It;
		if (!Animator || Animator->GetAnimatorActor() == InActor)
		{
			It.RemoveCurrent();
		}
	}

	if (CloneAnimators.IsEmpty())
	{
		return false;
	}

	UE_LOGF(LogAvaPropertyAnimatorEditorOutlinerDropHandler, Log, "Dropping %i animator(s) on actor %ls"
		, CloneAnimators.Num()
		, *InActor->GetActorNameOrLabel());

	const TSet<UPropertyAnimatorCoreBase*> NewAnimators = AnimatorSubsystem->CloneAnimators(CloneAnimators, InActor, true);

	if (NewAnimators.Num() != CloneAnimators.Num())
	{
		UE_LOGF(LogAvaPropertyAnimatorEditorOutlinerDropHandler, Warning, "%ls : Could not clone all %i animators to target actor", *InActor->GetActorNameOrLabel(), CloneAnimators.Num());

		FNotificationInfo NotificationInfo(LOCTEXT("CloneAnimatorsFail", "An issue occured while cloning animators on an actor"));
		NotificationInfo.ExpireDuration = 3.f;
		NotificationInfo.bFireAndForget = true;
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);
	}

	// When ALT is pressed : copy animators on target actor
	// When ALT is not pressed : move animators on target actor (copy + delete)
	if (!FSlateApplication::Get().GetModifierKeys().IsAltDown())
	{
		if (!AnimatorSubsystem->RemoveAnimators(CloneAnimators, /** Transact */true))
		{
			UE_LOGF(LogAvaPropertyAnimatorEditorOutlinerDropHandler, Warning, "Could not remove the %i cloned animators on source actor", CloneAnimators.Num());

			FNotificationInfo NotificationInfo(LOCTEXT("RemoveAnimatorsFail", "An issue occured while removing animators on an actor"));
			NotificationInfo.ExpireDuration = 3.f;
			NotificationInfo.bFireAndForget = true;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
	}

	return NewAnimators.Num() > 0;
}

bool FAvaPropertyAnimatorEditorOutlinerDropHandler::DropAnimatorsOnAnimator(UPropertyAnimatorCoreBase* InTargetAnimator, EItemDropZone InDropZone) const
{
	if (!IsValid(InTargetAnimator))
	{
		return false;
	}

	return DropAnimatorsOnActor(InTargetAnimator->GetAnimatorActor(), InDropZone);
}

#undef LOCTEXT_NAMESPACE
