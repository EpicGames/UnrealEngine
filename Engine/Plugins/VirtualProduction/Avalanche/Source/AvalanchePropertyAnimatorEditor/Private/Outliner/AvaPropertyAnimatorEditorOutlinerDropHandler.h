// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragDropOps/Handlers/AvaOutlinerItemDropHandler.h"

class UPropertyAnimatorCoreBase;

/** Class that handles dropping animator item into a target item */
class FAvaPropertyAnimatorEditorOutlinerDropHandler : public FAvaOutlinerItemDropHandler
{
public:
	UE_AVA_INHERITS(FAvaPropertyAnimatorEditorOutlinerDropHandler, FAvaOutlinerItemDropHandler);

protected:
	//~ Begin FAvaOutlinerItemDropHandler
	virtual bool IsDraggedItemSupported(const FAvaOutlinerItemPtr& InDraggedItem) const override;
	virtual TOptional<EItemDropZone> CanDrop(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem) const override;
	virtual bool Drop(EItemDropZone InDropZone, FAvaOutlinerItemPtr InTargetItem) override;
	//~ End FAvaOutlinerItemDropHandler

	TSet<UPropertyAnimatorCoreBase*> GetDraggedAnimators() const;
	TOptional<EItemDropZone> CanDropOnActor(AActor* InActor, EItemDropZone InDropZone) const;
	bool DropAnimatorsOnActor(AActor* InActor, EItemDropZone InDropZone) const;
	bool DropAnimatorsOnAnimator(UPropertyAnimatorCoreBase* InTargetAnimator, EItemDropZone InDropZone) const;
};
