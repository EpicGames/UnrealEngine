// Copyright Epic Games, Inc. All Rights Reserved.

#include "Outliner/AvaPropertyAnimatorEditorOutlinerProxy.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Components/PropertyAnimatorCoreComponent.h"
#include "IAvaOutliner.h"
#include "Item/AvaOutlinerActor.h"
#include "Outliner/AvaPropertyAnimatorEditorOutliner.h"
#include "Selection/AvaOutlinerScopedSelection.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/PropertyAnimatorCoreSubsystem.h"

#define LOCTEXT_NAMESPACE "AvaPropertyAnimatorEditorOutlinerProxy"

FAvaPropertyAnimatorEditorOutlinerProxy::FAvaPropertyAnimatorEditorOutlinerProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem)
	: Super(InOutliner, InParentItem)
{
	ItemIcon = FSlateIconFinder::FindIconForClass(UPropertyAnimatorCoreComponent::StaticClass());
}

UPropertyAnimatorCoreComponent* FAvaPropertyAnimatorEditorOutlinerProxy::GetPropertyAnimatorComponent() const
{
	const FAvaOutlinerItemPtr Parent = GetParent();

	if (!Parent.IsValid())
	{
		return nullptr;
	}

	const FAvaOutlinerActor* const ActorItem = Parent->CastTo<FAvaOutlinerActor>();

	if (!ActorItem)
	{
		return nullptr;
	}

	const AActor* Actor = ActorItem->GetActor();

	if (!Actor)
	{
		return nullptr;
	}

	return Actor->FindComponentByClass<UPropertyAnimatorCoreComponent>();
}

void FAvaPropertyAnimatorEditorOutlinerProxy::OnItemRegistered()
{
	Super::OnItemRegistered();
	BindDelegates();
}

void FAvaPropertyAnimatorEditorOutlinerProxy::OnItemUnregistered()
{
	Super::OnItemUnregistered();
	UnbindDelegates();
}

void FAvaPropertyAnimatorEditorOutlinerProxy::Select(FAvaOutlinerScopedSelection& InSelection) const
{
	if (UPropertyAnimatorCoreComponent* const PropertyAnimatorComponent = GetPropertyAnimatorComponent())
	{
		InSelection.Select(PropertyAnimatorComponent);
	}
}

FText FAvaPropertyAnimatorEditorOutlinerProxy::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Animators");
}

FSlateIcon FAvaPropertyAnimatorEditorOutlinerProxy::GetIcon() const
{
	return ItemIcon;
}

FText FAvaPropertyAnimatorEditorOutlinerProxy::GetIconTooltipText() const
{
	return LOCTEXT("Tooltip", "Shows all the animators found in the property animator component of an actor");
}

bool FAvaPropertyAnimatorEditorOutlinerProxy::CanDelete() const
{
	return IsValid(GetPropertyAnimatorComponent());
}

bool FAvaPropertyAnimatorEditorOutlinerProxy::Delete()
{
	const UPropertyAnimatorCoreComponent* AnimatorComponent = GetPropertyAnimatorComponent();
	const UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get();

	if (IsValid(AnimatorComponent) && AnimatorSubsystem)
	{
		return AnimatorSubsystem->RemoveAnimators(TSet<UPropertyAnimatorCoreBase*>{AnimatorComponent->GetAnimators()}, /** Transact */false);
	}

	return false;
}

void FAvaPropertyAnimatorEditorOutlinerProxy::GetProxiedItems(const TSharedRef<IAvaOutlinerItem>& InParent
	, TArray<FAvaOutlinerItemPtr>& OutChildren, bool bInRecursive)
{
	if (const UPropertyAnimatorCoreComponent* const PropertyAnimatorComponent = GetPropertyAnimatorComponent())
	{
		for (UPropertyAnimatorCoreBase* const PropertyAnimator : PropertyAnimatorComponent->GetAnimators())
		{
			if (!PropertyAnimator)
			{
				continue;
			}

			const FAvaOutlinerItemPtr AnimatorItem = Outliner.FindOrAdd<FAvaPropertyAnimatorEditorOutliner>(PropertyAnimator);
			AnimatorItem->SetParent(SharedThis(this));

			OutChildren.Add(AnimatorItem);

			if (bInRecursive)
			{
				AnimatorItem->FindChildren(OutChildren, bInRecursive);
			}
		}
	}
}

void FAvaPropertyAnimatorEditorOutlinerProxy::BindDelegates()
{
	UnbindDelegates();
	UPropertyAnimatorCoreBase::OnPropertyAnimatorAdded().AddSP(this, &FAvaPropertyAnimatorEditorOutlinerProxy::OnPropertyAnimatorUpdated);
	UPropertyAnimatorCoreBase::OnPropertyAnimatorRemoved().AddSP(this, &FAvaPropertyAnimatorEditorOutlinerProxy::OnPropertyAnimatorUpdated);
	UPropertyAnimatorCoreBase::OnPropertyAnimatorRenamed().AddSP(this, &FAvaPropertyAnimatorEditorOutlinerProxy::OnPropertyAnimatorUpdated);
}

void FAvaPropertyAnimatorEditorOutlinerProxy::UnbindDelegates()
{
	UPropertyAnimatorCoreBase::OnPropertyAnimatorAdded().RemoveAll(this);
	UPropertyAnimatorCoreBase::OnPropertyAnimatorRemoved().RemoveAll(this);
	UPropertyAnimatorCoreBase::OnPropertyAnimatorRenamed().RemoveAll(this);
}

void FAvaPropertyAnimatorEditorOutlinerProxy::OnPropertyAnimatorUpdated(UPropertyAnimatorCoreComponent* InComponent, UPropertyAnimatorCoreBase* InAnimator, EPropertyAnimatorCoreUpdateEvent InReason)
{
	const UPropertyAnimatorCoreComponent* ActiveAnimatorComponent = GetPropertyAnimatorComponent();

	if (IsValid(InAnimator) && IsValid(ActiveAnimatorComponent))
	{
		if (InComponent == ActiveAnimatorComponent)
		{
			RefreshChildren();
			Outliner.RequestRefresh();
		}
	}
}

#undef LOCTEXT_NAMESPACE
