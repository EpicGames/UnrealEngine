// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Item/AvaOutlinerItemProxy.h"
#include "Textures/SlateIcon.h"

class UPropertyAnimatorCoreBase;
class UPropertyAnimatorCoreComponent;
enum class EPropertyAnimatorCoreUpdateEvent : uint8;

/** Creates property animator based on all the item found in the component of an Actor */
class FAvaPropertyAnimatorEditorOutlinerProxy : public FAvaOutlinerItemProxy
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaPropertyAnimatorEditorOutlinerProxy, FAvaOutlinerItemProxy);

	FAvaPropertyAnimatorEditorOutlinerProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem);

	UPropertyAnimatorCoreComponent* GetPropertyAnimatorComponent() const;

	//~ Begin IAvaOutlinerItem
	virtual void OnItemRegistered() override;
	virtual void OnItemUnregistered() override;
	virtual void Select(FAvaOutlinerScopedSelection& InSelection) const override;
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	virtual FText GetIconTooltipText() const override;
	virtual bool CanDelete() const override;
	virtual bool Delete() override;
	//~ End IAvaOutlinerItem

	//~ Begin FAvaOutlinerItemProxy
	virtual void GetProxiedItems(const TSharedRef<IAvaOutlinerItem>& InParent, TArray<FAvaOutlinerItemPtr>& OutChildren, bool bInRecursive) override;
	//~ End FAvaOutlinerItemProxy

protected:
	void BindDelegates();
	void UnbindDelegates();

	void OnPropertyAnimatorUpdated(UPropertyAnimatorCoreComponent* InComponent, UPropertyAnimatorCoreBase* InAnimator, EPropertyAnimatorCoreUpdateEvent InReason);

	FSlateIcon ItemIcon;
};
