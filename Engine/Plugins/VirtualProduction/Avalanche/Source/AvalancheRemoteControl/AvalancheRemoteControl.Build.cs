// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheRemoteControl : ModuleRules
{
    public AvalancheRemoteControl(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "Engine",
                "RemoteControl",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "RemoteControlComponents", 
                "RemoteControlLogic",
                "Slate",
                "SlateCore",
            }
        );

        ShortName = "AvRC";
    }
}