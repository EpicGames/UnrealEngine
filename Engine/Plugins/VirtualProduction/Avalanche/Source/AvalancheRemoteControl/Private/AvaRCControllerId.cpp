// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRCControllerId.h"
#include "RCVirtualProperty.h"
#include "RemoteControlPreset.h"

FAvaRCControllerId::FAvaRCControllerId(URCVirtualPropertyBase* InController)
	: Name(InController ? InController->DisplayName : NAME_None)
{
}

URCVirtualPropertyBase* FAvaRCControllerId::FindController(URemoteControlPreset* InPreset) const
{
	if (InPreset)
	{
		return InPreset->GetControllerByDisplayName(Name);
	}
	return nullptr;
}

FText FAvaRCControllerId::ToText() const
{
	return FText::FromName(Name);
}
