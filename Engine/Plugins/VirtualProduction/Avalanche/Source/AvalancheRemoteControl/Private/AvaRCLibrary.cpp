// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRCLibrary.h"
#include "Action/RCAction.h"
#include "Behaviour/RCBehaviour.h"
#include "Controller/RCController.h"
#include "Engine/World.h"
#include "RemoteControlPreset.h"

TArray<AActor*> UAvaRCLibrary::GetControlledActors(UObject* InWorldContextObject, URCVirtualPropertyBase* InController)
{
	if (!InWorldContextObject)
	{
		return {};
	}

	/**
	 * UObject::GetWorld will return the OwningWorld for Actors/Components/Levels. Instead, RC Bindings for streamed in actors
	 * Will be based on the Streamed World (i.e. world outer), instead of the owning World.
	 */
	const UWorld* World = InWorldContextObject->IsA<UWorld>()
		? CastChecked<UWorld>(InWorldContextObject)
		: InWorldContextObject->GetTypedOuter<UWorld>();

	if (!World)
	{
		return {};
	}

	URCController* Controller = Cast<URCController>(InController);
	if (!Controller)
	{
		return {};
	}

	URemoteControlPreset* Preset = Controller->GetTypedOuter<URemoteControlPreset>();
	if (!Preset)
	{
		return {};
	}

	TArray<AActor*> ControlledActors;

	for (URCBehaviour* Behavior : Controller->GetBehaviors())
	{
		if (!Behavior || !Behavior->ActionContainer)
		{
			continue;
		}

		Behavior->ActionContainer->ForEachAction([Preset, World, &ControlledActors](URCAction* InAction)
			{
				TSharedPtr<FRemoteControlEntity> Entity = Preset->GetExposedEntity(InAction->ExposedFieldId).Pin();
				if (!Entity.IsValid())
				{
					return;
				}

				UObject* Object = Entity->GetBoundObjectForWorld(World);
				if (!Object)
				{
					return;
				}

				if (AActor* Actor = Cast<AActor>(Object))
				{
					ControlledActors.Add(Actor);
				}
				else if (AActor* OuterActor = Object->GetTypedOuter<AActor>())
				{
					ControlledActors.Add(OuterActor);
				}
			}
			, /*bRecursive*/true);
	}

	return ControlledActors;
}
