// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRemoteControlRebind.h"
#include "EngineUtils.h"
#include "RemoteControlActor.h"
#include "RemoteControlPreset.h"

DEFINE_LOG_CATEGORY_STATIC(LogAvaRemoteControlRebind, Log, All);

namespace UE::AvaRemoteControl::Private
{
	const FRemoteControlInitialBindingContext* GetBindingContext(URemoteControlLevelDependantBinding* InLevelDependantBinding)
	{
		const FStructProperty* const BindingContextProperty = FindFProperty<FStructProperty>(InLevelDependantBinding->GetClass(), TEXT("BindingContext"));
		if (BindingContextProperty && BindingContextProperty->Struct->IsChildOf(FRemoteControlInitialBindingContext::StaticStruct()))
		{
			return BindingContextProperty->ContainerPtrToValuePtr<FRemoteControlInitialBindingContext>(InLevelDependantBinding);
		}
		return nullptr;
	}
	
	static FTopLevelAssetPath GetSupportedClassPathFromBindings(const TArray<TWeakObjectPtr<URemoteControlBinding>>& InBindings)
	{
		for (const TWeakObjectPtr<URemoteControlBinding>& Binding : InBindings)
		{
			if (!Binding.IsValid())
			{
				continue;
			}
			
			URemoteControlLevelDependantBinding* LevelDependantBinding = Cast<URemoteControlLevelDependantBinding>(Binding.Get());
			if (!LevelDependantBinding)
			{
				continue;
			}

			const FRemoteControlInitialBindingContext* const BindingContext = GetBindingContext(LevelDependantBinding);
			if (!BindingContext)
			{
				continue;
			}

			if (!BindingContext->SupportedClass)
			{
				continue;
			}

			return BindingContext->SupportedClass->GetClassPathName();
		}

		return FTopLevelAssetPath();
	}	
}

/**
 *	Since the runtime Motion Design Scene Object is not in the world where the
 *	remote control bindings were done, it needs to be rebound.
 *	
 *	The original FRemoteControlPresetRebindingManager::Rebind_NewAlgo
 *	prevents rebinding with transient actors, in  RCPresetRebindingManager::GetActorsOfClass.
 *
 *	So, we need to pre-rebind the bindings and allow transient objects.
 */
class FAvaRemoteControlRebindPrivateHelper
{
public:
	TSet<TPair<AActor*, UObject*>> BoundObjects;
	FString BindFailureReason;

	/**
	 * Version of the function returning only actors from the specified level.
	 * This is required for support Motion Design graphics instancing as sub-levels.
	 * There is one RCP per sub-level and must resolve only to that level.
	 */
	static bool GetActorsOfClass(const ULevel* InLevel, const UClass* InTargetClass, TArray<AActor*>& OutActors)
	{
		if (InLevel)
		{
			OutActors.Reserve(InLevel->Actors.Num());
			for (const TObjectPtr<AActor>& Actor : InLevel->Actors)
			{
				if (IsValid(Actor.Get()) && Actor->GetClass()->IsChildOf(InTargetClass))
				{
					OutActors.Add(Actor);
				}
			}
		}
		return !OutActors.IsEmpty();
	}

	/**
	 *	The original function RCPresetRebindingManager::GetActorsOfClass
	 *	prevents rebinding with transient actors. All the runtime RCP from embedded
	 *	Motion Design graphics are transient. This is the reason this has to be rewritten.
	 */
	static bool GetActorsOfClass(UWorld* InPresetWorld, UClass* InTargetClass, TArray<AActor*>& OutActors)
	{
		if (InPresetWorld)
		{
			for (TActorIterator<AActor> It(InPresetWorld, InTargetClass, EActorIteratorFlags::SkipPendingKill); It; ++It)
			{
				// Note: In original Remote Control Binding code, this is where the
				// transient actors would be filtered out.
				if (*It)
				{
					OutActors.Add(*It);
				}
			}
		}
		return !OutActors.IsEmpty();
	}

	static UObject* FindByName(const TArray<AActor*>& InPotentialMatches, FName InTargetName)
	{
		for (AActor* Actor : InPotentialMatches)
		{
			// Attempt finding by name.
			if (Actor->GetFName() == InTargetName)
			{
				return Actor;
			}
		}
		return nullptr;
	}

	bool TryRebindPair(URemoteControlLevelDependantBinding* InBindingToRebind, AActor* InActor, UObject* InSubObject)
	{
		const TPair<AActor*, UObject*> Pair = TPair<AActor*, UObject*> { InActor, InSubObject };
		if (!BoundObjects.Contains(Pair))
		{
			BoundObjects.Add(Pair);
			InBindingToRebind->Modify();
			if (InSubObject)
			{
				InBindingToRebind->SetBoundObject(InSubObject);
			}
			else
			{
				InBindingToRebind->SetBoundObject(InActor);
			}
			return true;
		}

		BindFailureReason = FString::Printf(TEXT("Another binding is already bound to {Owner:%s, SubObject:%s}.\n"),
			*GetPathNameSafe(InActor), *GetPathNameSafe(InSubObject));
		return false;
	}
	
	static FString GetSubObjectPath(const FRemoteControlInitialBindingContext* InBindingContext)
	{
		return InBindingContext->HasValidSubObjectPath() ? InBindingContext->SubObjectPath : InBindingContext->ComponentName.ToString();
	}

	UObject* GetSubObjectBasedOnName(const FRemoteControlInitialBindingContext* InBindingContext, UObject* InObject) const
	{
		UObject* TargetSubObject = InBindingContext->HasValidSubObjectPath() ? FindObject<UObject>(InObject, *InBindingContext->SubObjectPath) : nullptr;
		if (!TargetSubObject)
		{
			const FName InitialComponentName = InBindingContext->ComponentName;
			TargetSubObject = FindObject<UObject>(InObject, *InitialComponentName.ToString());
		}

		if (TargetSubObject && TargetSubObject->GetClass()->IsChildOf(InBindingContext->SupportedClass.LoadSynchronous()))
		{
			return TargetSubObject;
		}
		return nullptr;
	};

	bool RebindComponentBasedOnClass(const FRemoteControlInitialBindingContext* InBindingContext, URemoteControlLevelDependantBinding* InBinding, AActor* InActorMatch)
	{
		if (UClass* SupportedClass = InBindingContext->SupportedClass.LoadSynchronous())
		{
			if (InActorMatch && SupportedClass->IsChildOf(UActorComponent::StaticClass()))
			{
				TArray<UActorComponent*> Components;
				InActorMatch->GetComponents(SupportedClass, Components);

				for (UActorComponent* Component : Components)
				{
					if (TryRebindPair(InBinding, Component->GetOwner(), Component))
					{
						return true;
					}
				}
			}
		}
		return false;
	};

	bool TryRebindSubObject(const FRemoteControlInitialBindingContext* InBindingContext, URemoteControlLevelDependantBinding* InBindingToRebind, UObject* InPotentialMatch)
	{
		if (UObject* TargetSubObject = GetSubObjectBasedOnName(InBindingContext, InPotentialMatch))
		{
			return TryRebindPair(InBindingToRebind, TargetSubObject->GetTypedOuter<AActor>(), TargetSubObject);
		}

		const bool bBound = RebindComponentBasedOnClass(InBindingContext, InBindingToRebind, Cast<AActor>(InPotentialMatch));
		if (bBound)
		{
			UE_LOGF(LogAvaRemoteControlRebind, Warning,
				"Binding \"%ls\": Could not find component (\"%ls\") by name nor by path (\"%ls\"). Bound to \"%ls\" by class instead.",
				*InBindingToRebind->Name, *InBindingContext->ComponentName.ToString(), *InBindingContext->SubObjectPath,
				*InBindingToRebind->GetLastBoundObjectPath().ToString());
		}
		else
		{
			BindFailureReason = FString::Printf(TEXT("Binding by component class failed: Actor: %s had no components of class %s.\n"), *InPotentialMatch->GetFullName(), *InBindingContext->SupportedClass.ToString());
		}
		return bBound;
	}

	bool FindSubObjectToRebind(const FRemoteControlInitialBindingContext* InBindingContext, URemoteControlLevelDependantBinding* InBindingToRebind, const TArray<AActor*>& InObjectsWithSupportedClass)
	{
		if (UObject* ActorMatch = FindByName(InObjectsWithSupportedClass, InBindingContext->OwnerActorName))
		{
			// Found an owner object with matching name. Try to find a component under it with a matching name.
			if (!TryRebindSubObject(InBindingContext, InBindingToRebind, ActorMatch))
			{
				return false;
			}
			return true;
		}
		else
		{
			// Could not find an actor with the same name as the initial binding, rely only on class instead.
			for (UObject* Object : InObjectsWithSupportedClass)
			{
				if (TryRebindSubObject(InBindingContext, InBindingToRebind, Object))
				{
					UE_LOGF(LogAvaRemoteControlRebind, Warning,
						"Binding \"%ls\": Could not bind actor by name (\"%ls\"). Bound to \"%ls\" by class instead.",
							*InBindingToRebind->Name, *InBindingContext->OwnerActorName.ToString(), *InBindingToRebind->GetLastBoundObjectPath().ToString());
					return true;
				}
			}

			BindFailureReason = FString::Printf(TEXT("Bind by SubObjectPath: Actor \"%s\" not found, and couldn't find sub-object \"%s\" under any other actors of class \"%s\"."),
				*InBindingContext->OwnerActorName.ToString(), *GetSubObjectPath(InBindingContext), *InBindingContext->OwnerActorClass.ToString());
		}
		return false;
	}	

	bool RebindUsingActorName(const FRemoteControlInitialBindingContext* InBindingContext, URemoteControlLevelDependantBinding* InBindingToRebind, const TArray<AActor*>& InObjectsWithSupportedClass)
	{
		if (UObject* Match = FindByName(InObjectsWithSupportedClass, InBindingContext->OwnerActorName))
		{
			return TryRebindPair(InBindingToRebind, Cast<AActor>(Match), nullptr);
		}
		return false;
	}

	bool RebindUsingClass(URemoteControlLevelDependantBinding* InBindingToRebind, const TArray<AActor*>& InObjectsWithSupportedClass)
	{
		for (UObject* PotentialMatch : InObjectsWithSupportedClass)
		{
			if (TryRebindPair(InBindingToRebind, Cast<AActor>(PotentialMatch), nullptr))
			{
				return true;
			}
		}
		return false;
	}

	bool Rebind(URemoteControlLevelDependantBinding* InBindingToRebind, const ULevel* InLevel, const URemoteControlPreset* InPreset)
	{
		using namespace UE::AvaRemoteControl::Private;
		const FRemoteControlInitialBindingContext* const BindingContext = GetBindingContext(InBindingToRebind);
		if (!BindingContext)
		{
			BindFailureReason = FString::Printf(TEXT("Can't get binding context."));
			return false;
		}

		UClass* OwnerClass = BindingContext->OwnerActorClass.LoadSynchronous();
		if (!OwnerClass)
		{
			BindFailureReason = FString::Printf(TEXT("Can't load owner class \"%s\". Missing plugin?"), *BindingContext->OwnerActorClass.ToString());
			return false;
		}
		
		TArray<AActor*> ObjectsWithSupportedClass;
		if (InLevel)
		{
			if (!GetActorsOfClass(InLevel, OwnerClass, ObjectsWithSupportedClass))
			{
				BindFailureReason = FString::Printf(TEXT("Level \"%s\" doesn't have actors of class \"%s\"."), *InLevel->GetName(), *BindingContext->OwnerActorClass.ToString());
				return false;
			}
		}
		else
		{
			if (!GetActorsOfClass(InPreset->GetWorld(false), OwnerClass, ObjectsWithSupportedClass))
			{
				BindFailureReason = FString::Printf(TEXT("World doesn't have actors of class \"%s\"."), *BindingContext->OwnerActorClass.ToString());
				return false;
			}
		}
		
		if (BindingContext->HasValidSubObjectPath() || BindingContext->HasValidComponentName())
		{
			return FindSubObjectToRebind(BindingContext, InBindingToRebind, ObjectsWithSupportedClass);
		}
		else
		{
			if (!RebindUsingActorName(BindingContext, InBindingToRebind, ObjectsWithSupportedClass))
			{
				if (RebindUsingClass(InBindingToRebind, ObjectsWithSupportedClass))
				{
					UE_LOGF(LogAvaRemoteControlRebind, Warning,
						"Binding \"%ls\": Could not bind actor by name (\"%ls\"). Bound to \"%ls\" by class instead.",
						*InBindingToRebind->Name, *BindingContext->OwnerActorName.ToString(), *InBindingToRebind->GetLastBoundObjectPath().ToString());
					return true;
				}
				else
				{
					BindFailureReason = FString::Printf(TEXT("Failed to bind by name (\"%s\") or class (\"%s\"."),
						*BindingContext->OwnerActorName.ToString(), *BindingContext->OwnerActorClass.ToString());
				}
			}
			else
			{
				return true;
			}
		}
		return false;
	}

	FString GetBindingDescription(URemoteControlBinding* InBinding) const
	{
		using namespace UE::AvaRemoteControl::Private;
		if (URemoteControlLevelDependantBinding* LevelDependantBinding = Cast<URemoteControlLevelDependantBinding>(InBinding))
		{
			if (const FRemoteControlInitialBindingContext* BindingContext = GetBindingContext(LevelDependantBinding))
			{
				// will use sub-object path to bind. (and class)
				if (BindingContext->HasValidSubObjectPath())
				{
					return FString::Printf(TEXT("Name: %s - Actor: %s, SubObject: %s"),
						*InBinding->Name, *BindingContext->OwnerActorName.ToString(), *BindingContext->SubObjectPath);
				}
				// will use component name to bind. (and class)
				if (BindingContext->HasValidComponentName())
				{
					return FString::Printf(TEXT("Name: %s - Actor: %s, Component: %s"),
						*InBinding->Name, *BindingContext->OwnerActorName.ToString(), *BindingContext->ComponentName.ToString());
				}
				return FString::Printf(TEXT("Name: %s - Actor: %s, Class: %s (subobject path invalid)"),
					*InBinding->Name, *BindingContext->OwnerActorName.ToString(), *BindingContext->OwnerActorClass.ToString());
			}
		}
		
		// todo: other types of bindings.
		return FString::Printf(TEXT("Name: %s - Not level dependent."), *InBinding->Name);
	}
	
	static FString ListEntitiesForBinding(URemoteControlPreset* InPreset, const URemoteControlBinding* InBinding)
	{
		FString Entities;
		for (TWeakPtr<const FRemoteControlEntity> EntityWeak : InPreset->GetExposedEntities())
		{
			if (const TSharedPtr<const FRemoteControlEntity> Entity = EntityWeak.Pin())
			{
				for (const TWeakObjectPtr<URemoteControlBinding> Binding : Entity->GetBindings())
				{
					if (Binding.Get() == InBinding)
					{
						if (!Entities.IsEmpty())
						{
							Entities += TEXT(", ");
						}
						Entities += Entity->GetLabel().ToString();
					}
				}
			}
		}
		return Entities;
	}

	void FixupBindings(URemoteControlPreset* InPreset, const ULevel* InLevel)
	{
		using namespace UE::AvaRemoteControl::Private;
		
		// Fixup the bindings so they don't fail.
		for (URemoteControlBinding* Binding : InPreset->Bindings)
		{
			URemoteControlLevelDependantBinding* LevelDependantBinding = Cast<URemoteControlLevelDependantBinding>(Binding);
			if (!LevelDependantBinding)
			{
				UE_LOGF(LogAvaRemoteControlRebind, Warning, "Binding \"%ls\" is of type \"%ls\" expected \"URemoteControlLevelDependantBinding\".",
					*Binding->Name, *Binding->GetClass()->GetFName().ToString());
				continue;
			}

			BindFailureReason = FString();
			if (!Rebind(LevelDependantBinding, InLevel, InPreset))
			{
				// Print why the binding failed.
				UE_LOGF(LogAvaRemoteControlRebind, Error, "Binding {%ls} for entities {%ls} failed: %ls",
					*GetBindingDescription(LevelDependantBinding), *ListEntitiesForBinding(InPreset, LevelDependantBinding), *BindFailureReason); 
			}
			
			// Remark: not implementing the NDisplay rebind for now, but maybe it is needed.
		}
	}
};

void FAvaRemoteControlRebind::RebindUnboundEntities(URemoteControlPreset* InPreset, const ULevel* InLevel)
{
	if (InPreset)
	{
		FAvaRemoteControlRebindPrivateHelper Helper;
		Helper.FixupBindings(InPreset, InLevel);
		
		InPreset->RebindUnboundEntities();
		ResolveAllFieldPathInfos(InPreset);
	}
}

void FAvaRemoteControlRebind::ResolveAllFieldPathInfos(URemoteControlPreset* InPreset)
{
	check(InPreset);
	TArray<TWeakPtr<FRemoteControlField>> RCFields = InPreset->GetExposedEntities<FRemoteControlField>();
	for (TWeakPtr<FRemoteControlField> RCFieldWeak : RCFields)
	{			
		if (const TSharedPtr<FRemoteControlField> RCField = RCFieldWeak.Pin())
		{
			if (!RCField->FieldPathInfo.IsResolved())
			{
				bool bResolved = false;

				const TArray<UObject*> Objects = RCField->GetBoundObjects();
				if (Objects.Num() != 0)
				{
					bResolved = RCField->FieldPathInfo.Resolve(Objects[0]);
				}

				if (!bResolved)
				{
					// Logging unresolved fields.
					using namespace UE::AvaRemoteControl::Private;
					UE_LOGF(LogAvaRemoteControlRebind, Error, "Unresolved Field \"%ls\" of class \"%ls\".",
						*RCField->FieldPathInfo.ToString(),
						*GetSupportedClassPathFromBindings(RCField->GetBindings()).ToString());
				}
			}
		}
	}
}