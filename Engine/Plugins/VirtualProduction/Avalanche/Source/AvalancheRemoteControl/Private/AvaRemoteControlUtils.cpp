// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRemoteControlUtils.h"
#include "Engine/Level.h"
#include "IRemoteControlModule.h"
#include "RemoteControlPreset.h"
#include "Subsystems/RemoteControlComponentsSubsystem.h"

DEFINE_LOG_CATEGORY_STATIC(LogAvaRemoteControlUtils, Log, All);

bool FAvaRemoteControlUtils::RegisterRemoteControlPreset(URemoteControlPreset* InRemoteControlPreset, bool bInEnsureUniqueId)
{
	// Skip registering RCP with invalid controller containers. Web RC expects valid controller container.
	if (!InRemoteControlPreset || !ensure(InRemoteControlPreset->IsControllerContainerValid()))
	{
		return false;
	}

	IRemoteControlModule& RemoteControlModule = IRemoteControlModule::Get();

	// First thing is to ensure the preset is not already registered.
	// Registering the same preset multiple time is not terribly bad, but it is not good either.
	if (const URemoteControlPreset* ResolvedPresetByName = RemoteControlModule.ResolvePreset(InRemoteControlPreset->GetPresetName()))
	{
		if (ResolvedPresetByName == InRemoteControlPreset)
		{
			UE_LOGF(LogAvaRemoteControlUtils, Verbose, "Preset \"%ls\" (id:%ls) is already registered.",
				*InRemoteControlPreset->GetPresetName().ToString(), *InRemoteControlPreset->GetPresetId().ToString());
			return true;	// Already registered.
		}
	}

	if (bInEnsureUniqueId)
	{
		// Check if a preset with the same Id is already registered (but is not the same).
		if (const URemoteControlPreset* RegisteredPresetById = RemoteControlModule.ResolvePreset(InRemoteControlPreset->GetPresetId()))
		{
			// That should be detected in the first test. If this happens, should investigate.
			if (RegisteredPresetById == InRemoteControlPreset)
			{
				UE_LOGF(LogAvaRemoteControlUtils, Warning, "Preset \"%ls\" (id:%ls) is already registered. Can't register again.",
					*InRemoteControlPreset->GetPresetName().ToString(), *InRemoteControlPreset->GetPresetId().ToString());
				return true;	// Already registered.
			}
		
			// Workaround to enable multiple instances of the same RCP to be registered.
			// This is a normal occurence with Motion Design graphics that are meant to be templates that are used in
			// multiple pages. Because of page transitions, we must support having multiple instances of the same template
			// running together at the same time.
			// We need to assign a new unique ID otherwise all the RCP instances will have the
			// same id and will collide in the RC module's lookup maps (by ids).
			if (const FProperty* const Property = FindFProperty<FProperty>(URemoteControlPreset::StaticClass(), TEXT("PresetId")))
			{
				if (FGuid* TheGuid = Property->ContainerPtrToValuePtr<FGuid>(InRemoteControlPreset))
				{
					*TheGuid = FGuid::NewGuid();
				}
			}
		}
	}

	UE_LOGF(LogAvaRemoteControlUtils, Verbose, "Registering RC Preset: \"%ls\" (id:%ls).",
		*InRemoteControlPreset->GetPresetName().ToString(), *InRemoteControlPreset->GetPresetId().ToString());

	if (URemoteControlComponentsSubsystem* RemoteControlComponentsSubsystem = URemoteControlComponentsSubsystem::Get())
	{
		// Let the Remote Control Component Subsystem know that it should now handle the current Ava Preset
		RemoteControlComponentsSubsystem->RegisterPreset(InRemoteControlPreset);
	}

	// Motion Design's RCPs, from an ava asset (e.g. level) are considered "embedded" because
	// they are not an asset.
	constexpr bool bReplaceExisting = true;
	return RemoteControlModule.RegisterEmbeddedPreset(InRemoteControlPreset, bReplaceExisting);
}

void FAvaRemoteControlUtils::UnregisterRemoteControlPreset(URemoteControlPreset* InRemoteControlPreset)
{
	if (!InRemoteControlPreset)
	{
		return;
	}
	
	// Should we check if the module is still loaded?
	IRemoteControlModule& RemoteControlModule = IRemoteControlModule::Get();

	// Remark: if unregistering from within a BeginDestroy (RCP is marked as garbage),
	// ResolvePreset will return null because embedded RCPs are kept in a map with weak ptr,
	// and will be seen as invalid. But we still need to unregister otherwise it leaves a stale RCP.
	const URemoteControlPreset* ResolvedPreset = RemoteControlModule.ResolvePreset(InRemoteControlPreset->GetPresetName());

	// Don't unregister if the preset has already been replaced.
	if (ResolvedPreset && ResolvedPreset != InRemoteControlPreset)
	{
		UE_LOGF(LogAvaRemoteControlUtils, Verbose, "Didn't unregister RC Preset: \"%ls\" (id:%ls), was replaced by Preset with id:%ls."
			, *InRemoteControlPreset->GetPresetName().ToString()
			, *InRemoteControlPreset->GetPresetId().ToString()
			, ResolvedPreset ? *ResolvedPreset->GetPresetId().ToString() : TEXT("None"));
		return;
	}

	UE_LOGF(LogAvaRemoteControlUtils, Verbose, "Unregistering RC Preset: \"%ls\" (id:%ls)."
		, *InRemoteControlPreset->GetPresetName().ToString()
		, *InRemoteControlPreset->GetPresetId().ToString());

	if (URemoteControlComponentsSubsystem* RemoteControlComponentsSubsystem = URemoteControlComponentsSubsystem::Get())
	{
		RemoteControlComponentsSubsystem->UnregisterPreset(InRemoteControlPreset);
	}
	
	RemoteControlModule.UnregisterEmbeddedPreset(InRemoteControlPreset);
}

URemoteControlPreset* FAvaRemoteControlUtils::FindEmbeddedPresetInLevel(ULevel* InLevel)
{
	if (!InLevel)
	{
		return nullptr;
	}

	TArray<TWeakObjectPtr<URemoteControlPreset>> EmbeddedPresets;
	IRemoteControlModule::Get().GetEmbeddedPresets(EmbeddedPresets);

	for (const TWeakObjectPtr<URemoteControlPreset>& EmbeddedPresetWeak : EmbeddedPresets)
	{
		URemoteControlPreset* EmbeddedPreset = EmbeddedPresetWeak.Get();
		if (EmbeddedPreset && EmbeddedPreset->GetTypedOuter<ULevel>() == InLevel)
		{
			return EmbeddedPreset;
		}
	}

	return nullptr;
}
