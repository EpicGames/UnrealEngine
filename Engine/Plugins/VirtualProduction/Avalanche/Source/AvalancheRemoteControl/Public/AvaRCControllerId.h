// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "AvaRCControllerId.generated.h"

class URCVirtualPropertyBase;
class URemoteControlPreset;

/** Struct describing data to identify/find a Controller in a given preset */
USTRUCT(BlueprintType, DisplayName="Motion Design RC Controller Id")
struct FAvaRCControllerId
{
	GENERATED_BODY()

	FAvaRCControllerId() = default;

	AVALANCHEREMOTECONTROL_API explicit FAvaRCControllerId(URCVirtualPropertyBase* InController); 

	AVALANCHEREMOTECONTROL_API URCVirtualPropertyBase* FindController(URemoteControlPreset* InPreset) const;

	AVALANCHEREMOTECONTROL_API FText ToText() const;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Remote Control")
	FName Name;
};
