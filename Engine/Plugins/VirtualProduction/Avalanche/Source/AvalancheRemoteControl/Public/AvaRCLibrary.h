// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "AvaRCLibrary.generated.h"

class URCVirtualPropertyBase;

UCLASS()
class UAvaRCLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category="Remote Control", meta=(WorldContext="InWorldContextObject"))
	static AVALANCHEREMOTECONTROL_API TArray<AActor*> GetControlledActors(UObject* InWorldContextObject, URCVirtualPropertyBase* InController);
};
