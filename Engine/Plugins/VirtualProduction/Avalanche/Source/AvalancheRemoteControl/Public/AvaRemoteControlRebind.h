// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class ULevel;
class URemoteControlPreset;

struct FAvaRemoteControlRebind
{
	/**
	 * @brief Resolved the bindings and exposed entities, transient objects are allowed. 
	 * @param InPreset Preset to rebind. 
	 * @param InLevel Specify the level to constrain the binding resolution to. Will use the parent world if not specified. 
	 */
	AVALANCHEREMOTECONTROL_API static void RebindUnboundEntities(URemoteControlPreset* InPreset, const ULevel* InLevel = nullptr);

	/**
	 * Ensure all the Remote Control Fields have a properly resolved FieldPathInfo.
	 * This must be called after RebindUnboundEntities.
	 */
	AVALANCHEREMOTECONTROL_API static void ResolveAllFieldPathInfos(URemoteControlPreset* InPreset);
};
