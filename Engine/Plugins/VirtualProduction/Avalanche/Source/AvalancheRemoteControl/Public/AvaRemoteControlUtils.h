// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class ULevel;
class URemoteControlPreset;

struct FAvaRemoteControlUtils
{
	/**
	 *	Register the given RemoteControlPreset in the RemoteControl module.	
	 *	Registered RemoteControlPreset are made accessible, by name, to the other systems, like the Web interface.
	 *	
	 *	@param InRemoteControlPreset Preset to register
	 *	@param bInEnsureUniqueId The registered preset will be given a unique id if another preset instance has already been registered.
	 *	@return whether registration was successful.
	 */
	AVALANCHEREMOTECONTROL_API static bool RegisterRemoteControlPreset(URemoteControlPreset* InRemoteControlPreset, bool bInEnsureUniqueId);

	/**
	 *	Unregister the given RemoteControlPreset from the RemoteControl module.
	 */
	AVALANCHEREMOTECONTROL_API static void UnregisterRemoteControlPreset(URemoteControlPreset* InRemoteControlPreset);

	/**
	 * Iterates the active Embedded Presets and finds the one that is within the given Level
	 * Note: this will not find presets that have not been registered, even if outered to the given Level.
	 * @param InLevel the level containing the preset
	 * @return the registered preset if found, null otherwise
	 */
	AVALANCHEREMOTECONTROL_API static URemoteControlPreset* FindEmbeddedPresetInLevel(ULevel* InLevel);
};
