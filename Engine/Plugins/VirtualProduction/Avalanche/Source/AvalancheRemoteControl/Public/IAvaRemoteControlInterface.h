// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IAvaRemoteControlInterface.generated.h"

/**
 * Interface for runtime remote control events.
 */
UINTERFACE(MinimalAPI, Blueprintable, meta=(DisplayName="Motion Design Remote Control Interface"))
class UAvaRemoteControlInterface : public UInterface
{
	GENERATED_BODY()
};

class IAvaRemoteControlInterface
{
	GENERATED_BODY()

public:
	/**
	 * Called when the remote control values are applied.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "Remote Control")
	AVALANCHEREMOTECONTROL_API void OnValuesApplied();

	virtual void OnValuesApplied_Implementation() {}
};
