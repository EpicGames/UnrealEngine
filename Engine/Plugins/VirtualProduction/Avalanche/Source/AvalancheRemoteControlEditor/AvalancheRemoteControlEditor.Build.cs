// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheRemoteControlEditor : ModuleRules
{
    public AvalancheRemoteControlEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "AvalancheEditorCore",
                "AvalancheOutliner",
                "AvalancheRemoteControl",
                "Core",
                "CoreUObject",
                "Engine",
                "InputCore",
                "RemoteControl",
                "RemoteControlComponents",
                "RemoteControlComponentsEditor",
                "RemoteControlUI",
                "Slate",
                "SlateCore",
                "ToolMenus",
                "UnrealEd",
            }
        );

        ShortName = "AvRCEd";
    }
}