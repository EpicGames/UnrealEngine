// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvalancheRemoteControlEditorModule.h"
#include "AvaRCControllerId.h"
#include "Customizations/AvaRCControllerIdCustomization.h"
#include "IAvaOutliner.h"
#include "IAvaOutlinerModule.h"
#include "Item/AvaOutlinerActor.h"
#include "ItemProxies/AvaOutlinerItemProxyRegistry.h"
#include "Outliner/AvaOutlinerRCComponentsContextMenu.h"
#include "Outliner/AvaOutlinerRCTrackerComponentProxy.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "FAvalancheRemoteControlEditorModule"

void FAvalancheRemoteControlEditorModule::StartupModule()
{
	RegisterCustomizations();

	FAvaOutlinerItemProxyRegistry& ItemProxyRegistry = IAvaOutlinerModule::Get().GetItemProxyRegistry();

	ItemProxyRegistry.RegisterItemProxyWithDefaultFactory<FAvaOutlinerRCTrackerComponentProxy, 40>();
	
	OutlinerProxiesExtensionDelegateHandle = IAvaOutlinerModule::Get().GetOnExtendItemProxiesForItem().AddLambda(
		[this](IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InItem, TArray<TSharedPtr<FAvaOutlinerItemProxy>>& OutItemProxies)
		{
			if (InItem->IsA<FAvaOutlinerActor>())
			{
				if (const TSharedPtr<FAvaOutlinerItemProxy> RemoteControlTrackerProxy = InOutliner.GetOrCreateItemProxy<FAvaOutlinerRCTrackerComponentProxy>(InItem))
				{
					OutItemProxies.Add(RemoteControlTrackerProxy);
				}
			}
		});
	
	IAvaOutlinerModule::Get().GetOnExtendOutlinerItemContextMenu()
		.AddStatic(&FAvaOutlinerRCComponentsContextMenu::OnExtendOutlinerContextMenu);
}

void FAvalancheRemoteControlEditorModule::ShutdownModule()
{
	UnregisterCustomizations();

	if (IAvaOutlinerModule::IsLoaded())
	{
		IAvaOutlinerModule& OutlinerModule = IAvaOutlinerModule::Get();

		OutlinerModule.GetOnExtendItemProxiesForItem().Remove(OutlinerProxiesExtensionDelegateHandle);
		OutlinerProxiesExtensionDelegateHandle.Reset();
	}
}

void FAvalancheRemoteControlEditorModule::RegisterCustomizations()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomPropertyTypeLayout(CustomPropertyTypeLayouts.Add_GetRef(FAvaRCControllerId::StaticStruct()->GetFName())
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAvaRCControllerIdCustomization::MakeInstance));
}

void FAvalancheRemoteControlEditorModule::UnregisterCustomizations()
{
	if (FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		for (FName PropertyTypeLayout : CustomPropertyTypeLayouts)
		{
			PropertyEditorModule->UnregisterCustomPropertyTypeLayout(PropertyTypeLayout);
		}
		CustomPropertyTypeLayouts.Empty();
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FAvalancheRemoteControlEditorModule, AvalancheRemoteControlEditor)
