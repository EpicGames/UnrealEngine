// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaRCControllerIdCustomization.h"
#include "AvaRCControllerId.h"
#include "DetailWidgetRow.h"
#include "Widgets/SAvaRCControllerPicker.h"
#include "Widgets/SBoxPanel.h"

TSharedRef<IPropertyTypeCustomization> FAvaRCControllerIdCustomization::MakeInstance()
{
	return MakeShared<FAvaRCControllerIdCustomization>();
}

void FAvaRCControllerIdCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	TSharedPtr<IPropertyHandle> NamePropertyHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaRCControllerId, Name));
	check(NamePropertyHandle);

	InHeaderRow
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.FillWidth(1.f)
			[
				SNew(SAvaRCControllerPicker, InPropertyHandle)
			]
		];
}
