// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaRCControllerPicker.h"
#include "AvaRCControllerId.h"
#include "AvaRemoteControlUtils.h"
#include "DetailLayoutBuilder.h"
#include "Engine/Level.h"
#include "RCVirtualProperty.h"
#include "RemoteControlPreset.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaRCControllerPicker"

void SAvaRCControllerPicker::Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InControllerIdHandle)
{
	NameHandle = InControllerIdHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaRCControllerId, Name));

	// Figure out the Level to use
	{
		TArray<UObject*> OuterObjects;
		InControllerIdHandle->GetOuterObjects(OuterObjects);

		for (UObject* OuterObject : OuterObjects)
		{
			if (ULevel* Level = OuterObject->GetTypedOuter<ULevel>())
			{
				LevelWeak = Level;
				break;
			}
		}
	}

	// Editable Text box to deal with the Name Handle
	TSharedRef<SWidget> Widget = SNew(SEditableTextBox)
		.Text(this, &SAvaRCControllerPicker::GetControllerName)
		.OnTextCommitted(this, &SAvaRCControllerPicker::OnControllerNameCommitted)
		.Font(IDetailLayoutBuilder::GetDetailFont());

	// if the found level is valid, then use a combo box to display the options found for that level instead
	if (LevelWeak.IsValid())
	{
		RefreshOptions();

		Widget = SAssignNew(ComboBox, SComboBox<TSharedPtr<FAvaRCControllerPickerOption>>)
			.OptionsSource(&Options)
			.InitiallySelectedItem(SelectedOption)
			.OnGenerateWidget(this, &SAvaRCControllerPicker::GenerateOptionWidget)
			.OnComboBoxOpening(this, &SAvaRCControllerPicker::RefreshOptions)
			.OnSelectionChanged(this, &SAvaRCControllerPicker::OnOptionSelectionChanged)
			[
				Widget
			];
	}

	ChildSlot
	[
		Widget
	];
}

FText SAvaRCControllerPicker::GetControllerName() const
{
	FName CurrentControllerName;
	const FPropertyAccess::Result Result = NameHandle->GetValue(CurrentControllerName);

	if (Result == FPropertyAccess::Fail)
	{
		return FText::GetEmpty();
	}

	if (Result == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	return FText::FromName(CurrentControllerName);
}

void SAvaRCControllerPicker::SetControllerName(FName InControllerName)
{
	if (!NameHandle.IsValid())
	{
		return;
	}

	FName CurrentControllerName;
	bool bSucceeded = NameHandle->GetValue(CurrentControllerName) == FPropertyAccess::Success;

	// Only skip if successfully retrieved the current value and it matches the new value.
	// Else, attempt to set the new value in the handle
	if (bSucceeded && CurrentControllerName == InControllerName)
	{
		return;
	}

	NameHandle->SetValue(InControllerName, EPropertyValueSetFlags::DefaultFlags);
}

void SAvaRCControllerPicker::OnControllerNameCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	SetControllerName(*InText.ToString());
}

void SAvaRCControllerPicker::RefreshOptions()
{
	Options.Reset();
	SelectedOption.Reset();

	if (!NameHandle.IsValid())
	{
		return;	
	}

	ULevel* Level = LevelWeak.Get();
	if (!Level)
	{
		return;
	}

	URemoteControlPreset* Preset = FAvaRemoteControlUtils::FindEmbeddedPresetInLevel(Level);
	if (!Preset)
	{
		return;
	}

	FName CurrentControllerName;
	NameHandle->GetValue(CurrentControllerName);

	TArray<URCVirtualPropertyBase*> Controllers = Preset->GetControllers();
	Options.Reserve(Controllers.Num());

	TSet<FName> ProcessedNames;
	ProcessedNames.Reserve(Controllers.Num());

	Controllers.RemoveAll([](URCVirtualPropertyBase* InController)
		{
			return InController == nullptr;
		});

	Controllers.StableSort([](const URCVirtualPropertyBase& A, const URCVirtualPropertyBase& B)
		{
			return A.DisplayIndex < B.DisplayIndex;
		});

	for (URCVirtualPropertyBase* Controller : Controllers)
	{
		// Since FAvaRCControllerPickerOption only handles names directly, avoid adding the same name entry twice
		bool bAlreadyInSet;
		ProcessedNames.Add(Controller->DisplayName, &bAlreadyInSet);
		if (bAlreadyInSet)
		{
			continue;
		}

		TSharedRef<FAvaRCControllerPickerOption> Item = MakeShared<FAvaRCControllerPickerOption>();
		Item->ControllerName = Controller->DisplayName;

		// Set the first Item that matches the current Controller Name as the selected item
		if (!SelectedOption.IsValid() && Item->ControllerName == CurrentControllerName)
		{
			SelectedOption = Item;
		}

		Options.Emplace(MoveTemp(Item));
	}

	// Update the Selected Item to the selected option (can be null if there's no controller found matching the current name stored)
	if (ComboBox.IsValid())
	{
		ComboBox->RefreshOptions();
		ComboBox->SetSelectedItem(SelectedOption);
	}
}

void SAvaRCControllerPicker::OnOptionSelectionChanged(TSharedPtr<FAvaRCControllerPickerOption> InSelectedOption, ESelectInfo::Type InSelectInfo)
{
	if (InSelectedOption.IsValid())
	{
		SelectedOption = InSelectedOption;
		SetControllerName(SelectedOption->ControllerName);
	}
}

TSharedRef<SWidget> SAvaRCControllerPicker::GenerateOptionWidget(TSharedPtr<FAvaRCControllerPickerOption> InOption)
{
	return SNew(STextBlock)
		.Text(FText::FromName(InOption->ControllerName))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

#undef LOCTEXT_NAMESPACE
