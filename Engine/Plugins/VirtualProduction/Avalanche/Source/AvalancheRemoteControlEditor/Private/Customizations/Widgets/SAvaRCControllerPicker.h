// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Widgets/SCompoundWidget.h"

class IPropertyHandle;
class ULevel;
template<typename OptionType> class SComboBox;

struct FAvaRCControllerPickerOption
{
	FName ControllerName;
};

class SAvaRCControllerPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaRCControllerPicker) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InControllerIdHandle);

private:
	FText GetControllerName() const;

	void SetControllerName(FName InControllerName);

	void OnControllerNameCommitted(const FText& InText, ETextCommit::Type InCommitType);

	void RefreshOptions();

	void OnOptionSelectionChanged(TSharedPtr<FAvaRCControllerPickerOption> InSelectedOption, ESelectInfo::Type InSelectInfo);

	TSharedRef<SWidget> GenerateOptionWidget(TSharedPtr<FAvaRCControllerPickerOption> InOption);

	TSharedPtr<SComboBox<TSharedPtr<FAvaRCControllerPickerOption>>> ComboBox;

	TWeakObjectPtr<ULevel> LevelWeak;

	TSharedPtr<IPropertyHandle> NameHandle;

	TArray<TSharedPtr<FAvaRCControllerPickerOption>> Options;

	TSharedPtr<FAvaRCControllerPickerOption> SelectedOption;
};
