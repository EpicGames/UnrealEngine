// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerRCComponentsContextMenu.h"

#include "RemoteControlComponentsUtils.h"
#include "GameFramework/Actor.h"
#include "Item/AvaOutlinerActor.h"
#include "RemoteControlTrackerComponent.h"
#include "ToolMenuContext/AvaOutlinerItemsContext.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerRemoteControlComponentsContextMenu"

void FAvaOutlinerRCComponentsContextMenu::OnExtendOutlinerContextMenu(UToolMenu* InToolMenu)
{
	if (!InToolMenu)
	{
		return;
	}

	TSet<TWeakObjectPtr<AActor>> OutObjects;	
	GetActorItems(InToolMenu, OutObjects);

	if (!OutObjects.IsEmpty())
	{
		AddRemoteControlComponentsMenuEntries(InToolMenu, OutObjects);
	}
}

void FAvaOutlinerRCComponentsContextMenu::GetActorItems(const UToolMenu* InToolMenu, TSet<TWeakObjectPtr<AActor>>& OutActors)
{
	const UAvaOutlinerItemsContext* OutlinerItemsContext = InToolMenu->Context.FindContext<UAvaOutlinerItemsContext>();

	if (!IsValid(OutlinerItemsContext))
	{
		return;
	}

	for (FAvaOutlinerItemWeakPtr ItemWeak : OutlinerItemsContext->GetItems())
	{
		const TSharedPtr<IAvaOutlinerItem> Item = ItemWeak.Pin();

		// Is it an actor
		if (const FAvaOutlinerActor* ActorItem = Item->CastTo<FAvaOutlinerActor>())
		{
			if (AActor* Actor = ActorItem->GetActor())
			{
				OutActors.Add(Actor);
			}
		}		
	}
}

void FAvaOutlinerRCComponentsContextMenu::AddRemoteControlComponentsMenuEntries(UToolMenu* InToolMenu, const TSet<TWeakObjectPtr<AActor>>& OutObjects)
{
	FToolMenuSection& MenuSection = InToolMenu->AddSection(TEXT("RemoteControlComponentsActions"), LOCTEXT("RemoteControlComponentsActionsMenuHeading", "Remote Control Components"));

	MenuSection.AddMenuEntry(
		TEXT("RemoteControlComponents_UnexposeAllProperties"),
		LOCTEXT("UnexposeAllPropertiesMenuEntry", "Unexpose all properties"),
		LOCTEXT("UnexposeAllPropertiesMenuEntryTooltip", "Unexposes all properties of the selected Actor(s) from Remote Control"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"),
		FUIAction(
			FExecuteAction::CreateStatic(&FAvaOutlinerRCComponentsContextMenu::ExecuteUnexposeAllPropertiesAction, OutObjects),
			FCanExecuteAction::CreateStatic(&FAvaOutlinerRCComponentsContextMenu::CanExecuteUnexposeAllPropertiesAction, OutObjects),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateStatic(&FAvaOutlinerRCComponentsContextMenu::DoesSelectionHaveTrackerComponent, OutObjects))
	);
	
	MenuSection.AddMenuEntry(
		TEXT("RemoteControlComponents_AddTracker"),
		LOCTEXT("AddTrackerMenuEntry", "Start Remote Control Tracking"),
		LOCTEXT("AddTrackerMenuEntryTooltip", "Adds a Remote Control Tracker Component to the selected Actor(s)"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"),
		FUIAction(
			FExecuteAction::CreateStatic(&FAvaOutlinerRCComponentsContextMenu::ExecuteAddTrackerAction, OutObjects),
			FCanExecuteAction::CreateStatic(&FAvaOutlinerRCComponentsContextMenu::CanExecuteAddTrackerAction, OutObjects),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateStatic(&FAvaOutlinerRCComponentsContextMenu::CanExecuteAddTrackerAction, OutObjects))
	);

	MenuSection.AddMenuEntry(
		TEXT("RemoteControlComponents_RemoveTracker"),
		LOCTEXT("RemoveTrackerMenuEntry", "Stop Remote Control Tracking"),
		LOCTEXT("RemoveTrackerMenuEntryTooltip", "Removes Remote Control Tracker Component from the selected Actor(s)"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"),
		FUIAction(
			FExecuteAction::CreateStatic(&FAvaOutlinerRCComponentsContextMenu::ExecuteRemoveTrackerAction, OutObjects),
			FCanExecuteAction::CreateStatic(&FAvaOutlinerRCComponentsContextMenu::DoesSelectionHaveTrackerComponent, OutObjects),
			FIsActionChecked(),
			FIsActionButtonVisible::CreateStatic(&FAvaOutlinerRCComponentsContextMenu::DoesSelectionHaveTrackerComponent, OutObjects))
	);
}

void FAvaOutlinerRCComponentsContextMenu::ExecuteUnexposeAllPropertiesAction(TSet<TWeakObjectPtr<AActor>> InActors)
{
	for (const TWeakObjectPtr<AActor> Actor : InActors)
	{
		if (Actor.IsValid())
		{
			FRemoteControlComponentsUtils::UnexposeAllProperties(Actor.Get());			
		}
	}
}

bool FAvaOutlinerRCComponentsContextMenu::CanExecuteUnexposeAllPropertiesAction(TSet<TWeakObjectPtr<AActor>> InActors)
{
	for (const TWeakObjectPtr<AActor> Actor : InActors)
	{
		if (Actor.IsValid())
		{
			if (const URemoteControlTrackerComponent* TrackerComponent = Actor->FindComponentByClass<URemoteControlTrackerComponent>())
			{
				if (TrackerComponent->HasTrackedProperties())
				{
					return true;
				}
			}
		}
	}

	return false;
}

void FAvaOutlinerRCComponentsContextMenu::ExecuteAddTrackerAction(TSet<TWeakObjectPtr<AActor>> InActors)
{
	FRemoteControlComponentsUtils::AddTrackerComponent(InActors);	
}

bool FAvaOutlinerRCComponentsContextMenu::CanExecuteAddTrackerAction(TSet<TWeakObjectPtr<AActor>> InActors)
{
	const bool bCanExecute = !InActors.IsEmpty();
	
	for (const TWeakObjectPtr<AActor> Actor : InActors)
	{
		if (Actor.IsValid())
		{
			// Check for Tracker Component
			if (Actor->FindComponentByClass<URemoteControlTrackerComponent>())
			{
				return false;
			}
		}
	}

	return bCanExecute;
}

void FAvaOutlinerRCComponentsContextMenu::ExecuteRemoveTrackerAction(TSet<TWeakObjectPtr<AActor>> InActors)
{	
	FRemoteControlComponentsUtils::RemoveTrackerComponent(InActors);
}

bool FAvaOutlinerRCComponentsContextMenu::DoesSelectionHaveTrackerComponent(TSet<TWeakObjectPtr<AActor>> InActors)
{
	for (const TWeakObjectPtr<AActor> Actor : InActors)
	{
		if (Actor.IsValid())
		{
			// If there's at least one actor with a Tracker Component, let's return true.
			if (Actor->FindComponentByClass<URemoteControlTrackerComponent>())
			{
				return true;
			}
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
