// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "ToolMenus.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UAvaOutlinerItemsContext;

class FAvaOutlinerRCComponentsContextMenu
{
public:
	/** Extend outliner context menu */
	static void OnExtendOutlinerContextMenu(UToolMenu* InToolMenu);

private:
	static void GetActorItems(const UToolMenu* InToolMenu, TSet<TWeakObjectPtr<AActor>>& OutActors);
	static void AddRemoteControlComponentsMenuEntries(UToolMenu* InToolMenu, const TSet<TWeakObjectPtr<AActor>>& OutObjects);
	
	static void ExecuteUnexposeAllPropertiesAction(TSet<TWeakObjectPtr<AActor>> InActors);
	static bool CanExecuteUnexposeAllPropertiesAction(TSet<TWeakObjectPtr<AActor>> InActors);

	static void ExecuteAddTrackerAction(TSet<TWeakObjectPtr<AActor>> InActors);
	static bool CanExecuteAddTrackerAction(TSet<TWeakObjectPtr<AActor>> InActors);

	static void ExecuteRemoveTrackerAction(TSet<TWeakObjectPtr<AActor>> InActors);
	static bool DoesSelectionHaveTrackerComponent(TSet<TWeakObjectPtr<AActor>> InActors);
};
