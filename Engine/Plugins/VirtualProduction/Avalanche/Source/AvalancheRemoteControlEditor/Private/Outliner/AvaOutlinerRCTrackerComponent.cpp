// Copyright Epic Games, Inc. All Rights Reserved.

#include "Outliner/AvaOutlinerRCTrackerComponent.h"
#include "Editor.h"
#include "IAvaOutliner.h"
#include "RemoteControlPreset.h"
#include "RemoteControlTrackerComponent.h"
#include "Selection/AvaOutlinerScopedSelection.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerRemoteControlComponent"

namespace UE::AvaRCEditor::Private
{
	void OpenRCPreset(const URemoteControlTrackerComponent& InTrackerComponent)
	{
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		if (!AssetEditorSubsystem)
		{
			return;
		}

		if (URemoteControlPreset* RCPreset = InTrackerComponent.GetCurrentPreset())
		{
			AssetEditorSubsystem->OpenEditorForAsset(RCPreset, EToolkitMode::WorldCentric);
		}
	}
}

FAvaOutlinerRCTrackerComponent::FAvaOutlinerRCTrackerComponent(IAvaOutliner& InOutliner, URemoteControlTrackerComponent* InComponent)
	: FAvaOutlinerObject(InOutliner, InComponent)
	, TrackerComponentWeak(InComponent)
{
	TrackerIcon = FSlateIconFinder::FindIconForClass(URemoteControlTrackerComponent::StaticClass());
}

FSlateIcon FAvaOutlinerRCTrackerComponent::GetIcon() const
{
	return TrackerIcon;
}

bool FAvaOutlinerRCTrackerComponent::ShowVisibility(EAvaOutlinerVisibilityType InVisibilityType) const
{
	return true;
}

bool FAvaOutlinerRCTrackerComponent::GetVisibility(EAvaOutlinerVisibilityType InVisibilityType) const
{
	return TrackerComponentWeak.IsValid();
}

void FAvaOutlinerRCTrackerComponent::OnVisibilityChanged(EAvaOutlinerVisibilityType InVisibilityType, bool bInNewVisibility)
{
	FAvaOutlinerObject::OnVisibilityChanged(InVisibilityType, bInNewVisibility);
}

void FAvaOutlinerRCTrackerComponent::Select(FAvaOutlinerScopedSelection& InSelection) const
{
	FAvaOutlinerObject::Select(InSelection);

	if (const URemoteControlTrackerComponent* TrackerComponent = TrackerComponentWeak.Get())
	{
		if (InSelection.IsSelected(TrackerComponent))
		{
			UE::AvaRCEditor::Private::OpenRCPreset(*TrackerComponent);
		}
	}
}

void FAvaOutlinerRCTrackerComponent::SetObject_Impl(UObject* InObject)
{
	FAvaOutlinerObject::SetObject_Impl(InObject);
	TrackerComponentWeak = Cast<URemoteControlTrackerComponent>(InObject);
}

#undef LOCTEXT_NAMESPACE
