// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Item/AvaOutlinerObject.h"
#include "Textures/SlateIcon.h"

class URemoteControlTrackerComponent;

/**
 * Inheriting from FAvaOutlinerObject and not from FAvaOutlinerComponent, which only takes into account Scene Components.
 * This class needs instead to handle URemoteControlTrackerComponent, which is of type UActorComponent, and not Scene Component.
 */
class FAvaOutlinerRCTrackerComponent : public FAvaOutlinerObject
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerRCTrackerComponent, FAvaOutlinerObject);

	FAvaOutlinerRCTrackerComponent(IAvaOutliner& InOutliner, URemoteControlTrackerComponent* InComponent);

	//~ Begin IAvaOutlinerItem
	virtual FSlateIcon GetIcon() const override;
	virtual bool ShowVisibility(EAvaOutlinerVisibilityType InVisibilityType) const override;
	virtual bool GetVisibility(EAvaOutlinerVisibilityType InVisibilityType) const override;
	virtual void OnVisibilityChanged(EAvaOutlinerVisibilityType InVisibilityType, bool bInNewVisibility) override;
	virtual void Select(FAvaOutlinerScopedSelection& InSelection) const override;
	//~ End IAvaOutlinerItem

protected:
	//~ Begin FAvaOutlinerObjectItem
	virtual void SetObject_Impl(UObject* InObject) override;
	//~ End FAvaOutlinerObjectItem

	TWeakObjectPtr<URemoteControlTrackerComponent> TrackerComponentWeak;

	FSlateIcon TrackerIcon;
};
