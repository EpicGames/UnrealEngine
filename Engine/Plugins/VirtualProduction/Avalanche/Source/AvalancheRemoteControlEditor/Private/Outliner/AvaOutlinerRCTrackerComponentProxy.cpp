// Copyright Epic Games, Inc. All Rights Reserved.

#include "Outliner/AvaOutlinerRCTrackerComponentProxy.h"
#include "GameFramework/Actor.h"
#include "IAvaOutliner.h"
#include "Item/AvaOutlinerActor.h"
#include "Outliner/AvaOutlinerRCTrackerComponent.h"
#include "RemoteControlTrackerComponent.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/RemoteControlComponentsSubsystem.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerRemoteControlComponentProxy"

FAvaOutlinerRCTrackerComponentProxy::FAvaOutlinerRCTrackerComponentProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem)
	: Super(InOutliner, InParentItem)
{
	TrackerIcon = FSlateIconFinder::FindIconForClass(URemoteControlTrackerComponent::StaticClass());
}

URemoteControlTrackerComponent* FAvaOutlinerRCTrackerComponentProxy::GetTrackerComponent() const
{
	const FAvaOutlinerItemPtr Parent = GetParent();
	
	if (!Parent.IsValid())
	{
		return nullptr;
	}

	if (const FAvaOutlinerActor* const ActorItem = Parent->CastTo<FAvaOutlinerActor>())
	{
		return ActorItem->GetActor()->FindComponentByClass<URemoteControlTrackerComponent>();
	}

	return nullptr;
}

FText FAvaOutlinerRCTrackerComponentProxy::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Remote Control Trackers");
}

FSlateIcon FAvaOutlinerRCTrackerComponentProxy::GetIcon() const
{
	return TrackerIcon;
}

FText FAvaOutlinerRCTrackerComponentProxy::GetIconTooltipText() const
{
	return LOCTEXT("Tooltip", "Show all Remote Control Trackers");
}

void FAvaOutlinerRCTrackerComponentProxy::OnItemRegistered()
{
	FAvaOutlinerItemProxy::OnItemRegistered();
	BindDelegates();
}

void FAvaOutlinerRCTrackerComponentProxy::OnItemUnregistered()
{
	FAvaOutlinerItemProxy::OnItemUnregistered();
	UnbindDelegates();
}

void FAvaOutlinerRCTrackerComponentProxy::GetProxiedItems(const TSharedRef<IAvaOutlinerItem>& InParent,
	TArray<FAvaOutlinerItemPtr>& OutChildren, bool bInRecursive)
{
	if (URemoteControlTrackerComponent* const TrackerComponent = GetTrackerComponent())
	{
		const FAvaOutlinerItemPtr TrackerComponentItem = Outliner.FindOrAdd<FAvaOutlinerRCTrackerComponent>(TrackerComponent);
		TrackerComponentItem->SetParent(SharedThis(this));

		OutChildren.Add(TrackerComponentItem);

		if (bInRecursive)
		{
			TrackerComponentItem->FindChildren(OutChildren, bInRecursive);
		}		
	}
}

void FAvaOutlinerRCTrackerComponentProxy::UnbindDelegates()
{
	if (URemoteControlComponentsSubsystem* RemoteControlComponentsSubsystem = URemoteControlComponentsSubsystem::Get())
	{
		RemoteControlComponentsSubsystem->OnTrackedActorRegistered().RemoveAll(this);
		RemoteControlComponentsSubsystem->OnTrackedActorUnregistered().RemoveAll(this);
	}
}

void FAvaOutlinerRCTrackerComponentProxy::OnTrackedActorsChanged(AActor* InActor)
{
	Outliner.RequestRefresh();
}

void FAvaOutlinerRCTrackerComponentProxy::BindDelegates()
{
	UnbindDelegates();

	if (URemoteControlComponentsSubsystem* RemoteControlComponentsSubsystem = URemoteControlComponentsSubsystem::Get())
	{
		RemoteControlComponentsSubsystem->OnTrackedActorRegistered().AddSP(this, &FAvaOutlinerRCTrackerComponentProxy::OnTrackedActorsChanged);
		RemoteControlComponentsSubsystem->OnTrackedActorUnregistered().AddSP(this, &FAvaOutlinerRCTrackerComponentProxy::OnTrackedActorsChanged);
	}
}

#undef LOCTEXT_NAMESPACE
