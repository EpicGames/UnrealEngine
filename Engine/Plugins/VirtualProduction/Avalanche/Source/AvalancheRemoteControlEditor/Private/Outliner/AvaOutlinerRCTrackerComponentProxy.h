// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Item/AvaOutlinerItemProxy.h"
#include "Textures/SlateIcon.h"

class URemoteControlTrackerComponent;

class FAvaOutlinerRCTrackerComponentProxy : public FAvaOutlinerItemProxy
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerRCTrackerComponentProxy, FAvaOutlinerItemProxy);

	FAvaOutlinerRCTrackerComponentProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem);

	URemoteControlTrackerComponent* GetTrackerComponent() const;
	
	//~ Begin IAvaOutlinerItem
	virtual FSlateIcon GetIcon() const override;
	virtual FText GetDisplayName() const override;
	virtual FText GetIconTooltipText() const override;
	virtual void OnItemRegistered() override;
	virtual void OnItemUnregistered() override;
	//~ End IAvaOutlinerItem
	
	//~ Begin FAvaOutlinerItemProxy
	virtual void GetProxiedItems(const TSharedRef<IAvaOutlinerItem>& InParent, TArray<FAvaOutlinerItemPtr>& OutChildren, bool bInRecursive) override;
	//~ End FAvaOutlinerItemProxy

protected:
	void BindDelegates();
	void UnbindDelegates();

	void OnTrackedActorsChanged(AActor* InActor);
	
	FSlateIcon TrackerIcon;
};
