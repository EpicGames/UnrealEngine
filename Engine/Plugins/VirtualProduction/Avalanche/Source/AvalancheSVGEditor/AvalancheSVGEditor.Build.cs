// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheSVGEditor : ModuleRules
{
    public AvalancheSVGEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
	            "ActorModifierCore",
	            "AvalancheInteractiveTools",
	            "AvalancheModifiers",
                "Core",
	            "CoreUObject",
                "Engine",
                "InteractiveToolsFramework",
                "Slate",
                "SlateCore",
                "SVGImporter",
                "SVGImporterEditor",
                "ToolMenus",
            }
        );
    }
}
