// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSVGEditorModule.h"
#include "AvaInteractiveToolsDelegates.h"
#include "GameFramework/Actor.h"
#include "IAvalancheInteractiveToolsModule.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/AvaBevelModifier.h"
#include "Modifiers/AvaExtrudeModifier.h"
#include "ProceduralMeshes/SVGDynamicMeshComponent.h"
#include "SVGEngineSubsystem.h"
#include "SVGImporter.h"
#include "SVGShapesParentActor.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"

#define LOCTEXT_NAMESPACE "AvalancheSVGEditorModule"

void FAvaSVGEditorModule::StartupModule()
{
	USVGEngineSubsystem::OnSVGActorSplit().BindRaw(this, &FAvaSVGEditorModule::OnSVGActorSplit);
	USVGEngineSubsystem::OnSVGShapesUpdated().BindRaw(this, &FAvaSVGEditorModule::OnSVGShapesUpdated);
}

void FAvaSVGEditorModule::ShutdownModule()
{
	USVGEngineSubsystem::OnSVGActorSplit().Unbind();
	USVGEngineSubsystem::OnSVGShapesUpdated().Unbind();
}

void FAvaSVGEditorModule::OnSVGActorSplit(ASVGShapesParentActor* InSVGShapesParent)
{
	if (!InSVGShapesParent)
	{
		return;
	}

	const UActorModifierCoreSubsystem* ModifierCoreSubsystem = UActorModifierCoreSubsystem::Get();

	if (!ModifierCoreSubsystem)
	{
		return;
	}

	TMap<TObjectPtr<AActor>, TObjectPtr<USVGDynamicMeshComponent>> ShapesMap;
	InSVGShapesParent->GetShapes(ShapesMap);
	for (const TPair<TObjectPtr<AActor>, TObjectPtr<USVGDynamicMeshComponent>>& ShapePair : ShapesMap)
	{
		if (USVGDynamicMeshComponent* Shape = ShapePair.Value)
		{
			const float ExtrudeValue = Shape->GetExtrudeDepth();
			ESVGExtrudeType ExtrudeType = Shape->ExtrudeType;
			Shape->FlattenShape();

			if (AActor* ShapeActor = ShapePair.Key)
			{
				UActorModifierCoreStack* ModifierStack = ModifierCoreSubsystem->GetActorModifierStack(ShapeActor);
				if (!ModifierStack)
				{
					ModifierStack = ModifierCoreSubsystem->AddActorModifierStack(ShapeActor);
				}

				FActorModifierCoreStackInsertOp ExtrudeModifierInsertOp;
				ExtrudeModifierInsertOp.NewModifierName = ModifierCoreSubsystem->GetRegisteredModifierName(UAvaExtrudeModifier::StaticClass());

				if (!ModifierCoreSubsystem->GetAllowedModifiers(ShapeActor).Contains(ExtrudeModifierInsertOp.NewModifierName))
				{
					continue;
				}

				UActorModifierCoreBase* Modifier = ModifierCoreSubsystem->InsertModifier(ModifierStack, ExtrudeModifierInsertOp);

				if (UAvaExtrudeModifier* ExtrudeModifier = Cast<UAvaExtrudeModifier>(Modifier))
				{
					EAvaExtrudeMode ExtrudeMode = EAvaExtrudeMode::Symmetrical;

					switch (ExtrudeType)
					{
						case ESVGExtrudeType::FrontFaceOnly:
							ExtrudeMode = EAvaExtrudeMode::Front;
							break;

						case ESVGExtrudeType::None:
						case ESVGExtrudeType::FrontBackMirror:
							ExtrudeMode = EAvaExtrudeMode::Symmetrical;
							break;

						default: ;
					}

					ExtrudeModifier->SetDepth(ExtrudeValue);
					ExtrudeModifier->SetExtrudeMode(ExtrudeMode);

					if (Shape->GetShapeType() == TEXT("Stroke"))
					{
						ExtrudeModifier->SetCloseBack(false);
					}
				}

				if (Shape->Bevel > 0.0f)
				{
					FActorModifierCoreStackInsertOp BevelModifierInsertOp;
					BevelModifierInsertOp.NewModifierName = ModifierCoreSubsystem->GetRegisteredModifierName(UAvaBevelModifier::StaticClass());

					if (!ModifierCoreSubsystem->GetAllowedModifiers(ShapeActor).Contains(BevelModifierInsertOp.NewModifierName))
					{
						continue;
					}

					Modifier = ModifierCoreSubsystem->InsertModifier(ModifierStack, BevelModifierInsertOp);

					if (UAvaBevelModifier* BevelModifier = Cast<UAvaBevelModifier>(Modifier))
					{
						BevelModifier->SetInset(Shape->Bevel);
					}
				}
			}
		}
	}
}

void FAvaSVGEditorModule::OnSVGShapesUpdated(AActor* InActor) const
{
	const UActorModifierCoreSubsystem* ModifierCoreSubsystem = UActorModifierCoreSubsystem::Get();

	if (!ModifierCoreSubsystem)
	{
		return;
	}

	UActorModifierCoreStack* ModifierStack = ModifierCoreSubsystem->GetActorModifierStack(InActor);

	if (!ModifierStack)
	{
		return;
	}

 	ModifierStack->MarkModifierDirty();
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FAvaSVGEditorModule, AvalancheSVGEditor)
