// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Modules/ModuleManager.h"

class AActor;
class ASVGShapesParentActor;
class FBindingContext;
class IAvalancheInteractiveToolsModule;
class UEdMode;
struct FAvaInteractiveToolsToolParameters;

class FAvaSVGEditorModule : public IModuleInterface
{
public:
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;

private:
	void OnSVGActorSplit(ASVGShapesParentActor* InSVGShapesParent);
	void OnSVGShapesUpdated(AActor* InActor) const;
};
