// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tool/AvaSVGActorTool.h"
#include "AvaInteractiveToolsSettings.h"
#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "Factories/SVGActorFactory.h"
#include "SVGActor.h"
#include "SVGImporter.h"
#include "SVGImporterEditorCommands.h"

UAvaSVGActorTool::UAvaSVGActorTool()
{
	ActorClass = ASVGActor::StaticClass();
}

void UAvaSVGActorTool::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	// Load svg module
	FSVGImporterModule::Get();

	FAvaInteractiveToolsToolParameters ToolParameters = CreateDefaultToolParameters();
	ToolParameters.UICommand = FSVGImporterEditorCommands::GetExternal().SpawnSVGActor;
	ToolParameters.Priority = 6000;
	ToolParameters.ActorFactory.Set<TObjectPtr<UActorFactory>>(CreateActorFactory<USVGActorFactory>());

	InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryNameActor, MoveTemp(ToolParameters));
}
