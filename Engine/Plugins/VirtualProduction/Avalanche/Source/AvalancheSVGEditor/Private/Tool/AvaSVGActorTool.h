// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaInteractiveToolsActorPointToolBase.h"
#include "AvaSVGActorTool.generated.h"

UCLASS()
class UAvaSVGActorTool : public UAvaInteractiveToolsActorPointToolBase
{
	GENERATED_BODY()

public:
	UAvaSVGActorTool();

	//~ Begin UAvaInteractiveToolsToolBase
	virtual void OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule) override;
	//~ End UAvaInteractiveToolsToolBase
};
