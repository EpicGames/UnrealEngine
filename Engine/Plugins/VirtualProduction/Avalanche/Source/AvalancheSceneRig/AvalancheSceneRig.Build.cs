// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheSceneRig : ModuleRules
{
	public AvalancheSceneRig(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
				"Engine"
			}
		);
	}
}
