// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneRigSubsystem.h"
#include "AvaSceneRigAssetTags.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Containers/Array.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "Engine/LevelStreamingDynamic.h"
#include "Engine/World.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Class.h"
#include "UObject/NoExportTypes.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPtr.h"

DEFINE_LOG_CATEGORY(AvaSceneRigSubsystemLog);

#define LOCTEXT_NAMESPACE "AvaSceneRigSubsystem"

TSet<TSubclassOf<AActor>> UAvaSceneRigSubsystem::SupportedActorClasses = {};

void UAvaSceneRigSubsystem::Initialize(FSubsystemCollectionBase& InOutCollection)
{
	Super::Initialize(InOutCollection);

#if WITH_EDITOR
	if (!HasAnyFlags(EObjectFlags::RF_ClassDefaultObject))
	{
		WorldTagGetterDelegate = UObject::FAssetRegistryTag::OnGetExtraObjectTagsWithContext.AddUObject(this, &UAvaSceneRigSubsystem::OnGetWorldTags);
	}
#endif
}

void UAvaSceneRigSubsystem::Deinitialize()
{
	Super::Deinitialize();

#if WITH_EDITOR
	UObject::FAssetRegistryTag::OnGetExtraObjectTagsWithContext.Remove(WorldTagGetterDelegate);
	WorldTagGetterDelegate.Reset();
#endif
}

#if WITH_EDITOR
void UAvaSceneRigSubsystem::OnGetWorldTags(FAssetRegistryTagsContext InContext) const
{
	// Outer of this subsystem should always be the Motion Design level the scene rig exists in
	const UWorld* const OuterWorld = GetTypedOuter<UWorld>();
	if (!IsValid(OuterWorld))
	{
		return;
	}

	// The context object should be the scene rig world asset
	const UWorld* const SceneRigWorld = Cast<UWorld>(InContext.GetObject());
	if (!IsValid(SceneRigWorld) || SceneRigWorld == OuterWorld)
	{
		return;
	}

	// NOTE: Are there other cases where a context world object has the motion design level as outer?
	// May need to add an additional check here in that case.

	using namespace UE::AvaSceneRig::AssetTags;
	InContext.AddTag(UObject::FAssetRegistryTag(SceneRig, Values::Enabled, UObject::FAssetRegistryTag::TT_Alphabetical));
}
#endif

UAvaSceneRigSubsystem* UAvaSceneRigSubsystem::ForWorld(const UWorld* const InWorld)
{
	if (IsValid(InWorld))
	{
		return InWorld->GetSubsystem<UAvaSceneRigSubsystem>();
	}
	return nullptr;
}

bool UAvaSceneRigSubsystem::ShouldCreateSubsystem(UObject* const InOuter) const
{
	if (!IsValid(InOuter))
	{
		return false;
	}

	const FAssetData AssetData(InOuter);

	// Only create scene rig subsystems for Motion Design scenes
	const FAssetDataTagMapSharedView::FFindTagResult SceneTagResult = AssetData.TagsAndValues.FindTag(TEXT("MotionDesignScene"));
	return SceneTagResult.IsSet() && SceneTagResult.GetValue().Equals(TEXT("Enabled"));
}

bool UAvaSceneRigSubsystem::IsSceneRigAssetData(const FAssetData& InAssetData)
{
	if (!InAssetData.IsValid())
	{
		return false;
	}

	using namespace UE::AvaSceneRig::AssetTags;
	
	FString TagValue;
	InAssetData.GetTagValue(SceneRig, TagValue);

	return TagValue.Equals(Values::Enabled);
}

bool UAvaSceneRigSubsystem::IsSceneRigAsset(UObject* const InObject)
{
	return IsSceneRigAssetData(FAssetData(InObject));
}

FString UAvaSceneRigSubsystem::GetSceneRigAssetSuffix()
{
	static const TCHAR* Suffix = TEXT("_SceneRig");
	return Suffix;
}

void UAvaSceneRigSubsystem::RegisterSupportedActorClasses(const TSet<TSubclassOf<AActor>>& InClasses)
{
	for (const TSubclassOf<AActor>& Class : InClasses)
	{
		SupportedActorClasses.Add(Class);
	}
}

void UAvaSceneRigSubsystem::UnregisterSupportedActorClasses(const TSet<TSubclassOf<AActor>>& InClasses)
{
	for (const TSubclassOf<AActor>& Class : InClasses)
	{
		const FSetElementId SetId = SupportedActorClasses.FindId(Class);
		if (SetId.IsValidId())
		{
			SupportedActorClasses.Remove(SetId);
		}
	}
}

const TSet<TSubclassOf<AActor>>& UAvaSceneRigSubsystem::GetSupportedActorClasses()
{
	return SupportedActorClasses;
}

bool UAvaSceneRigSubsystem::IsSupportedActorClass(TSubclassOf<AActor> InClass)
{
	// Iterate until the class is invalid (i.e. not an Actor subclass)
	while (*InClass)
	{
		if (SupportedActorClasses.Contains(InClass))
		{
			return true;
		}
		InClass = InClass->GetSuperClass();
	}
	return false;
}

bool UAvaSceneRigSubsystem::AreActorsSupported(const TArray<AActor*>& InActors)
{
	bool bAllItemClassesSupported = !InActors.IsEmpty();

	for (const AActor* const Actor : InActors)
	{
		if (!UAvaSceneRigSubsystem::IsSupportedActorClass(Actor->GetClass()))
		{
			bAllItemClassesSupported = false;
			break;
		}
	}

	return bAllItemClassesSupported;
}

ULevelStreaming* UAvaSceneRigSubsystem::SceneRigFromActor(AActor* const InActor)
{
	const UWorld* const World = InActor->GetLevel()->GetWorld();
	if (!IsValid(World))
	{
		return nullptr;
	}

	const UAvaSceneRigSubsystem* const SceneRigSubsystem = ForWorld(World);
	if (!IsValid(SceneRigSubsystem))
	{
		return nullptr;
	}

	return SceneRigSubsystem->FindFirstActiveSceneRig();
}

bool UAvaSceneRigSubsystem::AreAllActorsInLevel(ULevel* const InLevel, const TArray<AActor*>& InActors)
{
	if (!IsValid(InLevel))
	{
		return false;
	}

	for (const AActor* const Actor : InActors)
	{
		if (!InLevel->Actors.Contains(Actor))
		{
			return false;
		}
	}

	return true;
}

bool UAvaSceneRigSubsystem::AreSomeActorsInLevel(ULevel* const InLevel, const TArray<AActor*>& InActors)
{
	if (!IsValid(InLevel))
	{
		return false;
	}

	for (const AActor* const Actor : InActors)
	{
		if (InLevel->Actors.Contains(Actor))
		{
			return true;
		}
	}

	return false;
}

TArray<ULevelStreaming*> UAvaSceneRigSubsystem::FindAllSceneRigs() const
{
	TArray<ULevelStreaming*> OutStreamingLevels;

	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	for (ULevelStreaming* const LevelStreaming : GetWorldRef().GetStreamingLevels())
	{
		if (const TSoftObjectPtr<UWorld>& WorldAsset = LevelStreaming->GetWorldAsset())
		{
			if (IsSceneRigAsset(WorldAsset.Get()))
			{
				OutStreamingLevels.Add(LevelStreaming);
			}
		}
	}

	return OutStreamingLevels;
}

ULevelStreaming* UAvaSceneRigSubsystem::FindFirstActiveSceneRig() const
{
	const TArray<ULevelStreaming*> SceneRigs = FindAllSceneRigs();
	if (SceneRigs.IsEmpty())
	{
		return nullptr;
	}

	return SceneRigs[0];
}

UWorld* UAvaSceneRigSubsystem::FindFirstActiveSceneRigAsset() const
{
	ULevelStreaming* const ActiveSceneRig = UAvaSceneRigSubsystem::FindFirstActiveSceneRig();
	if (!IsValid(ActiveSceneRig))
	{
		return nullptr;
	}

	return ActiveSceneRig->GetWorldAsset().Get();
}

bool UAvaSceneRigSubsystem::IsActiveSceneRigActor(AActor* const InActor) const
{
	ULevelStreaming* const SceneRig = FindFirstActiveSceneRig();
	if (IsValid(SceneRig))
	{
		const UWorld* const WorldAsset = SceneRig->GetWorldAsset().Get();
		if (IsValid(WorldAsset) && IsValid(WorldAsset->PersistentLevel))
		{
			return WorldAsset->PersistentLevel->Actors.Contains(InActor);
		}
	}
	return false;
}

void UAvaSceneRigSubsystem::ForEachActiveSceneRigActor(TFunction<void(AActor* const InActor)> InFunction) const
{
	ULevelStreaming* const ActiveSceneRig = FindFirstActiveSceneRig();
	if (IsValid(ActiveSceneRig))
	{
		UWorld* const SceneRigAsset = ActiveSceneRig->GetWorldAsset().Get();
		if (IsValid(SceneRigAsset) && IsValid(SceneRigAsset->PersistentLevel))
		{
			for (AActor* const Actor : SceneRigAsset->PersistentLevel->Actors)
			{
				if (IsValid(Actor))
				{
					InFunction(Actor);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
