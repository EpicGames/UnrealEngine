// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::AvaSceneRig::AssetTags
{
	constexpr const TCHAR* SceneRig = TEXT("AvalancheSceneRig");

	namespace Values
	{
		constexpr const TCHAR* Enabled = TEXT("Enabled");
	}
}
