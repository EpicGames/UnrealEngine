// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Delegates/IDelegateInstance.h"
#include "Engine/AssetUserData.h"
#include "Subsystems/WorldSubsystem.h"
#include "Templates/Function.h"
#include "Templates/SubclassOf.h"
#include "AvaSceneRigSubsystem.generated.h"

class AActor;
class FAssetRegistryTagsContext;
class FName;
class UClass;
class ULevel;
class ULevelStreaming;
class UObject;
class UWorld;
struct FAssetData;

DECLARE_LOG_CATEGORY_EXTERN(AvaSceneRigSubsystemLog, Log, All);

UCLASS(MinimalAPI)
class UAvaSceneRigSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	/** Returns the Scene Rig Subsystem for a world. */
	AVALANCHESCENERIG_API static UAvaSceneRigSubsystem* ForWorld(const UWorld* const InWorld);

	/** Returns true if the asset data is for a Scene Rig level. */
	AVALANCHESCENERIG_API static bool IsSceneRigAssetData(const FAssetData& InAssetData);

	/** Returns true if the asset object is Scene Rig level. */
	AVALANCHESCENERIG_API static bool IsSceneRigAsset(UObject* const InObject);

	/** Returns the Scene Rig level suffix string. Recommended to append to level asset name to differentiate them from other levels. */
	AVALANCHESCENERIG_API static FString GetSceneRigAssetSuffix();

	/** Registers actor classes that are allowed to be added to a Scene Rig. */
	AVALANCHESCENERIG_API static void RegisterSupportedActorClasses(const TSet<TSubclassOf<AActor>>& InClasses);

	/** Unregisters actor classes that are allowed to be added to a Scene Rig. */
	AVALANCHESCENERIG_API static void UnregisterSupportedActorClasses(const TSet<TSubclassOf<AActor>>& InClasses);

	/** Returns the list of supported object classes that are allowed to be added to a scene rig level. */
	AVALANCHESCENERIG_API static const TSet<TSubclassOf<AActor>>& GetSupportedActorClasses();

	/** Returns true if the class is a supported Scene Rig class. */
	AVALANCHESCENERIG_API static bool IsSupportedActorClass(TSubclassOf<AActor> InClass);

	/** Returns true if the list of actors are supported Scene Rig classes. */
	AVALANCHESCENERIG_API static bool AreActorsSupported(const TArray<AActor*>& InActors);
	
	/** Returns the active Scene Rig that an actor belongs to. */
	AVALANCHESCENERIG_API static ULevelStreaming* SceneRigFromActor(AActor* const InActor);

	/** Returns true if all actors exist in the level. */
	AVALANCHESCENERIG_API static bool AreAllActorsInLevel(ULevel* const InLevel, const TArray<AActor*>& InActors);

	/** Returns true if at least one actor exists in the level. */
	AVALANCHESCENERIG_API static bool AreSomeActorsInLevel(ULevel* const InLevel, const TArray<AActor*>& InActors);

	/** Find all scene rig streaming levels in the current persistent level. */
	AVALANCHESCENERIG_API TArray<ULevelStreaming*> FindAllSceneRigs() const;

	/** Returns the first active Scene Rig object in the persistent level. NOTE: There should only be one Scene Rig in a world at any time. */
	AVALANCHESCENERIG_API ULevelStreaming* FindFirstActiveSceneRig() const;

	/** Returns all Scene Rig asset level paths that exist in the persistent level. NOTE: There should only be one Scene Rig in a world at any time. */
	AVALANCHESCENERIG_API UWorld* FindFirstActiveSceneRigAsset() const;

	/** Removes a list of actors from the active Scene Rig. */
	AVALANCHESCENERIG_API bool IsActiveSceneRigActor(AActor* const InActor) const;

	AVALANCHESCENERIG_API void ForEachActiveSceneRigActor(TFunction<void(AActor* const InActor)> InFunction) const;

	//~ Begin UObject
	virtual void Initialize(FSubsystemCollectionBase& InOutCollection) override;
	virtual void Deinitialize() override;
	//~ End UObject

private:
#if WITH_EDITOR
	void OnGetWorldTags(FAssetRegistryTagsContext InContext) const;
#endif

	//~ Begin WorldSubsystem
	virtual bool ShouldCreateSubsystem(UObject* const InOuter) const override;
	//~ End WorldSubsystem

#if WITH_EDITOR
	FDelegateHandle WorldTagGetterDelegate;
#endif

	static TSet<TSubclassOf<AActor>> SupportedActorClasses;
};
