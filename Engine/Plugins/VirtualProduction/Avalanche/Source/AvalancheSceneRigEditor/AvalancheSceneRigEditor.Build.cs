// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheSceneRigEditor : ModuleRules
{
	public AvalancheSceneRigEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
				"AssetTools",
				"Avalanche",
				"AvalancheOutliner",
				"AvalancheSceneRig",
				"Engine",
				"PropertyEditor",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"UnrealEd"
			}
		);
	}
}
