// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneRigEditorCommands.h"

#define LOCTEXT_NAMESPACE "AvaSceneRigEditorCommands"

const FAvaSceneRigEditorCommands& FAvaSceneRigEditorCommands::GetExternal()
{
	if (!IsRegistered())
	{
		Register();
	}

	return Get();
}

const FAvaSceneRigEditorCommands& FAvaSceneRigEditorCommands::GetInternal()
{
	return Get();
}

void FAvaSceneRigEditorCommands::RegisterCommands()
{
	UI_COMMAND(PromptToSaveSceneRigFromOutlinerItems
		, "Save as new Scene Rig..."
		, "Save the selected actors as a new scene rig level asset and sets it as the active scene rig."
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(AddOutlinerItemsToSceneRig
		, "Move to active Scene Rig"
		, "Move the selected actors to the active scene rig."
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(RemoveOutlinerItemsToSceneRig
		, "Remove from active Scene Rig"
		, "Remove the selected actors from the active scene rig."
		, EUserInterfaceActionType::Button
		, FInputChord());
}

#undef LOCTEXT_NAMESPACE
