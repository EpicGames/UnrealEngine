// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneRigEditorModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "AvaSceneRigAssetTags.h"
#include "AvaSceneRigEditorCommands.h"
#include "AvaSceneRigSubsystem.h"
#include "AvaSceneSettings.h"
#include "AvaSceneSubsystem.h"
#include "ContentBrowserModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "EditorDirectories.h"
#include "EditorLevelUtils.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "Engine/LevelStreamingDynamic.h"
#include "Engine/World.h"
#include "Factories/WorldFactory.h"
#include "FileHelpers.h"
#include "GameFramework/Actor.h"
#include "IAvaOutliner.h"
#include "IAvaOutlinerModule.h"
#include "IAvaSceneInterface.h"
#include "IAvaSceneRigEditorModule.h"
#include "IContentBrowserSingleton.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "Item/AvaOutlinerActor.h"
#include "Item/AvaOutlinerItemProxy.h"
#include "ItemProxies/AvaOutlinerItemProxyRegistry.h"
#include "LevelUtils.h"
#include "Misc/MessageDialog.h"
#include "Outliner/AvaOutlinerSceneRigProxy.h"
#include "Subsystems/EditorAssetSubsystem.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SSceneRigPicker.h"
#include "UObject/GCObjectScopeGuard.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPath.h"

DEFINE_LOG_CATEGORY(AvaSceneRigEditorLog);

#define LOCTEXT_NAMESPACE "AvaSceneRigEditorModule"

void FAvaSceneRigEditorModule::StartupModule()
{
	FAvaSceneRigEditorCommands::Register();

	if (IAvaOutlinerModule::IsLoaded())
	{
		FAvaOutlinerItemProxyRegistry& ItemProxyRegistry = IAvaOutlinerModule::Get().GetItemProxyRegistry();
		ItemProxyRegistry.RegisterItemProxyWithDefaultFactory<FAvaOutlinerSceneRigProxy, 0>();

		RegisterOutlinerItems();
	}
}

void FAvaSceneRigEditorModule::ShutdownModule()
{
	FAvaSceneRigEditorCommands::Unregister();
	
	if (IAvaOutlinerModule::IsLoaded())
	{
		FAvaOutlinerItemProxyRegistry& ItemProxyRegistry = IAvaOutlinerModule::Get().GetItemProxyRegistry();
		ItemProxyRegistry.UnregisterItemProxyFactory<FAvaOutlinerSceneRigProxy>();

		UnregisterOutlinerItems();
	}
}

void FAvaSceneRigEditorModule::RegisterOutlinerItems()
{
	if (!IAvaOutlinerModule::IsLoaded())
	{
		return;
	}

	OutlinerProxiesExtensionDelegateHandle = IAvaOutlinerModule::Get().GetOnExtendItemProxiesForItem().AddLambda(
		[this](IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InItem, TArray<TSharedPtr<FAvaOutlinerItemProxy>>& OutItemProxies)
		{
			if (InItem->IsA<FAvaOutlinerActor>())
			{
				if (const TSharedPtr<FAvaOutlinerItemProxy> SceneRigProxy = InOutliner.GetOrCreateItemProxy<FAvaOutlinerSceneRigProxy>(InItem))
				{
					OutItemProxies.Add(SceneRigProxy);
				}
			}
		});
}

void FAvaSceneRigEditorModule::UnregisterOutlinerItems()
{
	if (!IAvaOutlinerModule::IsLoaded())
	{
		return;
	}

	IAvaOutlinerModule& OutlinerModule = IAvaOutlinerModule::Get();

	FAvaOutlinerItemProxyRegistry& ItemProxyRegistry = OutlinerModule.GetItemProxyRegistry();
	ItemProxyRegistry.UnregisterItemProxyFactory<FAvaOutlinerSceneRigProxy>();

	OutlinerModule.GetOnExtendItemProxiesForItem().Remove(OutlinerProxiesExtensionDelegateHandle);
	OutlinerProxiesExtensionDelegateHandle.Reset();
}

void FAvaSceneRigEditorModule::CustomizeSceneRig(const TSharedRef<IPropertyHandle>& InSceneRigHandle, IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	if (ObjectsBeingCustomized.IsEmpty() || !ObjectsBeingCustomized[0].IsValid())
	{
		return;
	}

	InSceneRigHandle->MarkHiddenByCustomization();

	IDetailCategoryBuilder& SceneRigCategory = DetailBuilder.EditCategory(TEXT("Scene Rig"));

	SceneRigCategory.AddCustomRow(LOCTEXT("SceneRig", "Scene Rig"))
		.WholeRowContent()
		.HAlign(HAlign_Fill)
		[
			SNew(SSceneRigPicker, ObjectsBeingCustomized[0])
		];
}

ULevelStreaming* FAvaSceneRigEditorModule::SetActiveSceneRig(UWorld* const InWorld, const FSoftObjectPath& InSceneRigAssetPath) const
{
	if (!IsValid(InWorld) || !InSceneRigAssetPath.IsValid())
	{
		return nullptr;
	}

	if (!UAvaSceneRigSubsystem::IsSceneRigAssetData(InSceneRigAssetPath.TryLoad()))
	{
		return nullptr;
	}

	const FString LevelPackageName = InSceneRigAssetPath.GetLongPackageName();

	ULevelStreaming* OutStreamingLevel = FLevelUtils::FindStreamingLevel(InWorld, *LevelPackageName);
	if (IsValid(OutStreamingLevel))
	{
		if (!IsValid(OutStreamingLevel->GetLoadedLevel()))
		{
			InWorld->LoadSecondaryLevels();
			ensure(OutStreamingLevel->GetLoadedLevel());
		}

		return OutStreamingLevel;
	}

	InWorld->Modify();

	RemoveAllSceneRigs(InWorld);

	OutStreamingLevel = EditorLevelUtils::AddLevelToWorld(InWorld, *LevelPackageName, ULevelStreamingDynamic::StaticClass());

	if (!IsValid(OutStreamingLevel))
	{
		return nullptr;
	}

	// Adding the level to the world sets it as the current. We don't want this.
	InWorld->SetCurrentLevel(InWorld->PersistentLevel);

	OutStreamingLevel->LevelColor = FLinearColor::Yellow;

	UAvaSceneSettings* const SceneSettings = GetSceneSettings(InWorld);
	if (IsValid(SceneSettings))
	{
		SceneSettings->SetSceneRig(InSceneRigAssetPath);
	}
	
	OnSceneRigChangedDelegate.Broadcast(InWorld, OutStreamingLevel);

	return OutStreamingLevel;
}

FSoftObjectPath FAvaSceneRigEditorModule::GetActiveSceneRig(UWorld* const InWorld) const
{
	const UAvaSceneSettings* const SceneSettings = GetSceneSettings(InWorld);
	if (!IsValid(SceneSettings))
	{
		return FSoftObjectPath();
	}

	return SceneSettings->GetSceneRig();
}

bool FAvaSceneRigEditorModule::IsActiveSceneRigActor(UWorld* const InWorld, AActor* const InActor) const
{
	const FSoftObjectPath SceneRigPath = GetActiveSceneRig(InWorld);

	UWorld* const SceneRigAsset = Cast<UWorld>(SceneRigPath.TryLoad());
	if (!IsValid(SceneRigAsset) || !IsValid(SceneRigAsset->PersistentLevel))
	{
		return false;
	}

	return SceneRigAsset->PersistentLevel->Actors.Contains(InActor);
}

bool FAvaSceneRigEditorModule::RemoveAllSceneRigs(UWorld* const InWorld) const
{
	UAvaSceneRigSubsystem* const SceneRigSubsystem = InWorld->GetSubsystem<UAvaSceneRigSubsystem>();
	if (!IsValid(SceneRigSubsystem))
	{
		return false;
	}

	for (const ULevelStreaming* const LevelStreaming : SceneRigSubsystem->FindAllSceneRigs())
	{
		const TSoftObjectPtr<UWorld>& WorldAsset = LevelStreaming->GetWorldAsset();
		if (!WorldAsset.IsValid())
		{
			continue;
		}

		if (UAvaSceneRigSubsystem::IsSceneRigAsset(WorldAsset.Get()))
		{
			WorldAsset->Modify();

			EditorLevelUtils::RemoveLevelFromWorld(LevelStreaming->GetLoadedLevel());
		}
	}

	UAvaSceneSettings* const SceneSettings = GetSceneSettings(InWorld);
	if (IsValid(SceneSettings))
	{
		SceneSettings->SetSceneRig(FSoftObjectPath());
	}

	OnSceneRigChangedDelegate.Broadcast(InWorld, nullptr);

	return true;
}

void FAvaSceneRigEditorModule::AddActiveSceneRigActors(UWorld* const InWorld, const TArray<AActor*>& InActors) const
{
	if (!IsValid(InWorld) || InActors.IsEmpty())
	{
		return;
	}

	if (!UAvaSceneRigSubsystem::AreActorsSupported(InActors))
	{
		return;
	}

	const UAvaSceneRigSubsystem* const SceneRigSubsystem = UAvaSceneRigSubsystem::ForWorld(InWorld);
	if (!IsValid(SceneRigSubsystem))
	{
		return;
	}

	ULevelStreaming* const SceneRig = SceneRigSubsystem->FindFirstActiveSceneRig();
	if (!IsValid(SceneRig))
	{
		return;
	}

	TArray<AActor*> MovedActors;
	const int32 NumActorsMoved = EditorLevelUtils::MoveActorsToLevel(InActors, SceneRig->GetLoadedLevel());

	if (NumActorsMoved > 0)
	{
		OnSceneRigActorsAddedDelegate.Broadcast(InWorld, MovedActors);
	}
}

void FAvaSceneRigEditorModule::RemoveActiveSceneRigActors(UWorld* const InWorld, const TArray<AActor*>& InActors) const
{
	if (!IsValid(InWorld) || !IsValid(InWorld->PersistentLevel) || InActors.IsEmpty())
	{
		return;
	}

	const UAvaSceneRigSubsystem* const SceneRigSubsystem = UAvaSceneRigSubsystem::ForWorld(InWorld);
	if (!IsValid(SceneRigSubsystem))
	{
		return;
	}

	ULevelStreaming* const SceneRig = SceneRigSubsystem->FindFirstActiveSceneRig();
	if (!IsValid(SceneRig))
	{
		return;
	}

	TArray<AActor*> MovedActors;
	const int32 NumActorsMoved = EditorLevelUtils::MoveActorsToLevel(InActors, InWorld->PersistentLevel, true, true, false, &MovedActors);

	if (NumActorsMoved > 0)
	{
		OnSceneRigActorsRemovedDelegate.Broadcast(InWorld, MovedActors);
	}
}

UAvaSceneSettings* FAvaSceneRigEditorModule::GetSceneSettings(UWorld* const InWorld) const
{
	if (!IsValid(InWorld))
	{
		return nullptr;
	}

	const UAvaSceneSubsystem* const SceneSubsystem = InWorld->GetSubsystem<UAvaSceneSubsystem>();
	if (!IsValid(SceneSubsystem))
	{
		return nullptr;
	}

	const IAvaSceneInterface* const AvaScene = SceneSubsystem->GetSceneInterface();
	if (!AvaScene)
	{
		return nullptr;
	}

	return AvaScene->GetSceneSettings();
}

FSoftObjectPath FAvaSceneRigEditorModule::CreateSceneRigAssetWithDialog() const
{
	if (!GEditor)
	{
		return nullptr;
	}

	UWorldFactory* const WorldFactory = NewObject<UWorldFactory>();
	if (!IsValid(WorldFactory))
	{
		return nullptr;
	}

	FGCObjectScopeGuard DontGCFactory(WorldFactory);

	FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
	IAssetTools& AssetTools = AssetToolsModule.Get();

	// Determine the starting path. Try to use the most recently used directory
	FString AssetPath;

	const FString DefaultFilesystemDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::NEW_ASSET);
	if (DefaultFilesystemDirectory.IsEmpty() || !FPackageName::TryConvertFilenameToLongPackageName(DefaultFilesystemDirectory, AssetPath))
	{
		// No saved path, just use the game content root
		AssetPath = TEXT("/Game");
	}

	const FString NewAssetPath = AssetPath / TEXT("New") + UAvaSceneRigSubsystem::GetSceneRigAssetSuffix();

	FString PackageName;
	FString AssetName;
	AssetTools.CreateUniqueAssetName(NewAssetPath, TEXT(""), PackageName, AssetName);

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	// Ask the user for the path to save to
	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Scene Rig Asset As");
	SaveAssetDialogConfig.DefaultAssetName = AssetName;
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;
	SaveAssetDialogConfig.AssetClassNames.Add(WorldFactory->GetSupportedClass()->GetClassPathName());

	const FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
	if (SaveObjectPath.IsEmpty())
	{
		return FSoftObjectPath();
	}

	// Add "_SceneRig" suffix if needed and create asset
	const FString SavePackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
	const FString SavePackagePath = FPaths::GetPath(SavePackageName);
	const FString Suffix = UAvaSceneRigSubsystem::GetSceneRigAssetSuffix();

	FString SaveAssetName = FPaths::GetBaseFilename(SavePackageName);
	if (!SaveAssetName.EndsWith(Suffix, ESearchCase::IgnoreCase))
	{
		SaveAssetName.Append(Suffix);
	}
	
	UWorld* const NewSceneRigAsset = StaticCast<UWorld*>(
		AssetTools.CreateAsset(SaveAssetName, SavePackagePath, UWorld::StaticClass(), WorldFactory));

	if (!IsValid(NewSceneRigAsset) || !IsValid(NewSceneRigAsset->PersistentLevel))
	{
		UE_LOGF(AvaSceneRigEditorLog, Warning, "Failed to create new Scene Rig asset!");
		return FSoftObjectPath();
	}

	// Save package
	TArray<UPackage*> PackagesToSave;
	PackagesToSave.Add(NewSceneRigAsset->GetPackage());
	UEditorLoadingAndSavingUtils::SavePackages(PackagesToSave, true);

	// Save last directory path for dialogs
	FEditorDirectories::Get().SetLastDirectory(ELastDirectory::NEW_ASSET, SavePackagePath);

	return NewSceneRigAsset->GetPathName();
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FAvaSceneRigEditorModule, AvalancheSceneRigEditor)
