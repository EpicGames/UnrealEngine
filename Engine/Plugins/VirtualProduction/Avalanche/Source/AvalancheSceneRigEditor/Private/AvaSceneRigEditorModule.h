// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/IDelegateInstance.h"
#include "IAvaSceneRigEditorModule.h"
#include "Templates/SharedPointer.h"
#include "UObject/NoExportTypes.h"

DECLARE_LOG_CATEGORY_EXTERN(AvaSceneRigEditorLog, Log, All);

class AActor;
class IDetailLayoutBuilder;
class IPropertyHandle;
class UAvaSceneSettings;
class ULevelStreaming;
class UWorld;
struct FSoftObjectPath;

class FAvaSceneRigEditorModule : public IAvaSceneRigEditorModule
{
public:
	//~ Begin IAvaSceneRigEditorModule
	virtual void CustomizeSceneRig(const TSharedRef<IPropertyHandle>& InSceneRigHandle, IDetailLayoutBuilder& DetailBuilder) override;
	virtual ULevelStreaming* SetActiveSceneRig(UWorld* const InWorld, const FSoftObjectPath& InSceneRigAssetPath) const override;
	virtual FSoftObjectPath GetActiveSceneRig(UWorld* const InWorld) const override;
	virtual bool IsActiveSceneRigActor(UWorld* const InWorld, AActor* const InActor) const override;
	virtual bool RemoveAllSceneRigs(UWorld* const InWorld) const override;
	virtual void AddActiveSceneRigActors(UWorld* const InWorld, const TArray<AActor*>& InActors) const override;
	virtual void RemoveActiveSceneRigActors(UWorld* const InWorld, const TArray<AActor*>& InActors) const override;
	virtual FSoftObjectPath CreateSceneRigAssetWithDialog() const override;
	virtual FOnSceneRigChanged& OnSceneRigChanged() override { return OnSceneRigChangedDelegate; }
	virtual FOnSceneRigActorsAdded& OnSceneRigActorsAdded() override { return OnSceneRigActorsAddedDelegate; }
	virtual FOnSceneRigActorsRemoved& OnSceneRigActorsRemoved() override { return OnSceneRigActorsRemovedDelegate; }
	//~ End IAvaSceneRigEditorModule

	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

private:
	void RegisterOutlinerItems();
	void UnregisterOutlinerItems();

	UAvaSceneSettings* GetSceneSettings(UWorld* const InWorld) const;

	FDelegateHandle OutlinerProxiesExtensionDelegateHandle;

	FOnSceneRigChanged OnSceneRigChangedDelegate;
	FOnSceneRigActorsAdded OnSceneRigActorsAddedDelegate;
	FOnSceneRigActorsRemoved OnSceneRigActorsRemovedDelegate;
};
