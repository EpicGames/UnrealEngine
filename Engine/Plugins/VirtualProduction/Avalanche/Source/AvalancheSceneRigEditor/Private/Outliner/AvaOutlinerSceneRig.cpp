// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerSceneRig.h"
#include "AvaSceneRigSubsystem.h"
#include "EditorModeManager.h"
#include "Engine/LevelStreaming.h"
#include "Engine/World.h"
#include "Framework/Docking/TabManager.h"
#include "Input/Reply.h"
#include "Selection/AvaOutlinerScopedSelection.h"
#include "Styling/AppStyle.h"
#include "Toolkits/IToolkitHost.h"
#include "UObject/NameTypes.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerSceneRig"

FAvaOutlinerSceneRig::FAvaOutlinerSceneRig(IAvaOutliner& InOutliner, ULevelStreaming* const InSceneRig, const FAvaOutlinerItemPtr& InReferencingItem)
	: Super(InOutliner, InSceneRig, InReferencingItem, TEXT("SceneRig"))
	, StreamingLevelWeak(InSceneRig)
{
	Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("LandscapeEditor.NoiseTool"));
}

FText FAvaOutlinerSceneRig::GetDisplayName() const
{
	if (StreamingLevelWeak.IsValid())
	{
		const UWorld* const WorldAsset = StreamingLevelWeak->GetWorldAsset().Get();
		if (IsValid(WorldAsset))
		{
			return FText::Format(LOCTEXT("Tooltip", "This actor is a member of the Scene Rig: {0}")
				, FText::FromString(WorldAsset->GetName()));
		}
	}
	return FText();
}

FSlateIcon FAvaOutlinerSceneRig::GetIcon() const
{
	return Icon;
}

void FAvaOutlinerSceneRig::Select(FAvaOutlinerScopedSelection& InSelection) const
{
	if (!StreamingLevelWeak.IsValid())
	{
		return;
	}

	const UWorld* const WorldAsset = StreamingLevelWeak->GetWorldAsset().Get();
	if (!IsValid(WorldAsset))
	{
		return;
	}

	if (ReferencingItemWeak.IsValid())
	{
		ReferencingItemWeak.Pin()->Select(InSelection);
	}

	if (const TSharedPtr<IToolkitHost> ToolkitHost = InSelection.GetEditorModeTools().GetToolkitHost())
	{
		if (const TSharedPtr<FTabManager> TabManager = ToolkitHost->GetTabManager())
		{
			static const FName TabName = TEXT("AvalancheSceneSettingsTabSpawner");
			TabManager->TryInvokeTab(TabName);
		}
	}
}

void FAvaOutlinerSceneRig::SetObject_Impl(UObject* InObject)
{
	FAvaOutlinerObject::SetObject_Impl(InObject);
	StreamingLevelWeak = Cast<ULevelStreaming>(InObject);
}

#undef LOCTEXT_NAMESPACE
