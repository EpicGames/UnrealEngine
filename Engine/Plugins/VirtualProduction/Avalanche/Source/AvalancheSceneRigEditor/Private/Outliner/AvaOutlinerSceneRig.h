// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Item/AvaOutlinerItem.h"
#include "Item/AvaOutlinerObjectReference.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FAvaOutlinerScopedSelection;
class FText;
class ULevelStreaming;
class UObject;

/**
 * Item in Outliner representing a Scene Rig object.
 * Inherits from FAvaOutlinerObjectReference as multiple objects can be in the same Scene Rig.
 */
class FAvaOutlinerSceneRig : public FAvaOutlinerObjectReference
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerSceneRig, FAvaOutlinerObjectReference);

	FAvaOutlinerSceneRig(IAvaOutliner& InOutliner, ULevelStreaming* const InSceneRig, const FAvaOutlinerItemPtr& InReferencingItem);

	//~ Begin IAvaOutlinerItem
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	virtual void Select(FAvaOutlinerScopedSelection& InSelection) const override;
	//~ End IAvaOutlinerItem
	
private:
	//~ Begin FAvaOutlinerObjectItem
	virtual void SetObject_Impl(UObject* InObject) override;
	//~ End FAvaOutlinerObjectItem

	FSlateIcon Icon;

	TWeakObjectPtr<ULevelStreaming> StreamingLevelWeak;
};
