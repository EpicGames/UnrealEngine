// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerSceneRigProxy.h"
#include "AvaOutlinerSceneRig.h"
#include "AvaSceneRigSubsystem.h"
#include "AvaTypeSharedPointer.h"
#include "Engine/Level.h"
#include "Engine/LevelStreaming.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "IAvaOutliner.h"
#include "IAvaSceneRigEditorModule.h"
#include "Item/AvaOutlinerActor.h"
#include "Item/AvaOutlinerItemParameters.h"
#include "Selection/AvaOutlinerScopedSelection.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerSceneRigProxy"

FAvaOutlinerSceneRigProxy::FAvaOutlinerSceneRigProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem)
	: Super(InOutliner, InParentItem)
{
	Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("LandscapeEditor.NoiseTool"));

	if (UAvaSceneRigSubsystem* const SceneRigSubsystem = UAvaSceneRigSubsystem::ForWorld(Outliner.GetWorld()))
	{
		StreamingLevelWeak = SceneRigSubsystem->FindFirstActiveSceneRig();
	}
}

FAvaOutlinerSceneRigProxy::~FAvaOutlinerSceneRigProxy()
{
	UnbindDelegates();
}

void FAvaOutlinerSceneRigProxy::OnItemRegistered()
{
	Super::OnItemRegistered();

	BindDelegates();
}

void FAvaOutlinerSceneRigProxy::OnItemUnregistered()
{
	Super::OnItemUnregistered();
	
	UnbindDelegates();
}

FText FAvaOutlinerSceneRigProxy::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Scene Rig");
}

FSlateIcon FAvaOutlinerSceneRigProxy::GetIcon() const
{
	return Icon;
}

FText FAvaOutlinerSceneRigProxy::GetIconTooltipText() const
{
	return FText();
}

void FAvaOutlinerSceneRigProxy::GetProxiedItems(const TSharedRef<IAvaOutlinerItem>& InParent, TArray<FAvaOutlinerItemPtr>& OutChildren, bool bInRecursive)
{
	if (const FAvaOutlinerActor* const ActorItem = InParent->CastTo<FAvaOutlinerActor>())
	{
		if (AActor* const Actor = ActorItem->GetActor())
		{
			if (ULevelStreaming* const ActiveSceneRig = StreamingLevelWeak.Get())
			{
				const UWorld* const World = ActiveSceneRig->GetWorldAsset().Get();
				if (World && World->PersistentLevel)
				{
					if (World->PersistentLevel->Actors.Contains(Actor))
					{
						FAvaOutlinerItemPtr SceneRigItem = Outliner.FindOrAdd<FAvaOutlinerSceneRig>(ActiveSceneRig, InParent);
						SceneRigItem->SetParent(SharedThis(this));

						OutChildren.Add(SceneRigItem);

						if (bInRecursive)
						{
							SceneRigItem->FindChildren(OutChildren, bInRecursive);
						}
					}
				}
			}
		}
	}
}

void FAvaOutlinerSceneRigProxy::BindDelegates()
{
	UnbindDelegates();

	IAvaSceneRigEditorModule& SceneRigEditorModule = IAvaSceneRigEditorModule::Get();

	OnSceneRigChangedHandle = SceneRigEditorModule.OnSceneRigChanged().AddSP(this, &FAvaOutlinerSceneRigProxy::OnSceneRigChanged);
	OnSceneRigActorAddedHandle = SceneRigEditorModule.OnSceneRigActorsAdded().AddSP(this, &FAvaOutlinerSceneRigProxy::OnSceneRigActorAdded);
	OnSceneRigActorRemovedHandle = SceneRigEditorModule.OnSceneRigActorsRemoved().AddSP(this, &FAvaOutlinerSceneRigProxy::OnSceneRigActorRemoved);
}

void FAvaOutlinerSceneRigProxy::UnbindDelegates()
{
	IAvaSceneRigEditorModule& SceneRigEditorModule = IAvaSceneRigEditorModule::Get();

	SceneRigEditorModule.OnSceneRigChanged().Remove(OnSceneRigChangedHandle);
	OnSceneRigChangedHandle.Reset();

	SceneRigEditorModule.OnSceneRigActorsAdded().Remove(OnSceneRigActorAddedHandle);
	OnSceneRigActorAddedHandle.Reset();

	SceneRigEditorModule.OnSceneRigActorsRemoved().Remove(OnSceneRigActorRemovedHandle);
	OnSceneRigActorRemovedHandle.Reset();
}

void FAvaOutlinerSceneRigProxy::OnSceneRigChanged(UWorld* const InWorld, ULevelStreaming* const InSceneRig)
{
	StreamingLevelWeak = InSceneRig;

	Outliner.RequestRefresh();
	RefreshChildren();
}

void FAvaOutlinerSceneRigProxy::OnSceneRigActorAdded(UWorld* const InWorld, const TArray<AActor*>& InActors)
{
	Outliner.RequestRefresh();
	RefreshChildren();
}

void FAvaOutlinerSceneRigProxy::OnSceneRigActorRemoved(UWorld* const InWorld, const TArray<AActor*>& InActors)
{
	Outliner.RequestRefresh();
	RefreshChildren();
}

ULevelStreaming* FAvaOutlinerSceneRigProxy::GetSceneRigAsset() const
{
	const FAvaOutlinerItemPtr Parent = GetParent();
	if (!Parent.IsValid())
	{
		return nullptr;
	}

	const FAvaOutlinerActor* const ActorItem = Parent->CastTo<FAvaOutlinerActor>();
	if (!ActorItem)
	{
		return nullptr;
	}

	UAvaSceneRigSubsystem* const SceneRigSubsystem = UAvaSceneRigSubsystem::ForWorld(Outliner.GetWorld());
	if (!SceneRigSubsystem)
	{
		return nullptr;
	}

	ULevelStreaming* const StreamingLevel = SceneRigSubsystem->SceneRigFromActor(ActorItem->GetActor());
	if (!StreamingLevel)
	{
		return nullptr;
	}

	return SceneRigSubsystem->FindFirstActiveSceneRig();
}

#undef LOCTEXT_NAMESPACE
