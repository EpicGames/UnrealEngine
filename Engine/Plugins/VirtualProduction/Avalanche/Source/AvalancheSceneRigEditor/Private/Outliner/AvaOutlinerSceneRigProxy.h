// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Item/AvaOutlinerItemProxy.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"

class UAvaSceneRigSubsystem;
class ULevelStreaming;

class FAvaOutlinerSceneRigProxy : public FAvaOutlinerItemProxy
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerSceneRigProxy, FAvaOutlinerItemProxy);

	FAvaOutlinerSceneRigProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem);

	virtual ~FAvaOutlinerSceneRigProxy() override;

	//~ Begin IAvaOutlinerItem
	virtual void OnItemRegistered() override;
	virtual void OnItemUnregistered() override;
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	virtual FText GetIconTooltipText() const override;
	//~ End IAvaOutlinerItem

	//~ Begin FAvaOutlinerItemProxy
	virtual void GetProxiedItems(const TSharedRef<IAvaOutlinerItem>& InParent, TArray<FAvaOutlinerItemPtr>& OutChildren, bool bInRecursive) override;
	//~ End FAvaOutlinerItemProxy

private:
	void BindDelegates();
	void UnbindDelegates();

	void OnSceneRigChanged(UWorld* const InWorld, ULevelStreaming* const InSceneRig);
	void OnSceneRigActorAdded(UWorld* const InWorld, const TArray<AActor*>& InActors);
	void OnSceneRigActorRemoved(UWorld* const InWorld, const TArray<AActor*>& InActors);

	ULevelStreaming* GetSceneRigAsset() const;

	FSlateIcon Icon;

	FDelegateHandle OnSceneRigChangedHandle;
	FDelegateHandle OnSceneRigActorAddedHandle;
	FDelegateHandle OnSceneRigActorRemovedHandle;

	TWeakObjectPtr<ULevelStreaming> StreamingLevelWeak;
};
