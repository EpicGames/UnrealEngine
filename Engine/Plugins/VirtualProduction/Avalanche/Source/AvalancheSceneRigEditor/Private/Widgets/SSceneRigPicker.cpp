// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSceneRigPicker.h"
#include "AssetRegistry/AssetData.h"
#include "AvaSceneRigSubsystem.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/World.h"
#include "IAvaSceneRigEditorModule.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "Misc/Attribute.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Styling/AppStyle.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UnrealType.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SSceneRigPicker"

void SSceneRigPicker::Construct(const FArguments& InArgs, const TWeakObjectPtr<UObject>& InObjectBeingCustomized)
{
	ObjectBeingCustomized = InObjectBeingCustomized;

	ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			[
				SNew(SObjectPropertyEntryBox)
				.AllowedClass(UWorld::StaticClass())
				.DisplayBrowse(true)
				.DisplayUseSelected(true)
				.DisplayCompactSize(false)
				.EnableContentPicker(true)
				.AllowClear(true)
				.AllowCreate(false)
				.DisplayThumbnail(true)
				.OnShouldFilterAsset(this, &SSceneRigPicker::ShouldFilterAsset)
				.ObjectPath(this, &SSceneRigPicker::GetObjectPath)
				.OnObjectChanged(this, &SSceneRigPicker::OnObjectChanged)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				PropertyCustomizationHelpers::MakeAddButton(FSimpleDelegate::CreateSP(this, &SSceneRigPicker::OnAddNewSceneRigClick)
					, LOCTEXT("AddSceneRigTooltip", "Create and add a new scene rig to the level.\n\n"
						"If a scene rig already exists in the level, it will be replaced."))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				PropertyCustomizationHelpers::MakeRemoveButton(FSimpleDelegate::CreateSP(this, &SSceneRigPicker::OnRemoveSceneRigClick)
					, LOCTEXT("RemoveSceneRigTooltip", "Remove scene rig from the level.")
					, TAttribute<bool>::CreateSP(this, &SSceneRigPicker::IsRemoveButtonEnabled))
			]
		];
}

bool SSceneRigPicker::ShouldFilterAsset(const FAssetData& InAssetData) const
{
	// Return false to show this asset
	return !UAvaSceneRigSubsystem::IsSceneRigAssetData(InAssetData);
}

FString SSceneRigPicker::GetObjectPath() const
{
	return IAvaSceneRigEditorModule::Get().GetActiveSceneRig(GetObjectWorld()).ToString();
}

void SSceneRigPicker::OnObjectChanged(const FAssetData& InAssetData) const
{
	if (InAssetData.IsValid())
	{
		IAvaSceneRigEditorModule::Get().SetActiveSceneRig(GetObjectWorld(), InAssetData.GetSoftObjectPath());
	}
	else
	{
		IAvaSceneRigEditorModule::Get().RemoveAllSceneRigs(GetObjectWorld());
	}
}

void SSceneRigPicker::OnAddNewSceneRigClick() const
{
	IAvaSceneRigEditorModule& SceneRigEditorModule = IAvaSceneRigEditorModule::Get();

	const FSoftObjectPath NewSceneRigPath = SceneRigEditorModule.CreateSceneRigAssetWithDialog();
	if (!NewSceneRigPath.IsValid())
	{
		return;
	}

	SceneRigEditorModule.SetActiveSceneRig(GetObjectWorld(), NewSceneRigPath);
}

void SSceneRigPicker::OnRemoveSceneRigClick() const
{
	IAvaSceneRigEditorModule::Get().RemoveAllSceneRigs(GetObjectWorld());
}

bool SSceneRigPicker::IsRemoveButtonEnabled() const
{
	return IAvaSceneRigEditorModule::Get().GetActiveSceneRig(GetObjectWorld()).IsAsset();
}

UWorld* SSceneRigPicker::GetObjectWorld() const
{
	if (!ObjectBeingCustomized.IsValid())
	{
		return nullptr;
	}

	return ObjectBeingCustomized->GetWorld();
}

#undef LOCTEXT_NAMESPACE
