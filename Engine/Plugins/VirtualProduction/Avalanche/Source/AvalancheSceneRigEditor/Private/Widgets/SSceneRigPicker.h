// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/SCompoundWidget.h"

class UObject;
class UWorld;
struct FAssetData;

class SSceneRigPicker : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SSceneRigPicker) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakObjectPtr<UObject>& InObjectBeingCustomized);

protected:
	bool ShouldFilterAsset(const FAssetData& InAssetData) const;
	FString GetObjectPath() const;
	void OnObjectChanged(const FAssetData& InAssetData) const;

	void OnAddNewSceneRigClick() const;
	void OnRemoveSceneRigClick() const;
	bool IsRemoveButtonEnabled() const;

	UWorld* GetObjectWorld() const;

	TWeakObjectPtr<UObject> ObjectBeingCustomized;
};
