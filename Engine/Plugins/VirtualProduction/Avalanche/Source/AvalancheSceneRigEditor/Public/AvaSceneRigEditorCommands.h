// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

class FAvaSceneRigEditorCommands : public TCommands<FAvaSceneRigEditorCommands>
{
public:
	AVALANCHESCENERIGEDITOR_API static const FAvaSceneRigEditorCommands& GetExternal();

	static const FAvaSceneRigEditorCommands& GetInternal();

	FAvaSceneRigEditorCommands()
		: TCommands<FAvaSceneRigEditorCommands>(TEXT("AvaSceneRigCommands")
		, NSLOCTEXT("AvaSceneRigCommands", "AvaSceneRigCommands", "Avalanche Scene Rig Commands")
		, NAME_None
		, FAppStyle::GetAppStyleSetName())
	{}

	//~ Begin TCommands
	virtual void RegisterCommands() override;	
	//~ End TCommands
	
	TSharedPtr<FUICommandInfo> PromptToSaveSceneRigFromOutlinerItems;

	TSharedPtr<FUICommandInfo> AddOutlinerItemsToSceneRig;
	TSharedPtr<FUICommandInfo> RemoveOutlinerItemsToSceneRig;

private:
	// Make this unavailable to the public
	using TCommands<FAvaSceneRigEditorCommands>::Get;
};
