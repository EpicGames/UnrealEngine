// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "HAL/Platform.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointerFwd.h"

class AActor;
class IDetailLayoutBuilder;
class IPropertyHandle;
class ULevel;
class ULevelStreaming;
class UToolMenu;
class UWorld;
struct FSoftObjectPath;

DECLARE_EVENT_TwoParams(IAvaSceneRigEditorModule, FOnSceneRigChanged, UWorld*, ULevelStreaming*);
DECLARE_EVENT_TwoParams(IAvaSceneRigEditorModule, FOnSceneRigActorsAdded, UWorld*, const TArray<AActor*>&);
DECLARE_EVENT_TwoParams(IAvaSceneRigEditorModule, FOnSceneRigActorsRemoved, UWorld*, const TArray<AActor*>&);

class IAvaSceneRigEditorModule : public IModuleInterface
{
	static constexpr const TCHAR* ModuleName = TEXT("AvalancheSceneRigEditor");

public:
	static bool IsLoaded()
	{
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	static IAvaSceneRigEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IAvaSceneRigEditorModule>(ModuleName);
	}

	virtual void CustomizeSceneRig(const TSharedRef<IPropertyHandle>& InSceneRigHandle, IDetailLayoutBuilder& DetailBuilder) = 0;

	/** Adds a Scene Rig streaming level to the world from a path to a Scene Rig level asset. */
	virtual ULevelStreaming* SetActiveSceneRig(UWorld* const InWorld, const FSoftObjectPath& InSceneRigAssetPath) const = 0;

	/** As opposed to UAvaSceneSubsystem::FindFirstActiveSceneRig, this returns the cached Scene State Scene Rig. */
	virtual FSoftObjectPath GetActiveSceneRig(UWorld* const InWorld) const = 0;

	/** Returns true if the specified actor is a member of the active Scene Rig. */
	virtual bool IsActiveSceneRigActor(UWorld* const InWorld, AActor* const InActor) const = 0;

	/** Removes all Scene Rig objects from the persistent level. */
	virtual bool RemoveAllSceneRigs(UWorld* const InWorld) const = 0;

	/** Adds a list of actors from another streaming level to the specified Scene Rig. */
	virtual void AddActiveSceneRigActors(UWorld* const InWorld, const TArray<AActor*>& InActors) const = 0;

	/** Removes a list of actors from the active Scene Rig. */
	virtual void RemoveActiveSceneRigActors(UWorld* const InWorld, const TArray<AActor*>& InActors) const = 0;

	/**
	 * Creates a new Scene Rig level asset. Asks the user for a location to save the new asset to.
	 * Optionally adds actors to the new Scene Rig if a world and actors are specified.
	 */
	virtual FSoftObjectPath CreateSceneRigAssetWithDialog() const = 0;

	virtual FOnSceneRigChanged& OnSceneRigChanged() = 0;

	virtual FOnSceneRigActorsAdded& OnSceneRigActorsAdded() = 0;

	virtual FOnSceneRigActorsRemoved& OnSceneRigActorsRemoved() = 0;
};
