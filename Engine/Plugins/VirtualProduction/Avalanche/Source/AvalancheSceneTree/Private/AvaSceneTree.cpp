// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneTree.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "Misc/EnumerateRange.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FDefaultModuleImpl, AvalancheSceneTree)

FAvaSceneTree::FAvaSceneTree()
{
	RootNode.OwningTree = this;
}

void FAvaSceneTree::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		UpdateTreeNodes();
	}
}

FAvaSceneTreeNode* FAvaSceneTree::FindTreeNode(const FAvaSceneItem& InItem)
{
	if (InItem.IsValid())
	{
		return ItemTreeMap.Find(InItem);
	}
	return nullptr;
}

const FAvaSceneTreeNode* FAvaSceneTree::FindObjectTreeNode(const UObject* InObject, const UObject* InOuter) const
{
	const FAvaSceneTreeNode* TreeNode = ObjectToNodeMap.Find({ InObject, InOuter });
	if (TreeNode)
	{
		// Fast path: object was already in the map, return the tree node only if it was valid.
		// If invalid, the object was already looked up and does not exist in the item tree map, so return null.
		return TreeNode->IsValid() ? TreeNode : nullptr;
	}

	// Slow path: item was never cached in the object node map.
	TreeNode = FindTreeNode(FAvaSceneItem(InObject, InOuter));
	ObjectToNodeMap.Emplace({ InObject, InOuter }, TreeNode ? *TreeNode : FAvaSceneTreeNode());
	return TreeNode;
}

const FAvaSceneTreeNode* FAvaSceneTree::FindTreeNode(const FAvaSceneItem& InItem) const
{
	if (InItem.IsValid())
	{
		return ItemTreeMap.Find(InItem);
	}
	return nullptr;
}

const FAvaSceneItem* FAvaSceneTree::GetItemAtIndex(int32 InIndex) const
{
	if (SceneItems.IsValidIndex(InIndex))
	{
		return &SceneItems[InIndex];
	}
	return nullptr;
}

FAvaSceneTreeNode& FAvaSceneTree::GetOrAddTreeNode(const FAvaSceneItem& InItem, const FAvaSceneItem& InParentItem)
{
	if (FAvaSceneTreeNode* const ExistingNode = FindTreeNode(InItem))
	{
		// @bug: this was unused, so commented out
		// ExistingNode->ChildrenIndices;
		return *ExistingNode;
	}

	// If Item Tree Map did not find the Item, Scene Items should not have it too
	checkSlow(!SceneItems.Contains(InItem));

	FAvaSceneTreeNode* ParentNode = FindTreeNode(InParentItem);
	if (!ParentNode)
	{
		ParentNode = &RootNode;
	}

	FAvaSceneTreeNode TreeNode;
	TreeNode.GlobalIndex = SceneItems.Add(InItem);
	TreeNode.LocalIndex  = ParentNode->ChildrenIndices.Add(TreeNode.GlobalIndex);
	TreeNode.ParentIndex = ParentNode->GlobalIndex;
	TreeNode.OwningTree  = this;
	bSorted = false;
	return ItemTreeMap.Add(InItem, MoveTemp(TreeNode));
}

const FAvaSceneTreeNode* FAvaSceneTree::FindLowestCommonAncestor(const TArray<const FAvaSceneTreeNode*>& InItems)
{
	TSet<const FAvaSceneTreeNode*> IntersectedAncestors;

	for (const FAvaSceneTreeNode* Item : InItems)
	{
		const FAvaSceneTreeNode* Parent = Item->GetParentTreeNode();
		TSet<const FAvaSceneTreeNode*> ItemAncestors;

		while (Parent)
		{
			ItemAncestors.Add(Parent);
			Parent = Parent->GetParentTreeNode();
		}

		if (IntersectedAncestors.Num() == 0)
		{
			IntersectedAncestors = ItemAncestors;
		}
		else
		{
			IntersectedAncestors = IntersectedAncestors.Intersect(ItemAncestors);

			if (IntersectedAncestors.Num() == 1)
			{
				break;
			}
		}
	}

	const FAvaSceneTreeNode* LowestCommonAncestor = nullptr;
	for (const FAvaSceneTreeNode* Item : IntersectedAncestors)
	{
		if (!LowestCommonAncestor || Item->CalculateHeight() > LowestCommonAncestor->CalculateHeight())
		{
			LowestCommonAncestor = Item;
		}
	}
	return LowestCommonAncestor;
}

bool FAvaSceneTree::CompareTreeItemOrder(const FAvaSceneTreeNode* InA, const FAvaSceneTreeNode* InB)
{
	if (!InA || !InB || !InA->OwningTree || InA->OwningTree != InB->OwningTree)
	{
		return false;
	}

	// Fast path: items are sorted by their global order in the tree, so their global index matches the item order. No need for extra computation
	if (InA->OwningTree->bSorted)
	{
		return InA->GlobalIndex < InB->GlobalIndex;
	}

	if (const FAvaSceneTreeNode* LowestCommonAncestor = FindLowestCommonAncestor({InA, InB}))
	{
		const TArray<const FAvaSceneTreeNode*> PathToA = LowestCommonAncestor->FindPath({InA});
		const TArray<const FAvaSceneTreeNode*> PathToB = LowestCommonAncestor->FindPath({InB});

		int32 Index = 0;

		int32 PathAIndex = -1;
		int32 PathBIndex = -1;

		while (PathAIndex == PathBIndex)
		{
			if (!PathToA.IsValidIndex(Index))
			{
				return true;
			}
			if (!PathToB.IsValidIndex(Index))
			{
				return false;
			}

			PathAIndex = PathToA[Index]->GetLocalIndex();
			PathBIndex = PathToB[Index]->GetLocalIndex();
			Index++;
		}
		return PathAIndex < PathBIndex;
	}
	return false;
}

void FAvaSceneTree::Reset()
{
	RootNode.Reset();
	ItemTreeMap.Reset();
	SceneItems.Reset();
	ObjectToNodeMap.Reset();
	bSorted = false;
}

void FAvaSceneTree::UpdateTreeNodes()
{
	for (TPair<FAvaSceneItem, FAvaSceneTreeNode>& Pair : ItemTreeMap)
	{
		FAvaSceneTreeNode& Node = Pair.Value;
		Node.OwningTree = this;
	}
}

TArray<AActor*> FAvaSceneTree::GetChildActors(const AActor* InParentActor) const
{
	TArray<AActor*> OutActors;

	// Todo: using Typed Outer here as right now the Nodes are saved with the Editor World path.
	// Ideally, these should be storing with the Levels Instead
	UWorld* const ActorWorld = InParentActor->GetTypedOuter<UWorld>();
	check(IsValid(ActorWorld));

	const FAvaSceneTreeNode* const ActorSceneTreeNode = FindObjectTreeNode(InParentActor, ActorWorld);
	if (ActorSceneTreeNode)
	{
		const TConstArrayView<int32> ChildIndices = ActorSceneTreeNode->GetChildrenIndices();

		OutActors.Reserve(ChildIndices.Num());

		for (int32 ChildIndex : ChildIndices)
		{
			const FAvaSceneItem* const ActorSceneItem = GetItemAtIndex(ChildIndex);
			if (ActorSceneItem)
			{
				if (AActor* const ChildActor = ActorSceneItem->Resolve<AActor>(ActorWorld))
				{
					OutActors.Add(ChildActor);
				}
			}
		}
	}

	return OutActors;
}

TArray<AActor*> FAvaSceneTree::GetRootActors(const ULevel* InLevel) const
{
	TArray<AActor*> OutActors;

	UWorld* Outer = InLevel->GetTypedOuter<UWorld>();
	check(IsValid(Outer));

	const TConstArrayView<int32> ChildIndices = RootNode.GetChildrenIndices();
	OutActors.Reserve(ChildIndices.Num());
	
	for (const int32 ChildIndex : ChildIndices)
	{
		const FAvaSceneItem* const ActorSceneItem = GetItemAtIndex(ChildIndex);
		
		if (ActorSceneItem)
		{
			AActor* const ChildActor = ActorSceneItem->Resolve<AActor>(Outer);
			OutActors.Add(ChildActor);
		}
	}
	
	return OutActors;
}

void FAvaSceneTree::SortItems()
{
	if (bSorted)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FAvaSceneTree::SortItems);

	// Edit a new copy of the scene items as the items will be index to the old structure. 
	TArray<FAvaSceneItem> NewSceneItems = SceneItems;
	NewSceneItems.Sort(
		[This=this](const FAvaSceneItem& A, const FAvaSceneItem& B)
		{
			const FAvaSceneTreeNode& NodeA = This->ItemTreeMap[A];
			const FAvaSceneTreeNode& NodeB = This->ItemTreeMap[B];
			return CompareTreeItemOrder(&NodeA, &NodeB);
		});

	auto FixIndex = [&NewSceneItems, This=this](int32& InOutIndex)
		{
			// index is pointing to root
			if (InOutIndex == INDEX_NONE)
			{
				return;
			}
			InOutIndex = NewSceneItems.Find(This->SceneItems[InOutIndex]);
		};

	// Fix up indices for a given tree node
	// No need to fix local index as that was the rule for sorting
	for (TPair<FAvaSceneItem, FAvaSceneTreeNode>& Pair : ItemTreeMap)
	{
		FAvaSceneTreeNode& Node = Pair.Value;
		FixIndex(Node.GlobalIndex);
		FixIndex(Node.ParentIndex);

		int32 LocalIndex = 0;
		for (int32& ChildIndex : Node.ChildrenIndices)
		{
			FixIndex(ChildIndex);
		}
	}

	SceneItems = MoveTemp(NewSceneItems);
	ObjectToNodeMap.Reset();
	bSorted = true;
}

void FAvaSceneTree::ResolveObjects(UObject* InOuter)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAvaSceneTree::ResolveObjects);

	TPair<FObjectKey, FObjectKey> ObjectPair;
	ObjectPair.Value = InOuter;

	ObjectToNodeMap.Empty(ItemTreeMap.Num());
	for (const TPair<FAvaSceneItem, FAvaSceneTreeNode>& Pair : ItemTreeMap)
	{
		if (UObject* Object = Pair.Key.Resolve(InOuter))
		{
			ObjectPair.Key = Object;
			ObjectToNodeMap.Add(ObjectPair, Pair.Value);
		}
	}
}
