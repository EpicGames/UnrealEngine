// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneTreeNode.h"
#include "AvaSceneTree.h"

const FAvaSceneTreeNode* FAvaSceneTreeNode::GetParentTreeNode() const
{
	if (!OwningTree || &OwningTree->GetRootNode() == this)
	{
		return nullptr;
	}

	const FAvaSceneTreeNode* FoundParentTreeNode = nullptr;

	if (const FAvaSceneItem* const ParentItem = OwningTree->GetItemAtIndex(ParentIndex))
	{
		FoundParentTreeNode = OwningTree->FindTreeNode(*ParentItem);
	}

	return FoundParentTreeNode ? FoundParentTreeNode : &OwningTree->GetRootNode();
}

int32 FAvaSceneTreeNode::CalculateHeight() const
{
	int32 Height = 0;

	const FAvaSceneTreeNode* ParentTreeNode = GetParentTreeNode();
	while (ParentTreeNode)
	{
		++Height;
		ParentTreeNode = ParentTreeNode->GetParentTreeNode();
	}

	return Height;
}

TArray<const FAvaSceneTreeNode*> FAvaSceneTreeNode::FindPath(const TArray<const FAvaSceneTreeNode*>& InItems) const
{
	TArray<const FAvaSceneTreeNode*> Path;
	for (const FAvaSceneTreeNode* Item : InItems)
	{
		Path.Reset();
		const FAvaSceneTreeNode* CurrentItem = Item;
		while (CurrentItem)
		{
			if (this == CurrentItem)
			{
				Algo::Reverse(Path);
				return Path;
			}
			Path.Add(CurrentItem);
			CurrentItem = CurrentItem->GetParentTreeNode();
		}
	}
	return TArray<const FAvaSceneTreeNode*>();
}

void FAvaSceneTreeNode::Reset()
{
	GlobalIndex = INDEX_NONE;
	LocalIndex = INDEX_NONE;
	ParentIndex = INDEX_NONE;
	ChildrenIndices.Reset();
}
