// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "UObject/Object.h"
#include "AvaSceneItem.generated.h"

UENUM()
enum class EAvaSceneItemIdType : uint8
{
	/** Id represents a UObject Path */
	ObjectPath,
	/** Id has no specifics on what it represents */
	Other,
};

USTRUCT()
struct FAvaSceneItem
{
	GENERATED_BODY()

	FAvaSceneItem() = default;

	FAvaSceneItem(const UObject* InObject, const UObject* InStopOuter)
	{
		check(InObject && (!InStopOuter || InObject->IsIn(InStopOuter)));
		Id     = InObject->GetPathName(InStopOuter);
		IdType = EAvaSceneItemIdType::ObjectPath;
	}

	FAvaSceneItem(const FString& InString)
	{
		Id     = InString;
		IdType = EAvaSceneItemIdType::Other;
	}

	/**
	 * Resolve an Object against an outer.
	 * NOTE: If resolving an Actor/Component or anything in a World, be sure to use the appropriate World passed in.
	 * GetWorld() is not the same as GetTypedOuter<UWorld>() for streamed actors
	 * E.g. the appropriate way to pass in the outer for a streamed level would be AcquaintedActor->GetTypedOuter<UWorld>()
	 */
	template<typename InObjectType = UObject, typename = typename TEnableIf<TIsDerivedFrom<InObjectType, UObject>::Value>::Type>
	InObjectType* Resolve(UObject* InOuter) const
	{
		// Don't resolve object if it isn't an Object Path
		if (IdType != EAvaSceneItemIdType::ObjectPath)
		{
			return nullptr;
		}
		return FindObject<InObjectType>(InOuter, *Id, EFindObjectFlags::None);
	}

	bool IsValid() const { return !Id.IsEmpty(); }

	friend uint32 GetTypeHash(const FAvaSceneItem& InItem)
	{
		return HashCombineFast(GetTypeHash(InItem.Id), GetTypeHash(InItem.IdType));
	}

	bool operator==(const FAvaSceneItem& InOther) const
	{
		return Id == InOther.Id && IdType == InOther.IdType;
	}

private:
	UPROPERTY()
	FString Id;

	UPROPERTY()
	EAvaSceneItemIdType IdType = EAvaSceneItemIdType::ObjectPath;
};
