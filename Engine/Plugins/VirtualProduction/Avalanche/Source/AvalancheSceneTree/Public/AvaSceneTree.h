// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSceneItem.h"
#include "AvaSceneTreeNode.h"
#include "Containers/Map.h"
#include "UObject/ObjectKey.h"
#include "AvaSceneTree.generated.h"

class AActor;
class ULevel;

USTRUCT()
struct AVALANCHESCENETREE_API FAvaSceneTree
{
	GENERATED_BODY()

	FAvaSceneTree();

	void PostSerialize(const FArchive& Ar);

	FAvaSceneTreeNode& GetRootNode()
	{
		return RootNode;
	}

	const FAvaSceneTreeNode& GetRootNode() const
	{
		return RootNode;
	}

	FAvaSceneTreeNode* FindTreeNode(const FAvaSceneItem& InItem);

	int32 GetSceneItemCount() const
	{
		return SceneItems.Num();
	}

	/**
	 * Finds tree node using the object map to avoid FAvaSceneItem allocation
	 * ResolveObjects() must've been called for the tree for the object map to be up-to-date.
	 */
	const FAvaSceneTreeNode* FindObjectTreeNode(const UObject* InObject, const UObject* InOuter) const;

	const FAvaSceneTreeNode* FindTreeNode(const FAvaSceneItem& InItem) const;

	const FAvaSceneItem* GetItemAtIndex(int32 InIndex) const;

	FAvaSceneTreeNode& GetOrAddTreeNode(const FAvaSceneItem& InItem, const FAvaSceneItem& InParentItem);

	static const FAvaSceneTreeNode* FindLowestCommonAncestor(const TArray<const FAvaSceneTreeNode*>& InItems);
	
	static bool CompareTreeItemOrder(const FAvaSceneTreeNode* InA, const FAvaSceneTreeNode* InB);

	void Reset();

	/** Returns all resolved child actors of a specified parent actor in the scene tree. */
	TArray<AActor*> GetChildActors(const AActor* InParentActor) const;

	/** Returns all root actors in this tree */
	TArray<AActor*> GetRootActors(const ULevel* InLevel) const;

	bool IsSorted() const
	{
		return bSorted;
	}

	/** Sorts items based on tree item ordering. No-op if the tree is already sorted. */
	void SortItems();

	/** Fills the object to node map for faster look up */
	void ResolveObjects(UObject* InOuter);

private:
	void UpdateTreeNodes();

	UPROPERTY()
	FAvaSceneTreeNode RootNode;

	UPROPERTY()
	TArray<FAvaSceneItem> SceneItems;

	UPROPERTY()
	TMap<FAvaSceneItem, FAvaSceneTreeNode> ItemTreeMap;

	/** (Object, Outer) pair to their tree node */
	mutable TMap<TPair<FObjectKey, FObjectKey>, FAvaSceneTreeNode> ObjectToNodeMap;

	/** Whether scene items have been sorted to their latest structure */
	UPROPERTY()
	bool bSorted = false;
};

template<>
struct TStructOpsTypeTraits<FAvaSceneTree> : TStructOpsTypeTraitsBase2<FAvaSceneTree>
{
	enum 
	{
		WithPostSerialize = true,
	};
};
