// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "AvaSceneTreeNode.generated.h"

struct FAvaSceneTree;

/** Struct containing hierarchy index information of the Tree Node */
USTRUCT()
struct AVALANCHESCENETREE_API FAvaSceneTreeNode
{
	GENERATED_BODY()

	friend FAvaSceneTree;

	bool IsValid() const
	{
		return OwningTree && LocalIndex != INDEX_NONE && GlobalIndex != INDEX_NONE;
	}

	int32 GetLocalIndex()  const { return LocalIndex; }

	int32 GetGlobalIndex() const { return GlobalIndex; }

	int32 GetParentIndex() const { return ParentIndex; }

	TConstArrayView<int32> GetChildrenIndices() const { return ChildrenIndices; }

	const FAvaSceneTreeNode* GetParentTreeNode() const;

	int32 CalculateHeight() const;

	TArray<const FAvaSceneTreeNode*> FindPath(const TArray<const FAvaSceneTreeNode*>& InItems) const;

	void Reset();

private:
	/** Index of this Tree Node relative to the Parent Node Children Items. Can be used as means of Ordering */
	UPROPERTY()
	int32 LocalIndex = INDEX_NONE;

	/** Index of this Tree Node in the Owning Tree Scene Item List */
	UPROPERTY()
	int32 GlobalIndex = INDEX_NONE;

	/** Absolute Index of the Parent Node in the Owning Tree Scene Item List. If INDEX_NONE, it means Parent is Root */
	UPROPERTY()
	int32 ParentIndex = INDEX_NONE;

	/** Absolute Indices of the Children in the Owning Tree Scene Item List */
	UPROPERTY()
	TArray<int32> ChildrenIndices;

	/** Pointer to the Tree that owns this Node */
	FAvaSceneTree* OwningTree = nullptr;
};
