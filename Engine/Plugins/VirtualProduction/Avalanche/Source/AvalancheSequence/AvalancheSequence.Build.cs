// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheSequence : ModuleRules
{
    public AvalancheSequence(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "AvalancheTag",
                "AvalancheTransition",
                "Core",
                "CoreUObject",
                "Engine",
                "LevelSequence",
                "MovieScene",
                "MovieSceneTracks",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "AvalancheCore",
                "StateTreeModule",
            });

        if (Target.Type == TargetRules.TargetType.Editor)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "BlueprintGraph",
                    "Sequencer",
                });
        }
    }
}