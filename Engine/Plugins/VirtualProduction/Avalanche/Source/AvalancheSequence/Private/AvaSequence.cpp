// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequence.h"
#include "AvaField.h"
#include "AvaSequenceController.h"
#include "AvaSequencePlayer.h"
#include "AvaSequenceVersion.h"
#include "AvaTag.h"
#include "Director/AvaSequenceDirector.h"
#include "Director/AvaSequenceDirectorBlueprint.h"
#include "EngineUtils.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Volume.h"
#include "GameFramework/WorldSettings.h"
#include "IAvaSequenceProvider.h"
#include "IMovieScenePlayer.h"
#include "MovieScene.h"
#include "Tracks/MovieScene3DAttachTrack.h"
#include "Tracks/MovieScene3DPathTrack.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Tracks/MovieSceneCVarTrack.h"
#include "Tracks/MovieSceneCameraCutTrack.h"
#include "Tracks/MovieSceneEventTrack.h"
#include "Tracks/MovieSceneFadeTrack.h"
#include "Tracks/MovieSceneMaterialParameterCollectionTrack.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "Tracks/MovieSceneSlomoTrack.h"
#include "UObject/CoreRedirects.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "BlueprintActionDatabase.h"
#include "Engine/Blueprint.h"
#endif

#define LOCTEXT_NAMESPACE "AvaSequence"

#if WITH_EDITOR
UAvaSequence::FOnSequenceEditUndoEvent UAvaSequence::OnSequenceEditUndoDelegate;
#endif

UAvaSequence::UAvaSequence(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
	, Label(TEXT("Sequence"))
	, PreviewMarkLabel(TEXT("Preview"))
{
	const FFrameRate TickRate    = FFrameRate(60000, 1);
	const FFrameRate DisplayRate = FFrameRate(60000, 1001);

	MovieScene = CreateDefaultSubobject<UMovieScene>("MovieScene");
	MovieScene->SetClockSource(EUpdateClockSource::Tick);
	MovieScene->SetEvaluationType(EMovieSceneEvaluationType::WithSubFrames);
	MovieScene->SetTickResolutionDirectly(TickRate);
	MovieScene->SetDisplayRate(DisplayRate);

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectTransacted.AddUObject(this, &UAvaSequence::OnObjectTransacted);
#endif
	FWorldDelegates::OnWorldCleanup.AddUObject(this, &UAvaSequence::OnWorldCleanup);
}

UAvaSequence::~UAvaSequence()
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);
#endif
	FWorldDelegates::OnWorldCleanup.RemoveAll(this);
}

IAvaSequenceProvider* UAvaSequence::GetSequenceProvider() const
{
	return Cast<IAvaSequenceProvider>(GetOuter());
}

UWorld* UAvaSequence::GetContextWorld() const
{
	if (IAvaSequenceProvider* const Provider = GetSequenceProvider())
	{
		return Provider->GetContextWorld();
	}
	return GetWorld();
}

FName UAvaSequence::GetLabel() const
{
	return Label;
}

void UAvaSequence::SetLabel(FName InLabel)
{
	Label = InLabel;
}

FAvaTagHandle UAvaSequence::GetSequenceTag() const
{
	return Tag;
}

void UAvaSequence::SetSequenceTag(const FAvaTagHandle& InSequenceTag)
{
	Tag = InSequenceTag;
}

double UAvaSequence::GetStartTime() const
{
	return MovieScene->GetPlaybackRange().GetLowerBoundValue() / MovieScene->GetTickResolution();
}

double UAvaSequence::GetEndTime() const
{
	return MovieScene->GetPlaybackRange().GetUpperBoundValue() / MovieScene->GetTickResolution();
}

void UAvaSequence::UpdateTreeNode()
{
	// Remove Invalid Sequences
	ChildAnimations.RemoveAll([](const TWeakObjectPtr<UAvaSequence>& InSequenceWeak)
		{
			return !InSequenceWeak.IsValid();
		});

	// Update Children
	for (const TWeakObjectPtr<UAvaSequence>& ChildSequence : ChildAnimations)
	{
		ChildSequence->SetParent(this);
		ChildSequence->UpdateTreeNode();
	}

	OnTreeNodeUpdated.Broadcast();
}

void UAvaSequence::OnSequenceRemoved()
{
	TArray<TWeakObjectPtr<UAvaSequence>> OldChildSequences;

	if (ParentAnimation.IsValid())
	{
		OldChildSequences = GetChildren();
	}

	RemoveAllChildren();

	if (ParentAnimation.IsValid())
	{
		for (TWeakObjectPtr<UAvaSequence> ChildSequence : OldChildSequences)
		{
			ParentAnimation->AddChild(ChildSequence.Get());
		}
	}

	MarkAsGarbage();
}

int32 UAvaSequence::AddChild(UAvaSequence* InChild)
{
	if (!InChild)
	{
		return false;
	}

#if WITH_EDITOR
	if (GUndo)
	{
		Modify();
	}
#endif

	if (InChild->ParentAnimation.IsValid())
	{
		InChild->ParentAnimation->RemoveChild(InChild);
	}

	InChild->SetParent(this);
	int32 OutIndex = ChildAnimations.AddUnique(InChild);

	if (IAvaSequenceProvider* SequenceProvider = GetSequenceProvider())
	{
		SequenceProvider->ScheduleRebuildSequenceTree();
	}

	return OutIndex;
}

bool UAvaSequence::RemoveChild(UAvaSequence* InChild)
{
#if WITH_EDITOR
	if (GUndo)
	{
		Modify();
	}
#endif

	const bool bRemovedSequence = ChildAnimations.Remove(InChild) > 0;

	if (InChild && bRemovedSequence)
	{
		InChild->SetParent(nullptr);
	}

	if (IAvaSequenceProvider* SequenceProvider = GetSequenceProvider())
	{
		SequenceProvider->ScheduleRebuildSequenceTree();
	}

	return bRemovedSequence;
}

void UAvaSequence::RemoveAllChildren()
{
#if WITH_EDITOR
	if (GUndo)
	{
		Modify();
	}
#endif

	for (TWeakObjectPtr<UAvaSequence> Child : ChildAnimations)
	{
		if (Child.IsValid())
		{
			Child->SetParent(nullptr);
		}
	}

	ChildAnimations.Empty();

	if (IAvaSequenceProvider* SequenceProvider = GetSequenceProvider())
	{
		SequenceProvider->ScheduleRebuildSequenceTree();
	}
}

void UAvaSequence::SetParent(UAvaSequence* InParent)
{
#if WITH_EDITOR
	if (GUndo)
	{
		Modify();
	}
#endif
	ParentAnimation = InParent;
}

UAvaSequence* UAvaSequence::GetParent() const
{
	return ParentAnimation.Get();
}

const FAvaMark* UAvaSequence::GetPreviewMark() const
{
	if (PreviewMarkLabel.IsEmpty())
	{
		return nullptr;
	}
	return Marks.Find(PreviewMarkLabel);
}

const FAvaMark* UAvaSequence::FindMark(const FMovieSceneMarkedFrame& InMarkedFrame) const
{
	return Marks.Find(InMarkedFrame);
}

FAvaMark& UAvaSequence::FindOrAddMark(const FString& InMarkLabel)
{
	return Marks.FindOrAdd(InMarkLabel);
}

bool UAvaSequence::GetMark(const FString& InMarkLabel, FAvaMark& OutMark) const
{
	if (const FAvaMark* const FoundMark = Marks.Find(InMarkLabel))
	{
		OutMark = CopyTemp(*FoundMark);
		return true;
	}
	return false;
}

bool UAvaSequence::SetMark(const FString& InMarkLabel, const FAvaMark& InMark)
{
	if (FAvaMark* const FoundMark = Marks.Find(InMarkLabel))
	{
		FoundMark->CopyFromMark(InMark);
		return true;
	}
	return false;
}

void UAvaSequence::UpdateMarkList()
{
	if (!MovieScene)
	{
		return;
	}

	// Map from Frame to the Labels of the Marks at that Frame
	TMap<int32, TArray<FString>> OldMarkLabels;

	// Clear all the Frames as they might've changed, while also counting the amount of marks per frame
	for (FAvaMark& Mark : Marks)
	{
		for (int32 Frame : Mark.Frames)
		{
			OldMarkLabels.FindOrAdd(Frame).Emplace(Mark.GetLabel());
		}
		Mark.Frames.Reset();
	}

	const TArray<FMovieSceneMarkedFrame>& MarkedFrames = MovieScene->GetMarkedFrames();

	TSet<FAvaMark> SeenMarks;
	SeenMarks.Reserve(MarkedFrames.Num());

	// Map from Frame to the Labels of the marks in that frame
	// this is to compare this with the OldMarkLabels to find if a 'new' mark is actually just a rename of an old mark
	TMap<int32, TArray<FString>> SeenMarkLabels;
	SeenMarkLabels.Reserve(MarkedFrames.Num());

	TSet<FAvaMark> NewMarks;
	
	for (const FMovieSceneMarkedFrame& MarkedFrame : MarkedFrames)
	{
		const FAvaMark Mark(MarkedFrame);
		const int32 Frame = MarkedFrame.FrameNumber.Value;

		SeenMarks.Add(Mark);
		SeenMarkLabels.FindOrAdd(Frame).Emplace(Mark.GetLabel());

		// If the given Mark matches in Label, add its Frame
		if (FAvaMark* const FoundMark = Marks.Find(Mark))
		{
			FoundMark->Frames.Add(Frame);
		}
		// else just add it to the NewMarks set.
		// But not directly to the "Marks" since it might've been a change in Label, rather than a new entry.
		else
		{
			NewMarks.Add(Mark);
		}
	}

	// Process New Entries or Modifies ones
	for (const FAvaMark& SeenMark : SeenMarks)
	{
		if (FAvaMark* const NewMark = NewMarks.Find(SeenMark))
		{
			//The New Mark should have exactly 1 frame, which is the one copied over from FMovieSceneMarkedFrame
			check(NewMark->Frames.Num() == 1);
			const int32 Frame = NewMark->Frames[0];

			//Get the Old Mark Count and New Mark Labels at this Frame
			TArray<FString>* const OldLabels = OldMarkLabels.Find(Frame);
			TArray<FString>* const NewLabels = SeenMarkLabels.Find(Frame);

			//If there are more Old Marks than New Marks at this Frame then it means Marks were deleted
			//If there are more New Marks than Old Marks at this Frame then it means Marks were added
			//so for a rename, the Counts should be the same
			if (OldLabels && NewLabels
				&& OldLabels->Num() == NewLabels->Num()
				&& OldLabels->Num() > 0)
			{
				int32 MarkIndex = INDEX_NONE;
					
				//Find first Old Label in this Frame that does not Exist in New Labels
				for (int32 Index = 0; Index < OldLabels->Num(); ++Index)
				{
					const FString& MarkLabel = (*OldLabels)[Index];
					if (!NewLabels->Contains(MarkLabel))
					{
						MarkIndex = Index;
						break;
					}
				}

				if (MarkIndex != INDEX_NONE)
				{
					const FString& OldLabel = (*OldLabels)[MarkIndex];
					FAvaMark* Mark = Marks.Find(OldLabel);

					check(Mark);

					// If the Mark we found has no frames then it means it couldn't find any more
					// frames that have this mark label and can simply rename it (i.e. remove it from New Marks and edit it directly)				
					if (Mark->Frames.Num() == 0)
					{
						Marks.Remove(OldLabel);
						FSetElementId Id = Marks.Emplace(NewMark->GetLabel());
						Mark = &Marks[Id];
						Mark->Frames = NewMark->Frames;

						// Remove this from new marks as it's no longer a new mark but just a changed one
						NewMarks.Remove(*NewMark);
					}
					// else we keep it in the NewMarks list and have it be a new entry in Marks
					// while also copying the Role and Direction from its old mark
					else
					{
						NewMark->Role      = Mark->Role;
						NewMark->Direction = Mark->Direction;
					}

					//Decrease both here as we don't want other possible marks to reuse the exact same mark label
					OldLabels->RemoveAt(MarkIndex);
					NewLabels->Remove(FString(Mark->GetLabel()));
				}
			}
		}
	}

	// If there are still marks in the NewMarks list, treat them as actual new entries
	for (const FAvaMark& NewMark : NewMarks)
	{
		Marks.Add(NewMark);
	}

	// Remove the rest of the Marks that were not in the Seen Marks.
	TSet<FAvaMark> MarkDifference = Marks.Difference(SeenMarks);
	for (const FAvaMark& MarkToRemove : MarkDifference)
	{
		Marks.Remove(MarkToRemove);
	}

	// Remove Marks with no frames
	for (TSet<FAvaMark>::TIterator Iter(Marks); Iter; ++Iter)
	{
		if (Iter->Frames.IsEmpty())
		{
			Iter.RemoveCurrent();
		}
	}
}

TArray<UObject*> UAvaSequence::GetBoundObjects(UObject* InPlaybackContext) const
{
	TArray<UObject*> OutObjects;
	if (!IsValid(MovieScene))
	{
		return OutObjects;
	}

	for (const FMovieSceneBinding& Binding : ((const UMovieScene*)MovieScene)->GetBindings())
	{
		TArray<UObject*, TInlineAllocator<1>> BoundObject;
		LocateBoundObjects(Binding.GetObjectGuid(), UE::UniversalObjectLocator::FResolveParams(InPlaybackContext), MovieSceneHelpers::CreateTransientSharedPlaybackState(InPlaybackContext, MovieScene->GetTypedOuter<UMovieSceneSequence>()), BoundObject);
		OutObjects.Append(BoundObject);
	}

	return OutObjects;
}

void UAvaSequence::Initialize()
{
	// Note: Override to prevent calling ULevelSequence::Initialize as MovieScene is created in UAvaSequence ctor
}

#if WITH_EDITOR
FText UAvaSequence::GetDisplayName() const
{
	return FText::FromName(GetLabel());
}
#endif

UObject* UAvaSequence::CreateDirectorInstance(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID InSequenceID)
{
	IAvaSequenceProvider* Provider = GetSequenceProvider();

	UObject* DirectorInstance = nullptr;

	IMovieScenePlayer* Player = UE::MovieScene::FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);

	if (Provider && Provider->CreateDirectorInstance(*this, *Player, InSequenceID, DirectorInstance))
	{
		return DirectorInstance;
	}

	UAvaSequencePlayer* SequencePlayer = Cast<UAvaSequencePlayer>(Player->AsUObject());

	UObject* DirectorOuter = SequencePlayer ? SequencePlayer : Player->GetPlaybackContext();

	if (DirectorClass && DirectorOuter && DirectorClass->IsChildOf<UAvaSequenceDirector>())
	{
		FName DirectorName = NAME_None;

#if WITH_EDITOR
		DirectorName = MakeUniqueObjectName(DirectorOuter, DirectorClass, *(GetName() + TEXT("_Director")));
#endif

		UAvaSequenceDirector* NewDirector = NewObject<UAvaSequenceDirector>(DirectorOuter, DirectorClass, DirectorName, RF_Transient);
		NewDirector->Player = nullptr;
		NewDirector->MovieScenePlayerIndex = Player->GetUniqueIndex();
		NewDirector->SubSequenceID  = InSequenceID.GetInternalValue();
		NewDirector->Initialize(*Player, GetSequenceProvider());
		NewDirector->OnCreated();
		return NewDirector;
	}

	return nullptr;
}

bool UAvaSequence::CanPossessObject(UObject& InObject, UObject* InPlaybackContext) const
{
	// Accept any object that is either an Actor or has an Actor as an outer.
	return Super::CanPossessObject(InObject, InPlaybackContext)
		|| InObject.IsA<AActor>()
		|| InObject.GetTypedOuter<AActor>();
}

UObject* UAvaSequence::GetParentObject(UObject* InObject) const
{
	if (UObject* ParentObject = Super::GetParentObject(InObject))
	{
		return ParentObject;
	}

	return InObject
		? InObject->GetTypedOuter<AActor>()
		: nullptr;
}

bool UAvaSequence::CanAnimateObject(UObject& InObject) const
{
	return Super::CanAnimateObject(InObject) || InObject.GetTypedOuter<AActor>();
}

void UAvaSequence::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FAvaSequenceVersion::GUID);
	Super::Serialize(Ar);
}

void UAvaSequence::PostLoad()
{
	Super::PostLoad();
	UpdateMarkList();
}

#if WITH_EDITOR
void UAvaSequence::PostEditUndo()
{
	Super::PostEditUndo();
	OnSequenceEditUndoDelegate.Broadcast(this);
}

void UAvaSequence::OnObjectTransacted(UObject* InObject, const FTransactionObjectEvent& InTransactionEvent)
{
	if (MovieScene == InObject)
	{
		UpdateMarkList();
	}
}
#endif

int32 UAvaSequence::UpdateBindings(const FTopLevelAssetPath* InOldContext, const FTopLevelAssetPath& InNewContext)
{
	if (!MovieScene)
	{
		return 0;
	}

	FProperty* BindingsProperty      = UE::AvaCore::GetProperty<UAvaSequence>(GET_MEMBER_NAME_CHECKED(UAvaSequence, BindingReferences));
	FProperty* ExternalPathProperty  = UE::AvaCore::GetProperty<FLevelSequenceBindingReference>(TEXT("ExternalObjectPath"));
	FMapProperty* BindingMapProperty = UE::AvaCore::GetProperty<FLevelSequenceBindingReferences, FMapProperty>(TEXT("BindingIdToReferences"));

	if (!BindingsProperty || !BindingMapProperty || !ExternalPathProperty)
	{
		return 0;
	}

	uint8* BindingReferencesAddress   = BindingsProperty->ContainerPtrToValuePtr<uint8>(this);
	uint8* BindingReferenceMapAddress = BindingMapProperty->ContainerPtrToValuePtr<uint8>(BindingReferencesAddress);

	FScriptMapHelper BindingMapHelper(BindingMapProperty, BindingReferenceMapAddress);

	int32 BindingsUpdatedCount = 0;

	for (int32 ElementIndex = 0; ElementIndex < BindingMapHelper.GetMaxIndex(); ++ElementIndex)
	{
		if (!BindingMapHelper.IsValidIndex(ElementIndex))
		{
			continue;
		}

		// Do not fix Bindings that rely on a Parent Possessable to resolve
		FGuid* BindingId = reinterpret_cast<FGuid*>(BindingMapHelper.GetKeyPtr(ElementIndex));
		if (BindingId && AreParentContextsSignificant())
		{
			FMovieScenePossessable* Possessable = MovieScene->FindPossessable(*BindingId);
			if (Possessable && Possessable->GetParent().IsValid())
			{
				continue;
			}
		}

		FLevelSequenceBindingReferenceArray* BindingReferenceArray = reinterpret_cast<FLevelSequenceBindingReferenceArray*>(BindingMapHelper.GetValuePtr(ElementIndex));
		if (!BindingReferenceArray)
		{
			continue;
		}

		for (FLevelSequenceBindingReference& Reference : BindingReferenceArray->References)
		{
			FSoftObjectPath* ExternalPathAddress = ExternalPathProperty->ContainerPtrToValuePtr<FSoftObjectPath>(&Reference);
			if (!ExternalPathAddress)
			{
				continue;	
			}

			// Only replace if the Old Context Matches the existing Binding Context
			if (!InOldContext || *InOldContext == ExternalPathAddress->GetAssetPath())
			{
				ExternalPathAddress->SetPath(InNewContext, ExternalPathAddress->GetSubPathString());
				++BindingsUpdatedCount;
			}
		}
	}

	return BindingsUpdatedCount;
}

TArrayView<TWeakObjectPtr<>> UAvaSequence::FindObjectsFromGuid(const FGuid& InGuid)
{
	using namespace UE::MovieScene;

	if (!InGuid.IsValid())
	{
		return TArrayView<TWeakObjectPtr<>>();
	}

	FSharedPlaybackStateCreateParams CreateParams;
	TSharedRef<FSharedPlaybackState> TransientPlaybackState = MakeShared<FSharedPlaybackState>(*this, CreateParams);

	FMovieSceneEvaluationState State;
	TransientPlaybackState->AddCapabilityRaw(&State);
	State.AssignSequence(MovieSceneSequenceID::Root, *this, TransientPlaybackState);

	return State.FindBoundObjects(InGuid, MovieSceneSequenceID::Root, TransientPlaybackState);
}

FGuid UAvaSequence::FindGuidFromObject(UObject* InObject)
{
	using namespace UE::MovieScene;

	if (!InObject)
	{
		return FGuid();
	}

	FSharedPlaybackStateCreateParams CreateParams;
	TSharedRef<FSharedPlaybackState> TransientPlaybackState = MakeShared<FSharedPlaybackState>(*this, CreateParams);

	FMovieSceneEvaluationState State;
	TransientPlaybackState->AddCapabilityRaw(&State);
	State.AssignSequence(MovieSceneSequenceID::Root, *this, TransientPlaybackState);

	return State.FindObjectId(*InObject, MovieSceneSequenceID::Root, TransientPlaybackState);
}

#if WITH_EDITOR
void UAvaSequence::OnOuterWorldRenamed(UObject* InNewOuter)
{
	if (DirectorBlueprint)
	{
		DirectorBlueprint->Rename(*GetDirectorBlueprintName(), InNewOuter, REN_DontCreateRedirectors);
	}
}
#endif

void UAvaSequence::Cleanup()
{
#if WITH_EDITOR
	if (DirectorBlueprint)
	{
		FBlueprintActionDatabase& BlueprintActionDatabase = FBlueprintActionDatabase::Get();
		BlueprintActionDatabase.ClearAssetActions(DirectorBlueprint.Get());
	}
#endif
}

void UAvaSequence::OnWorldCleanup(UWorld* InWorld, bool bInSessionEnded, bool bInCleanupResources)
{
	// Ignore cleanups from other worlds
	if (GetTypedOuter<UWorld>() != InWorld)
	{
		return;
	}

	if (!bInCleanupResources)
	{
		return;
	}

	auto CleanupObject = []<typename T>(TObjectPtr<T>& InOutObject)
	{
		if (!InOutObject)
		{
			return;
		}

		FName ObjectName = MakeUniqueObjectName(GetTransientPackage()
			, InOutObject->GetClass()
			, FName(*FString::Printf(TEXT("%s_Trashed"), *InOutObject->GetName())));

		InOutObject->Rename(*ObjectName.ToString()
			, GetTransientPackage()
			, REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty);
	};

	Cleanup();

#if WITH_EDITORONLY_DATA
	CleanupObject(DirectorBlueprint);
#endif
	CleanupObject(DirectorClass);
}

#undef LOCTEXT_NAMESPACE
