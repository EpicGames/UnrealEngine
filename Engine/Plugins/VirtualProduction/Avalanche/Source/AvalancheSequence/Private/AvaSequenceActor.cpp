// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceActor.h"
#include "AvaSequence.h"
#include "AvaSequencePlayer.h"
#include "AvaSequenceSubsystem.h"

AAvaSequenceActor::AAvaSequenceActor(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer.SetDefaultSubobjectClass(TEXT("AnimationPlayer"), UAvaSequencePlayer::StaticClass()))
{
}

void AAvaSequenceActor::Initialize(UAvaSequence* InSequence)
{
	if (ensure(!GetSequencePlayer()->IsPlaying()))
	{
		LevelSequenceAsset = InSequence;
		InitSequencePlayer(InSequence);
	}
}

void AAvaSequenceActor::PostInitializeComponents()
{
	Super::PostInitializeComponents();
	InitSequencePlayer(Cast<UAvaSequence>(LevelSequenceAsset));
}

void AAvaSequenceActor::InitSequencePlayer(UAvaSequence* InSequence)
{
	if (!InSequence)
	{
		return;
	}

	ULevel* Level = GetLevel();
	if (!Level)
	{
		return;
	}

	UAvaSequencePlayer* Player = Cast<UAvaSequencePlayer>(GetSequencePlayer());
	if (!Player)
	{
		return;
	}

	UAvaSequenceSubsystem* SequenceSubsystem = UAvaSequenceSubsystem::Get(this);
	if (!SequenceSubsystem)
	{
		return;
	}

	IAvaSequencePlaybackObject* PlaybackObject = SequenceSubsystem->FindPlaybackObject(Level);
	if (!PlaybackObject)
	{
		return;
	}

	Player->InitSequence(InSequence, PlaybackObject, Level, CameraSettings);
}
