// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceController.h"
#include "AvaSequence.h"
#include "AvaSequencePlaybackContext.h"
#include "HAL/IConsoleManager.h"
#include "Marks/AvaMarkRole_Jump.h"
#include "Marks/AvaMarkRole_Pause.h"
#include "Marks/AvaMarkRole_Reverse.h"
#include "Marks/AvaMarkRole_Stop.h"
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"

DEFINE_LOG_CATEGORY_STATIC(LogAvaSequenceController, Log, All);

namespace UE::Ava::Private
{
	double GetFrameTimeAsDisplayDecimal(UMovieScene& InMovieScene, const FFrameTime& InTickTime)
	{
		return ConvertFrameTime(InTickTime, InMovieScene.GetTickResolution(), InMovieScene.GetDisplayRate()).AsDecimal();
	}

#if !NO_LOGGING
	bool bAvaLogMarkedFrames = false;
	FAutoConsoleVariableRef CVarLogAvaSequenceController(TEXT("Log.MotionDesign.SequenceController")
		, bAvaLogMarkedFrames
		, TEXT("Log FAvaSequenceController"));
#endif
};

class FAvaSequenceControllerTickScope
{
public:
	FAvaSequenceControllerTickScope(UAvaSequence& InSequence)
		: Sequence(InSequence)
	{
		UE_CLOGF(FAvaSequenceControllerTickScope::IsLoggingEnabled(), LogAvaSequenceController, Log
			, "----- %ls Begin Sequence Controller Tick"
			, *Sequence.GetName());
	}

	~FAvaSequenceControllerTickScope()
	{
		UE_CLOGF(FAvaSequenceControllerTickScope::IsLoggingEnabled(), LogAvaSequenceController, Log
			, "----- %ls End Sequence Controller Tick\n"
			, *Sequence.GetName());
	}

	static bool IsLoggingEnabled()
	{
#if !NO_LOGGING
		return UE::Ava::Private::bAvaLogMarkedFrames;
#else
		return false;
#endif
	}

private:
	UAvaSequence& Sequence;
};

FAvaSequenceController::FAvaSequenceController(UAvaSequence& InSequence, IAvaSequencePlaybackObject* InPlaybackObject)
	: PlaybackContext(MakeShared<FAvaSequencePlaybackContext>(InSequence, InPlaybackObject))
	, PlaybackStatus(EMovieScenePlayerStatus::Stopped)
	, SequenceWeak(&InSequence)
	, PlayDirection(EPlayDirection::Forwards)
{
	MarkRoleHandler.RegisterRole<FAvaMarkRole_Stop>();
	MarkRoleHandler.RegisterRole<FAvaMarkRole_Pause>();
	MarkRoleHandler.RegisterRole<FAvaMarkRole_Jump>();
	MarkRoleHandler.RegisterRole<FAvaMarkRole_Reverse>();

	InSequence.GetOnTreeNodeUpdated().AddRaw(this, &FAvaSequenceController::OnTreeNodeCleanup);
}

FAvaSequenceController::~FAvaSequenceController()
{
	if (UAvaSequence* const Sequence = SequenceWeak.Get(/*bEvenIfPendingKill*/true))
	{
		Sequence->GetOnTreeNodeUpdated().RemoveAll(this);
	}
}

void FAvaSequenceController::SetTime(const FFrameTime& InNewTime, bool bInResetState)
{
	CurrentFrame  = InNewTime;
	PreviousFrame = InNewTime;

	if (bInResetState)
	{
		ResetState();	
	}
}

void FAvaSequenceController::ResetState()
{
	PlaybackContext->RequestRefreshChildren();
	MarkRoleHandler.ResetMarkStates();
	MarkFramesProcessed.Reset();
}

void FAvaSequenceController::Tick(const FAvaSequencePlayerVariant& InPlayerVariant, const FFrameTime& InDeltaFrameTime, float InDeltaSeconds)
{
	if (!SequenceWeak.IsValid())
	{
		return;
	}

	UAvaSequence& Sequence = *SequenceWeak;
	if (!Sequence.GetMovieScene())
	{
		return;
	}

	UMovieScene& MovieScene = *Sequence.GetMovieScene();

	PlaybackContext->GetTicker().Tick(InDeltaSeconds);
	PlaybackContext->Resolve(InPlayerVariant, InDeltaFrameTime);

	// Update Playback Status
	const EMovieScenePlayerStatus::Type PreviousPlaybackStatus = PlaybackStatus;
	PlaybackStatus = PlaybackContext->GetPlaybackStatus();

	// Early Return if both this tick and previous did were not playing
	// Allow the first tick when stopped to be fully processed as there might've been a last mark in between last frame and the end of the sequence
	if (PlaybackStatus != EMovieScenePlayerStatus::Playing && PreviousPlaybackStatus != EMovieScenePlayerStatus::Playing)
	{
		return;
	}

	FAvaSequenceControllerTickScope TickScope(Sequence);

	UpdatePlayers();
	UpdatePlaybackVariables();

	UE_CLOGF(FAvaSequenceControllerTickScope::IsLoggingEnabled(), LogAvaSequenceController, Log
		, "Previous Frame: %f ---  Current Frame: %f"
		, UE::Ava::Private::GetFrameTimeAsDisplayDecimal(MovieScene, PreviousFrame)
		, UE::Ava::Private::GetFrameTimeAsDisplayDecimal(MovieScene, CurrentFrame));

	SortMarks(MovieScene);

	TArray<const FMovieSceneMarkedFrame*> IntersectedMarks = FindIntersectedMarks(MovieScene);
	if (IntersectedMarks.IsEmpty())
	{
		if (PlaybackStatus == EMovieScenePlayerStatus::Playing)
		{
			MarkFramesProcessed.Reset();
		}
		return;
	}

	// Marks are sorted from Lowest to Highest Frame, Reverse order if playing Backwards
	if (PlayDirection == EPlayDirection::Backwards)
	{
		Algo::Reverse(IntersectedMarks);
	}

	// Process the Intersected Marks until we reach one that Executes successfully
	for (const FMovieSceneMarkedFrame* const Mark : IntersectedMarks)
	{
		if (!ensure(Mark))
		{
			continue;
		}

		// Skip if this Mark has already been processed
		if (MarkFramesProcessed.Contains(Mark->FrameNumber))
		{
			UE_CLOGF(FAvaSequenceControllerTickScope::IsLoggingEnabled(), LogAvaSequenceController, Log
				, "Mark [%ls] has already been processed"
				, *GetMarkAsString(MovieScene, *Mark));	
			continue;
		}

		// Only process if Mark is valid (e.g. correct play direction, etc)
		if (IsMarkValid(*Mark, Sequence))
		{
			const EAvaMarkRoleReply RoleReply = ExecuteMark(*Mark, Sequence);

			// If Mark executed successfully add to mark processed frame and break,
			// we should only run one mark at a time, though we could add a EAvaMarkRoleReply to support multi-execution
			if (RoleReply == EAvaMarkRoleReply::Executed)
			{
				UE_CLOGF(FAvaSequenceControllerTickScope::IsLoggingEnabled(), LogAvaSequenceController, Log
					, "Executed Mark [%ls] with Role [%ls]"
					, *GetMarkAsString(MovieScene, *Mark)
					, *GetMarkRoleAsString(Sequence, *Mark));

				MarkFramesProcessed.Add(Mark->FrameNumber);

				// Set the Current Frame to this Mark's Frame so that next time when playing
				// marks not executed are considered
				CurrentFrame = Mark->FrameNumber;
				break;
			}
			else
			{
				UE_CLOGF(FAvaSequenceControllerTickScope::IsLoggingEnabled(), LogAvaSequenceController, Log
					, "Could not Execute Mark [%ls] with Role [%ls], as it had invalid parameters"
                	, *GetMarkAsString(MovieScene, *Mark)
                	, *GetMarkRoleAsString(Sequence, *Mark));
			}
		}
		else
		{
			UE_CLOGF(FAvaSequenceControllerTickScope::IsLoggingEnabled(), LogAvaSequenceController, Log
				, "Skipped Mark [%ls] with Role [%ls], as it did not satisfy the appropriate conditions"
				, *GetMarkAsString(MovieScene, *Mark)
				, *GetMarkRoleAsString(Sequence, *Mark));
		}
	}
}

FString FAvaSequenceController::GetMarkRoleAsString(const UAvaSequence& InSequence, const FMovieSceneMarkedFrame& InMark) const
{
	if (const FAvaMark* const FoundMark = InSequence.GetMarks().Find(InMark))
	{
		return UEnum::GetValueAsString(FoundMark->Role);
	}
	static const FString InvalidText(TEXT("(no role found)"));
	return InvalidText;
}

FString FAvaSequenceController::GetMarkAsString(UMovieScene& InMovieScene, const FMovieSceneMarkedFrame& InMark) const
{
	const double DisplayFrame = UE::Ava::Private::GetFrameTimeAsDisplayDecimal(InMovieScene, InMark.FrameNumber);

	return InMark.Label
		+ TEXT(" @ ")
		+ FString::SanitizeFloat(DisplayFrame);
}

TSharedRef<IAvaSequencePlaybackContext> FAvaSequenceController::GetPlaybackContext() const
{
	return PlaybackContext;
}

void FAvaSequenceController::OnTreeNodeCleanup()
{
	PlaybackContext->RequestRefreshChildren();
}

void FAvaSequenceController::UpdatePlayers()
{
	const IMovieScenePlayer* const PreviousPlayer = LastPlayer;
	const IMovieScenePlayer* const CurrentPlayer  = PlaybackContext->GetPlayer();

	LastPlayer = CurrentPlayer;

	// If Players differ, reset the State
	if (CurrentPlayer != PreviousPlayer)
	{
		UE_CLOGF(FAvaSequenceControllerTickScope::IsLoggingEnabled(), LogAvaSequenceController, Log
			, "Sequence changed Movie Scene Players. Resetting Mark States.");
		ResetState();
	}
}

void FAvaSequenceController::UpdatePlaybackVariables()
{
	PlayDirection = PlaybackContext->IsPlayingForwards() ? EPlayDirection::Forwards : EPlayDirection::Backwards;

	// NOTE: Marks in both Sequencer + Motion Design Player are processed prior to these Players processing Tick this frame
	// So Global Time is currently the result of the Last Frame, rather than the result of this frame
	const FFrameTime LastFrameTime = PlaybackContext->GetGlobalTime();

	// If not playing, then Delta should be considered 0
	const FFrameTime DeltaFrameTime  = PlaybackStatus == EMovieScenePlayerStatus::Playing
		? PlaybackContext->GetDeltaFrameTime()
		: FFrameTime(0);

	if (TOptional<FFrameTime> JumpedFrame = PlaybackContext->GetLastJumpedFrame())
	{
		PreviousFrame = *JumpedFrame;
		CurrentFrame  = LastFrameTime + DeltaFrameTime;

		// When Jumping, forget about which Marked Frames were processed, as the same Mark that jumped could be coming up next (i.e. loop back)
		MarkFramesProcessed.Reset();
	}
	else
	{
		PreviousFrame = MoveTemp(CurrentFrame);
		CurrentFrame  = LastFrameTime + DeltaFrameTime;
	}
}

void FAvaSequenceController::SortMarks(UMovieScene& InMovieScene)
{
	const TArray<FMovieSceneMarkedFrame>& MarkedFrames = InMovieScene.GetMarkedFrames();
	if (MarkedFrames.IsEmpty())
	{
		return;
	}

	const FFrameNumber* PreviousFrameNumber = nullptr;

	bool bRequiresSorting = false;

	for (const FMovieSceneMarkedFrame& MarkedFrame : InMovieScene.GetMarkedFrames())
	{
		if (PreviousFrameNumber && *PreviousFrameNumber > MarkedFrame.FrameNumber)
		{
			bRequiresSorting = true;
			break;
		}
		PreviousFrameNumber = &MarkedFrame.FrameNumber;
	}

	if (bRequiresSorting)
	{
		UE_CLOGF(FAvaSequenceControllerTickScope::IsLoggingEnabled(), LogAvaSequenceController, Log
			, "Unsorted Marks found. Sorting.");	
		InMovieScene.SortMarkedFrames();
	}
}

TArray<const FMovieSceneMarkedFrame*> FAvaSequenceController::FindIntersectedMarks(UMovieScene& InMovieScene)
{
	TArray<const FMovieSceneMarkedFrame*> IntersectedMarks;

	const FFrameTime LowerBound = (PlayDirection == EPlayDirection::Forwards ? PreviousFrame : CurrentFrame);
	const FFrameTime UpperBound = (PlayDirection == EPlayDirection::Forwards ? CurrentFrame : PreviousFrame);

	// Todo: marked frames are sorted at this point so could optimize getting the marked frames using binary search
	for (const FMovieSceneMarkedFrame& MarkedFrame : InMovieScene.GetMarkedFrames())
	{
		if (FMath::IsWithinInclusive(MarkedFrame.FrameNumber, LowerBound, UpperBound))
		{
			IntersectedMarks.Add(&MarkedFrame);
		}
	}

	if (FAvaSequenceControllerTickScope::IsLoggingEnabled())
	{
		if (IntersectedMarks.IsEmpty())
		{
			UE_LOGF(LogAvaSequenceController, Log, "No Marks between %f and %f"
				, UE::Ava::Private::GetFrameTimeAsDisplayDecimal(InMovieScene, LowerBound)
				, UE::Ava::Private::GetFrameTimeAsDisplayDecimal(InMovieScene, UpperBound));	
		}
		else
		{
			FString MarkList;
			for (const FMovieSceneMarkedFrame* const Mark : IntersectedMarks)
			{
				MarkList += GetMarkAsString(InMovieScene, *Mark) + TEXT(", ");
			}
			UE_LOGF(LogAvaSequenceController, Log, "%d Marks between %f and %f --- [ %ls ]"
				, IntersectedMarks.Num()
				, UE::Ava::Private::GetFrameTimeAsDisplayDecimal(InMovieScene, LowerBound)
				, UE::Ava::Private::GetFrameTimeAsDisplayDecimal(InMovieScene, UpperBound)
				, *MarkList);	
		}
	}

	return IntersectedMarks;
}

bool FAvaSequenceController::IsMarkValid(const FMovieSceneMarkedFrame& InMarkedFrame, const UAvaSequence& InSequence) const
{
	if (const FAvaMark* const Mark = InSequence.FindMark(InMarkedFrame))
	{
		return MarkRoleHandler.IsMarkValid(*Mark, InMarkedFrame.FrameNumber, PlayDirection);
	}
	return false;
}

EAvaMarkRoleReply FAvaSequenceController::ExecuteMark(const FMovieSceneMarkedFrame& InMarkedFrame, const UAvaSequence& InSequence)
{
	PlaybackContext->UpdateMarkedFrame(InMarkedFrame);
	return MarkRoleHandler.ExecuteRole(PlaybackContext);
}
