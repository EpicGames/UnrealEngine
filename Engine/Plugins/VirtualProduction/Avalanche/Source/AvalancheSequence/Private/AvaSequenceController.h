// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/Set.h"
#include "Evaluation/MovieScenePlayback.h"
#include "IAvaSequenceController.h"
#include "Marks/AvaMarkRole.h"
#include "Marks/AvaMarkShared.h"
#include "Misc/FrameTime.h"
#include "MovieSceneFwd.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class FAvaSequencePlaybackContext;
class IAvaSequencePlaybackObject;
class IMovieScenePlayer;
class UAvaSequence;
class UMovieScene;
enum class EPlayDirection;
struct FFrameNumber;
struct FMovieSceneMarkedFrame;

class FAvaSequenceController : public IAvaSequenceController
{
public:
	FAvaSequenceController(UAvaSequence& InSequence, IAvaSequencePlaybackObject* InPlaybackObject);

	virtual ~FAvaSequenceController() override;

	//~ Begin IAvaSequenceController
	virtual void SetTime(const FFrameTime& InNewTime, bool bInResetState) override;
	virtual void Tick(const FAvaSequencePlayerVariant& InPlayerVariant, const FFrameTime& InDeltaFrameTime, float InDeltaSeconds) override;
	virtual UAvaSequence* GetSequence() const override { return SequenceWeak.Get(); }
	virtual TSharedRef<IAvaSequencePlaybackContext> GetPlaybackContext() const override;
	//~ End IAvaSequenceController

private:
	FString GetMarkRoleAsString(const UAvaSequence& InSequence, const FMovieSceneMarkedFrame& InMark) const;

	FString GetMarkAsString(UMovieScene& InMovieScene, const FMovieSceneMarkedFrame& InMark) const;

	void OnTreeNodeCleanup();

	void ResetState();

	void UpdatePlayers();

	/** Updates Current Frame, returning the Old Current Frame */
	void UpdatePlaybackVariables();

	void SortMarks(UMovieScene& InMovieScene);

	TArray<const FMovieSceneMarkedFrame*> FindIntersectedMarks(UMovieScene& InMovieScene);

	bool IsMarkValid(const FMovieSceneMarkedFrame& InMarkedFrame, const UAvaSequence& InSequence) const;

	EAvaMarkRoleReply ExecuteMark(const FMovieSceneMarkedFrame& InMarkedFrame, const UAvaSequence& InSequence);

	TSharedRef<FAvaSequencePlaybackContext> PlaybackContext;

	FAvaMarkRoleHandler MarkRoleHandler;

	EMovieScenePlayerStatus::Type PlaybackStatus;

	TWeakObjectPtr<UAvaSequence> SequenceWeak;

	FFrameTime PreviousFrame;

	FFrameTime CurrentFrame;

	/** The Marks that were processed/executed last time */
	TSet<FFrameNumber> MarkFramesProcessed;

	/** Used to check if the Players differ */
	const IMovieScenePlayer* LastPlayer = nullptr;

	EPlayDirection PlayDirection;
};
