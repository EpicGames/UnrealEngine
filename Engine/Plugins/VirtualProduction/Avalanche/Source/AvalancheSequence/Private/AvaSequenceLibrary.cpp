// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceLibrary.h"
#include "AvaSequencePlaybackObject.h"
#include "AvaSequenceSubsystem.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/World.h"

TScriptInterface<IAvaSequencePlaybackObject> UAvaSequenceLibrary::GetPlaybackObject(const UObject* InWorldContextObject)
{
	if (!InWorldContextObject || !GEngine)
	{
		return nullptr;
	}

	UWorld* World = GEngine->GetWorldFromContextObject(InWorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
	if (!World)
	{
		return nullptr;
	}

	UAvaSequenceSubsystem* SequenceSubsystem = World->GetSubsystem<UAvaSequenceSubsystem>();
	if (!SequenceSubsystem)
	{
		return nullptr;
	}

	ULevel* Level = InWorldContextObject->GetTypedOuter<ULevel>();
	if (!Level)
	{
		Level = World->PersistentLevel;
		if (!Level)
		{
			return nullptr;
		}
	}

	IAvaSequencePlaybackObject* PlaybackObject = SequenceSubsystem->FindPlaybackObject(Level);
	if (!PlaybackObject)
	{
		return nullptr;
	}

	return TScriptInterface<IAvaSequencePlaybackObject>(PlaybackObject->ToUObject());	
}

FAvaSequencePlayParams UAvaSequenceLibrary::MakeSingleFramePlaySettings(const FAvaSequenceTime& InTargetTime, EAvaSequencePlayMode InPlayMode)
{
	FAvaSequencePlayParams PlaySettings;

	// Set start to the largest double, so that it gets clamped down to be End Time (i.e. time that the Sequence should evaluate)
	// Start is not set to End as the passed in time could be unset (no time constraint) -- which then could resolve to different values for Start/End.
	PlaySettings.Start = FAvaSequenceTime(TNumericLimits<double>::Max());
	PlaySettings.End = InTargetTime;
	PlaySettings.PlayMode = InPlayMode;

	return PlaySettings;
}
