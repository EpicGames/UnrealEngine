// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceName.h"

bool FAvaSequenceName::SerializeFromMismatchedTag(const FPropertyTag& InPropertyTag, FStructuredArchive::FSlot InSlot)
{
	if (InPropertyTag.GetType().GetName() == NAME_NameProperty)
	{
		InSlot << Name;
		return true;
	}
	return false;
}
