// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequencePlaybackActor.h"
#include "AvaSequencePlayer.h"
#include "AvaSequenceSubsystem.h"
#include "AvaTag.h"
#include "Engine/World.h"
#include "IAvaSequenceProvider.h"
#include "MovieSceneSequenceTickManager.h"
#include "UObject/UObjectThreadContext.h"

DEFINE_LOG_CATEGORY_STATIC(LogAvaSequencePlayback, Log, All);

AAvaSequencePlaybackActor::AAvaSequencePlaybackActor()
{
	// Tick Settings that follow through FMovieSceneSequenceTickInterval
	PrimaryActorTick.TickInterval = 0.f;
	PrimaryActorTick.bTickEvenWhenPaused = true;

	OnSequenceFinishedDelegate = UAvaSequencePlayer::OnSequenceFinished().AddUObject(this, &AAvaSequencePlaybackActor::OnSequenceFinished);

	OnWorldCleanupDelegate = FWorldDelegates::OnWorldCleanup.AddUObject(this, &AAvaSequencePlaybackActor::OnWorldCleanup);
}

void AAvaSequencePlaybackActor::SetSequenceProvider(IAvaSequenceProvider& InSequenceProvider)
{
	SequenceProvider.SetObject(InSequenceProvider.ToUObject());
	SequenceProvider.SetInterface(&InSequenceProvider);
}

void AAvaSequencePlaybackActor::CleanupPlayers()
{
	// "OnSequenceFinished" is called while cleaning up a player which as a result removes the players from these sets.
	// To prevent removing element one by one and while iterating these sets, these sets are moved to temp set.

	TSet<TObjectPtr<UAvaSequencePlayer>> SequencePlayers;
	SequencePlayers.Reserve(StoppedSequencePlayers.Num() + ActiveSequencePlayers.Num());

	SequencePlayers.Append(MoveTemp(StoppedSequencePlayers));
	SequencePlayers.Append(MoveTemp(ActiveSequencePlayers));

	StoppedSequencePlayers.Reset();
	ActiveSequencePlayers.Reset();

	for (UAvaSequencePlayer* Player : SequencePlayers)
	{
		if (Player)
		{
			Player->Cleanup();
		}
	}
}

UAvaSequencePlayer* AAvaSequencePlaybackActor::PlaySequence(UAvaSequence* InSequence, const FAvaSequencePlayParams& InPlaySettings)
{
	if (!IsValid(InSequence))
	{
		return nullptr;
	}

	if (UAvaSequencePlayer* const Player = GetOrAddSequencePlayer(InSequence))
	{
		Player->SetPlaySettings(InPlaySettings);
		Player->PlaySequence();

		UE_LOGF(LogAvaSequencePlayback, Verbose
			, "Playing Sequence '%ls' ('%ls') with Player '%ls'"
			, *InSequence->GetLabel().ToString()
			, *InSequence->GetName()
			, *Player->GetName());

		return Player;
	}

	UE_LOGF(LogAvaSequencePlayback, Warning
		, "Failed to play Sequence '%ls' ('%ls'). It was valid, but a Player could not be allocated."
		, *InSequence->GetLabel().ToString()
		, *InSequence->GetName());

	return nullptr;
}

UAvaSequencePlayer* AAvaSequencePlaybackActor::PreviewFrame(UAvaSequence* InSequence)
{
	if (!IsValid(InSequence))
	{
		return nullptr;
	}

	const FAvaMark* PreviewMark = InSequence->GetPreviewMark();
	if (!PreviewMark)
	{
		UE_LOGF(LogAvaSequencePlayback, Warning
			, "Failed to preview Sequence '%ls' ('%ls'). Missing Preview Mark."
			, *InSequence->GetLabel().ToString()
			, *InSequence->GetName());
		return nullptr;
	}

	FFrameTime PreviewPosition;
	{
		FMovieSceneSequencePlaybackParams PlaybackParams;
		PlaybackParams.MarkedFrame = PreviewMark->GetLabel();
		PlaybackParams.PositionType = EMovieScenePositionType::MarkedFrame;

		PreviewPosition = PlaybackParams.GetPlaybackPosition(InSequence);
	}

	FAvaSequencePlayParams PlaySettings;
	PlaySettings.Start = PlaySettings.End = FAvaSequenceTime(PreviewPosition);

	return PlaySequence(InSequence, PlaySettings);
}

UAvaSequencePlayer* AAvaSequencePlaybackActor::PlaySequenceBySoftReference(TSoftObjectPtr<UAvaSequence> InSequence, const FAvaSequencePlayParams& InPlaySettings)
{
	if (UAvaSequence* const ResolvedSequence = InSequence.Get())
	{
		return PlaySequence(ResolvedSequence, InPlaySettings);
	}

	UE_LOGF(LogAvaSequencePlayback, Warning
		, "Failed to play sequence '%ls'. It is invalid, or not yet loaded"
		, *InSequence.ToString());

	return nullptr;
}

TArray<UAvaSequencePlayer*> AAvaSequencePlaybackActor::PlaySequencesByLabel(FName InSequenceLabel, const FAvaSequencePlayParams& InPlaySettings)
{
	return PlaySequencesByLabels({ InSequenceLabel }, InPlaySettings);
}

TArray<UAvaSequencePlayer*> AAvaSequencePlaybackActor::PlaySequencesBySoftReference(const TArray<TSoftObjectPtr<UAvaSequence>>& InSequences, const FAvaSequencePlayParams& InPlaySettings)
{
	TArray<UAvaSequencePlayer*> Players;
	Players.Reserve(InSequences.Num());

	for (const TSoftObjectPtr<UAvaSequence>& Sequence : InSequences)
	{
		Players.Add(PlaySequenceBySoftReference(Sequence, InPlaySettings));
	}

	return Players;
}

TArray<UAvaSequencePlayer*> AAvaSequencePlaybackActor::PlaySequencesByTag(const FAvaTagHandle& InTagHandle, bool bInExactMatch, const FAvaSequencePlayParams& InPlaySettings)
{
	if (!InTagHandle.IsValid())
	{
		return TArray<UAvaSequencePlayer*>();
	}

	const TArray<UAvaSequence*> SequencesToPlay = GetSequencesByTag(InTagHandle, bInExactMatch);

	TArray<UAvaSequencePlayer*> SequencePlayers;
	SequencePlayers.Reserve(SequencesToPlay.Num());

	for (UAvaSequence* Sequence : SequencesToPlay)
	{
		if (UAvaSequencePlayer* SequencePlayer = PlaySequence(Sequence, InPlaySettings))
		{
			SequencePlayers.Add(SequencePlayer);	
		}
	}

	return SequencePlayers;
}

TArray<UAvaSequencePlayer*> AAvaSequencePlaybackActor::PlaySequencesByLabels(const TArray<FName>& InSequenceLabels, const FAvaSequencePlayParams& InPlaySettings)
{
	if (InSequenceLabels.IsEmpty())
	{
		return TArray<UAvaSequencePlayer*>();
	}

	const TArray<UAvaSequence*> SequencesToPlay = GetSequencesByLabel(InSequenceLabels);

	TArray<UAvaSequencePlayer*> SequencePlayers;
	SequencePlayers.Reserve(SequencesToPlay.Num());

	for (UAvaSequence* Sequence : SequencesToPlay)
	{
		SequencePlayers.Add(PlaySequence(Sequence, InPlaySettings));
	}

	return SequencePlayers;
}

TArray<UAvaSequencePlayer*> AAvaSequencePlaybackActor::PlayScheduledSequences()
{
	return PlaySequencesByLabels(ScheduledSequenceNames, ScheduledPlaySettings);
}

UAvaSequencePlayer* AAvaSequencePlaybackActor::ContinueSequence(UAvaSequence* InSequence)
{
	if (UAvaSequencePlayer* const Player = GetSequencePlayer(InSequence))
	{
		Player->ContinueSequence();
		return Player;
	}
	return nullptr;
}

TArray<UAvaSequencePlayer*> AAvaSequencePlaybackActor::ContinueSequencesByLabel(FName InSequenceLabel)
{
	return ContinueSequencesByLabels({ InSequenceLabel });
}

TArray<UAvaSequencePlayer*> AAvaSequencePlaybackActor::ContinueSequencesByTag(const FAvaTagHandle& InTagHandle, bool bInExactMatch)
{
	TArray<UAvaSequencePlayer*> SequencePlayers;
	
	const TArray<UAvaSequence*> SequencesToContinue = GetSequencesByTag(InTagHandle, bInExactMatch);
	for (UAvaSequence* Sequence : SequencesToContinue)
	{
		if (UAvaSequencePlayer* Player = ContinueSequence(Sequence))
		{
			SequencePlayers.Add(Player);
		}
	}

	return SequencePlayers;
}

void AAvaSequencePlaybackActor::PauseSequence(UAvaSequence* InSequence)
{
	if (UAvaSequencePlayer* const Player = GetSequencePlayer(InSequence))
	{
		Player->Pause();
	}
}

void AAvaSequencePlaybackActor::PauseSequencesByLabel(FName InSequenceLabel)
{
	PauseSequencesByLabels({ InSequenceLabel });
}

void AAvaSequencePlaybackActor::PauseSequencesByLabels(const TArray<FName>& InSequenceLabels)
{
	const TArray<UAvaSequencePlayer*> SequencePlayers = GetSequencePlayersByLabels(InSequenceLabels);
	for (UAvaSequencePlayer* SequencePlayer : SequencePlayers)
	{
		SequencePlayer->Pause();
	}
}

void AAvaSequencePlaybackActor::PauseSequencesByTag(const FAvaTagHandle& InTagHandle, bool bInExactMatch)
{
	const TArray<UAvaSequencePlayer*> SequencePlayers = GetSequencePlayersByTag(InTagHandle, bInExactMatch);
	for (UAvaSequencePlayer* SequencePlayer : SequencePlayers)
	{
		SequencePlayer->Pause();
	}
}

TArray<UAvaSequencePlayer*> AAvaSequencePlaybackActor::ContinueSequencesByLabels(const TArray<FName>& InSequenceLabels)
{
	TArray<UAvaSequencePlayer*> SequencePlayers;
	SequencePlayers.Reserve(InSequenceLabels.Num());

	const TArray<UAvaSequence*> SequencesToContinue = GetSequencesByLabel(InSequenceLabels);
	for (UAvaSequence* Sequence : SequencesToContinue)
	{
		SequencePlayers.Add(ContinueSequence(Sequence));
	}

	return SequencePlayers;
}

void AAvaSequencePlaybackActor::StopSequence(UAvaSequence* InSequence)
{
	if (UAvaSequencePlayer* const Player = GetSequencePlayer(InSequence))
	{
		Player->Stop();
	}
}

void AAvaSequencePlaybackActor::StopSequencesByLabel(FName InSequenceLabel)
{
	StopSequencesByLabels({ InSequenceLabel });
}

void AAvaSequencePlaybackActor::StopSequencesByLabels(const TArray<FName>& InSequenceLabels)
{
	const TArray<UAvaSequencePlayer*> SequencePlayers = GetSequencePlayersByLabels(InSequenceLabels);
	for (UAvaSequencePlayer* SequencePlayer : SequencePlayers)
	{
		SequencePlayer->Stop();
	}
}

void AAvaSequencePlaybackActor::StopSequencesByTag(const FAvaTagHandle& InTagHandle, bool bInExactMatch)
{
	const TArray<UAvaSequencePlayer*> SequencePlayers = GetSequencePlayersByTag(InTagHandle, bInExactMatch);
	for (UAvaSequencePlayer* SequencePlayer : SequencePlayers)
	{
		SequencePlayer->Stop();
	}
}

void AAvaSequencePlaybackActor::UpdateCameraCut(const UE::MovieScene::FOnCameraCutUpdatedParams& InCameraCutParams)
{
	OnCameraCut.Broadcast(InCameraCutParams.ViewTarget, InCameraCutParams.bIsJumpCut);
}

void AAvaSequencePlaybackActor::OnSequenceFinished(UAvaSequencePlayer* InPlayer, UAvaSequence* InSequence)
{
	if (ActiveSequencePlayers.Remove(InPlayer) > 0)
	{
		StoppedSequencePlayers.Add(InPlayer);
	}
}

const TArray<UAvaSequence*>& AAvaSequencePlaybackActor::GetAllSequences() const
{
	IAvaSequenceProvider* SequenceProviderInterface = SequenceProvider.GetInterface();

	if (!SequenceProviderInterface)
	{
		UE_LOGF(LogAvaSequencePlayback, Warning
			, "Failed to get any sequences by name. Sequence Provider is invalid in playback actor %ls"
			, *GetName());

		static const TArray<UAvaSequence*> EmptySequences;
		return EmptySequences;
	}

	const TArray<TObjectPtr<UAvaSequence>>& AllSequences = SequenceProviderInterface->GetSequences();

	if (AllSequences.IsEmpty())
	{
		UE_LOGF(LogAvaSequencePlayback, Warning
			, "Failed to get sequences. Sequence Provider '%ls' returned an empty list of playable sequences"
			, *SequenceProviderInterface->GetSequenceProviderDebugName().ToString());
		return AllSequences;
	}

	return AllSequences;
}

TArray<UAvaSequence*> AAvaSequencePlaybackActor::GetSequencesByLabel(TConstArrayView<FName> InSequenceLabels) const
{
	if (InSequenceLabels.IsEmpty())
	{
		UE_LOGF(LogAvaSequencePlayback, Verbose
			, "Input Sequence Names Array was empty in playback actor %ls"
			, *GetName());
		return TArray<UAvaSequence*>();
	}

	const TArray<UAvaSequence*>& AllSequences = GetAllSequences();
	if (AllSequences.IsEmpty())
	{
		return TArray<UAvaSequence*>();
	}

	TMap<FName, TArray<UAvaSequence*>> SequenceLabelMap;
	SequenceLabelMap.Reserve(AllSequences.Num());

	for (UAvaSequence* Sequence : AllSequences)
	{
		if (Sequence)
		{
			SequenceLabelMap.FindOrAdd(Sequence->GetLabel()).Add(Sequence);
		}
	}

	TArray<UAvaSequence*> OutSequences;
	OutSequences.Reserve(InSequenceLabels.Num());

	for (FName SequenceLabel : InSequenceLabels)
	{
		if (TArray<UAvaSequence*>* FoundSequences = SequenceLabelMap.Find(SequenceLabel))
		{
			OutSequences.Append(*FoundSequences);
		}
		else
		{
			UE_LOGF(LogAvaSequencePlayback, Warning
				, "Failed to find '%ls', as a valid sequence with such label could not be found in the Sequence Provider list of playable sequences."
				, *SequenceLabel.ToString());
		}
	}

	return OutSequences;
}

TArray<UAvaSequence*> AAvaSequencePlaybackActor::GetSequencesByTag(const FAvaTagHandle& InTagHandle, bool bInExactMatch) const
{
	if (!InTagHandle.IsValid())
	{
		UE_LOGF(LogAvaSequencePlayback, Verbose
			, "Input Tag was empty in playback actor %ls"
			, *GetName());
		return TArray<UAvaSequence*>();
	}

	const TArray<UAvaSequence*>& AllSequences = GetAllSequences();
	if (AllSequences.IsEmpty())
	{
		return TArray<UAvaSequence*>();
	}

	// Worst case: All Sequences match the tag
	TArray<UAvaSequence*> OutSequences;
	OutSequences.Reserve(AllSequences.Num());

	for (UAvaSequence* Sequence : AllSequences)
	{
		if (!Sequence)
		{
			continue;
		}

		bool bTagMatches;
		if (bInExactMatch)
		{
			bTagMatches = Sequence->GetSequenceTag().MatchesExact(InTagHandle);
		}
		else
		{
			bTagMatches = Sequence->GetSequenceTag().Overlaps(InTagHandle);
		}

		if (bTagMatches)
		{
			OutSequences.Add(Sequence);
		}
	}

	return OutSequences;
}

void AAvaSequencePlaybackActor::BP_PlayScheduledSequences()
{
	PlayScheduledSequences();
}

UObject* AAvaSequencePlaybackActor::GetPlaybackContext() const
{
	return GetWorld();
}

UObject* AAvaSequencePlaybackActor::CreateDirectorInstance(IMovieScenePlayer& InPlayer, FMovieSceneSequenceID InSequenceID)
{
	return this;
}

UAvaSequencePlayer* AAvaSequencePlaybackActor::GetSequencePlayer(const UAvaSequence* InSequence) const
{
	if (!InSequence)
	{
		return nullptr;
	}

	for (UAvaSequencePlayer* ActivePlayer : ActiveSequencePlayers)
	{
		if (ActivePlayer && ActivePlayer->GetSequence() == InSequence)
		{
			return ActivePlayer;
		}
	}

	return nullptr;
}

TArray<UAvaSequencePlayer*> AAvaSequencePlaybackActor::GetSequencePlayers(TConstArrayView<const UAvaSequence*> InSequences) const
{
	TArray<UAvaSequencePlayer*> SequencePlayers;
	SequencePlayers.Reserve(InSequences.Num());

	for (UAvaSequencePlayer* SequencePlayer : ActiveSequencePlayers)
	{
		if (SequencePlayer && InSequences.Contains(SequencePlayer->GetSequence()))
		{
			SequencePlayers.Add(SequencePlayer);
		}
	}

	return SequencePlayers;
}

TArray<UAvaSequencePlayer*> AAvaSequencePlaybackActor::GetSequencePlayersByLabel(FName InSequenceLabel) const
{
	return GetSequencePlayersByLabels({ InSequenceLabel });
}

TArray<UAvaSequencePlayer*> AAvaSequencePlaybackActor::GetSequencePlayersByTag(const FAvaTagHandle& InTagHandle, bool bInExactMatch) const
{
	return GetSequencePlayers(GetSequencesByTag(InTagHandle, bInExactMatch));
}

TArray<UAvaSequencePlayer*> AAvaSequencePlaybackActor::GetAllSequencePlayers() const
{
	TArray<UAvaSequencePlayer*> SequencePlayers;
	SequencePlayers.Reserve(ActiveSequencePlayers.Num());
	for (UAvaSequencePlayer* SequencePlayer : ActiveSequencePlayers)
	{
		if (SequencePlayer)
		{
			SequencePlayers.Add(SequencePlayer);	
		}
	}
	return SequencePlayers;
}

bool AAvaSequencePlaybackActor::HasActiveSequencePlayers() const
{
	return !ActiveSequencePlayers.IsEmpty();
}

TArray<UAvaSequencePlayer*> AAvaSequencePlaybackActor::GetSequencePlayersByLabels(const TArray<FName>& InSequenceLabels) const
{
	return GetSequencePlayers(GetSequencesByLabel(InSequenceLabels));
}

UAvaSequencePlayer* AAvaSequencePlaybackActor::GetOrAddSequencePlayer(UAvaSequence* InSequence)
{
	if (InSequence && !bStoppingAllSequences)
	{
		RegisterPlaybackObject();

		UAvaSequencePlayer* PlayerToGet = GetSequencePlayer(InSequence);

		if (!PlayerToGet)
		{
			PlayerToGet = NewObject<UAvaSequencePlayer>(this, NAME_None, RF_Transient);
			ActiveSequencePlayers.Add(PlayerToGet);
			PlayerToGet->InitSequence(InSequence, this, GetLevel(), FLevelSequenceCameraSettings());
		}

		return PlayerToGet;
	}
	return nullptr;
}

void AAvaSequencePlaybackActor::RegisterPlaybackObject()
{
	if (UAvaSequenceSubsystem* SequenceSubsystem = UAvaSequenceSubsystem::Get(this))
	{
		SequenceSubsystem->AddPlaybackObject(this);
	}
}

void AAvaSequencePlaybackActor::UnregisterPlaybackObject()
{
	if (UAvaSequenceSubsystem* SequenceSubsystem = UAvaSequenceSubsystem::Get(this))
	{
		SequenceSubsystem->RemovePlaybackObject(this);
	}

	CleanupPlayers();
}

void AAvaSequencePlaybackActor::OnWorldCleanup(UWorld* InWorld, bool bInSessionEnded, bool bInCleanupResources)
{
	if (bInCleanupResources && GetWorld() == InWorld)
	{
		// Ensure the players tear down to unregister themselves from Tick Manager holding them as external references
		CleanupPlayers();
	}
}

void AAvaSequencePlaybackActor::BeginDestroy()
{
	Super::BeginDestroy();

	UAvaSequencePlayer::OnSequenceFinished().Remove(OnSequenceFinishedDelegate);
	OnSequenceFinishedDelegate.Reset();

	FWorldDelegates::OnWorldCleanup.Remove(OnWorldCleanupDelegate);
	OnWorldCleanupDelegate.Reset();
}

void AAvaSequencePlaybackActor::EndPlay(const EEndPlayReason::Type InEndPlayReason)
{
	Super::EndPlay(InEndPlayReason);
	UnregisterPlaybackObject();
}
