// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequencePlaybackObject.h"
#include "AvaSequenceShared.h"
#include "GameFramework/Actor.h"
#include "AvaSequencePlaybackActor.generated.h"

class IAvaSequenceProvider;
class UAvaSequence;
class UAvaSequencePlayer;
class UMovieSceneSequenceTickManager;

/** Base Actor for Ava Sequence Playback Management */
UCLASS(NotPlaceable, Hidden, DisplayName = "Motion Design Sequence Playback Actor")
class AAvaSequencePlaybackActor : public AActor, public IAvaSequencePlaybackObject
{
	GENERATED_BODY()

public:
	AAvaSequencePlaybackActor();

	void SetSequenceProvider(IAvaSequenceProvider& InSequenceProvider);

protected:
	//~ Begin IAvaPlaybackObject
	virtual UObject* ToUObject() override { return this; }
	virtual ULevel* GetPlaybackLevel() const { return GetLevel(); }
	virtual void CleanupPlayers() override;
	virtual UAvaSequencePlayer* PlaySequence(UAvaSequence* InSequence, const FAvaSequencePlayParams& InPlaySettings = FAvaSequencePlayParams()) override;
	virtual UAvaSequencePlayer* PreviewFrame(UAvaSequence* InSequence) override;
	virtual UAvaSequencePlayer* PlaySequenceBySoftReference(TSoftObjectPtr<UAvaSequence> InSequence, const FAvaSequencePlayParams& InPlaySettings) override;
	virtual TArray<UAvaSequencePlayer*> PlaySequencesByLabel(FName InSequenceLabel, const FAvaSequencePlayParams& InPlaySettings) override;
	virtual TArray<UAvaSequencePlayer*> PlaySequencesBySoftReference(const TArray<TSoftObjectPtr<UAvaSequence>>& InSequences, const FAvaSequencePlayParams& InPlaySettings) override;
	virtual TArray<UAvaSequencePlayer*> PlaySequencesByLabels(const TArray<FName>& InSequenceLabels, const FAvaSequencePlayParams& InPlaySettings) override;
	virtual TArray<UAvaSequencePlayer*> PlaySequencesByTag(const FAvaTagHandle& InTagHandle, bool bInExactMatch, const FAvaSequencePlayParams& InPlaySettings) override;
	virtual TArray<UAvaSequencePlayer*> PlayScheduledSequences() override;
	virtual UAvaSequencePlayer* ContinueSequence(UAvaSequence* InSequence) override;
	virtual TArray<UAvaSequencePlayer*> ContinueSequencesByLabel(FName InSequenceLabel) override;
	virtual TArray<UAvaSequencePlayer*> ContinueSequencesByLabels(const TArray<FName>& InSequenceLabels) override;
	virtual TArray<UAvaSequencePlayer*> ContinueSequencesByTag(const FAvaTagHandle& InTagHandle, bool bInExactMatch) override;
	virtual void PauseSequence(UAvaSequence* InSequence) override;
	virtual void PauseSequencesByLabel(FName InSequenceLabel) override;
	virtual void PauseSequencesByLabels(const TArray<FName>& InSequenceLabels) override;
	virtual void PauseSequencesByTag(const FAvaTagHandle& InTagHandle, bool bInExactMatch) override;
	virtual void StopSequence(UAvaSequence* InSequence) override;
	virtual void StopSequencesByLabel(FName InSequenceLabel) override;
	virtual void StopSequencesByLabels(const TArray<FName>& InSequenceLabels) override;
	virtual void StopSequencesByTag(const FAvaTagHandle& InTagHandle, bool bInExactMatch) override;
	virtual UObject* GetPlaybackContext() const override;
	virtual UAvaSequencePlayer* GetSequencePlayer(const UAvaSequence* InSequence) const override;
	virtual TArray<UAvaSequencePlayer*> GetSequencePlayers(TConstArrayView<const UAvaSequence*> InSequences) const override;
	virtual TArray<UAvaSequencePlayer*> GetSequencePlayersByLabel(FName InSequenceLabel) const override;
	virtual TArray<UAvaSequencePlayer*> GetSequencePlayersByLabels(const TArray<FName>& InSequenceLabels) const override;
	virtual TArray<UAvaSequencePlayer*> GetSequencePlayersByTag(const FAvaTagHandle& InTagHandle, bool bInExactMatch) const override;
	virtual TArray<UAvaSequencePlayer*> GetAllSequencePlayers() const override;
	virtual bool HasActiveSequencePlayers() const override;
	virtual void UpdateCameraCut(const UE::MovieScene::FOnCameraCutUpdatedParams& InCameraCutParams) override;
	virtual UObject* CreateDirectorInstance(IMovieScenePlayer& InPlayer, FMovieSceneSequenceID InSequenceID) override;
	virtual FOnCameraCut& GetOnCameraCut() override { return OnCameraCut; }
	//~ End IAvaPlaybackObject

	//~ Begin AActor
	virtual void BeginDestroy() override;
	virtual void EndPlay(const EEndPlayReason::Type InEndPlayReason) override;
#if WITH_EDITOR
	virtual bool IsSelectable() const override { return false; }
	virtual bool SupportsExternalPackaging() const override { return false; }
#endif
	//~ End AActor

	UAvaSequencePlayer* GetOrAddSequencePlayer(UAvaSequence* InSequence);

	/** Registers this Playback Object to the World's Sequence Tick Manager */
	void RegisterPlaybackObject();

	/** Unregisters this Playback Object from the World's Sequence Tick Manager */
	void UnregisterPlaybackObject();

	/** Called when on FWorldDelegates::OnWorldCleanup */
	void OnWorldCleanup(UWorld* InWorld, bool bInSessionEnded, bool bInCleanupResources);

	void OnSequenceFinished(UAvaSequencePlayer* InPlayer, UAvaSequence* InSequence);

private:
	const TArray<UAvaSequence*>& GetAllSequences() const;

	TArray<UAvaSequence*> GetSequencesByLabel(TConstArrayView<FName> InSequenceLabels) const;

	TArray<UAvaSequence*> GetSequencesByTag(const FAvaTagHandle& InTagHandle, bool bInExactMatch) const;

	UFUNCTION(BlueprintCallable, CallInEditor, DisplayName="Play Scheduled Sequences", Category="Scheduled Playback")
	void BP_PlayScheduledSequences();

	/** Name of the Sequences play when PlayScheduledSequence is called, separated by comma */
	UPROPERTY(EditAnywhere, DisplayName="Scheduled Sequences", Category="Scheduled Playback", meta=(AllowPrivateAccess="true"))
	TArray<FName> ScheduledSequenceNames;

	UPROPERTY(EditAnywhere, Category="Scheduled Playback", meta=(AllowPrivateAccess="true"))
	FAvaSequencePlayParams ScheduledPlaySettings;

	/** All the sequence players currently playing */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TSet<TObjectPtr<UAvaSequencePlayer>> ActiveSequencePlayers;

	/** List of sequence players to cache and clean up when safe */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TSet<TObjectPtr<UAvaSequencePlayer>> StoppedSequencePlayers;

	UPROPERTY()
	TScriptInterface<IAvaSequenceProvider> SequenceProvider;

	/** Called when a Camera Cut occurs. */
	FOnCameraCut OnCameraCut;

	FDelegateHandle OnSequenceFinishedDelegate;
	FDelegateHandle OnWorldCleanupDelegate;

	bool bStoppingAllSequences = false;
};
