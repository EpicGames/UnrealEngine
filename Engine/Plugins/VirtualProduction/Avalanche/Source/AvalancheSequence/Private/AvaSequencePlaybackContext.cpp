// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequencePlaybackContext.h"
#include "AvaSequence.h"
#include "AvaSequenceController.h"
#include "AvaSequencePlaybackObject.h"
#include "AvaSequencePlayer.h"
#include "AvaSequencePlayerVariant.h"
#include "IAvaSequenceProvider.h"

#if WITH_EDITOR
#include "ISequencer.h"
#endif

FAvaSequencePlaybackContext::FAvaSequencePlaybackContext(UAvaSequence& InSequence, IAvaSequencePlaybackObject* InPlaybackObject)
	: SequenceWeak(&InSequence)
	, PlaybackObjectWeak(InPlaybackObject)
{
}

void FAvaSequencePlaybackContext::Resolve(const FAvaSequencePlayerVariant& InPlayerVariant, const FFrameTime& InDeltaFrameTime)
{
	Player = InPlayerVariant.Get();
	DeltaFrameTime = InDeltaFrameTime;
	SequencePlayer.Reset();
#if WITH_EDITOR
	EditorSequencer.Reset();
#endif

	LastJumpedFrame = JumpedFrame;
	JumpedFrame.Reset();

	// Find if the Player is an Motion Design Player 
	if (UAvaSequencePlayer* const MotionDesignPlayer = InPlayerVariant.TryGet<UAvaSequencePlayer>())
	{
		SequencePlayer   = MotionDesignPlayer;
		bPlayingForwards = !MotionDesignPlayer->IsReversed();
	}
#if WITH_EDITOR
	// Else it could be the Sequencer Player
	else if (ISequencer* Sequencer = InPlayerVariant.TryGet<ISequencer>())
	{
		EditorSequencer  = Sequencer->AsWeak();
		bPlayingForwards = Sequencer->GetPlaybackSpeed() >= 0.f;
	}

	// one of the two must be valid at this point
	ensure(SequencePlayer.IsValid() || EditorSequencer.IsValid());
#endif

	if (LastJumpedFrame.IsSet() && GetPlaybackStatus() == EMovieScenePlayerStatus::Jumping)
	{
		Continue();
	}

	// Resolve Child Contexts only if we will execute on Children
	if (bShouldRunChildren)
	{
		for (const TWeakPtr<IAvaSequencePlaybackContext>& ChildWeak : ChildrenWeak)
		{
			if (TSharedPtr<IAvaSequencePlaybackContext> Child = ChildWeak.Pin())
			{
				Child->ResolveFromParent(*this);
			}
		}
	}
}

void FAvaSequencePlaybackContext::ResolveFromParent(const IAvaSequencePlaybackContext& InParent)
{
	/** set to true here as this is only executed from parent having run children enabled */
	bShouldRunChildren = true;

	IAvaSequencePlaybackObject* const PlaybackObject = GetPlaybackObject();
	UAvaSequence* const Sequence = GetSequence();

	if (!PlaybackObject || !Sequence)
	{
		return;
	}

	// Currently Editor Sequencer is not Supported
	if (UAvaSequencePlayer* const ActivePlayer = PlaybackObject->GetSequencePlayer(Sequence))
	{
		Resolve(ActivePlayer, InParent.GetDeltaFrameTime());
	}
}

IAvaSequencePlaybackObject* FAvaSequencePlaybackContext::GetPlaybackObject() const
{
	return PlaybackObjectWeak.Get();
}

FFrameTime FAvaSequencePlaybackContext::GetGlobalTime() const
{
#if WITH_EDITOR
	if (TSharedPtr<ISequencer> Sequencer = EditorSequencer.Pin())
	{
		return Sequencer->GetGlobalTime().Time;
	}
#endif
	if (SequencePlayer.IsValid())
	{
		return SequencePlayer->GetGlobalTime().Time;
	}

	return 0;
}

EMovieScenePlayerStatus::Type FAvaSequencePlaybackContext::GetPlaybackStatus() const
{
	if (Player)
	{
		return Player->GetPlaybackStatus();
	}
	return EMovieScenePlayerStatus::MAX;
}

IMovieScenePlayer* FAvaSequencePlaybackContext::GetPlayer() const
{
	return Player;
}

void FAvaSequencePlaybackContext::Pause()
{
#if WITH_EDITOR
	if (TSharedPtr<ISequencer> Sequencer = EditorSequencer.Pin())
	{
		// Note: FSequencer::Pause is not called since it Fires the Stopped Delegate, and sets the Playback Status to Stopped
		Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Paused);
		Sequencer->ForceEvaluate();
	}
	else
#endif
	if (SequencePlayer.IsValid())
	{
		SequencePlayer->Pause();
	}

	RunChildren(&IAvaSequencePlaybackContext::Pause);
}

void FAvaSequencePlaybackContext::Continue()
{
#if WITH_EDITOR
	if (TSharedPtr<ISequencer> Sequencer = EditorSequencer.Pin())
	{
		const float PlaybackSpeed = FMath::Abs(Sequencer->GetPlaybackSpeed());
		Sequencer->SetPlaybackSpeed(bPlayingForwards ? +PlaybackSpeed : -PlaybackSpeed);
		Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Playing);
	}
	else
#endif
	if (SequencePlayer.IsValid())
	{
		SequencePlayer->ContinueSequence();
	}

	//NOTE: There might be the need to execute only the Local Continue (e.g. Possibly when Unpausing after Timed Pause)
	RunChildren(&IAvaSequencePlaybackContext::Continue);
}

void FAvaSequencePlaybackContext::Reverse()
{
#if WITH_EDITOR
	if (TSharedPtr<ISequencer> Sequencer = EditorSequencer.Pin())
	{
		const float PlaybackSpeed = Sequencer->GetPlaybackSpeed();
		Sequencer->SetPlaybackSpeed(PlaybackSpeed * -1.f);
		bPlayingForwards = Sequencer->GetPlaybackSpeed() >= 0.f;
	}
	else
#endif
	if (SequencePlayer.IsValid())
	{
		SequencePlayer->ChangePlaybackDirection();
		bPlayingForwards = !SequencePlayer->IsReversed();
	}
	RunChildren(&IAvaSequencePlaybackContext::Reverse);
}

void FAvaSequencePlaybackContext::JumpTo(const FFrameTime& InFrameTime, bool bInEvaluateJumpedFrames)
{
#if WITH_EDITOR
	if (TSharedPtr<ISequencer> Sequencer = EditorSequencer.Pin())
	{
		Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Jumping);
		Sequencer->SetGlobalTime(InFrameTime, bInEvaluateJumpedFrames);
	}
	else
#endif
	if (SequencePlayer.IsValid())
	{
		SequencePlayer->JumpTo(InFrameTime, bInEvaluateJumpedFrames);
	}

	JumpedFrame = InFrameTime;
	RunChildren(&IAvaSequencePlaybackContext::JumpTo, InFrameTime, bInEvaluateJumpedFrames);
}

void FAvaSequencePlaybackContext::JumpToSelf()
{
	// Evaluate as we're moving from Last Tick Frame to this Marked Frame (and want to evaluate everything in between)
	constexpr bool bEvaluateJumpedFrames = true;

	// Restore Jumped Last Frame as Jump to Self shouldn't be really considered a Jump
	TOptional<FFrameTime> PrevJumpedFrame = JumpedFrame;

	FFrameTime TargetFrame = GetMarkedFrame().FrameNumber;

	// Add a Small Offset to Target Frame when Jumping to Self so as not to evaluate anything that starts directly on that frame
	TargetFrame += bPlayingForwards
		? -FFrameTime(0, static_cast<float>(UE::AvaSequence::SmallSubFrame))
		: FFrameTime(0, static_cast<float>(UE::AvaSequence::SmallSubFrame));

	JumpTo(TargetFrame, bEvaluateJumpedFrames);

	JumpedFrame = PrevJumpedFrame;
}

void FAvaSequencePlaybackContext::UpdateMarkedFrame(const FMovieSceneMarkedFrame& InMarkedFrame)
{
	MarkedFrame = InMarkedFrame;

	const FAvaMark& Mark = GetMark();
	if (Mark.GetLabel().IsEmpty())
	{
		bShouldRunChildren = false;
		return;
	}

	bShouldRunChildren = !Mark.bIsLocalMark;

	if (bPendingChildrenRefresh)
	{
		bPendingChildrenRefresh = false;

		// Gather all Child Contexts even if the Context is Local (i.e. should not propagate actions to children)
		TArray<TSharedRef<IAvaSequencePlaybackContext>> ChildContexts = GatherChildContexts();
		ChildrenWeak.Empty(ChildContexts.Num());
		ChildrenWeak.Append(ChildContexts);
	}
}

TArray<TSharedRef<IAvaSequencePlaybackContext>> FAvaSequencePlaybackContext::GatherChildContexts() const
{
	TArray<TSharedRef<IAvaSequencePlaybackContext>> OutContexts;

	UAvaSequence* const Sequence = GetSequence();
	IAvaSequencePlaybackObject* const PlaybackObject = GetPlaybackObject();

	if (!Sequence || !PlaybackObject)
	{
		return OutContexts;
	}

	const TArray<TWeakObjectPtr<UAvaSequence>>& ChildSequences = Sequence->GetChildren();
	OutContexts.Reserve(ChildSequences.Num());

	for (const TWeakObjectPtr<UAvaSequence>& ChildSequence: ChildSequences)
	{
		UAvaSequencePlayer* const ChildPlayer = PlaybackObject->GetSequencePlayer(ChildSequence.Get());
		if (ChildPlayer && ChildPlayer->GetSequenceController())
		{
			OutContexts.Add(ChildPlayer->GetSequenceController()->GetPlaybackContext());
		}
	}

	return OutContexts;
}

void FAvaSequencePlaybackContext::RequestRefreshChildren()
{
	bPendingChildrenRefresh = true;
}

const TSet<FAvaMark>& FAvaSequencePlaybackContext::GetAllMarks() const
{
	if (UAvaSequence* const Sequence = GetSequence())
	{
		return Sequence->GetMarks();
	}
	static const TSet<FAvaMark> EmptySet;
	return EmptySet;
}

TConstArrayView<TWeakPtr<IAvaSequencePlaybackContext>> FAvaSequencePlaybackContext::GetChildrenInContext() const
{
	if (bShouldRunChildren)
	{
		return ChildrenWeak;
	}
	return {};
}

const FAvaMark& FAvaSequencePlaybackContext::GetMark() const
{
	if (UAvaSequence* const Sequence = GetSequence())
	{
		if (const FAvaMark* const FoundMark = Sequence->GetMarks().Find(MarkedFrame))
		{
			return *FoundMark;
		}
	}
	static const FAvaMark InvalidMark;
	return InvalidMark;
}
