// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Ticker.h"
#include "IAvaSequenceController.h"
#include "IAvaSequencePlaybackContext.h"
#include "Misc/EngineVersionComparison.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "Misc/Optional.h"
#include "MovieScene.h"
#include "MovieSceneFwd.h"
#include "MovieSceneMarkedFrame.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakInterfacePtr.h"
#include "UObject/WeakObjectPtr.h"

class IAvaSequencePlaybackObject;
class IMovieScenePlayer;
class UAvaSequence;
class UAvaSequencePlayer;
struct FAvaMark;

#if WITH_EDITOR
class ISequencer;
#endif

/*
 * PlaybackContext contains and resolves data for a given Sequence and its Player
 * This class resolves the Player type to recognized types (e.g. Sequencer or AvaSequencePlayer) to have more things available
 * to call like PlayPosition, Pause, Continue, etc
 */
class FAvaSequencePlaybackContext : public IAvaSequencePlaybackContext
{
public:
	FAvaSequencePlaybackContext(UAvaSequence& InSequence, IAvaSequencePlaybackObject* InPlaybackObject);

	virtual ~FAvaSequencePlaybackContext() override = default;

	void Resolve(const FAvaSequencePlayerVariant& InPlayerVariant, const FFrameTime& InDeltaFrameTime);

	UAvaSequence* GetSequence() const { return SequenceWeak.Get(); }

	IAvaSequencePlaybackObject* GetPlaybackObject() const;

	EMovieScenePlayerStatus::Type GetPlaybackStatus() const;
	
	IMovieScenePlayer* GetPlayer() const;

	/** Returns Last Tick's Jumped Frame, if it did jump. Else it returns it unset */
	TOptional<FFrameTime> GetLastJumpedFrame() const { return LastJumpedFrame; }

	void UpdateMarkedFrame(const FMovieSceneMarkedFrame& InMarkedFrame);

	TArray<TSharedRef<IAvaSequencePlaybackContext>> GatherChildContexts() const;

	void RequestRefreshChildren();

	//~ Begin IAvaSequencePlaybackContext
	FFrameTime GetGlobalTime() const;
	FFrameTime GetDeltaFrameTime() const { return DeltaFrameTime; }
	virtual bool IsPlayingForwards() const override { return bPlayingForwards; }
	virtual void Pause() override;
	virtual void Continue() override;
	virtual void Reverse() override;
	virtual void JumpTo(const FFrameTime& InFrameTime, bool bInEvaluateJumpedFrames) override;
	virtual void JumpToSelf() override;
	virtual const FAvaMark& GetMark() const override;
	virtual const TSet<FAvaMark>& GetAllMarks() const override;
	virtual const FMovieSceneMarkedFrame& GetMarkedFrame() const override { return MarkedFrame; }
	virtual FTSTicker& GetTicker() override { return Ticker; }
	virtual TConstArrayView<TWeakPtr<IAvaSequencePlaybackContext>> GetChildrenInContext() const override;
	virtual void ResolveFromParent(const IAvaSequencePlaybackContext& InParent) override;
	//~ End IAvaSequencePlaybackContext

private:
	/** Iteratively Execute on all Children and if set also their Children */
	template<typename Func, typename...Params>
	void RunChildren(Func&& InFunc, Params&&... InParams)
	{
		if (bShouldRunChildren)
		{
			TArray<TWeakPtr<IAvaSequencePlaybackContext>> ContextsToRun = ChildrenWeak;

			while (!ContextsToRun.IsEmpty())
			{
				if (TSharedPtr<IAvaSequencePlaybackContext> Child = ContextsToRun.Pop().Pin())
				{
					(*Child.*InFunc)(Forward<Params>(InParams)...);
					ContextsToRun.Append(Child->GetChildrenInContext());
				}
			}
		}
	}

	TWeakObjectPtr<UAvaSequence> SequenceWeak;

	TWeakInterfacePtr<IAvaSequencePlaybackObject> PlaybackObjectWeak;

	TWeakObjectPtr<UAvaSequencePlayer> SequencePlayer;

	TArray<TWeakPtr<IAvaSequencePlaybackContext>> ChildrenWeak;

	/** Marked Frame Reached */
	FMovieSceneMarkedFrame MarkedFrame;

#if WITH_EDITOR
	TWeakPtr<ISequencer> EditorSequencer;
#endif

	FFrameNumber LastFrame;

	IMovieScenePlayer* Player = nullptr;

	FFrameTime DeltaFrameTime;

	TOptional<FFrameTime> LastJumpedFrame;

	TOptional<FFrameTime> JumpedFrame;

	FTSTicker Ticker;

	bool bPlayingForwards        = true;
	bool bShouldRunChildren      = false;
	bool bPendingChildrenRefresh = true;
};
