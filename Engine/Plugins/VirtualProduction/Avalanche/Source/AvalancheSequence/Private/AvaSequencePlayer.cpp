// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequencePlayer.h"
#include "AvaSequence.h"
#include "AvaSequencePlaybackObject.h"
#include "AvaSequencePlayerVariant.h"
#include "AvaSequenceShared.h"
#include "AvaSequenceSubsystem.h"
#include "Engine/Level.h"
#include "IAvaSequenceController.h"
#include "IAvaSequenceProvider.h"

UAvaSequencePlayer::FOnSequenceEvent UAvaSequencePlayer::OnSequenceStartedDelegate;
UAvaSequencePlayer::FOnSequenceEvent UAvaSequencePlayer::OnSequencePausedDelegate;
UAvaSequencePlayer::FOnSequenceEvent UAvaSequencePlayer::OnSequenceFinishedDelegate;

UAvaSequencePlayer::UAvaSequencePlayer(const FObjectInitializer& InObjectInitializer)
	: ULevelSequencePlayer(InObjectInitializer)
{
	if (!IsTemplate())
	{
		OnNativeFinished.BindUObject(this, &UAvaSequencePlayer::NotifySequenceFinished);
		// Remark: UMovieSceneSequencePlayer has a virtual OnPaused() function, but it is not called. Using event instead.
		OnPause.AddDynamic(this, &UAvaSequencePlayer::NotifySequencePaused);
	}
}

void UAvaSequencePlayer::InitSequence(UAvaSequence* InSequence, IAvaSequencePlaybackObject* InPlaybackObject, ULevel* InLevel, const FLevelSequenceCameraSettings& InCameraSettings)
{
	check(InSequence);
	Super::Initialize(InSequence, InLevel, InCameraSettings);
	SequenceController = UAvaSequenceSubsystem::CreateSequenceController(*InSequence, InPlaybackObject);
	PlaybackObjectWeak = InPlaybackObject;
	PlaybackLevelWeak  = InLevel;
}

UAvaSequence* UAvaSequencePlayer::GetAvaSequence() const
{
	return Cast<UAvaSequence>(Sequence);
}

IAvaSequencePlaybackObject* UAvaSequencePlayer::GetPlaybackObject() const
{
	return PlaybackObjectWeak.Get();
}

FQualifiedFrameTime UAvaSequencePlayer::GetGlobalTime() const
{
	FFrameTime RootTime = ConvertFrameTime(PlayPosition.GetCurrentPosition(), PlayPosition.GetInputRate(), PlayPosition.GetOutputRate());
	return FQualifiedFrameTime(RootTime, PlayPosition.GetOutputRate());
}

void UAvaSequencePlayer::SetPlaySettings(const FAvaSequencePlayParams& InPlaySettings)
{
	UAvaSequence* PlaybackSequence = Cast<UAvaSequence>(Sequence);
	if (!PlaybackSequence)
	{
		return;
	}

	UMovieScene* MovieScene = PlaybackSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	bReversePlayback = InPlaySettings.PlayMode == EAvaSequencePlayMode::Reverse;

	const double TotalDuration    = MovieScene->GetTickResolution().AsSeconds(MovieScene->GetPlaybackRange().Size<FFrameNumber>());
	double StartTimeSeconds = InPlaySettings.Start.ToSeconds(*PlaybackSequence, *MovieScene, 0.0);
	double EndTimeSeconds   = InPlaySettings.End.ToSeconds(*PlaybackSequence, *MovieScene, TotalDuration);

	// Ensure that StartTime <= EndTime
	StartTimeSeconds = FMath::Min(StartTimeSeconds, EndTimeSeconds);
	EndTimeSeconds   = FMath::Max(StartTimeSeconds, EndTimeSeconds);

	double DurationSeconds = FMath::Abs(EndTimeSeconds - StartTimeSeconds);

	// Ensure that Duration Seconds is at least 1 Frame Long, as this gets Rounded to the nearest Frame and if it results in 0, the Sequence does not get evaluated
	DurationSeconds = FMath::Max(DurationSeconds, PlayPosition.GetInputRate().AsSeconds(FFrameNumber(1)));

	if (bReversePlayback)
	{
		StartTimeSeconds = EndTimeSeconds - DurationSeconds;
	}

	SetTimeRange(StartTimeSeconds, DurationSeconds);

	PlaybackSettings = FMovieSceneSequencePlaybackSettings();
	PlaybackSettings.PlayRate = InPlaySettings.AdvancedSettings.PlaybackSpeed;
	PlaybackSettings.LoopCount.Value = InPlaySettings.AdvancedSettings.LoopCount;
	PlaybackSettings.FinishCompletionStateOverride = InPlaySettings.AdvancedSettings.bRestoreState
		? EMovieSceneCompletionModeOverride::ForceRestoreState
		: EMovieSceneCompletionModeOverride::ForceKeepState;
}

void UAvaSequencePlayer::PlaySequence()
{
	if (NeedsQueueLatentAction())
	{
		QueueLatentAction(FMovieSceneSequenceLatentActionDelegate::CreateUObject(this, &UAvaSequencePlayer::PlaySequence));
		return;
	}

	PlayPosition.Reset(StartTime);
	PlayInternal();
}

void UAvaSequencePlayer::ContinueSequence()
{
	if (NeedsQueueLatentAction())
	{
		QueueLatentAction(FMovieSceneSequenceLatentActionDelegate::CreateUObject(this, &UAvaSequencePlayer::ContinueSequence));
		return;
	}

	PlayInternal();
}

void UAvaSequencePlayer::JumpTo(FFrameTime InJumpToFrame, bool bInEvaluate)
{
	if (NeedsQueueLatentAction())
	{
		QueueLatentAction(FMovieSceneSequenceLatentActionDelegate::CreateUObject(this, &UAvaSequencePlayer::JumpTo, InJumpToFrame, bInEvaluate));
		return;
	}

	InJumpToFrame = ConvertFrameTime(InJumpToFrame, GetTickResolution(), GetDisplayRate());

	if (bInEvaluate)
	{
		constexpr bool bHasJumped = true;
		UpdateTimeCursorPosition(InJumpToFrame, EUpdatePositionMethod::Jump, bHasJumped);
	}

	PlayPosition.JumpTo(InJumpToFrame);
	TimeController->Reset(GetCurrentTime());
}

FFrameRate UAvaSequencePlayer::GetTickResolution() const
{
	return Sequence && Sequence->GetMovieScene() ? Sequence->GetMovieScene()->GetTickResolution() : FFrameRate();
}

void UAvaSequencePlayer::Cleanup()
{
	if (ensureAlwaysMsgf(!IsEvaluating(), TEXT("Calling UAvaSequencePlayer::Cleanup while still evaluating is not allowed!")))
	{
		Stop();
		TearDown();
	}
}

void UAvaSequencePlayer::OnStartedPlaying()
{
	Super::OnStartedPlaying();
	NotifySequenceStarted();
}

void UAvaSequencePlayer::OnStopped()
{
	Super::OnStopped();

	// At the moment, Stop means to completely finish
	NotifySequenceFinished();

	// Defer cleanup as there is an action flush that assumes the Tick Manager is still alive after this Stop callback
	// see UMovieSceneSequencePlayer::RunLatentActions
	QueueLatentAction(FMovieSceneSequenceLatentActionDelegate::CreateUObject(this, &UAvaSequencePlayer::Cleanup));
}

void UAvaSequencePlayer::TickFromSequenceTickManager(float InDeltaSeconds, FMovieSceneEntitySystemRunner* InRunner)
{
	if (ensure(SequenceController.IsValid()))
	{
		const FFrameTime DeltaFrameTime = CalculateDeltaFrameTime(InDeltaSeconds);
		SequenceController->Tick(FAvaSequencePlayerVariant(this), DeltaFrameTime, InDeltaSeconds);
	}
	Super::TickFromSequenceTickManager(InDeltaSeconds, InRunner);
}

void UAvaSequencePlayer::OnCameraCutUpdated(const UE::MovieScene::FOnCameraCutUpdatedParams& InCameraCutParams)
{
	Super::OnCameraCutUpdated(InCameraCutParams);

	if (IAvaSequencePlaybackObject* const PlaybackObject = GetPlaybackObject())
	{
		PlaybackObject->UpdateCameraCut(InCameraCutParams);
	}
}

FFrameTime UAvaSequencePlayer::CalculateDeltaFrameTime(float InDeltaSeconds) const
{
	if (IsReversed())
	{
		InDeltaSeconds *= -1.0;
	}
	return InDeltaSeconds * GetPlayRate() * GetTickResolution();
}

void UAvaSequencePlayer::NotifySequenceStarted()
{
	OnSequenceStartedDelegate.Broadcast(this, GetAvaSequence());
}

void UAvaSequencePlayer::NotifySequencePaused()
{
	OnSequencePausedDelegate.Broadcast(this, GetAvaSequence());
}

void UAvaSequencePlayer::NotifySequenceFinished()
{
	OnSequenceFinishedDelegate.Broadcast(this, GetAvaSequence());
}
