// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceShared.h"
#include "AvaSequence.h"
#include "AvaSequenceVersion.h"

bool FAvaSequenceTime::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FAvaSequenceVersion::GUID);

	UScriptStruct* ScriptStruct = FAvaSequenceTime::StaticStruct();
	ScriptStruct->SerializeTaggedProperties(Ar, reinterpret_cast<uint8*>(this), ScriptStruct, nullptr);

	if (Ar.IsLoading() && Ar.CustomVer(FAvaSequenceVersion::GUID) < FAvaSequenceVersion::SequenceTimeConstraintOption)
	{
		switch (TimeType)
		{
		case EAvaSequenceTimeType::Frame:
			bHasTimeConstraint = Frame >= 0;
			break;

		case EAvaSequenceTimeType::Seconds:
			bHasTimeConstraint = Seconds >= 0.0;
			break;

		case EAvaSequenceTimeType::Mark:
			bHasTimeConstraint = !MarkLabel.IsEmpty();
			break;
		}
	}

	return true;
}

double FAvaSequenceTime::ToSeconds(const UAvaSequence& InSequence, const UMovieScene& InMovieScene, double InDefaultTime) const
{
	if (!bHasTimeConstraint)
	{
		return InDefaultTime;
	}

	switch (TimeType)
	{
	case EAvaSequenceTimeType::Frame:
		return InMovieScene.GetDisplayRate().AsSeconds(FFrameTime(Frame, SubFrame));

	case EAvaSequenceTimeType::Seconds:
		return Seconds;

	case EAvaSequenceTimeType::Mark:
		{
			FAvaMark Mark;
			if (InSequence.GetMark(MarkLabel, Mark) && !Mark.Frames.IsEmpty())
			{
				return InMovieScene.GetTickResolution().AsSeconds(Mark.Frames[0]);
			}
		}
	}

	return InDefaultTime;
}
