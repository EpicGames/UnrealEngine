// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceSubsystem.h"
#include "AvaSequenceController.h"
#include "AvaSequencePlaybackActor.h"
#include "AvaSequencePlaybackObject.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "IAvaSequenceProvider.h"
#include "UObject/UObjectThreadContext.h"

namespace UE::AvaSequence::Private
{
	static const TSet<EWorldType::Type> GUnsupportedWorlds
		{
			EWorldType::Type::None,
			EWorldType::Type::Inactive,
			EWorldType::Type::EditorPreview,
		};
}

UAvaSequenceSubsystem* UAvaSequenceSubsystem::Get(UObject* InPlaybackContext)
{
	if (!IsValid(InPlaybackContext))
	{
		return nullptr;
	}

	UWorld* const World = InPlaybackContext->GetWorld();
	if (!IsValid(World))
	{
		return nullptr;
	}

	return World->GetSubsystem<UAvaSequenceSubsystem>();
}

TSharedRef<IAvaSequenceController> UAvaSequenceSubsystem::CreateSequenceController(UAvaSequence& InSequence, IAvaSequencePlaybackObject* InPlaybackObject)
{
	return MakeShared<FAvaSequenceController>(InSequence, InPlaybackObject);
}

IAvaSequencePlaybackObject* UAvaSequenceSubsystem::FindOrCreatePlaybackObject(ULevel* InLevel, IAvaSequenceProvider& InSequenceProvider)
{
	if (!EnsureLevelIsAppropriate(InLevel))
	{
		return nullptr;
	}

	if (IAvaSequencePlaybackObject* ExistingPlaybackObject = FindPlaybackObject(InLevel))
	{
		// Update the Existing Playback to this Sequence Provider 
		if (AAvaSequencePlaybackActor* ExistingPlaybackActor = Cast<AAvaSequencePlaybackActor>(ExistingPlaybackObject))
		{
			ExistingPlaybackActor->SetSequenceProvider(InSequenceProvider);
		}

		return ExistingPlaybackObject;
	}

	// If no existing scene playback actor found, then spawn a new one
	FActorSpawnParameters SpawnParameters;
	SpawnParameters.OverrideLevel = InLevel;
#if WITH_EDITOR
	SpawnParameters.bHideFromSceneOutliner = true;
#endif

	AAvaSequencePlaybackActor* PlaybackActor = InLevel->OwningWorld->SpawnActor<AAvaSequencePlaybackActor>(SpawnParameters);
	check(PlaybackActor);

	PlaybackActor->SetSequenceProvider(InSequenceProvider);
	return PlaybackActor;
}

IAvaSequencePlaybackObject* UAvaSequenceSubsystem::FindPlaybackObject(ULevel* InLevel) const
{
	if (!EnsureLevelIsAppropriate(InLevel))
	{
		return nullptr;
	}

	// find existing playback actor in level
	AAvaSequencePlaybackActor* PlaybackActor;
	if (InLevel->Actors.FindItemByClass(&PlaybackActor))
	{
		return PlaybackActor;
	}

	for (const TWeakInterfacePtr<IAvaSequencePlaybackObject>& PlaybackObjectWeak : PlaybackObjects)
	{
		IAvaSequencePlaybackObject* PlaybackObject = PlaybackObjectWeak.Get();
		if (PlaybackObject && PlaybackObject->GetPlaybackLevel() == InLevel)
		{
			return PlaybackObject;
		}
	}

	return nullptr;
}

void UAvaSequenceSubsystem::AddPlaybackObject(IAvaSequencePlaybackObject* InPlaybackObject)
{
	PlaybackObjects.AddUnique(InPlaybackObject);
}

void UAvaSequenceSubsystem::RemovePlaybackObject(IAvaSequencePlaybackObject* InPlaybackObject)
{
	PlaybackObjects.Remove(InPlaybackObject);
}

IAvaSequenceProvider* UAvaSequenceSubsystem::FindSequenceProvider(const ULevel* InLevel) const
{
	if (const TWeakInterfacePtr<IAvaSequenceProvider>* SequenceProvider = SequenceProviders.Find(InLevel))
	{
		return SequenceProvider->Get();
	}
	return nullptr;
}

void UAvaSequenceSubsystem::RegisterSequenceProvider(const ULevel* InLevel, IAvaSequenceProvider* InSequenceProvider)
{
	SequenceProviders.FindOrAdd(InLevel) = InSequenceProvider;
}

bool UAvaSequenceSubsystem::UnregisterSequenceProvider(const ULevel* InLevel, IAvaSequenceProvider* InSequenceProvider)
{
	if (TWeakInterfacePtr<IAvaSequenceProvider>* SequenceProvider = SequenceProviders.Find(InLevel))
	{
		if (*SequenceProvider == InSequenceProvider)
		{
			SequenceProviders.Remove(InLevel);
			return true;
		}
	}
	return false;
}

bool UAvaSequenceSubsystem::DoesSupportWorldType(const EWorldType::Type InWorldType) const
{
	const bool bDisallowedWorld = UE::AvaSequence::Private::GUnsupportedWorlds.Contains(InWorldType);
	return !bDisallowedWorld;
}

bool UAvaSequenceSubsystem::EnsureLevelIsAppropriate(ULevel*& InLevel) const
{
	UWorld* const World = GetWorld();
	if (!World)
	{
		return false;
	}

	// if passed in nullptr, set it to the persistent level
	if (!InLevel)
	{
		InLevel = World->PersistentLevel;
	}

	// Ensure the provided level belongs to the world of this subsystem
	return InLevel && ensureAlways(InLevel->OwningWorld == World);
}
