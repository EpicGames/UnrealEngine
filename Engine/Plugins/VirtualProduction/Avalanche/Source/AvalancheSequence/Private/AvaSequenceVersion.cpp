// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FAvaSequenceVersion::GUID(0xD3B4C40C, 0xD15A4DA1, 0xADF25940, 0xB2C65167);

FCustomVersionRegistration GRegisterAvaSequenceVersion(FAvaSequenceVersion::GUID
	, FAvaSequenceVersion::LatestVersion
	, TEXT("AvalancheSequenceVersion"));
