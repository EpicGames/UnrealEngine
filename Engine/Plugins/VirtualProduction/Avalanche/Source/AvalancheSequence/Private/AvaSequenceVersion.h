// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"

struct FAvaSequenceVersion
{
private:
	FAvaSequenceVersion() = delete;

public:
	enum Type : uint8
	{
		PreVersioning = 0,

		/** Moved to become child class of ULevelSequence */
		LevelSequence, 

		/** Introduced Time Constraint Option to FAvaSequenceTime */
		SequenceTimeConstraintOption,

		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	const static FGuid GUID;
};
