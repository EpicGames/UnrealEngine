// Copyright Epic Games, Inc. All Rights Reserved.

#include "Director/AvaSequenceDirector.h"
#include "AvaSequence.h"
#include "AvaSequencePlaybackActor.h"
#include "AvaSequencePlayer.h"
#include "AvaSequenceSubsystem.h"
#include "Director/AvaSequenceDirectorGeneratedClass.h"

TScriptInterface<IAvaSequencePlaybackObject> UAvaSequenceDirector::GetPlaybackObject() const
{
	if (PlaybackObjectInterfaceWeak.IsValid())
	{
		return PlaybackObjectInterfaceWeak.ToScriptInterface();
	}

	// If Playback Object is null, try to update from SequencePlayer PlaybackObject
	const_cast<UAvaSequenceDirector*>(this)->UpdatePlaybackObject();
	return PlaybackObjectInterfaceWeak.ToScriptInterface();
}

void UAvaSequenceDirector::UpdatePlaybackObject()
{
	UAvaSequencePlayer* SequencePlayer = SequencePlayerWeak.Get();
	if (!SequencePlayer)
	{
		return;
	}

	if (IAvaSequencePlaybackObject* PlaybackObjectInterface = SequencePlayer->GetPlaybackObject())
	{
		PlaybackObjectInterfaceWeak = PlaybackObjectInterface;
	}
}

void UAvaSequenceDirector::UpdateProperties()
{
	if (UAvaSequenceDirectorGeneratedClass* GeneratedClass = Cast<UAvaSequenceDirectorGeneratedClass>(GetClass()))
	{
		GeneratedClass->UpdateProperties(this);
	}
}

void UAvaSequenceDirector::PostLoad()
{
	Super::PostLoad();
	UpdateProperties();
}

void UAvaSequenceDirector::PostDuplicate(EDuplicateMode::Type InDuplicateMode)
{
	Super::PostDuplicate(InDuplicateMode);
	UpdateProperties();
}

void UAvaSequenceDirector::Initialize(IMovieScenePlayer& InPlayer, IAvaSequenceProvider* InSequenceProvider)
{
	UAvaSequencePlayer* SequencePlayer = Cast<UAvaSequencePlayer>(InPlayer.AsUObject());
	SequencePlayerWeak = SequencePlayer;

	IAvaSequencePlaybackObject* NewPlaybackObject = nullptr;

	if (SequencePlayer)
	{
		NewPlaybackObject = SequencePlayer->GetPlaybackObject();
	}
	else if (InSequenceProvider)
	{
		if (UAvaSequenceSubsystem* SequenceSubsystem = UAvaSequenceSubsystem::Get(InPlayer.GetPlaybackContext()))
		{
			NewPlaybackObject = SequenceSubsystem->FindOrCreatePlaybackObject(nullptr, *InSequenceProvider);
		}
	}

	if (NewPlaybackObject)
	{
		PlaybackObjectInterfaceWeak = NewPlaybackObject;
	}
	else
	{
		PlaybackObjectInterfaceWeak = nullptr;
	}

	UpdateProperties();
}

void UAvaSequenceDirector::PlaySequencesByLabel(FName InSequenceLabel, FAvaSequencePlayParams InPlaySettings)
{
	if (IAvaSequencePlaybackObject* PlaybackObjectInterface = GetPlaybackObject().GetInterface())
	{
		PlaybackObjectInterface->PlaySequencesByLabel(InSequenceLabel, MoveTemp(InPlaySettings));
	}
}

void UAvaSequenceDirector::PlayScheduledSequences()
{
	if (IAvaSequencePlaybackObject* PlaybackObjectInterface = GetPlaybackObject().GetInterface())
	{
		PlaybackObjectInterface->PlayScheduledSequences();
	}
}
