// Copyright Epic Games, Inc. All Rights Reserved.

#include "Director/AvaSequenceDirectorBlueprint.h"
#include "Director/AvaSequenceDirectorGeneratedClass.h"
#include "IAvaSequenceProvider.h"

TConstArrayView<FAvaSequenceInfo> UAvaSequenceDirectorBlueprint::GetSequenceInfos()
{
	UpdateSequenceInfos();
	return SequenceInfos;
}

#if WITH_EDITOR
UClass* UAvaSequenceDirectorBlueprint::GetBlueprintClass() const
{
	return UAvaSequenceDirectorGeneratedClass::StaticClass();
}

void UAvaSequenceDirectorBlueprint::GetReparentingRules(TSet<const UClass*>& OutAllowedChildrenOfClasses, TSet<const UClass*>& OutDisallowedChildrenOfClasses) const
{
	OutAllowedChildrenOfClasses.Add(UAvaSequenceDirectorBlueprint::StaticClass());
}
#endif

void UAvaSequenceDirectorBlueprint::UpdateSequenceInfos()
{
	UAvaSequence* OwningSequence = GetTypedOuter<UAvaSequence>();
	if (!OwningSequence)
	{
		return;
	}

	IAvaSequenceProvider* SequenceProvider = OwningSequence->GetSequenceProvider();
	if (!SequenceProvider)
	{
		return;
	}

	// Skip if it's not loaded yet
	UObject* SequenceProviderObject = SequenceProvider->ToUObject();
	if (SequenceProviderObject && !SequenceProviderObject->HasAllFlags(RF_LoadCompleted))
	{
		return;
	}

	const TArray<TObjectPtr<UAvaSequence>>& Sequences = SequenceProvider->GetSequences();
	SequenceInfos.Empty(Sequences.Num());

	for (UAvaSequence* Sequence : ReverseIterate(Sequences))
	{
		if (!IsValid(Sequence))
		{
			continue;
		}

		FAvaSequenceInfo Info;
		Info.SequenceName = Sequence->GetFName();
		Info.Sequence     = Sequence;
		SequenceInfos.Add(MoveTemp(Info));
	}
}
