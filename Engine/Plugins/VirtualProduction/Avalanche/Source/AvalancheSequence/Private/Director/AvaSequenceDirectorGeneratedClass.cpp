// Copyright Epic Games, Inc. All Rights Reserved.

#include "Director/AvaSequenceDirectorGeneratedClass.h"
#include "AvaSequence.h"
#include "Director/AvaSequenceDirector.h"

void UAvaSequenceDirectorGeneratedClass::UpdateProperties(UAvaSequenceDirector* InDirector)
{
	if (!InDirector)
	{
		return;
	}

	TMap<FName, FSoftObjectProperty*> PropertyMap;

	for (FSoftObjectProperty* Property : TFieldRange<FSoftObjectProperty>(this))
	{
		check(Property);

		ensureMsgf(!PropertyMap.Contains(Property->GetFName())
			, TEXT("There are properties with the same name: '%s'"), *Property->GetName());

		PropertyMap.Add(Property->GetFName(), Property);
	}

	for (const FAvaSequenceInfo& Info : SequenceInfos)
	{
		// Find property with the same name as the sequence and assign the sequence to it.
		if (FSoftObjectProperty* const* Property = PropertyMap.Find(Info.SequenceName))
		{
			check(*Property);
			(*Property)->SetObjectPropertyValue_InContainer(InDirector, Info.Sequence.Get());
		}
	}
}
