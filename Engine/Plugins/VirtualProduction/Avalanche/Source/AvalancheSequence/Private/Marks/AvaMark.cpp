// Copyright Epic Games, Inc. All Rights Reserved.

#include "Marks/AvaMark.h"
#include "MovieScene.h"

FAvaMark::FAvaMark(const FMovieSceneMarkedFrame& InMarkedFrame)
	: Label(InMarkedFrame.Label)
	, Frames({InMarkedFrame.FrameNumber.Value})
{
}

FAvaMark::FAvaMark(const FString& InLabel)
	: Label(InLabel)
{
}

FAvaMark::FAvaMark(const FStringView& InLabel)
	: Label(InLabel)
{
}

void FAvaMark::CopyFromMark(const FAvaMark& InOther)
{
	FString OriginalLabel = MoveTemp(Label);
	TArray<int32> OriginalFrames = MoveTemp(Frames);

	*this = InOther;

	Label  = MoveTemp(OriginalLabel);
	Frames = MoveTemp(OriginalFrames);
}
