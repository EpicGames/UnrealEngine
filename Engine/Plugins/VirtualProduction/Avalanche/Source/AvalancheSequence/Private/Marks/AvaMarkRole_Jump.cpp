// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMarkRole_Jump.h"
#include "Math/NumericLimits.h"
#include "AvaSequence.h"

EAvaMarkRoleReply FAvaMarkRole_Jump::Execute()
{
	const TSet<FAvaMark> Marks = Context->GetAllMarks();
	const FAvaMark& SourceMark = Context->GetMark();

	const FAvaMark* const Mark = Marks.Find(SourceMark.JumpLabel);

	if (!Mark)
	{
		return EAvaMarkRoleReply::NotExecuted;
	}

	int32 NextFrameIndex = INDEX_NONE;
	int32 MinDeltaFrame = TNumericLimits<int32>::Max();

	EAvaMarkDirection CheckDirection = EAvaMarkDirection::Both;
	{
		switch (SourceMark.SearchDirection)
		{
		case EAvaMarkSearchDirection::All:
			CheckDirection = EAvaMarkDirection::Both;
			break;

		case EAvaMarkSearchDirection::SameDirection:
			CheckDirection = Context->IsPlayingForwards()
				? EAvaMarkDirection::Forwards
				: EAvaMarkDirection::Backwards;
			break;

		case EAvaMarkSearchDirection::OppositeDirection:
			CheckDirection = Context->IsPlayingForwards()
				? EAvaMarkDirection::Backwards
				: EAvaMarkDirection::Forwards;
			break;

		case EAvaMarkSearchDirection::AbsoluteForwards:
			CheckDirection = EAvaMarkDirection::Forwards;
			break;

		case EAvaMarkSearchDirection::AbsoluteBackwards:
			CheckDirection = EAvaMarkDirection::Backwards;
			break;
		}
	}

	const TArray<int32>& Frames = Mark->Frames;

	// Figure out the nearest Mark from current position with the given Label
	for (int32 Index = 0; Index < Frames.Num(); ++Index)
	{
		int32 DeltaFrame = Frames[Index] - Context->GetMarkedFrame().FrameNumber.Value;

		if (CheckDirection == EAvaMarkDirection::Forwards && DeltaFrame < 0)
		{
			continue;
		}

		if (CheckDirection == EAvaMarkDirection::Backwards && DeltaFrame > 0)
		{
			continue;
		}

		DeltaFrame = FMath::Abs(DeltaFrame);

		// Only consider the nearest marked frames that is not self
		if (DeltaFrame != 0 &&  DeltaFrame < MinDeltaFrame)
		{
			MinDeltaFrame = DeltaFrame;
			NextFrameIndex = Index;
		}
	}

	if (NextFrameIndex != INDEX_NONE)
	{
		constexpr bool bEvaluateJumpedFrames = false;
		Context->JumpToSelf();
		Context->JumpTo(Frames[NextFrameIndex], bEvaluateJumpedFrames);
		return EAvaMarkRoleReply::Executed;
	}

	return EAvaMarkRoleReply::NotExecuted;
}
