// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMarkRole_Pause.h"

FAvaMarkRole_Pause::~FAvaMarkRole_Pause()
{
	FAvaMarkRole_Pause::Reset();
}

EAvaMarkRoleReply FAvaMarkRole_Pause::Execute()
{
	const FAvaMark& Mark = Context->GetMark();
	if (Mark.PauseTime > 0.f)
	{
		Reset();

		Context->JumpToSelf();
		Context->Pause();

		TickerHandle = Context->GetTicker().AddTicker(FTickerDelegate::CreateSP(this, &FAvaMarkRole_Pause::Unpause)
			, Mark.PauseTime);

		PauseStartFrame = Context->GetGlobalTime().FrameNumber;

		return EAvaMarkRoleReply::Executed;
	}

	return EAvaMarkRoleReply::NotExecuted;
}

void FAvaMarkRole_Pause::Reset()
{
	if (TickerHandle.IsValid() && Context.IsValid())
	{
		Context->GetTicker().RemoveTicker(TickerHandle);
		TickerHandle.Reset();
	}
}

bool FAvaMarkRole_Pause::Unpause(float DeltaTime)
{
	if (Context->GetGlobalTime().FrameNumber == PauseStartFrame)
	{
		Context->Continue();
	}
	Reset();
	return true;
}
