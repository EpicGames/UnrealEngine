// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Marks/AvaMarkRole.h"
#include "Misc/FrameNumber.h"

class FAvaMarkRole_Pause : public FAvaMarkRole
{
public:
	virtual ~FAvaMarkRole_Pause() override;

	//~ Begin FAvaMarkRole
	virtual EAvaMarkRole GetRole() const override { return EAvaMarkRole::Pause; }
	virtual EAvaMarkRoleReply Execute() override;
	virtual void Reset() override;
	//~ End FAvaMarkRole

protected:
	bool Unpause(float DeltaTime);

	FTSTicker::FDelegateHandle TickerHandle;

	FFrameNumber PauseStartFrame;
};
