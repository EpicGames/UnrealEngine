// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMarkRole_Reverse.h"

EAvaMarkRoleReply FAvaMarkRole_Reverse::Execute()
{
	Context->JumpToSelf();
	Context->Reverse();
	Context->Continue();
	return EAvaMarkRoleReply::Executed;
}
