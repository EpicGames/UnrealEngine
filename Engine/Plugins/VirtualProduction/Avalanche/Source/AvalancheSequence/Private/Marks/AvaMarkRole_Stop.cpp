// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMarkRole_Stop.h"
#include "AvaSequence.h"

EAvaMarkRoleReply FAvaMarkRole_Stop::Execute()
{
	Context->JumpToSelf();
	Context->Pause();
	return EAvaMarkRoleReply::Executed;
}
