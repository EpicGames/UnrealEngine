// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Marks/AvaMarkRole.h"

class FAvaMarkRole_Stop : public FAvaMarkRole
{
public:
	virtual ~FAvaMarkRole_Stop() override = default;

	//~ Begin FAvaMarkRole
	virtual EAvaMarkRole GetRole() const override { return EAvaMarkRole::Stop; }
	virtual EAvaMarkRoleReply Execute() override;
	//~ End FAvaMarkRole
};
