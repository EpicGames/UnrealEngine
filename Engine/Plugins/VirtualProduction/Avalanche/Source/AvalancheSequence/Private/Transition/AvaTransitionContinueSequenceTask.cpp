// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transition/AvaTransitionContinueSequenceTask.h"
#include "AvaSequencePlaybackObject.h"
#include "StateTreeExecutionContext.h"

#define LOCTEXT_NAMESPACE "AvaTransitionContinueSequenceTask"

#if WITH_EDITOR
FText FAvaTransitionContinueSequenceTask::GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const
{
	const FInstanceDataType& InstanceData = InInstanceDataView.Get<FInstanceDataType>();

	TArray<FText> AdditionalArgs;
	AdditionalArgs.Reserve(1);
	AdditionalArgs.Add(UEnum::GetDisplayValueAsText(InstanceData.WaitType));

	const FText SequenceQueryText = GetSequenceQueryText(InstanceData, InFormatting);
	const FText AddOnText = FText::Format(INVTEXT("( {0} )"), FText::Join(FText::FromString(TEXT(" | ")), AdditionalArgs));

	return InFormatting == EStateTreeNodeFormatting::RichText
		? FText::Format(LOCTEXT("DescRich", "Continue {0} <s>{1}</>"), SequenceQueryText, AddOnText)
		: FText::Format(LOCTEXT("Desc", "Continue {0} {1}"), SequenceQueryText, AddOnText);
}
#endif

TArray<UAvaSequencePlayer*> FAvaTransitionContinueSequenceTask::ExecuteSequenceTask(FStateTreeExecutionContext& InContext) const
{
	IAvaSequencePlaybackObject* PlaybackObject = GetPlaybackObject(InContext);
	if (!PlaybackObject)
	{
		return TArray<UAvaSequencePlayer*>();
	}

	const FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);

	switch (InstanceData.QueryType)
	{
	case EAvaTransitionSequenceQueryType::Name:
		return PlaybackObject->ContinueSequencesByLabel(InstanceData.SequenceName);

	case EAvaTransitionSequenceQueryType::Tag:
		return PlaybackObject->ContinueSequencesByTag(InstanceData.SequenceTag, InstanceData.bPerformExactMatch);
	}

	checkNoEntry();
	return TArray<UAvaSequencePlayer*>();
}

#undef LOCTEXT_NAMESPACE
