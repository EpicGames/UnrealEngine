// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transition/AvaTransitionInitializeSequence.h"
#include "AvaSequenceLibrary.h"
#include "AvaSequencePlaybackObject.h"
#include "AvaTransitionUtils.h"
#include "Math/NumericLimits.h"
#include "StateTreeExecutionContext.h"
#include "Transition/AvaTransitionSequenceUtils.h"

EAvaTransitionSequenceWaitType FAvaTransitionInitializeSequence::GetWaitType(FStateTreeExecutionContext& InContext) const
{
	return EAvaTransitionSequenceWaitType::NoWait;
}

TArray<UAvaSequencePlayer*> FAvaTransitionInitializeSequence::ExecuteSequenceTask(FStateTreeExecutionContext& InContext) const
{
	IAvaSequencePlaybackObject* PlaybackObject = GetPlaybackObject(InContext);
	if (!PlaybackObject)
	{
		return TArray<UAvaSequencePlayer*>();
	}

	const FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);

	const FAvaSequencePlayParams PlaySettings = UAvaSequenceLibrary::MakeSingleFramePlaySettings(InstanceData.InitializeTime, InstanceData.PlayMode);

	switch (InstanceData.QueryType)
	{
	case EAvaTransitionSequenceQueryType::Name:
		return PlaybackObject->PlaySequencesByLabel(InstanceData.SequenceName, PlaySettings);

	case EAvaTransitionSequenceQueryType::Tag:
		return PlaybackObject->PlaySequencesByTag(InstanceData.SequenceTag, InstanceData.bPerformExactMatch, PlaySettings);
	}

	checkNoEntry();
	return TArray<UAvaSequencePlayer*>();
}
