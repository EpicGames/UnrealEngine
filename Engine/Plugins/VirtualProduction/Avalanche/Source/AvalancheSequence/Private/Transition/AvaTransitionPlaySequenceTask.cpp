// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transition/AvaTransitionPlaySequenceTask.h"
#include "AvaSequence.h"
#include "AvaSequencePlaybackObject.h"
#include "AvaSequencePlayer.h"
#include "AvaTransitionContext.h"
#include "AvaTransitionUtils.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeLinker.h"
#include "Transition/AvaTransitionSequenceUtils.h"

#define LOCTEXT_NAMESPACE "AvaTransitionPlaySequenceTask"

#if WITH_EDITOR
FText FAvaTransitionPlaySequenceTask::GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const
{
	const FInstanceDataType& InstanceData = InInstanceDataView.Get<FInstanceDataType>();

	TArray<FText> AdditionalArgs;
	AdditionalArgs.Reserve(2);
	AdditionalArgs.Add(UEnum::GetDisplayValueAsText(InstanceData.PlaySettings.PlayMode));
	AdditionalArgs.Add(UEnum::GetDisplayValueAsText(InstanceData.WaitType));

	const FText SequenceQueryText = GetSequenceQueryText(InstanceData, InFormatting);
	const FText AddOnText = FText::Format(INVTEXT("( {0} )"), FText::Join(FText::FromString(TEXT(" | ")), AdditionalArgs));

	return InFormatting == EStateTreeNodeFormatting::RichText
		? FText::Format(LOCTEXT("DescRich", "Play {0} <s>{1}</>"), SequenceQueryText, AddOnText)
		: FText::Format(LOCTEXT("Desc", "Play {0} {1}"), SequenceQueryText, AddOnText);
}
#endif

TArray<UAvaSequencePlayer*> FAvaTransitionPlaySequenceTask::ExecuteSequenceTask(FStateTreeExecutionContext& InContext) const
{
	IAvaSequencePlaybackObject* PlaybackObject = GetPlaybackObject(InContext);
	if (!PlaybackObject)
	{
		return TArray<UAvaSequencePlayer*>();
	}

	const FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);

	switch (InstanceData.QueryType)
	{
	case EAvaTransitionSequenceQueryType::Name:
		return PlaybackObject->PlaySequencesByLabel(InstanceData.SequenceName, InstanceData.PlaySettings);

	case EAvaTransitionSequenceQueryType::Tag:
		return PlaybackObject->PlaySequencesByTag(InstanceData.SequenceTag, InstanceData.bPerformExactMatch, InstanceData.PlaySettings);
	}

	checkNoEntry();
	return TArray<UAvaSequencePlayer*>();
}

#undef LOCTEXT_NAMESPACE
