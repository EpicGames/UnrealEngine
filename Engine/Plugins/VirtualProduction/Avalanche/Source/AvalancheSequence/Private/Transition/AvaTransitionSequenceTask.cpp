// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transition/AvaTransitionSequenceTask.h"
#include "AvaTransitionUtils.h"
#include "StateTreeExecutionContext.h"
#include "Transition/AvaTransitionSequenceUtils.h"

EAvaTransitionSequenceWaitType FAvaTransitionSequenceTask::GetWaitType(FStateTreeExecutionContext& InContext) const
{
	return InContext.GetInstanceData(*this).WaitType;
}
