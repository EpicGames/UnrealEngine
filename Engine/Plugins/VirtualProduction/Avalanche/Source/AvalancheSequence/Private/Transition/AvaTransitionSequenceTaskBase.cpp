// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transition/AvaTransitionSequenceTaskBase.h"
#include "AvaSequence.h"
#include "AvaSequencePlayer.h"
#include "AvaSequenceSubsystem.h"
#include "AvaTransitionContext.h"
#include "AvaTransitionUtils.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeLinker.h"
#include "Transition/AvaTransitionSequenceUtils.h"

#define LOCTEXT_NAMESPACE "AvaTransitionSequenceTaskBase"

TArray<UAvaSequencePlayer*> FAvaTransitionSequenceTaskBase::ExecuteSequenceTask(FStateTreeExecutionContext& InContext) const
{
	return TArray<UAvaSequencePlayer*>();
}

bool FAvaTransitionSequenceTaskBase::IsSequenceQueryValid(const FInstanceDataType& InInstanceData) const
{
	switch (InInstanceData.QueryType)
	{
	case EAvaTransitionSequenceQueryType::Name:
		return !InInstanceData.SequenceName.IsNone();

	case EAvaTransitionSequenceQueryType::Tag:
		return InInstanceData.SequenceTag.IsValid();
	}
	checkNoEntry();
	return false;
}

IAvaSequencePlaybackObject* FAvaTransitionSequenceTaskBase::GetPlaybackObject(FStateTreeExecutionContext& InContext) const
{
	const FAvaTransitionContext& TransitionContext = InContext.GetExternalData(TransitionContextHandle);
	const UAvaSequenceSubsystem& SequenceSubsystem = InContext.GetExternalData(SequenceSubsystemHandle);

	return FAvaTransitionSequenceUtils::GetPlaybackObject(TransitionContext, SequenceSubsystem);
}

TArray<TWeakObjectPtr<UAvaSequence>> FAvaTransitionSequenceTaskBase::GetActiveSequences(TConstArrayView<UAvaSequencePlayer*> InSequencePlayers) const
{
	TArray<TWeakObjectPtr<UAvaSequence>> OutActiveSequences;
	OutActiveSequences.Reserve(InSequencePlayers.Num());

	Algo::TransformIf(InSequencePlayers, OutActiveSequences,
		[](UAvaSequencePlayer* InPlayer)->bool
		{
			return IsValid(InPlayer);
		},
		[](UAvaSequencePlayer* InPlayer)->TWeakObjectPtr<UAvaSequence>
		{
			return TWeakObjectPtr<UAvaSequence>(InPlayer->GetAvaSequence());
		});

	return OutActiveSequences;
}

EStateTreeRunStatus FAvaTransitionSequenceTaskBase::WaitForActiveSequences(FStateTreeExecutionContext& InContext) const
{
	FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);

	EAvaTransitionSequenceWaitType WaitType = GetWaitType(InContext);

	// No wait should only succeed in Enter State, and so return failed run if active sequences is empty 
	if (WaitType == EAvaTransitionSequenceWaitType::NoWait)
	{
		if (InstanceData.ActiveSequences.IsEmpty())
		{
			return EStateTreeRunStatus::Failed;
		}
		return EStateTreeRunStatus::Succeeded;
	}

	IAvaSequencePlaybackObject* PlaybackObject = GetPlaybackObject(InContext);
	if (!PlaybackObject)
	{
		return EStateTreeRunStatus::Failed;
	}

	return FAvaTransitionSequenceUtils::UpdatePlayerRunStatus(*PlaybackObject, InstanceData.ActiveSequences, WaitType);
}

void FAvaTransitionSequenceTaskBase::StopActiveSequences(FStateTreeExecutionContext& InContext) const
{
	IAvaSequencePlaybackObject* PlaybackObject = GetPlaybackObject(InContext);
	if (!PlaybackObject)
	{
		return;
	}

	const FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);

	for (const TWeakObjectPtr<UAvaSequence>& SequenceWeak : InstanceData.ActiveSequences)
	{
		if (UAvaSequence* Sequence = SequenceWeak.Get())
		{
			PlaybackObject->StopSequence(Sequence);
		}
	}
}

FText FAvaTransitionSequenceTaskBase::GetSequenceQueryText(const FInstanceDataType& InInstanceData, EStateTreeNodeFormatting InFormatting) const
{
	switch (InInstanceData.QueryType)
	{
	case EAvaTransitionSequenceQueryType::Name:
		return InFormatting == EStateTreeNodeFormatting::RichText
			? FText::Format(INVTEXT("'<b>{0}</>'"), FText::FromName(InInstanceData.SequenceName))
			: FText::Format(INVTEXT("'{0}'"), FText::FromName(InInstanceData.SequenceName));

	case EAvaTransitionSequenceQueryType::Tag:
		return InFormatting == EStateTreeNodeFormatting::RichText
			? FText::Format(LOCTEXT("SequenceQueryTagRich", "<s>tag</> '<b>{0}</>'"), FText::FromName(InInstanceData.SequenceTag.ToName()))
			: FText::Format(LOCTEXT("SequenceQueryTag", "tag '{0}'"), FText::FromName(InInstanceData.SequenceTag.ToName()));
	}

	checkNoEntry();
	return FText::GetEmpty();
}

EStateTreeRunStatus FAvaTransitionSequenceTaskBase::EnterState(FStateTreeExecutionContext& InContext, const FStateTreeTransitionResult& InTransition) const
{
	FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);

	if (!IsSequenceQueryValid(InstanceData))
	{
		return EStateTreeRunStatus::Failed;
	}

	TArray<UAvaSequencePlayer*> SequencePlayers = ExecuteSequenceTask(InContext);
	if (SequencePlayers.IsEmpty())
	{
		return EStateTreeRunStatus::Succeeded;
	}

	InstanceData.ActiveSequences = FAvaTransitionSequenceUtils::GetSequences(SequencePlayers);
	return WaitForActiveSequences(InContext);
}

EStateTreeRunStatus FAvaTransitionSequenceTaskBase::Tick(FStateTreeExecutionContext& InContext, const float InDeltaTime) const
{
	return WaitForActiveSequences(InContext);
}

void FAvaTransitionSequenceTaskBase::ExitState(FStateTreeExecutionContext& InContext, const FStateTreeTransitionResult& InTransition) const
{
	if (InTransition.CurrentRunStatus == EStateTreeRunStatus::Stopped)
	{
		StopActiveSequences(InContext);
	}
}

bool FAvaTransitionSequenceTaskBase::Link(FStateTreeLinker& InLinker)
{
	Super::Link(InLinker);
	InLinker.LinkExternalData(SequenceSubsystemHandle);
	return true;
}

#undef LOCTEXT_NAMESPACE
