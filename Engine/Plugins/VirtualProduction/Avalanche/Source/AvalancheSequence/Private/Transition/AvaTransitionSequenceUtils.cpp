// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transition/AvaTransitionSequenceUtils.h"
#include "AvaSequence.h"
#include "AvaSequencePlaybackObject.h"
#include "AvaSequencePlayer.h"
#include "AvaSequenceSubsystem.h"
#include "AvaTransitionContext.h"
#include "StateTreeExecutionContext.h"

IAvaSequencePlaybackObject* FAvaTransitionSequenceUtils::GetPlaybackObject(const FAvaTransitionContext& InTransitionContext, const UAvaSequenceSubsystem& InSequenceSubsystem)
{
	const FAvaTransitionScene* TransitionScene = InTransitionContext.GetTransitionScene();
	if (!TransitionScene)
	{
		return nullptr;
	}

	ULevel* const Level = TransitionScene->GetLevel();
	if (!Level)
	{
		return nullptr;
	}

	return InSequenceSubsystem.FindPlaybackObject(Level);
}

EStateTreeRunStatus FAvaTransitionSequenceUtils::UpdatePlayerRunStatus(const IAvaSequencePlaybackObject& InPlaybackObject, TArray<TWeakObjectPtr<UAvaSequence>>& InOutSequences, EAvaTransitionSequenceWaitType InWaitType)
{
	bool bWaitUntilStop = InWaitType == EAvaTransitionSequenceWaitType::WaitUntilStop;

	InOutSequences.RemoveAll(
		[&InPlaybackObject, bWaitUntilStop](const TWeakObjectPtr<UAvaSequence>& InSequenceWeak)
		{
			UAvaSequencePlayer* SequencePlayer = InPlaybackObject.GetSequencePlayer(InSequenceWeak.Get());
			// Remove if there's no Active Sequence Player for the given Sequence
			// Or if we are supposed to succeed on a stop point, return true
			// TODO: right now, a Temporal Pause Point will also be removed. The aim is to conditionally remove Stop Points only
			// (i.e. the points that pause the player indefinitely)
			return !SequencePlayer || (bWaitUntilStop && SequencePlayer->GetPlaybackStatus() == EMovieScenePlayerStatus::Paused);
		});

	// If no remaining active sequences, end with success
	if (InOutSequences.IsEmpty())
	{
		return EStateTreeRunStatus::Succeeded;
	}

	return EStateTreeRunStatus::Running;
}

TArray<TWeakObjectPtr<UAvaSequence>> FAvaTransitionSequenceUtils::GetSequences(TConstArrayView<UAvaSequencePlayer*> InSequencePlayers)
{
	TArray<TWeakObjectPtr<UAvaSequence>> OutActiveSequences;
	OutActiveSequences.Reserve(InSequencePlayers.Num());

	Algo::TransformIf(InSequencePlayers, OutActiveSequences,
		[](UAvaSequencePlayer* InPlayer)->bool
		{
			return IsValid(InPlayer);
		},
		[](UAvaSequencePlayer* InPlayer)->TWeakObjectPtr<UAvaSequence>
		{
			return TWeakObjectPtr<UAvaSequence>(InPlayer->GetAvaSequence());
		});

	return OutActiveSequences;
}
