// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transition/AvaTransitionWaitForAllSequencesTask.h"
#include "AvaSequencePlaybackObject.h"
#include "AvaSequenceSubsystem.h"
#include "AvaTransitionContext.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeLinker.h"
#include "Transition/AvaTransitionSequenceUtils.h"

EStateTreeRunStatus FAvaTransitionWaitForAllSequencesTask::EnterState(FStateTreeExecutionContext& InContext, const FStateTreeTransitionResult& InTransition) const
{
	IAvaSequencePlaybackObject* PlaybackObject = GetPlaybackObject(InContext);
	if (!PlaybackObject)
	{
		return EStateTreeRunStatus::Failed;
	}

	TArray<UAvaSequencePlayer*> SequencePlayers = PlaybackObject->GetAllSequencePlayers();
	if (SequencePlayers.IsEmpty())
	{
		return EStateTreeRunStatus::Succeeded;
	}

	FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);
	InstanceData.ActiveSequences = FAvaTransitionSequenceUtils::GetSequences(SequencePlayers);
	return WaitForAllSequences(InContext);
}

EStateTreeRunStatus FAvaTransitionWaitForAllSequencesTask::Tick(FStateTreeExecutionContext& InContext, const float InDeltaTime) const
{
	return WaitForAllSequences(InContext);
}

bool FAvaTransitionWaitForAllSequencesTask::Link(FStateTreeLinker& InLinker)
{
	Super::Link(InLinker);
	InLinker.LinkExternalData(SequenceSubsystemHandle);
	return true;
}

IAvaSequencePlaybackObject* FAvaTransitionWaitForAllSequencesTask::GetPlaybackObject(FStateTreeExecutionContext& InContext) const
{
	const FAvaTransitionContext& TransitionContext = InContext.GetExternalData(TransitionContextHandle);
	const UAvaSequenceSubsystem& SequenceSubsystem = InContext.GetExternalData(SequenceSubsystemHandle);
	return FAvaTransitionSequenceUtils::GetPlaybackObject(TransitionContext, SequenceSubsystem);
}

EStateTreeRunStatus FAvaTransitionWaitForAllSequencesTask::WaitForAllSequences(FStateTreeExecutionContext& InContext) const
{
	IAvaSequencePlaybackObject* PlaybackObject = GetPlaybackObject(InContext);
	if (!PlaybackObject)
	{
		return EStateTreeRunStatus::Failed;
	}

	FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);
	return FAvaTransitionSequenceUtils::UpdatePlayerRunStatus(*PlaybackObject, InstanceData.ActiveSequences, EAvaTransitionSequenceWaitType::WaitUntilStop);
}
