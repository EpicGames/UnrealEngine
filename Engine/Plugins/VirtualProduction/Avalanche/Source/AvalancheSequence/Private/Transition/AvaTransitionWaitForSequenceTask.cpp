// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transition/AvaTransitionWaitForSequenceTask.h"
#include "AvaSequencePlaybackObject.h"
#include "AvaSequencePlayer.h"
#include "StateTreeExecutionContext.h"

#define LOCTEXT_NAMESPACE "AvaTransitionWaitForSequenceTask"

#if WITH_EDITOR
FText FAvaTransitionWaitForSequenceTask::GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const
{
	const FInstanceDataType& InstanceData = InInstanceDataView.Get<FInstanceDataType>();

	const FText SequenceQueryText = GetSequenceQueryText(InstanceData, InFormatting);

	return InFormatting == EStateTreeNodeFormatting::RichText
		? FText::Format(LOCTEXT("DescRich", "Wait <s>for</> {0}"), SequenceQueryText)
		: FText::Format(LOCTEXT("Desc", "Wait for {0}"), SequenceQueryText);
}
#endif

TArray<UAvaSequencePlayer*> FAvaTransitionWaitForSequenceTask::ExecuteSequenceTask(FStateTreeExecutionContext& InContext) const
{
	IAvaSequencePlaybackObject* PlaybackObject = GetPlaybackObject(InContext);
	if (!PlaybackObject)
	{
		return TArray<UAvaSequencePlayer*>();
	}

	const FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);

	switch (InstanceData.QueryType)
	{
	case EAvaTransitionSequenceQueryType::Name:
		return PlaybackObject->GetSequencePlayersByLabel(InstanceData.SequenceName);

	case EAvaTransitionSequenceQueryType::Tag:
		return PlaybackObject->GetSequencePlayersByTag(InstanceData.SequenceTag, InstanceData.bPerformExactMatch);
	}

	checkNoEntry();
	return TArray<UAvaSequencePlayer*>();
}

#undef LOCTEXT_NAMESPACE
