// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequenceShared.h"
#include "AvaTagHandle.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "LevelSequence.h"
#include "Logging/LogMacros.h"
#include "Marks/AvaMark.h"
#include "Marks/AvaMarkRole.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "AvaSequence.generated.h"

class IAvaSequenceProvider;
class IPropertyHandle;
class ISequencer;
class UAvaStopObject;
class UMovieScene3DTransformTrack;
class UMovieScene;
class UMovieScenePropertyTrack;
class UObject;
class UWorld;
struct FMovieSceneMarkedFrame;
struct FMovieScenePossessable;

UCLASS(BlueprintType, Config=MotionDesign)
class AVALANCHESEQUENCE_API UAvaSequence : public ULevelSequence
{
	GENERATED_BODY()

public:
	UAvaSequence(const FObjectInitializer& InObjectInitializer);

	virtual ~UAvaSequence() override;

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSequenceEditUndoEvent, UAvaSequence*);

	/** Called on Sequence Edit Undo */
	static FOnSequenceEditUndoEvent::RegistrationType& OnSequenceEditUndo()
	{
		return OnSequenceEditUndoDelegate;
	}
#endif

	IAvaSequenceProvider* GetSequenceProvider() const;

	/**
	 * Gets the World of the Sequencer Provider. May be null if world has not been set
	 * Note: UObject::GetWorld isn't overriden to allow the IAvaSequenceProvider or other Outer to decide on the UWorld to return for that.
	 */
	UWorld* GetContextWorld() const;

	UFUNCTION(BlueprintPure, Category = "Label")
	FName GetLabel() const;

	UFUNCTION(BlueprintCallable, Category = "Label")
	void SetLabel(FName InLabel);

	UFUNCTION(BlueprintPure, Category = "Motion Design|Sequence")
	FAvaTagHandle GetSequenceTag() const;

	void SetSequenceTag(const FAvaTagHandle& InSequenceTag);

	/** Gets the Start Time of this Sequence */
	UFUNCTION(BlueprintPure, Category = "Motion Design|Sequence")
	double GetStartTime() const;

	/** Gets the End Time of this Sequence */
	UFUNCTION(BlueprintPure, Category = "Motion Design|Sequence")
	double GetEndTime() const;

	/** Cleans up this sequence and recursively calls this on each Child Sequence */
	void UpdateTreeNode();

	FSimpleMulticastDelegate& GetOnTreeNodeUpdated() { return OnTreeNodeUpdated; }

	/** Called when this Sequence has been removed from its Manager */
	void OnSequenceRemoved();

	int32 AddChild(UAvaSequence* InChild);

	bool RemoveChild(UAvaSequence* InChild);

	void RemoveAllChildren();

	void SetParent(UAvaSequence* InParent);

	UAvaSequence* GetParent() const;

	const TArray<TWeakObjectPtr<UAvaSequence>>& GetChildren() const { return ChildAnimations; }

	UFUNCTION(BlueprintPure, Category = "Motion Design|Sequence")
	const TSet<FAvaMark>& GetMarks() const { return Marks; }

	/**
	 * Returns the Preview Mark found using PreviewMarkLabel
	 * @return Pointer to the Mark set as Preview. Might be null if no preview mark was found, or if PreviewMarkLabel is empty
	 */
	const FAvaMark* GetPreviewMark() const;

	const FAvaMark* FindMark(const FMovieSceneMarkedFrame& InMarkedFrame) const;

	FAvaMark& FindOrAddMark(const FString& InMarkLabel);

	UFUNCTION(BlueprintCallable, Category = "Marks")
	bool GetMark(const FString& InMarkLabel, FAvaMark& OutMark) const;

	UFUNCTION(BlueprintCallable, Category = "Marks")
	bool SetMark(const FString& InMarkLabel, const FAvaMark& InMark);

	void UpdateMarkList();

	TArray<UObject*> GetBoundObjects(UObject* InPlaybackContext) const;

	//~ Begin ULevelSequence
	virtual void Initialize() override;
	//~ End ULevelSequence

	//~ Begin UMovieSceneSequence
#if WITH_EDITOR
	virtual FText GetDisplayName() const override;
#endif
	virtual UObject* CreateDirectorInstance(TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID InSequenceID) override;
	virtual bool CanPossessObject(UObject& InObject, UObject* InPlaybackContext) const override;
	virtual UObject* GetParentObject(UObject* InObject) const override;
	virtual bool CanAnimateObject(UObject& InObject) const override;
	//~ End UMovieSceneSequence

	//~ Begin UObject
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditUndo() override;
	void OnObjectTransacted(UObject* InObject, const FTransactionObjectEvent& InTransactionEvent);
#endif
	//~ End UObject

	int32 UpdateBindings(const FTopLevelAssetPath* InOldContext, const FTopLevelAssetPath& InNewContext);

	/** Gets the Objects Bound for this Sequence. Potentially slow */
	TArrayView<TWeakObjectPtr<>> FindObjectsFromGuid(const FGuid& InGuid);

	/* Gets the Object Id for the given Object in this Sequence. Potentially slow */
	FGuid FindGuidFromObject(UObject* InObject);

#if WITH_EDITOR
	void OnOuterWorldRenamed(UObject* InNewOuter);
#endif

	/** Called to clean up resources */
	void Cleanup();

	void OnWorldCleanup(UWorld* InWorld, bool bInSessionEnded, bool bInCleanupResources);

protected:
#if WITH_EDITOR
	/** Called on Sequence Edit Undo */
	static FOnSequenceEditUndoEvent OnSequenceEditUndoDelegate;
#endif

	/** Label used to identify a sequence. Multiple sequences can share the same label. */
	UPROPERTY()
	FName Label;

	UPROPERTY(DuplicateTransient, TextExportTransient)
	TWeakObjectPtr<UAvaSequence> ParentAnimation;

	UPROPERTY(DuplicateTransient, TextExportTransient)
	TArray<TWeakObjectPtr<UAvaSequence>> ChildAnimations;

	UPROPERTY(EditAnywhere, DisplayName="Tags", Category = "Sequence Settings", meta=(AllowPrivateAccess="true"))
	FAvaTagHandle Tag;

	/** The Mark to use to Preview the Sequence */
	UPROPERTY(EditAnywhere, DisplayName="Preview Mark", Category = "Sequence Settings", meta=(AllowPrivateAccess="true"))
	FString PreviewMarkLabel;

	/** The list of Marks in this Sequence */
	UPROPERTY(EditAnywhere, EditFixedSize, Category = "Sequence Settings", meta=(NoResetToDefault, AllowPrivateAccess="true"))
	TSet<FAvaMark> Marks;

	/** Delegate called after the node has finished cleaning up itself its children */
	FSimpleMulticastDelegate OnTreeNodeUpdated;
};
