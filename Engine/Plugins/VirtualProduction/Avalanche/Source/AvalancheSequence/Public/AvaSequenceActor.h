// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSequenceActor.h"
#include "AvaSequenceActor.generated.h"

class UAvaSequence;
class UAvaSequencePlayer;

UCLASS(MinimalAPI)
class AAvaSequenceActor : public ALevelSequenceActor
{
	GENERATED_BODY()

public:
	AAvaSequenceActor(const FObjectInitializer& InObjectInitializer);

	AVALANCHESEQUENCE_API void Initialize(UAvaSequence* InSequence);

protected:
	//~ Begin AActor
	virtual void PostInitializeComponents() override;
	//~ End AActor

private:
	void InitSequencePlayer(UAvaSequence* InSequence);
};
