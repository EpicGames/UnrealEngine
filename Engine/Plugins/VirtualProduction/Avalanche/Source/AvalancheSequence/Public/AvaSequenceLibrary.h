// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "AvaSequenceLibrary.generated.h"

class IAvaSequencePlaybackObject;
enum class EAvaSequencePlayMode : uint8;
struct FAvaSequencePlayParams;
struct FAvaSequenceTime;
template<typename InInterfaceType> class TScriptInterface;

UCLASS(MinimalAPI, DisplayName = "Motion Design Sequence Library", meta=(ScriptName = "MotionDesignSequenceLibrary"))
class UAvaSequenceLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Motion Design|Sequence", meta=(WorldContext = "InWorldContextObject"))
	static AVALANCHESEQUENCE_API TScriptInterface<IAvaSequencePlaybackObject> GetPlaybackObject(const UObject* InWorldContextObject);

	/** Helper function to build Play Settings for Single Frame Playback */
	UFUNCTION(BlueprintPure, Category = "Motion Design|Sequence")
	static AVALANCHESEQUENCE_API FAvaSequencePlayParams MakeSingleFramePlaySettings(const FAvaSequenceTime& InTargetTime, EAvaSequencePlayMode InPlayMode);
};
