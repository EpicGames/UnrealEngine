// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "AvaSequenceName.generated.h"

USTRUCT(BlueprintType, DisplayName="Motion Design Sequence Name")
struct FAvaSequenceName
{
	GENERATED_BODY()

	FAvaSequenceName() = default;

	FAvaSequenceName(FName InSequenceName)
		: Name(InSequenceName)
	{
	}

	AVALANCHESEQUENCE_API bool SerializeFromMismatchedTag(const FPropertyTag& InPropertyTag, FStructuredArchive::FSlot InSlot);

	bool IsNone() const
	{
		return Name.IsNone();
	}

	operator FName() const
	{
		return Name;
	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Motion Design Sequence")
	FName Name;
};

template<>
struct TStructOpsTypeTraits<FAvaSequenceName> : public TStructOpsTypeTraitsBase2<FAvaSequenceName>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};
