// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequenceShared.h"
#include "MovieSceneSequenceID.h"
#include "UObject/Interface.h"
#include "AvaSequencePlaybackObject.generated.h"

class IMovieScenePlayer;
class UAvaSequence;
class UAvaSequencePlayer;
class ULevel;
class UObject;
struct FAvaTagHandle;

namespace UE::MovieScene
{
	struct FOnCameraCutUpdatedParams;
}

UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UAvaSequencePlaybackObject : public UInterface
{
	GENERATED_BODY()
};

class IAvaSequencePlaybackObject
{
	GENERATED_BODY()

public:
	virtual UObject* ToUObject() = 0;

	virtual ULevel* GetPlaybackLevel() const = 0;

	/**
	 * Tears down both the Active and Stopped Players in this Playback Object
	 * Should be only be called when ending play
	 */
	virtual void CleanupPlayers() = 0;

	virtual UAvaSequencePlayer* PlaySequence(UAvaSequence* InSequence, const FAvaSequencePlayParams& InPlaySettings = FAvaSequencePlayParams()) = 0;

	/**
	 * Evaluates the Preview Frame of a Sequence.
	 * Does nothing if the Sequence has no preview frame.
	 * @param InSequence the sequence to preview
	 * @return the player instantiated for the Sequence, or null if Sequence was not valid or did not have a preview mark
	 */
	virtual UAvaSequencePlayer* PreviewFrame(UAvaSequence* InSequence) = 0;

	/**
	 * Plays a single sequence by its soft reference
	 * @param InSequence soft reference of the sequence to play
	 * @param InPlaySettings the play settings to use for playback
	 * @return the player instantiated for the Sequence, or null if Sequence was not valid for playback
	 */
	UFUNCTION(BlueprintCallable, DisplayName = "Play Sequence (by Soft Reference)", Category = "Playback")
	virtual UAvaSequencePlayer* PlaySequenceBySoftReference(TSoftObjectPtr<UAvaSequence> InSequence, const FAvaSequencePlayParams& InPlaySettings) = 0;

	/**
	 * Plays all the sequences that have the provided label
	 * @param InSequenceLabel the label of the sequences to play
	 * @param InPlaySettings the play settings to use for playback
	 * @return an array of the Sequence Players with possible invalid/null entries kept so that each Player matches in Index with the input Sequence it is playing
	 */
	UFUNCTION(BlueprintCallable, DisplayName = "Play Sequences (by Label)", Category = "Playback")
	virtual TArray<UAvaSequencePlayer*> PlaySequencesByLabel(FName InSequenceLabel, const FAvaSequencePlayParams& InPlaySettings) = 0;

	/**
	 * Plays multiple Sequences by their Soft Reference
	 * @param InSequences the array of soft reference sequences to play
	 * @param InPlaySettings the play settings to use for playback
	 * @return an array of the Sequence Players with possible invalid/null entries kept so that each Player matches in Index with the input Sequence it is playing
	 */
	UFUNCTION(BlueprintCallable, DisplayName = "Play Sequences (by Soft Reference)", Category = "Playback")
	virtual TArray<UAvaSequencePlayer*> PlaySequencesBySoftReference(const TArray<TSoftObjectPtr<UAvaSequence>>& InSequences, const FAvaSequencePlayParams& InPlaySettings) = 0;

	/**
	 * Plays multiple Sequences by an array of sequence labels
	 * @param InSequenceLabels the array of sequence labels to play
	 * @param InPlaySettings the play settings to use for playback
	 * @return an array of the Sequence Players with possible invalid/null entries kept so that each Player matches in Index with the input Sequence it is playing
	 */
	UFUNCTION(BlueprintCallable, DisplayName = "Play Sequences (by Labels)", Category = "Playback")
	virtual TArray<UAvaSequencePlayer*> PlaySequencesByLabels(const TArray<FName>& InSequenceLabels, const FAvaSequencePlayParams& InPlaySettings) = 0;

	/**
	 * Plays all the Sequences that match the given gameplay tag(s)
	 * @param InTagHandle the tag to match
	 * @param bInExactMatch whether to only consider sequences that have the tag exactly
	 * @param InPlaySettings the play settings to use for playback
	 * @return an array of the Sequence Players with only valid entries kept
	 */
	UFUNCTION(BlueprintCallable, DisplayName = "Play Sequences (by Tag)", Category = "Playback")
	virtual TArray<UAvaSequencePlayer*> PlaySequencesByTag(const FAvaTagHandle& InTagHandle, bool bInExactMatch, const FAvaSequencePlayParams& InPlaySettings) = 0;

	/**
	 * Plays the Scheduled Sequences with the Scheduled Play Settings
	 * @return an array of the Sequence Players with possible invalid/null entries kept so that each Player matches in Index with the Scheduled Sequence it is playing
	 */
	UFUNCTION(BlueprintCallable, Category = "Playback")
	virtual TArray<UAvaSequencePlayer*> PlayScheduledSequences() = 0;

	/**
	 * Triggers Continue for given sequence
	 * @param InSequence the sequence to continue
	 * @return the active sequence player that fired the continue, or null if there was no active player for the sequence
	 */
	virtual UAvaSequencePlayer* ContinueSequence(UAvaSequence* InSequence) = 0;

	/**
	 * Triggers Continue for the playing sequences that match the given label
	 * @param InSequenceLabel the label of the sequences to continue
	 * @return the sequence players that fired the continue, or null if there were no active players
	 */
	UFUNCTION(BlueprintCallable, DisplayName = "Continue Sequences (by Label)", Category = "Playback")
	virtual TArray<UAvaSequencePlayer*> ContinueSequencesByLabel(FName InSequenceLabel) = 0;

	/**
	 * Triggers Continues in multiple playing sequences given by an array of sequence labels
	 * @param InSequenceLabels the array of sequence labels to trigger continue (must be an active sequence playing)
	 * @return the sequence player array that fired the continue. It might have possible invalid/null entries so that each Player matches in Index with the input Sequence labels
	 */
	UFUNCTION(BlueprintCallable, DisplayName = "Continue Sequences (by Labels)", Category = "Playback")
	virtual TArray<UAvaSequencePlayer*> ContinueSequencesByLabels(const TArray<FName>& InSequenceLabels) = 0;

	/**
	 * Triggers Continues in all the sequences matching the provided tag
	 * @param InTagHandle the tag to match
	 * @param bInExactMatch whether to only consider sequences that have the tag exactly
	 * @return the array of the Sequence Players with only valid entries that fired the continue
	 */
	UFUNCTION(BlueprintCallable, DisplayName = "Continue Sequences (by Tag)", Category = "Playback")
	virtual TArray<UAvaSequencePlayer*> ContinueSequencesByTag(const FAvaTagHandle& InTagHandle, bool bInExactMatch) = 0;

	/**
	 * Pauses all players for a given sequence
	 * @param InSequence the sequence to pause
	 */
	virtual void PauseSequence(UAvaSequence* InSequence) = 0;

	/**
	 * Pauses all players for sequences that match the given label
	 * @param InSequenceLabel the label of the sequences to pause
	 */
	UFUNCTION(BlueprintCallable, DisplayName = "Pause Sequences (by Label)", Category = "Playback")
	virtual void PauseSequencesByLabel(FName InSequenceLabel) = 0;

	/**
	 * Pauses all players for sequences that match the given labels
	 * @param InSequenceLabels the array of sequence labels to pause (must be an active sequence playing)
	 */
	UFUNCTION(BlueprintCallable, DisplayName = "Pause Sequences (by Labels)", Category = "Playback")
	virtual void PauseSequencesByLabels(const TArray<FName>& InSequenceLabels) = 0;

	/**
	 * Pauses all players for sequences that match the provided tag
	 * @param InTagHandle the tag to match
	 * @param bInExactMatch whether to only consider sequences that match the tag exactly
	 */
	UFUNCTION(BlueprintCallable, DisplayName = "Pause Sequences (by Tag)", Category = "Playback")
	virtual void PauseSequencesByTag(const FAvaTagHandle& InTagHandle, bool bInExactMatch) = 0;

	/**
	 * Stops all players for a given sequence
	 * @param InSequence the sequence to stop
	 */
	virtual void StopSequence(UAvaSequence* InSequence) = 0;

	/**
	 * Stops all players for sequences that match the given label
	 * @param InSequenceLabel the label of the sequences to stop
	 */
	UFUNCTION(BlueprintCallable, DisplayName = "Stop Sequences (by Label)", Category = "Playback")
	virtual void StopSequencesByLabel(FName InSequenceLabel) = 0;

	/**
	 * Stops all players for sequences that match the given labels
	 * @param InSequenceLabels the array of sequence labels to stop (must be an active sequence playing)
	 */
	UFUNCTION(BlueprintCallable, DisplayName = "Stop Sequences (by Labels)", Category = "Playback")
	virtual void StopSequencesByLabels(const TArray<FName>& InSequenceLabels) = 0;

	/**
	 * Stops all players for sequences that match the provided tag
	 * @param InTagHandle the tag to match
	 * @param bInExactMatch whether to only consider sequences that match the tag exactly
	 */
	UFUNCTION(BlueprintCallable, DisplayName = "Stop Sequences (by Tag)", Category = "Playback")
	virtual void StopSequencesByTag(const FAvaTagHandle& InTagHandle, bool bInExactMatch) = 0;

	virtual void UpdateCameraCut(const UE::MovieScene::FOnCameraCutUpdatedParams& InCameraCutParams) = 0;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnCameraCut, UObject* /*CameraObject*/, bool /*bJump*/)
	virtual FOnCameraCut& GetOnCameraCut() = 0;

	virtual UObject* GetPlaybackContext() const = 0;

	virtual UObject* CreateDirectorInstance(IMovieScenePlayer& InPlayer, FMovieSceneSequenceID InSequenceID) = 0;

	virtual UAvaSequencePlayer* GetSequencePlayer(const UAvaSequence* InSequence) const = 0;

	/** Retrieves the active sequence players for the given sequences, if any */
	virtual TArray<UAvaSequencePlayer*> GetSequencePlayers(TConstArrayView<const UAvaSequence*> InSequences) const = 0;

	/** Retrieves the active sequence players for the given sequences that match the given label, if any */
	virtual TArray<UAvaSequencePlayer*> GetSequencePlayersByLabel(FName InSequenceLabel) const = 0;

	/** Retrieves the active sequence players for the given sequences that match the given labels, if any */
	virtual TArray<UAvaSequencePlayer*> GetSequencePlayersByLabels(const TArray<FName>& InSequenceLabels) const = 0;

	/** Retrieves the active sequence players for the given sequences that match the given tag, if any */
	virtual TArray<UAvaSequencePlayer*> GetSequencePlayersByTag(const FAvaTagHandle& InTagHandle, bool bInExactMatch) const = 0;

	/** Retrieves all Active Sequence Players */
	UFUNCTION(BlueprintCallable, DisplayName = "Get Active Sequence Players", Category = "Playback")
	virtual TArray<UAvaSequencePlayer*> GetAllSequencePlayers() const = 0;

	/** Returns true if there are any Active Sequence Players */
	UFUNCTION(BlueprintCallable, DisplayName = "Has Active Sequence Players", Category = "Playback")
	virtual bool HasActiveSequencePlayers() const = 0;
};
