// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequencePlaybackObject.h"
#include "Delegates/Delegate.h"
#include "LevelSequencePlayer.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakInterfacePtr.h"
#include "AvaSequencePlayer.generated.h"

class IAvaSequenceController;
class IAvaSequencePlaybackObject;
class UAvaSequence;
struct FAvaSequencePlayParams;
struct FLevelSequenceCameraSettings;

UCLASS(Transient, BlueprintType)
class AVALANCHESEQUENCE_API UAvaSequencePlayer : public ULevelSequencePlayer
{
	GENERATED_BODY()

public:
	UAvaSequencePlayer(const FObjectInitializer& InObjectInitializer);

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSequenceEvent, UAvaSequencePlayer*, UAvaSequence*);
	static FOnSequenceEvent& OnSequenceStarted() { return OnSequenceStartedDelegate; }
	static FOnSequenceEvent& OnSequencePaused() { return OnSequencePausedDelegate; }
	static FOnSequenceEvent& OnSequenceFinished() { return OnSequenceFinishedDelegate; }

	void InitSequence(UAvaSequence* InSequence, IAvaSequencePlaybackObject* InPlaybackObject, ULevel* InLevel, const FLevelSequenceCameraSettings& InCameraSettings);

	UAvaSequence* GetAvaSequence() const;

	IAvaSequencePlaybackObject* GetPlaybackObject() const;

	TSharedPtr<IAvaSequenceController> GetSequenceController() const { return SequenceController; }

	FQualifiedFrameTime GetGlobalTime() const;

	void SetPlaySettings(const FAvaSequencePlayParams& InPlaySettings);

	void PlaySequence();

	void ContinueSequence();

	/** Jump to the given Frame, in Tick Resolution space */
	void JumpTo(FFrameTime InJumpToFrame, bool bInEvaluate);

	using Super::GetDisplayRate;

	using Super::GetPlaybackStatus;

	FFrameRate GetTickResolution() const;

	void Cleanup();

protected:

	//~ Begin UMovieSceneSequencePlayer
	virtual void OnStartedPlaying() override;
	virtual void OnStopped() override;
	//~ End UMovieSceneSequencePlayer

	//~ Begin IMovieSceneSequenceTickManagerClient
	virtual void TickFromSequenceTickManager(float InDeltaSeconds, FMovieSceneEntitySystemRunner* InRunner) override;
	//~ End IMovieSceneSequenceTickManagerClient

	//~ Begin FCameraCutPlaybackCapability
	virtual void OnCameraCutUpdated(const UE::MovieScene::FOnCameraCutUpdatedParams& InCameraCutParams) override;
	//~ End FCameraCutPlaybackCapability

private:
	FFrameTime CalculateDeltaFrameTime(float InDeltaSeconds) const;

	void NotifySequenceStarted();

	UFUNCTION()
	void NotifySequencePaused();

	void NotifySequenceFinished();

	static FOnSequenceEvent OnSequenceStartedDelegate;
	static FOnSequenceEvent OnSequencePausedDelegate;
	static FOnSequenceEvent OnSequenceFinishedDelegate;

	/** Encapsulate OnNativeFinished as it's a simple delegate that only this Player will bind to and through it call the multicast version (OnSequenceFinishedDelegate) */
	using Super::OnNativeFinished;

	TWeakInterfacePtr<IAvaSequencePlaybackObject> PlaybackObjectWeak;

	TSharedPtr<IAvaSequenceController> SequenceController;

	TWeakObjectPtr<ULevel> PlaybackLevelWeak;
};
