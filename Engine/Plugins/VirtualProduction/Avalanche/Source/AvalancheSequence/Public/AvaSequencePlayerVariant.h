// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/TVariant.h"

class IMovieScenePlayer;

template<typename... InPlayerTypes>
struct TAvaSequencePlayerVariant
{
	template<typename InPlayerType UE_REQUIRES(TIsDerivedFrom<InPlayerType, IMovieScenePlayer>::Value)>
	TAvaSequencePlayerVariant(InPlayerType* InPlayer)
		: PlayerVariant(TInPlaceType<InPlayerType*>(), InPlayer)
	{
	}

	IMovieScenePlayer* Get() const
	{
		return ::Visit(*this, PlayerVariant);
	}

	template<typename InPlayerType>
	IMovieScenePlayer* operator()(InPlayerType* InPlayer) const
	{
		return InPlayer;
	}

	template<typename InPlayerType>
	InPlayerType* TryGet() const
	{
		if (InPlayerType* const * Player = PlayerVariant.template TryGet<InPlayerType*>())
		{
			return *Player;
		}
		return nullptr;
	}

private:
	TVariant<InPlayerTypes*...> PlayerVariant;
};
