// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "AvaSequenceShared.generated.h"

class UAvaSequence;
class UMovieScene;

namespace UE::AvaSequence
{
	constexpr double SmallSubFrame = 0.00000005;
}

/** Enumeration of actions that can be performed on a sequence */
UENUM()
enum class EAvaSequenceActionType : uint8
{
	None UMETA(Hidden),
	Play,
	Continue,
	Pause,
	Stop,
};

UENUM(BlueprintType, DisplayName="Motion Design Sequence Play Mode")
enum class EAvaSequencePlayMode : uint8
{
	/** Sequence plays and loops from the beginning to the end. */
	Forward,

	/** Sequence plays and loops from the end to the beginning. */
	Reverse,
};

UENUM(BlueprintType, DisplayName="Motion Design Sequence Time Type")
enum class EAvaSequenceTimeType : uint8
{
	None UMETA(Hidden),
	Frame,
	Seconds,
	Mark,
};

USTRUCT(BlueprintType, DisplayName="Motion Design Sequence Time")
struct FAvaSequenceTime
{
	GENERATED_BODY()

	enum ENoTimeConstraint {NoTimeConstraint};

	FAvaSequenceTime() = default;

	explicit FAvaSequenceTime(ENoTimeConstraint)
		: bHasTimeConstraint(false)
	{
	}

	explicit FAvaSequenceTime(FFrameTime FrameTime)
		: TimeType(EAvaSequenceTimeType::Frame)
		, Frame(FrameTime.GetFrame().Value)
		, SubFrame(FrameTime.GetSubFrame())
		, bHasTimeConstraint(true)
	{
	}

	explicit FAvaSequenceTime(double InSeconds)
		: TimeType(EAvaSequenceTimeType::Seconds)
		, Seconds(InSeconds)
		, bHasTimeConstraint(true)
	{
	}

	explicit FAvaSequenceTime(const FString& InMarkLabel)
		: TimeType(EAvaSequenceTimeType::Mark)
		, MarkLabel(InMarkLabel)
		, bHasTimeConstraint(true)
	{
	}

	// Returns whether the provided Time is 'absolute' (i.e. should not be manipulated by whether it's playing forward/reversed) or not
	bool IsAbsoluteTime() const
	{
		return TimeType == EAvaSequenceTimeType::Mark;
	}

	AVALANCHESEQUENCE_API bool Serialize(FArchive& Ar);

	/**
	 * Converts this Motion Design Sequence Time struct to time frame in seconds
	 * @param InSequence the sequence to use to query for marks (only applicable if TimeType is set to Mark)
	 * @param InMovieScene the movie scene used to get the Tick Resolution or Display Rate
	 * @param InDefaultTime the default time to return if time is invalid or unset
	 * @return the timeframe in seconds
	 */
	AVALANCHESEQUENCE_API double ToSeconds(const UAvaSequence& InSequence, const UMovieScene& InMovieScene, double InDefaultTime) const;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Sequence", meta=(EditCondition="bHasTimeConstraint"))
	EAvaSequenceTimeType TimeType = EAvaSequenceTimeType::Frame;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Sequence", meta=(EditCondition="bHasTimeConstraint && TimeType==EAvaSequenceTimeType::Frame"))
	int32 Frame = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Sequence", meta=(EditCondition="bHasTimeConstraint && TimeType==EAvaSequenceTimeType::Frame"))
	double SubFrame = 0.0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Sequence", meta=(EditCondition="bHasTimeConstraint && TimeType==EAvaSequenceTimeType::Seconds", Unit=s))
	double Seconds = 0.0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Sequence", meta=(EditCondition="bHasTimeConstraint && TimeType==EAvaSequenceTimeType::Mark"))
	FString MarkLabel;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Sequence")
	bool bHasTimeConstraint = false;
};

template<>
struct TStructOpsTypeTraits<FAvaSequenceTime> : public TStructOpsTypeTraitsBase2<FAvaSequenceTime>
{
	enum
	{
		WithSerializer = true
	};
};

USTRUCT(BlueprintType, DisplayName="Motion Design Sequence Play Advanced Settings")
struct FAvaSequencePlayAdvancedSettings
{
	GENERATED_BODY()

	/** Number of times to loop playback. -1 for infinite, else the number of times to loop before stopping */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Sequence")
	int32 LoopCount = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Sequence")
	float PlaybackSpeed = 1.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Sequence")
	bool bRestoreState = false;
};

USTRUCT(BlueprintType, DisplayName="Motion Design Sequence Play Parameters")
struct FAvaSequencePlayParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Sequence")
	FAvaSequenceTime Start = FAvaSequenceTime(0.0);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Sequence")
	FAvaSequenceTime End = FAvaSequenceTime(FAvaSequenceTime::NoTimeConstraint);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Sequence")
	EAvaSequencePlayMode PlayMode = EAvaSequencePlayMode::Forward;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design Sequence")
	FAvaSequencePlayAdvancedSettings AdvancedSettings;
};
