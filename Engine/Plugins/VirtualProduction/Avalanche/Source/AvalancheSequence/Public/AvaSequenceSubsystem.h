// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Delegates/IDelegateInstance.h"
#include "Subsystems/WorldSubsystem.h"
#include "UObject/ObjectKey.h"
#include "UObject/WeakInterfacePtr.h"
#include "AvaSequenceSubsystem.generated.h"

class IAvaSequenceController;
class IAvaSequencePlaybackObject;
class IAvaSequenceProvider;
class UAvaSequence;
class ULevel;

UCLASS(MinimalAPI)
class UAvaSequenceSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	UAvaSequenceSubsystem() = default;

	AVALANCHESEQUENCE_API static UAvaSequenceSubsystem* Get(UObject* InPlaybackContext);

	AVALANCHESEQUENCE_API static TSharedRef<IAvaSequenceController> CreateSequenceController(UAvaSequence& InSequence, IAvaSequencePlaybackObject* InPlaybackObject);

	AVALANCHESEQUENCE_API IAvaSequencePlaybackObject* FindOrCreatePlaybackObject(ULevel* InLevel, IAvaSequenceProvider& InSequenceProvider);

	AVALANCHESEQUENCE_API IAvaSequencePlaybackObject* FindPlaybackObject(ULevel* InLevel) const;

	AVALANCHESEQUENCE_API void AddPlaybackObject(IAvaSequencePlaybackObject* InPlaybackObject);
	AVALANCHESEQUENCE_API void RemovePlaybackObject(IAvaSequencePlaybackObject* InPlaybackObject);

	AVALANCHESEQUENCE_API IAvaSequenceProvider* FindSequenceProvider(const ULevel* InLevel) const;

	AVALANCHESEQUENCE_API void RegisterSequenceProvider(const ULevel* InLevel, IAvaSequenceProvider* InSequenceProvider);
	AVALANCHESEQUENCE_API bool UnregisterSequenceProvider(const ULevel* InLevel, IAvaSequenceProvider* InSequenceProvider);

protected:
	//~ Begin UWorldSubsystem
	AVALANCHESEQUENCE_API virtual bool DoesSupportWorldType(const EWorldType::Type InWorldType) const override;
	//~ End UWorldSubsystem

private:
	bool EnsureLevelIsAppropriate(ULevel*& InLevel) const;

	TArray<TWeakInterfacePtr<IAvaSequencePlaybackObject>> PlaybackObjects;

	TMap<TObjectKey<ULevel>, TWeakInterfacePtr<IAvaSequenceProvider>> SequenceProviders;
};
