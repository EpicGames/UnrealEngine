// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSequenceDirector.h"
#include "UObject/WeakInterfacePtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "AvaSequenceDirector.generated.h"

class IAvaSequencePlaybackObject;
class IAvaSequenceProvider;
class UAvaSequencePlayer;
struct FAvaSequencePlayParams;

UCLASS(Blueprintable, MinimalAPI)
class UAvaSequenceDirector : public ULevelSequenceDirector
{
	GENERATED_BODY()

	friend class UAvaSequence;

public:
	AVALANCHESEQUENCE_API void UpdateProperties();

protected:
	//~ Begin UObject
	AVALANCHESEQUENCE_API virtual void PostLoad() override;
	AVALANCHESEQUENCE_API virtual void PostDuplicate(EDuplicateMode::Type InDuplicateMode) override;
	//~ End UObject

private:
	void Initialize(IMovieScenePlayer& InPlayer, IAvaSequenceProvider* InSequenceProvider);

	UFUNCTION(BlueprintCallable, Category = "Playback")
	void PlayScheduledSequences();

	UFUNCTION(BlueprintCallable, Category = "Playback")
	void PlaySequencesByLabel(FName InSequenceLabel, FAvaSequencePlayParams InPlaySettings);

	UFUNCTION(BlueprintPure, Category = "Playback")
	TScriptInterface<IAvaSequencePlaybackObject> GetPlaybackObject() const;

	void UpdatePlaybackObject();

	TWeakObjectPtr<UAvaSequencePlayer> SequencePlayerWeak;

	TWeakInterfacePtr<IAvaSequencePlaybackObject> PlaybackObjectInterfaceWeak;
};
