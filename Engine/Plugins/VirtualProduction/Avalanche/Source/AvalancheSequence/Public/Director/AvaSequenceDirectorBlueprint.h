// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequenceDirectorShared.h"
#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Engine/Blueprint.h"
#include "AvaSequenceDirectorBlueprint.generated.h"

UCLASS(MinimalAPI)
class UAvaSequenceDirectorBlueprint : public UBlueprint
{
	GENERATED_BODY()

public:
	AVALANCHESEQUENCE_API TConstArrayView<FAvaSequenceInfo> GetSequenceInfos();

protected:
	//~ Begin UBlueprint
#if WITH_EDITOR
	virtual UClass* GetBlueprintClass() const override;
	virtual void GetReparentingRules(TSet<const UClass*>& OutAllowedChildrenOfClasses, TSet<const UClass*>& OutDisallowedChildrenOfClasses) const override;
#endif
	//~ End UBlueprint

private:
	void UpdateSequenceInfos();

	UPROPERTY()
	TArray<FAvaSequenceInfo> SequenceInfos;
};
