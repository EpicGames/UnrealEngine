// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequenceDirectorShared.h"
#include "Containers/Array.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "UObject/Package.h"
#include "AvaSequenceDirectorGeneratedClass.generated.h"

class UAvaSequenceDirector;

UCLASS(MinimalAPI)
class UAvaSequenceDirectorGeneratedClass : public UBlueprintGeneratedClass
{
	GENERATED_BODY()

public:
	void UpdateProperties(UAvaSequenceDirector* InDirector);

	UPROPERTY()
	TArray<FAvaSequenceInfo> SequenceInfos;
};
