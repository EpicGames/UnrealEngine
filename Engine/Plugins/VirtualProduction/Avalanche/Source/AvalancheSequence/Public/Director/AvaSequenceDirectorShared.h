// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "UObject/SoftObjectPtr.h"
#include "AvaSequenceDirectorShared.generated.h"

class UAvaSequence;

USTRUCT()
struct FAvaSequenceInfo
{
	GENERATED_BODY()

	UPROPERTY()
	FName SequenceName;

	UPROPERTY()
	TSoftObjectPtr<UAvaSequence> Sequence;
};
