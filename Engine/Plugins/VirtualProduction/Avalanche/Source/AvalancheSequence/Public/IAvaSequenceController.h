// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

class IAvaSequencePlaybackContext;
class IMovieScenePlayer;
class UAvaSequence;
class UAvaSequencePlayer;
struct FFrameTime;
template<typename... InPlayerTypes> struct TAvaSequencePlayerVariant;

#if WITH_EDITOR
class ISequencer;
using FAvaSequencePlayerVariant = TAvaSequencePlayerVariant<UAvaSequencePlayer, ISequencer>;
#else
using FAvaSequencePlayerVariant = TAvaSequencePlayerVariant<UAvaSequencePlayer>;
#endif

/** Interface to processing how Marks are processed when ticking a sequence */
class IAvaSequenceController
{
public:
	virtual ~IAvaSequenceController() = default;

	virtual void SetTime(const FFrameTime& InNewTime, bool bInResetState) = 0;

	virtual void Tick(const FAvaSequencePlayerVariant& InPlayerVariant, const FFrameTime& InDeltaFrameTime, float InDeltaSeconds) = 0;

	virtual UAvaSequence* GetSequence() const = 0;

	virtual TSharedRef<IAvaSequencePlaybackContext> GetPlaybackContext() const = 0;
};
