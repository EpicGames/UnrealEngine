// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Containers/ContainersFwd.h"

class FTSTicker;
struct FAvaMark;
struct FFrameTime;
struct FMovieSceneMarkedFrame;

/** Interface for Marks to interact with the Sequence being played */
class IAvaSequencePlaybackContext : public TSharedFromThis<IAvaSequencePlaybackContext>
{
public:
	virtual ~IAvaSequencePlaybackContext() = default;

	virtual FFrameTime GetGlobalTime() const = 0;

	virtual FFrameTime GetDeltaFrameTime() const = 0;

	virtual bool IsPlayingForwards() const = 0;

	virtual void Continue() = 0;

	virtual void Pause() = 0;

	virtual void Reverse() = 0;

	virtual void JumpTo(const FFrameTime& InFrameTime, bool bInEvaluateJumpedFrames) = 0;

	virtual void JumpToSelf() = 0;

	virtual const FAvaMark& GetMark() const = 0;

	virtual const TSet<FAvaMark>& GetAllMarks() const = 0;

	virtual const FMovieSceneMarkedFrame& GetMarkedFrame() const = 0;

	virtual FTSTicker& GetTicker() = 0;

	virtual TConstArrayView<TWeakPtr<IAvaSequencePlaybackContext>> GetChildrenInContext() const = 0;

	virtual void ResolveFromParent(const IAvaSequencePlaybackContext& InParent) = 0;
};
