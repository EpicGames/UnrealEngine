// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequence.h"
#include "Containers/ContainersFwd.h"
#include "MovieSceneFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "Templates/SharedPointer.h"
#include "IAvaSequenceProvider.generated.h"

class IMovieScenePlayer;
class UBlueprint;
class UObject;
class UWorld;
struct FMovieSceneSequenceID;

#if WITH_EDITOR
class ISequencer;
#endif

/** Interface for Objects that use UAvaSequence and need to be handled by IAvaSequencer */
UINTERFACE(MinimalAPI, meta = (CannotImplementInterfaceInBlueprint))
class UAvaSequenceProvider : public UInterface
{
	GENERATED_BODY()
};

class IAvaSequenceProvider
{
	GENERATED_BODY()

public:
	/** returns a debug name for the Sequence Provider */
	virtual FName GetSequenceProviderDebugName() const = 0;

#if WITH_EDITOR
	/** Called after the ISequencer Instance has been created */
    virtual void OnEditorSequencerCreated(const TSharedPtr<ISequencer>& InSequencer) = 0;

	virtual TSharedPtr<ISequencer> GetEditorSequencer() const = 0;

	/** Called to retrieve the Director Blueprint which should only be called in Editor Time */
	virtual bool GetDirectorBlueprint(UAvaSequence& InSequence, UBlueprint*& OutBlueprint) = 0;
#endif

	/** Returns this as UObject Outer to use when handling Sequence Objects */
	virtual UObject* ToUObject() = 0;

	virtual UWorld* GetContextWorld() const = 0;

	virtual bool CreateDirectorInstance(UAvaSequence& InSequence, IMovieScenePlayer& InPlayer, const FMovieSceneSequenceID& InSequenceID, UObject*& OutDirectorInstance) = 0;

	virtual bool AddSequence(UAvaSequence* InSequence) = 0;

	/**
	 * Adds the given sequences to the sequence provider
	 * @param InSequences the sequences to add, if valid and not already in the list.
	 * @return the number of sequences that were successfully added.
	 */
	virtual uint32 AddSequences(TConstArrayView<UAvaSequence*> InSequences) = 0;

	virtual void RemoveSequence(UAvaSequence* InSequence) = 0;

	/** Sets the Default Sequence to use when no particular sequence is active */
	virtual void SetDefaultSequence(UAvaSequence* InSequence) = 0;

	/** Gets the Default Sequence to use when no particular sequence is active. May return null */
	virtual UAvaSequence* GetDefaultSequence() const = 0;

	virtual const TArray<TObjectPtr<UAvaSequence>>& GetSequences() const = 0;

	virtual const TArray<TWeakObjectPtr<UAvaSequence>>& GetRootSequences() const = 0;

	virtual TArray<TWeakObjectPtr<UAvaSequence>>& GetRootSequencesMutable() = 0;

	/** Delegate called when the Sequence Tree has been Rebuilt */
	virtual FSimpleMulticastDelegate& GetOnSequenceTreeRebuilt() = 0;

	/** Executes Rebuild Sequence Tree in a Deferred way rather than executing immediately */
	virtual void ScheduleRebuildSequenceTree() = 0;

	virtual void RebuildSequenceTree()
	{
		TSet<TWeakObjectPtr<UAvaSequence>> AllSequences;
		TSet<TWeakObjectPtr<UAvaSequence>> ChildrenSequences;

		const TArray<TObjectPtr<UAvaSequence>>& CurrentSequences = GetSequences();

		AllSequences.Reserve(CurrentSequences.Num());
		ChildrenSequences.Reserve(CurrentSequences.Num());

		// Gather all Child Sequences
		for (const TObjectPtr<UAvaSequence>& Sequence : CurrentSequences)
		{
			if (Sequence)
			{
				AllSequences.Add(Sequence.Get());
				ChildrenSequences.Append(Sequence->GetChildren());
			}
		}

		// The Root Sequences are those that are not children of the rest
		TArray<TWeakObjectPtr<UAvaSequence>>& RootSequencesMutable = GetRootSequencesMutable();
		RootSequencesMutable = AllSequences.Difference(ChildrenSequences).Array();

		// Update Tree (Self & Children)
		for (const TWeakObjectPtr<UAvaSequence>& RootSequence : RootSequencesMutable)
		{
			if (RootSequence.IsValid())
			{
				RootSequence->UpdateTreeNode();
			}
		}

		GetOnSequenceTreeRebuilt().Broadcast();
	}
};
