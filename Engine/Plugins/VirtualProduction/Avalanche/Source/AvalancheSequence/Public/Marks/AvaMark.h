// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMarkShared.h"
#include "Containers/Array.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "AvaMark.generated.h"

struct FMovieSceneMarkedFrame;

USTRUCT(BlueprintType)
struct AVALANCHESEQUENCE_API FAvaMark
{
	GENERATED_BODY()

	FAvaMark() = default;
	FAvaMark(const FMovieSceneMarkedFrame& InMarkedFrame);
	FAvaMark(const FString& InLabel);
	explicit FAvaMark(const FStringView& InLabel);

	/** Set all Values without changing its Label or Frames */
	void CopyFromMark(const FAvaMark& InOther);

	bool operator==(const FAvaMark& Other) const
	{
		return Label == Other.Label;
	}

	FStringView GetLabel() const { return Label; }

	static const FName GetLabelPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(FAvaMark, Label);
	}

	friend uint32 GetTypeHash(const FAvaMark& InMark)
	{
		return GetTypeHash(InMark.Label);
	}

private:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design", meta=(NoResetToDefault, AllowPrivateAccess="true"))
	FString Label;

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Marks")
	EAvaMarkDirection Direction = EAvaMarkDirection::Both;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Marks")
	EAvaMarkRole Role = EAvaMarkRole::None;

	/**
	 * Whether Executing this Mark should affect the Local Sequence
	 * Set to True to only Execute Roles in the Owning Sequence
	 * Set to False to Affect Child Sequences of the Owning Sequence
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Marks")
	bool bIsLocalMark = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Marks")
	bool bLimitPlayCountEnabled = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Marks", meta = (EditCondition = "bLimitPlayCountEnabled"))
	int32 LimitPlayCount = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Marks", meta = (EditCondition = "Role==EAvaMarkRole::Pause", EditConditionHides))
	float PauseTime = 3.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Marks", meta = (EditCondition = "Role==EAvaMarkRole::Jump", EditConditionHides))
	FString JumpLabel;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Marks", meta = (EditCondition = "Role==EAvaMarkRole::Jump", EditConditionHides))
	EAvaMarkSearchDirection SearchDirection = EAvaMarkSearchDirection::All;

	/** The Frames where the Marked Frames with the same Label are at */
	UPROPERTY(Transient)
	TArray<int32> Frames;
};
