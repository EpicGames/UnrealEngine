// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMark.h"
#include "AvaMarkShared.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Evaluation/MovieScenePlayback.h"
#include "IAvaSequencePlaybackContext.h"
#include "Misc/FrameNumber.h"
#include "MovieScene.h"
#include "Templates/EnableIf.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTypeTraits.h"

/** Class involved in executing specific role-related logic via its Context */
class FAvaMarkRole : public TSharedFromThis<FAvaMarkRole>
{
	friend class FAvaMarkRoleHandler;

public:
	FAvaMarkRole() = default;

	virtual ~FAvaMarkRole() {};

	virtual EAvaMarkRole GetRole() const = 0;

	virtual EAvaMarkRoleReply Execute() = 0;

	virtual void Reset() {}

protected:
	TSharedPtr<IAvaSequencePlaybackContext> Context;
};

/** Class Containing Instance data of a Mark, such as current Play Count */
class FAvaMarkInstance
{
public:
	FAvaMarkInstance() = default;

	FAvaMarkInstance(const FAvaMark& InMark, const FFrameNumber& InFrame)
		: Label(InMark.GetLabel())
		, Frame(InFrame)
		, MaxPlayCounts(InMark.LimitPlayCount)
		, bPlayCountEnabled(InMark.bLimitPlayCountEnabled)
	{
	}

	FAvaMarkInstance(const TSharedPtr<IAvaSequencePlaybackContext>& InContext)
		: FAvaMarkInstance(InContext->GetMark(), InContext->GetMarkedFrame().FrameNumber)
	{
	}

	friend uint32 GetTypeHash(const FAvaMarkInstance& Instance)
	{
		return HashCombine(GetTypeHash(Instance.Label), GetTypeHash(Instance.Frame));
	}

	bool operator==(const FAvaMarkInstance& Other) const
	{
		return this->Label == Other.Label && this->Frame == Other.Frame;
	}

	void IncreasePlayCount()
	{
		if (bPlayCountEnabled)
		{
			++PlayCounts;	
		}
	}

	bool CanExecuteMark() const
	{
		if (!bPlayCountEnabled)
		{
			return true;
		}
		return PlayCounts <= MaxPlayCounts;
	}

private:
	/** Label of the Mark */
	FString Label;

	/** Frame that this Mark Instance is at */
	FFrameNumber Frame;

	/** Number of current Play Counts */
	int32 PlayCounts = 0;

	/** Max number of Play Counts allowed, if bPlayCountEnabled is true */
	int32 MaxPlayCounts = 0;

	/** Determines if there if Play Count should be considered when determining if Role can be executed */
	bool bPlayCountEnabled = false;
};

class FAvaMarkRoleHandler
{
public:
	template<typename InMarkRoleType, typename = typename TEnableIf<TIsDerivedFrom<InMarkRoleType, FAvaMarkRole>::IsDerived>::Type>
	void RegisterRole()
	{
		TSharedPtr<FAvaMarkRole> MarkRole = MakeShared<InMarkRoleType>();
		MarkRoles.Add(MarkRole->GetRole(), MarkRole);
	}

	bool IsMarkValid(const FAvaMark& InMark, const FFrameNumber& InFrame, EPlayDirection InPlayDirection) const
	{
		const FAvaMarkInstance* const MarkInstance = MarkInstances.Find(FAvaMarkInstance(InMark, InFrame));
		if (MarkInstance && !MarkInstance->CanExecuteMark())
		{
			return false;
		}

		// If the Limit Play Count is enabled and it's 0, then it should never execute
		if (InMark.bLimitPlayCountEnabled && InMark.LimitPlayCount == 0)
		{
			return false;
		}

		// Don't consider Marks with no Role.
		if (InMark.Role == EAvaMarkRole::None)
		{
			return false;
		}

		if (InMark.Direction == EAvaMarkDirection::Both)
		{
			return true;
		}

		if (InMark.Direction == EAvaMarkDirection::Forwards && InPlayDirection == EPlayDirection::Forwards)
		{
			return true;
		}

		if (InMark.Direction == EAvaMarkDirection::Backwards && InPlayDirection == EPlayDirection::Backwards)
		{
			return true;
		}

		return false;
	}

	EAvaMarkRoleReply ExecuteRole(const TSharedPtr<IAvaSequencePlaybackContext>& InContext)
	{
		if (!InContext.IsValid())
		{
			return EAvaMarkRoleReply::NotExecuted;
		}

		if (TSharedPtr<FAvaMarkRole>* const FoundRole = MarkRoles.Find(InContext->GetMark().Role))
		{
			if (FoundRole->IsValid())
			{
				FAvaMarkRole& MarkRole = *(FoundRole->Get());
				MarkRole.Context = InContext;

				FAvaMarkInstance& Instance = MarkInstances.FindOrAdd(InContext);

				Instance.IncreasePlayCount();

				if (Instance.CanExecuteMark())
				{
					return MarkRole.Execute();
				}
			}
		}

		return EAvaMarkRoleReply::NotExecuted;
	}

	void ResetMarkStates()
	{
		for (const TPair<EAvaMarkRole, TSharedPtr<FAvaMarkRole>>& Pair : MarkRoles)
		{
			if (Pair.Value.IsValid())
			{
				Pair.Value->Reset();
			}
		}
		MarkInstances.Reset();
	}

private:
	TMap<EAvaMarkRole, TSharedPtr<FAvaMarkRole>> MarkRoles;

	TSet<FAvaMarkInstance> MarkInstances;
};
