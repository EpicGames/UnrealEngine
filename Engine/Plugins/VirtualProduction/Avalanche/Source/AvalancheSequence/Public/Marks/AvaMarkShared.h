// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EnumClassFlags.h"
#include "AvaMarkShared.generated.h"

UENUM()
enum class EAvaMarkRole : uint8
{
	// Mark: Does nothing. Used for frame referencing
	None UMETA(DisplayName="Mark"),

	// Stop Point: Waits for 'Continue' input before resuming playback
	Stop,

	// Pause Point: Waits a set amount of time before resuming playback
	Pause,

	// Jump Point: Jumps to the nearest Marked Frame with the given Label, or continues playback if it doesn't exist
	Jump,

	// Reverse Point: Reverses the Playback Direction
	Reverse,
};

UENUM()
enum class EAvaMarkDirection : uint8
{
	Both,
	Forwards,
	Backwards,
};

UENUM()
enum class EAvaMarkSearchDirection : uint8
{
	//Search All Directions
	All,

	//Search in the Same Direction of Playback
	SameDirection,

	//Search Opposite in the opposite Direction of Playback (what causes Loops)
	OppositeDirection,

	//Forwards Fixed Direction, regardless of Playback Direction
	AbsoluteForwards,

	//Backwards Fixed Direction, regardless of Playback Direction
	AbsoluteBackwards,
};

enum class EAvaMarkRoleReply : uint8
{
	NotExecuted = 0,
	Executed    = 1 << 0,
};
ENUM_CLASS_FLAGS(EAvaMarkRoleReply);
