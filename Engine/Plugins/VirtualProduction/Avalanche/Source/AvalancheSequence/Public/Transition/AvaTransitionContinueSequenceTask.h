// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionSequenceTask.h"
#include "AvaTransitionContinueSequenceTask.generated.h"

USTRUCT(DisplayName = "Continue Sequence", Category="Sequence Playback")
struct AVALANCHESEQUENCE_API FAvaTransitionContinueSequenceTask : public FAvaTransitionSequenceTask
{
	GENERATED_BODY()

	//~ Begin FStateTreeNodeBase
#if WITH_EDITOR
	virtual FText GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const override;
#endif
	//~ End FStateTreeNodeBase

	//~ Begin FAvaTransitionSequenceTaskBase
	virtual TArray<UAvaSequencePlayer*> ExecuteSequenceTask(FStateTreeExecutionContext& InContext) const override;
	//~ End FAvaTransitionSequenceTaskBase
};
