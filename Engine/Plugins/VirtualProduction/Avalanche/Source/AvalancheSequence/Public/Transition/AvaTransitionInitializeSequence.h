// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequenceShared.h"
#include "AvaTransitionSequenceTask.h"
#include "AvaTransitionInitializeSequence.generated.h"

USTRUCT()
struct FAvaTransitionInitSequenceTaskInstanceData : public FAvaTransitionSequenceTaskBaseInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Parameter")
	FAvaSequenceTime InitializeTime = FAvaSequenceTime(0.0);

	UPROPERTY(EditAnywhere, Category = "Parameter")
	EAvaSequencePlayMode PlayMode = EAvaSequencePlayMode::Forward;
};

USTRUCT(DisplayName = "Initialize Sequence", Category = "Sequence Playback")
struct AVALANCHESEQUENCE_API FAvaTransitionInitializeSequence : public FAvaTransitionSequenceTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaTransitionInitSequenceTaskInstanceData;

	//~ Begin FStateTreeNodeBase
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	//~ End FStateTreeNodeBase

	//~ Begin FAvaTransitionSequenceTaskBase
	virtual EAvaTransitionSequenceWaitType GetWaitType(FStateTreeExecutionContext& InContext) const override;
	virtual TArray<UAvaSequencePlayer*> ExecuteSequenceTask(FStateTreeExecutionContext& InContext) const override;
	//~ End FAvaTransitionSequenceTaskBase
};
