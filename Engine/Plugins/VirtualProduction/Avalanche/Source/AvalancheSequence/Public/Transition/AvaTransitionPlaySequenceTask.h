// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequenceShared.h"
#include "AvaTransitionSequenceTask.h"
#include "AvaTransitionPlaySequenceTask.generated.h"

USTRUCT()
struct FAvaTransitionPlaySequenceTaskInstanceData : public FAvaTransitionSequenceTaskInstanceData
{
	GENERATED_BODY()

	/** Sequence Play Settings */
	UPROPERTY(EditAnywhere, Category="Parameter")
	FAvaSequencePlayParams PlaySettings;
};

USTRUCT(DisplayName = "Play Sequence", Category="Sequence Playback")
struct AVALANCHESEQUENCE_API FAvaTransitionPlaySequenceTask : public FAvaTransitionSequenceTask
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaTransitionPlaySequenceTaskInstanceData;

	//~ Begin FStateTreeNodeBase
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
#if WITH_EDITOR
	virtual FText GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const override;
#endif
	//~ End FStateTreeNodeBase

	//~ Begin FAvaTransitionSequenceTaskBase
	virtual TArray<UAvaSequencePlayer*> ExecuteSequenceTask(FStateTreeExecutionContext& InContext) const override;
	//~ End FAvaTransitionSequenceTaskBase
};
