// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionSequenceEnums.generated.h"

UENUM(BlueprintType, DisplayName="Motion Design Sequence Wait Type")
enum class EAvaTransitionSequenceWaitType : uint8
{
	None UMETA(Hidden),

	/** Fire and Forget */
	NoWait,

	/** Wait until the Sequence stops midway or finishes to complete */
	WaitUntilStop,
};

UENUM(BlueprintType, DisplayName="Motion Design Sequence Query Type")
enum class EAvaTransitionSequenceQueryType : uint8
{
	None UMETA(Hidden),

	/** Identify the Sequence by its Name */
	Name,

	/** Identify the Sequence by its Tag */
	Tag,
};
