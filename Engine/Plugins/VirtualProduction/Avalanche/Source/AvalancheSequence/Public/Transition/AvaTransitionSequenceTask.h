// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionSequenceTaskBase.h"
#include "AvaTransitionSequenceTask.generated.h"

USTRUCT()
struct FAvaTransitionSequenceTaskInstanceData : public FAvaTransitionSequenceTaskBaseInstanceData
{
	GENERATED_BODY()

	/** The wait type before this task completes */
	UPROPERTY(EditAnywhere, Category="Parameter")
	EAvaTransitionSequenceWaitType WaitType = EAvaTransitionSequenceWaitType::WaitUntilStop;
};

/** Base Task but with additional Parameters */
USTRUCT(meta=(Hidden))
struct AVALANCHESEQUENCE_API FAvaTransitionSequenceTask : public FAvaTransitionSequenceTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaTransitionSequenceTaskInstanceData;

	//~ Begin FStateTreeNodeBase
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	//~ End FStateTreeNodeBase

	//~ Begin FAvaTransitionSequenceTaskBase
	virtual EAvaTransitionSequenceWaitType GetWaitType(FStateTreeExecutionContext& InContext) const override;
	//~ End FAvaTransitionSequenceTaskBase
};
