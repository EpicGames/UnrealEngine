// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequenceName.h"
#include "AvaSequenceShared.h"
#include "AvaTagHandle.h"
#include "AvaTransitionSequenceEnums.h"
#include "Tasks/AvaTransitionTask.h"
#include "AvaTransitionSequenceTaskBase.generated.h"

class IAvaSequencePlaybackObject;
class UAvaSequence;
class UAvaSequencePlayer;
class UAvaSequenceSubsystem;

USTRUCT()
struct FAvaTransitionSequenceInstanceData
{
	GENERATED_BODY()

	/** Sequences being played */
	UPROPERTY()
	TArray<TWeakObjectPtr<UAvaSequence>> ActiveSequences;
};

USTRUCT()
struct FAvaTransitionSequenceTaskBaseInstanceData : public FAvaTransitionSequenceInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Parameter", DisplayName="Sequence Query Type", meta=(DisplayPriority=1))
	EAvaTransitionSequenceQueryType QueryType = EAvaTransitionSequenceQueryType::Name;

	UPROPERTY(EditAnywhere, Category="Parameter", meta=(DisplayPriority=1, EditCondition="QueryType==EAvaTransitionSequenceQueryType::Name", EditConditionHides))
	FAvaSequenceName SequenceName;

	UPROPERTY(EditAnywhere, Category="Parameter", meta=(DisplayPriority=1, EditCondition="QueryType==EAvaTransitionSequenceQueryType::Tag", EditConditionHides))
	FAvaTagHandle SequenceTag;

	UPROPERTY()
	bool bPerformExactMatch = false;
};

USTRUCT(meta=(Hidden))
struct AVALANCHESEQUENCE_API FAvaTransitionSequenceTaskBase : public FAvaTransitionTask
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaTransitionSequenceTaskBaseInstanceData;

	/**
	 * Execute the Sequence Task (overriden by implementation)
	 * @return the sequence players that are relevant to the task
	 */
	virtual TArray<UAvaSequencePlayer*> ExecuteSequenceTask(FStateTreeExecutionContext& InContext) const;

	/** Gets the Wait Type to use when Waiting for Active Sequences */
	virtual EAvaTransitionSequenceWaitType GetWaitType(FStateTreeExecutionContext& InContext) const
	{
		return EAvaTransitionSequenceWaitType::None;
	}

	/** Determines whether the Current Sequence information is valid for query */
	bool IsSequenceQueryValid(const FInstanceDataType& InInstanceData) const;

	/** Attempts to Retrieve the Playback Object from the given Execution Context */
	IAvaSequencePlaybackObject* GetPlaybackObject(FStateTreeExecutionContext& InContext) const;

	/** Gets all the Sequences from the provided Sequence Players that are Active (playing) */
	TArray<TWeakObjectPtr<UAvaSequence>> GetActiveSequences(TConstArrayView<UAvaSequencePlayer*> InSequencePlayers) const;

	/**
	 * Helper function to determine the Tree Run Status by updating and checking if all activated Sequence Players
	 * are in a state that match the Wait Type
	 * @param InContext the execution context where the Instance Data of the Active Sequences is retrieved
	 * @return the suggested tree run status
	 */
	EStateTreeRunStatus WaitForActiveSequences(FStateTreeExecutionContext& InContext) const;

	/**
	 * Helper function to stop all the currently active sequences
	 * @param InContext the execution context where the Instance Data of the Active Sequences is retrieved
	 */
	void StopActiveSequences(FStateTreeExecutionContext& InContext) const;

	FText GetSequenceQueryText(const FInstanceDataType& InInstanceData, EStateTreeNodeFormatting InFormatting) const;

	//~ Begin FStateTreeTaskBase
	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& InContext, const FStateTreeTransitionResult& InTransition) const override;
	virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& InContext, const float InDeltaTime) const override;
	virtual void ExitState(FStateTreeExecutionContext& InContext, const FStateTreeTransitionResult& InTransition) const override;
	//~ End FStateTreeTaskBase

	//~ Begin FStateTreeNodeBase
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	virtual bool Link(FStateTreeLinker& InLinker) override;
	//~ End FStateTreeNodeBase

	TStateTreeExternalDataHandle<UAvaSequenceSubsystem> SequenceSubsystemHandle;
};
