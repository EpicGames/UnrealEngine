// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionSequenceEnums.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "UObject/WeakObjectPtrFwd.h"

class IAvaSequencePlaybackObject;
class UAvaSequence;
class UAvaSequencePlayer;
class UAvaSequenceSubsystem;
enum class EStateTreeRunStatus : uint8;
struct FAvaTransitionContext;

struct FAvaTransitionSequenceUtils
{
	/** Attempts to Retrieve the Playback Object from the given Execution Context */
	static AVALANCHESEQUENCE_API IAvaSequencePlaybackObject* GetPlaybackObject(const FAvaTransitionContext& InTransitionContext
		, const UAvaSequenceSubsystem& InSequenceSubsystem);

	/** Retrieves the Player Run Status and Updates the Sequence Array for a given Wait Type */
	static AVALANCHESEQUENCE_API EStateTreeRunStatus UpdatePlayerRunStatus(const IAvaSequencePlaybackObject& InPlaybackObject
		, TArray<TWeakObjectPtr<UAvaSequence>>& InOutSequences
		, EAvaTransitionSequenceWaitType InWaitType);

	/** Retrieves an array of Weak Sequences based on a list of Sequence Players */
	static AVALANCHESEQUENCE_API TArray<TWeakObjectPtr<UAvaSequence>> GetSequences(TConstArrayView<UAvaSequencePlayer*> InSequencePlayers);
};
