// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionSequenceTask.h"
#include "Tasks/AvaTransitionTask.h"
#include "AvaTransitionWaitForAllSequencesTask.generated.h"

USTRUCT(DisplayName = "Wait for all Sequences", Category="Sequence Playback")
struct AVALANCHESEQUENCE_API FAvaTransitionWaitForAllSequencesTask : public FAvaTransitionTask
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaTransitionSequenceInstanceData;

	//~ Begin FStateTreeTaskBase
	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& InContext, const FStateTreeTransitionResult& InTransition) const override;
	virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& InContext, const float InDeltaTime) const override;
	//~ End FStateTreeTaskBase

	//~ Begin FStateTreeNodeBase
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	virtual bool Link(FStateTreeLinker& InLinker) override;
	//~ End FStateTreeNodeBase

	IAvaSequencePlaybackObject* GetPlaybackObject(FStateTreeExecutionContext& InContext) const;

	EStateTreeRunStatus WaitForAllSequences(FStateTreeExecutionContext& InContext) const;

	TStateTreeExternalDataHandle<UAvaSequenceSubsystem> SequenceSubsystemHandle;
};
