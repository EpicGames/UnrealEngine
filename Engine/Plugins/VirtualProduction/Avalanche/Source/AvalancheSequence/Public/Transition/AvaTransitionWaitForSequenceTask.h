// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionSequenceTask.h"
#include "AvaTransitionWaitForSequenceTask.generated.h"

USTRUCT(DisplayName = "Wait for Sequence", Category="Sequence Playback")
struct AVALANCHESEQUENCE_API FAvaTransitionWaitForSequenceTask : public FAvaTransitionSequenceTask
{
	GENERATED_BODY()

	//~ Begin FStateTreeNodeBase
#if WITH_EDITOR
	virtual FText GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const override;
#endif
	//~ End FStateTreeNodeBase

	//~ Begin FAvaTransitionSequenceTaskBase
	virtual TArray<UAvaSequencePlayer*> ExecuteSequenceTask(FStateTreeExecutionContext& InContext) const override;
	//~ End FAvaTransitionSequenceTaskBase
};
