// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheSequencer : ModuleRules
{
	public AvalancheSequencer(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AvalancheSequence",
				"AvalancheTag",
				"Core",
				"CoreUObject",
				"MovieScene",
				"Sequencer",
				"SlateCore",
				"UnrealEd",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"Avalanche",
				"AvalancheCore",
				"AvalancheEditorCore",
				"AvalancheOutliner",
				"AvalancheTransition",
				"AvalancheTransitionEditor",
				"BlueprintGraph",
				"ClassViewer",
				"CustomDetailsView",
				"DeveloperSettings",
				"EditorSubsystem",
				"Engine",
				"GeometryCacheTracks",
				"InputCore",
				"Json",
				"JsonUtilities",
				"KismetCompiler",
				"LevelSequence",
				"MediaCompositing",
				"MediaCompositingEditor",
				"MovieSceneTools",
				"MovieSceneTracks",
				"Projects",
				"SceneOutliner",
				"SequencerCore",
				"Settings",
				"Slate",
				"StateTreeEditorModule",
				"StateTreeModule",
				"TimeManagement",
				"ToolMenus",
				"ToolWidgets",
				"UMG",
			}
		);

		if (Target.Version.MinorVersion >= 2)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"GeometryCacheSequencer",
				}
			);
		}
		else
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"GeometryCache",
				}
			);
		}
	}
}
