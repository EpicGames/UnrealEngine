// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceDefaultTags.h"
#include "AvaTag.h"
#include "AvaTagCollection.h"
#include "UObject/SoftObjectPtr.h"

const FAvaSequenceDefaultTags& FAvaSequenceDefaultTags::Get()
{
	static const FAvaSequenceDefaultTags DefaultTags;
	return DefaultTags;
}

FAvaSequenceDefaultTags::FAvaSequenceDefaultTags()
{
	TSoftObjectPtr<UAvaTagCollection> TagCollection(FSoftObjectPath(TEXT("/Avalanche/Tags/DefaultSequenceTags.DefaultSequenceTags")));

	In     = FAvaTagSoftHandle(TagCollection, FAvaTagId(FGuid(0x42E03738, 0x41702862, 0x08B6C195, 0x897C5ED5)));
	Out    = FAvaTagSoftHandle(TagCollection, FAvaTagId(FGuid(0xD9F19093, 0x483551E9, 0x9B69D08F, 0xEFCD0763)));
	Change = FAvaTagSoftHandle(TagCollection, FAvaTagId(FGuid(0xA5C8CC77, 0x4CDD1700, 0x226A32A2, 0xD58A039B)));
}
