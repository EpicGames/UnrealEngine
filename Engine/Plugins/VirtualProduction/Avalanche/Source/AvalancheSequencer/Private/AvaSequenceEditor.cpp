// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceEditor.h"
#include "AvaSequence.h"
#include "Director/AvaSequenceDirector.h"
#include "Director/AvaSequenceDirectorBlueprint.h"
#include "Director/AvaSequenceDirectorGeneratedClass.h"
#include "IAvaSequenceProvider.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Modules/ModuleManager.h"

bool FAvaSequenceEditor::CanCreateEvents(UMovieSceneSequence* InSequence) const
{
	return true;
}

bool FAvaSequenceEditor::CanCreateDirectorBlueprint(UMovieSceneSequence* Sequence) const
{
#if WITH_EDITOR
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
	const TSharedPtr<IClassViewerFilter>& GlobalClassFilter = ClassViewerModule.GetGlobalClassViewerFilter();
	TSharedRef<FClassViewerFilterFuncs> ClassFilterFuncs = ClassViewerModule.CreateFilterFuncs();
	FClassViewerInitializationOptions ClassViewerOptions = {};

	if (GlobalClassFilter.IsValid())
	{
		return GlobalClassFilter->IsClassAllowed(ClassViewerOptions, UBlueprint::StaticClass(), ClassFilterFuncs);
	}
#endif
	return true;
}

UBlueprint* FAvaSequenceEditor::GetBlueprintForSequence(UMovieSceneSequence* InSequence) const
{
	UAvaSequence* const Sequence = CastChecked<UAvaSequence>(InSequence);
	if (!Sequence)
	{
		return nullptr;
	}

	UBlueprint* DirectorBlueprint = nullptr;

	// Opportunity for the Provider to return a custom Blueprint instead of the Sequence Director BP
	IAvaSequenceProvider* const Provider = Sequence->GetSequenceProvider();
	if (Provider && Provider->GetDirectorBlueprint(*Sequence, DirectorBlueprint))
	{
		return DirectorBlueprint;
	}

	DirectorBlueprint = Sequence->GetDirectorBlueprint();
	return DirectorBlueprint;
}

UBlueprint* FAvaSequenceEditor::CreateBlueprintForSequence(UMovieSceneSequence* InSequence) const
{
	UBlueprint* Blueprint = GetBlueprintForSequence(InSequence);

	if (!ensureMsgf(!Blueprint, TEXT("Should not call CreateBlueprintForSequence when one already exists")))
	{
		return Blueprint;
	}

	UAvaSequence* const Sequence = CastChecked<UAvaSequence>(InSequence);

	Blueprint = FKismetEditorUtilities::CreateBlueprint(UAvaSequenceDirector::StaticClass()
		, InSequence
		, *Sequence->GetDirectorBlueprintName()
		, BPTYPE_Normal
		, UAvaSequenceDirectorBlueprint::StaticClass()
		, UAvaSequenceDirectorGeneratedClass::StaticClass());

	Blueprint->ClearFlags(RF_Public | RF_Standalone);

	Sequence->SetDirectorBlueprint(Blueprint);

	return Blueprint;
}
