// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSequenceEditor.h"

struct FAvaSequenceEditor : FMovieSceneSequenceEditor
{
private:
	//~ Begin FMovieSceneSequenceEditor
	virtual bool CanCreateEvents(UMovieSceneSequence* InSequence) const override;
	virtual bool CanCreateDirectorBlueprint(UMovieSceneSequence* InSequence) const override;
	virtual UBlueprint* GetBlueprintForSequence(UMovieSceneSequence* InSequence) const override;
	virtual UBlueprint* CreateBlueprintForSequence(UMovieSceneSequence* InSequence) const override;
	//~ End FMovieSceneSequenceEditor
};
