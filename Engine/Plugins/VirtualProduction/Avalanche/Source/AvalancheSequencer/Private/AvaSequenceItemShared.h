// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Delegates/MulticastDelegateBase.h"
#include "Templates/SharedPointer.h"

class IAvaSequenceItem;

using FAvaSequenceItemPtr = TSharedPtr<IAvaSequenceItem>;

/*
 * Delegate Wrapper so that things other than IAvaSequenceItem can only Add or Remove Listeners but not call Broadcast
 */
template<typename InMulticastDelegateType>
class TAvaSequenceItemDelegate
{
	friend IAvaSequenceItem;

public:
	FDelegateHandle AddListener(const typename InMulticastDelegateType::FDelegate& InDelegate)
	{
		return MulticastDelegate.Add(InDelegate);
	}

	bool  RemoveListener(const FDelegateHandle& InDelegateHandle)
	{
		return MulticastDelegate.Remove(InDelegateHandle);
	}

	int32 RemoveListener(FDelegateUserObjectConst InUserObject)
	{
		return MulticastDelegate.RemoveAll(InUserObject);
	}

private:
	void Broadcast()
	{
		MulticastDelegate.Broadcast();
	}

	InMulticastDelegateType MulticastDelegate;
};
