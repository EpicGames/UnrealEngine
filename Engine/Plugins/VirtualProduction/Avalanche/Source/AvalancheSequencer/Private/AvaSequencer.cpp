// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequencer.h"
#include "AvaEditorCoreStyle.h"
#include "AvaSequence.h"
#include "AvaSequenceActor.h"
#include "AvaSequencePlaybackObject.h"
#include "AvaSequencePlayer.h"
#include "AvaSequencerArgs.h"
#include "AvaSequencerSubsystem.h"
#include "AvaSequencerUtils.h"
#include "Clipboard/AvaSequenceExporter.h"
#include "Clipboard/AvaSequenceImporter.h"
#include "Commands/AvaSequencerAction.h"
#include "Commands/AvaSequencerCommands.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "ContentBrowserModule.h"
#include "CoreGlobals.h"
#include "DetailsView/SAvaMarkDetails.h"
#include "DetailsView/Section/AvaSequencePlaybackDetails.h"
#include "DetailsView/Section/AvaSequenceSettingsDetails.h"
#include "DetailsView/Section/AvaSequenceTreeDetails.h"
#include "Editor.h"
#include "Editor/Sequencer/Private/Sequencer.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IAvaSequenceProvider.h"
#include "IContentBrowserSingleton.h"
#include "ISequencer.h"
#include "ISequencerModule.h"
#include "ISequencerTrackEditor.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/Views/SOutlinerView.h"
#include "Misc/MessageDialog.h"
#include "Misc/NotifyHook.h"
#include "MovieScene.h"
#include "Playback/AvaSequencerCleanView.h"
#include "Playback/AvaSequencerController.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"
#include "Selection/AvaEditorSelection.h"
#include "SequenceTree/AvaSequenceItem.h"
#include "SequenceTree/Columns/AvaSequenceNameColumn.h"
#include "SequenceTree/Columns/AvaSequenceStatusColumn.h"
#include "SequenceTree/IAvaSequenceItem.h"
#include "SequenceTree/Widgets/SAvaSequenceTree.h"
#include "SequencerCommands.h"
#include "SequencerSettings.h"
#include "SequencerUtilities.h"
#include "Settings/AvaSequencerSettings.h"
#include "Sidebar/SidebarDrawerConfig.h"
#include "StaggerTool/AvaStaggerTool.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Widgets/Views/SHeaderRow.h"

DEFINE_LOG_CATEGORY_STATIC(LogAvaSequencer, Log, All);

#define LOCTEXT_NAMESPACE "AvaSequencer"

namespace UE::AvaSequencer::Private
{
	void SetObjectSelection(USelection* InSelection, const TArray<UObject*>& InObjects, bool bIsTransactional)
	{
		if (!InSelection)
		{
			return;
		}

		if (bIsTransactional)
		{
			InSelection->Modify();
		}
		InSelection->BeginBatchSelectOperation();
		InSelection->DeselectAll();

		for (UObject* const Object : InObjects)
		{
			InSelection->Select(Object);
		}

		InSelection->EndBatchSelectOperation(bIsTransactional);
	}
	
	struct FScopedSelection
	{
		FScopedSelection(USelection* InTargetSelection, USelection* InSourceSelection)
			: TargetSelection(InTargetSelection)
		{
			check(InTargetSelection && InSourceSelection);

			TArray<UObject*> SourceSelectedObjects;
			InSourceSelection->GetSelectedObjects(SourceSelectedObjects);

			TargetSelection->GetSelectedObjects(OriginalTargetSelectedObjects);
			SetObjectSelection(TargetSelection, SourceSelectedObjects, false);
		}

		~FScopedSelection()
		{
			check(TargetSelection);
			SetObjectSelection(TargetSelection, OriginalTargetSelectedObjects, false);
		}

	private:
		/** The Target we're temporarily selecting and mirroring Source Selection */
		USelection* TargetSelection;

		/** Cached selected objects of Target */
		TArray<UObject*> OriginalTargetSelectedObjects;
	};

	struct FRenameBindingParams
	{
		/** The shared playback state */
		TSharedRef<MovieScene::FSharedPlaybackState> PlaybackState;
		/** The state to assign sequences and find object ids */
		FMovieSceneEvaluationState* EvaluationState;
		/** Describes the hierarchy of the sequence (how sub-sequences are within a root sequence) */
		const FMovieSceneSequenceHierarchy* SequenceHierarchy;
	};
	struct FRenameBindingSequenceParams
	{
		/** currently processed sequence. Starts with root */
		UMovieSceneSequence* Sequence;
		/** id of the currently processed sequence. Starts with root id */
		FMovieSceneSequenceIDRef SequenceId;
	};
	/** Renames all bindings of the given actor to its updated actor label */
	void RenameBindingRecursive(AActor* InActor, const FRenameBindingParams& InParams, const FRenameBindingSequenceParams& InSequenceParams)
	{
		// check parameters that should be guaranteed valid by caller
		check(InActor && InParams.SequenceHierarchy && InParams.EvaluationState);

		if (!InSequenceParams.Sequence)
		{
			return;
		}

		UMovieScene* const MovieScene = InSequenceParams.Sequence->GetMovieScene();
		if (!MovieScene)
		{
			return;
		}

		InParams.EvaluationState->AssignSequence(InSequenceParams.SequenceId, *InSequenceParams.Sequence, InParams.PlaybackState);

		const FGuid ObjectId = InParams.EvaluationState->FindObjectId(*InActor, InSequenceParams.SequenceId, InParams.PlaybackState);
		if (ObjectId.IsValid())
		{
			if (FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectId))
			{
				InSequenceParams.Sequence->Modify();
				Possessable->SetName(InActor->GetActorLabel());
			}
			else if (FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(ObjectId))
			{
				InSequenceParams.Sequence->Modify();
				Spawnable->SetName(InActor->GetActorLabel());
			}
		}

		// Recurse into child nodes
		if (const FMovieSceneSequenceHierarchyNode* Node = InParams.SequenceHierarchy->FindNode(InSequenceParams.SequenceId))
		{
			for (FMovieSceneSequenceIDRef ChildId : Node->Children)
			{
				if (const FMovieSceneSubSequenceData* SubData = InParams.SequenceHierarchy->FindSubData(ChildId))
				{
					const FRenameBindingSequenceParams ChildSequenceParams
						{
							.Sequence = SubData->GetSequence(),
							.SequenceId = ChildId
						};
					RenameBindingRecursive(InActor, InParams, ChildSequenceParams);
				}
			}
		}
	}
} // UE::AvaSequencer::Private

const FName FAvaSequencer::SidebarDrawerId = TEXT("MotionDesign");

FAvaSequencer::FAvaSequencer(IAvaSequencerProvider& InProvider, FAvaSequencerArgs&& InArgs)
	: Provider(InProvider)
	, CommandList(MakeShared<FUICommandList>())
	, CleanView(MakeShared<FAvaSequencerCleanView>())
	, bUseCustomCleanPlaybackMode(InArgs.bUseCustomCleanPlaybackMode)
	, bCanProcessSequencerSelections(InArgs.bCanProcessSequencerSelections)
{
	SequencerController = MoveTemp(InArgs.SequencerController);

	SequencerActions =
		{
			MakeShared<FAvaStaggerTool>(*this),
		};

	BindCommands();

	OnSequenceEditUndoHandle = UAvaSequence::OnSequenceEditUndo().AddRaw(this, &FAvaSequencer::OnSequenceEditUndo);

	OnSequenceStartedHandle = UAvaSequencePlayer::OnSequenceStarted().AddLambda(
		[this](UAvaSequencePlayer*, UAvaSequence*){ return NotifyOnSequencePlayed(); });

	OnSequenceFinishedHandle = UAvaSequencePlayer::OnSequenceFinished().AddLambda(
		[this](UAvaSequencePlayer*, UAvaSequence*){ return NotifyOnSequenceStopped(); });

	OnActorLabelChangedHandle = FCoreDelegates::OnActorLabelChanged.AddRaw(this, &FAvaSequencer::OnActorLabelChanged);

	// Register sequencer menu extenders.
	ISequencerModule& SequencerModule = FAvaSequencerUtils::GetSequencerModule();
	{
		const int32 NewIndex = SequencerModule.GetAddTrackMenuExtensibilityManager()->GetExtenderDelegates().Add(
			FAssetEditorExtender::CreateRaw(this, &FAvaSequencer::GetAddTrackSequencerExtender));

		SequencerAddTrackExtenderHandle = SequencerModule.GetAddTrackMenuExtensibilityManager()->GetExtenderDelegates()[NewIndex].GetHandle();

		SidebarExtender = MakeShared<FExtender>();

		SidebarExtender->AddMenuExtension(
			TEXT("MarkedFrames"),
			EExtensionHook::After,
			CommandList,
			FMenuExtensionDelegate::CreateRaw(this, &FAvaSequencer::ExtendSidebarMarkedFramesMenu));

		SequencerModule.GetSidebarExtensibilityManager()->AddExtender(SidebarExtender);
	}

	// Register to update when an undo/redo operation has been called to update our list of items
	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
}

FAvaSequencer::~FAvaSequencer()
{
	UAvaSequence::OnSequenceEditUndo().Remove(OnSequenceEditUndoHandle);
	OnSequenceEditUndoHandle.Reset();

	UAvaSequencePlayer::OnSequenceStarted().Remove(OnSequenceStartedHandle);
	OnSequenceStartedHandle.Reset();

	UAvaSequencePlayer::OnSequenceFinished().Remove(OnSequenceFinishedHandle);
	OnSequenceFinishedHandle.Reset();

	FCoreDelegates::OnActorLabelChanged.Remove(OnActorLabelChangedHandle);
	OnActorLabelChangedHandle.Reset();

	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
	
	if (FAvaSequencerUtils::IsSequencerModuleLoaded())
	{
		ISequencerModule& SequencerModule = FAvaSequencerUtils::GetSequencerModule();
		SequencerModule.GetAddTrackMenuExtensibilityManager()->GetExtenderDelegates().RemoveAll(
			[this](const FAssetEditorExtender& Extender)
			{
				return SequencerAddTrackExtenderHandle == Extender.GetHandle();
			});
	}

	if (const TSharedPtr<ISequencer> Sequencer = SequencerWeak.Pin())
	{
		Sequencer->UnregisterDrawerSection(SidebarDrawerId, FAvaSequenceTreeDetails::UniqueId);
		Sequencer->UnregisterDrawerSection(SidebarDrawerId, FAvaSequencePlaybackDetails::UniqueId);
		Sequencer->UnregisterDrawerSection(SidebarDrawerId, FAvaSequenceSettingsDetails::UniqueId);

		Sequencer->UnregisterDrawer(SidebarDrawerId);

		if (SidebarExtender.IsValid())
		{
			ISequencerModule& SequencerModule = FAvaSequencerUtils::GetSequencerModule();

			SequencerModule.GetSidebarExtensibilityManager()->RemoveExtender(SidebarExtender);

			SidebarExtender.Reset();
		}
	}
}

void FAvaSequencer::BindCommands()
{
	const FAvaSequencerCommands& AvaSequencerCommands = FAvaSequencerCommands::Get();

	for (const TSharedRef<FAvaSequencerAction>& SequencerAction : SequencerActions)
	{
		SequencerAction->MapAction(CommandList);
	}

	CommandList->MapAction(AvaSequencerCommands.ApplyCurrentState
		, FExecuteAction::CreateRaw(this, &FAvaSequencer::ApplyCurrentState));

	CommandList->MapAction(AvaSequencerCommands.FixBindingPaths
		, FExecuteAction::CreateRaw(this, &FAvaSequencer::FixBindingPaths));

	CommandList->MapAction(AvaSequencerCommands.FixInvalidBindings
		, FExecuteAction::CreateRaw(this, &FAvaSequencer::FixInvalidBindings));

	CommandList->MapAction(AvaSequencerCommands.FixBindingHierarchy
		, FExecuteAction::CreateRaw(this, &FAvaSequencer::FixBindingHierarchy));

	const FSequencerCommands& SequencerCommands = FSequencerCommands::Get();

	CommandList->MapAction(SequencerCommands.AddTransformKey
		, FExecuteAction::CreateRaw(this, &FAvaSequencer::AddTransformKey, EMovieSceneTransformChannel::All));

	CommandList->MapAction(SequencerCommands.AddTranslationKey
		, FExecuteAction::CreateRaw(this, &FAvaSequencer::AddTransformKey, EMovieSceneTransformChannel::Translation));

	CommandList->MapAction(SequencerCommands.AddRotationKey
		, FExecuteAction::CreateRaw(this, &FAvaSequencer::AddTransformKey, EMovieSceneTransformChannel::Rotation));

	CommandList->MapAction(SequencerCommands.AddScaleKey
		, FExecuteAction::CreateRaw(this, &FAvaSequencer::AddTransformKey, EMovieSceneTransformChannel::Scale));
}

TSharedPtr<IAvaSequenceColumn> FAvaSequencer::FindSequenceColumn(FName InColumnName) const
{
	if (const TSharedPtr<IAvaSequenceColumn>* const FoundColumn = SequenceColumns.Find(InColumnName))
	{
		return *FoundColumn;
	}
	
	return nullptr;
}

void FAvaSequencer::EnsureSequencer()
{
	TSharedPtr<ISequencer> Sequencer = SequencerWeak.Pin();
	if (Sequencer.IsValid())
	{
		return;
	}

	// Instantiate Sequencer Controller first so it Ticks before FSequencer
	if (!SequencerController.IsValid())
	{
		SequencerController = MakeShared<FAvaSequencerController>();
	}

	Sequencer = Provider.GetExternalSequencer();

	// External Implementation could call GetSequencer again (e.g. to get the underlying sequencer widget),
	// so need to give priority to that call and initialize from there.
	// If this is the case, SequencerWeak is now initialized/valid and should return early to avoid double init.
	if (SequencerWeak.IsValid())
	{
		return;
	}

	if (Sequencer.IsValid())
	{
		checkf(Sequencer.GetSharedReferenceCount() > 1
			, TEXT("IAvaSequencerProvider::GetExternalSequencer should return a sequencer and hold reference to it"));
	}
	else
	{
		// Create Sequencer if one was not provided
		Sequencer = CreateSequencer();
		check(Sequencer.IsValid());
	}

	SequencerWeak = Sequencer;

	SequencerController->SetSequencer(Sequencer);

	GetDefaultSequence();

	InitSequencerCommandList();

	// Register Events
	Sequencer->OnActivateSequence().AddSP(this, &FAvaSequencer::OnActivateSequence);
	Sequencer->OnPlayEvent().AddSP(this, &FAvaSequencer::NotifyOnSequencePlayed);
	Sequencer->OnStopEvent().AddSP(this, &FAvaSequencer::NotifyOnSequenceStopped);
	Sequencer->OnMovieSceneBindingsPasted().AddSP(this, &FAvaSequencer::OnMovieSceneBindingsPasted);
	Sequencer->GetSelectionChangedObjectGuids().AddSP(this, &FAvaSequencer::OnSequencerSelectionChanged);
	Sequencer->OnGetIsBindingVisible().BindSP(this, &FAvaSequencer::IsBindingSelected);
	Sequencer->OnCameraCut().AddSP(this, &FAvaSequencer::OnUpdateCameraCut);
	Sequencer->OnCloseEvent().AddSP(this, &FAvaSequencer::OnSequencerClosed);

	if (IAvaSequenceProvider* const SequenceProvider = Provider.GetSequenceProvider())
	{
		SequenceProvider->OnEditorSequencerCreated(Sequencer);

		for (const TSharedRef<FAvaSequencerAction>& SequencerAction : SequencerActions)
		{
			SequencerAction->OnSequencerCreated();
		}
	}
}

TSharedRef<ISequencer> FAvaSequencer::CreateSequencer()
{
	const UAvaSequencerSettings* SequencerSettings = GetDefault<UAvaSequencerSettings>();

	// Configure Init Params
	FSequencerInitParams SequencerInitParams;
	{
		UAvaSequence* const DefaultSequence = GetDefaultSequence();
		SetViewedSequence(DefaultSequence);
		ensure(GetViewedSequence() == DefaultSequence);

		SequencerInitParams.RootSequence           = GetViewedSequence();
		SequencerInitParams.bEditWithinLevelEditor = false;
		SequencerInitParams.ToolkitHost            = Provider.GetSequencerToolkitHost();
		SequencerInitParams.PlaybackContext.Bind(this, &FAvaSequencer::GetPlaybackContext);

		SequencerInitParams.ViewParams.UniqueName      = SequencerSettings->GetName();
		SequencerInitParams.ViewParams.ScrubberStyle   = ESequencerScrubberStyle::FrameBlock;
		SequencerInitParams.ViewParams.ToolbarExtender = MakeShared<FExtender>();

		// Host Capabilities
		SequencerInitParams.HostCapabilities.bSupportsCurveEditor = true;
		SequencerInitParams.HostCapabilities.bSupportsSaveMovieSceneAsset = false;
		SequencerInitParams.HostCapabilities.bSupportsSidebar = true;
	};

	InstancedSequencer = FAvaSequencerUtils::GetSequencerModule().CreateSequencer(SequencerInitParams);

	return InstancedSequencer.ToSharedRef();
}

void FAvaSequencer::GetSelectedObjects(const TArray<FGuid>& InObjectGuids
	, TArray<UObject*>& OutSelectedActors
	, TArray<UObject*>& OutSelectedComponents
	, TArray<UObject*>& OutSelectedObjects) const
{
	TSharedPtr<ISequencer> Sequencer = GetSequencerPtr();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UMovieSceneSequence* const ActiveSequence = Sequencer->GetFocusedMovieSceneSequence();
	UObject* const PlaybackContext = Provider.GetPlaybackContext();

	TSet<UObject*> ProcessedObjects;
	// Prepare for Worst Case where all Objects are Unique
	ProcessedObjects.Reserve(InObjectGuids.Num());

	for (const FGuid& Guid : InObjectGuids)
	{
		TArrayView<TWeakObjectPtr<>> BoundObjects = ResolveBoundObjects(Guid, ActiveSequence);

		if (BoundObjects.IsEmpty())
		{
			continue;
		}

		UObject* const BoundObject = BoundObjects[0].Get();

		// Skip invalid or objects already processed
		if (!BoundObject || ProcessedObjects.Contains(BoundObject))
		{
			continue;
		}

		ProcessedObjects.Add(BoundObject);

		if (AActor* const Actor = Cast<AActor>(BoundObject))
		{
			OutSelectedActors.AddUnique(Actor);
		}
		else if (UActorComponent* ActorComponent = Cast<UActorComponent>(BoundObject))
		{
			OutSelectedComponents.AddUnique(ActorComponent);
		}
		else
		{
			OutSelectedObjects.AddUnique(BoundObject);
		}
	}
}

bool FAvaSequencer::IsBindingSelected(const FMovieSceneBinding& InBinding) const
{
	if (!ViewedSequenceWeak.IsValid())
	{
		return false;
	}

	TArrayView<TWeakObjectPtr<>> ResolvedObjects = ResolveBoundObjects(InBinding.GetObjectGuid(), ViewedSequenceWeak.Get());
	
	if (ResolvedObjects.IsEmpty())
	{
		return false;
	}

	if (FEditorModeTools* const ModeTools = Provider.GetSequencerModeTools())
	{
		UObject* const ResolvedObject = ResolvedObjects[0].Get();

		if (Cast<AActor>(ResolvedObject))
		{
			return ModeTools->GetSelectedActors()->IsSelected(ResolvedObject);
		}

		if (Cast<UActorComponent>(ResolvedObject))
		{
			return ModeTools->GetSelectedComponents()->IsSelected(ResolvedObject);
		}

		return ModeTools->GetSelectedObjects()->IsSelected(ResolvedObject);
	}

	return false;
}

void FAvaSequencer::OnSequencerSelectionChanged(TArray<FGuid> InObjectGuids)
{
	using namespace UE::AvaSequencer::Private;

	if (!bCanProcessSequencerSelections || bUpdatingSelection)
	{
		return;
	}

	TGuardValue<bool> Guard(bUpdatingSelection, true);

	TArray<UObject*> SelectedActors;
	TArray<UObject*> SelectedComponents;
	TArray<UObject*> SelectedObjects;

	GetSelectedObjects(InObjectGuids, SelectedActors, SelectedComponents, SelectedObjects);

	if (FEditorModeTools* const ModeTools = Provider.GetSequencerModeTools())
	{
		SetObjectSelection(ModeTools->GetSelectedActors(), SelectedActors, true);
		SetObjectSelection(ModeTools->GetSelectedComponents(), SelectedComponents, true);
		SetObjectSelection(ModeTools->GetSelectedObjects(), SelectedObjects, true);
	}
	
	bSelectedFromSequencer = true;
}

TSharedRef<FExtender> FAvaSequencer::GetAddTrackSequencerExtender(const TSharedRef<FUICommandList> InCommandList
	, const TArray<UObject*> InContextSensitiveObjects)
{
	TSharedRef<FExtender> AddTrackMenuExtender = MakeShared<FExtender>();
	
	AddTrackMenuExtender->AddMenuExtension(
		SequencerMenuExtensionPoints::AddTrackMenu_PropertiesSection,
		EExtensionHook::Before,
		InCommandList,
		FMenuExtensionDelegate::CreateSP(this, &FAvaSequencer::ExtendSequencerAddTrackMenu
			, InContextSensitiveObjects));
	
	return AddTrackMenuExtender;
}

void FAvaSequencer::ExtendSequencerAddTrackMenu(FMenuBuilder& OutAddTrackMenuBuilder
	, const TArray<UObject*> InContextObjects)
{
}

void FAvaSequencer::NotifyViewedSequenceChanged(UAvaSequence* InOldSequence)
{
	UAvaSequence* ViewedSequence = ViewedSequenceWeak.Get();

	Provider.OnViewedSequenceChanged(InOldSequence, ViewedSequence);

	OnViewedSequenceChanged.Broadcast(ViewedSequence);

	TSharedPtr<ISequencer> Sequencer = SequencerWeak.Pin();
	if (Sequencer.IsValid() && ViewedSequence)
	{
		Sequencer->ResetToNewRootSequence(*ViewedSequence);
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::ActiveMovieSceneChanged);
	}

	if (SequenceTree.IsValid())
	{
		SequenceTree->OnPostSetViewedSequence(ViewedSequence);
	}
}

UAvaSequence* FAvaSequencer::GetDefaultSequence() const
{
	IAvaSequenceProvider* const SequenceManager = Provider.GetSequenceProvider();
	if (!SequenceManager)
	{
		return nullptr;
	}

	if (UAvaSequence* const DefaultSequence = SequenceManager->GetDefaultSequence())
	{
		return DefaultSequence;
	}

	UAvaSequence* const NewDefaultSequence = CreateSequence();
	SequenceManager->SetDefaultSequence(NewDefaultSequence);
	return NewDefaultSequence;
}

UAvaSequence* FAvaSequencer::CreateSequence() const
{
	IAvaSequenceProvider* const SequenceProvider = Provider.GetSequenceProvider();
	if (!SequenceProvider)
	{
		return nullptr;
	}

	UObject* const Outer = SequenceProvider->ToUObject();
	if (!Outer)
	{
		return nullptr;
	}

	UAvaSequence* const Sequence = NewObject<UAvaSequence>(Outer, NAME_None, RF_Transactional);
	check(Sequence);

	UMovieScene* const MovieScene = Sequence->GetMovieScene();
	if (!ensure(MovieScene))
	{
		return Sequence;
	}

	const UAvaSequencerSettings* Settings = GetDefault<UAvaSequencerSettings>();
	if (!ensure(Settings))
	{
		return Sequence;
	}

	MovieScene->SetDisplayRate(Settings->GetDisplayRate());

	const double InTime  = Settings->GetStartTime();
	const double OutTime = Settings->GetEndTime();

	const FFrameTime InFrame  = InTime  * MovieScene->GetTickResolution();
	const FFrameTime OutFrame = OutTime * MovieScene->GetTickResolution();
	
	MovieScene->SetPlaybackRange(TRange<FFrameNumber>(InFrame.FrameNumber, OutFrame.FrameNumber+1));
	MovieScene->GetEditorData().WorkStart = InTime;
	MovieScene->GetEditorData().WorkEnd   = OutTime;
	
	return Sequence;
}

UObject* FAvaSequencer::GetPlaybackContext() const
{
	return Provider.GetPlaybackContext();
}

TSharedRef<SWidget> FAvaSequencer::GetSequenceTreeWidget()
{
	if (!SequenceTree.IsValid())
	{
		SequenceTreeHeaderRow = SNew(SHeaderRow)
			.Visibility(EVisibility::Visible)
			.CanSelectGeneratedColumn(true);

		SequenceColumns.Reset();
		SequenceTreeHeaderRow->ClearColumns();

		TArray<TSharedPtr<IAvaSequenceColumn>> Columns;
		Columns.Add(MakeShared<FAvaSequenceNameColumn>());
		Columns.Add(MakeShared<FAvaSequenceStatusColumn>());

		const TSharedPtr<FAvaSequencer> This = SharedThis(this);

		for (const TSharedPtr<IAvaSequenceColumn>& Column : Columns)
		{
			const FName ColumnId = Column->GetColumnId();
			SequenceColumns.Add(ColumnId, Column);
			SequenceTreeHeaderRow->AddColumn(Column->ConstructHeaderRowColumn());
			SequenceTreeHeaderRow->SetShowGeneratedColumn(ColumnId, true);
		}

		SequenceTree = SNew(SAvaSequenceTree, This, SequenceTreeHeaderRow);
		SequenceTreeView = SequenceTree->GetSequenceTreeView();

		// Make sure the Tree is synced to latest viewed sequence
		NotifyOnSequenceTreeChanged();
		NotifyViewedSequenceChanged(nullptr);
	}
	return SequenceTree.ToSharedRef();
}

TSharedRef<SWidget> FAvaSequencer::CreatePlayerToolBar(const TSharedRef<FUICommandList>& InCommandList)
{
	FSlimHorizontalToolBarBuilder ToolBarBuilder(InCommandList, FMultiBoxCustomization::None);
	ToolBarBuilder.SetLabelVisibility(EVisibility::Collapsed);

	const FAvaSequencerCommands& Commands = FAvaSequencerCommands::Get();

	ToolBarBuilder.AddToolBarButton(Commands.PlaySelected, NAME_None, TAttribute<FText>(), TAttribute<FText>()
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Toolbar.Play")));

	ToolBarBuilder.AddToolBarButton(Commands.ContinueSelected, NAME_None, TAttribute<FText>(), TAttribute<FText>()
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.JumpToEvent")));

	ToolBarBuilder.AddToolBarButton(Commands.StopSelected, NAME_None, TAttribute<FText>(), TAttribute<FText>()
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Toolbar.Stop")));

	return SNew(SBox)
		.HAlign(EHorizontalAlignment::HAlign_Center)
		[
			ToolBarBuilder.MakeWidget()
		];
}

void FAvaSequencer::OnSequenceSearchChanged(const FText& InSearchText, FText& OutErrorMessage)
{
	NotifyOnSequenceTreeChanged();
	
	if (!InSearchText.IsEmpty())
	{
		TTextFilter<FAvaSequenceItemPtr> TextFilter(TTextFilter<FAvaSequenceItemPtr>::FItemToStringArray::CreateLambda(
			[](FAvaSequenceItemPtr InSequence, TArray<FString>& OutFilterStrings)
			{
				OutFilterStrings.Add(InSequence->GetDisplayNameText().ToString());
			}));
		
		TextFilter.SetRawFilterText(InSearchText);
		OutErrorMessage = TextFilter.GetFilterErrorText();

		//TODO: Tree View is not accounted for here
		for (TArray<FAvaSequenceItemPtr>::TIterator Iter(RootSequenceItems); Iter; ++Iter)
		{
			const FAvaSequenceItemPtr& Item = *Iter;
			if (!Item.IsValid() || !TextFilter.PassesFilter(Item))
			{
				Iter.RemoveCurrent();
			}
		}
	}
	else
	{
		OutErrorMessage = FText::GetEmpty();
	}
}

void FAvaSequencer::OnActivateSequence(FMovieSceneSequenceIDRef InSequenceID)
{
	TSharedPtr<ISequencer> Sequencer = SequencerWeak.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	FMovieSceneRootEvaluationTemplateInstance& RootInstance = Sequencer->GetEvaluationTemplate();
	UMovieSceneSequence* Sequence = RootInstance.GetSequence(InSequenceID);

	SetViewedSequence(Cast<UAvaSequence>(Sequence));
}

void FAvaSequencer::NotifyOnSequencePlayed()
{
	if (!bUseCustomCleanPlaybackMode)
	{
		return;
	}

	if (const USequencerSettings* const SequencerSettings = GetSequencerSettings())
	{
		if (SequencerSettings->GetCleanPlaybackMode())
		{
			TArray<TWeakPtr<FEditorViewportClient>> ViewportClients;
			Provider.GetCustomCleanViewViewportClients(ViewportClients);
			CleanView->Apply(ViewportClients);
		}
		else
		{
			CleanView->Restore();
		}
	}
}

void FAvaSequencer::NotifyOnSequenceStopped()
{
	if (!bUseCustomCleanPlaybackMode)
	{
		return;
	}

	CleanView->Restore();
}

void FAvaSequencer::OnActorLabelChanged(AActor* InActor)
{
	using namespace UE::AvaSequencer;

	// Skip actors that are invalid or preview actors
	if (!InActor || InActor->bIsEditorPreviewActor)
	{
		return;
	}

	// Ignore processing label changed for PIE
	const UWorld* const World = InActor->GetWorld();
	if (World && World->IsPlayInEditor())
	{
		return;
	}

	IAvaSequenceProvider* const SequenceProvider = Provider.GetSequenceProvider();
	if (!SequenceProvider)
	{
		return;
	}

	UE::MovieScene::FSharedPlaybackStateCreateParams CreateParams;

	for (UAvaSequence* Sequence : SequenceProvider->GetSequences())
	{
		if (!Sequence || !Sequence->GetMovieScene())
		{
			continue;
		}

		FMovieSceneSequenceHierarchy SequenceHierarchy;
		UMovieSceneCompiledDataManager::CompileHierarchy(Sequence, &SequenceHierarchy, EMovieSceneServerClientMask::All);

		const TSharedRef<UE::MovieScene::FSharedPlaybackState> PlaybackState = MakeShared<UE::MovieScene::FSharedPlaybackState>(*Sequence, CreateParams);
		FMovieSceneEvaluationState EvaluationState;
		PlaybackState->AddCapabilityRaw(&EvaluationState);

		const Private::FRenameBindingParams RenameParams
			{
				.PlaybackState = PlaybackState,
				.EvaluationState = &EvaluationState,
				.SequenceHierarchy = &SequenceHierarchy,
			};

		const Private::FRenameBindingSequenceParams SequenceParams
			{
				.Sequence = Sequence,
				.SequenceId = MovieSceneSequenceID::Root,
			};
		Private::RenameBindingRecursive(InActor, RenameParams, SequenceParams);
	}
}

void FAvaSequencer::OnMovieSceneBindingsPasted(const TArray<FMovieSceneBinding>& InBindings)
{
	UAvaSequence* const Sequence = GetViewedSequence();
	if (!Sequence)
	{
		return;
	}

	UMovieScene* const MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	UObject* const PlaybackContext = GetPlaybackContext();
	if (!PlaybackContext)
	{
		return;
	}

	TSet<const FMovieScenePossessable*> ProcessedPossessables;

	for (const FMovieSceneBinding& Binding : InBindings)
	{
		if (FMovieScenePossessable* const Possessable = MovieScene->FindPossessable(Binding.GetObjectGuid()))
		{
			FixPossessable(*Sequence, *Possessable, PlaybackContext, ProcessedPossessables);
		}
	}
}

void FAvaSequencer::AddTransformKey(EMovieSceneTransformChannel InTransformChannel)
{
	using namespace UE::AvaSequencer::Private;

	if (!GEditor)
	{
		return;
	}

	FEditorModeTools* const EditorModeTools = Provider.GetSequencerModeTools();
	if (!EditorModeTools)
	{
		return;
	}

	TSharedPtr<ISequencer> Sequencer = GetSequencerPtr();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TArray<TSharedPtr<ISequencerTrackEditor>>& TrackEditors = StaticCastSharedPtr<FSequencer>(Sequencer)->GetTrackEditors();

	TArray<TSharedPtr<ISequencerTrackEditor>> TransformTrackEditors;

	bool bUseOverridePriority = false;

	for (const TSharedPtr<ISequencerTrackEditor>& TrackEditor : TrackEditors)
	{
		if (TrackEditor.IsValid() && TrackEditor->HasTransformKeyBindings())
		{
			TransformTrackEditors.Add(TrackEditor);
			bUseOverridePriority |= TrackEditor->HasTransformKeyOverridePriority();
		}
	}

	if (TransformTrackEditors.IsEmpty())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddTransformKey", "Add Transform Key"));

	// Temporarily set the GEditor Selections to our Ed Mode Tools Selections
	FScopedSelection ActorSelection(GEditor->GetSelectedActors(), EditorModeTools->GetSelectedActors());
	FScopedSelection CompSelection(GEditor->GetSelectedComponents(), EditorModeTools->GetSelectedComponents());
	FScopedSelection ObjectSelection(GEditor->GetSelectedObjects(), EditorModeTools->GetSelectedObjects());

	for (const TSharedPtr<ISequencerTrackEditor>& TransformTrackEditor : TransformTrackEditors)
	{
		if (!bUseOverridePriority || TransformTrackEditor->HasTransformKeyOverridePriority())
		{
			TransformTrackEditor->OnAddTransformKeysForSelectedObjects(InTransformChannel);
		}
	}
}

void FAvaSequencer::ApplyDefaultPresetToSelection(FName InPresetName)
{
	const UAvaSequencerSettings* SequencerSettings = GetDefault<UAvaSequencerSettings>();
	if (!SequencerSettings)
	{
		return;
	}

	const TConstArrayView<FAvaSequencePreset> DefaultSequencePresets = SequencerSettings->GetDefaultSequencePresets();

	const int32 PresetIndex = DefaultSequencePresets.Find(FAvaSequencePreset(InPresetName));
	if (PresetIndex == INDEX_NONE)
	{
		return;
	}

	ApplyPresetToSelection(DefaultSequencePresets[PresetIndex]);
}

void FAvaSequencer::ApplyCustomPresetToSelection(FName InPresetName)
{
	const UAvaSequencerSettings* SequencerSettings = GetDefault<UAvaSequencerSettings>();
	if (!SequencerSettings)
	{
		return;
	}

	const FAvaSequencePreset* SequencePreset = SequencerSettings->GetCustomSequencePresets().Find(FAvaSequencePreset(InPresetName));
	if (!SequencePreset)
	{
		return;
	}

	ApplyPresetToSelection(*SequencePreset);
}

void FAvaSequencer::ApplyPresetToSelection(const FAvaSequencePreset& InPreset)
{
	const TArray<UAvaSequence*> SelectedSequences = GetSelectedSequences();
	if (SelectedSequences.IsEmpty())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ApplySequencePreset", "Apply Sequence Preset"));

	for (UAvaSequence* const AvaSequence : SelectedSequences)
	{
		if (AvaSequence)
		{
			InPreset.ApplyPreset(AvaSequence);
		}
	}
}

bool FAvaSequencer::AddSequence_CanExecute() const
{
	return GetProvider().CanEditOrPlaySequences();
}

void FAvaSequencer::AddSequence_Execute()
{
	IAvaSequenceProvider* const SequenceProvider = Provider.GetSequenceProvider();
	if (!SequenceProvider)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("AddSequenceTransaction", "Add Sequence"));

	UAvaSequence* const Sequence = CreateSequence();
	SequenceProvider->AddSequence(Sequence);
}

bool FAvaSequencer::DuplicateSequences_CanExecute() const
{
	return GetProvider().CanEditOrPlaySequences();
}

void FAvaSequencer::DuplicateSequences_Execute()
{
	IAvaSequenceProvider* const SequenceProvider = Provider.GetSequenceProvider();
	if (!SequenceTreeView.IsValid() || !SequenceProvider)
	{
		return;
	}

	const TArray<FAvaSequenceItemPtr> SelectedItems = SequenceTreeView->GetSelectedItems();
	if (SelectedItems.IsEmpty())
	{
		return;
	}

	UObject* const Outer = SequenceProvider->ToUObject();
	if (!Outer)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("DuplicateSequenceTransaction", "Duplicate Sequence"));

	Outer->Modify();

	for (const FAvaSequenceItemPtr& Item : SelectedItems)
	{
		UAvaSequence* const TemplateSequence = Item->GetSequence();
		check(TemplateSequence);

		UAvaSequence* const Sequence = DuplicateObject<UAvaSequence>(TemplateSequence, Outer);
		SequenceProvider->AddSequence(Sequence);
	}
}

bool FAvaSequencer::ExportSequences_CanExecute() const
{
	return SequenceTreeView.IsValid()
		&& !SequenceTreeView->GetSelectedItems().IsEmpty();
}

void FAvaSequencer::ExportSequences_Execute()
{
	Provider.ExportSequences(GetSelectedSequences());
}

bool FAvaSequencer::SpawnPlayers_CanExecute() const
{
	return SequenceTreeView.IsValid()
		&& !SequenceTreeView->GetSelectedItems().IsEmpty();
}

void FAvaSequencer::SpawnPlayers_Execute()
{
	if (!GEditor)
	{
		return;
	}

	UWorld* World = Provider.GetPlaybackContext()->GetWorld();
	if (!World)
	{
		return;
	}

	UActorFactory* ActorFactory = GEditor->FindActorFactoryForActorClass(AAvaSequenceActor::StaticClass());
	if (!ensure(ActorFactory))
	{
		return;
	}

	TArray<UAvaSequence*> Sequences = GetSelectedSequences();
	if (Sequences.Num() != 1)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("SpawnSequencePlayers", "Spawn Sequence Players"));

	for (UAvaSequence* Sequence : Sequences)
	{
		check(Sequence);
		GEditor->UseActorFactory(ActorFactory, FAssetData(Sequence), &FTransform::Identity);
	}
}

bool FAvaSequencer::DeleteSequences_CanExecute() const
{
	return GetProvider().CanEditOrPlaySequences();
}

void FAvaSequencer::DeleteSequences_Execute()
{
	IAvaSequenceProvider* const SequenceProvider = Provider.GetSequenceProvider();
	if (!SequenceProvider)
	{
		return;
	}

	UObject* const Outer = SequenceProvider->ToUObject();
	if (!Outer)
	{
		return;
	}

	TArray<FAvaSequenceItemPtr> SelectedItems = SequenceTreeView->GetSelectedItems();

	const FScopedTransaction Transaction(LOCTEXT("DeleteSequenceTransaction", "Delete Sequence"));

	Outer->Modify();

	TArray<UAvaSequence*> RemovedSequences;
	RemovedSequences.Reserve(SelectedItems.Num());

	// Remove the Selected Sequences from the List (not marked as garbage yet)
	for (const FAvaSequenceItemPtr& Item : SelectedItems)
	{
		check(Item.IsValid());
		if (UAvaSequence* Sequence = Item->GetSequence())
		{
			Sequence->Modify();
			SequenceProvider->RemoveSequence(Sequence);
			RemovedSequences.Add(Sequence);
		}
	}

	// Set the Viewed Sequence to the Default one
	SetViewedSequence(GetDefaultSequence());

	// Once a new viewed sequence is set, the removed sequences can now be marked as garbage 
	for (UAvaSequence* Sequence : RemovedSequences)
	{
		Sequence->OnSequenceRemoved();
	}

	if (UBlueprint* const Blueprint = Cast<UBlueprint>(Outer))
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}
}

bool FAvaSequencer::RelabelSequence_CanExecute() const
{
	const bool bCanEditSequences = GetProvider().CanEditOrPlaySequences();
	return bCanEditSequences && SequenceTreeView->GetNumItemsSelected() == 1;
}

void FAvaSequencer::RelabelSequence_Execute()
{
	TArray<FAvaSequenceItemPtr> SelectedItems = SequenceTreeView->GetSelectedItems();
	check(SelectedItems.Num() == 1);

	const FAvaSequenceItemPtr& SelectedItem = SelectedItems[0];
	SelectedItem->RequestRelabel();
}

bool FAvaSequencer::PlaySequences_CanExecute() const
{
	const bool bCanEditSequences = GetProvider().CanEditOrPlaySequences();
	return bCanEditSequences && SequenceTreeView->GetNumItemsSelected() > 0;
}

void FAvaSequencer::PlaySequences_Execute()
{
	const TArray<FAvaSequenceItemPtr> SelectedItems = SequenceTreeView->GetSelectedItems();

	FAvaSequencePlayParams PlaySettings;
	PlaySettings.AdvancedSettings.bRestoreState = true;

	for (const FAvaSequenceItemPtr& Item : SelectedItems)
	{
		UAvaSequence* const Sequence = Item.IsValid()
			? Item->GetSequence()
			: nullptr;

		IAvaSequencePlaybackObject* const PlaybackObject = Provider.GetPlaybackObject();
		if (Sequence && PlaybackObject)
		{
			PlaybackObject->PlaySequence(Sequence, PlaySettings);
		}
	}
}

bool FAvaSequencer::ContinueSequences_CanExecute() const
{
	const bool bCanEditSequences = GetProvider().CanEditOrPlaySequences();
	return bCanEditSequences && SequenceTreeView->GetNumItemsSelected() > 0;
}

void FAvaSequencer::ContinueSequences_Execute()
{
	const TArray<FAvaSequenceItemPtr> SelectedItems = SequenceTreeView->GetSelectedItems();
	
	for (const FAvaSequenceItemPtr& Item : SelectedItems)
	{
		UAvaSequence* const Sequence = Item.IsValid()
			? Item->GetSequence()
			: nullptr;

		IAvaSequencePlaybackObject* const PlaybackObject = Provider.GetPlaybackObject();
		if (Sequence && PlaybackObject)
		{
			PlaybackObject->ContinueSequence(Sequence);
		}
	}
}

bool FAvaSequencer::StopSequences_CanExecute() const
{
	return SequenceTreeView->GetNumItemsSelected() > 0;
}

void FAvaSequencer::StopSequences_Execute()
{
	const TArray<FAvaSequenceItemPtr> SelectedItems = SequenceTreeView->GetSelectedItems();

	for (const FAvaSequenceItemPtr& Item : SelectedItems)
	{
		UAvaSequence* const Sequence = Item.IsValid()
			? Item->GetSequence()
			: nullptr;

		IAvaSequencePlaybackObject* const PlaybackObject = Provider.GetPlaybackObject();
		if (Sequence && PlaybackObject)
		{
			PlaybackObject->StopSequence(Sequence);
		}
	}
}

void FAvaSequencer::OnSequenceEditUndo(UAvaSequence* InSequence)
{
	if (!InSequence)
	{
		return;
	}

	const UAvaSequence* ViewedSequence = ViewedSequenceWeak.Get(/*bEvenIfPendingKill*/true);

	// Set to the default sequence if the sequence is the one being edited and has just been marked invalid
	if (InSequence == ViewedSequence && !IsValid(ViewedSequence))
	{
		// Since a new viewed sequence is being set, Sequencer will try to restore the current viewed sequence (now invalid)
		// and ensure fail when trying to resolve its weak ptrs (defaulting to bEvenIfPendingKill=false).
		// To circumvent this, temporarily unmark this sequence as garbage, so that these weak ptrs can resolve properly.
		InSequence->ClearGarbage();
		SetViewedSequence(GetDefaultSequence());
		InSequence->MarkAsGarbage();
	}
}

bool FAvaSequencer::CanAddSequence() const
{
	return GetProvider().CanEditOrPlaySequences();
}

UAvaSequence* FAvaSequencer::AddSequence(UAvaSequence* const InParentSequence)
{
	IAvaSequenceProvider* const SequenceProvider = Provider.GetSequenceProvider();
	if (!SequenceProvider)
	{
		return nullptr;
	}

	FScopedTransaction Transaction(LOCTEXT("AddSequenceTransaction", "Add Sequence"));

	if (UObject* SequenceProviderObject = SequenceProvider->ToUObject())
	{
		SequenceProviderObject->Modify();
	}

	UAvaSequence* const NewSequence = CreateSequence();
	if (!NewSequence)
	{
		Transaction.Cancel();
		return nullptr;
	}

	SequenceProvider->AddSequence(NewSequence);

	if (InParentSequence)
	{
		InParentSequence->Modify();
		InParentSequence->AddChild(NewSequence);
	}

	OnSequenceAddedDelegate.Broadcast(NewSequence);

	return NewSequence;
}

void FAvaSequencer::DeleteSequences(const TSet<UAvaSequence*>& InSequences)
{
	if (InSequences.IsEmpty())
	{
		return;
	}

	IAvaSequenceProvider* const SequenceProvider = Provider.GetSequenceProvider();
	if (!SequenceProvider)
	{
		return;
	}

	UObject* const Outer = SequenceProvider->ToUObject();
	if (!Outer)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("DeleteSequencesTransaction", "Delete Sequence(s)"));

	Outer->Modify();

	TArray<UAvaSequence*> RemovedSequences;
	RemovedSequences.Reserve(InSequences.Num());

	// Remove the sequences from the list (not marked as garbage yet)
	for (UAvaSequence* const Sequence : InSequences)
	{
		check(Sequence);

		Sequence->Modify();
		SequenceProvider->RemoveSequence(Sequence);

		RemovedSequences.Add(Sequence);

		OnSequenceRemovedDelegate.Broadcast(Sequence);
	}

	// Set the Viewed Sequence to the Default one
	SetViewedSequence(GetDefaultSequence());

	// Once a new viewed sequence is set, the removed sequences can now be marked as garbage 
	for (UAvaSequence* Sequence : RemovedSequences)
	{
		Sequence->OnSequenceRemoved();
	}

	if (UBlueprint* const Blueprint = Cast<UBlueprint>(Outer))
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}
}

uint32 FAvaSequencer::AddSequenceFromPresets(TConstArrayView<const FAvaSequencePreset*> InPresets)
{
	IAvaSequenceProvider* const SequenceProvider = Provider.GetSequenceProvider();
	if (!SequenceProvider)
	{
		return 0;
	}

	TArray<UAvaSequence*> NewSequences;
	NewSequences.Reserve(InPresets.Num());

	for (const FAvaSequencePreset* Preset : InPresets)
	{
		if (!Preset)
		{
			continue;
		}

		if (UAvaSequence* const Sequence = CreateSequence())
		{
			Preset->ApplyPreset(Sequence);
			NewSequences.Add(Sequence);
		}
	}

	return SequenceProvider->AddSequences(NewSequences);
}

void FAvaSequencer::AddSequencesFromPresetGroup(FName InPresetGroup)
{
	const UAvaSequencerSettings* SequencerSettings = GetDefault<UAvaSequencerSettings>();
	if (!SequencerSettings)
	{
		return;
	}

	TArray<const FAvaSequencePreset*> Presets = SequencerSettings->GatherPresetsFromGroup(InPresetGroup);
	if (Presets.IsEmpty())
	{
		return;
	}

	IAvaSequenceProvider* const SequenceProvider = Provider.GetSequenceProvider();
	if (!SequenceProvider)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddSequencesFromPresetGroup", "Add Sequences from Preset Group"));

	if (UObject* SequenceProviderObject = SequenceProvider->ToUObject())
	{
		SequenceProviderObject->Modify();
	}

	const uint32 AddedCount = AddSequenceFromPresets(Presets);
	if (AddedCount == 0)
	{
		Transaction.Cancel();
	}
}

UObject* FAvaSequencer::FindObjectToPossess(UObject* InParentObject, const FMovieScenePossessable& InPossessable)
{
	if (!InParentObject)
	{
		return nullptr;
	}

	UClass* const PossessableClass = const_cast<UClass*>(InPossessable.GetPossessedObjectClass());

	// Try to find the Object that matches BOTH the Possessable Name and Possessed Object Class
	if (UObject* const FoundObject = StaticFindObject(PossessableClass, InParentObject, *InPossessable.GetName(), EFindObjectFlags::ExactClass))
	{
		return FoundObject;
	}

	const FName ObjectName(*InPossessable.GetName(), FNAME_Add);

	// If nothing was found via StaticFindObject, there is the possibility this is a nested subobject that just happens
	// to be under the Parent Object (e.g. an Actor) to avoid nesting in outliner or limitations of how the sequence resolves bindings
	TArray<UObject*> Objects;
	GetObjectsWithOuter(InParentObject, Objects, EGetObjectsFlags::IncludeNestedObjects);

	for (UObject* Object : Objects)
	{
		if (!Object)
		{
			continue;
		}

		const bool bMatchesName  = Object->GetFName() == ObjectName;
		const bool bMatchesClass = Object->GetClass() == PossessableClass;

		if (bMatchesName && bMatchesClass)
		{
			return Object;
		}
	}

	return nullptr;
}

void FAvaSequencer::ApplyCurrentState()
{
	const EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::YesNo
		, LOCTEXT("ApplyStateMessage", "Are you sure you want to discard the currently saved pre-animated state, and apply the current state? (This cannot be undone)")
		, LOCTEXT("ApplyStateTitle", "Apply Current Animated State to World"));

	if (Response == EAppReturnType::Yes)
	{
		if (TSharedPtr<ISequencer> Sequencer = GetSequencerPtr())
		{
			Sequencer->PreAnimatedState.DiscardPreAnimatedState();
		}
	}
}

void FAvaSequencer::FixBindingPaths()
{
	UAvaSequence* const Sequence   = GetViewedSequence();
	UObject* const PlaybackContext = GetPlaybackContext();
	if (!Sequence || !PlaybackContext)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("FixBindingPaths", "Fix Binding Paths"));

	Sequence->Modify();

	// pass in a null old context, which forces a replacement of all base bindings. Without further parameters, there's no knowledge of what the old context is
	int32 BindingsUpdatedCount = Sequence->UpdateBindings(nullptr, FTopLevelAssetPath(PlaybackContext));
	if (BindingsUpdatedCount == 0)
	{
		Transaction.Cancel();
		return;
	}

	if (TSharedPtr<ISequencer> Sequencer = GetSequencerPtr())
	{
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	}
}

void FAvaSequencer::FixInvalidBindings()
{
	UObject* const PlaybackContext = GetPlaybackContext();
	if (!PlaybackContext)
	{
		return;
	}

	UAvaSequence* const Sequence = GetViewedSequence();
	if (!Sequence)
	{
		return;
	}

	UMovieScene* const MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	if (MovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("FixInvalidBindings", "Fix Invalid Bindings"));

	TSet<const FMovieScenePossessable*> ProcessedPossessables;

	for (int32 PossessableIndex = 0; PossessableIndex < MovieScene->GetPossessableCount(); ++PossessableIndex)
	{
		FMovieScenePossessable& Possessable = MovieScene->GetPossessable(PossessableIndex);
		FixPossessable(*Sequence, Possessable, PlaybackContext, ProcessedPossessables);
	}
}

FString FAvaSequencer::GetObjectName(UObject* InObject)
{
	check(InObject);
	if (AActor* const Actor = Cast<AActor>(InObject))
	{
		return Actor->GetActorLabel();
	}
	return InObject->GetName();
}

UObject* FAvaSequencer::FindResolutionContext(UAvaSequence& InSequence
	, UMovieScene& InMovieScene
	, const FGuid& InParentPossessableGuid
	, UObject* InPlaybackContext
	, TFunctionRef<TArray<UObject*, TInlineAllocator<1>>(const FGuid&, UObject*)> InFindObjectsFunc)
{
	if (!InPlaybackContext || !InParentPossessableGuid.IsValid() || !InSequence.AreParentContextsSignificant())
	{
		return InPlaybackContext;
	}

	UObject* ResolutionContext = nullptr;

	// Recursive call up the hierarchy
	if (FMovieScenePossessable* const ParentPossessable = InMovieScene.FindPossessable(InParentPossessableGuid))
	{
		ResolutionContext = FAvaSequencer::FindResolutionContext(InSequence
			, InMovieScene
			, ParentPossessable->GetParent()
			, InPlaybackContext
			, InFindObjectsFunc);
	}

	if (!ResolutionContext)
	{
		ResolutionContext = InPlaybackContext;
	}

	TArray<UObject*, TInlineAllocator<1>> FoundObjects = InFindObjectsFunc(InParentPossessableGuid, ResolutionContext);
	if (FoundObjects.IsEmpty())
	{
		return InPlaybackContext;
	}

	return FoundObjects[0] ? FoundObjects[0] : InPlaybackContext;
}

UObject* FAvaSequencer::FindResolutionContext(UAvaSequence& InSequence
	, UMovieScene& InMovieScene
	, const FGuid& InParentGuid
	, UObject* InPlaybackContext)
{
	auto FindObjectsFunc = [&InSequence](const FGuid& InGuid, UObject* InContextChecked)
		{
			TArray<UObject*, TInlineAllocator<1>> BoundObjects;

			InSequence.LocateBoundObjects(InGuid, UE::UniversalObjectLocator::FResolveParams(InContextChecked), MovieSceneHelpers::CreateTransientSharedPlaybackState(InContextChecked, &InSequence), BoundObjects);
			return BoundObjects;
		};

	return FindResolutionContext(InSequence, InMovieScene, InParentGuid, InPlaybackContext, FindObjectsFunc);
}

bool FAvaSequencer::FixPossessable(UAvaSequence& InSequence
	, const FMovieScenePossessable& InPossessable
	, UObject* InPlaybackContext
	, TSet<const FMovieScenePossessable*>& InOutProcessedPossessables)
{
	if (!ensureAlways(InPlaybackContext && InSequence.GetMovieScene()))
	{
		return false;
	}

	// This Possessable has already been fixed or was already verified as valid, skip
	if (InOutProcessedPossessables.Contains(&InPossessable))
	{
		return true;
	}

	UMovieScene& MovieScene = *InSequence.GetMovieScene();

	const FGuid& ParentGuid = InPossessable.GetParent();

	// Fix Parent Possessable first since if Parent Contexts are significant the parent will be needed to resolve the child
	if (FMovieScenePossessable* const PossessableParent = MovieScene.FindPossessable(ParentGuid))
	{
		if (!FixPossessable(InSequence, *PossessableParent, InPlaybackContext, InOutProcessedPossessables))
		{
			UE_LOGF(LogAvaSequencer, Warning
				, "Parent '%ls' of Possessable '%ls' could not be fixed."
				, *PossessableParent->GetName()
				, *InPossessable.GetName());
			return false;
		}
	}

	const FGuid& Guid = InPossessable.GetGuid();

	UObject* const ResolutionContext = FAvaSequencer::FindResolutionContext(InSequence
		, MovieScene
		, ParentGuid
		, InPlaybackContext);


	TArrayView<TWeakObjectPtr<>> BoundObjects = ResolveBoundObjects(Guid, &InSequence);

	// If Bound Objects isn't empty, then it means Possessable is valid, so add to Valid List and early return
	if (!BoundObjects.IsEmpty() && BoundObjects[0].IsValid())
	{
		InOutProcessedPossessables.Add(&InPossessable);
		return true;
	}

	if (UObject* const Object = FAvaSequencer::FindObjectToPossess(ResolutionContext, InPossessable))
	{
		InSequence.Modify();
		InSequence.BindPossessableObject(Guid, *Object, ResolutionContext);
		InOutProcessedPossessables.Add(&InPossessable);
		return true;
	}

	return false;
}

void FAvaSequencer::FixBindingHierarchy()
{
	UAvaSequence* const Sequence   = GetViewedSequence();
    UObject* const PlaybackContext = GetPlaybackContext();
    if (!Sequence || !PlaybackContext)
    {
    	return;
    }

	UMovieScene* const MovieScene = Sequence->GetMovieScene();
	if (!MovieScene || MovieScene->IsReadOnly())
	{
		return;
	}

	TSharedPtr<ISequencer> Sequencer = GetSequencerPtr();
	if (!Sequencer.IsValid())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("FixBindingHierarchy", "Fix Binding Hierarchy"));
	MovieScene->Modify();

	// Iterates all the possessables that are not necessarily under a set parent (via FMovieScenePossessable::GetParent), but
	// have a bound object that does have a valid parent found via UAvaSequence::GetParentObject, hence the word usage of "found" over "set"
	auto ForEachPossessableWithFoundParent = [this, MovieScene, Sequence, PlaybackContext]
		(TFunctionRef<void(UObject& InObject, FMovieScenePossessable& InPossessable, UObject& InParentObject)> InFunc)
		{
			for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
			{
				FMovieScenePossessable& Possessable = MovieScene->GetPossessable(Index);

				TArrayView<TWeakObjectPtr<>> BoundObjects = ResolveBoundObjects(Possessable.GetGuid(), Sequence);

				if (BoundObjects.IsEmpty() || !BoundObjects[0].IsValid())
				{
					continue;
				}

				if (UObject* ParentObject = Sequence->GetParentObject(BoundObjects[0].Get()))
				{
					InFunc(*BoundObjects[0], Possessable, *ParentObject);
				}
			}
		};

	// Pass #1: Ensure that all the Parent Objects have a valid possessable handle
	ForEachPossessableWithFoundParent(
		[&Sequencer](UObject& InPossessableObject, FMovieScenePossessable& InPossessable, UObject& InParentObject)
		{
			constexpr bool bCreateHandleToObject = true;
			Sequencer->GetHandleToObject(&InParentObject, bCreateHandleToObject);
		});

	// Pass #2: Fix the hierarchy now that all relevant objects have a valid handle
	ForEachPossessableWithFoundParent(
		[&Sequencer, MovieScene, Sequence, PlaybackContext](UObject& InObject, FMovieScenePossessable& InPossessable, UObject& InParentObject)
		{
			constexpr bool bCreateHandleToObject = false;
			const FGuid ParentGuid = Sequencer->GetHandleToObject(&InParentObject, bCreateHandleToObject);

			// Parent Guid must be valid, as it was created in pass #1 if missing
			if (!ParentGuid.IsValid())
			{
				UE_LOGF(LogAvaSequencer, Error
					, "Could not create handle to parent object %ls for Possessable %ls (GUID: %ls)"
					, *InParentObject.GetName()
					, *InPossessable.GetName()
					, *InPossessable.GetGuid().ToString());
				return;
			}

			if (InPossessable.GetParent() != ParentGuid)
			{
				InPossessable.SetParent(ParentGuid, MovieScene);

				UObject* const Context = Sequence->AreParentContextsSignificant()
					? &InParentObject
					: PlaybackContext;

				// Recalculate the Binding Path
				Sequence->UnbindPossessableObjects(InPossessable.GetGuid());
				Sequence->BindPossessableObject(InPossessable.GetGuid(), InObject, Context);
			}
		});

	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}

TArrayView<TWeakObjectPtr<>> FAvaSequencer::ResolveBoundObjects(const FGuid& InBindingId, class UMovieSceneSequence* Sequence) const
{
	TSharedPtr<ISequencer> Sequencer = GetSequencerPtr();
	if (!Sequencer.IsValid())
	{
		return TArrayView<TWeakObjectPtr<>>();
	}

	TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState = Sequencer->GetSharedPlaybackState();
	// TODO: It would be better if FAvaSequencer saved the SequenceID. It's possible looking that this might always be FMovieSceneSequenceID::Root, but I'm unsure.
	if (FMovieSceneEvaluationState* EvaluationState = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>())
	{
		FMovieSceneSequenceIDRef SequenceID = EvaluationState->FindSequenceId(Sequence);
		return Sequencer->FindBoundObjects(InBindingId, SequenceID);
	}
	return TArrayView<TWeakObjectPtr<>>();
}

TSharedRef<ISequencer> FAvaSequencer::GetOrCreateSequencer() const
{
	const_cast<FAvaSequencer*>(this)->EnsureSequencer();
	return SequencerWeak.Pin().ToSharedRef();
}

TSharedPtr<ISequencer> FAvaSequencer::GetSequencerPtr() const
{
	return SequencerWeak.Pin();
}

USequencerSettings* FAvaSequencer::GetSequencerSettings() const
{
	if (TSharedPtr<ISequencer> Sequencer = SequencerWeak.Pin())
	{
		return Sequencer->GetSequencerSettings();
	}
	return nullptr;
}

void FAvaSequencer::SetBaseCommandList(const TSharedPtr<FUICommandList>& InBaseCommandList)
{
	if (InBaseCommandList.IsValid())
	{
		InBaseCommandList->Append(CommandList);	
	}
}

UAvaSequence* FAvaSequencer::GetViewedSequence() const
{
	return ViewedSequenceWeak.Get();
}

void FAvaSequencer::SetViewedSequence(UAvaSequence* InSequenceToView)
{
	if (InSequenceToView == ViewedSequenceWeak)
	{
		return;
	}

	UAvaSequence* OldSequence = ViewedSequenceWeak.Get(/*bEvenIfPendingKill*/true);
	ViewedSequenceWeak = InSequenceToView;
	NotifyViewedSequenceChanged(OldSequence);
}

TArray<UAvaSequence*> FAvaSequencer::GetSequencesForObject(UObject* InObject) const
{
	TArray<UAvaSequence*> OutSequences;

	if (!InObject)
	{
		return OutSequences;
	}

	IAvaSequenceProvider* const SequenceProvider = Provider.GetSequenceProvider();
	if (!SequenceProvider)
	{
		return OutSequences;
	}

	for (UAvaSequence* const Sequence : SequenceProvider->GetSequences())
	{
		if (!Sequence)
		{
			continue;
		}

		const FGuid Guid = Sequence->FindGuidFromObject(InObject);

		if (Guid.IsValid())
		{
			OutSequences.Add(Sequence);
		}
	}
	return OutSequences;
}

TSharedRef<SWidget> FAvaSequencer::CreateSequenceWidget()
{
	// Force the SequencerWeak ptr to be invalid if this AvaSequencer doesn't explicitly own the sequencer (i.e. InstancedSequencer is null)
	// This is to force the sequencer to look for a new sequencer again
	if (!InstancedSequencer.IsValid())
	{
		SequencerWeak.Reset();
	}

	const TSharedRef<ISequencer> Sequencer = GetOrCreateSequencer();

	USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
	check(SequencerSettings);
	FSidebarState& SidebarState = SequencerSettings->GetSidebarState();

	if (SidebarState.IsVisible())
	{
		FSidebarDrawerConfig MotionDesignDrawerConfig;
		MotionDesignDrawerConfig.UniqueId = SidebarDrawerId;
		MotionDesignDrawerConfig.ButtonText = LOCTEXT("MotionDesignLabel", "Motion Design");
		MotionDesignDrawerConfig.ToolTipText = LOCTEXT("SequenceTooltip", "Open the Sequence options panel");
		MotionDesignDrawerConfig.Icon = FAvaEditorCoreStyle::Get().GetBrush(TEXT("Icons.MotionDesign"));
		MotionDesignDrawerConfig.InitialState = SidebarState.FindOrAddDrawerState(SidebarDrawerId);
		Sequencer->RegisterDrawer(MoveTemp(MotionDesignDrawerConfig));

		const TSharedRef<FAvaSequencer> ThisSequencerRef = SharedThis(this);
		Sequencer->RegisterDrawerSection(SidebarDrawerId, MakeShared<FAvaSequenceTreeDetails>(ThisSequencerRef));
		Sequencer->RegisterDrawerSection(SidebarDrawerId, MakeShared<FAvaSequencePlaybackDetails>(ThisSequencerRef));
		Sequencer->RegisterDrawerSection(SidebarDrawerId, MakeShared<FAvaSequenceSettingsDetails>(ThisSequencerRef));
	}

	if (UObject* const PlaybackContextObject = Provider.GetPlaybackContext())
	{
		if (UWorld* const World = PlaybackContextObject->GetWorld())
		{
			if (UAvaSequencerSubsystem* const SequencerSubsystem = World->GetSubsystem<UAvaSequencerSubsystem>())
			{
				SequencerSubsystem->OnSequencerCreated().Broadcast(SharedThis(this));
			}
		}
	}

	return Sequencer->GetSequencerWidget();
}

void FAvaSequencer::OnActorsCopied(FString& InOutCopiedData, TConstArrayView<AActor*> InCopiedActors)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAvaSequencer::OnActorsCopied);
	EnsureSequencer();
	FAvaSequenceExporter Exporter(SharedThis(this));
	Exporter.ExportText(InOutCopiedData, InCopiedActors);
}

void FAvaSequencer::OnActorsPasted(FStringView InPastedData, const TMap<FName, AActor*>& InPastedActors)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAvaSequencer::OnActorsPasted);
	EnsureSequencer();
	FAvaSequenceImporter Importer(SharedThis(this));
	Importer.ImportText(InPastedData, InPastedActors);
}

void FAvaSequencer::OnEditorSelectionChanged(const FAvaEditorSelection& InEditorSelection)
{
	if (!bCanProcessSequencerSelections)
	{
		return;
	}

	if (bUpdatingSelection || bSelectedFromSequencer)
	{
		bSelectedFromSequencer = false;
		return;
	}

	TSharedPtr<ISequencer> Sequencer = GetSequencerPtr();
	if (!Sequencer.IsValid())
	{
		return;
	}

	TGuardValue<bool> Guard(bUpdatingSelection, true);
	Sequencer->EmptySelection();

	using namespace UE::Sequencer;

	{
		FSelectionEventSuppressor SuppressSelectionEvents = Sequencer->GetViewModel()->GetSelection()->SuppressEvents();

		for (UObject* const SelectedObject : InEditorSelection.GetSelectedObjects<UObject, EAvaSelectionSource::All>())
		{
			if (SelectedObject)
			{
				FGuid SelectedObjectGuid = Sequencer->FindObjectId(*SelectedObject, Sequencer->GetFocusedTemplateID());
				Sequencer->SelectObject(SelectedObjectGuid);
			}
		}
	}

	// Scroll Selected Node to View
	if (TSharedPtr<SOutlinerView> OutlinerView = GetOutlinerView())
	{
		FOutlinerSelection& Outliner = Sequencer->GetViewModel()->GetSelection()->Outliner;
		if (Outliner.Num() > 0)
		{
			TViewModelPtr<FViewModel> SelectedNode = Outliner.GetSelected().FindArbitraryElement()->Pin();
			TSharedPtr<FViewModel> Parent = SelectedNode->GetParent();
			while (Parent.IsValid())
			{
				OutlinerView->SetItemExpansion(UE::Sequencer::CastViewModel<IOutlinerExtension>(Parent), true);
				Parent = Parent->GetParent();
			}
			OutlinerView->RequestScrollIntoView(UE::Sequencer::CastViewModel<IOutlinerExtension>(SelectedNode));
		}
	}
}

void FAvaSequencer::NotifyOnSequenceTreeChanged()
{
	IAvaSequenceProvider* const SequenceProvider = Provider.GetSequenceProvider();
	if (!SequenceTreeView.IsValid() || !SequenceProvider)
	{
		return;
	}

	const TSet<TWeakObjectPtr<UAvaSequence>> RootSequences(SequenceProvider->GetRootSequences());

	TSet<TWeakObjectPtr<UAvaSequence>> SeenRoots;
	SeenRoots.Reserve(RootSequences.Num());

	// Remove Current Root Items that are not in the Latest Root Set
	for (TArray<FAvaSequenceItemPtr>::TIterator Iter(RootSequenceItems); Iter; ++Iter)
	{
		const FAvaSequenceItemPtr Item = *Iter;

		if (!Item.IsValid())
		{
			Iter.RemoveCurrent();
			continue;
		}

		TObjectPtr<UAvaSequence> UnderlyingSequence = Item->GetSequence();

		if (UnderlyingSequence && RootSequences.Contains(UnderlyingSequence))
		{
			SeenRoots.Add(UnderlyingSequence);
		}
		else
		{
			Iter.RemoveCurrent();
		}
	}

	// Make New Root Items for the Sequences that were not Seen
	{
		TArray<TWeakObjectPtr<UAvaSequence>> NewRoots = RootSequences.Difference(SeenRoots).Array();
		
		RootSequenceItems.Reserve(RootSequenceItems.Num() + NewRoots.Num());

		const TSharedPtr<FAvaSequencer> This = SharedThis(this);
		
		for (const TWeakObjectPtr<UAvaSequence>& Sequence : NewRoots)
		{
			TSharedPtr<FAvaSequenceItem> NewItem = MakeShared<FAvaSequenceItem>(Sequence.Get(), This);
			RootSequenceItems.Add(NewItem);
		}
	}

	//Refresh Children Iteratively
	TArray<FAvaSequenceItemPtr> RemainingItems = RootSequenceItems;
	while (!RemainingItems.IsEmpty())
	{
		if (FAvaSequenceItemPtr Item = RemainingItems.Pop())
		{
			Item->RefreshChildren();
			RemainingItems.Append(Item->GetChildren());
		}
	}

	// Ensure the new item representing the Viewed Sequence is selected
	UAvaSequence* ViewedSequence = GetViewedSequence();
	if (ViewedSequence && SequenceTree.IsValid())
	{
		SequenceTree->OnPostSetViewedSequence(ViewedSequence);
	}

	SequenceTreeView->RequestTreeRefresh();
}

void FAvaSequencer::PostUndo(const bool bInSuccess)
{
	if (IAvaSequenceProvider* const SequenceProvider = Provider.GetSequenceProvider())
	{
		SequenceProvider->RebuildSequenceTree();
	}

	// A just-added sequence might be removed due to this undo, so refresh 
	NotifyOnSequenceTreeChanged();
}

void FAvaSequencer::PostRedo(const bool bInSuccess)
{
	PostUndo(bInSuccess);
}

TArray<UAvaSequence*> FAvaSequencer::GetSelectedSequences() const
{
	const TArray<TSharedPtr<IAvaSequenceItem>> SelectedItems = SequenceTreeView->GetSelectedItems();
	if (SelectedItems.IsEmpty())
	{
		return {};
	}

	TArray<UAvaSequence*> SelectedAvaSequences;
	SelectedAvaSequences.Reserve(SelectedItems.Num());

	for (const TSharedPtr<IAvaSequenceItem>& SequenceItem : SelectedItems)
	{
		if (UAvaSequence* const AvaSequence = SequenceItem->GetSequence())
		{
			SelectedAvaSequences.Add(AvaSequence);
		}
	}

	return SelectedAvaSequences;
}

TSharedPtr<UE::Sequencer::SOutlinerView> FAvaSequencer::GetOutlinerView() const
{
	if (OutlinerViewWeak.IsValid())
	{
		return OutlinerViewWeak.Pin();
	}

	TSharedPtr<ISequencer> Sequencer = SequencerWeak.Pin();
	if (!Sequencer.IsValid())
	{
		return nullptr;
	}

	TSharedRef<SWidget> SequencerWidget = Sequencer->GetSequencerWidget();

	TArray<FChildren*> ChildrenRemaining = { SequencerWidget->GetChildren() };

	while (!ChildrenRemaining.IsEmpty())
	{
		FChildren* const Children = ChildrenRemaining.Pop();
		if (!Children)
		{
			continue;
		}

		const int32 WidgetCount = Children->Num();

		for (int32 Index = 0; Index < WidgetCount; ++Index)
		{
			const TSharedRef<SWidget> Widget = Children->GetChildAt(Index);
			if (Widget->GetType() == TEXT("SOutlinerView"))
			{
				TSharedRef<UE::Sequencer::SOutlinerView> OutlinerView = StaticCastSharedRef<UE::Sequencer::SOutlinerView>(Widget);
				const_cast<FAvaSequencer*>(this)->OutlinerViewWeak = OutlinerView;
				return OutlinerView;
			}
			ChildrenRemaining.Add(Widget->GetChildren());
		}
	}

	return nullptr;
}

void FAvaSequencer::InitSequencerCommandList()
{
	TSharedPtr<ISequencer> Sequencer = SequencerWeak.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	TSharedPtr<FUICommandList> SequencerCommandList = Sequencer->GetCommandBindings(ESequencerCommandBindings::Sequencer);
	if (!ensure(SequencerCommandList.IsValid()))
	{
		return;
	}

	SequencerCommandList->Append(CommandList);

	const FGenericCommands& GenericCommands = FGenericCommands::Get();

	// Remap Duplicate Action
	if (const FUIAction* const DuplicateAction = SequencerCommandList->GetActionForCommand(GenericCommands.Duplicate))
	{
		FUIAction OverrideAction;

		OverrideAction.ExecuteAction = FExecuteAction::CreateSP(this
			, &FAvaSequencer::ExecuteSequencerDuplication
			, DuplicateAction->ExecuteAction);

		OverrideAction.CanExecuteAction = DuplicateAction->CanExecuteAction;

		SequencerCommandList->UnmapAction(GenericCommands.Duplicate);
		SequencerCommandList->MapAction(GenericCommands.Duplicate, OverrideAction);
	}

	// Unmap Key Transform Commands
	const FSequencerCommands& SequencerCommands = FSequencerCommands::Get();
	SequencerCommandList->UnmapAction(SequencerCommands.AddTransformKey);
	SequencerCommandList->UnmapAction(SequencerCommands.AddTranslationKey);
	SequencerCommandList->UnmapAction(SequencerCommands.AddRotationKey);
	SequencerCommandList->UnmapAction(SequencerCommands.AddScaleKey);
}

void FAvaSequencer::ExecuteSequencerDuplication(FExecuteAction InExecuteAction)
{
	if (UWorld* InWorld = Provider.GetPlaybackContext()->GetWorld())
	{
		// HACK: Sequencer Duplicates Actors via UUnrealEdEngine::edactDuplicateSelected
		// Sequencer then expects that after this function is called, the GSelectedActors are the newly duplicated actors.
		// However, ULevelFactory::FactoryCreateText only changes selection when the World in question is the GWorld,
		// which is not true for Motion Design.
		// So for this we temporarily set GWorld to our Motion Design World so that selections happen correctly.
		UWorld* const OldGWorld = GWorld;
		GWorld = InWorld;
		InExecuteAction.ExecuteIfBound();
		GWorld = OldGWorld;
	}
}

void FAvaSequencer::OnUpdateCameraCut(UObject* InCameraObject, bool bInJumpCut)
{
	GetProvider().OnUpdateCameraCut(InCameraObject, bInJumpCut);
}

void FAvaSequencer::ExtendSidebarMarkedFramesMenu(FMenuBuilder& OutMenuBuilder)
{
	UAvaSequence* const Sequence = GetViewedSequence();
	if (!Sequence)
	{
		return;
	}

	UMovieScene* const MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	TSharedPtr<ISequencer> Sequencer = GetSequencerPtr();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<UE::Sequencer::FSequencerSelection> SequencerSelection = Sequencer->GetViewModel()->GetSelection();
	if (!SequencerSelection.IsValid())
	{
		return;
	}

	const TArray<FMovieSceneMarkedFrame>& MarkedFrames = MovieScene->GetMarkedFrames();

	for (const int32 MarkIndex : SequencerSelection->MarkedFrames)
	{
		if (MarkedFrames.IsValidIndex(MarkIndex))
		{
			const TSharedRef<SWidget> DetailsWidget = SNew(SAvaMarkDetails, Sequence, MarkedFrames[MarkIndex]);
			OutMenuBuilder.AddWidget(DetailsWidget, FText::GetEmpty(), /*bInNoIndent=*/true);
		}
	}
}

void FAvaSequencer::OnSequencerClosed(const TSharedRef<ISequencer> InSequencer)
{
	if (InSequencer != SequencerWeak)
	{
		return;
	}

	for (const TSharedRef<FAvaSequencerAction>& SequencerAction : SequencerActions)
	{
		SequencerAction->OnSequencerClosed();
	}

	// Reset as the sequencer object might still remain valid, but FAvaSequencer should no longer link to that sequencer weak instance.
	SequencerWeak.Reset();
}

const TArray<TWeakObjectPtr<UAvaSequence>>& FAvaSequencer::GetRootSequences() const
{
	IAvaSequenceProvider* const SequenceProvider = Provider.GetSequenceProvider();
	check(SequenceProvider);
	return SequenceProvider->GetRootSequences();
}

#undef LOCTEXT_NAMESPACE
