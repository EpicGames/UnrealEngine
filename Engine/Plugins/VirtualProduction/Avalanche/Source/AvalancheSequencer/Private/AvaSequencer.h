// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequenceItemShared.h"
#include "EditorUndoClient.h"
#include "Framework/Commands/UIAction.h"
#include "IAvaSequencer.h"
#include "IAvaSequencerProvider.h"
#include "MovieSceneSequenceID.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Views/STreeView.h"

class AActor;
class FAvaEditorSelection;
class FAvaNavigationToolProvider;
class FAvaSequencerAction;
class FAvaSequencerCleanView;
class FExtender;
class FMenuBuilder;
class FUICommandList;
class IAvaSequenceColumn;
class IAvaSequencerController;
class ISequencer;
class SAvaSequenceTree;
class SHeaderRow;
class SWidget;
class UAvaSequence;
class UMovieScene;
class USequencerSettings;
enum class EMovieSceneTransformChannel : uint32;
struct FAvaSequencePreset;
struct FAvaSequencerArgs;
struct FMovieSceneBinding;
struct FMovieScenePossessable;
struct FSidebarState;

namespace UE
{
	namespace Sequencer
	{
		class SOutlinerView;
	}

	namespace SequenceNavigator
	{
		class FNavigationToolAvaSequence;
	}
}

class FAvaSequencer : public IAvaSequencer, public FEditorUndoClient, public TSharedFromThis<FAvaSequencer>
{
public:
	static const FName SidebarDrawerId;

	explicit FAvaSequencer(IAvaSequencerProvider& InProvider, FAvaSequencerArgs&& InArgs);

	virtual ~FAvaSequencer() override;

	void BindCommands();

	TSharedPtr<FUICommandList> GetCommandList() const { return CommandList; }

	TSharedPtr<IAvaSequenceColumn> FindSequenceColumn(FName InColumnName) const;

	/** Makes sure the Sequencer is instantiated */
	void EnsureSequencer();

	TSharedRef<ISequencer> CreateSequencer();

	IAvaSequencerProvider& GetProvider() { return Provider; }

	void GetSelectedObjects(const TArray<FGuid>& InObjectGuids
		, TArray<UObject*>& OutSelectedActors
		, TArray<UObject*>& OutSelectedComponents
		, TArray<UObject*>& OutSelectedObjects) const;

	bool IsBindingSelected(const FMovieSceneBinding& InBinding) const;

	/** Syncs from Sequencer Selection to Editor Selection */
	void OnSequencerSelectionChanged(TArray<FGuid> InObjectGuids);

	/** Gets the extender to use for sequencers context sensitive menus and toolbars */
	TSharedRef<FExtender> GetAddTrackSequencerExtender(const TSharedRef<FUICommandList> InCommandList
		, const TArray<UObject*> InContextSensitiveObjects);

	/** Extends the sequencer add track menu */
	void ExtendSequencerAddTrackMenu(FMenuBuilder& OutAddTrackMenuBuilder
		, const TArray<UObject*> InContextObjects);

	void NotifyViewedSequenceChanged(UAvaSequence* InOldSequence);

	UAvaSequence* CreateSequence() const;

	UObject* GetPlaybackContext() const;

	TSharedRef<SWidget> GetSequenceTreeWidget();

	TSharedRef<SWidget> CreatePlayerToolBar(const TSharedRef<FUICommandList>& InCommandList);

	void OnSequenceSearchChanged(const FText& InSearchText, FText& OutErrorMessage);

	void OnActivateSequence(FMovieSceneSequenceIDRef InSequenceID);

	/** Called when an Sequence has Started Playing in either Sequencer or the Sequence Panel */
	void NotifyOnSequencePlayed();

	/** Called when an Sequence has Stopped Playing in either Sequencer or the Sequence Panel */
	void NotifyOnSequenceStopped();

	/** Called when an actor's label has changed */
	void OnActorLabelChanged(AActor* InActor);

	void OnMovieSceneBindingsPasted(const TArray<FMovieSceneBinding>& InBindings);

	const TArray<FAvaSequenceItemPtr>& GetRootSequenceItems() const { return RootSequenceItems; }

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnViewedSequenceChanged, UAvaSequence*);
	FOnViewedSequenceChanged& GetOnViewedSequenceChanged() { return OnViewedSequenceChanged; }

	FOnSequenceAdded& OnSequenceAdded() override { return OnSequenceAddedDelegate; }
	FOnSequenceRemoved& OnSequenceRemoved() override { return OnSequenceRemovedDelegate; }

	void AddTransformKey(EMovieSceneTransformChannel InTransformChannel);

	void ApplyDefaultPresetToSelection(FName InPresetName);
	void ApplyCustomPresetToSelection(FName InPresetName);
	void ApplyPresetToSelection(const FAvaSequencePreset& InPreset);

	/**
 	 * Attempts to find the best matching subobject of the given Parent Object
 	 * based on the information provided by the given possessable
 	 * @param InParentObject the outer from which to find the object
 	 * @param InPossessable the possessable to retrieve things like name (label) and object class
 	 * @return the matching subobject if found, or nullptr if not found
 	 */
	static UObject* FindObjectToPossess(UObject* InParentObject, const FMovieScenePossessable& InPossessable);

	/** Resets the pre-animated state of the sequencer, keeping the current state of the world even when sequencer is saved/closed */
	void ApplyCurrentState();

	void FixBindingPaths();

	void FixInvalidBindings();

	/** Gets the Object Name to use for a Possessable Name */
	static FString GetObjectName(UObject* InObject);

	/**
	 * Recursively finds the most appropriate Resolution Context for a given Parent Guid of a Possessable
	 * Uses InFindObjectsFunc to resolve the object.
	 */
	static UObject* FindResolutionContext(UAvaSequence& InSequence
		, UMovieScene& InMovieScene
		, const FGuid& InParentPossessableGuid
		, UObject* InPlaybackContext
		, TFunctionRef<TArray<UObject*, TInlineAllocator<1>>(const FGuid& InGuid, UObject* InContextChecked)> InFindObjectsFunc);

	static UObject* FindResolutionContext(UAvaSequence& InSequence
		, UMovieScene& InMovieScene
		, const FGuid& InParentGuid
		, UObject* InPlaybackContext);

	bool FixPossessable(UAvaSequence& InSequence
		, const FMovieScenePossessable& InPossessable
		, UObject* InPlaybackContext
		, TSet<const FMovieScenePossessable*>& InOutProcessedPossessables);

	void FixBindingHierarchy();

	TArrayView<TWeakObjectPtr<>> ResolveBoundObjects(const FGuid& InBindingId, UMovieSceneSequence* Sequence) const;

	TSharedRef<ISequencer> GetOrCreateSequencer() const;

	//~ Begin IAvaSequencer
	virtual const IAvaSequencerProvider& GetProvider() const override { return Provider; }
	virtual TSharedPtr<ISequencer> GetSequencerPtr() const override;
	virtual USequencerSettings* GetSequencerSettings() const override;
	virtual void SetBaseCommandList(const TSharedPtr<FUICommandList>& InBaseCommandList) override;
	virtual UAvaSequence* GetViewedSequence() const override;
	virtual UAvaSequence* GetDefaultSequence() const override;
	virtual void SetViewedSequence(UAvaSequence* InSequenceToView) override;
	virtual TArray<UAvaSequence*> GetSequencesForObject(UObject* InObject) const override;
	virtual TSharedRef<SWidget> CreateSequenceWidget() override;
	virtual void OnActorsCopied(FString& InOutCopiedData, TConstArrayView<AActor*> InCopiedActors) override;
	virtual void OnActorsPasted(FStringView InPastedData, const TMap<FName, AActor*>& InPastedActors) override;
	virtual void OnEditorSelectionChanged(const FAvaEditorSelection& InEditorSelection) override;
	virtual void NotifyOnSequenceTreeChanged() override;
	virtual const TArray<TWeakObjectPtr<UAvaSequence>>& GetRootSequences() const override;
	virtual bool CanAddSequence() const override;
	virtual UAvaSequence* AddSequence(UAvaSequence* const InParentSequence = nullptr) override;
	virtual void DeleteSequences(const TSet<UAvaSequence*>& InSequences) override;
	virtual uint32 AddSequenceFromPresets(TConstArrayView<const FAvaSequencePreset*> InPresets) override;
	//~ End IAvaSequencer

	/** Creates new sequences based on a given preset group */
	void AddSequencesFromPresetGroup(FName InPresetGroup);

	bool AddSequence_CanExecute() const;
	void AddSequence_Execute();

	bool DuplicateSequences_CanExecute() const;
	void DuplicateSequences_Execute();

	bool ExportSequences_CanExecute() const;
	void ExportSequences_Execute();

	bool SpawnPlayers_CanExecute() const;
	void SpawnPlayers_Execute();

	bool DeleteSequences_CanExecute() const;
	void DeleteSequences_Execute();

	bool RelabelSequence_CanExecute() const;
	void RelabelSequence_Execute();

	bool PlaySequences_CanExecute() const;
	void PlaySequences_Execute();

	bool ContinueSequences_CanExecute() const;
	void ContinueSequences_Execute();

	bool StopSequences_CanExecute() const;
	void StopSequences_Execute();

	/** Called when PostUndo has been called for the given sequence */
	void OnSequenceEditUndo(UAvaSequence* InSequence);

	//~ Begin FEditorUndoClient
	virtual void PostUndo(bool bInSuccess) override;
	virtual void PostRedo(bool bInSuccess) override;
	// End of FEditorUndoClient

private:
	TArray<UAvaSequence*> GetSelectedSequences() const;

	TSharedPtr<UE::Sequencer::SOutlinerView> GetOutlinerView() const;

	void InitSequencerCommandList();

	void ExecuteSequencerDuplication(FExecuteAction InExecuteAction);

	void OnUpdateCameraCut(UObject* InCameraObject, bool bInJumpCut);

	void ExtendSidebarMarkedFramesMenu(FMenuBuilder& OutMenuBuilder);

	void OnSequencerClosed(const TSharedRef<ISequencer> InSequencer);

	FOnViewedSequenceChanged OnViewedSequenceChanged;

	FOnSequenceAdded OnSequenceAddedDelegate;
	FOnSequenceRemoved OnSequenceRemovedDelegate;

	IAvaSequencerProvider& Provider;

	/** The active sequencer to use. This is set to Weak as FSequencer::Tick does not allow to have more than 1 ref (outside of menus) */
	TWeakPtr<ISequencer> SequencerWeak;

	/** the instanced sequencer if no external sequencer specified. Will be null if using an External Sequencer */
	TSharedPtr<ISequencer> InstancedSequencer;

	TWeakPtr<UE::Sequencer::SOutlinerView> OutlinerViewWeak; 

	TSharedPtr<IAvaSequencerController> SequencerController;

	TSharedRef<FUICommandList> CommandList;

	TWeakObjectPtr<UAvaSequence> ViewedSequenceWeak;

	FDelegateHandle SequencerAddTrackExtenderHandle;

	// BEGIN REMOVE AFTER NAVIGATION TOOL INTEGRATION
	TMap<FName, TSharedPtr<IAvaSequenceColumn>> SequenceColumns;

	TArray<FAvaSequenceItemPtr> RootSequenceItems;

	TSharedPtr<SHeaderRow> SequenceTreeHeaderRow;

	TSharedPtr<SAvaSequenceTree> SequenceTree;

	TSharedPtr<STreeView<FAvaSequenceItemPtr>> SequenceTreeView;
	// END REMOVE AFTER NAVIGATION TOOL INTEGRATION

	TSharedRef<FAvaSequencerCleanView> CleanView;

	/** Handle to the Sequence Edit Undo delegate*/
	FDelegateHandle OnSequenceEditUndoHandle;

	FDelegateHandle OnSequenceStartedHandle;

	FDelegateHandle OnSequenceFinishedHandle;

	/** Handle to when the actor label has changed */
	FDelegateHandle OnActorLabelChangedHandle;

	TArray<TSharedRef<FAvaSequencerAction>> SequencerActions;

	/** prevents reentry when synchronizing Sequencer <-> Provider Selection  */
	bool bUpdatingSelection = false;

	bool bSelectedFromSequencer = false;

	/** Whether custom clean playback mode will ever be used (set via FAvaSequencerArgs::bUseCustomCleanPlaybackMode) */
	const bool bUseCustomCleanPlaybackMode;

	/** Whether FAvaSequencer is allowed to select to/from the ISequencer instance */
	const bool bCanProcessSequencerSelections;

	TSharedPtr<FExtender> SidebarExtender;

	FDelegateHandle SidebarSelectionExtenderHandle;
};
