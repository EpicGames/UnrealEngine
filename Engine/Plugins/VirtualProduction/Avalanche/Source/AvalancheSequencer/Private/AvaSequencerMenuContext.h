// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "AvaSequencerMenuContext.generated.h"

class FAvaSequencer;

UCLASS()
class UAvaSequencerMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TSharedPtr<FAvaSequencer> GetAvaSequencer() const
	{
		return AvaSequencerWeak.Pin();
	}

	void SetAvaSequencer(const TWeakPtr<FAvaSequencer>& InAvaSequencerWeak)
	{
		AvaSequencerWeak = InAvaSequencerWeak;
	}

private:
	TWeakPtr<FAvaSequencer> AvaSequencerWeak;
};
