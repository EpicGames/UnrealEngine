// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequencerModule.h"
#include "AvaSequence.h"
#include "AvaSequenceEditor.h"
#include "AvaSequenceName.h"
#include "AvaSequencer.h"
#include "AvaSequencerUtils.h"
#include "Commands/AvaSequencerCommands.h"
#include "Customization/AvaDisplayRateCustomization.h"
#include "Customization/AvaMarkSettingCustomization.h"
#include "Customization/AvaSequenceCustomization.h"
#include "Customization/AvaSequenceNameCustomization.h"
#include "Customization/AvaSequenceTimeCustomization.h"
#include "Director/AvaSequenceDirectorBlueprint.h"
#include "Director/AvaSequenceDirectorCompiler.h"
#include "Editor.h"
#include "IAvaOutliner.h"
#include "IAvaOutlinerModule.h"
#include "ISequencerModule.h"
#include "Item/AvaOutlinerActor.h"
#include "ItemProxies/AvaOutlinerItemProxyRegistry.h"
#include "Modules/ModuleManager.h"
#include "ObjectBinding/AvaSequencerObjectBinding.h"
#include "Outliner/AvaOutlinerSequence.h"
#include "Outliner/AvaOutlinerSequenceProxy.h"
#include "Schemas/AvaActorSubobjectSchema.h"
#include "SequencerChannelInterface.h"
#include "SequencerCustomizationManager.h"
#include "SequencerSettings.h"
#include "Settings/AvaSequencerSettings.h"

#define LOCTEXT_NAMESPACE "AvaSequencerModule"

void FAvaSequencerModule::StartupModule()
{
	FAvaSequencerCommands::Register();

	ActorSubobjectSchema = MakeShared<FAvaActorSubobjectSchema>();

	ISequencerModule& SequencerModule = FAvaSequencerUtils::GetSequencerModule();

	SequencerModule.RegisterObjectSchema(ActorSubobjectSchema);

	RegisterSequenceEditor(SequencerModule);
	RegisterTrackEditors(SequencerModule);
	RegisterObjectBindings(SequencerModule);
	RegisterCustomizations(SequencerModule);
	RegisterOutlinerItems();
	RegisterCustomLayouts();
	RegisterDirectorCompiler();
}

void FAvaSequencerModule::ShutdownModule()
{
	FAvaSequencerCommands::Unregister();

	if (FAvaSequencerUtils::IsSequencerModuleLoaded())
	{
		ISequencerModule& SequencerModule = FAvaSequencerUtils::GetSequencerModule();

		SequencerModule.UnregisterObjectSchema(ActorSubobjectSchema);
		ActorSubobjectSchema.Reset();

		UnregisterSequenceEditor(SequencerModule);
		UnregisterTrackEditors(SequencerModule);
		UnregisterObjectBindings(SequencerModule);
		UnregisterCustomizations(SequencerModule);
	}

	UnregisterOutlinerItems();
	UnregisterCustomLayouts();
}

void FAvaSequencerModule::RegisterSequenceEditor(ISequencerModule& InSequencerModule)
{
	SequenceEditorHandle = InSequencerModule.RegisterSequenceEditor(UAvaSequence::StaticClass()
		, MakeUnique<FAvaSequenceEditor>());
}

void FAvaSequencerModule::UnregisterSequenceEditor(ISequencerModule& InSequencerModule)
{
	InSequencerModule.UnregisterSequenceEditor(SequenceEditorHandle);
}

void FAvaSequencerModule::RegisterTrackEditors(ISequencerModule& InSequencerModule)
{
}

void FAvaSequencerModule::UnregisterTrackEditors(ISequencerModule& InSequencerModule)
{
	for (FDelegateHandle TrackEditorHandle : TrackEditorHandles)
	{
		InSequencerModule.UnRegisterTrackEditor(TrackEditorHandle);
	}
	TrackEditorHandles.Empty();
}

void FAvaSequencerModule::RegisterObjectBindings(ISequencerModule& InSequencerModule)
{
	RegisterObjectBinding<FAvaSequencerObjectBinding>(InSequencerModule);
}

void FAvaSequencerModule::UnregisterObjectBindings(ISequencerModule& InSequencerModule)
{
	for (FDelegateHandle ObjectBindingHandle : ObjectBindingHandles)
	{
		InSequencerModule.UnRegisterEditorObjectBinding(ObjectBindingHandle);
	}
	ObjectBindingHandles.Empty();
}

void FAvaSequencerModule::RegisterCustomizations(ISequencerModule& InSequencerModule)
{
	RegisterCustomization<UAvaSequence, FAvaSequenceCustomization>(InSequencerModule);
}

void FAvaSequencerModule::UnregisterCustomizations(ISequencerModule& InSequencerModule)
{
	TSharedPtr<FSequencerCustomizationManager> CustomizationManager = InSequencerModule.GetSequencerCustomizationManager();
	
	if (!CustomizationManager.IsValid())
	{
		return;
	}
	
	for (const UClass* Class : CustomizedClasses)
	{
		CustomizationManager->UnregisterInstancedSequencerCustomization(Class);
	}
	CustomizedClasses.Empty();
}

void FAvaSequencerModule::RegisterOutlinerItems()
{
	FAvaOutlinerItemProxyRegistry& ItemProxyRegistry = IAvaOutlinerModule::Get().GetItemProxyRegistry();

	ItemProxyRegistry.RegisterItemProxyWithDefaultFactory<FAvaOutlinerSequenceProxy, 30>();

	OutlinerProxiesExtensionDelegateHandle = IAvaOutlinerModule::Get().GetOnExtendItemProxiesForItem().AddLambda(
		[](IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InItem, TArray<TSharedPtr<FAvaOutlinerItemProxy>>& OutItemProxies)
		{
			if (InItem->IsA<FAvaOutlinerActor>())
			{
				if (TSharedPtr<FAvaOutlinerItemProxy> SequenceProxy = InOutliner.GetOrCreateItemProxy<FAvaOutlinerSequenceProxy>(InItem))
				{
					OutItemProxies.Add(SequenceProxy);
				}
			}
		});
}

void FAvaSequencerModule::UnregisterOutlinerItems()
{
	if (IAvaOutlinerModule::IsLoaded())
	{
		IAvaOutlinerModule::Get().GetOnExtendItemProxiesForItem().Remove(OutlinerProxiesExtensionDelegateHandle);
		OutlinerProxiesExtensionDelegateHandle.Reset();
	}
}

void FAvaSequencerModule::RegisterCustomLayouts()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	RegisterCustomPropertyTypeLayout<FAvaSequencerDisplayRate, FAvaDisplayRateCustomization>(PropertyModule);
	RegisterCustomPropertyTypeLayout<FAvaMarkSetting, FAvaMarkSettingCustomization>(PropertyModule);
	RegisterCustomPropertyTypeLayout<FAvaSequenceTime, FAvaSequenceTimeCustomization>(PropertyModule);
	RegisterCustomPropertyTypeLayout<FAvaSequenceName, FAvaSequenceNameCustomization>(PropertyModule);
}

void FAvaSequencerModule::UnregisterCustomLayouts()
{
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		// Unregister Custom Property Type Layouts
		for (FName PropertyTypeLayout : PropertyTypeLayouts)
		{
			PropertyModule.UnregisterCustomPropertyTypeLayout(PropertyTypeLayout);
		}
		PropertyTypeLayouts.Empty();
	}
}

void FAvaSequencerModule::RegisterDirectorCompiler()
{
	// Register Compiler Context
	FKismetCompilerContext::RegisterCompilerForBP(UAvaSequenceDirectorBlueprint::StaticClass(),
		[](UBlueprint* InBlueprint, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompilerOptions)
		{
			return MakeShared<FAvaSequenceDirectorCompilerContext>(CastChecked<UAvaSequenceDirectorBlueprint>(InBlueprint)
				, InMessageLog
				, InCompilerOptions);
		}
	);
}

IMPLEMENT_MODULE(FAvaSequencerModule, AvalancheSequencer)

#undef LOCTEXT_NAMESPACE
