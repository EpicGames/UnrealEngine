// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/IDelegateInstance.h"
#include "ISequencerModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SequencerCustomizationManager.h"

class FAvaActorSubobjectSchema;
class ISequencerModule;
class UClass;

class FAvaSequencerModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	static FAvaSequencerModule& Get()
	{
		return FModuleManager::GetModuleChecked<FAvaSequencerModule>(TEXT("AvalancheSequencer"));
	}

private:
	void RegisterSequenceEditor(ISequencerModule& InSequencerModule);
	void UnregisterSequenceEditor(ISequencerModule& InSequencerModule);

	template<typename InTrackEditorType>
	void RegisterTrackEditor(ISequencerModule& InSequencerModule)
	{
		TrackEditorHandles.Add(InSequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateLambda(
			[](const TSharedRef<ISequencer>& InSequencer)->TSharedRef<InTrackEditorType>
			{
				return MakeShared<InTrackEditorType>(InSequencer);
			}))
		);
	}

	template<typename InTrackEditorType>
	void RegisterPropertyTrackEditor(ISequencerModule& InSequencerModule)
	{
		TrackEditorHandles.Add(InSequencerModule.RegisterPropertyTrackEditor<InTrackEditorType>());
	}

	void RegisterTrackEditors(ISequencerModule& InSequencerModule);
	void UnregisterTrackEditors(ISequencerModule& InSequencerModule);

	template<typename InObjectBindingType>
	void RegisterObjectBinding(ISequencerModule& InSequencerModule)
	{
		ObjectBindingHandles.Add(InSequencerModule.RegisterEditorObjectBinding(FOnCreateEditorObjectBinding::CreateLambda(
			[](const TSharedRef<ISequencer>& InSequencer)->TSharedRef<InObjectBindingType>
			{
				return MakeShared<InObjectBindingType>(InSequencer);
			}))
		);
	}
	
	void RegisterObjectBindings(ISequencerModule& InSequencerModule);
	void UnregisterObjectBindings(ISequencerModule& InSequencerModule);

	template<typename InClassType, typename InCustomizationType>
	void RegisterCustomization(ISequencerModule& InSequencerModule)
	{
		CustomizedClasses.Add(InClassType::StaticClass());
		
		InSequencerModule.GetSequencerCustomizationManager()->RegisterInstancedSequencerCustomization(InClassType::StaticClass()
			, FOnGetSequencerCustomizationInstance::CreateLambda([]()
			{
				return new InCustomizationType;
			}));
	}

	void RegisterCustomizations(ISequencerModule& InSequencerModule);
	void UnregisterCustomizations(ISequencerModule& InSequencerModule);

	void RegisterOutlinerItems();
	void UnregisterOutlinerItems();

	template<typename InType>
	FName GetTypeName() const
	{
		if constexpr (TModels<CStaticStructProvider, InType>::Value)
		{
			return InType::StaticStruct()->GetFName();
		}
		else if constexpr (TModels<CStaticClassProvider, InType>::Value)
		{
			return InType::StaticClass()->GetFName();
		}
		else
		{
			return NAME_None;
		}
	}

	template<typename InPropertyType, typename InPropertyTypeCustomizationType>
	void RegisterCustomPropertyTypeLayout(FPropertyEditorModule& InPropertyModule)
	{
		const FName TypeName = GetTypeName<InPropertyType>();
		PropertyTypeLayouts.Add(TypeName);
		
		InPropertyModule.RegisterCustomPropertyTypeLayout(TypeName
			, FOnGetPropertyTypeCustomizationInstance::CreateLambda([]
			{
				return MakeShared<InPropertyTypeCustomizationType>();
			}));
	}

	void RegisterCustomLayouts();
	void UnregisterCustomLayouts();

	void RegisterDirectorCompiler();

	TSharedPtr<FAvaActorSubobjectSchema> ActorSubobjectSchema;

	FDelegateHandle SequenceEditorHandle;

	TArray<FName> PropertyTypeLayouts;

	TArray<FDelegateHandle> TrackEditorHandles;

	TArray<FDelegateHandle> ObjectBindingHandles;

	FDelegateHandle OutlinerProxiesExtensionDelegateHandle;

	TArray<const UClass*> CustomizedClasses;
};
