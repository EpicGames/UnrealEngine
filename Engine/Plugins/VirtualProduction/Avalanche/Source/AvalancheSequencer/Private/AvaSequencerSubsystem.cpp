// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequencerSubsystem.h"
#include "AvaSequencer.h"
#include "Engine/EngineTypes.h"

TSharedRef<IAvaSequencer> UAvaSequencerSubsystem::GetOrCreateSequencer(IAvaSequencerProvider& InProvider, FAvaSequencerArgs&& InArgs)
{
	TSharedPtr<IAvaSequencer> Sequencer = SequencerWeak.Pin();
	if (!Sequencer.IsValid() || &Sequencer->GetProvider() != &InProvider)
	{
		Sequencer = MakeShared<FAvaSequencer>(InProvider, MoveTemp(InArgs));
	}
	SequencerWeak = Sequencer;
	return Sequencer.ToSharedRef();
}

bool UAvaSequencerSubsystem::DoesSupportWorldType(const EWorldType::Type InWorldType) const
{
	return InWorldType == EWorldType::Type::Editor;
}
