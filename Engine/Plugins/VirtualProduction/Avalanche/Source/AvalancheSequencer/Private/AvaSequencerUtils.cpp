// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequencerUtils.h"
#include "AvaSceneSubsystem.h"
#include "AvaSequencerSubsystem.h"
#include "Engine/World.h"
#include "IAvaSceneInterface.h"
#include "IAvaSequencer.h"
#include "Playback/AvaSequencerController.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/IToolkitHost.h"

TSharedRef<IAvaSequencerController> FAvaSequencerUtils::CreateSequencerController()
{
	return MakeShared<FAvaSequencerController>();
}

UWorld* FAvaSequencerUtils::GetSequencerWorld(const TSharedRef<ISequencer>& InSequencer)
{
	if (const TSharedPtr<IToolkitHost> ToolkitHost = InSequencer->GetToolkitHost())
	{
		return ToolkitHost->GetWorld();
	}
	return nullptr;
}

UAvaSequencerSubsystem* FAvaSequencerUtils::GetSequencerSubsystem(const TSharedRef<ISequencer>& InSequencer)
{
	if (const UWorld* const World = GetSequencerWorld(InSequencer))
	{
		return World->GetSubsystem<UAvaSequencerSubsystem>();
	}
	return nullptr;
}

UAvaSceneSubsystem* FAvaSequencerUtils::GetSceneSubsystem(const TSharedRef<ISequencer>& InSequencer)
{
	if (const UWorld* const World = GetSequencerWorld(InSequencer))
	{
		return World->GetSubsystem<UAvaSceneSubsystem>();
	}
	return nullptr;
}

IAvaSceneInterface* FAvaSequencerUtils::GetSceneInterface(const TSharedRef<ISequencer>& InSequencer)
{
	if (UAvaSceneSubsystem* const SceneSubsystem = GetSceneSubsystem(InSequencer))
	{
		return SceneSubsystem->GetSceneInterface();
	}
	return nullptr;
}

IAvaSequenceProvider* FAvaSequencerUtils::GetSequenceProvider(const TSharedRef<ISequencer>& InSequencer)
{
	if (IAvaSceneInterface* const SceneInterface = GetSceneInterface(InSequencer))
	{
		return SceneInterface->GetSequenceProvider();
	}
	return nullptr;
}

TSharedPtr<IAvaSequencer> FAvaSequencerUtils::GetAvaSequencer(const TSharedRef<ISequencer>& InSequencer)
{
	if (UAvaSequencerSubsystem* const SequencerSubsystem = GetSequencerSubsystem(InSequencer))
	{
		return SequencerSubsystem->GetSequencer();
	}
	return nullptr;
}
