// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceExporter.h"
#include "AvaSequencer.h"
#include "GameFramework/Actor.h"
#include "Misc/StringOutputDevice.h"
#include "MovieScene.h"
#include "MovieSceneBindingProxy.h"
#include "AvaSequence.h"
#include "UnrealExporter.h"
#include "UObject/Package.h"
#include "SequencerUtilities.h"

namespace UE::AvaSequencer::Private
{
	class FAvaSequenceExportObjectInnerContext : public FExportObjectInnerContext
	{
	public:
		explicit FAvaSequenceExportObjectInnerContext(const TSharedRef<FAvaSequencer>& InAvaSequencer)
			: AvaSequencerWeak(InAvaSequencer)
		{
		}
		virtual ~FAvaSequenceExportObjectInnerContext() override = default;

		//~ Begin FExportObjectInnerContext
		virtual bool IsObjectSelected(const UObject* InObject) const override { return true; }
		//~ End FExportObjectInnerContext

		void SetBoundActors(const TArray<AActor*>& InBoundActors)
		{
			BoundActors = InBoundActors;
		}

		TConstArrayView<AActor*> GetBoundActors() const
		{
			return BoundActors;
		}

		UObject* GetPlaybackContext() const
		{
			return AvaSequencerWeak.IsValid() ? AvaSequencerWeak.Pin()->GetPlaybackContext() : nullptr;
		}

		TSharedPtr<ISequencer> GetSequencer() const
		{
			return AvaSequencerWeak.IsValid() ? AvaSequencerWeak.Pin()->GetSequencerPtr() : nullptr;
		}

	private:
		TArray<TObjectPtr<AActor>> BoundActors;

		TWeakPtr<FAvaSequencer> AvaSequencerWeak;
	};
}

FAvaSequenceExporter::FAvaSequenceExporter(const TSharedRef<FAvaSequencer>& InAvaSequencer)
	: AvaSequencerWeak(InAvaSequencer)
{
}

void FAvaSequenceExporter::ExportText(FString& InOutCopiedData, TConstArrayView<AActor*> InCopiedActors)
{
	TSharedPtr<FAvaSequencer> AvaSequencer = AvaSequencerWeak.Pin();
	if (!AvaSequencer.IsValid())
	{
		return;
	}

	TMap<UAvaSequence*, TArray<AActor*>> SequenceMap;

	// Gather the Sequences that the Copied Actors are bound to
	for (AActor* const Actor : InCopiedActors)
	{
		TArray<UAvaSequence*> Sequences = AvaSequencer->GetSequencesForObject(Actor);
		for (UAvaSequence* const Sequence : Sequences)
		{
			SequenceMap.FindOrAdd(Sequence).Add(Actor);
		}
	}

	// If there are no sequences, no need to add anything on our side, early return.
	if (SequenceMap.IsEmpty())
	{
		return;
	}

	UObject* const PlaybackContext = AvaSequencer->GetPlaybackContext();
	UE::AvaSequencer::Private::FAvaSequenceExportObjectInnerContext ExportContext(AvaSequencer.ToSharedRef());

	const TCHAR* const Filetype = TEXT("copy");
	const uint32 PortFlags = PPF_DeepCompareInstances | PPF_ExportsNotFullyQualified;
	const int32 IndentLevel = 0;

	for (const TPair<UAvaSequence*, TArray<AActor*>>& Pair : SequenceMap)
	{
		UAvaSequence* const Sequence = Pair.Key;
		const TArray<AActor*>& BoundActors = Pair.Value;

		if (BoundActors.IsEmpty())
		{
			continue;
		}

		ExportContext.SetBoundActors(BoundActors);

		FStringOutputDevice Ar;
		UExporter::ExportToOutputDevice(&ExportContext, Sequence, nullptr, Ar, Filetype, IndentLevel, PortFlags);
		InOutCopiedData += Ar;
	}
}

UAvaSequenceExporter::UAvaSequenceExporter()
{
	SupportedClass = UAvaSequence::StaticClass();
	bText = true;
	PreferredFormatIndex = 0;
	FormatExtension.Add(TEXT("copy"));
	FormatDescription.Add(TEXT("Motion Design Sequence"));
}

bool UAvaSequenceExporter::ExportText(const FExportObjectInnerContext* InContext, UObject* InObject, const TCHAR* InType
	, FOutputDevice& Ar, FFeedbackContext* InWarn, uint32 InPortFlags)
{
	UAvaSequence* const Sequence = Cast<UAvaSequence>(InObject);
	if (!InContext || !IsValid(Sequence))
	{
		return false;
	}

	const UE::AvaSequencer::Private::FAvaSequenceExportObjectInnerContext& Context
		= static_cast<const UE::AvaSequencer::Private::FAvaSequenceExportObjectInnerContext&>(*InContext);

	TSharedPtr<ISequencer> Sequencer = Context.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	UObject* const PlaybackContext = Context.GetPlaybackContext();

	// Gather guids for the object nodes and any child object nodes
	TArray<FMovieSceneBindingProxy> Bindings;

	auto TryGetBinding = [&Bindings, Sequence](UObject* InBoundObject)->bool
		{
			const FGuid Guid = Sequence->FindGuidFromObject(InBoundObject);

			if (Guid.IsValid())
			{
				Bindings.Add(FMovieSceneBindingProxy(Guid, Sequence));
				return true;
			}

			return false;
		};

	constexpr EObjectFlags ExclusionFlags = RF_Transient | RF_TextExportTransient | RF_BeginDestroyed | RF_FinishDestroyed;

	for (AActor* const Actor : Context.GetBoundActors())
	{
		if (TryGetBinding(Actor))
		{
			TArray<UObject*> Subobjects;
			GetObjectsWithOuter(Actor, Subobjects, EGetObjectsFlags::IncludeNestedObjects, ExclusionFlags);

			for (UObject* const Subobject : Subobjects)
			{
				TryGetBinding(Subobject);
			}
		}
	}

	if (Bindings.IsEmpty())
	{
		return false;
	}

	Ar.Logf(TEXT("%sBegin Sequence Label=%s\r\n")
		, FCString::Spc(TextIndent)
		, *Sequence->GetLabel().ToString());

	TArray<UMovieSceneFolder*> Folders;
	FSequencerUtilities::CopyBindings(Sequencer.ToSharedRef(), Bindings, Folders, Ar);

	Ar.Logf(TEXT("%sEnd Sequence\r\n"), FCString::Spc(TextIndent));

	return true;
}
