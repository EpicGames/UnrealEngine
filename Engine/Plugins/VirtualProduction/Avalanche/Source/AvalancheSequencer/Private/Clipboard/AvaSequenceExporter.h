// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Exporters/Exporter.h"
#include "Templates/SharedPointer.h"
#include "AvaSequenceExporter.generated.h"

class AActor;
class FAvaSequencer;
class UAvaSequenceCopyableBinding;

/** Handles exporting the given Copied Actors' Sequence Data to String Text */
class FAvaSequenceExporter
{
public:
	explicit FAvaSequenceExporter(const TSharedRef<FAvaSequencer>& InAvaSequencer);

	void ExportText(FString& InOutCopiedData, TConstArrayView<AActor*> InCopiedActors);

private:
	TWeakPtr<FAvaSequencer> AvaSequencerWeak;
};

UCLASS()
class UAvaSequenceExporter : public UExporter
{
	GENERATED_BODY()

public:
	UAvaSequenceExporter();

	//~ Begin UExporter
	virtual bool ExportText(const FExportObjectInnerContext* InContext, UObject* InObject, const TCHAR* InType
		, FOutputDevice& Ar, FFeedbackContext* InWarn, uint32 InPortFlags) override;
	//~ End UExporter
};