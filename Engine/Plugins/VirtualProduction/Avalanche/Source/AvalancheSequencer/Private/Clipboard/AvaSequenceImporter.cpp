// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceImporter.h"
#include "AvaSequence.h"
#include "AvaSequencer.h"
#include "EngineUtils.h"
#include "Factories.h"
#include "IAvaSequenceProvider.h"
#include "ISequencer.h"
#include "MovieScenePossessable.h"
#include "SequencerUtilities.h"
#include "UObject/Package.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Algo/Transform.h"

FAvaSequenceImporter::FAvaSequenceImporter(const TSharedRef<FAvaSequencer>& InAvaSequencer)
	: AvaSequencerWeak(InAvaSequencer)
{
}

void FAvaSequenceImporter::ImportText(FStringView InPastedData, const TMap<FName, AActor*>& InPastedActors)
{
	TSharedPtr<FAvaSequencer> AvaSequencer = AvaSequencerWeak.Pin();
	if (!AvaSequencer.IsValid())
	{
		return;
	}

	IAvaSequencerProvider& Provider  = AvaSequencer->GetProvider();

	TSharedPtr<ISequencer> Sequencer = AvaSequencer->GetSequencerPtr();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UAvaSequence* const OriginallyViewedSequence = AvaSequencer->GetViewedSequence();

	const TCHAR* Buffer = InPastedData.GetData();

	FString StringLine;
	while (FParse::Line(&Buffer, StringLine))
	{
		const TCHAR* Str = *StringLine;
		if (!ParseCommand(&Str, TEXT("Begin")))
		{
			continue;
		}

		FName SequenceLabel;
		FParse::Value(Str, TEXT("Label="), SequenceLabel);

		UAvaSequence* const SequenceToUse = GetOrCreateSequence(*AvaSequencer, SequenceLabel);
		if (!ensure(IsValid(SequenceToUse)))
		{
			continue;
		}

		UMovieScene* const MovieScene = SequenceToUse->GetMovieScene();
		if (!ensure(IsValid(MovieScene)))
		{
			continue;
		}

		UsedSequences.Add(SequenceToUse);

		AvaSequencer->SetViewedSequence(SequenceToUse);

		FString BindingsString, BindingsStringLine;
		while (!ParseCommand(&Buffer, TEXT("End")) && FParse::Line(&Buffer, BindingsStringLine))
		{
			BindingsString += *BindingsStringLine;
			BindingsString += TEXT("\r\n");
		}

		TArray<FNotificationInfo> PasteErrors; 
		TArray<FMovieSceneBindingProxy> OutBindings;
		FMovieScenePasteBindingsParams PasteParams;
		Algo::Transform(InPastedActors, PasteParams.PastedActors, [](TTuple<FName, AActor*> Pair) { return TTuple<FName, TObjectPtr<AActor>>(Pair.Key, Pair.Value);});
		FSequencerUtilities::PasteBindings(BindingsString, Sequencer.ToSharedRef(), PasteParams, OutBindings, PasteErrors);
	}

	AvaSequencer->SetViewedSequence(OriginallyViewedSequence);
}

bool FAvaSequenceImporter::ParseCommand(const TCHAR** InStream, const TCHAR* InToken)
{
	static const TCHAR* const SequenceToken = TEXT("Sequence");
	const TCHAR* Original = *InStream;

	if (FParse::Command(InStream, InToken) && FParse::Command(InStream, SequenceToken))
	{
		return true;
	}

	*InStream = Original;

	return false;
}

void FAvaSequenceImporter::ResetCopiedTracksFlags(UMovieSceneTrack* InTrack)
{
	if (!IsValid(InTrack))
	{
		return;
	}

	InTrack->ClearFlags(RF_Transient);

	for (UMovieSceneSection* Section : InTrack->GetAllSections())
	{
		Section->ClearFlags(RF_Transient);
		Section->PostPaste();
	}
}

UAvaSequence* FAvaSequenceImporter::GetOrCreateSequence(FAvaSequencer& InAvaSequencer, FName InSequenceLabel)
{
	IAvaSequenceProvider* const SequenceProvider = InAvaSequencer.GetProvider().GetSequenceProvider();
	if (!SequenceProvider)
	{
		return nullptr;
	}

	// Find an unused sequence that matches the given label
	const TObjectPtr<UAvaSequence>* const  FoundSequence = SequenceProvider->GetSequences().FindByPredicate(
		[InSequenceLabel, this](const UAvaSequence* const InSequence)
		{
			return IsValid(InSequence)
				&& InSequence->GetLabel() == InSequenceLabel
				&& !UsedSequences.Contains(InSequence);
		});

	if (FoundSequence)
	{
		return *FoundSequence;
	}

	UAvaSequence* const NewSequence = InAvaSequencer.CreateSequence();
	NewSequence->SetLabel(InSequenceLabel);
	SequenceProvider->AddSequence(NewSequence);

	return NewSequence;
}
