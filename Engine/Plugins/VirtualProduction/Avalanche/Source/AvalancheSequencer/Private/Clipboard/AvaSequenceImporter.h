// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/Set.h"
#include "MovieSceneSequenceID.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectKey.h"

class AActor;
class FAvaSequencer;
class ISequencer;
class UAvaSequence;
class UAvaSequenceCopyableBinding;
class UMovieScene;
class UMovieSceneTrack;
struct FMovieSceneBinding;
struct FMovieSceneBindingProxy;

/** Handles parsing String Data to import Bindings into an existing or new UAvaSequence instance */
class FAvaSequenceImporter
{
public:
	explicit FAvaSequenceImporter(const TSharedRef<FAvaSequencer>& InAvaSequencer);

	void ImportText(FStringView InPastedData, const TMap<FName, AActor*>& InPastedActors);

private:
	struct FImportContext
	{
		TSharedRef<ISequencer> Sequencer;
		UAvaSequence& Sequence;
		UMovieScene& MovieScene;
		FMovieSceneSequenceIDRef SequenceIDRef;
		const TMap<FName, AActor*>& PastedActors; 

		UAvaSequenceCopyableBinding* CopyableBinding;
	};

	bool ParseCommand(const TCHAR** InStream, const TCHAR* InToken);

	void ResetCopiedTracksFlags(UMovieSceneTrack* InTrack);

	UAvaSequence* GetOrCreateSequence(FAvaSequencer& InAvaSequencer, FName InSequenceLabel);

	TArray<UAvaSequenceCopyableBinding*> ImportBindings(ISequencer& InSequencerRef, const FString& InBindingsString);

	void ProcessImportedBindings(const TArray<UAvaSequenceCopyableBinding*>& InImportedBindings
		, const TMap<FName, AActor*>& InPastedActors);

	FMovieSceneBinding BindPossessable(const FImportContext& InContext);

	FMovieSceneBinding BindSpawnable(const FImportContext& InContext);

	void AddObjectsToBinding(const TSharedRef<ISequencer>& InSequencer
		, const TArray<UObject*>& InObjectsToAdd
		, const FMovieSceneBindingProxy& InObjectBinding
		, UObject* InResolutionContext);
	
	TWeakPtr<FAvaSequencer> AvaSequencerWeak;

	TSet<FObjectKey> UsedSequences;
};
