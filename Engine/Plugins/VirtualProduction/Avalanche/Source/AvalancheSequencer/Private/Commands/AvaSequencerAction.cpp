// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequencerAction.h"
#include "AvaSequencer.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"

TSharedPtr<UE::Sequencer::FSequencerSelection> FAvaSequencerAction::GetSequencerSelection() const
{
	using namespace UE::Sequencer;

	TSharedPtr<ISequencer> Sequencer = Owner.GetSequencerPtr();
	if (!Sequencer.IsValid())
	{
		return nullptr;
	}

	if (TSharedPtr<FSequencerEditorViewModel> EditorViewModel = Sequencer->GetViewModel())
	{
		return EditorViewModel->GetSelection();
	}

	return nullptr;
}
