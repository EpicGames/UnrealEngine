// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FAvaSequencer;
class FUICommandList;

namespace UE::Sequencer
{
	class FSequencerSelection;
}

class FAvaSequencerAction : public TSharedFromThis<FAvaSequencerAction>
{
public:
	explicit FAvaSequencerAction(FAvaSequencer& InOwner)
		: Owner(InOwner)
	{
	}

	virtual ~FAvaSequencerAction() = default;

	virtual void MapAction(const TSharedRef<FUICommandList>& InCommandList) = 0;

	virtual void OnSequencerCreated() {}

	virtual void OnSequencerClosed() {}

protected:
	/** Utility function to directly retrieve the Sequencer Selection from the Owner Ava Sequencer */
	TSharedPtr<UE::Sequencer::FSequencerSelection> GetSequencerSelection() const;

	FAvaSequencer& Owner;
}; 
