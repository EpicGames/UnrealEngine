// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequencerCommands.h"

#define LOCTEXT_NAMESPACE "AvaSequencerCommands"

void FAvaSequencerCommands::RegisterCommands()
{
	UI_COMMAND(AddNew
		, "Add New"
		, "Adds a new embedded Sequence to the level"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(PlaySelected
		, "Play"
		, "Plays all selected Sequences"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(ContinueSelected
		, "Continue"
		, "Continues all selected Sequences"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(StopSelected
		, "Stop"
		, "Stops all selected Sequences"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(ApplyCurrentState
		, "Apply Current State"
		, "Applies the current state of the sequence to the world by resetting the pre-animated state of the sequence."
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(FixBindingPaths
		, "Fix Binding Paths"
		, "Fixes the base path of the bindings, that typically breaks when duplicating or renaming an asset"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(FixInvalidBindings
		, "Fix Invalid Bindings"
		, "Attempts to fix the invalid bindings in the current viewed sequence"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(FixBindingHierarchy
		, "Fix Binding Hierarchy"
		, "Attempts to fix bindings with an invalid parent to the correct parent"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(ExportSequence
		, "Export"
		, "Exports the Sequence into its own Asset"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SpawnSequencePlayer
		, "Spawn Sequence Player"
		, "Spawns a Sequence Player for the selected Sequence"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(OpenStaggerTool
		, "Stagger Tool..."
		, "Staggers the selected layer bars"
		, EUserInterfaceActionType::Button
		, FInputChord());
}

#undef LOCTEXT_NAMESPACE
