// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

class FAvaSequencerCommands : public TCommands<FAvaSequencerCommands>
{
public:
	FAvaSequencerCommands()
		: TCommands<FAvaSequencerCommands>(TEXT("AvaSequencerCommands")
		, NSLOCTEXT("MotionDesignSequencerCommands", "MotionDesignSequencerCommands", "Motion Design Sequencer Commands")
		, NAME_None
		, FAppStyle::GetAppStyleSetName())
	{
	}

	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> AddNew;

	TSharedPtr<FUICommandInfo> PlaySelected;

	TSharedPtr<FUICommandInfo> ContinueSelected;

	TSharedPtr<FUICommandInfo> StopSelected;

	/** Applies the current state of the sequence to the world by resetting the pre-animated state of the sequence */
	TSharedPtr<FUICommandInfo> ApplyCurrentState;

	TSharedPtr<FUICommandInfo> FixBindingPaths;

	TSharedPtr<FUICommandInfo> FixInvalidBindings;

	TSharedPtr<FUICommandInfo> FixBindingHierarchy;

	TSharedPtr<FUICommandInfo> ExportSequence;

	TSharedPtr<FUICommandInfo> SpawnSequencePlayer;

	TSharedPtr<FUICommandInfo> OpenStaggerTool;
};
