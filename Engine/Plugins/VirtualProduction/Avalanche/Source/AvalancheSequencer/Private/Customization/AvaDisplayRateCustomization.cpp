// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaDisplayRateCustomization.h"
#include "DetailWidgetRow.h"
#include "Widgets/SAvaDisplayRate.h"

void FAvaDisplayRateCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle
	, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	InHeaderRow
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SAvaDisplayRate, InPropertyHandle)
		];
}
