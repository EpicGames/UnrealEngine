// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaMarkSettingCustomization.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "Settings/AvaMarkSetting.h"

void FAvaMarkSettingCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	TSharedPtr<IPropertyHandle> LabelProperty = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaMarkSetting, Label));
	if (!ensure(LabelProperty.IsValid()))
	{
		return;
	}

	InHeaderRow
		.WholeRowContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.MaxWidth(125.f)
			[
				LabelProperty->CreatePropertyValueWidget(/*bDisplayDefaultPropertyButtons*/false)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				InPropertyHandle->CreateDefaultPropertyButtonWidgets()
			]
		];
}

void FAvaMarkSettingCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	TSharedPtr<IPropertyHandle> FrameNumberProperty = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaMarkSetting, FrameNumber));
	if (ensure(FrameNumberProperty.IsValid()))
	{
		InChildBuilder.AddProperty(FrameNumberProperty.ToSharedRef());
	}

	CustomizeMark(InPropertyHandle, InChildBuilder);
}

void FAvaMarkSettingCustomization::CustomizeMark(const TSharedRef<IPropertyHandle>& InPropertyHandle, IDetailChildrenBuilder& InChildBuilder)
{
	TSharedPtr<IPropertyHandle> MarkProperty = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaMarkSetting, Mark));
	if (!ensure(MarkProperty.IsValid()))
	{
		return;
	}

	// Hide Label
	if (TSharedPtr<IPropertyHandle> LabelProperty = MarkProperty->GetChildHandle(FAvaMark::GetLabelPropertyName()))
	{
		LabelProperty->MarkHiddenByCustomization();
	}

	uint32 NumChildren = 0;
	MarkProperty->GetNumChildren(NumChildren);

	for (uint32 Index = 0; Index < NumChildren; ++Index)
	{
		TSharedPtr<IPropertyHandle> ChildProperty = MarkProperty->GetChildHandle(Index);

		// Add property only if it's not hidden by customization (i.e. do not add label property as it was already hidden)
		if (ChildProperty.IsValid() && !ChildProperty->IsCustomized())
		{
			InChildBuilder.AddProperty(ChildProperty.ToSharedRef());
		}
	}
}
