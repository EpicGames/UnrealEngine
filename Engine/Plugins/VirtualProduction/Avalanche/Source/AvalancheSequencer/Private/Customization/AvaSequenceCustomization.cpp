// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceCustomization.h"
#include "AvaSequence.h"
#include "AvaSequencerUtils.h"
#include "AvaTypeSharedPointer.h"
#include "Commands/AvaSequencerCommands.h"
#include "DragDropOps/AvaOutlinerItemDragDropOp.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "ISequencer.h"
#include "Item/AvaOutlinerActor.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "ScopedTransaction.h"
#include "SequencerCommands.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AvaSequenceCustomization"

FAvaSequenceCustomization::FAvaSequenceCustomization()
{
	CreateChildrenCustomizations();
}

void FAvaSequenceCustomization::RegisterSequencerCustomization(FSequencerCustomizationBuilder& InBuilder)
{
	SequencerWeak = InBuilder.GetSequencer().AsShared();
	SequenceWeak  = Cast<UAvaSequence>(&InBuilder.GetFocusedSequence());

	// Customizations are added into array and iterated in-order.
	// Prioritize the children customizations first
	for (const TUniquePtr<ISequencerCustomization>& Customization : ChildrenCustomizations)
	{
		if (Customization.IsValid())
		{
			Customization->RegisterSequencerCustomization(InBuilder);	
		}
	}

	FSequencerCustomizationInfo CustomizationInfo;
	CustomizationInfo.OnReceivedDragOver.BindRaw(this, &FAvaSequenceCustomization::OnSequencerReceiveDragOver);
	CustomizationInfo.OnReceivedDrop.BindRaw(this, &FAvaSequenceCustomization::OnSequencerReceiveDrop);

	// Toolbar Extension
	{
		TSharedRef<FExtender> ToolbarExtender = MakeShared<FExtender>();

		ToolbarExtender->AddToolBarExtension("CurveEditor", EExtensionHook::After, nullptr
			, FToolBarExtensionDelegate::CreateRaw(this, &FAvaSequenceCustomization::ExtendSequencerToolbar));

		CustomizationInfo.ToolbarExtender = ToolbarExtender;
	}

	InBuilder.AddCustomization(CustomizationInfo);
}

void FAvaSequenceCustomization::UnregisterSequencerCustomization()
{
	for (const TUniquePtr<ISequencerCustomization>& Customization : ChildrenCustomizations)
	{
		if (Customization.IsValid())
		{
			Customization->UnregisterSequencerCustomization();	
		}
	}
}

void FAvaSequenceCustomization::CreateChildrenCustomizations()
{
	if (!ensureAlways(FAvaSequencerUtils::IsSequencerModuleLoaded()))
	{
		return;
	}

	ISequencerModule& SequencerModule = FAvaSequencerUtils::GetSequencerModule();

	TSharedPtr<FSequencerCustomizationManager> CustomizationManager = SequencerModule.GetSequencerCustomizationManager();
	if (!CustomizationManager.IsValid())
	{
		return;
	}

	// Add Level Sequence Customizations
	CustomizationManager->GetSequencerCustomizations(GetMutableDefault<ULevelSequence>(), ChildrenCustomizations);
}

void FAvaSequenceCustomization::ExtendSequencerToolbar(FToolBarBuilder& InToolbarBuilder)
{
	const FName SequencerToolbarStyleName = "SequencerToolbar";

	InToolbarBuilder.BeginStyleOverride(SequencerToolbarStyleName);

	InToolbarBuilder.AddComboButton(FUIAction()
		, FOnGetContent::CreateRaw(this, &FAvaSequenceCustomization::MakePlaybackMenu)
		, LOCTEXT("AvaSequenceOptionsLabel", "Motion Design Sequence Options")
		, LOCTEXT("AvaSequenceOptionsToolTip", "Motion Design Sequence Options")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Settings"));

	InToolbarBuilder.EndStyleOverride();
}

TSharedRef<SWidget> FAvaSequenceCustomization::MakePlaybackMenu() const
{
	TSharedPtr<ISequencer> Sequencer = SequencerWeak.Pin();
	if (!Sequencer.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return SNullWidget::NullWidget;
	}

	constexpr const TCHAR* MenuName = TEXT("AvaPlaybackMenu");

	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* const ToolMenu = ToolMenus->RegisterMenu(MenuName, NAME_None, EMultiBoxType::Menu);

		const FAvaSequencerCommands& AvaSequencerCommands = FAvaSequencerCommands::Get();

		FToolMenuSection& SequencerSection = ToolMenu->FindOrAddSection(TEXT("AvaSequencerActions"), LOCTEXT("AvaSequencerActions", "Sequence"));
		SequencerSection.AddMenuEntry(AvaSequencerCommands.ApplyCurrentState);
		SequencerSection.AddMenuEntry(AvaSequencerCommands.OpenStaggerTool);

		FToolMenuSection& BindingSection = ToolMenu->FindOrAddSection(TEXT("AvaBindingActions"), LOCTEXT("AvaBindingActions", "Binding"));
		BindingSection.AddMenuEntry(AvaSequencerCommands.FixBindingPaths);
		BindingSection.AddMenuEntry(AvaSequencerCommands.FixInvalidBindings);
		BindingSection.AddMenuEntry(AvaSequencerCommands.FixBindingHierarchy);
	}

	return ToolMenus->GenerateWidget(MenuName, FToolMenuContext(Sequencer->GetCommandBindings()));
}

bool FAvaSequenceCustomization::OnSequencerReceiveDragOver(const FGeometry& InGeometry
	, const FDragDropEvent& InDragDropEvent
	, FReply& OutReply) const
{
	if (InDragDropEvent.GetOperationAs<FAvaOutlinerItemDragDropOp>())
	{
		OutReply = FReply::Handled();
		return true;
	}
	
	OutReply = FReply::Unhandled();
	return false;
}

bool FAvaSequenceCustomization::OnSequencerReceiveDrop(const FGeometry& InGeometry
	, const FDragDropEvent& InDragDropEvent
	, FReply& OutReply) const
{
	OutReply = FReply::Unhandled();

	TSharedPtr<ISequencer> Sequencer = SequencerWeak.Pin();
	if (!Sequencer.IsValid() || !SequenceWeak.IsValid())
	{
		return false;
	}

	if (TSharedPtr<FAvaOutlinerItemDragDropOp> ItemDragDropOp = InDragDropEvent.GetOperationAs<FAvaOutlinerItemDragDropOp>())
	{
		// Worst case: All dragged items are Actors
		TArray<TWeakObjectPtr<AActor>> ActorsToAdd;
		ActorsToAdd.Reserve(ItemDragDropOp->GetItems().Num());

		for (const FAvaOutlinerItemPtr& Item : ItemDragDropOp->GetItems())
		{
			if (TSharedPtr<FAvaOutlinerActor> ActorItem = UE::AvaCore::CastSharedPtr<FAvaOutlinerActor>(Item))
			{
				if (AActor* const Actor = ActorItem->GetActor())
				{
					ActorsToAdd.Add(Actor);
				}
			}
		}

		constexpr bool bSelectActors = false;
		TArray<FGuid> PossessableGuids = Sequencer->AddActors(ActorsToAdd, bSelectActors);

		if (!PossessableGuids.IsEmpty())
		{
			OutReply = FReply::Handled();
		}

		return true;
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
