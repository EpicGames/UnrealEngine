// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "SequencerCustomizationManager.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/WeakObjectPtr.h"

class FDragDropEvent;
class FReply;
class FToolBarBuilder;
class IAvaSequencer;
class ISequencer;
class SWidget;
class UAvaSequence;
struct FGeometry;

class FAvaSequenceCustomization : public ISequencerCustomization
{
public:
	FAvaSequenceCustomization();

protected:
	//~ Begin ISequencerCustomization
	virtual void RegisterSequencerCustomization(FSequencerCustomizationBuilder& InBuilder) override;
	virtual void UnregisterSequencerCustomization() override;
	//~ End ISequencerCustomization

private:
	void CreateChildrenCustomizations();

	void ExtendSequencerToolbar(FToolBarBuilder& InToolbarBuilder);

	TSharedRef<SWidget> MakePlaybackMenu() const;

	bool OnSequencerReceiveDragOver(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent, FReply& OutReply) const;

	bool OnSequencerReceiveDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent, FReply& OutReply) const;

	/** Additional Sequencer Customizations that this class applies */
	TArray<TUniquePtr<ISequencerCustomization>> ChildrenCustomizations;

	TWeakPtr<ISequencer> SequencerWeak;

	TWeakObjectPtr<UAvaSequence> SequenceWeak;
};
