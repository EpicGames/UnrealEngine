// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceNameCustomization.h"
#include "DetailWidgetRow.h"
#include "Widgets/SAvaSequencePicker.h"

void FAvaSequenceNameCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	InHeaderRow
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.FillWidth(1.f)
			[
				SNew(SAvaSequencePicker, InPropertyHandle)
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				InPropertyHandle->CreateDefaultPropertyButtonWidgets()
			]
		];
}
