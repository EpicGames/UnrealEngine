// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceTimeCustomization.h"
#include "AvaSequenceShared.h"
#include "DetailWidgetRow.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

void FAvaSequenceTimeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	auto CreatePropertySlotWidget =
		[InPropertyHandle](FName InPropertyName)->TSharedRef<SWidget>
		{
			TSharedPtr<IPropertyHandle> ChildPropertyHandle = InPropertyHandle->GetChildHandle(InPropertyName);

			TSharedRef<SWidget> PropertyValueWidget = ChildPropertyHandle->CreatePropertyValueWidget();

			PropertyValueWidget->SetVisibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda(
				[ChildPropertyHandle]
				{
					return ChildPropertyHandle->IsEditable()
						? EVisibility::SelfHitTestInvisible
						: EVisibility::Collapsed;
				})));

			return PropertyValueWidget;
		};

	constexpr bool bDisplayDefaultPropertyButtons = false;

	TSharedPtr<IPropertyHandle> TimeTypeHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSequenceTime, TimeType));
	TSharedPtr<IPropertyHandle> HasTimeHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSequenceTime, bHasTimeConstraint));

	InHeaderRow
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					HasTimeHandle->CreatePropertyValueWidget(bDisplayDefaultPropertyButtons)
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				.FillWidth(1.f)
				[
					TimeTypeHandle->CreatePropertyValueWidget(bDisplayDefaultPropertyButtons)
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					InPropertyHandle->CreateDefaultPropertyButtonWidgets()
				]
			]
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.AutoHeight()
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					CreatePropertySlotWidget(GET_MEMBER_NAME_CHECKED(FAvaSequenceTime, Frame))
				]
				+ SOverlay::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					CreatePropertySlotWidget(GET_MEMBER_NAME_CHECKED(FAvaSequenceTime, Seconds))
				]
				+ SOverlay::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					CreatePropertySlotWidget(GET_MEMBER_NAME_CHECKED(FAvaSequenceTime, MarkLabel))
				]
			]
		];
}
