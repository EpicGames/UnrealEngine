// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaDisplayRate.h"
#include "CommonFrameRates.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "Settings/AvaSequencerDisplayRate.h"
#include "UObject/UnrealType.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaSequencerDisplayRate"

void SAvaDisplayRate::Construct(const FArguments& InArgs,  const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	PropertyHandleWeak = InPropertyHandle;
	
	ChildSlot
	.VAlign(VAlign_Fill)
	[
		SNew(SComboButton)
		.ContentPadding(FMargin(2.f, 1.0f))
		.VAlign(VAlign_Fill)
		.OnGetMenuContent(this, &SAvaDisplayRate::CreateDisplayRateOptions)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &SAvaDisplayRate::GetDisplayRateText)
		]
	];
	
	if (FAvaSequencerDisplayRate* const DisplayRate = GetDisplayRate(InPropertyHandle))
	{
		DisplayRateText = DisplayRate->FrameRate.ToPrettyText();
	}
}

TSharedRef<SWidget> SAvaDisplayRate::CreateDisplayRateOptions()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("DisplayRates", "Display Rates"));
	{
		TConstArrayView<FCommonFrameRateInfo> CommonFrameRates = FCommonFrameRates::GetAll();
		
		// Sort by Frame Rate since FCommonFrameRates::GetAll() has the NTSC Frame Rates at the End
		TArray<FCommonFrameRateInfo> FrameRates(CommonFrameRates.GetData(), CommonFrameRates.Num());
		FrameRates.Sort([](const FCommonFrameRateInfo& A, const FCommonFrameRateInfo& B)
			{
				return A.FrameRate.AsDecimal() < B.FrameRate.AsDecimal();
			});
		
		for (const FCommonFrameRateInfo& Info : FrameRates)
		{
			AddMenuEntry(MenuBuilder, Info);
		}
	}
	MenuBuilder.EndSection();
	
	return MenuBuilder.MakeWidget();
}

FText SAvaDisplayRate::GetDisplayRateText() const
{
	return DisplayRateText;
}

void SAvaDisplayRate::SetDisplayRate(FFrameRate InFrameRate)
{
	TSharedPtr<IPropertyHandle> PropertyHandle = PropertyHandleWeak.Pin();
	
	if (FAvaSequencerDisplayRate* const DisplayRate = GetDisplayRate(PropertyHandle))
	{
		// If DisplayRate returned valid pointer, PropertyHandle must've been Struct Property
		const FStructProperty* const StructProperty = CastField<const FStructProperty>(PropertyHandle->GetProperty());
		
		FAvaSequencerDisplayRate NewDisplayRate;
		NewDisplayRate.FrameRate = InFrameRate;
		
		FString TextValue;
		StructProperty->Struct->ExportText(TextValue, &NewDisplayRate, DisplayRate, nullptr, EPropertyPortFlags::PPF_None, nullptr);
		ensure(PropertyHandle->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
		
		DisplayRateText = NewDisplayRate.FrameRate.ToPrettyText();
	}
}

bool SAvaDisplayRate::IsSameDisplayRate(FFrameRate InFrameRate) const
{
	if (FAvaSequencerDisplayRate* const DisplayRate = GetDisplayRate(PropertyHandleWeak.Pin()))
	{
		return DisplayRate->FrameRate == InFrameRate;
	}
	return false;
}

FAvaSequencerDisplayRate* SAvaDisplayRate::GetDisplayRate(const TSharedPtr<IPropertyHandle>& PropertyHandle) const
{
	if (!PropertyHandle.IsValid())
	{
		return nullptr;
	}
	
	const FStructProperty* const StructProperty = CastField<const FStructProperty>(PropertyHandle->GetProperty());
	if (!StructProperty || StructProperty->Struct != FAvaSequencerDisplayRate::StaticStruct())
	{
		return nullptr;
	}
	
	void* StructData = nullptr;
	
	if (PropertyHandle->GetValueData(StructData) != FPropertyAccess::Success)
	{
		return nullptr;
	}
	
	check(StructData);
	return static_cast<FAvaSequencerDisplayRate*>(StructData);
}

void SAvaDisplayRate::AddMenuEntry(FMenuBuilder& MenuBuilder, const FCommonFrameRateInfo& Info)
{
	MenuBuilder.AddMenuEntry(Info.DisplayName, Info.Description, FSlateIcon()
		, FUIAction(FExecuteAction::CreateSP(this, &SAvaDisplayRate::SetDisplayRate, Info.FrameRate)
			, FCanExecuteAction::CreateLambda([]{ return true; })
			, FIsActionChecked::CreateSP(this, &SAvaDisplayRate::IsSameDisplayRate, Info.FrameRate))
		, NAME_None
		, EUserInterfaceActionType::RadioButton);
}

#undef LOCTEXT_NAMESPACE
