// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FMenuBuilder;
class IPropertyHandle;
struct FAvaSequencerDisplayRate;
struct FCommonFrameRateInfo;

class SAvaDisplayRate : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaDisplayRate) {}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InPropertyHandle);
	
	TSharedRef<SWidget> CreateDisplayRateOptions();

	FText GetDisplayRateText() const;
	
	void SetDisplayRate(FFrameRate InFrameRate);
	
	bool IsSameDisplayRate(FFrameRate InFrameRate) const;
	
private:
	FAvaSequencerDisplayRate* GetDisplayRate(const TSharedPtr<IPropertyHandle>& PropertyHandle) const;
	
	void AddMenuEntry(FMenuBuilder& MenuBuilder, const FCommonFrameRateInfo& Info);
	
	TWeakPtr<IPropertyHandle> PropertyHandleWeak;

	FText DisplayRateText;
};
