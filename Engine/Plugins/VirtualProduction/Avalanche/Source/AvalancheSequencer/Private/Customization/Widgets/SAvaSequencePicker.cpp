// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaSequencePicker.h"
#include "AvaSequenceName.h"
#include "AvaSequenceSubsystem.h"
#include "DetailLayoutBuilder.h"
#include "Engine/Level.h"
#include "IAvaSequenceProvider.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "SAvaSequencePicker"

void SAvaSequencePicker::Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InSequenceNameHandle)
{
	NameHandle = InSequenceNameHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSequenceName, Name));

	// Figure out the Level to use
	{
		TArray<UObject*> OuterObjects;
		InSequenceNameHandle->GetOuterObjects(OuterObjects);

		for (UObject* OuterObject : OuterObjects)
		{
			if (ULevel* Level = OuterObject->GetTypedOuter<ULevel>())
			{
				LevelWeak = Level;
				break;
			}
		}
	}

	// Editable Text box to deal with the Name Handle
	TSharedRef<SWidget> Widget = SNew(SEditableTextBox)
		.Text(this, &SAvaSequencePicker::GetSequenceName)
		.OnTextCommitted(this, &SAvaSequencePicker::OnSequenceNameCommitted)
		.Font(IDetailLayoutBuilder::GetDetailFont());

	// if the found level is valid, then use a combo box to display the options found for that level instead
	if (LevelWeak.IsValid())
	{
		RefreshOptions();

		Widget = SAssignNew(ComboBox, SComboBox<TSharedPtr<FAvaSequencePickerOption>>)
			.OptionsSource(&Options)
			.InitiallySelectedItem(SelectedOption)
			.OnGenerateWidget(this, &SAvaSequencePicker::GenerateOptionWidget)
			.OnComboBoxOpening(this, &SAvaSequencePicker::RefreshOptions)
			.OnSelectionChanged(this, &SAvaSequencePicker::OnOptionSelectionChanged)
			[
				Widget
			];
	}

	ChildSlot
	[
		Widget
	];
}

FText SAvaSequencePicker::GetSequenceName() const
{
	FName CurrentSequenceName;
	const FPropertyAccess::Result Result = NameHandle->GetValue(CurrentSequenceName);

	if (Result == FPropertyAccess::Fail)
	{
		return FText::GetEmpty();
	}

	if (Result == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	return FText::FromName(CurrentSequenceName);
}

void SAvaSequencePicker::SetSequenceName(FName InSequenceName)
{
	if (!NameHandle.IsValid())
	{
		return;
	}

	FName CurrentSequenceName;
	bool bSucceeded = NameHandle->GetValue(CurrentSequenceName) == FPropertyAccess::Success;

	// Only skip if successfully retrieved the current value and it matches the new value.
	// Else, attempt to set the new value in the handle
	if (bSucceeded && CurrentSequenceName == InSequenceName)
	{
		return;
	}

	NameHandle->SetValue(InSequenceName, EPropertyValueSetFlags::DefaultFlags);
}

void SAvaSequencePicker::OnSequenceNameCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	SetSequenceName(*InText.ToString());
}

void SAvaSequencePicker::RefreshOptions()
{
	Options.Reset();
	SelectedOption.Reset();

	if (!NameHandle.IsValid())
	{
		return;	
	}

	ULevel* Level = LevelWeak.Get();
	if (!Level)
	{
		return;
	}

	UAvaSequenceSubsystem* SequenceSubsystem = UAvaSequenceSubsystem::Get(Level);
	if (!SequenceSubsystem)
	{
		return;
	}

	IAvaSequenceProvider* SequenceProvider = SequenceSubsystem->FindSequenceProvider(Level);
	if (!SequenceProvider)
	{
		return;
	}

	FName CurrentSequenceName;
	NameHandle->GetValue(CurrentSequenceName);

	TConstArrayView<UAvaSequence*> Sequences = SequenceProvider->GetSequences();
	Options.Reserve(Sequences.Num());

	TSet<FName> ProcessedNames;
	ProcessedNames.Reserve(Sequences.Num());

	for (UAvaSequence* Sequence : Sequences)
	{
		if (Sequence)
		{
			const FName SequenceName = Sequence->GetLabel();

			// Since FAvaSequencePickerOption only handles names directly, avoid adding the same name entry twice
			bool bAlreadyInSet;
			ProcessedNames.Add(SequenceName, &bAlreadyInSet);
			if (bAlreadyInSet)
			{
				continue;
			}

			TSharedRef<FAvaSequencePickerOption> Item = MakeShared<FAvaSequencePickerOption>();
			Item->SequenceName = Sequence->GetLabel();

			// Set the first Item that matches the Sequence Name as the selected item
			if (!SelectedOption.IsValid() && Item->SequenceName == CurrentSequenceName)
			{
				SelectedOption = Item;
			}

			Options.Emplace(MoveTemp(Item));
		}
	}

	// Update the Selected Item to the selected option (can be null if there's no sequence found matching the current name stored)
	if (ComboBox.IsValid())
	{
		ComboBox->SetSelectedItem(SelectedOption);
	}
}

void SAvaSequencePicker::OnOptionSelectionChanged(TSharedPtr<FAvaSequencePickerOption> InSelectedOption, ESelectInfo::Type InSelectInfo)
{
	if (InSelectedOption.IsValid())
	{
		SelectedOption = InSelectedOption;
		SetSequenceName(SelectedOption->SequenceName);
	}
}

TSharedRef<SWidget> SAvaSequencePicker::GenerateOptionWidget(TSharedPtr<FAvaSequencePickerOption> InOption)
{
	return SNew(STextBlock)
		.Text(FText::FromName(InOption->SequenceName))
		.Font(IDetailLayoutBuilder::GetDetailFont());
}

#undef LOCTEXT_NAMESPACE
