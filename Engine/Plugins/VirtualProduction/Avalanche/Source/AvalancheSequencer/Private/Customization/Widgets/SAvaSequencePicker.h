// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Widgets/SCompoundWidget.h"

class IPropertyHandle;
class UAvaSequence;
class ULevel;
template<typename OptionType> class SComboBox;

struct FAvaSequencePickerOption
{
	/** Sequence name converted to Text */
	FName SequenceName;
};

class SAvaSequencePicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaSequencePicker) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InSequenceNameHandle);

private:
	FText GetSequenceName() const;

	void SetSequenceName(FName InSequenceName);

	void OnSequenceNameCommitted(const FText& InText, ETextCommit::Type InCommitType);

	void RefreshOptions();

	void OnOptionSelectionChanged(TSharedPtr<FAvaSequencePickerOption> InSelectedOption, ESelectInfo::Type InSelectInfo);

	TSharedRef<SWidget> GenerateOptionWidget(TSharedPtr<FAvaSequencePickerOption> InOption);

	TSharedPtr<SComboBox<TSharedPtr<FAvaSequencePickerOption>>> ComboBox;

	TWeakObjectPtr<ULevel> LevelWeak;

	TSharedPtr<IPropertyHandle> NameHandle;

	TArray<TSharedPtr<FAvaSequencePickerOption>> Options;

	TSharedPtr<FAvaSequencePickerOption> SelectedOption;
};
