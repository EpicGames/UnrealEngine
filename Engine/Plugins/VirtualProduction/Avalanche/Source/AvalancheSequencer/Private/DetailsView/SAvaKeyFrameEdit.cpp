// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaKeyFrameEdit.h"
#include "AvaSequencer.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SScrollBox.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "SAvaKeyFrameEdit"

void SAvaKeyFrameEdit::Construct(const FArguments& InArgs, const TSharedRef<FAvaSequencer>& InAvaSequencer)
{
	AvaSequencerWeak = InAvaSequencer;
	KeyEditData = InArgs._KeyEditData;

	const TSharedPtr<ISequencer> Sequencer = InAvaSequencer->GetSequencerPtr();
	if (!Sequencer.IsValid())
	{
		return;
	}

	if (const TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = Sequencer->GetViewModel())
	{
		SequencerSelectionWeak = SequencerViewModel->GetSelection();
	}

	const TSharedPtr<FAvaSequencer> AvaSequencer = AvaSequencerWeak.Pin();

	ChildSlot
		[
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			.AutoSize()
			.HAlign(HAlign_Fill)
			[
				SNew(SKeyEditInterface, Sequencer.ToSharedRef())
				.EditData(KeyEditData)
			]
		];
}

#undef LOCTEXT_NAMESPACE
