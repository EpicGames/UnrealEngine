// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SKeyEditInterface.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FAvaSequencer;

namespace UE::Sequencer
{
	class FSequencerSelection;
}

class SAvaKeyFrameEdit : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaKeyFrameEdit) {}
		SLATE_ATTRIBUTE(FKeyEditData, KeyEditData)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FAvaSequencer>& InSequencer);

protected:
	TWeakPtr<FAvaSequencer> AvaSequencerWeak;
	TWeakPtr<UE::Sequencer::FSequencerSelection> SequencerSelectionWeak;

	TAttribute<FKeyEditData> KeyEditData;
};
