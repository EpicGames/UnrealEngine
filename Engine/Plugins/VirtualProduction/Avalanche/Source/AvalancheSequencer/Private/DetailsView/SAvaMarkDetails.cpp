// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaMarkDetails.h"
#include "AvaSequence.h"
#include "CustomDetailsViewArgs.h"
#include "CustomDetailsViewModule.h"
#include "ICustomDetailsView.h"
#include "Items/CustomDetailsViewItemId.h"
#include "Marks/AvaMark.h"
#include "MovieSceneMarkedFrame.h"
#include "UObject/StructOnScope.h"
#include "UObject/UnrealType.h"

void SAvaMarkDetails::Construct(const FArguments& InArgs, UAvaSequence* InSequence, const FMovieSceneMarkedFrame& InMarkedFrame)
{
	SequenceToModify = InSequence;
	check(IsValid(SequenceToModify));

	FCustomDetailsViewArgs CustomDetailsViewArgs;
	CustomDetailsViewArgs.IndentAmount = 0.f;
	CustomDetailsViewArgs.bShowCategories = true;
	CustomDetailsViewArgs.bAllowGlobalExtensions = true;
	CustomDetailsViewArgs.CategoryAllowList.Allow(TEXT("Marks"));
	CustomDetailsViewArgs.ItemAllowList.Disallow(FCustomDetailsViewItemId::MakeCustomId(TEXT("Label")));
	CustomDetailsViewArgs.ExpansionState.Add(FCustomDetailsViewItemId::MakePropertyId<UAvaSequence>(TEXT("MotionDesign")), ECustomDetailsViewExpansion::SelfExpanded);
	CustomDetailsViewArgs.ExpansionState.Add(FCustomDetailsViewItemId::MakePropertyId<UAvaSequence>(TEXT("Marks")), ECustomDetailsViewExpansion::SelfExpanded);

	TSharedRef<ICustomDetailsView> SettingsDetailsView = ICustomDetailsViewModule::Get().CreateCustomDetailsView(CustomDetailsViewArgs);

	FAvaMark& AvaMark = SequenceToModify->FindOrAddMark(InMarkedFrame.Label);
	AvaMarkStruct = MakeShared<FStructOnScope>(FAvaMark::StaticStruct(), (uint8*)&AvaMark);
	SettingsDetailsView->SetStruct(AvaMarkStruct);

	ChildSlot
		[
			SettingsDetailsView
		];
}

void SAvaMarkDetails::NotifyPreChange(FProperty* InPropertyAboutToChange)
{
	SequenceToModify->Modify();
}

void SAvaMarkDetails::NotifyPreChange(FEditPropertyChain* InPropertyAboutToChange)
{
	SequenceToModify->Modify();
}
