// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FEditPropertyChain;
class FProperty;
class FStructOnScope;
class UAvaSequence;
struct FMovieSceneMarkedFrame;

class SAvaMarkDetails : public SCompoundWidget, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(SAvaMarkDetails) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UAvaSequence* InSequence, const FMovieSceneMarkedFrame& InMarkedFrame);

protected:
	//~ Begin FNotifyHook
	virtual void NotifyPreChange(FProperty* InPropertyAboutToChange) override;
	virtual void NotifyPreChange(FEditPropertyChain* InPropertyAboutToChange) override;
	//~ End FNotifyHook

	TObjectPtr<UAvaSequence> SequenceToModify;
	TSharedPtr<FStructOnScope> AvaMarkStruct;
};
