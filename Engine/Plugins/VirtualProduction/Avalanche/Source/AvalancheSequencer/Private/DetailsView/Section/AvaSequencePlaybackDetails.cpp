// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequencePlaybackDetails.h"
#include "AvaSequencePlaybackObject.h"
#include "AvaSequencer.h"
#include "CustomDetailsViewArgs.h"
#include "CustomDetailsViewModule.h"

#define LOCTEXT_NAMESPACE "AvaSequencePlaybackDetails"

const FName FAvaSequencePlaybackDetails::UniqueId = TEXT("AvaSequencePlaybackDetails");

FAvaSequencePlaybackDetails::FAvaSequencePlaybackDetails(const TSharedRef<FAvaSequencer>& InAvaSequencer)
	: AvaSequencerWeak(InAvaSequencer)
{
}

FName FAvaSequencePlaybackDetails::GetUniqueId() const
{
	return UniqueId;
}

FName FAvaSequencePlaybackDetails::GetSectionId() const
{
	return TEXT("Playback");
}

FText FAvaSequencePlaybackDetails::GetSectionDisplayText() const
{
	return LOCTEXT("PlaybackLabel", "Playback");
}

bool FAvaSequencePlaybackDetails::ShouldShowSection() const
{
	return AvaSequencerWeak.IsValid();
}

int32 FAvaSequencePlaybackDetails::GetSortOrder() const
{
	return 2;
}

TSharedRef<SWidget> FAvaSequencePlaybackDetails::CreateContentWidget()
{
	FCustomDetailsViewArgs CustomDetailsViewArgs;
	CustomDetailsViewArgs.IndentAmount = 0.f;
	CustomDetailsViewArgs.ValueColumnWidth  = 0.5f;
	CustomDetailsViewArgs.bShowCategories = true;
	CustomDetailsViewArgs.bAllowGlobalExtensions = true;
	CustomDetailsViewArgs.CategoryAllowList.Allow(TEXT("Scheduled Playback"));
	CustomDetailsViewArgs.ExpansionState.Add(FCustomDetailsViewItemId::MakeCategoryId("Scheduled Playback"), ECustomDetailsViewExpansion::SelfExpanded);

	const TSharedRef<ICustomDetailsView> PlaybackDetailsView = ICustomDetailsViewModule::Get().CreateCustomDetailsView(CustomDetailsViewArgs);

	if (const TSharedPtr<FAvaSequencer> AvaSequencer = AvaSequencerWeak.Pin())
	{
		IAvaSequencePlaybackObject* const PlaybackObject = AvaSequencer->GetProvider().GetPlaybackObject();
		if (ensureAlways(PlaybackObject))
		{
			PlaybackDetailsView->SetObject(PlaybackObject->ToUObject());
		}
	}

	return PlaybackDetailsView;
}

#undef LOCTEXT_NAMESPACE
