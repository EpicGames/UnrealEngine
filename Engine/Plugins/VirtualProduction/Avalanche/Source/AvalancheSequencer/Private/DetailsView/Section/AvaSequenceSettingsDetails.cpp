// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceSettingsDetails.h"
#include "AvaSequence.h"
#include "AvaSequencer.h"
#include "CustomDetailsViewArgs.h"
#include "CustomDetailsViewModule.h"

#define LOCTEXT_NAMESPACE "AvaSequenceSettingsDetails"

const FName FAvaSequenceSettingsDetails::UniqueId = TEXT("AvaSequenceSettingsDetails");

FAvaSequenceSettingsDetails::FAvaSequenceSettingsDetails(const TSharedRef<FAvaSequencer>& InAvaSequencer)
	: AvaSequencerWeak(InAvaSequencer)
{
}

FName FAvaSequenceSettingsDetails::GetUniqueId() const
{
	return UniqueId;
}

FName FAvaSequenceSettingsDetails::GetSectionId() const
{
	return TEXT("Settings");
}

FText FAvaSequenceSettingsDetails::GetSectionDisplayText() const
{
	return LOCTEXT("SettingsLabel", "Settings");
}

bool FAvaSequenceSettingsDetails::ShouldShowSection() const
{
	return AvaSequencerWeak.IsValid();
}

int32 FAvaSequenceSettingsDetails::GetSortOrder() const
{
	return 3;
}

TSharedRef<SWidget> FAvaSequenceSettingsDetails::CreateContentWidget()
{
	FCustomDetailsViewArgs CustomDetailsViewArgs;
	CustomDetailsViewArgs.IndentAmount = 0.f;
	CustomDetailsViewArgs.bShowCategories = true;
	CustomDetailsViewArgs.bAllowGlobalExtensions = true;
	CustomDetailsViewArgs.CategoryAllowList.Allow(TEXT("Sequence Settings"));
	CustomDetailsViewArgs.ExpansionState.Add(FCustomDetailsViewItemId::MakeCategoryId("Sequence Settings"), ECustomDetailsViewExpansion::SelfExpanded);
	CustomDetailsViewArgs.ExpansionState.Add(FCustomDetailsViewItemId::MakePropertyId<UAvaSequence>(TEXT("Marks")), ECustomDetailsViewExpansion::SelfExpanded);

	SettingsDetailsView = ICustomDetailsViewModule::Get().CreateCustomDetailsView(CustomDetailsViewArgs);

	if (const TSharedPtr<FAvaSequencer> AvaSequencer = AvaSequencerWeak.Pin())
	{
		AvaSequencer->GetOnViewedSequenceChanged().AddSP(this, &FAvaSequenceSettingsDetails::OnViewedSequenceChanged);

		UAvaSequence* const ViewedSequence = AvaSequencer->GetViewedSequence();
		if (IsValid(ViewedSequence))
		{
			OnViewedSequenceChanged(ViewedSequence);
		}
	}

	return SettingsDetailsView.ToSharedRef();
}

void FAvaSequenceSettingsDetails::OnViewedSequenceChanged(UAvaSequence* const InSequence)
{
	SettingsDetailsView->SetObject(InSequence);
}

#undef LOCTEXT_NAMESPACE
