// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sidebar/ISidebarDrawerContent.h"
#include "Templates/SharedPointer.h"

class FAvaSequencer;
class ICustomDetailsView;
class UAvaSequence;

class FAvaSequenceSettingsDetails : public ISidebarDrawerContent
{
public:
	static const FName UniqueId;

	FAvaSequenceSettingsDetails(const TSharedRef<FAvaSequencer>& InAvaSequencer);

	//~ Begin ISidebarDrawerContent
	virtual FName GetUniqueId() const override;
	virtual FName GetSectionId() const override;
	virtual FText GetSectionDisplayText() const override;
	virtual bool ShouldShowSection() const override;
	virtual TOptional<float> GetSectionFillHeight() const override { return 1.f; }
	virtual int32 GetSortOrder() const override;
	virtual TSharedRef<SWidget> CreateContentWidget() override;
	//~ End ISidebarDrawerContent

protected:
	void OnViewedSequenceChanged(UAvaSequence* const InSequence);

	TWeakPtr<FAvaSequencer> AvaSequencerWeak;

	TSharedPtr<ICustomDetailsView> SettingsDetailsView;
};
