// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceTreeDetails.h"
#include "AvaSequencer.h"

#define LOCTEXT_NAMESPACE "AvaSequenceTreeDetails"

const FName FAvaSequenceTreeDetails::UniqueId = TEXT("AvaSequenceTreeDetails");

FAvaSequenceTreeDetails::FAvaSequenceTreeDetails(const TSharedRef<FAvaSequencer>& InAvaSequencer)
	: AvaSequencerWeak(InAvaSequencer)
{
}

FName FAvaSequenceTreeDetails::GetUniqueId() const
{
	return UniqueId;
}

FName FAvaSequenceTreeDetails::GetSectionId() const
{
	return TEXT("SequenceTree");
}

FText FAvaSequenceTreeDetails::GetSectionDisplayText() const
{
	return LOCTEXT("SequenceTreeLabel", "Tree");
}

bool FAvaSequenceTreeDetails::ShouldShowSection() const
{
	return AvaSequencerWeak.IsValid();
}

int32 FAvaSequenceTreeDetails::GetSortOrder() const
{
	return 1;
}

TSharedRef<SWidget> FAvaSequenceTreeDetails::CreateContentWidget()
{
	if (const TSharedPtr<FAvaSequencer> AvaSequencer = AvaSequencerWeak.Pin())
	{
		return AvaSequencer->GetSequenceTreeWidget();
	}
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE
