// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceDirectorCompiler.h"
#include "AvaSequence.h"
#include "Director/AvaSequenceDirector.h"
#include "Director/AvaSequenceDirectorBlueprint.h"
#include "Director/AvaSequenceDirectorGeneratedClass.h"
#include "IAvaSequenceProvider.h"
#include "Kismet2/KismetReinstanceUtilities.h"

#define LOCTEXT_NAMESPACE "AvaSequenceDirectorCompiler"

FAvaSequenceDirectorCompilerContext::FAvaSequenceDirectorCompilerContext(UBlueprint* InBlueprint, FCompilerResultsLog& InResultsLog, const FKismetCompilerOptions& InCompilerOptions)
	: FKismetCompilerContext(InBlueprint, InResultsLog, InCompilerOptions)
{
}

UAvaSequenceDirectorBlueprint* FAvaSequenceDirectorCompilerContext::GetDirectorBlueprint() const
{
	return Cast<UAvaSequenceDirectorBlueprint>(Blueprint);
}

void FAvaSequenceDirectorCompilerContext::SpawnNewClass(const FString& InNewClassName)
{
	NewBlueprintClass = FindObject<UAvaSequenceDirectorGeneratedClass>(Blueprint->GetOutermost(), *InNewClassName);

	if (!NewBlueprintClass)
	{
		NewBlueprintClass = NewObject<UAvaSequenceDirectorGeneratedClass>(Blueprint->GetOutermost()
			, *InNewClassName
			, RF_Public | RF_Transactional);
	}
	else
	{
		// Already existed, but wasn't linked in the Blueprint yet due to load ordering issues
		FBlueprintCompileReinstancer::Create(NewBlueprintClass);
	}

	NewClass = NewBlueprintClass;
}

void FAvaSequenceDirectorCompilerContext::OnNewClassSet(UBlueprintGeneratedClass* InClassToUse)
{
	NewBlueprintClass = CastChecked<UAvaSequenceDirectorGeneratedClass>(InClassToUse);
}

void FAvaSequenceDirectorCompilerContext::CleanAndSanitizeClass(UBlueprintGeneratedClass* InClassToClean, UObject*& InOutOldCDO)
{
	FKismetCompilerContext::CleanAndSanitizeClass(InClassToClean, InOutOldCDO);

	check(NewBlueprintClass);
	NewBlueprintClass->SequenceInfos.Reset();
}

void FAvaSequenceDirectorCompilerContext::SaveSubObjectsFromCleanAndSanitizeClass(FSubobjectCollection& OutSubObjectsToSave, UBlueprintGeneratedClass* InClassToClean)
{
	FKismetCompilerContext::SaveSubObjectsFromCleanAndSanitizeClass(OutSubObjectsToSave, InClassToClean);

	// Make sure our typed pointer is set
	check(InClassToClean == NewClass);

	NewBlueprintClass = CastChecked<UAvaSequenceDirectorGeneratedClass>(NewClass);
}

void FAvaSequenceDirectorCompilerContext::EnsureProperGeneratedClass(UClass*& InOutTargetClass)
{
	if (InOutTargetClass && !InOutTargetClass->UObjectBaseUtility::IsA<UAvaSequenceDirectorGeneratedClass>())
	{
		check(Blueprint);
		FKismetCompilerUtilities::ConsignToOblivion(InOutTargetClass, Blueprint->bIsRegeneratingOnLoad);
		InOutTargetClass = nullptr;
	}
}

void FAvaSequenceDirectorCompilerContext::CreateClassVariablesFromBlueprint()
{
	FKismetCompilerContext::CreateClassVariablesFromBlueprint();

	UAvaSequenceDirectorBlueprint* const DirectorBlueprint = GetDirectorBlueprint();
	if (!DirectorBlueprint || !DirectorBlueprint->ParentClass)
	{
		return;
	}

	UClass* const ParentClass = DirectorBlueprint->ParentClass;
	check(ParentClass);

	const FEdGraphPinType SequencePinType(UEdGraphSchema_K2::PC_SoftObject
		, NAME_None
		, UAvaSequence::StaticClass()
		, EPinContainerType::None
		, false
		, FEdGraphTerminalType());

	// Add Sequence Variables
	for (const FAvaSequenceInfo& Info : DirectorBlueprint->GetSequenceInfos())
	{
		if (Info.Sequence.IsNull())
		{
			continue;
		}

		FSoftObjectProperty* const ExistingProperty = CastField<FSoftObjectProperty>(ParentClass->FindPropertyByName(Info.SequenceName));
		if (ExistingProperty && ExistingProperty->PropertyClass->IsChildOf<UAvaSequence>())
		{
			SequencePropertyMap.Add(Info.Sequence, ExistingProperty);
			continue;
		}

		if (FProperty* const SequenceProperty = CreateVariable(Info.SequenceName, SequencePinType))
		{
			SequenceProperty->SetMetaData(TEXT("Category"), TEXT("Sequences"));
			SequenceProperty->SetPropertyFlags(CPF_Transient);
			SequenceProperty->SetPropertyFlags(CPF_BlueprintVisible);
			SequenceProperty->SetPropertyFlags(CPF_BlueprintReadOnly);
			SequenceProperty->SetPropertyFlags(CPF_RepSkip);

			SequencePropertyMap.Add(Info.Sequence, SequenceProperty);
		}
	}
}

void FAvaSequenceDirectorCompilerContext::FinishCompilingClass(UClass* InClass)
{
	UAvaSequenceDirectorBlueprint* const DirectorBlueprint = GetDirectorBlueprint();
	if (!InClass || !DirectorBlueprint)
	{
		return;
	}

	UClass* const ParentClass = DirectorBlueprint->ParentClass;
	if (!ParentClass)
	{
		return;
	}

	UAvaSequenceDirectorGeneratedClass* const BlueprintGeneratedClass = CastChecked<UAvaSequenceDirectorGeneratedClass>(InClass);
	if (!BlueprintGeneratedClass)
	{
		return;
	}

	const bool bIsSkeletonOnly = CompileOptions.CompileType == EKismetCompileType::SkeletonOnly;

	// Don't do slow work on the skeleton generated class.
	if (!bIsSkeletonOnly)
	{
		BlueprintGeneratedClass->SequenceInfos = DirectorBlueprint->GetSequenceInfos();
	}

	if (bIsSkeletonOnly || DirectorBlueprint->SkeletonGeneratedClass != InClass)
	{
		for (FSoftObjectProperty* const SequenceProperty : TFieldRange<FSoftObjectProperty>(ParentClass))
		{
			if (!SequenceProperty || !SequenceProperty->PropertyClass->IsChildOf<UAvaSequence>())
			{
				continue;
			}

			const TSoftObjectPtr<UAvaSequence>* FoundSequence = SequencePropertyMap.FindKey(SequenceProperty);
			if (!FoundSequence)
			{
				const FText RequiredSequenceNotBound = LOCTEXT("RequiredSequenceNotBound"
					, "A required Motion Design Sequence binding @@ was not found.");

				if (Blueprint->bIsNewlyCreated)
				{
					MessageLog.Warning(*RequiredSequenceNotBound.ToString(), SequenceProperty);
				}
				else
				{
					MessageLog.Error(*RequiredSequenceNotBound.ToString(), SequenceProperty);
				}
			}
		}
	}

	FKismetCompilerContext::FinishCompilingClass(InClass);
}

void FAvaSequenceDirectorCompilerContext::OnPostCDOCompiled(const UObject::FPostCDOCompiledContext& InContext)
{
	FKismetCompilerContext::OnPostCDOCompiled(InContext);

	if (InContext.bIsSkeletonOnly)
	{
		return;
	}

	SequencePropertyMap.Empty();
}

void FAvaSequenceDirectorCompilerContext::PostCompile()
{
	if (Blueprint && Blueprint->GeneratedClass)
	{
		if (UAvaSequenceDirector* Director = Cast<UAvaSequenceDirector>(Blueprint->GeneratedClass->GetDefaultObject()))
		{
			Director->UpdateProperties();
		}
	}

	FKismetCompilerContext::PostCompile();
}

#undef LOCTEXT_NAMESPACE
