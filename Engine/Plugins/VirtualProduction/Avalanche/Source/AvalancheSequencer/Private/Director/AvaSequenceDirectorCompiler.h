// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "KismetCompiler.h"
#include "KismetCompilerModule.h"

class FProperty;
class UAvaSequence;
class UAvaSequenceDirectorBlueprint;
class UAvaSequenceDirectorGeneratedClass;

class FAvaSequenceDirectorCompilerContext : public FKismetCompilerContext
{
public:
	FAvaSequenceDirectorCompilerContext(UBlueprint* InBlueprint
		, FCompilerResultsLog& InResultsLog
		, const FKismetCompilerOptions& InCompilerOptions);

private:
	UAvaSequenceDirectorBlueprint* GetDirectorBlueprint() const;

	//~ Begin FKismetCompilerContext
	virtual void SpawnNewClass(const FString& InNewClassName) override;
	virtual void OnNewClassSet(UBlueprintGeneratedClass* InClassToUse) override;
	virtual void CleanAndSanitizeClass(UBlueprintGeneratedClass* InClassToClean, UObject*& InOutOldCDO) override;
	virtual void SaveSubObjectsFromCleanAndSanitizeClass(FSubobjectCollection& OutSubObjectsToSave, UBlueprintGeneratedClass* InClassToClean) override;
	virtual void EnsureProperGeneratedClass(UClass*& InOutTargetClass) override;
	virtual void CreateClassVariablesFromBlueprint() override;
	virtual void FinishCompilingClass(UClass* InClass) override;
	virtual void OnPostCDOCompiled(const UObject::FPostCDOCompiledContext& InContext) override;
	virtual void PostCompile() override;
	//~ End FKismetCompilerContext

	UAvaSequenceDirectorGeneratedClass* NewBlueprintClass = nullptr;

	TMap<TSoftObjectPtr<UAvaSequence>, FProperty*> SequencePropertyMap;
};
