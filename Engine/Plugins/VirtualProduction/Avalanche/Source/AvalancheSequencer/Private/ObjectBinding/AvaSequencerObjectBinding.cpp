// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequencerObjectBinding.h"
#include "ActorTreeItem.h"
#include "Editor.h"
#include "EditorModeManager.h"
#include "Engine/Selection.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISequencer.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "SceneOutlinerModule.h"
#include "SceneOutlinerPublicTypes.h"
#include "AvaSequence.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Toolkits/IToolkitHost.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "AvaSequencerObjectBinding"

FAvaSequencerObjectBinding::FAvaSequencerObjectBinding(TSharedRef<ISequencer> InSequencer)
	: SequencerWeak(InSequencer)
{
}

FText FAvaSequencerObjectBinding::GetDisplayName() const
{
	return LOCTEXT("ActorTrackEditor_DisplayName", "Motion Design Actor");
}

void FAvaSequencerObjectBinding::BuildSequencerAddMenu(FMenuBuilder& OutMenuBuilder)
{
	const TSharedPtr<ISequencer> Sequencer = SequencerWeak.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<IToolkitHost> ToolkitHost = Sequencer->GetToolkitHost();
	if (!ToolkitHost.IsValid())
	{
		return;
	}

	// Skip menu extension if the actor selection is the global actor selection, as this is already handled by FLevelSequenceEditorActorBinding
	USelection* const ActorSelection = ToolkitHost->GetEditorModeManager().GetSelectedActors();
	if (!ActorSelection || ActorSelection == GEditor->GetSelectedActors())
	{
		return;
	}

	OutMenuBuilder.AddSubMenu(
		LOCTEXT("AddActor_Label", "Add Actor Track"),
		LOCTEXT("AddActor_ToolTip", "Allow sequencer to possess an actor that already exists in the current level."),
		FNewMenuDelegate::CreateRaw(this, &FAvaSequencerObjectBinding::AddPossessActorMenuExtensions),
		false /*bInOpenSubMenuOnClick*/,
		FSlateIcon("LevelSequenceEditorStyle", "LevelSequenceEditor.PossessNewActor")
	);
}

bool FAvaSequencerObjectBinding::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	return InSequence->GetClass() == UAvaSequence::StaticClass();
}

void FAvaSequencerObjectBinding::AddPossessActorMenuExtensions(FMenuBuilder& OutMenuBuilder)
{
	// This is called for every actor in the map, and asking the sequencer for a handle to the object to check if we have
	// already bound is an issue on maps that have tens of thousands of actors. The current sequence is will almost always
	// have actors than the map, so instead we'll cache off all of the actors already bound and check against that map locally.
	// This list is checked via an async filter, but we don't need to store them as weak pointers because we're doing a direct
	// pointer comparison and not an object comparison, and the async list shouldn't run the filter if the object is no longer valid.
	// We don't need to check against Sequencer spawnables as they're not valid for possession.
	
	TSharedPtr<ISequencer> Sequencer = SequencerWeak.Pin();
	if (!Sequencer.IsValid() || !Sequencer->GetToolkitHost().IsValid())
	{
		return;
	}

	if (!GEditor || !GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
	{
		return;
	}

	UAvaSequence* const Sequence = Cast<UAvaSequence>(Sequencer->GetFocusedMovieSceneSequence());

	if (!Sequence)
	{
		return;
	}

	TSet<UObject*> ExistingPossessedObjects;
	if (UMovieScene* const MovieScene = Sequence->GetMovieScene())
	{
		for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); Index++)
		{
			FMovieScenePossessable& Possessable = MovieScene->GetPossessable(Index);

			TArray<UObject*, TInlineAllocator<1>> BoundObjects;
			static_cast<UMovieSceneSequence*>(Sequence)->LocateBoundObjects(Possessable.GetGuid(), UE::UniversalObjectLocator::FResolveParams(Sequencer->GetPlaybackContext()), Sequencer->GetSharedPlaybackState(), BoundObjects);

			// A possession guid can apply to more than one object, so we get all bound objects for the GUID and add them to our set.
			ExistingPossessedObjects.Append(BoundObjects);
		}
	}
	
	USelection* ActorSelection = Sequencer->GetToolkitHost()->GetEditorModeManager().GetSelectedActors();
	check(ActorSelection);

	TArray<AActor*> SelectedActors;
	ActorSelection->GetSelectedObjects(SelectedActors);

	auto IsActorValidForPossession = [](const AActor* InActor, TSet<UObject*> InPossessedObjectSet)
		{
			return !InPossessedObjectSet.Contains(InActor);
		};
	
	SelectedActors.RemoveAll([&ExistingPossessedObjects, &IsActorValidForPossession](AActor* InActor)->bool
	{
		return !IsActorValidForPossession(InActor, ExistingPossessedObjects);
	});

	FText SelectedLabel;
	FSlateIcon ActorIcon = FSlateIconFinder::FindIconForClass(AActor::StaticClass());
	
	if (SelectedActors.Num() == 1)
	{
		SelectedLabel = FText::Format(LOCTEXT("AddSpecificActor", "Add '{0}'"), FText::FromString(SelectedActors[0]->GetActorLabel()));
		ActorIcon = FSlateIconFinder::FindIconForClass(SelectedActors[0]->GetClass());
	}
	else if (SelectedActors.Num() > 1)
	{
		SelectedLabel = FText::Format(LOCTEXT("AddCurrentActorSelection", "Add Current Selection ({0} actors)"), FText::AsNumber(SelectedActors.Num()));
	}

	if (!SelectedLabel.IsEmpty())
	{
		// Copy the array into the lambda - probably not that big a deal
		OutMenuBuilder.AddMenuEntry(SelectedLabel, FText(), ActorIcon, FExecuteAction::CreateLambda([this, SelectedActors]{
			FSlateApplication::Get().DismissAllMenus();
			AddActorsToSequencer(SelectedActors);
		}));
	}

	// Add an entry for an empty binding
	OutMenuBuilder.AddMenuEntry(LOCTEXT("EmptyBinding", "New Empty Binding"),
		LOCTEXT("EmptyBindingTooltip", "Add a new empty binding to Sequencer which can be connected to an object or actor afterwards in the Binding Properties"),
		FSlateIcon(), // TODO: empty icon?
		FExecuteAction::CreateLambda([this] {
			FSlateApplication::Get().DismissAllMenus();
			SequencerWeak.Pin()->AddEmptyBinding();
			}));

	OutMenuBuilder.BeginSection("ChooseActorSection", LOCTEXT("ChooseActor", "Choose Actor:"));

	// Set up a menu entry to add any arbitrary actor to the sequencer
	FSceneOutlinerInitializationOptions InitOptions;
	{
		// We hide the header row to keep the UI compact.
		InitOptions.bShowHeaderRow = false;
		InitOptions.bShowSearchBox = true;
		InitOptions.bShowCreateNewFolder = false;
		InitOptions.bFocusSearchBoxWhenOpened = true;

		// Only want the actor label column
		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label()
			, FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0));

		// Only display actors that are not possessed already
		InitOptions.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateLambda(IsActorValidForPossession
			, ExistingPossessedObjects));
	}

	// actor selector to allow the user to choose an actor
	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");

	TSharedRef<SWidget> ActorPicker = SNew(SBox)
		.MaxDesiredHeight(400.0f)
		.WidthOverride(300.0f)
		[
			SceneOutlinerModule.CreateActorPicker(InitOptions
				, FOnActorPicked::CreateLambda([this](AActor* Actor)
					{
						// Create a new binding for this actor
						if (!FSlateApplication::Get().GetModifierKeys().IsShiftDown())
						{
							FSlateApplication::Get().DismissAllMenus();
						}
						AddActorsToSequencer({ Actor });
					})
				, Sequence->GetContextWorld()
			)
		];

	OutMenuBuilder.AddWidget(ActorPicker, FText::GetEmpty(), true);
	OutMenuBuilder.EndSection();
}

void FAvaSequencerObjectBinding::AddActorsToSequencer(const TArray<AActor*>& InActors)
{
	TArray<TWeakObjectPtr<AActor>> Actors;
	Actors.Reserve(InActors.Num());
	
	for (AActor* const Actor : InActors)
	{
		Actors.Add(Actor);
	}
	SequencerWeak.Pin()->AddActors(Actors);
}

#undef LOCTEXT_NAMESPACE
