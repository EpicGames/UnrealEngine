// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "ISequencerEditorObjectBinding.h"
#include "Templates/SharedPointer.h"

class AActor;
class ISequencer;

class FAvaSequencerObjectBinding : public ISequencerEditorObjectBinding
{
public:
	FAvaSequencerObjectBinding(TSharedRef<ISequencer> InSequencer);

	//~ Begin ISequencerEditorObjectBinding
	virtual FText GetDisplayName() const override;
	virtual void BuildSequencerAddMenu(FMenuBuilder& OutMenuBuilder) override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	//~ End ISequencerEditorObjectBinding
	
private:
	/** Menu extension callback for the add menu */
	void AddPossessActorMenuExtensions(FMenuBuilder& OutMenuBuilder);
	
	/** Add the specified actors to the sequencer */
	void AddActorsToSequencer(const TArray<AActor*>& InActors);

	TWeakPtr<ISequencer> SequencerWeak;
};