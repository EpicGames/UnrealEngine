// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerSequence.h"
#include "AvaSequencer.h"
#include "AvaSequencerSubsystem.h"
#include "Engine/World.h"
#include "Input/Reply.h"
#include "AvaSequence.h"
#include "Styling/AppStyle.h"

FAvaOutlinerSequence::FAvaOutlinerSequence(IAvaOutliner& InOutliner, UAvaSequence* InSequence, const FAvaOutlinerItemPtr& InReferencingItem)
	: Super(InOutliner, InSequence, InReferencingItem, TEXT("Sequence"))
	, Sequence(InSequence)
{
	SequenceIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "UMGEditor.AnimTabIcon");
}

FText FAvaOutlinerSequence::GetDisplayName() const
{
	return Sequence.IsValid() ? Sequence->GetDisplayName() : FText::GetEmpty();
}

FSlateIcon FAvaOutlinerSequence::GetIcon() const
{
	return SequenceIcon;
}

void FAvaOutlinerSequence::Select(FAvaOutlinerScopedSelection& InSelection) const
{
	if (!Sequence.IsValid())
	{
		return;
	}

	if (UWorld* const World = Sequence->GetContextWorld())
	{
		UAvaSequencerSubsystem* const SequencerSubsystem = World->GetSubsystem<UAvaSequencerSubsystem>();
		check(SequencerSubsystem);

		if (TSharedPtr<IAvaSequencer> Sequencer = SequencerSubsystem->GetSequencer())
		{
			Sequencer->SetViewedSequence(Sequence.Get());
			if (ReferencingItemWeak.IsValid())
			{
				ReferencingItemWeak.Pin()->Select(InSelection);
			}
		}
	}
}

void FAvaOutlinerSequence::SetObject_Impl(UObject* InObject)
{
	FAvaOutlinerObjectReference::SetObject_Impl(InObject);
	Sequence = Cast<UAvaSequence>(InObject);
}
