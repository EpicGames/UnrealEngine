// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Item/AvaOutlinerObjectReference.h"
#include "Textures/SlateIcon.h"
#include "UObject/WeakObjectPtr.h"

class UAvaSequence;

/**
 * Item in Outliner representing an Sequence.
 * Inherits from FAvaOutlinerObjectReference as multiple objects can be in the same sequence
 */
class FAvaOutlinerSequence : public FAvaOutlinerObjectReference
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerSequence, FAvaOutlinerObjectReference);

	FAvaOutlinerSequence(IAvaOutliner& InOutliner, UAvaSequence* InSequence, const FAvaOutlinerItemPtr& InReferencingItem);

	UAvaSequence* GetSequence() const { return Sequence.Get(IsIgnoringPendingKill()); }
	
	//~ Begin IAvaOutlinerItem
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	virtual void Select(FAvaOutlinerScopedSelection& InSelection) const override;
	//~ End IAvaOutlinerItem
	
protected:
	//~ Begin FAvaOutlinerObjectItem
	virtual void SetObject_Impl(UObject* InObject) override;
	//~ End FAvaOutlinerObjectItem
	
	TWeakObjectPtr<UAvaSequence> Sequence;
	
	FSlateIcon SequenceIcon;
};
