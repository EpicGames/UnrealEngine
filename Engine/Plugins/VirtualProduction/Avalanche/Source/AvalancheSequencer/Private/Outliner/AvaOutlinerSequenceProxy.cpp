// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaOutlinerSequenceProxy.h"
#include "AvaSequence.h"
#include "AvaSequencer.h"
#include "AvaSequencerSubsystem.h"
#include "AvaTypeSharedPointer.h"
#include "Engine/World.h"
#include "IAvaOutliner.h"
#include "ISequencer.h"
#include "Item/AvaOutlinerActor.h"
#include "Outliner/AvaOutlinerSequence.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "AvaOutlinerSequenceProxy"

FAvaOutlinerSequenceProxy::FAvaOutlinerSequenceProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem)
	: Super(InOutliner, InParentItem)
{
	SequenceIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "UMGEditor.AnimTabIcon");

	if (UWorld* const World = Outliner.GetWorld())
	{
		UAvaSequencerSubsystem* const SequencerSubsystem = World->GetSubsystem<UAvaSequencerSubsystem>();
		check(SequencerSubsystem);

		if (TSharedPtr<IAvaSequencer> AvaSequencer = SequencerSubsystem->GetSequencer())
		{
			AvaSequencerWeak = AvaSequencer;
		}
	}
}

FAvaOutlinerSequenceProxy::~FAvaOutlinerSequenceProxy()
{
	UnbindDelegates();
}

void FAvaOutlinerSequenceProxy::OnActorAddedToSequencer(AActor* InActor, const FGuid InGuid)
{
	TSharedPtr<FAvaOutlinerActor> ParentActorItem = UE::AvaCore::CastSharedPtr<FAvaOutlinerActor>(GetParent());
	if (ParentActorItem.IsValid() && InActor == ParentActorItem->GetActor())
	{
		RefreshChildren();
	}
}

void FAvaOutlinerSequenceProxy::OnMovieSceneDataChanged(EMovieSceneDataChangeType InChangeType)
{
	static const TSet<EMovieSceneDataChangeType> ChangesRequiringRefresh
		{
			EMovieSceneDataChangeType::MovieSceneStructureItemAdded,
			EMovieSceneDataChangeType::MovieSceneStructureItemRemoved,
			EMovieSceneDataChangeType::MovieSceneStructureItemsChanged,
			EMovieSceneDataChangeType::RefreshAllImmediately,
			EMovieSceneDataChangeType::Unknown,
			EMovieSceneDataChangeType::RefreshTree
		};

	if (!ChangesRequiringRefresh.Contains(InChangeType))
	{
		return;
	}
	
	if (ParentWeak.IsValid() && ParentWeak.Pin()->IsA<FAvaOutlinerActor>())
	{
		RefreshChildren();
	}
}

void FAvaOutlinerSequenceProxy::OnItemRegistered()
{
	Super::OnItemRegistered();
	BindDelegates();
}

void FAvaOutlinerSequenceProxy::OnItemUnregistered()
{
	Super::OnItemUnregistered();
	UnbindDelegates();
}

FText FAvaOutlinerSequenceProxy::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Sequences");
}

FSlateIcon FAvaOutlinerSequenceProxy::GetIcon() const
{
	return SequenceIcon;
}

FText FAvaOutlinerSequenceProxy::GetIconTooltipText() const
{
	return LOCTEXT("Tooltip", "Shows all the Sequences an Actor is bound to");
}

void FAvaOutlinerSequenceProxy::GetProxiedItems(const TSharedRef<IAvaOutlinerItem>& InParent, TArray<FAvaOutlinerItemPtr>& OutChildren, bool bInRecursive)
{
	if (const FAvaOutlinerActor* const ActorItem = InParent->CastTo<FAvaOutlinerActor>())
	{
		TSharedPtr<IAvaSequencer> AvaSequencer = AvaSequencerWeak.Pin();

		AActor* const Actor = ActorItem->GetActor();

		if (AvaSequencer.IsValid() && Actor)
		{
			const TArray<UAvaSequence*> Sequences = AvaSequencer->GetSequencesForObject(Actor);
			
			for (UAvaSequence* const Sequence : Sequences)
			{
				const FAvaOutlinerItemPtr SequenceItem = Outliner.FindOrAdd<FAvaOutlinerSequence>(Sequence, InParent);
				SequenceItem->SetParent(SharedThis(this));

				OutChildren.Add(SequenceItem);

				if (bInRecursive)
				{
					SequenceItem->FindChildren(OutChildren, bInRecursive);
				}
			}
		}
	}
}

void FAvaOutlinerSequenceProxy::BindDelegates()
{
	UnbindDelegates();

	TSharedPtr<IAvaSequencer> AvaSequencer = AvaSequencerWeak.Pin();
	if (!AvaSequencer.IsValid())
	{
		return;
	}

	if (TSharedPtr<ISequencer> Sequencer = AvaSequencer->GetSequencerPtr())
	{
		OnMovieSceneDataChangedHandle = Sequencer->OnMovieSceneDataChanged().AddSP(this, &FAvaOutlinerSequenceProxy::OnMovieSceneDataChanged);
		OnActorAddedToSequencerHandle = Sequencer->OnActorAddedToSequencer().AddSP(this, &FAvaOutlinerSequenceProxy::OnActorAddedToSequencer);
		LastBoundSequencerWeak = Sequencer;
	}
}

void FAvaOutlinerSequenceProxy::UnbindDelegates()
{
	if (TSharedPtr<ISequencer> Sequencer = LastBoundSequencerWeak.Pin())
	{
		Sequencer->OnMovieSceneDataChanged().Remove(OnMovieSceneDataChangedHandle);
		Sequencer->OnActorAddedToSequencer().Remove(OnActorAddedToSequencerHandle);

		OnMovieSceneDataChangedHandle.Reset();
		OnActorAddedToSequencerHandle.Reset();

		LastBoundSequencerWeak.Reset();
	}
}

#undef LOCTEXT_NAMESPACE
