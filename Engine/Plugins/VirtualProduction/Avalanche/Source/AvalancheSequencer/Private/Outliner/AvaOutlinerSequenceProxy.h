// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Item/AvaOutlinerItemProxy.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"

class IAvaSequencer;
class ISequencer;
class UAvaSequencerSubsystem;
enum class EMovieSceneDataChangeType;

/** Gathers the Sequences that contain the given Item (UObject) */
class FAvaOutlinerSequenceProxy : public FAvaOutlinerItemProxy
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaOutlinerSequenceProxy, FAvaOutlinerItemProxy);

	FAvaOutlinerSequenceProxy(IAvaOutliner& InOutliner, const FAvaOutlinerItemPtr& InParentItem);

	virtual ~FAvaOutlinerSequenceProxy() override;

	void OnActorAddedToSequencer(AActor* InActor, const FGuid InGuid);

	void OnMovieSceneDataChanged(EMovieSceneDataChangeType InChangeType);

	//~ Begin IAvaOutlinerItem
	virtual void OnItemRegistered() override;
	virtual void OnItemUnregistered() override;
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	virtual FText GetIconTooltipText() const override;
	//~ End IAvaOutlinerItem

	//~ Begin FAvaOutlinerItemProxy
	virtual void GetProxiedItems(const TSharedRef<IAvaOutlinerItem>& InParent, TArray<FAvaOutlinerItemPtr>& OutChildren, bool bInRecursive) override;
	//~ End FAvaOutlinerItemProxy

private:
	void BindDelegates();

	void UnbindDelegates();

	FSlateIcon SequenceIcon;

	TWeakPtr<IAvaSequencer> AvaSequencerWeak;

	/** Used to keep track of the latest Sequencer instance that our delegates are bound to */
	TWeakPtr<ISequencer> LastBoundSequencerWeak;

	FDelegateHandle OnMovieSceneDataChangedHandle;

	FDelegateHandle OnActorAddedToSequencerHandle;
};
