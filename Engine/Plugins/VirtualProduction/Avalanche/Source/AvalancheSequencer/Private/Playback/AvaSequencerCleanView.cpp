// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequencerCleanView.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"

void FAvaSequencerCleanView::Apply(const TArray<TWeakPtr<FEditorViewportClient>>& InViewportClients)
{
	for (const TWeakPtr<FEditorViewportClient>& ViewportClientWeak : InViewportClients)
	{
		if (TSharedPtr<FEditorViewportClient> ViewportClient = ViewportClientWeak.Pin())
		{
			ViewportClientStates.FindOrAdd(ViewportClientWeak, ViewportClient->IsInGameView());
			ViewportClient->SetGameView(true);
			
			if (FEditorModeTools* const ModeTools = ViewportClient->GetModeTools())
			{
				ModeToolsStates.FindOrAdd(ModeTools, ModeTools->IsViewportUIHidden());
				ModeTools->SetHideViewportUI(true);
			}
		}
	}
}

void FAvaSequencerCleanView::Restore(bool bInDeallocateSpaceUsed)
{
	for (const TPair<TWeakPtr<FEditorViewportClient>, bool>& Pair : ViewportClientStates)
	{
		if (TSharedPtr<FEditorViewportClient> ViewportClient = Pair.Key.Pin())
		{
			ViewportClient->SetGameView(Pair.Value);
		}
	}
	
	for (const TPair<FEditorModeTools*, bool>& Pair : ModeToolsStates)
	{
		if (Pair.Key)
		{
			Pair.Key->SetHideViewportUI(Pair.Value);
		}
	}
	
	if (bInDeallocateSpaceUsed)
	{
		ViewportClientStates.Empty(0);
		ModeToolsStates.Empty(0);
	}
	else
	{
		ViewportClientStates.Reset();
		ModeToolsStates.Reset();
	}
}
