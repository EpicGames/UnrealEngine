// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Templates/SharedPointer.h"

class FEditorModeTools;
class FEditorViewportClient;

/**
 * Class that makes sure all the Entities (e.g. ViewportClient, ModeTools, etc)
 * can be Set to Clean View and Restored Back to their previous state
 */
class FAvaSequencerCleanView
{
public:
	/** Adds all the Viewport Clients and their Editor Mode Tools to this Clean View State */
	void Apply(const TArray<TWeakPtr<FEditorViewportClient>>& InViewportClients);
	
	/**
	 * Restores all the added entities back to their original state
	 * @param bInDeallocateSpaceUsed whether to remove the allocations of the maps. Defaults to false since it's expected this to be used repetitively.
	 */
	void Restore(bool bInDeallocateSpaceUsed = false);
	
private:
	/** Map of the Editor VP Client to their Initial Clean View State */
	TMap<TWeakPtr<FEditorViewportClient>, bool> ViewportClientStates;
	
	/** Map of the Editor Mode Tools to their Initial Clean View State*/
	TMap<FEditorModeTools*, bool> ModeToolsStates;
};
