// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequencerController.h"
#include "AvaSequence.h"
#include "AvaSequencePlaybackObject.h"
#include "AvaSequencePlayerVariant.h"
#include "AvaSequenceSubsystem.h"
#include "AvaSequencer.h"
#include "AvaSequencerSubsystem.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"
#include "IAvaSequenceController.h"
#include "ISequencer.h"

FAvaSequencerController::~FAvaSequencerController()
{
	UnbindDelegates();
}

UAvaSequence* FAvaSequencerController::GetCurrentSequence() const
{
	if (TSharedPtr<ISequencer> Sequencer = GetSequencer())
	{
		return Cast<UAvaSequence>(Sequencer->GetFocusedMovieSceneSequence());
	}
	return nullptr;
}

void FAvaSequencerController::OnPlay()
{
	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (Sequencer.IsValid() && SequenceController.IsValid())
	{
		// Reset State only if it is completely stopped (i.e. not in a Stop Point or Pause)
		SequenceController->SetTime(Sequencer->GetGlobalTime().Time, bSequencerStopped);
		bSequencerStopped = false;
	}
}

void FAvaSequencerController::OnStop()
{
	bSequencerStopped = true;
}

void FAvaSequencerController::OnScrub()
{
	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (Sequencer.IsValid() && SequenceController.IsValid())
	{
		SequenceController->SetTime(Sequencer->GetGlobalTime().Time, true);
	}
}

void FAvaSequencerController::SetSequencer(const TSharedPtr<ISequencer>& InSequencer)
{
	UnbindDelegates();

	SequencerWeak = InSequencer;

	if (InSequencer.IsValid())
	{
		InSequencer->OnPlayEvent().AddRaw(this, &FAvaSequencerController::OnPlay);
		InSequencer->OnStopEvent().AddRaw(this, &FAvaSequencerController::OnStop);
		InSequencer->OnEndScrubbingEvent().AddRaw(this, &FAvaSequencerController::OnScrub);
	}
}

void FAvaSequencerController::Tick(float InDeltaTime)
{
	TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	if (UAvaSequence* const Sequence = GetCurrentSequence())
	{
		InDeltaTime *= GetEffectiveTimeDilation(*Sequence);
		UpdatePlaybackObject(*Sequence);

		if (!SequenceController.IsValid() || SequenceController->GetSequence() != Sequence)
		{
			SequenceController = UAvaSequenceSubsystem::CreateSequenceController(*Sequence, PlaybackObjectWeak.Get());
		}

		const FFrameTime DeltaFrameTime = InDeltaTime
			* Sequencer->GetPlaybackSpeed()
			* Sequencer->GetFocusedTickResolution();

		SequenceController->Tick(FAvaSequencePlayerVariant(Sequencer.Get()), DeltaFrameTime, InDeltaTime);
	}
}

TStatId FAvaSequencerController::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FAvaSequencerController, STATGROUP_Tickables);
}

float FAvaSequencerController::GetEffectiveTimeDilation(UAvaSequence& InSequence) const
{
	if (UWorld* const World = InSequence.GetWorld())
	{
		AWorldSettings* const WorldSettings = World->GetWorldSettings();
		return WorldSettings ? WorldSettings->GetEffectiveTimeDilation() : 1.f;
	}
	return 1.f;
}

void FAvaSequencerController::UnbindDelegates()
{
	if (TSharedPtr<ISequencer> Sequencer = GetSequencer())
	{
		Sequencer->OnPlayEvent().RemoveAll(this);
		Sequencer->OnStopEvent().RemoveAll(this);
		Sequencer->OnEndScrubbingEvent().RemoveAll(this);
	}
}

void FAvaSequencerController::UpdatePlaybackObject(UAvaSequence& InSequence)
{
	UWorld* const World = InSequence.GetWorld();
	if (!World || PlaybackObjectWeak.IsValid())
	{
		return;
	}

	UAvaSequencerSubsystem* const SequencerSubsystem = World->GetSubsystem<UAvaSequencerSubsystem>();

	const TSharedPtr<IAvaSequencer> AvaSequencer = SequencerSubsystem
		? SequencerSubsystem->GetSequencer()
		: TSharedPtr<IAvaSequencer>();

	if (AvaSequencer.IsValid())
	{
		PlaybackObjectWeak = AvaSequencer->GetProvider().GetPlaybackObject();
	}
}
