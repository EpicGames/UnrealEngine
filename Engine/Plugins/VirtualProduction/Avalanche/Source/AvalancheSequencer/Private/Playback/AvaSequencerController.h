// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaSequencerController.h"
#include "Templates/SharedPointer.h"
#include "TickableEditorObject.h"
#include "UObject/WeakInterfacePtr.h"

class IAvaSequenceController;
class IAvaSequencePlaybackObject;
class ISequencer;
class UAvaSequence;

/*
 * Track Editor designed to follow the Sequencer under the hood
 * while not exposing its Track.
 * Helps manage things like Stop Points 
 */
class FAvaSequencerController : public IAvaSequencerController, public FTickableEditorObject
{
public:
	FAvaSequencerController() = default;

	virtual ~FAvaSequencerController() override;

	TSharedPtr<ISequencer> GetSequencer() const { return SequencerWeak.Pin(); }

	UAvaSequence* GetCurrentSequence() const;

	void OnPlay();
	void OnStop();
	void OnScrub();

	//~ Begin IAvaSequencerController
	virtual void SetSequencer(const TSharedPtr<ISequencer>& InSequencer) override;
	//~ End IAvaSequencerController

	//~ Begin FTickableEditorObject
	virtual void Tick(float InDeltaTime) override;
	virtual TStatId GetStatId() const override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	//~ End FTickableEditorObject

private:
	float GetEffectiveTimeDilation(UAvaSequence& InSequence) const;

	void UnbindDelegates();

	void UpdatePlaybackObject(UAvaSequence& InSequence);

	TWeakPtr<ISequencer> SequencerWeak;

	TSharedPtr<IAvaSequenceController> SequenceController;

	TWeakInterfacePtr<IAvaSequencePlaybackObject> PlaybackObjectWeak;

	bool bSequencerStopped = true;
};
