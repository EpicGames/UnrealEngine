// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceActorFactory.h"
#include "AvaSequence.h"
#include "AvaSequenceActor.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "AvaSequenceActorFactory"

UAvaSequenceActorFactory::UAvaSequenceActorFactory()
{
	DisplayName = LOCTEXT("DisplayName", "Motion Design Sequence");
	NewActorClass = AAvaSequenceActor::StaticClass();
}

bool UAvaSequenceActorFactory::CanCreateActorFrom(const FAssetData& InAssetData, FText& OutErrorMessage)
{
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	if (const TSharedPtr<IClassViewerFilter>& GlobalClassFilter = ClassViewerModule.GetGlobalClassViewerFilter())
	{
		TSharedRef<FClassViewerFilterFuncs> ClassFilterFuncs = ClassViewerModule.CreateFilterFuncs();
		FClassViewerInitializationOptions ClassViewerOptions = {};

		if (!GlobalClassFilter->IsClassAllowed(ClassViewerOptions, AAvaSequenceActor::StaticClass(), ClassFilterFuncs))
		{
			return false;
		}
	}

	if (UActorFactory::CanCreateActorFrom(InAssetData, OutErrorMessage))
	{
		return true;
	}

	if (InAssetData.IsValid() && !InAssetData.IsInstanceOf(UAvaSequence::StaticClass()))
	{
		OutErrorMessage = LOCTEXT("NoSequenceAsset", "A valid Motion Design Sequence must be specified.");
		return false;
	}

	return true;
}

AActor* UAvaSequenceActorFactory::SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams)
{
	if (AAvaSequenceActor* NewActor = Cast<AAvaSequenceActor>(Super::SpawnActor(InAsset, InLevel, InTransform, InSpawnParams)))
	{
		if (UAvaSequence* Sequence = Cast<UAvaSequence>(InAsset))
		{
			NewActor->Initialize(Sequence);
		}
		return NewActor;
	}
	return nullptr;
}

UObject* UAvaSequenceActorFactory::GetAssetFromActorInstance(AActor* InActorInstance)
{
	if (AAvaSequenceActor* SequenceActor = Cast<AAvaSequenceActor>(InActorInstance))
	{
		return SequenceActor->GetSequence();
	}
	return nullptr;
}

FString UAvaSequenceActorFactory::GetDefaultActorLabel(UObject* InAsset) const
{
	if (UAvaSequence* Sequence = Cast<UAvaSequence>(InAsset))
	{
		return Sequence->GetLabel().ToString();
	}
	return Super::GetDefaultActorLabel(InAsset);
}

#undef LOCTEXT_NAMESPACE
