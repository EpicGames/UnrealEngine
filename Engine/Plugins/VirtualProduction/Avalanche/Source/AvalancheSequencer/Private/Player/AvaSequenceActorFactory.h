// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"
#include "AvaSequenceActorFactory.generated.h"

UCLASS()
class UAvaSequenceActorFactory : public UActorFactory
{
	GENERATED_BODY()

public:
	UAvaSequenceActorFactory();

protected:
	//~ Begin UActorFactory
	virtual bool CanCreateActorFrom(const FAssetData& InAssetData, FText& OutErrorMessage) override;
	virtual AActor* SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams) override;
	virtual UObject* GetAssetFromActorInstance(AActor* InActorInstance) override;
	virtual FString GetDefaultActorLabel(UObject* InAsset) const override;
	//~ End UActorFactory
};
