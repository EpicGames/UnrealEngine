// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaActorSubobjectSchema.h"
#include "GameFramework/Actor.h"

UObject* FAvaActorSubobjectSchema::GetParentObject(UObject* InObject) const
{
	if (InObject)
	{
		return InObject->GetTypedOuter<AActor>();
	}
	return nullptr;
}

UE::Sequencer::FObjectSchemaRelevancy FAvaActorSubobjectSchema::GetRelevancy(const UObject* InObject) const
{
	// Return a non-zero priority so that it wins against the default relevancy
	return UE::Sequencer::FObjectSchemaRelevancy(nullptr, 1);
}

TSharedPtr<FExtender> FAvaActorSubobjectSchema::ExtendObjectBindingMenu(TSharedRef<FUICommandList> InCommandList
	, TWeakPtr<ISequencer> InSequencerWeak
	, TConstArrayView<UObject*> InContextSensitiveObjects) const
{
	return nullptr;
}
