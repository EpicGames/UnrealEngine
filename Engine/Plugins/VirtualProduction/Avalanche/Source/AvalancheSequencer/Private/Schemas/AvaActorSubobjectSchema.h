// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerObjectSchema.h"

class FAvaActorSubobjectSchema : public UE::Sequencer::IObjectSchema
{
public:
	//~ Begin IObjectSchema
	virtual UObject* GetParentObject(UObject* InObject) const override;
	virtual UE::Sequencer::FObjectSchemaRelevancy GetRelevancy(const UObject* InObject) const override;
	virtual TSharedPtr<FExtender> ExtendObjectBindingMenu(TSharedRef<FUICommandList> InCommandList, TWeakPtr<ISequencer> InSequencerWeak, TConstArrayView<UObject*> InContextSensitiveObjects) const override;
	//~ End IObjectSchema
};
