// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceItem.h"
#include "AvaSequence.h"
#include "AvaSequencePlaybackObject.h"
#include "AvaSequencePlayer.h"
#include "AvaSequencer.h"
#include "DragDropOps/AvaSequenceItemDragDropOp.h"
#include "Editor.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Player/AvaSequenceActorFactory.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "AvaSequenceItem"

FAvaSequenceItem::FAvaSequenceItem(UAvaSequence* InSequence, const TSharedPtr<FAvaSequencer>& InSequencer)
	: SequencerWeak(InSequencer)
	, SequenceWeak(InSequence)
{
}

UAvaSequence* FAvaSequenceItem::GetSequence() const
{
	return SequenceWeak.Get();
}

FText FAvaSequenceItem::GetDisplayNameText() const
{
	if (SequenceWeak.IsValid())
	{
		return SequenceWeak->GetDisplayName();
	}
	return FText::GetEmpty();
}

const FSlateBrush* FAvaSequenceItem::GetIconBrush() const
{
	return FAppStyle::Get().GetBrush("LevelEditor.OpenCinematic");
}

FName FAvaSequenceItem::GetLabel() const
{
	return SequenceWeak.IsValid() ? SequenceWeak->GetLabel() : NAME_None;
}

bool FAvaSequenceItem::CanRelabel(const FText& InText, FText& OutErrorMessage) const
{
	if (!SequenceWeak.IsValid())
	{
		OutErrorMessage = LOCTEXT("SequenceInvalid", "The sequence is in an invalid state");
		return false;
	}

	const FString NewLabel = InText.ToString();

	// Return true that we can relabel to the same label
	if (SequenceWeak->GetLabel() == *NewLabel)
	{
		return true;
	}
	
	if (NewLabel != SlugStringForValidName(NewLabel))
	{
		const FString InvalidCharacters = INVALID_OBJECTNAME_CHARACTERS;
		FString FoundInvalidCharacters;
	
		// Create a string with all invalid characters found to output with the error message.
		for (int32 StringIndex = 0; StringIndex < InvalidCharacters.Len(); ++StringIndex)
		{
			const FString CurrentInvalidCharacter = InvalidCharacters.Mid(StringIndex, 1);
			if (NewLabel.Contains(CurrentInvalidCharacter))
			{
				FoundInvalidCharacters += CurrentInvalidCharacter;
			}
		}
			
		OutErrorMessage = FText::Format(LOCTEXT("NameContainsInvalidCharacters"
			, "The object name may not contain the following characters:  {0}")
			, FText::FromString(FoundInvalidCharacters));
			
		return false;
	}

	return true;
}

void FAvaSequenceItem::Relabel(const FName InLabel)
{
	UAvaSequence* const Sequence = SequenceWeak.Get();
	if (!Sequence)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("RelabelSequence", "Relabel Sequence"));
	Sequence->Modify();
	Sequence->SetLabel(InLabel);
}

void FAvaSequenceItem::RequestRelabel()
{
	Broadcast(OnRelabel);
}

bool FAvaSequenceItem::GetSequenceStatus(EMovieScenePlayerStatus::Type* OutStatus
	, FFrameTime* OutCurrentFrame
	, FFrameTime* OutTotalFrames) const
{
	if (!SequencerWeak.IsValid() || !SequenceWeak.IsValid())
	{
		return false;
	}

	const IAvaSequencerProvider& Provider    = SequencerWeak.Pin()->GetProvider();
	IAvaSequencePlaybackObject* const PlaybackObject = Provider.GetPlaybackObject();

	if (!PlaybackObject)
	{
		return false;
	}

	UAvaSequencePlayer* const Player = PlaybackObject->GetSequencePlayer(SequenceWeak.Get());
	if (!Player)
	{
		return false;
	}

	if (OutStatus)
	{
		*OutStatus = Player->GetPlaybackStatus();
	}

	if (OutCurrentFrame)
	{
		*OutCurrentFrame = Player->GetCurrentTime().ConvertTo(Player->GetDisplayRate());
	}

	if (OutTotalFrames)
	{
		*OutTotalFrames = Player->GetDuration().ConvertTo(Player->GetDisplayRate());
	}

	return true;
}

void FAvaSequenceItem::RefreshChildren()
{
	if (!SequenceWeak.IsValid())
	{
		Children.Reset();
		return;
	}

	const TSet<TWeakObjectPtr<UAvaSequence>> ChildSequences(SequenceWeak->GetChildren());
	
	TSet<TWeakObjectPtr<UAvaSequence>> SeenSequences;
	SeenSequences.Reserve(ChildSequences.Num());

	//Remove Current Items that are not in the Latest Children Set
	for (TArray<FAvaSequenceItemPtr>::TIterator Iter(Children); Iter; ++Iter)
	{
		const FAvaSequenceItemPtr Item = *Iter;

		if (!Item.IsValid())
		{
			Iter.RemoveCurrent();
			continue;
		}

		TWeakObjectPtr<UAvaSequence> Sequence = Item->GetSequence();

		if (Sequence.IsValid() && ChildSequences.Contains(Sequence))
		{
			SeenSequences.Add(Sequence);
		}
		else
		{
			Iter.RemoveCurrent();
		}
	}

	//Make New Items for the Sequences that were not Seen
	{
		TSharedPtr<FAvaSequenceItem> This = SharedThis(this);
		TArray<TWeakObjectPtr<UAvaSequence>> NewSequences = ChildSequences.Difference(SeenSequences).Array();
		
		Children.Reserve(Children.Num() + NewSequences.Num());

		const TSharedPtr<FAvaSequencer> Sequencer = SequencerWeak.Pin();
		check(Sequencer.IsValid());
		
		for (const TWeakObjectPtr<UAvaSequence>& Sequence : NewSequences)
		{
			TSharedPtr<FAvaSequenceItem> NewItem = MakeShared<FAvaSequenceItem>(Sequence.Get(), Sequencer);
			Children.Add(NewItem);
		}
	}
}

const TArray<FAvaSequenceItemPtr>& FAvaSequenceItem::GetChildren() const
{
	return Children;
}

TOptional<EItemDropZone> FAvaSequenceItem::OnCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone)
{
	if (TSharedPtr<FAvaSequenceItemDragDropOp> SequenceItemDragDropOp = InDragDropEvent.GetOperationAs<FAvaSequenceItemDragDropOp>())
	{
		return SequenceItemDragDropOp->OnCanDropItem(SharedThis(this));
	}
	return TOptional<EItemDropZone>();
}

FReply FAvaSequenceItem::OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone)
{
	if (TSharedPtr<FAvaSequenceItemDragDropOp> SequenceItemDragDropOp = InDragDropEvent.GetOperationAs<FAvaSequenceItemDragDropOp>())
	{
		return SequenceItemDragDropOp->OnDropOnItem(SharedThis(this));
	}
	return FReply::Unhandled();
}

FReply FAvaSequenceItem::OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	UAvaSequenceActorFactory* SequenceActorFactory;
	if (GEditor && GEditor->ActorFactories.FindItemByClass(&SequenceActorFactory))
	{
		return FReply::Handled().BeginDragDrop(FAvaSequenceItemDragDropOp::New(SharedThis(this), SequenceActorFactory));
	}
	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
