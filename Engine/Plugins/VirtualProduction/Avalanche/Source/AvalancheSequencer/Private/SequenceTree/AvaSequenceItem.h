// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequenceTree/IAvaSequenceItem.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FAvaSequencer;

class FAvaSequenceItem : public IAvaSequenceItem
{
public:
	FAvaSequenceItem(UAvaSequence* InSequence, const TSharedPtr<FAvaSequencer>& InSequencer);

	//~ Begin IAvaSequenceItem
	virtual UAvaSequence* GetSequence() const override;
	virtual FText GetDisplayNameText() const override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual FName GetLabel() const override;
	virtual bool CanRelabel(const FText& InText, FText& OutErrorMessage) const override;
	virtual void Relabel(const FName InLabel) override;
	virtual void RequestRelabel() override;
	virtual TAvaSequenceItemDelegate<FSimpleMulticastDelegate>& GetOnRelabel() override { return OnRelabel; };
	virtual bool GetSequenceStatus(EMovieScenePlayerStatus::Type* OutStatus, FFrameTime* OutCurrentFrame, FFrameTime* OutTotalFrames) const override;
	virtual void RefreshChildren() override;
	virtual const TArray<FAvaSequenceItemPtr>& GetChildren() const override;
	virtual TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) override;
	virtual FReply OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) override;
	virtual FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	//~ End IAvaSequenceItem

protected:
	TWeakPtr<FAvaSequencer> SequencerWeak;

	TWeakObjectPtr<UAvaSequence> SequenceWeak;	
	
	TAvaSequenceItemDelegate<FSimpleMulticastDelegate> OnRelabel;

	TArray<FAvaSequenceItemPtr> Children;
};
