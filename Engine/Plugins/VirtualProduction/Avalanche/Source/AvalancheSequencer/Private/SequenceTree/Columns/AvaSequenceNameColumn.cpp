// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceNameColumn.h"
#include "Widgets/SAvaSequenceName.h"

#define LOCTEXT_NAMESPACE "AvaSequenceNameColumn"

FText FAvaSequenceNameColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("SequenceName_Name", "Label");
}

FText FAvaSequenceNameColumn::GetColumnToolTipText() const
{
	return LOCTEXT("SequenceName_ToolTip", "The label of the Sequence");
}

SHeaderRow::FColumn::FArguments FAvaSequenceNameColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnToolTipText())
		.FillWidth(0.5f)
		.ShouldGenerateWidget(true)
		.VAlignCell(EVerticalAlignment::VAlign_Center)
	;
}

TSharedRef<SWidget> FAvaSequenceNameColumn::ConstructRowWidget(const FAvaSequenceItemPtr& InItem, const TSharedPtr<SAvaSequenceItemRow>& InRow)
{
	return SNew(SAvaSequenceName, InItem, InRow);
}

#undef LOCTEXT_NAMESPACE
