// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequenceTree/Columns/IAvaSequenceColumn.h"

class FAvaSequenceNameColumn : public IAvaSequenceColumn
{
public:
	UE_AVA_INHERITS(FAvaSequenceNameColumn, IAvaSequenceColumn);

	//~ Begin IAvaSequenceColumn
	virtual FText GetColumnDisplayNameText() const override;
	virtual FText GetColumnToolTipText() const override;
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual TSharedRef<SWidget> ConstructRowWidget(const FAvaSequenceItemPtr& InItem, const TSharedPtr<SAvaSequenceItemRow>& InRow) override;
	//~ End IAvaSequenceColumn
};
