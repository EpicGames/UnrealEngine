// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceStatusColumn.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SAvaSequenceStatus.h"

#define LOCTEXT_NAMESPACE "AvaSequenceStatusColumn"

FText FAvaSequenceStatusColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("SequenceStatus_Name", "Status");
}

FText FAvaSequenceStatusColumn::GetColumnToolTipText() const
{
	return LOCTEXT("SequenceStatus_ToolTip", "The status of the Sequence, if playing");
}

SHeaderRow::FColumn::FArguments FAvaSequenceStatusColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnToolTipText())
		.FillWidth(0.5f)
		.VAlignCell(EVerticalAlignment::VAlign_Center)
	;
}

TSharedRef<SWidget> FAvaSequenceStatusColumn::ConstructRowWidget(const FAvaSequenceItemPtr& InItem, const TSharedPtr<SAvaSequenceItemRow>& InRow)
{
	return SNew(SBox)
		.Padding(5.f, 2.f, 5.f, 2.f)
		[
			SNew(SAvaSequenceStatus, InItem, InRow)
		];
}

#undef LOCTEXT_NAMESPACE
