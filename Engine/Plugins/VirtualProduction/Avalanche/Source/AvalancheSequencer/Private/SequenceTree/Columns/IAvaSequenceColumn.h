// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequenceItemShared.h"
#include "AvaType.h"
#include "Widgets/Views/SHeaderRow.h"

class SAvaSequenceItemRow;

class IAvaSequenceColumn : public IAvaTypeCastable, public TSharedFromThis<IAvaSequenceColumn>
{
public:
	UE_AVA_INHERITS(IAvaSequenceColumn, IAvaTypeCastable);

	FName GetColumnId() const { return GetTypeId().ToName(); }

	virtual FText GetColumnDisplayNameText() const = 0;

	virtual FText GetColumnToolTipText() const = 0;

	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() = 0;

	virtual TSharedRef<SWidget> ConstructRowWidget(const FAvaSequenceItemPtr& InItem, const TSharedPtr<SAvaSequenceItemRow>& InRow) = 0;
};
