// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaSequenceName.h"
#include "SequenceTree/IAvaSequenceItem.h"
#include "SequenceTree/Widgets/SAvaSequenceItemRow.h"
#include "AvaSequence.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaSequenceName"

void SAvaSequenceName::Construct(const FArguments& InArgs, const FAvaSequenceItemPtr& InItem, const TSharedPtr<SAvaSequenceItemRow>& InRow)
{
	ItemWeak = InItem;

	InItem->GetOnRelabel().AddListener(FSimpleDelegate::CreateRaw(this, &SAvaSequenceName::BeginRename));

	SetToolTipText(TAttribute<FText>::CreateSP(this, &SAvaSequenceName::GetToolTipText));

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SExpanderArrow, InRow)
			.IndentAmount(8)
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SAssignNew(InlineTextBlock, SInlineEditableTextBlock)
			.Font(FCoreStyle::Get().GetFontStyle("NormalFont"))
			.Text(this, &SAvaSequenceName::GetSequenceNameText)
			.OnVerifyTextChanged(this, &SAvaSequenceName::OnVerifyNameTextChanged)
			.OnTextCommitted(this, &SAvaSequenceName::OnNameTextCommitted)
			.IsSelected(InRow.ToSharedRef(), &SAvaSequenceItemRow::IsSelectedExclusively)
		]
	];	
}

SAvaSequenceName::~SAvaSequenceName()
{
	if (ItemWeak.IsValid())
	{
		ItemWeak.Pin()->GetOnRelabel().RemoveListener(this);
	}
}

FText SAvaSequenceName::GetSequenceNameText() const
{
	if (ItemWeak.IsValid())
	{
		return FText::FromName(ItemWeak.Pin()->GetLabel());
	}
	return FText::GetEmpty();
}

void SAvaSequenceName::BeginRename()
{
	InlineTextBlock->EnterEditingMode();
}

bool SAvaSequenceName::OnVerifyNameTextChanged(const FText& InText, FText& OutErrorMessage)
{
	TSharedPtr<IAvaSequenceItem> Item = ItemWeak.Pin();
	check(Item.IsValid());
	return Item->CanRelabel(InText, OutErrorMessage);
}

void SAvaSequenceName::OnNameTextCommitted(const FText& InText, ETextCommit::Type InCommitInfo)
{
	TSharedPtr<IAvaSequenceItem> Item = ItemWeak.Pin();
	check(Item.IsValid());
	Item->Relabel(FName(InText.ToString()));
}

FText SAvaSequenceName::GetToolTipText() const
{
	TSharedPtr<IAvaSequenceItem> Item = ItemWeak.Pin();
	check(Item.IsValid());
	return FText::Format(LOCTEXT("SequenceNameTooltip", 
		"Display Name: {0}\n"
		"Label: {1}\n\n"
		"Alt + Drag to add to current sequence"),
		Item->GetDisplayNameText(),
		FText::FromName(Item->GetLabel()));
}

#undef LOCTEXT_NAMESPACE
