// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequenceItemShared.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class SAvaSequenceItemRow;
class SInlineEditableTextBlock;

class SAvaSequenceName : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaSequenceName) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FAvaSequenceItemPtr& InItem, const TSharedPtr<SAvaSequenceItemRow>& InRow);

	virtual ~SAvaSequenceName() override;
	
	FText GetSequenceNameText() const;

	void BeginRename();

	bool OnVerifyNameTextChanged(const FText& InText, FText& OutErrorMessage);
	
	void OnNameTextCommitted(const FText& InText, ETextCommit::Type InCommitInfo);
	
protected:
	virtual FText GetToolTipText() const;

	TWeakPtr<IAvaSequenceItem> ItemWeak;
	
	TSharedPtr<SInlineEditableTextBlock> InlineTextBlock;
};
