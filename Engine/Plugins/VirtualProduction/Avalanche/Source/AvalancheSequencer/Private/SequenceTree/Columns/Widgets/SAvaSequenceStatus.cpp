// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaSequenceStatus.h"
#include "SequenceTree/IAvaSequenceItem.h"
#include "MovieSceneFwd.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "SAvaSequenceStatus"

void SAvaSequenceStatus::Construct(const FArguments& InArgs
	, const FAvaSequenceItemPtr& InItem
	, const TSharedPtr<SAvaSequenceItemRow>& InRow)
{
	ItemWeak = InItem;
	ChildSlot
	[
		SNew(SOverlay)
		+SOverlay::Slot()
		.Padding(1.f, 1.f)
		[
			SNew(SProgressBar)
			.Percent(this, &SAvaSequenceStatus::GetProgressPercent)
			.Visibility(EVisibility::Visible)
		]
		+SOverlay::Slot()
		.Padding(1.f, 0.f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(this, &SAvaSequenceStatus::GetProgressText)
			.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("SmallText"))
			.Justification(ETextJustify::Type::Center)
		]
	];
}

void SAvaSequenceStatus::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	FAvaSequenceItemPtr Item = ItemWeak.Pin();
	if (Item.IsValid())
	{
		EMovieScenePlayerStatus::Type Status;
		bSequenceInProgress = Item->GetSequenceStatus(&Status, &CurrentFrame, &TotalFrames);
		
		if (bSequenceInProgress)
		{
			bSequenceInProgress = true;

			FFormatNamedArguments Args;
			if (Status == EMovieScenePlayerStatus::Playing)
			{
				Args.Add("Status", LOCTEXT("SequenceStatus_Playing", "Playing"));
			}
			else
			{
				Args.Add("Status", LOCTEXT("SequenceStatus_Stopped" , "Stopped"));
			}
			Args.Add("Current", FText::AsNumber(CurrentFrame.GetFrame().Value));
			Args.Add("Total", FText::AsNumber(TotalFrames.GetFrame().Value));

			StatusText =  FText::Format(LOCTEXT("SequenceStatus_Text", "{Status} ({Current} / {Total})"), Args);

			if (!FMath::IsNearlyZero(TotalFrames.AsDecimal()))
			{
				Progress = CurrentFrame.AsDecimal() / TotalFrames.AsDecimal();
			}
			else
			{
				Progress = 0.f;
			}
		}
		else
		{
			StatusText = LOCTEXT("SequenceStatus_Unknown", "Not Playing");
			Progress = 0.f;
		}
	}
}

FText SAvaSequenceStatus::GetProgressText() const
{
	return StatusText;
}

TOptional<float> SAvaSequenceStatus::GetProgressPercent() const
{
	return Progress;
}

#undef LOCTEXT_NAMESPACE
