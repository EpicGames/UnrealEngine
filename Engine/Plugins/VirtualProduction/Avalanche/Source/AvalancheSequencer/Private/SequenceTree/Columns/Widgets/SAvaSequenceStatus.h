// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequenceItemShared.h"
#include "Internationalization/Text.h"
#include "Misc/FrameTime.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class SAvaSequenceItemRow;
template<typename OptionalType> struct TOptional;

class SAvaSequenceStatus : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaSequenceStatus) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FAvaSequenceItemPtr& InItem, const TSharedPtr<SAvaSequenceItemRow>& InRow);

	//~ Begin SWidget
	virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

	FText GetProgressText() const;

	TOptional<float> GetProgressPercent() const;

private:
	TWeakPtr<IAvaSequenceItem> ItemWeak;

	FText StatusText;

	FFrameTime CurrentFrame;

	FFrameTime TotalFrames;

	float Progress = 0.f;

	bool bSequenceInProgress = false;
};
