// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSequenceItemDragDropOp.h"
#include "AvaSequence.h"
#include "Player/AvaSequenceActorFactory.h"
#include "ScopedTransaction.h"
#include "SequenceTree/IAvaSequenceItem.h"

#define LOCTEXT_NAMESPACE "AvaSequenceItemDragDropOp"

TSharedRef<FAvaSequenceItemDragDropOp> FAvaSequenceItemDragDropOp::New(const FAvaSequenceItemPtr& InSequenceItem, UActorFactory* InActorFactory)
{
	TSharedRef<FAvaSequenceItemDragDropOp> DragDropOp = MakeShared<FAvaSequenceItemDragDropOp>();
	DragDropOp->Init(InSequenceItem, InActorFactory);
	return DragDropOp;
}

TOptional<EItemDropZone> FAvaSequenceItemDragDropOp::OnCanDropItem(FAvaSequenceItemPtr InTargetItem) const
{
	if (!InTargetItem.IsValid() || !SequenceItem.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	UAvaSequence* const Sequence       = SequenceItem->GetSequence();
	UAvaSequence* const TargetSequence = InTargetItem->GetSequence();

	if (!Sequence || !TargetSequence || Sequence == TargetSequence)
	{
		return TOptional<EItemDropZone>();
	}

	return EItemDropZone::OntoItem;
}

FReply FAvaSequenceItemDragDropOp::OnDropOnItem(FAvaSequenceItemPtr InTargetItem)
{
	if (!InTargetItem.IsValid() || !SequenceItem.IsValid())
	{
		return FReply::Unhandled();
	}

	UAvaSequence* const Sequence       = SequenceItem->GetSequence();
	UAvaSequence* const TargetSequence = InTargetItem->GetSequence();

	if (!Sequence || !TargetSequence || Sequence == TargetSequence)
	{
		return FReply::Unhandled();
	}

	FScopedTransaction Transaction(LOCTEXT("ReparentSequence", "Reparent Sequence"));

	//Check if Target Item is a Parent of Sequence Item (so that we can move the item up the hierarchy)
	if (TargetSequence == Sequence->GetParent())
	{
		TargetSequence->RemoveChild(Sequence);
		if (UAvaSequence* const TargetParent = TargetSequence->GetParent())
		{
			TargetParent->AddChild(Sequence);
		}
		return FReply::Handled();
	}
	
	//Check if the Target Item is a Child of Sequence Item (to prevent cycle)
	UAvaSequence* CurrentItem = TargetSequence;
    while (CurrentItem)
    {
    	if (CurrentItem->GetParent() == Sequence)
    	{
    		Sequence->RemoveChild(CurrentItem);
    		CurrentItem->SetParent(Sequence->GetParent());
    		break;
    	}
    	CurrentItem = CurrentItem->GetParent();
    }
	
	TargetSequence->AddChild(Sequence);
	return FReply::Handled();
}

void FAvaSequenceItemDragDropOp::Init(const FAvaSequenceItemPtr& InSequenceItem, UActorFactory* InActorFactory)
{
	FAssetDragDropOp::Init({ InSequenceItem->GetSequence() }, TArray<FString>(), InActorFactory);

	SequenceItem = InSequenceItem;
	CurrentHoverText = InSequenceItem->GetDisplayNameText();
	CurrentIconBrush = InSequenceItem->GetIconBrush();
	CurrentIconColorAndOpacity = FSlateColor::UseForeground();
	MouseCursor = EMouseCursor::GrabHandClosed;

	SetupDefaults();
	Construct();
}

TSharedPtr<SWidget> FAvaSequenceItemDragDropOp::GetDefaultDecorator() const
{
	// Skipping FAssetDragDropOp::GetDefaultDecorator
	return FDecoratedDragDropOp::GetDefaultDecorator();
}

#undef LOCTEXT_NAMESPACE
