// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequenceItemShared.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Widgets/Views/STableRow.h"

class FReply;
class UActorFactory;
template<typename OptionalType> struct TOptional;

class FAvaSequenceItemDragDropOp : public FAssetDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FAvaSequenceItemDragDropOp, FAssetDragDropOp)

	static TSharedRef<FAvaSequenceItemDragDropOp> New(const FAvaSequenceItemPtr& InSequenceItem, UActorFactory* InActorFactory);

	TOptional<EItemDropZone> OnCanDropItem(FAvaSequenceItemPtr InTargetItem) const;

	FReply OnDropOnItem(FAvaSequenceItemPtr InTargetItem);

protected:
	void Init(const FAvaSequenceItemPtr& InSequenceItem, UActorFactory* InActorFactory);

	//~ Begin FAssetDragDropOp
	virtual void InitThumbnail() override {}
	//~ End FAssetDragDropOp

	//~ Begin FDecoratedDragDropOp
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	//~ End FDecoratedDragDropOp

	/** Shared Ptr to keep Reference count while drag dropping */
	FAvaSequenceItemPtr SequenceItem;
};
