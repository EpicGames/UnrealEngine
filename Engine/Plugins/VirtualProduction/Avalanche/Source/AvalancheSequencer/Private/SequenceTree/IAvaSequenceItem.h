// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequenceItemShared.h"
#include "Delegates/Delegate.h"
#include "Misc/Optional.h"
#include "MovieSceneFwd.h"

class FDragDropEvent;
class FReply;
class FText;
class UAvaSequence;
enum class EItemDropZone;
struct FFrameTime;
struct FGeometry;
struct FPointerEvent;
struct FSlateBrush;

class IAvaSequenceItem : public TSharedFromThis<IAvaSequenceItem>
{
public:
	virtual ~IAvaSequenceItem() = default;

	virtual UAvaSequence* GetSequence() const = 0;

	virtual FText GetDisplayNameText() const = 0;

	virtual const FSlateBrush* GetIconBrush() const = 0;

	virtual FName GetLabel() const = 0;

	virtual bool CanRelabel(const FText& InText, FText& OutErrorMessage) const = 0;

	virtual void Relabel(const FName InLabel) = 0;

	virtual void RequestRelabel() = 0;

	virtual TAvaSequenceItemDelegate<FSimpleMulticastDelegate>& GetOnRelabel() = 0;
	
	virtual bool GetSequenceStatus(EMovieScenePlayerStatus::Type* OutStatus = nullptr
		, FFrameTime* OutCurrentFrame = nullptr
		, FFrameTime* OutTotalFrames = nullptr) const = 0;

	virtual void RefreshChildren() = 0;

	virtual const TArray<FAvaSequenceItemPtr>& GetChildren() const = 0; 

	virtual TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) = 0;

	virtual FReply OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) = 0;

	virtual FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) = 0;
	
protected:
	template<typename T>
	void Broadcast(TAvaSequenceItemDelegate<T>& InDelegate)
	{
		InDelegate.Broadcast();
	}
};
