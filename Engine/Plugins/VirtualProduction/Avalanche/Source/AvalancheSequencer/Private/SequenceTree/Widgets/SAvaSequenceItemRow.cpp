// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaSequenceItemRow.h"
#include "SequenceTree/Columns/IAvaSequenceColumn.h"
#include "AvaSequencer.h"
#include "SequenceTree/IAvaSequenceItem.h"

#define LOCTEXT_NAMESPACE "SAvaSequenceItemRow"

void SAvaSequenceItemRow::Construct(const FArguments& InArgs
	, const TSharedPtr<STableViewBase>& InOwnerTableView
	, const FAvaSequenceItemPtr& InItem
	, const TSharedPtr<FAvaSequencer>& InSequencer)
{
	ItemWeak      = InItem;
	SequencerWeak = InSequencer;

	SMultiColumnTableRow::Construct(FSuperRowType::FArguments()
			.OnDragDetected(InItem.ToSharedRef(), &IAvaSequenceItem::OnDragDetected)
			.OnCanAcceptDrop(this, &SAvaSequenceItemRow::OnCanAcceptDrop)
			.OnAcceptDrop(this, &SAvaSequenceItemRow::OnAcceptDrop)
			.Padding(FMargin(0.f, 3.f))
		, InOwnerTableView.ToSharedRef());
}

FReply SAvaSequenceItemRow::OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone
	, FAvaSequenceItemPtr InItem)
{
	if (InItem.IsValid())
	{
		return InItem->OnAcceptDrop(InDragDropEvent, InDropZone);
	}
	return FReply::Unhandled();
}

TOptional<EItemDropZone> SAvaSequenceItemRow::OnCanAcceptDrop(const FDragDropEvent& InDragDropEvent
	, EItemDropZone InDropZone, FAvaSequenceItemPtr InItem) const
{
	if (InItem.IsValid())
	{
		return InItem->OnCanAcceptDrop(InDragDropEvent, InDropZone);
	}
	return TOptional<EItemDropZone>();
}

TSharedRef<SWidget> SAvaSequenceItemRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	if (SequencerWeak.IsValid() && ItemWeak.IsValid())
	{
		if (const TSharedPtr<IAvaSequenceColumn> Column = SequencerWeak.Pin()->FindSequenceColumn(InColumnName))
		{
			return Column->ConstructRowWidget(ItemWeak.Pin().ToSharedRef(), SharedThis(this));
		}
	}
	return SNullWidget::NullWidget;
}

FReply SAvaSequenceItemRow::OnMouseButtonDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton
		&& InMouseEvent.IsAltDown()
		&& !InMouseEvent.IsShiftDown()
		&& !InMouseEvent.IsControlDown())
	{
		TSharedRef<ITypedTableView<FAvaSequenceItemPtr>> OwnerTable = OwnerTablePtr.Pin().ToSharedRef();

		return FReply::Handled()
			.DetectDrag(SharedThis(this), EKeys::LeftMouseButton)
			.SetUserFocus(OwnerTable->AsWidget(), EFocusCause::Mouse)
			.CaptureMouse(SharedThis(this));
	}

	return SMultiColumnTableRow::OnMouseButtonDown(InMyGeometry, InMouseEvent);
}

FReply SAvaSequenceItemRow::OnMouseButtonUp(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton
		&& InMouseEvent.IsAltDown()
		&& !InMouseEvent.IsShiftDown()
		&& !InMouseEvent.IsControlDown()
		&& HasMouseCapture())
	{
		const bool bIsUnderMouse = InMyGeometry.IsUnderLocation(InMouseEvent.GetScreenSpacePosition());
		if (bIsUnderMouse && !bDragWasDetected)
		{
			TSharedRef<ITypedTableView<FAvaSequenceItemPtr>> OwnerTable = OwnerTablePtr.Pin().ToSharedRef();

			if (const TObjectPtrWrapTypeOf<FAvaSequenceItemPtr>* MyItemPtr = GetItemForThis(OwnerTable))
			{
				OwnerTable->Private_ClearSelection();
				OwnerTable->Private_SetItemSelection(*MyItemPtr, true, true);
				OwnerTable->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);

				return FReply::Handled().ReleaseMouseCapture();
			}
		}
	}

	return SMultiColumnTableRow::OnMouseButtonUp(InMyGeometry, InMouseEvent);
}

#undef LOCTEXT_NAMESPACE
