// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequenceItemShared.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/STableRow.h"

class FAvaSequencer;
class FReply;
class SInlineEditableTextBlock;
template<typename OptionalType> struct TOptional;
class UAvaSequence;

class SAvaSequenceItemRow : public SMultiColumnTableRow<FAvaSequenceItemPtr>
{
public:
	SLATE_BEGIN_ARGS(SAvaSequenceItemRow) {}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs
		, const TSharedPtr<STableViewBase>& InOwnerTableView
		, const FAvaSequenceItemPtr& InItem
		, const TSharedPtr<FAvaSequencer>& InSequencer);
	
	FReply OnAcceptDrop(const FDragDropEvent& InDragDropEvent
		, EItemDropZone InDropZone
		, FAvaSequenceItemPtr InItem);
	
	TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& InDragDropEvent
		, EItemDropZone InDropZone
		, FAvaSequenceItemPtr InItem) const;
	
	//~ Begin SMultiColumnTableRow
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;
	//~ End SMultiColumnTableRow

protected:
	//~ Begin SWidget
	virtual FReply OnMouseButtonDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	//~ End SWidget

	TWeakPtr<FAvaSequencer> SequencerWeak;
	
	TWeakPtr<IAvaSequenceItem> ItemWeak;	
};
