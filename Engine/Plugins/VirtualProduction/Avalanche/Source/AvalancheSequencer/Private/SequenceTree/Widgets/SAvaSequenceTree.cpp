// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaSequenceTree.h"
#include "AvaSequence.h"
#include "AvaSequencer.h"
#include "AvaSequencerMenuContext.h"
#include "Commands/AvaSequencerCommands.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISettingsModule.h"
#include "Misc/TextFilter.h"
#include "SPositiveActionButton.h"
#include "SequenceTree/AvaSequenceItem.h"
#include "SequenceTree/Widgets/SAvaSequenceItemRow.h"
#include "Settings/AvaSequencerSettings.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "SAvaSequenceTree"

namespace UE::AvaSequencer::Private
{
	void GeneratePresetMenu(UToolMenu* InToolMenu)
	{
		const UAvaSequencerSettings* SequencerSettings = GetDefault<UAvaSequencerSettings>();

		UAvaSequencerMenuContext* MenuContext = InToolMenu ? InToolMenu->FindContext<UAvaSequencerMenuContext>() : nullptr;
		if (!SequencerSettings || !MenuContext)
		{
			return;
		}

		TSharedPtr<FAvaSequencer> AvaSequencer = MenuContext->GetAvaSequencer();
		if (!AvaSequencer.IsValid())
		{
			return;
		}

		TSharedRef<FAvaSequencer> AvaSequencerRef = AvaSequencer.ToSharedRef();

		// Default Presets
		TConstArrayView<FAvaSequencePreset> DefaultPresets = SequencerSettings->GetDefaultSequencePresets();
		if (!DefaultPresets.IsEmpty())
		{
			FToolMenuSection& DefaultPresetSection = InToolMenu->FindOrAddSection(TEXT("DefaultPresets"), LOCTEXT("DefaultPresetsLabel", "Default Presets"));
			for (const FAvaSequencePreset& Preset : DefaultPresets)
			{
				DefaultPresetSection.AddMenuEntry(Preset.PresetName
					, FText::FromName(Preset.PresetName)
					, FText::FromName(Preset.PresetName)
					, FSlateIcon()
					, FToolUIActionChoice(FExecuteAction::CreateSP(AvaSequencerRef, &FAvaSequencer::ApplyDefaultPresetToSelection, Preset.PresetName)));
			}
		}

		// Custom Presets
		const TSet<FAvaSequencePreset>& CustomPresets = SequencerSettings->GetCustomSequencePresets();
		if (!CustomPresets.IsEmpty())
		{
			FToolMenuSection& CustomPresetSection = InToolMenu->FindOrAddSection(TEXT("CustomPresets"), LOCTEXT("CustomPresetsLabel", "Custom Presets"));
			for (const FAvaSequencePreset& Preset : CustomPresets)
			{
				CustomPresetSection.AddMenuEntry(Preset.PresetName
					, FText::FromName(Preset.PresetName)
					, FText::FromName(Preset.PresetName)
					, FSlateIcon()
					, FToolUIActionChoice(FExecuteAction::CreateSP(AvaSequencerRef, &FAvaSequencer::ApplyCustomPresetToSelection, Preset.PresetName)));
			}
		}

		// Settings
		FToolMenuSection& SettingsSection = InToolMenu->FindOrAddSection(TEXT("Settings"));
		SettingsSection.AddSeparator(NAME_None);
		SettingsSection.AddMenuEntry(TEXT("OpenSettings")
			, LOCTEXT("OpenSettingsLabel", "Open Settings")
			, LOCTEXT("OpenSettingsTooltip", "Opens the Settings to customize the sequence presets")
			, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Settings")
			, FExecuteAction::CreateLambda([]
			{
				ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>(TEXT("Settings"));
				SettingsModule.ShowViewer("Editor", UAvaSequencerSettings::SettingsCategory, UAvaSequencerSettings::SettingsSection);
			}));
	}
}

void SAvaSequenceTree::Construct(const FArguments& InArgs
	, const TSharedPtr<FAvaSequencer>& InSequencer
	, const TSharedPtr<SHeaderRow>& InHeaderRow)
{
	SequencerWeak = InSequencer;
	BindCommands(InSequencer.ToSharedRef());

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(FMargin(2.0))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(2.f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(0.f)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.AutoWidth()
				[
					SNew(SPositiveActionButton)
					.OnGetMenuContent(this, &SAvaSequenceTree::CreateAddSequenceMenu)
					.Text(LOCTEXT("AddSequenceText", "Add"))
				]
				+ SHorizontalBox::Slot()
				.Padding(2.f, 0.f)
				.VAlign(VAlign_Center)
				[
					SAssignNew(SearchBox, SSearchBox)
					.HintText(LOCTEXT("SearchSequencesText", "Search"))
					.OnTextChanged(this, &SAvaSequenceTree::OnSearchChanged)
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew(SequenceTreeView, STreeView<FAvaSequenceItemPtr>)
				.HeaderRow(InHeaderRow)
				.OnContextMenuOpening(this, &SAvaSequenceTree::OnContextMenuOpening)
				.OnGenerateRow(this, &SAvaSequenceTree::OnGenerateRow)
				.OnGetChildren(this, &SAvaSequenceTree::OnGetChildren)
				.OnSelectionChanged(this, &SAvaSequenceTree::OnSelectionChanged)
				.SelectionMode(ESelectionMode::Multi)
				.ClearSelectionOnClick(false)
				.TreeItemsSource(&InSequencer->GetRootSequenceItems())
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				InSequencer->CreatePlayerToolBar(CommandList.ToSharedRef())
			]
		]
	];
}

TSharedPtr<SWidget> SAvaSequenceTree::OnContextMenuOpening() const
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return nullptr;
	}

	constexpr const TCHAR* ContextMenuName = TEXT("AvaSequenceTreeContextMenu");

	if (!ToolMenus->IsMenuRegistered(ContextMenuName))
	{
		UToolMenu* const ToolMenu = ToolMenus->RegisterMenu(ContextMenuName, NAME_None, EMultiBoxType::Menu);

		FToolMenuSection& Section = ToolMenu->FindOrAddSection(TEXT("SequenceActions"), LOCTEXT("SequenceActions", "Sequence Actions"));

		const FAvaSequencerCommands& SequencerCommands = FAvaSequencerCommands::Get();

		const FGenericCommands& GenericCommands = FGenericCommands::Get();

		Section.AddSubMenu(TEXT("ApplyPreset")
			, LOCTEXT("ApplyPresetLabel", "Apply Preset")
			, LOCTEXT("ApplyPresetTooltip", "Apply a Preset to the Selected Sequences")
			, FNewToolMenuDelegate::CreateStatic(&UE::AvaSequencer::Private::GeneratePresetMenu));

		Section.AddMenuEntry(SequencerCommands.SpawnSequencePlayer);
		Section.AddMenuEntry(SequencerCommands.ExportSequence);
		Section.AddMenuEntry(GenericCommands.Rename);
		Section.AddMenuEntry(GenericCommands.Duplicate);
		Section.AddSeparator(NAME_None);
		Section.AddMenuEntry(GenericCommands.Delete);
	}

	UAvaSequencerMenuContext* MenuContext = NewObject<UAvaSequencerMenuContext>();
	MenuContext->SetAvaSequencer(SequencerWeak);

	FToolMenuContext ToolMenuContext(CommandList, nullptr, MenuContext);	
	return ToolMenus->GenerateWidget(ContextMenuName, ToolMenuContext);
}

void SAvaSequenceTree::BindCommands(const TSharedRef<FAvaSequencer>& InSequencer)
{
	CommandList = MakeShared<FUICommandList>();

	const FGenericCommands& GenericCommands = FGenericCommands::Get();

	const FAvaSequencerCommands& AvaSequencerCommands = FAvaSequencerCommands::Get();

	CommandList->MapAction(GenericCommands.Duplicate
		, FExecuteAction::CreateSP(InSequencer, &FAvaSequencer::DuplicateSequences_Execute)
		, FCanExecuteAction::CreateSP(InSequencer, &FAvaSequencer::DuplicateSequences_CanExecute));

	CommandList->MapAction(GenericCommands.Delete
		, FExecuteAction::CreateSP(InSequencer, &FAvaSequencer::DeleteSequences_Execute)
		, FCanExecuteAction::CreateSP(InSequencer, &FAvaSequencer::DeleteSequences_CanExecute));

	CommandList->MapAction(GenericCommands.Rename
		, FExecuteAction::CreateSP(InSequencer, &FAvaSequencer::RelabelSequence_Execute)
		, FCanExecuteAction::CreateSP(InSequencer, &FAvaSequencer::RelabelSequence_CanExecute));

	CommandList->MapAction(AvaSequencerCommands.PlaySelected
		, FExecuteAction::CreateSP(InSequencer, &FAvaSequencer::PlaySequences_Execute)
		, FCanExecuteAction::CreateSP(InSequencer, &FAvaSequencer::PlaySequences_CanExecute));

	CommandList->MapAction(AvaSequencerCommands.ContinueSelected
		, FExecuteAction::CreateSP(InSequencer, &FAvaSequencer::ContinueSequences_Execute)
		, FCanExecuteAction::CreateSP(InSequencer, &FAvaSequencer::ContinueSequences_CanExecute));

	CommandList->MapAction(AvaSequencerCommands.StopSelected
		, FExecuteAction::CreateSP(InSequencer, &FAvaSequencer::StopSequences_Execute)
		, FCanExecuteAction::CreateSP(InSequencer, &FAvaSequencer::StopSequences_CanExecute));

	CommandList->MapAction(AvaSequencerCommands.AddNew
		, FExecuteAction::CreateSP(InSequencer, &FAvaSequencer::AddSequence_Execute)
		, FCanExecuteAction::CreateSP(InSequencer, &FAvaSequencer::AddSequence_CanExecute));

	CommandList->MapAction(AvaSequencerCommands.ExportSequence
		, FExecuteAction::CreateSP(InSequencer, &FAvaSequencer::ExportSequences_Execute)
		, FCanExecuteAction::CreateSP(InSequencer, &FAvaSequencer::ExportSequences_CanExecute));

	CommandList->MapAction(AvaSequencerCommands.SpawnSequencePlayer
		, FExecuteAction::CreateSP(InSequencer, &FAvaSequencer::SpawnPlayers_Execute)
		, FCanExecuteAction::CreateSP(InSequencer, &FAvaSequencer::SpawnPlayers_CanExecute));
}

TSharedRef<ITableRow> SAvaSequenceTree::OnGenerateRow(FAvaSequenceItemPtr InListItem
	, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	TSharedPtr<FAvaSequencer> Sequencer = SequencerWeak.Pin();
	check(Sequencer.IsValid());
	return SNew(SAvaSequenceItemRow, InOwnerTableView, InListItem, Sequencer);
}

void SAvaSequenceTree::OnGetChildren(FAvaSequenceItemPtr InItem, TArray<FAvaSequenceItemPtr>& OutChildren) const
{
	if (InItem)
	{
		OutChildren.Append(InItem->GetChildren());
	}
}

void SAvaSequenceTree::OnPostSetViewedSequence(UAvaSequence* InSequence)
{
	if (bSyncingSelection || !SequencerWeak.IsValid())
	{
		return;
	}

	if (InSequence == nullptr)
	{
		SequenceTreeView->ClearSelection();
		return;
	}

	TArray<FAvaSequenceItemPtr> RemainingItems = SequencerWeak.Pin()->GetRootSequenceItems();
	FAvaSequenceItemPtr FoundItem;

	while(!RemainingItems.IsEmpty())
	{
		const FAvaSequenceItemPtr Item = RemainingItems.Pop();
		if (!Item.IsValid())
		{
			continue;
		}
		if (Item->GetSequence() == InSequence)
		{
			FoundItem = Item;
			break;
		}
		RemainingItems.Append(Item->GetChildren());
	}

	if (FoundItem.IsValid())
	{
		SequenceTreeView->SetSelection(FoundItem);
	}
}

void SAvaSequenceTree::OnSelectionChanged(FAvaSequenceItemPtr InSelectedItem, ESelectInfo::Type InSelectionInfo)
{
	if (bSyncingSelection)
	{
		return;
	}

	TSharedPtr<FAvaSequencer> Sequencer = SequencerWeak.Pin();
	if (!Sequencer.IsValid() || !InSelectedItem.IsValid())
	{
		return;
	}

	if (UAvaSequence* const SelectedSequence = InSelectedItem->GetSequence())
	{
		TGuardValue<bool> Guard(bSyncingSelection, true);
		Sequencer->SetViewedSequence(SelectedSequence);
	}
}

void SAvaSequenceTree::OnSearchChanged(const FText& InSearchText)
{
	if (SequencerWeak.IsValid())
	{
		FText ErrorMessage;
		SequencerWeak.Pin()->OnSequenceSearchChanged(InSearchText, ErrorMessage);
		SearchBox->SetError(ErrorMessage);
	}
}

FReply SAvaSequenceTree::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SAvaSequenceTree::CreateAddSequencePresetMenu(UToolMenu* InMenu)
{
	const UAvaSequencerMenuContext* const MenuContext = InMenu->FindContext<UAvaSequencerMenuContext>();
	if (!MenuContext)
	{
		return;
	}

	const TSharedPtr<FAvaSequencer> AvaSequencer = MenuContext->GetAvaSequencer();
	if (!AvaSequencer.IsValid())
	{
		return;
	}

	const UAvaSequencerSettings* const SequencerSettings = GetDefault<UAvaSequencerSettings>();
	if (!SequencerSettings)
	{
		return;
	}

	auto AddGroup = [AvaSequenceRef = AvaSequencer.ToSharedRef()](FToolMenuSection& InSection, const FAvaSequencePresetGroup& InGroup)
		{
			FUIAction Action;
			Action.ExecuteAction.BindSP(AvaSequenceRef, &FAvaSequencer::AddSequencesFromPresetGroup, InGroup.GroupName);
			Action.CanExecuteAction.BindSP(AvaSequenceRef, &FAvaSequencer::CanAddSequence);

			InSection.AddMenuEntry(InGroup.GroupName
				, FText::FromName(InGroup.GroupName)
				, FText::FromName(InGroup.GroupName)
				, FSlateIcon()
				, FToolUIActionChoice(Action));
		};

	// Transition Logic Groups
	FToolMenuSection& TransitionLogicSection = InMenu->FindOrAddSection(TEXT("TransitionLogicGroups"));
	TransitionLogicSection.Label = LOCTEXT("TransitionLogicGroupsLabel", "Transition Logic");
	for (const FAvaSequencePresetGroup& Group : SequencerSettings->GetTransitionLogicPresetGroups())
	{
		AddGroup(TransitionLogicSection, Group);
	}

	// Custom Groups
	if (!SequencerSettings->GetCustomPresetGroups().IsEmpty())
	{
		FToolMenuSection& CustomSection = InMenu->FindOrAddSection(TEXT("CustomGroups"));
		CustomSection.Label = LOCTEXT("CustomGroupsLabel", "Custom");
		for (const FAvaSequencePresetGroup& Group : SequencerSettings->GetCustomPresetGroups())
		{
			AddGroup(CustomSection, Group);
		}
	}
}

TSharedRef<SWidget> SAvaSequenceTree::CreateAddSequenceMenu() const
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return SNullWidget::NullWidget;
	}

	constexpr const TCHAR* ContextMenuName = TEXT("SAvaSequenceAddNewContextMenu");

	if (!ToolMenus->IsMenuRegistered(ContextMenuName))
	{
		UToolMenu* const ToolMenu = ToolMenus->RegisterMenu(ContextMenuName, NAME_None, EMultiBoxType::Menu);

		FToolMenuSection& Section = ToolMenu->FindOrAddSection(NAME_None);
		Section.AddMenuEntry(FAvaSequencerCommands::Get().AddNew
			, LOCTEXT("AddSequenceLabel", "New sequence")
			, LOCTEXT("AddSequenceTooltip", "Adds a new Motion Design sequence.")
			, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Plus")));

		ToolMenu->AddDynamicSection(TEXT("SequencePresets"), FNewToolMenuDelegate::CreateStatic(&SAvaSequenceTree::CreateAddSequencePresetMenu));
	}

	UAvaSequencerMenuContext* MenuContext = NewObject<UAvaSequencerMenuContext>();
	MenuContext->SetAvaSequencer(SequencerWeak);

	FToolMenuContext ToolMenuContext(CommandList, /*Extender*/nullptr, MenuContext);	
	return ToolMenus->GenerateWidget(ContextMenuName, ToolMenuContext);
}

#undef LOCTEXT_NAMESPACE
