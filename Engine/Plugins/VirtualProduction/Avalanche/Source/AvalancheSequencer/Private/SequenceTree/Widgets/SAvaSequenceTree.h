// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvalancheSequencer/Private/AvaSequenceItemShared.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FAvaSequencer;
class FUICommandList;
class ITableRow;
class SHeaderRow;
class SSearchBox;
class STableViewBase;
class UAvaSequence;
class UToolMenu;
template<typename ItemType> class STreeView;

class SAvaSequenceTree : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaSequenceTree) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const TSharedPtr<FAvaSequencer>& InSequencer
		, const TSharedPtr<SHeaderRow>& InHeaderRow);

	TSharedPtr<STreeView<FAvaSequenceItemPtr>> GetSequenceTreeView() const { return SequenceTreeView; }
	
	AVALANCHESEQUENCER_API TSharedPtr<SWidget> OnContextMenuOpening() const;

	void BindCommands(const TSharedRef<FAvaSequencer>& InSequencer);
	
	TSharedRef<ITableRow> OnGenerateRow(FAvaSequenceItemPtr InItem, const TSharedRef<STableViewBase>& InOwnerTableView);
	
	void OnGetChildren(FAvaSequenceItemPtr InItem, TArray<FAvaSequenceItemPtr>& OutChildren) const;

	void OnPostSetViewedSequence(UAvaSequence* InSequence);

	void OnSelectionChanged(FAvaSequenceItemPtr InSelectedItem, ESelectInfo::Type InSelectionInfo);

	void OnSearchChanged(const FText& InSearchText);

	//~ Begin SWidget
	virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget
	
protected:
	/** Adds the preset options in the add sequence menu */
	static void CreateAddSequencePresetMenu(UToolMenu* InMenu);

	/** Creates the add sequence menu when the 'add' button gets clicked */
	TSharedRef<SWidget> CreateAddSequenceMenu() const;

	TWeakPtr<FAvaSequencer> SequencerWeak;
	
	TSharedPtr<STreeView<FAvaSequenceItemPtr>> SequenceTreeView;
	
	TSharedPtr<SSearchBox> SearchBox;

	TSharedPtr<FUICommandList> CommandList;
	bool bSyncingSelection = false;
};
