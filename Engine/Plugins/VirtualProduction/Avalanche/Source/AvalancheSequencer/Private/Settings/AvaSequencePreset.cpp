// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/AvaSequencePreset.h"
#include "AvaSequence.h"

bool FAvaSequencePreset::ShouldModifySequence() const
{
	return bEnableLabel || bEnableTag || bEnableMarks;
}

bool FAvaSequencePreset::ShouldModifyMovieScene() const
{
	return bEnableMarks;
}

void FAvaSequencePreset::ApplyPreset(UAvaSequence* InSequence) const
{
	// Return early if all tags are an
	if (!InSequence)
	{
		return;
	}

	if (ShouldModifySequence())
	{
		InSequence->Modify();
	}

	if (bEnableLabel && !SequenceLabel.IsNone())
	{
		InSequence->SetLabel(SequenceLabel);
	}

	if (bEnableTag)
	{
		InSequence->SetSequenceTag(SequenceTag.MakeTagHandle());
	}

	// From here on, apply Movie Scene related settings
	UMovieScene* MovieScene = InSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	if (ShouldModifyMovieScene())
	{
		MovieScene->Modify();	
	}

	const FFrameRate TickResolution = MovieScene->GetTickResolution();

	if (bEnableEndTime)
	{
		const FFrameTime EndFrame = EndTime * TickResolution;

		const FFrameNumber StartFrame = FMath::Min(FFrameNumber(0), EndFrame.FrameNumber);
		MovieScene->SetPlaybackRange(TRange<FFrameNumber>(StartFrame, EndFrame.FrameNumber+1));
	}

	if (bEnableMarks)
	{
		MovieScene->DeleteMarkedFrames();

		const FFrameRate DisplayRate = MovieScene->GetDisplayRate();

		for (const FAvaMarkSetting& MarkSetting : Marks)
		{
			FMovieSceneMarkedFrame MarkedFrame;

			MarkedFrame.Label = MarkSetting.Label;
			MarkedFrame.FrameNumber = FFrameRate::TransformTime(FFrameTime(MarkSetting.FrameNumber), DisplayRate, TickResolution).FloorToFrame();

			MovieScene->AddMarkedFrame(MarkedFrame);
		}

		InSequence->UpdateMarkList();

		for (const FAvaMarkSetting& MarkSetting : Marks)
		{
			InSequence->FindOrAddMark(MarkSetting.Label).CopyFromMark(MarkSetting.Mark);
		}
	}
}
