// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/AvaSequencerSettings.h"
#include "AvaSequenceDefaultTags.h"
#include "Misc/FrameRate.h"

namespace UE::AvaSequencer::Private
{
	FAvaSequencePreset CreateDefaultPreset(FName InSequenceType)
	{
		FAvaSequencePreset Preset;
		Preset.PresetName     = InSequenceType;
		Preset.bEnableLabel   = true;
		Preset.SequenceLabel  = InSequenceType;
		Preset.bEnableEndTime = true;
		Preset.EndTime        = 1.0;
		return Preset;
	}

	FAvaSequencePreset CreateDefaultPreset(FName InSequenceType, const FAvaTagSoftHandle& InSequenceTag)
	{
		FAvaSequencePreset Preset = CreateDefaultPreset(InSequenceType);
		Preset.bEnableTag  = true;
		Preset.SequenceTag = InSequenceTag;
		return Preset;
	}

	TArray<FAvaSequencePreset> MakeDefaultSequencePresets()
	{
		const FAvaSequenceDefaultTags& DefaultTags = FAvaSequenceDefaultTags::Get();

		// Guids found via the entries that DefaultSequenceTags had for In/Out/Change
		return
		{
			CreateDefaultPreset(TEXT("In"), DefaultTags.In),
			CreateDefaultPreset(TEXT("Out"), DefaultTags.Out),
			CreateDefaultPreset(UAvaSequencerSettings::DefaultChangeInName),
			CreateDefaultPreset(UAvaSequencerSettings::DefaultChangeOutName),
		};
	}

	TArray<FAvaSequencePresetGroup> MakeTransitionLogicPresetGroups()
	{
		TArray<FAvaSequencePresetGroup> PresetGroups;

		// Transition Logic Starter. 4 sequences: global in/out, and a sub layer's in/out pair
		{
			FAvaSequencePresetGroup& PresetGroup = PresetGroups.AddDefaulted_GetRef();
			PresetGroup.GroupName = UAvaSequencerSettings::TransitionLogicStarterGroup;

			PresetGroup.PresetNames.SetNum(4);
			PresetGroup.PresetNames[0] = TEXT("In");
			PresetGroup.PresetNames[1] = TEXT("Out");
			PresetGroup.PresetNames[2] = UAvaSequencerSettings::DefaultChangeInName;
			PresetGroup.PresetNames[3] = UAvaSequencerSettings::DefaultChangeOutName;
		}

		// Transition Logic Sub Layer. 2 sequences: a sub layer's in/out pair
		{
			FAvaSequencePresetGroup& PresetGroup = PresetGroups.AddDefaulted_GetRef();
			PresetGroup.GroupName = UAvaSequencerSettings::TransitionLogicSubLayerGroup;

			PresetGroup.PresetNames.SetNum(2);
			PresetGroup.PresetNames[0] = UAvaSequencerSettings::DefaultChangeInName;
			PresetGroup.PresetNames[1] = UAvaSequencerSettings::DefaultChangeOutName;
		}
		return PresetGroups;
	}
}

const FLazyName UAvaSequencerSettings::TransitionLogicStarterGroup = TEXT("Transition Logic Starter");
const FLazyName UAvaSequencerSettings::TransitionLogicSubLayerGroup = TEXT("Transition Logic Sub Layer");
const FLazyName UAvaSequencerSettings::DefaultChangeInName = TEXT("Layer1_ChangeIn");
const FLazyName UAvaSequencerSettings::DefaultChangeOutName = TEXT("Layer1_ChangeOut");

UAvaSequencerSettings::UAvaSequencerSettings()
{
	CategoryName = SettingsCategory;
	SectionName = SettingsSection;

	DisplayRate.FrameRate = FFrameRate(60000, 1001);
}

const FAvaSequencePreset* UAvaSequencerSettings::FindPreset(FName InPresetName) const
{
	const FAvaSequencePreset::FKeyType Key(InPresetName);

	if (const FAvaSequencePreset* FoundPreset = CustomSequencePresets.FindByHash(GetTypeHash(Key), Key))
	{
		return FoundPreset;
	}

	if (const FAvaSequencePreset* FoundPreset = GetDefaultSequencePresets().FindByKey(Key))
	{
		return FoundPreset;
	}

	return nullptr;
}

const FAvaSequencePresetGroup* UAvaSequencerSettings::FindPresetGroup(FName InGroupName) const
{
	const FAvaSequencePresetGroup::FKeyType Key(InGroupName);

	if (const FAvaSequencePresetGroup* FoundGroup = CustomPresetGroups.FindByHash(GetTypeHash(Key), Key))
	{
		return FoundGroup;
	}

	if (const FAvaSequencePresetGroup* FoundGroup = GetTransitionLogicPresetGroups().FindByKey(Key))
	{
		return FoundGroup;
	}

	return nullptr;
}

TArray<const FAvaSequencePreset*> UAvaSequencerSettings::GatherPresetsFromGroup(FName InGroupName) const
{
	const FAvaSequencePresetGroup* PresetGroup = FindPresetGroup(InGroupName);
	if (!PresetGroup)
	{
		return {};
	}

	TArray<const FAvaSequencePreset*> Presets;
	Presets.Reserve(PresetGroup->PresetNames.Num());

	for (const FName PresetName : PresetGroup->PresetNames)
	{
		if (const FAvaSequencePreset* Preset = FindPreset(PresetName))
		{
			Presets.Add(Preset);
		}
	}
	return Presets;
}

TConstArrayView<FAvaSequencePreset> UAvaSequencerSettings::GetDefaultSequencePresets() const
{
	static const TArray<FAvaSequencePreset> DefaultPresets = UE::AvaSequencer::Private::MakeDefaultSequencePresets();
	return DefaultPresets;
}

TConstArrayView<FAvaSequencePresetGroup> UAvaSequencerSettings::GetTransitionLogicPresetGroups() const
{
	static const TArray<FAvaSequencePresetGroup> TransitionLogicGroups = UE::AvaSequencer::Private::MakeTransitionLogicPresetGroups();
	return TransitionLogicGroups;
}
