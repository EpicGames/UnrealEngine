// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaStaggerBarElement.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/Extensions/ITrackAreaExtension.h"

using namespace UE::Sequencer;

FAvaStaggerBarElement FAvaStaggerBarElement::FromTrack(const TViewModelPtr<ITrackAreaExtension>& InTrackAreaExtension)
{
	for (const FViewModelPtr& TrackAreaModel : InTrackAreaExtension->GetTopLevelChildTrackAreaModels())
	{
		if (const TViewModelPtr<FLayerBarModel> BarModel = TrackAreaModel.ImplicitCast())
		{
			return BarModel;
		}
	}
	for (const TViewModelPtr<ILayerBarExtension>& TrackAreaModel : InTrackAreaExtension->GetTrackAreaModelListAs<ILayerBarExtension>())
	{
		if (const TViewModelPtr<ILayerBarExtension> BarModel = TrackAreaModel.ImplicitCast())
		{
			return BarModel;
		}
	}
	return FAvaStaggerBarElement();
}

FAvaStaggerBarElement::FAvaStaggerBarElement(const TViewModelPtr<FLayerBarModel>& InBarModel)
{
	if (const TViewModelPtr<IOutlinerExtension> LinkedOutlinerItem = InBarModel->GetLinkedOutlinerItem())
	{
		BarModel.Emplace<TViewModelPtr<FLayerBarModel>>(InBarModel);
		OutlinerItem = LinkedOutlinerItem;
		Range = InBarModel->ComputeRange();
		OriginalFrame = Range.GetLowerBoundValue();
	}
}

FAvaStaggerBarElement::FAvaStaggerBarElement(const TViewModelPtr<ILayerBarExtension>& InBarModel)
{
	if (const TViewModelPtr<FLinkedOutlinerExtension> LinkedOutlinerExtension = InBarModel.ImplicitCast())
	{
		BarModel.Emplace<TViewModelPtr<ILayerBarExtension>>(InBarModel);
		OutlinerItem = LinkedOutlinerExtension->GetLinkedOutlinerItem();
		Range = InBarModel->GetLayerBarRange();
		OriginalFrame = Range.GetLowerBoundValue();
	}
}

bool FAvaStaggerBarElement::IsValid() const
{
	if (const TViewModelPtr<FLayerBarModel>* LayerBarModel = BarModel.TryGet<TViewModelPtr<FLayerBarModel>>())
	{
		return LayerBarModel->IsValid();
	}
	if (const TViewModelPtr<ILayerBarExtension>* LayerBarExtension = BarModel.TryGet<TViewModelPtr<ILayerBarExtension>>())
	{
		return LayerBarExtension->IsValid();
	}
	return false;
}

void FAvaStaggerBarElement::Offset(const FFrameNumber InDelta)
{
	if (const TViewModelPtr<FLayerBarModel>* LayerBarModel = BarModel.TryGet<TViewModelPtr<FLayerBarModel>>())
	{
		(*LayerBarModel)->Offset(InDelta);
	}
	else if (const TViewModelPtr<ILayerBarExtension>* LayerBarExtension = BarModel.TryGet<TViewModelPtr<ILayerBarExtension>>())
	{
		(*LayerBarExtension)->OffsetLayerBar(InDelta);
	}
}
