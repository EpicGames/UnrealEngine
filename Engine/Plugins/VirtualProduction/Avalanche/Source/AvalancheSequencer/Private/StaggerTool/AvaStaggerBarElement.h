// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Range.h"
#include "MVVM/ViewModelPtr.h"
#include "MVVM/ViewModels/LayerBarModel.h"

struct FFrameNumber;

namespace UE::Sequencer
{
	class FLayerBarModel;
	class ILayerBarExtension;
	class IOutlinerExtension;
	class ITrackAreaExtension;
}

/** Structure that holds cached data for a stagger element operation */
struct FAvaStaggerBarElement
{
	static FAvaStaggerBarElement FromTrack(const UE::Sequencer::TViewModelPtr<UE::Sequencer::ITrackAreaExtension>& InTrackAreaExtension);

	FAvaStaggerBarElement() {}
	FAvaStaggerBarElement(const UE::Sequencer::TViewModelPtr<UE::Sequencer::FLayerBarModel>& InLayerBar);
	FAvaStaggerBarElement(const UE::Sequencer::TViewModelPtr<UE::Sequencer::ILayerBarExtension>& InSection);

	bool operator==(const FAvaStaggerBarElement& InOther)
	{
		if (BarModel.IsType<UE::Sequencer::TViewModelPtr<UE::Sequencer::FLayerBarModel>>())
		{
			return InOther.BarModel.IsType<UE::Sequencer::TViewModelPtr<UE::Sequencer::FLayerBarModel>>()
				&& InOther.BarModel.Get<UE::Sequencer::TViewModelPtr<UE::Sequencer::FLayerBarModel>>()
					== BarModel.Get<UE::Sequencer::TViewModelPtr<UE::Sequencer::FLayerBarModel>>();
		}
		if (BarModel.IsType<UE::Sequencer::TViewModelPtr<UE::Sequencer::ILayerBarExtension>>())
		{
			return InOther.BarModel.IsType<UE::Sequencer::TViewModelPtr<UE::Sequencer::ILayerBarExtension>>()
				&& InOther.BarModel.Get<UE::Sequencer::TViewModelPtr<UE::Sequencer::ILayerBarExtension>>()
					== BarModel.Get<UE::Sequencer::TViewModelPtr<UE::Sequencer::ILayerBarExtension>>();
		}
		return false;
	}

	bool IsValid() const;

	void Offset(const FFrameNumber InDelta);

	TVariant<UE::Sequencer::TViewModelPtr<UE::Sequencer::FLayerBarModel>
		, UE::Sequencer::TViewModelPtr<UE::Sequencer::ILayerBarExtension>> BarModel;

	UE::Sequencer::TViewModelPtr<UE::Sequencer::IOutlinerExtension> OutlinerItem;
	TRange<FFrameNumber> Range;

	FFrameNumber OriginalFrame;
	FFrameNumber LastOffset;
};
