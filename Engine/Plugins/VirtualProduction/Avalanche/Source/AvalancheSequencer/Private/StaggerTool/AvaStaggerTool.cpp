// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaStaggerTool.h"
#include "AvaStaggerBarElement.h"
#include "Channels/MovieSceneChannel.h"
#include "Commands/AvaSequencerCommands.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "IKeyArea.h"
#include "Kismet/KismetMathLibrary.h"
#include "Misc/FrameTime.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/LayerBarModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "ScopedTransaction.h"
#include "Tools/SequencerSelectionAlignmentUtils.h"
#include "Widgets/SAvaStaggerTool.h"

using namespace UE::Sequencer;

#define LOCTEXT_NAMESPACE "AvaSequencerStagger"

FAvaStaggerTool::FAvaStaggerKeyElement::FAvaStaggerKeyElement(TViewModelPtr<FChannelModel> InKeyChannelModel
	, FKeyHandle InKeyHandle, FFrameNumber InOriginalFrame)
	: KeyChannelModel(MoveTemp(InKeyChannelModel))
	, KeyHandle(MoveTemp(InKeyHandle))
	, OriginalFrame(MoveTemp(InOriginalFrame))
{
}

FAvaStaggerTool::FAvaStaggerTool(FAvaSequencer& InOwner)
	: FAvaSequencerAction(InOwner)
{
	Settings.Reset(NewObject<UAvaSequencerStaggerSettings>(GetTransientPackage(), NAME_None, RF_Transient | RF_Transactional));

	OnResetToDefaults();
}

FAvaStaggerTool::~FAvaStaggerTool()
{
	CloseToolWindow();
}

TObjectPtr<UAvaSequencerStaggerSettings> FAvaStaggerTool::GetSettings() const
{
	return Settings.Get();
}

void FAvaStaggerTool::MapAction(const TSharedRef<FUICommandList>& InCommandList)
{
	InCommandList->MapAction(FAvaSequencerCommands::Get().OpenStaggerTool
		, FExecuteAction::CreateSP(this, &FAvaStaggerTool::Execute)
		, FCanExecuteAction::CreateSP(this, &FAvaStaggerTool::CanExecute));
}

void FAvaStaggerTool::OnSequencerClosed()
{
	CloseToolWindow();
}

bool FAvaStaggerTool::CanExecute() const
{
	return true;
}

void FAvaStaggerTool::Execute()
{
	if (ToolWindow.IsValid())
	{
		ToolWindow->BringToFront();
		return;
	}

	CacheOriginalElements();

	ToolWindow =
		SNew(SWindow)
		.Title(LOCTEXT("StaggerToolDialogTitle", "Stagger Tool"))
		.SizingRule(ESizingRule::Autosized)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.Content()
		[
			SAssignNew(ToolWidget, SAvaStaggerTool, SharedThis(this))
			.OnResetToDefaults(this, &FAvaStaggerTool::OnResetToDefaults)
			.OnSettingChange(this, &FAvaStaggerTool::OnSettingChange)
			.OnApply(this, &FAvaStaggerTool::OnApply)
		];

	ToolWindow->SetOnWindowClosed(FOnWindowClosed::CreateLambda([this](const TSharedRef<SWindow>&)
		{
			ToolWindow.Reset();
			ToolWindow = nullptr;
		}));

	const TSharedPtr<ISequencer> Sequencer = Owner.GetSequencerPtr();

	const TSharedPtr<SWindow> ParentWindow = Sequencer.IsValid() ? FSlateApplication::Get().FindBestParentWindowForDialogs(Sequencer->GetSequencerWidget()) : nullptr;
	if (ParentWindow.IsValid())
	{
		FSlateApplication::Get().AddWindowAsNativeChild(ToolWindow.ToSharedRef(), ParentWindow.ToSharedRef());
	}
	else
	{
		FSlateApplication::Get().AddWindow(ToolWindow.ToSharedRef());
	}

	BindToSequencerSelectionChange();
}

void FAvaStaggerTool::BindToSequencerSelectionChange()
{
	TSharedPtr<ISequencer> Sequencer = Owner.GetSequencerPtr();
	if (!Sequencer.IsValid())
	{
		return;
	}

	if (const TSharedPtr<FSequencerEditorViewModel> ViewModel = Sequencer->GetViewModel())
	{
		if (const TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection())
		{
			Selection->OnChanged.AddSP(this, &FAvaStaggerTool::OnSequencerSelectionChanged);
		}
	}
}

void FAvaStaggerTool::UnbindToSequencerSelectionChange()
{
	TSharedPtr<ISequencer> Sequencer = Owner.GetSequencerPtr();
	if (!Sequencer.IsValid())
	{
		return;
	}

	if (const TSharedPtr<FSequencerEditorViewModel> ViewModel = Sequencer->GetViewModel())
	{
		if (const TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection())
		{
			Selection->OnChanged.RemoveAll(this);
		}
	}
}

void FAvaStaggerTool::OnResetToDefaults()
{
	Settings->ResetToolOptions();
}

void FAvaStaggerTool::OnSettingChange(const FName InPropertyName)
{
	if (InPropertyName == GET_MEMBER_NAME_CHECKED(UAvaSequencerStaggerSettings, bAutoApply)
		&& !Settings->bAutoApply)
	{
		return;
	}

	if (Settings->bAutoApply)
	{
		OnApply();
	}
}

void FAvaStaggerTool::OnApply()
{
	Stagger();
}

void FAvaStaggerTool::OnSequencerSelectionChanged()
{
	CacheOriginalElements();

	if (!HasValidSelection())
	{
		// Turn off auto apply if we've lost a valid selection
		Settings->bAutoApply = false;
	}
	else if (Settings->bAutoApply)
	{
		OnApply();
	}
}

void FAvaStaggerTool::CloseToolWindow()
{
	if (ToolWindow.IsValid())
	{
		ToolWindow->RequestDestroyWindow();
		ToolWindow.Reset();
		ToolWindow = nullptr;
	}
}

bool FAvaStaggerTool::HasValidSelection() const
{
	return IsBarSelection() || IsKeySelection();
}

bool FAvaStaggerTool::IsBarSelection() const
{
	return OriginalBarElements.Num() > 1 && OriginalKeyElements.Num() == 0;
}

bool FAvaStaggerTool::IsKeySelection() const
{
	return OriginalKeyElements.Num() > 1 && OriginalBarElements.Num() == 0;
}

int32 FAvaStaggerTool::GetSelectionCount() const
{
	if (IsBarSelection())
	{
		return OriginalBarElements.Num();
	}

	if (IsKeySelection())
	{
		return OriginalKeyElements.Num();
	}

	return 0;
}

FFrameNumber FAvaStaggerTool::ConvertTime(FFrameTime InSourceTime) const
{
	TSharedRef<ISequencer> Sequencer = Owner.GetOrCreateSequencer();
	const FFrameRate FocusedDisplayRate = Sequencer->GetFocusedDisplayRate();
	const FFrameRate FocusedTickResolution = Sequencer->GetFocusedTickResolution();
	return ConvertFrameTime(InSourceTime, FocusedDisplayRate, FocusedTickResolution).RoundToFrame();
}

TRange<FFrameNumber> FAvaStaggerTool::GetOperationRange(const EAvaSequencerStaggerRange InRange) const
{
	TSharedRef<ISequencer> Sequencer = Owner.GetOrCreateSequencer();

	switch (InRange)
	{
	default:
	case EAvaSequencerStaggerRange::Playback:
		return Sequencer->GetPlaybackRange();

	case EAvaSequencerStaggerRange::Selection:
		return Sequencer->GetSelectionRange();

	case EAvaSequencerStaggerRange::Custom:
		const FFrameNumber MaxRange = ConvertTime(Settings->ToolOptions.CustomRange);
		return TRange<FFrameNumber>(0, MaxRange);
	}
}

FFrameNumber FAvaStaggerTool::GetInterval(const FFrameNumber InRangeSize, const int32 InElementCount, const FFrameNumber InElementSize) const
{
	switch (Settings->ToolOptions.Distribution)
	{
	default:
	case EAvaSequencerStaggerDistribution::Increment:
		{
			return Settings->ToolOptions.bUseCurve && InElementSize.Value != 0 ? InElementSize : CachedInterval;
		}

	case EAvaSequencerStaggerDistribution::Range:
		return InRangeSize.Value / InElementCount;

	case EAvaSequencerStaggerDistribution::Random:
		const int32 AdjustedMax = FMath::Clamp(InRangeSize.Value - InElementSize.Value, 0, InRangeSize.Value);
		return RandomStream.RandRange(0, AdjustedMax);
	}
}

FFrameNumber FAvaStaggerTool::CalculateBarElementFrameSpan(const TArray<FAvaStaggerBarElement>& InElements, const int32 InStopIndex)
{
	FFrameNumber OutFrame;
	for (int32 Index = 0; Index < InElements.Num(); ++Index)
	{
		if (Index == InStopIndex)
		{
			break;
		}
		OutFrame += InElements[Index].Range.Size<FFrameNumber>();
	}
	return MoveTemp(OutFrame);
}

FFrameNumber FAvaStaggerTool::GetBarElementOperationOffset(const FAvaStaggerBarElement& InElement) const
{
	const FFrameNumber RangeSize = InElement.Range.Size<FFrameNumber>();
	const float Position = Settings->ToolOptions.OperationPoint * RangeSize.Value;
	return FMath::FloorToInt32(Position);
}

void FAvaStaggerTool::CacheOriginalElements()
{
	const TSharedPtr<FSequencerSelection> SequencerSelection = GetSequencerSelection();
	if (!SequencerSelection.IsValid())
	{
		return;
	}

	const int32 OutlinerCount = SequencerSelection->Outliner.Num();
	const int32 TrackAreaCount = SequencerSelection->TrackArea.Num();
	const int32 KeyFrameCount = SequencerSelection->KeySelection.Num();

	OriginalBarElements.Empty();
	OriginalKeyElements.Empty();

	if ((OutlinerCount > 1 && TrackAreaCount == 0 && KeyFrameCount == 0)
		|| (OutlinerCount <= 1 && TrackAreaCount > 1 && KeyFrameCount == 0))
	{
		OriginalBarElements = GatherSelectionBarElements(SequencerSelection);
	}
	else if (OutlinerCount == 0 && TrackAreaCount == 0 && KeyFrameCount > 1)
	{
		OriginalKeyElements = GatherSelectionKeyElements();
	}

	CachedBarCount = OriginalBarElements.Num();
	CachedKeyCount = OriginalKeyElements.Num();
}

FFrameNumber FAvaStaggerTool::FindFirstBarStaggerPoint() const
{
	FFrameNumber OutFrameNumber;

	switch (Settings->ToolOptions.Distribution)
	{
	case EAvaSequencerStaggerDistribution::Increment:
		break;

	case EAvaSequencerStaggerDistribution::Range:
		switch (Settings->ToolOptions.Range)
		{
		case EAvaSequencerStaggerRange::Playback:
			OutFrameNumber = Owner.GetOrCreateSequencer()->GetPlaybackRange().GetLowerBoundValue();
			OutFrameNumber += CachedShiftFrames;
			return MoveTemp(OutFrameNumber);

		case EAvaSequencerStaggerRange::Selection:
			OutFrameNumber = Owner.GetOrCreateSequencer()->GetSelectionRange().GetLowerBoundValue();
			OutFrameNumber += CachedShiftFrames;
			return MoveTemp(OutFrameNumber);

		case EAvaSequencerStaggerRange::Custom:
			break;
		}
		break;

	case EAvaSequencerStaggerDistribution::Random:
		const FFrameNumber ElementSize = OriginalBarElements[0].Range.Size<FFrameNumber>();
		const FFrameNumber Interval = GetInterval(CachedRangeSize, CachedBarCount, ElementSize);
		return CachedRange.GetLowerBoundValue() + Interval + CachedShiftFrames;
	}

	return GetStartPosition(OriginalBarElements);
}

FFrameNumber FAvaStaggerTool::FindFirstKeyStaggerPoint() const
{
	if (Settings->ToolOptions.Distribution == EAvaSequencerStaggerDistribution::Random)
	{
		const FFrameNumber Interval = GetInterval(CachedRangeSize, CachedKeyCount);
		return CachedRange.GetLowerBoundValue() + Interval + CachedShiftFrames;
	}

	return GetStartPosition(OriginalBarElements);
}

TArray<FAvaStaggerBarElement> FAvaStaggerTool::GatherSelectionBarElements(const TSharedPtr<FSequencerSelection>& InSequencerSelection) const
{
	if (!InSequencerSelection.IsValid())
	{
		return {};
	}

	TArray<FAvaStaggerBarElement> OutBarElements;

	const int32 OutlinerCount = InSequencerSelection->Outliner.Num();
	const int32 TrackAreaCount = InSequencerSelection->TrackArea.Num();

	if (OutlinerCount > 1)
	{
		OutBarElements.Reserve(OutlinerCount);

		for (const FViewModelPtr& ViewModel : InSequencerSelection->Outliner)
		{
			const TViewModelPtr<ITrackAreaExtension> TrackArea = ViewModel.ImplicitCast();
			if (!TrackArea.IsValid())
			{
				continue;
			}

			FAvaStaggerBarElement NewElement = FAvaStaggerBarElement::FromTrack(TrackArea);
			if (NewElement.IsValid())
			{
				OutBarElements.Add(NewElement);
			}
		}
	}
	else if (OutlinerCount <= 1 && TrackAreaCount > 0)
	{
		OutBarElements.Reserve(TrackAreaCount);

		for (const FViewModelPtr& ViewModel : InSequencerSelection->TrackArea)
		{
			if (const TViewModelPtr<FLayerBarModel> LayerBarModel = ViewModel.ImplicitCast())
			{
				OutBarElements.Add(LayerBarModel);
			}
			else if (const TViewModelPtr<ILayerBarExtension> LayerBarExtension = ViewModel.ImplicitCast())
			{
				OutBarElements.Add(LayerBarExtension);
			}
		}
	}

	// Disallow operations on all layer bars that have a descendant that is selected
	for (const FAvaStaggerBarElement& BarElement : OutBarElements)
	{
		if (!BarElement.OutlinerItem.IsValid())
		{
			continue;
		}

		for (const TViewModelPtr<IOutlinerExtension>& ChildOutlinerExtension : BarElement.OutlinerItem.AsModel()->GetDescendantsOfType<IOutlinerExtension>())
		{
			if (InSequencerSelection->Outliner.IsSelected(ChildOutlinerExtension))
			{
				return {};
			}
		}
	}

	return OutBarElements;
}

TArray<FAvaStaggerTool::FAvaStaggerKeyElement> FAvaStaggerTool::GatherSelectionKeyElements() const
{
	const TSharedPtr<FSequencerSelection> SequencerSelection = GetSequencerSelection();
	if (!SequencerSelection.IsValid())
	{
		return {};
	}

	TArray<FAvaStaggerKeyElement> OutElements;

	// Gather all the selected keyframe handles
	OutElements.Reserve(SequencerSelection->KeySelection.Num());

	for (const FKeyHandle KeyHandle : SequencerSelection->KeySelection)
	{
		if (KeyHandle != FKeyHandle::Invalid())
		{
			if (const TViewModelPtr<FChannelModel> KeyModel = SequencerSelection->KeySelection.GetModelForKey(KeyHandle))
			{
				const UMovieSceneSection* const KeySection = KeyModel->GetSection();
				if (KeySection && !KeySection->IsReadOnly())
				{
					FFrameNumber KeyFrameNumber;
					KeyModel->GetChannel()->GetKeyTime(KeyHandle, KeyFrameNumber);

					OutElements.Add(FAvaStaggerKeyElement(KeyModel, KeyHandle, KeyFrameNumber));
				}
			}
		}
	}

	return MoveTemp(OutElements);
}

FFrameNumber FAvaStaggerTool::CalculateLocalCurveOffset(const float InCurveTime, const TRange<FFrameNumber>& InRange) const
{
	const FRichCurve* const RichCurve = Settings->ToolOptions.Curve.GetRichCurve();
	if (!RichCurve)
	{
		return FFrameNumber();
	}

	float MinTime = 0.f;
	float MaxTime = 0.f;
	RichCurve->GetTimeRange(MinTime, MaxTime);

	float MinValue = 0.f;
	float MaxValue = 0.f;
	RichCurve->GetValueRange(MinValue, MaxValue);

	const FFrameNumber RangeSize = InRange.Size<FFrameNumber>();

	float AdjustedCurveTime = InCurveTime + Settings->ToolOptions.CurveOffset;
	if (AdjustedCurveTime > 1.f)
	{
		AdjustedCurveTime -= 1.f;
	}
	else if (AdjustedCurveTime < -1.f)
	{
		AdjustedCurveTime += 1.f;
	}

	// Map the 0 - 1 curve time to the min/max of the of the range
	// InCurveTime should be between 0 and 1
	// MinTime should typically be 0, but can be anything the user decides
	// MaxTime should typically be 1, but can be anything the user decides
	const float LocalMappedCurveTime = UKismetMathLibrary::MapRangeClamped(AdjustedCurveTime
		, 0.f, 1.f
		, MinTime, MaxTime);

	const float EvalValue = RichCurve->Eval(LocalMappedCurveTime);

	// Normalize the value and map to range since the value could be anything set by the user
	// (at least until we have a custom graph widget like the Ease Curve Tool to handle this)
	const float LocalFrameOffset = UKismetMathLibrary::MapRangeClamped(EvalValue
		, MinValue, MaxValue
		, 0.f, RangeSize.Value);

	return FMath::RoundToInt32(LocalFrameOffset);
}

FFrameNumber FAvaStaggerTool::FindNextStaggerLocation(const FAvaStaggerBarElement& InElement
	, const int32 InElementIndex
	, const FFrameNumber InFirstFrame
	, const FFrameNumber InCurrentFrame) const
{
	const FRichCurve* const RichCurve = Settings->ToolOptions.Curve.GetRichCurve();
	const FFrameNumber ElementSize = InElement.Range.Size<FFrameNumber>();
	const FFrameNumber Interval = GetInterval(CachedRangeSize, CachedBarCount, ElementSize);

	if (Settings->ToolOptions.bUseCurve)
	{
		if (!RichCurve)
		{
			return InFirstFrame;
		}

		const FFrameNumber FullFrameSpan = CalculateBarElementFrameSpan(OriginalBarElements);
		const FFrameNumber ElementFrameSpan = CalculateBarElementFrameSpan(OriginalBarElements, InElementIndex + 1);
		const float ElementStartPercentOfRange = static_cast<float>(ElementFrameSpan.Value) / static_cast<float>(FullFrameSpan.Value);

		if (Settings->ToolOptions.Distribution == EAvaSequencerStaggerDistribution::Increment)
		{
			const FFrameNumber LocalCurveOffset = CalculateLocalCurveOffset(ElementStartPercentOfRange, TRange<FFrameNumber>(0, FullFrameSpan));
			return InFirstFrame + LocalCurveOffset + Interval;
		}

		const float CurveStepTime = 1.f / CachedBarCount;
		const float CurveTime = CurveStepTime * InElementIndex;

		const FFrameNumber LocalCurveOffset = CalculateLocalCurveOffset(CurveTime, CachedRange);
		return InFirstFrame + LocalCurveOffset + Interval;
	}

	if (Settings->ToolOptions.Distribution == EAvaSequencerStaggerDistribution::Random)
	{
		if (OriginalBarElements.IsValidIndex(InElementIndex + 1))
		{
			const FAvaStaggerBarElement& NextElement = OriginalBarElements[InElementIndex + 1];
			const FFrameNumber NextElementSize = NextElement.Range.Size<FFrameNumber>();
			const FFrameNumber RandomInterval = GetInterval(CachedRangeSize, CachedBarCount, NextElementSize);
			return CachedRange.GetLowerBoundValue() + RandomInterval + CachedShiftFrames;
		}
		return CachedRange.GetLowerBoundValue() + CachedShiftFrames;
	}

	const FFrameNumber OperationOffset = GetBarElementOperationOffset(InElement);
	return InCurrentFrame + OperationOffset + Interval;
}

FFrameNumber FAvaStaggerTool::FindNextStaggerLocation(const FAvaStaggerKeyElement& InElement
	, const int32 InElementIndex
	, const FFrameNumber InFirstFrame
	, const FFrameNumber InCurrentFrame) const
{
	const FRichCurve* const RichCurve = Settings->ToolOptions.Curve.GetRichCurve();
	const FFrameNumber Interval = GetInterval(CachedRangeSize, CachedKeyCount);

	if (Settings->ToolOptions.bUseCurve)
	{
		if (!RichCurve)
		{
			return InFirstFrame;
		}

		const float CurveStepTime = 1.f / CachedKeyCount;
		const FFrameNumber NextStaggerFrameNumber = CalculateLocalCurveOffset(CurveStepTime * InElementIndex, CachedRange);

		return InFirstFrame + NextStaggerFrameNumber + Interval;
	}

	if (Settings->ToolOptions.Distribution == EAvaSequencerStaggerDistribution::Random)
	{
		return CachedRange.GetLowerBoundValue() + CachedShiftFrames;
	}

	return InCurrentFrame + Interval;
}

void FAvaStaggerTool::Stagger()
{
	CachedRange = GetOperationRange(Settings->ToolOptions.Range);
	CachedRangeSize = CachedRange.Size<FFrameNumber>();
	CachedInterval = ConvertTime(Settings->ToolOptions.Interval);
	CachedShiftFrames = ConvertTime(Settings->ToolOptions.Shift);

	if (Settings->ToolOptions.Distribution == EAvaSequencerStaggerDistribution::Random)
	{
		RandomStream = FRandomStream(Settings->ToolOptions.RandomSeed);
	}

	if (IsBarSelection())
	{
		// Reset layer bar offsets that were last applied
		for (FAvaStaggerBarElement& Element : OriginalBarElements)
		{
			Element.Offset(-Element.LastOffset);
			Element.LastOffset = 0;
		}

		StaggerBarElements();
	}
	else if (IsKeySelection())
	{
		// Reset key frames to their last location
		for (FAvaStaggerKeyElement& Element : OriginalKeyElements)
		{
			SetKeyElementTime(Element, Element.OriginalFrame);
		}

		StaggerKeyElements();
	}
}

bool FAvaStaggerTool::IsAutoApplying() const
{
	return Settings->bAutoApply;
}

void FAvaStaggerTool::StaggerBarElements()
{
	if (CachedBarCount <= 1)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("StaggerLayers", "Stagger Layers"));

	const FFrameNumber FirstStaggerPoint = FindFirstBarStaggerPoint();

	FFrameNumber NextStaggerPoint = FirstStaggerPoint;
	int32 CurrentElementNum = 0;
	int32 CurrentGroupNum = 0;

	for (FAvaStaggerBarElement& Element : OriginalBarElements)
	{
		Element.LastOffset = NextStaggerPoint - Element.OriginalFrame;
		Element.Offset(Element.LastOffset);

		++CurrentGroupNum;

		// Advance stagger point if the count has been reached for the current group
		if (CurrentGroupNum == Settings->ToolOptions.Grouping)
		{
			NextStaggerPoint = FindNextStaggerLocation(Element, CurrentElementNum, FirstStaggerPoint, NextStaggerPoint);

			CurrentGroupNum = 0;
		}

		++CurrentElementNum;
	}
}

void FAvaStaggerTool::StaggerKeyElements()
{
	if (CachedKeyCount <= 1)
	{
		return;
	}

	// Stagger all elements
	FScopedTransaction Transaction(LOCTEXT("StaggerKeyFrames", "Stagger Key Frames"));

	const FFrameNumber FirstStaggerPoint = FindFirstKeyStaggerPoint();

	FFrameNumber NextStaggerPoint = FirstStaggerPoint;
	int32 CurrentElementNum = 0;
	int32 CurrentGroupNum = 0;

	for (FAvaStaggerKeyElement& Element : OriginalKeyElements)
	{
		SetKeyElementTime(Element, NextStaggerPoint);

		++CurrentElementNum;
		++CurrentGroupNum;

		// Advanced stagger point if the count has been reached for the current group
		if (CurrentGroupNum == Settings->ToolOptions.Grouping)
		{
			NextStaggerPoint = FindNextStaggerLocation(Element, CurrentElementNum, FirstStaggerPoint, NextStaggerPoint);

			CurrentGroupNum = 0;
		}
	}
}

bool FAvaStaggerTool::CanAlignToPlayhead() const
{
	TSharedPtr<ISequencer> Sequencer = Owner.GetSequencerPtr();
	return Sequencer.IsValid() ? FSequencerSelectionAlignmentUtils::CanAlignSelection(*Sequencer) : false;
}

void FAvaStaggerTool::AlignToPlayhead()
{
	if (TSharedPtr<ISequencer> Sequencer = Owner.GetSequencerPtr())
	{
		FSequencerSelectionAlignmentUtils::AlignSelectionToPlayhead(*Sequencer);	
	}
}

void FAvaStaggerTool::SetKeyElementTime(const FAvaStaggerKeyElement& InElement, const FFrameNumber InKeyTime)
{
	if (!InElement.KeyChannelModel.IsValid())
	{
		return;
	}

	UMovieSceneSection* const Section = InElement.KeyChannelModel->GetSection();
	if (!IsValid(Section) || !Section->IsReadOnly())
	{
		return;
	}

	if (Section->TryModify())
	{
		InElement.KeyChannelModel->GetKeyArea()->SetKeyTime(InElement.KeyHandle, InKeyTime);
		Section->ExpandToFrame(InElement.OriginalFrame);
	}
}

#undef LOCTEXT_NAMESPACE
