// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequencer.h"
#include "AvaStaggerToolSettings.h"
#include "Commands/AvaSequencerAction.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "ISequencer.h"
#include "MVVM/ViewModelPtr.h"
#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "SequencerCoreFwd.h"

class SAvaStaggerTool;
class SWindow;
class UAvaStaggerCurve;
struct FAvaStaggerBarElement;

namespace UE::Sequencer
{
	class FChannelModel;
}

class FAvaStaggerTool : public FAvaSequencerAction
{
public:
	explicit FAvaStaggerTool(FAvaSequencer& InOwner);
	virtual ~FAvaStaggerTool() override;

	TObjectPtr<UAvaSequencerStaggerSettings> GetSettings() const;

	bool HasValidSelection() const;

	bool IsBarSelection() const;
	bool IsKeySelection() const;

	int32 GetSelectionCount() const;

	void Stagger();

	bool IsAutoApplying() const;

	void OnSequencerSelectionChanged();

	bool CanAlignToPlayhead() const;
	void AlignToPlayhead();

	void CloseToolWindow();

private:
	struct FAvaStaggerKeyElement
	{
		explicit FAvaStaggerKeyElement(UE::Sequencer::TViewModelPtr<UE::Sequencer::FChannelModel> InKeyChannelModel
			, FKeyHandle InKeyHandle, FFrameNumber InOriginalFrame = 0);

		UE::Sequencer::TViewModelPtr<UE::Sequencer::FChannelModel> KeyChannelModel;
		FKeyHandle KeyHandle;
		FFrameNumber OriginalFrame;
	};

	//~ Begin FAvaSequencerAction
	virtual void MapAction(const TSharedRef<FUICommandList>& InCommandList) override;
	virtual void OnSequencerClosed() override;
	//~ End FAvaSequencerAction

	bool CanExecute() const;
	void Execute();

	void BindToSequencerSelectionChange();
	void UnbindToSequencerSelectionChange();

	void OnResetToDefaults();
	void OnSettingChange(const FName InPropertyName);
	void OnApply();

	FFrameNumber ConvertTime(FFrameTime InSourceTime) const;

	template<typename InElementType>
	FFrameNumber GetStartPosition(const TArray<InElementType>& InElements) const
	{
		FFrameNumber OutFrameNumber;

		switch (Settings->ToolOptions.StartPosition)
		{
		case EAvaSequencerStaggerStartPosition::FirstSelected:
			if (!InElements.IsEmpty())
			{
				OutFrameNumber = InElements[0].OriginalFrame;
			}
			break;

		case EAvaSequencerStaggerStartPosition::FirstInTimeline:
			FindFirstFrameInTimeline(InElements, OutFrameNumber);
			break;

		case EAvaSequencerStaggerStartPosition::Playhead:
			OutFrameNumber = Owner.GetOrCreateSequencer()->GetGlobalTime().Time.FrameNumber;
			break;

		case EAvaSequencerStaggerStartPosition::PlaybackRange:
			OutFrameNumber = Owner.GetOrCreateSequencer()->GetPlaybackRange().GetLowerBoundValue();
			break;

		case EAvaSequencerStaggerStartPosition::SelectionRange:
			OutFrameNumber = Owner.GetOrCreateSequencer()->GetSelectionRange().GetLowerBoundValue();
			break;
		}

		OutFrameNumber += ConvertTime(Settings->ToolOptions.Shift);

		return OutFrameNumber;
	}

	TRange<FFrameNumber> GetOperationRange(const EAvaSequencerStaggerRange InRange) const;

	FFrameNumber GetInterval(const FFrameNumber InRangeSize, const int32 InElementCount, const FFrameNumber InElementSize = 0) const;

	FFrameNumber GetBarElementOperationOffset(const FAvaStaggerBarElement& InElement) const;

	static FFrameNumber CalculateBarElementFrameSpan(const TArray<FAvaStaggerBarElement>& InElements, const int32 InStopIndex = INDEX_NONE);

	void CacheOriginalElements();

	FFrameNumber FindFirstBarStaggerPoint() const;
	FFrameNumber FindFirstKeyStaggerPoint() const;

	TArray<FAvaStaggerBarElement> GatherSelectionBarElements(const TSharedPtr<UE::Sequencer::FSequencerSelection>& InSequencerSelection) const;
	TArray<FAvaStaggerKeyElement> GatherSelectionKeyElements() const;

	FFrameNumber CalculateLocalCurveOffset(const float InCurveTime, const TRange<FFrameNumber>& InRange) const;

	FFrameNumber FindNextStaggerLocation(const FAvaStaggerBarElement& InElement
		, const int32 InElementIndex
		, const FFrameNumber InFirstFrame
		, const FFrameNumber InCurrentFrame) const;
	FFrameNumber FindNextStaggerLocation(const FAvaStaggerKeyElement& InElement
		, const int32 InElementIndex
		, const FFrameNumber InFirstFrame
		, const FFrameNumber InCurrentFrame) const;

	void StaggerBarElements();
    void StaggerKeyElements();

	static void SetKeyElementTime(const FAvaStaggerKeyElement& InElement, const FFrameNumber InKeyTime);

	template<typename InElementType>
	static void FindFirstFrameInTimeline(const TArray<InElementType>& InElements, FFrameNumber& OutFrame)
	{
		bool bFound = false;
		for (const InElementType& Element : InElements)
		{
			if (!bFound || Element.OriginalFrame < OutFrame)
			{
				OutFrame = Element.OriginalFrame;
				bFound = true;
			}
		}
	}

	TSharedPtr<SWindow> ToolWindow;

	TSharedPtr<SAvaStaggerTool> ToolWidget;

	TStrongObjectPtr<UAvaSequencerStaggerSettings> Settings;

	// Cached values at the start of auto-updating
	TArray<FAvaStaggerBarElement> OriginalBarElements;
	TArray<FAvaStaggerKeyElement> OriginalKeyElements;

	int32 CachedBarCount = 0;
	int32 CachedKeyCount = 0;

	TRange<FFrameNumber> CachedRange;
	FFrameNumber CachedRangeSize = 0;
	FFrameNumber CachedInterval = 0;
	FFrameNumber CachedShiftFrames = 0;

	FRandomStream RandomStream;
};
