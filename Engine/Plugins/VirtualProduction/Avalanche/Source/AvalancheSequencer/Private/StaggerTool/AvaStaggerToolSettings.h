// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/CurveFloat.h"
#include "Misc/FrameTime.h"
#include "AvaStaggerToolSettings.generated.h"

UENUM()
enum class EAvaSequencerStaggerStartPosition : uint8
{
	/** Sequence placement will begin at the position of the first selected */
	FirstSelected = 0,
	/** Sequence placement will begin at the earliest sequence of the selection */
	FirstInTimeline = 1,
	/** Sequence placement will begin at the current position of the playhead */
	Playhead = 2,

	/** Sequence placement will begin at the start of the playback range */
	PlaybackRange = 3,
	/** Sequence placement will begin at the start of the selection range */
	SelectionRange = 4,

	Default = FirstSelected UMETA(hidden)
};

UENUM()
enum class EAvaSequencerStaggerDistribution : uint8
{
	/** Sequences placed one after the other */
	Increment = 0,
	/** Sequences spaced out between a range */
	Range = 1,
	/** Sequences randomly distributed in range */
	Random = 255,

	Default = Increment UMETA(hidden)
};

UENUM()
enum class EAvaSequencerStaggerRange : uint8
{
	/** Sequences distributed between playback range */
	Playback = 0,
	/** Sequences distributed between selection range */
	Selection = 1,
	/** Sequences distributed between custom user specified frame range */
	Custom = 2,

	Default = Playback UMETA(hidden)
};

USTRUCT()
struct FAvaSequencerStaggerOptions
{
	GENERATED_BODY()

	bool operator==(const FAvaSequencerStaggerOptions& InOther) const 
	{
		return Distribution == InOther.Distribution
			&& RandomSeed == InOther.RandomSeed
			&& Range == InOther.Range
			&& CustomRange == InOther.CustomRange
			&& StartPosition == InOther.StartPosition
			&& OperationPoint == InOther.OperationPoint
			&& Interval == InOther.Interval
			&& Shift == InOther.Shift
			&& Grouping == InOther.Grouping
			&& bUseCurve == InOther.bUseCurve
			&& Curve.EditorCurveData == InOther.Curve.EditorCurveData
			&& Curve.ExternalCurve == InOther.Curve.ExternalCurve
			&& CurveOffset == InOther.CurveOffset;
	}

	bool operator!=(const FAvaSequencerStaggerOptions& InOther) const
	{
		return !(*this == InOther);
	}

	UPROPERTY(EditAnywhere, Category = "Stagger Tool Options", meta = (Tooltip = "How the layers or keyframes should be distributed after each step"))
	EAvaSequencerStaggerDistribution Distribution = EAvaSequencerStaggerDistribution::Default;

	UPROPERTY(EditAnywhere, Category = "Stagger Tool Options", meta = (ClampMin = "0", UIMin = "0", Tooltip = "Seed value for randomization"))
	int32 RandomSeed = 0;

	UPROPERTY(EditAnywhere, Category = "Stagger Tool Options", meta = (Tooltip = "Time range to perform the operation on"))
	EAvaSequencerStaggerRange Range = EAvaSequencerStaggerRange::Default;

	UPROPERTY(EditAnywhere, Category = "Stagger Tool Options", meta = (UIMin = "-60", UIMax = "60", Tooltip = "Custom time range to perform the operation with"))
	int32 CustomRange = 30;

	UPROPERTY(EditAnywhere, Category = "Stagger Tool Options", meta = (Tooltip = "Position to start the first layer or keyframe"))
	EAvaSequencerStaggerStartPosition StartPosition = EAvaSequencerStaggerStartPosition::Default;

	UPROPERTY(EditAnywhere, Category = "Stagger Tool Options", meta = (Tooltip = "Point on the current layer bar at which the next layer is positioned"))
	float OperationPoint = 0.f;

	UPROPERTY(EditAnywhere, Category = "Stagger Tool Options", meta = (UIMin = "-60", UIMax = "60", Tooltip = "Stepping interval between layers or keyframes"))
	int32 Interval = 0;

	UPROPERTY(EditAnywhere, Category = "Stagger Tool Options", meta = (UIMin = "-60", UIMax = "60", Tooltip = "Offset of operation"))
	int32 Shift = 0;

	UPROPERTY(EditAnywhere, Category = "Stagger Tool Options", meta = (ClampMin = "1", ClampMax = "255", UIMin = "1", UIMax = "10", Tooltip = "Number of layers or keyframes to group together before moving to the next stagger point"))
	int32 Grouping = 1;

	UPROPERTY(EditAnywhere, Category = "Stagger Tool Options", meta = (Tooltip = "Use a curve to layout layers or keyframes"))
	bool bUseCurve = false;

	UPROPERTY(EditAnywhere, Category = "Stagger Tool Options", meta = (Tooltip = "Curve to use for distribution between layers or keyframes"))
	FRuntimeFloatCurve Curve;

	UPROPERTY(EditAnywhere, Category = "Stagger Tool Options", meta = (ClampMin = "-1", ClampMax = "1", UIMin = "-1", UIMax = "1", Tooltip = "Offset of operation"))
	float CurveOffset = 0.f;
};

UCLASS()
class UAvaSequencerStaggerSettings : public UObject
{
	GENERATED_BODY()

public:
	UAvaSequencerStaggerSettings()
	{
		ToolOptions.RandomSeed = FMath::Rand();

		ResetToolOptions();
	}

	bool CanResetToolOptions() const
	{
		return ToolOptions != FAvaSequencerStaggerOptions();
	}

	void ResetToolOptions()
	{
		int32 PreviousRandomSeed = ToolOptions.RandomSeed;
		ToolOptions = FAvaSequencerStaggerOptions();
		ToolOptions.RandomSeed = PreviousRandomSeed;

		if (FRichCurve* const RichCurve = ToolOptions.Curve.GetRichCurve())
		{
			RichCurve->AddKey(0.f, 0.f);
			RichCurve->AddKey(1.f, 0.f);
		}
	}

	UPROPERTY()
	bool bAutoApply = false;

	UPROPERTY(EditAnywhere, Category = "Stagger Tool Options", meta = (ShowOnlyInnerProperties))
	FAvaSequencerStaggerOptions ToolOptions;
};
