// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaStaggerToolSettingsDetailsCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "StaggerTool/AvaStaggerTool.h"
#include "StaggerTool/Widgets/SAvaStaggerOperationPoint.h"
#include "StaggerTool/Widgets/SAvaStaggerSettingsRadioGroup.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSpinBox.h"

#define LOCTEXT_NAMESPACE "AvaStaggerToolSettingsDetailsCustomization"

TSharedRef<IDetailCustomization> FAvaStaggerToolSettingsDetailsCustomization::MakeInstance(const TWeakPtr<FAvaStaggerTool> InWeakTool)
{
	return MakeShared<FAvaStaggerToolSettingsDetailsCustomization>(InWeakTool);
}

FAvaStaggerToolSettingsDetailsCustomization::FAvaStaggerToolSettingsDetailsCustomization(const TWeakPtr<FAvaStaggerTool>& InWeakTool)
	: WeakTool(InWeakTool)
{
}

void FAvaStaggerToolSettingsDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	ToolOptionsProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAvaSequencerStaggerSettings, ToolOptions));

	UseCurveProperty = ToolOptionsProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSequencerStaggerOptions, bUseCurve));
	CurveProperty = ToolOptionsProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSequencerStaggerOptions, Curve));
	CurveOffsetProperty = ToolOptionsProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSequencerStaggerOptions, CurveOffset));
	DistributionProperty = ToolOptionsProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSequencerStaggerOptions, Distribution));
	RandomSeedProperty = ToolOptionsProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSequencerStaggerOptions, RandomSeed));
	RangeProperty = ToolOptionsProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSequencerStaggerOptions, Range));
	CustomRangeProperty = ToolOptionsProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSequencerStaggerOptions, CustomRange));
	StartPositionProperty = ToolOptionsProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSequencerStaggerOptions, StartPosition));
	OperationPointProperty = ToolOptionsProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSequencerStaggerOptions, OperationPoint));
	IntervalProperty = ToolOptionsProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSequencerStaggerOptions, Interval));
	ShiftProperty = ToolOptionsProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSequencerStaggerOptions, Shift));
	GroupingProperty = ToolOptionsProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSequencerStaggerOptions, Grouping));

	IDetailCategoryBuilder& OptionsCategory = DetailBuilder.EditCategory(TEXT("Stagger Tool Options"));

	IDetailPropertyRow& DistributionPropertyRow = OptionsCategory.AddProperty(DistributionProperty);
	AddDistributionRow(DistributionPropertyRow, DistributionProperty.ToSharedRef());

	IDetailPropertyRow& RandomSeedPropertyRow = OptionsCategory.AddProperty(RandomSeedProperty)
		.Visibility(TAttribute<EVisibility>::CreateSP(this, &FAvaStaggerToolSettingsDetailsCustomization::GetRandomSeedPropertyVisibility));
	AddRandomSeedRow(RandomSeedPropertyRow, RandomSeedProperty.ToSharedRef());

	IDetailPropertyRow& RangePropertyRow = OptionsCategory.AddProperty(RangeProperty)
		.Visibility(TAttribute<EVisibility>::CreateSP(this, &FAvaStaggerToolSettingsDetailsCustomization::GetRangePropertyVisibility));
	AddRangeRow(RangePropertyRow, RangeProperty.ToSharedRef());

	IDetailPropertyRow& CustomRangePropertyRow = OptionsCategory.AddProperty(CustomRangeProperty)
		.Visibility(TAttribute<EVisibility>::CreateSP(this, &FAvaStaggerToolSettingsDetailsCustomization::GetCustomRangePropertyVisibility));
	AddCustomIntSpinBoxRow(CustomRangePropertyRow, CustomRangeProperty.ToSharedRef());

	IDetailPropertyRow& StartPositionPropertyRow = OptionsCategory.AddProperty(StartPositionProperty)
		.Visibility(TAttribute<EVisibility>::CreateSP(this, &FAvaStaggerToolSettingsDetailsCustomization::GetStartPositionPropertyVisibility));
	AddStartPositionRow(StartPositionPropertyRow, StartPositionProperty.ToSharedRef());

	IDetailPropertyRow& OperationPointPropertyRow = OptionsCategory.AddProperty(OperationPointProperty)
		.Visibility(TAttribute<EVisibility>::CreateSP(this, &FAvaStaggerToolSettingsDetailsCustomization::GetOperationPointPropertyVisibility));
	AddOperationPointRow(OperationPointPropertyRow, OperationPointProperty.ToSharedRef());

	IDetailPropertyRow& IntervalPropertyRow = OptionsCategory.AddProperty(IntervalProperty)
		.Visibility(TAttribute<EVisibility>::CreateSP(this, &FAvaStaggerToolSettingsDetailsCustomization::GetIntervalPropertyVisibility));
	AddCustomIntSpinBoxRow(IntervalPropertyRow, IntervalProperty.ToSharedRef());

	IDetailPropertyRow& ShiftPropertyRow = OptionsCategory.AddProperty(ShiftProperty);
	AddCustomIntSpinBoxRow(ShiftPropertyRow, ShiftProperty.ToSharedRef());

	IDetailPropertyRow& GroupingPropertyRow = OptionsCategory.AddProperty(GroupingProperty);
	AddCustomIntSpinBoxRow(GroupingPropertyRow, GroupingProperty.ToSharedRef());

	OptionsCategory.AddProperty(UseCurveProperty);

	OptionsCategory.AddProperty(CurveProperty)
		.Visibility(TAttribute<EVisibility>::CreateSP(this, &FAvaStaggerToolSettingsDetailsCustomization::GetCurvePropertyVisibility));

	IDetailPropertyRow& CurveOffsetPropertyRow = OptionsCategory.AddProperty(CurveOffsetProperty)
		.Visibility(TAttribute<EVisibility>::CreateSP(this, &FAvaStaggerToolSettingsDetailsCustomization::GetCurveOffsetPropertyVisibility));
	AddCustomFloatSpinBoxRow(CurveOffsetPropertyRow, CurveOffsetProperty.ToSharedRef());
}

EAvaSequencerStaggerDistribution FAvaStaggerToolSettingsDetailsCustomization::GetDistributionPropertyValue() const
{
	uint8 Value = 0;
	DistributionProperty->GetValue(Value);
	return static_cast<EAvaSequencerStaggerDistribution>(Value);
}

EAvaSequencerStaggerRange FAvaStaggerToolSettingsDetailsCustomization::GetRangePropertyValue() const
{
	uint8 Value = 0;
	RangeProperty->GetValue(Value);
	return static_cast<EAvaSequencerStaggerRange>(Value);
}

EAvaSequencerStaggerStartPosition FAvaStaggerToolSettingsDetailsCustomization::GetStartPositionPropertyValue() const
{
	uint8 Value = 0;
	StartPositionProperty->GetValue(Value);
	return static_cast<EAvaSequencerStaggerStartPosition>(Value);
}

float FAvaStaggerToolSettingsDetailsCustomization::GetOperationPointPropertyValue() const
{
	float Value = 0;
	OperationPointProperty->GetValue(Value);
	return Value;
}

bool FAvaStaggerToolSettingsDetailsCustomization::GetUseCurvePropertyValue() const
{
	bool Value = false;
	UseCurveProperty->GetValue(Value);
	return Value;
}

EVisibility FAvaStaggerToolSettingsDetailsCustomization::GetCurvePropertyVisibility() const
{
	return GetUseCurvePropertyValue() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FAvaStaggerToolSettingsDetailsCustomization::GetCurveOffsetPropertyVisibility() const
{
	return GetUseCurvePropertyValue() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FAvaStaggerToolSettingsDetailsCustomization::GetRandomSeedPropertyVisibility() const
{
	const EAvaSequencerStaggerDistribution DistributionValue = GetDistributionPropertyValue();
	return (DistributionValue == EAvaSequencerStaggerDistribution::Random)
		? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FAvaStaggerToolSettingsDetailsCustomization::GetIntervalPropertyVisibility() const
{
	const EAvaSequencerStaggerDistribution DistributionValue = GetDistributionPropertyValue();
	return (DistributionValue != EAvaSequencerStaggerDistribution::Range && DistributionValue != EAvaSequencerStaggerDistribution::Random)
		? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FAvaStaggerToolSettingsDetailsCustomization::GetShiftPropertyVisibility() const
{
	const EAvaSequencerStaggerDistribution DistributionValue = GetDistributionPropertyValue();
	return (DistributionValue != EAvaSequencerStaggerDistribution::Range)
		? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FAvaStaggerToolSettingsDetailsCustomization::GetRangePropertyVisibility() const
{
	const EAvaSequencerStaggerDistribution DistributionValue = GetDistributionPropertyValue();
	return (GetUseCurvePropertyValue() || DistributionValue == EAvaSequencerStaggerDistribution::Range || DistributionValue == EAvaSequencerStaggerDistribution::Random)
		? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FAvaStaggerToolSettingsDetailsCustomization::GetCustomRangePropertyVisibility() const
{
	const EAvaSequencerStaggerRange RangeValue = GetRangePropertyValue();
	return (GetRangePropertyVisibility() == EVisibility::Visible && RangeValue == EAvaSequencerStaggerRange::Custom)
		? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FAvaStaggerToolSettingsDetailsCustomization::GetStartPositionPropertyVisibility() const
{
	const EAvaSequencerStaggerDistribution TypeValue = GetDistributionPropertyValue();
	return (TypeValue != EAvaSequencerStaggerDistribution::Range && TypeValue != EAvaSequencerStaggerDistribution::Random)
		? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FAvaStaggerToolSettingsDetailsCustomization::GetOperationPointPropertyVisibility() const
{
	// Keyframes have no range to use
	if (WeakTool.IsValid() && WeakTool.Pin()->IsKeySelection())
	{
		return EVisibility::Collapsed;
	}

	const EAvaSequencerStaggerDistribution DistributionValue = GetDistributionPropertyValue();
	if (DistributionValue == EAvaSequencerStaggerDistribution::Random)
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

template<typename InValueType>
TSharedRef<SWidget> CreateSpinBox(const TWeakPtr<FAvaStaggerTool>& InWeakTool, const TSharedRef<IPropertyHandle> InProperty)
{
	const TWeakPtr<IPropertyHandle> WeakProperty = InProperty;

	auto GetMetaDataValue = [&InProperty](const FName InKey) -> TOptional<InValueType>
	{
		if (std::is_integral_v<InValueType>)
		{
			return InProperty->HasMetaData(InKey) ? InProperty->GetIntMetaData(InKey) : TOptional<InValueType>();	
		}
		if (std::is_floating_point_v<InValueType>)
		{
			return InProperty->HasMetaData(InKey) ? InProperty->GetFloatMetaData(InKey) : TOptional<InValueType>();	
		}
		return TOptional<InValueType>();	
	};
	const TOptional<InValueType> ClampMinValue = GetMetaDataValue(TEXT("ClampMin"));
	const TOptional<InValueType> ClampMaxValue = GetMetaDataValue(TEXT("ClampMax"));
	const TOptional<InValueType> UIMinValue = GetMetaDataValue(TEXT("UIMin"));
	const TOptional<InValueType> UIMaxValue = GetMetaDataValue(TEXT("UIMax"));

	return SNew(SSpinBox<InValueType>)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.MinValue(ClampMinValue)
		.MaxValue(ClampMaxValue)
		.MinSliderValue(UIMinValue)
		.MaxSliderValue(UIMaxValue)
		.ShiftMultiplier(2.0f) // Default is 10.0f
		.CtrlMultiplier(0.05f) // Default is 0.1f
		.Value_Lambda([WeakProperty]()
			{
				InValueType OutValue = 0;
				if (const TSharedPtr<IPropertyHandle> Property = WeakProperty.Pin())
				{
					Property->GetValue(OutValue);
				}
				return OutValue;
			})
		.OnValueChanged_Lambda([WeakProperty](const InValueType InNewValue)
			{
				if (const TSharedPtr<IPropertyHandle> Property = WeakProperty.Pin())
				{
					Property->SetValue(InNewValue);
				}
			})
		.OnValueCommitted_Lambda([InWeakTool, WeakProperty](const InValueType InNewValue, const ETextCommit::Type InCommitType)
			{
				if (const TSharedPtr<IPropertyHandle> Property = WeakProperty.Pin())
				{
					Property->SetValue(InNewValue);
				}
			});
}

void FAvaStaggerToolSettingsDetailsCustomization::AddCustomIntSpinBoxRow(IDetailPropertyRow& PropertyRow, const TSharedRef<IPropertyHandle> InProperty)
{
	PropertyRow.CustomWidget()
		.NameContent()
		[
			InProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			CreateSpinBox<int32>(WeakTool, InProperty)
		];
}

void FAvaStaggerToolSettingsDetailsCustomization::AddCustomFloatSpinBoxRow(IDetailPropertyRow& PropertyRow, const TSharedRef<IPropertyHandle> InProperty)
{
	PropertyRow.CustomWidget()
		.NameContent()
		[
			InProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			CreateSpinBox<float>(WeakTool, InProperty)
		];
}

void FAvaStaggerToolSettingsDetailsCustomization::AddDistributionRow(IDetailPropertyRow& PropertyRow, const TSharedRef<IPropertyHandle>& InProperty)
{
	PropertyRow.CustomWidget()
		.NameContent()
		[
			InProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SAvaStaggerSettingsRadioGroup<EAvaSequencerStaggerDistribution>, InProperty)
		];
}

void FAvaStaggerToolSettingsDetailsCustomization::AddRandomSeedRow(IDetailPropertyRow& PropertyRow, const TSharedRef<IPropertyHandle>& InProperty)
{
	PropertyRow.CustomWidget()
		.NameContent()
		[
			InProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 5.f, 0.f))
			[
				CreateSpinBox<int32>(WeakTool, InProperty)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.TextStyle(FAppStyle::Get(), TEXT("SmallText"))
				.Text(LOCTEXT("GenerateRandomSeedLabel", "Generate"))
				.OnClicked_Lambda([this]()
					{
						RandomSeedProperty->SetValue(FMath::Rand());
						return FReply::Handled();
					})
			]
		];
}

void FAvaStaggerToolSettingsDetailsCustomization::AddRangeRow(IDetailPropertyRow& PropertyRow, const TSharedRef<IPropertyHandle>& InProperty)
{
	PropertyRow.CustomWidget()
		.NameContent()
		[
			InProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SAvaStaggerSettingsRadioGroup<EAvaSequencerStaggerRange>, InProperty)
		];
}

void FAvaStaggerToolSettingsDetailsCustomization::AddStartPositionRow(IDetailPropertyRow& PropertyRow, const TSharedRef<IPropertyHandle>& InProperty)
{
	PropertyRow.CustomWidget()
		.NameContent()
		[
			InProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SAvaStaggerSettingsRadioGroup<EAvaSequencerStaggerStartPosition>, InProperty)
		];
}

void FAvaStaggerToolSettingsDetailsCustomization::AddOperationPointRow(IDetailPropertyRow& PropertyRow, const TSharedRef<IPropertyHandle>& InProperty)
{
	PropertyRow.CustomWidget()
		.NameContent()
		[
			InProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			SNew(SAvaStaggerOperationPoint, InProperty)
		];
}

#undef LOCTEXT_NAMESPACE
