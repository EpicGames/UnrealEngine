// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Layout/Visibility.h"
#include "StaggerTool/AvaStaggerToolSettings.h"

class FAvaStaggerTool;
class IDetailPropertyRow;
class IPropertyHandle;

class FAvaStaggerToolSettingsDetailsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance(const TWeakPtr<FAvaStaggerTool> InWeakTool);

	FAvaStaggerToolSettingsDetailsCustomization(const TWeakPtr<FAvaStaggerTool>& InWeakTool);
	virtual ~FAvaStaggerToolSettingsDetailsCustomization() override {}

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization

private:
	EAvaSequencerStaggerDistribution GetDistributionPropertyValue() const;
	EAvaSequencerStaggerRange GetRangePropertyValue() const;
	EAvaSequencerStaggerStartPosition GetStartPositionPropertyValue() const;
	float GetOperationPointPropertyValue() const;
	bool GetUseCurvePropertyValue() const;

	EVisibility GetCurvePropertyVisibility() const;
	EVisibility GetCurveOffsetPropertyVisibility() const;
	EVisibility GetRandomSeedPropertyVisibility() const;
	EVisibility GetIntervalPropertyVisibility() const;
	EVisibility GetShiftPropertyVisibility() const;
	EVisibility GetRangePropertyVisibility() const;
	EVisibility GetCustomRangePropertyVisibility() const;
	EVisibility GetStartPositionPropertyVisibility() const;
	EVisibility GetOperationPointPropertyVisibility() const;

	void AddCustomIntSpinBoxRow(IDetailPropertyRow& PropertyRow, const TSharedRef<IPropertyHandle> InProperty);
	void AddCustomFloatSpinBoxRow(IDetailPropertyRow& PropertyRow, const TSharedRef<IPropertyHandle> InProperty);

	void AddDistributionRow(IDetailPropertyRow& PropertyRow, const TSharedRef<IPropertyHandle>& InProperty);
	void AddRandomSeedRow(IDetailPropertyRow& PropertyRow, const TSharedRef<IPropertyHandle>& InProperty);
	void AddRangeRow(IDetailPropertyRow& PropertyRow, const TSharedRef<IPropertyHandle>& InProperty);
	void AddStartPositionRow(IDetailPropertyRow& PropertyRow, const TSharedRef<IPropertyHandle>& InProperty);
	void AddOperationPointRow(IDetailPropertyRow& PropertyRow, const TSharedRef<IPropertyHandle>& InProperty);

	TWeakPtr<FAvaStaggerTool> WeakTool;

	TSharedPtr<IPropertyHandle> ToolOptionsProperty;

	TSharedPtr<IPropertyHandle> UseCurveProperty;
	TSharedPtr<IPropertyHandle> CurveProperty;
	TSharedPtr<IPropertyHandle> CurveOffsetProperty;
	TSharedPtr<IPropertyHandle> DistributionProperty;
	TSharedPtr<IPropertyHandle> RandomSeedProperty;
	TSharedPtr<IPropertyHandle> RangeProperty;
	TSharedPtr<IPropertyHandle> CustomRangeProperty;
	TSharedPtr<IPropertyHandle> StartPositionProperty;
	TSharedPtr<IPropertyHandle> OperationPointProperty;
	TSharedPtr<IPropertyHandle> IntervalProperty;
	TSharedPtr<IPropertyHandle> ShiftProperty;
	TSharedPtr<IPropertyHandle> GroupingProperty;
};
