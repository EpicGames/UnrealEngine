// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaStaggerOperationPoint.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaStaggerOperationPoint"

void SAvaStaggerOperationPoint::Construct(const FArguments& InArgs, const TWeakPtr<IPropertyHandle>& InWeakProperty)
{
	WeakProperty = InWeakProperty;

	const TSharedRef<SCheckBox> LeftButton = CreateButton(0.f, TEXT("HorizontalAlignment_Left"));
	Buttons.Add(LeftButton);

	const TSharedRef<SCheckBox> CenterButton = CreateButton(0.5f, TEXT("HorizontalAlignment_Center"));
	Buttons.Add(CenterButton);

	const TSharedRef<SCheckBox> RightButton = CreateButton(1.f, TEXT("HorizontalAlignment_Right"));
	Buttons.Add(RightButton);

	ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(1.f, 0.f))
			[
				SNew(SBox)
				.WidthOverride(124.f)
				[
					SNew(SSpinBox<float>)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.MinValue(0.f)
					.MaxValue(1.f)
					.MinSliderValue(0.f)
					.MaxSliderValue(1.f)
					.Value_Lambda([this]()
						{
							float OutValue = 0;
							if (const TSharedPtr<IPropertyHandle> Property = WeakProperty.Pin())
							{
								Property->GetValue(OutValue);
							}
							return OutValue;
						})
					.OnValueChanged_Lambda([this](const float InNewValue)
						{
							if (const TSharedPtr<IPropertyHandle> Property = WeakProperty.Pin())
							{
								Property->SetValue(InNewValue);
							}
						})
					.OnValueCommitted_Lambda([this](const float InNewValue, const ETextCommit::Type InCommitType)
						{
							if (const TSharedPtr<IPropertyHandle> Property = WeakProperty.Pin())
							{
								Property->SetValue(InNewValue);
							}
						})
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(1.f, 0.f))
			[
				LeftButton
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(1.f, 0.f))
			[
				CenterButton
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(1.f, 0.f))
			[
				RightButton
			]
		];
}

TSharedRef<SCheckBox> SAvaStaggerOperationPoint::CreateButton(const float InValue, const FName InImageBrushName)
{
	return SNew(SCheckBox)
		.Type(ESlateCheckBoxType::ToggleButton)
		.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>(TEXT("Sequencer.Outliner.ToggleButton")))
		.HAlign(HAlign_Center)
		.Padding(FMargin(4.f, 3.f))
		.IsChecked_Lambda([this, InValue]() -> ECheckBoxState
			{
				float Value = 0.f;
				WeakProperty.Pin()->GetValue(Value);
				return InValue == Value ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
		.OnCheckStateChanged_Lambda([this, InValue](const ECheckBoxState InCheckBoxState)
			{
				if (const TSharedPtr<IPropertyHandle> Property = WeakProperty.Pin())
				{
					Property->SetValue(InValue);
				}
			})
		[
			SNew(SBox)
			.WidthOverride(14.f)
			.HeightOverride(14.f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush(InImageBrushName))
			]
		];
}

#undef LOCTEXT_NAMESPACE
