// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class IPropertyHandle;
class SCheckBox;

class SAvaStaggerOperationPoint : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaStaggerOperationPoint) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<IPropertyHandle>& InWeakProperty);

protected:
	TSharedRef<SCheckBox> CreateButton(const float InValue, const FName InImageBrushName);

private:
	TWeakPtr<IPropertyHandle> WeakProperty;

	TArray<TSharedRef<SCheckBox>> Buttons;
};
