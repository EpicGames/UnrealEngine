// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DetailLayoutBuilder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformWrapPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

template<typename InEnumType>
class SAvaStaggerSettingsRadioGroup : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FEnumValueChangedDelegate, const InEnumType);

	SLATE_BEGIN_ARGS(SAvaStaggerSettingsRadioGroup) {}
		SLATE_EVENT(FEnumValueChangedDelegate, OnValueChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<IPropertyHandle>& InWeakProperty)
	{
		const UEnum* const Enum = StaticEnum<InEnumType>();
		check(Enum);

		WeakProperty = InWeakProperty;

		OnValueChanged = InArgs._OnValueChanged;

		const TSharedRef<SUniformWrapPanel> WrapPanel = SNew(SUniformWrapPanel)
			.HAlign(HAlign_Fill)
			.SlotPadding(FMargin(0.f, 0.f, 1.f, 1.f))
			.NumColumnsOverride(3);

		for (int32 Index = 0; Index < Enum->NumEnums() - 1; ++Index)
		{
			const bool bIsHidden = Enum->HasMetaData(TEXT("Hidden"), Index);
			if (!bIsHidden)
			{
				const TSharedRef<SCheckBox> NewButton = CreateButton(static_cast<InEnumType>(Enum->GetValueByIndex(Index))
					, Enum->GetDisplayNameTextByIndex(Index)
					, Enum->GetToolTipTextByIndex(Index));
				Buttons.Add(NewButton);

				WrapPanel->AddSlot()
					.VAlign(VAlign_Center)
					[
						NewButton
					];
			}
		}

		ChildSlot
			.HAlign(HAlign_Fill)
			.Padding(FMargin(0.f, 3.f))
			[
				WrapPanel
			];
	}

protected:
	TSharedRef<SCheckBox> CreateButton(const InEnumType InValue, const FText& InText, const FText& InToolTipText)
	{
		return SNew(SCheckBox)
			.Type(ESlateCheckBoxType::ToggleButton)
			.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>(TEXT("Sequencer.Outliner.ToggleButton")))
			.HAlign(HAlign_Center)
			.Padding(FMargin(4.f, 3.f))
			.ToolTipText(InToolTipText)
			.IsChecked_Lambda([this, InValue]() -> ECheckBoxState
				{
					uint8 Value = 0;
					WeakProperty.Pin()->GetValue(Value);
					return InValue == static_cast<InEnumType>(Value) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
			.OnCheckStateChanged_Lambda([this, InValue](const ECheckBoxState InCheckBoxState)
				{
					if (const TSharedPtr<IPropertyHandle> Property = WeakProperty.Pin())
					{
						Property->SetValue(static_cast<uint8>(InValue));
					}
					OnValueChanged.ExecuteIfBound(InValue);
				})
			[
				SNew(SBox)
				.WidthOverride(90.f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(InText)
				]
			];
	}

	InEnumType GetEnumValue() const
	{
		if (const TSharedPtr<IPropertyHandle> Property = WeakProperty.Pin())
		{
			uint8 Value = 0; 
			Property->GetValue(Value);
			return static_cast<InEnumType>(Value);
		}
		return InEnumType::Default;
	}

private:
	TWeakPtr<IPropertyHandle> WeakProperty;

	FEnumValueChangedDelegate OnValueChanged;

	TArray<TSharedRef<SCheckBox>> Buttons;
};
