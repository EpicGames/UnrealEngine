// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaStaggerTool.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailsView.h"
#include "Input/Reply.h"
#include "Misc/Optional.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SequencerCommands.h"
#include "StaggerTool/AvaStaggerTool.h"
#include "StaggerTool/Customization/AvaStaggerToolSettingsDetailsCustomization.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaStaggerTool"

void SAvaStaggerTool::Construct(const FArguments& InArgs, const TWeakPtr<FAvaStaggerTool>& InWeakTool)
{
	check(InWeakTool.IsValid());
	WeakTool = InWeakTool;

	OnResetToDefaults = InArgs._OnResetToDefaults;
	OnSettingChange = InArgs._OnSettingChange;
	OnApply = InArgs._OnApply;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.f, 5.f, 5.f, 2.f)
		[
			ConstructApplyRow()
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.f, 0.f, 5.f, 5.f)
		[
			ConstructDetails()
		]
	];
}

TSharedRef<SWidget> SAvaStaggerTool::ConstructDetails()
{
	if (!WeakTool.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bShowScrollBar = true;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.NotifyHook = this;
	DetailsViewArgs.ColumnWidth = 0.75f;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->RegisterInstancedCustomPropertyLayout(UAvaSequencerStaggerSettings::StaticClass()
		, FOnGetDetailCustomizationInstance::CreateStatic(&FAvaStaggerToolSettingsDetailsCustomization::MakeInstance, WeakTool));

	DetailsView->SetObject(WeakTool.Pin()->GetSettings());

	return DetailsView.ToSharedRef();
}

TSharedRef<SWidget> SAvaStaggerTool::ConstructApplyRow()
{
	constexpr float ButtonSize = 22.f;
	constexpr float ImageSize = 14.f;
	constexpr float Spacing = 5.f;

	return SNew(SBox)
		.HeightOverride(ButtonSize)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Fill)
			.Padding(Spacing, 0.f, Spacing, 0.f)
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), TEXT("Sequencer.Outliner.ToggleButton"))
				.ToolTipText(LOCTEXT("AutoApplyToolTip", "Auto apply options on change"))
				.IsChecked_Lambda([this]()
					{
						return (WeakTool.IsValid() && WeakTool.Pin()->IsAutoApplying())
							? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
				.OnCheckStateChanged(this, &SAvaStaggerTool::OnToggleAutoUpdateClick)
				[
					SNew(SBox) 
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.DesiredSizeOverride(FVector2D(ImageSize))
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::GetBrush(TEXT("MaterialEditor.Apply")))
					]
				]
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Bottom)
			.Padding(Spacing, 0.f, 0.f, 0.f)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), TEXT("HintText"))
				.Text(this, &SAvaStaggerTool::GetSelectionText)
				.ToolTipText(this, &SAvaStaggerTool::GetSelectionToolTipText)
				.ColorAndOpacity(this, &SAvaStaggerTool::GetSelectionTextColor)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Fill)
			.Padding(Spacing, 0.f, 0.f, 0.f)
			[
				SNew(SButton)
				.ContentPadding(2.f)
				.ToolTipText(FSequencerCommands::Get().AlignSelectionToPlayhead->GetDescription())
				.IsEnabled(this, &SAvaStaggerTool::OnCanAlignToPlayhead)
				.OnClicked(this, &SAvaStaggerTool::OnAlignToPlayhead)
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(ImageSize))
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::GetBrush(TEXT("HorizontalAlignment_Left")))
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Fill)
			.Padding(Spacing, 0.f, 0.f, 0.f)
			[
				SNew(SButton)
				.ContentPadding(2.f)
				.Text(LOCTEXT("ApplyButtonText", "Apply"))
				.OnClicked(this, &SAvaStaggerTool::OnApplyButtonClick)
				.IsEnabled_Lambda([this]()
					{
						if (const TSharedPtr<FAvaStaggerTool> Tool = WeakTool.Pin())
						{
							return Tool->HasValidSelection() && !Tool->IsAutoApplying();
						}
						return false;
					})
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Fill)
			.Padding(Spacing, 0.f, Spacing, 0.f)
			[
				SNew(SBox)
				.WidthOverride(ButtonSize)
				[
					SNew(SButton)
					.ContentPadding(2.f)
					.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
					.ToolTipText(LOCTEXT("ResetToDefaultsToolTip", "Reset All to Defaults"))
					.OnClicked(this, &SAvaStaggerTool::OnResetToDefaultsClick)
					.IsEnabled_Lambda([this]()
						{
							return WeakTool.IsValid() ? WeakTool.Pin()->GetSettings()->CanResetToolOptions() : false;
						})
					[
						SNew(SImage)
						.DesiredSizeOverride(FVector2D(ImageSize))
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::GetBrush(TEXT("PropertyWindow.DiffersFromDefault")))
					]
				]
			]
		];
}

FText SAvaStaggerTool::GetSelectionText() const
{
	const TSharedPtr<FAvaStaggerTool> StaggerTool = WeakTool.Pin();
	if (!StaggerTool.IsValid())
	{
		return FText::GetEmpty();
	}

	if (StaggerTool->IsBarSelection())
	{
		const FText SelectionCountText = FText::FromString(FString::FromInt(StaggerTool->GetSelectionCount()));
		return FText::Format(LOCTEXT("LayerBarSelectionDetails", "{0} Layer Bars Selected"), SelectionCountText);
	}

	if (StaggerTool->IsKeySelection())
	{
		const FText SelectionCountText = FText::FromString(FString::FromInt(StaggerTool->GetSelectionCount()));
		return FText::Format(LOCTEXT("KeyFrameSelectionDetails", "{0} Key Frames Selected"), SelectionCountText);
	}

	return LOCTEXT("InvalidSelection", "Invalid Selection");
}

FText SAvaStaggerTool::GetSelectionToolTipText() const
{
	return LOCTEXT("InvalidSelectionTooltip",
		"Select at least two Sequencer outliner tracks containing a layer bar or at least two layer bars directly\n\n"
		" - OR -\n\n"
		"Select at least two Sequencer key frames\n\n"
		"*NOTE* Order of selection matters! Tracks will be staggered in the order they were selected");
}

FSlateColor SAvaStaggerTool::GetSelectionTextColor() const
{
	return (WeakTool.IsValid() && WeakTool.Pin()->HasValidSelection()) ? GetForegroundColor() : FStyleColors::Error;
}

FReply SAvaStaggerTool::OnApplyButtonClick()
{
	OnApply.ExecuteIfBound();

	return FReply::Handled();
}

FReply SAvaStaggerTool::OnResetToDefaultsClick()
{
	OnResetToDefaults.ExecuteIfBound();

	return FReply::Handled();
}

void SAvaStaggerTool::OnToggleAutoUpdateClick(const ECheckBoxState InNewState)
{
	if (!WeakTool.IsValid())
	{
		return;
	}

	UAvaSequencerStaggerSettings* const Settings = WeakTool.Pin()->GetSettings();

	Settings->bAutoApply = InNewState == ECheckBoxState::Checked;

	if (Settings->bAutoApply)
	{
		OnApply.ExecuteIfBound();
	}
}

bool SAvaStaggerTool::OnCanAlignToPlayhead() const
{
	if (const TSharedPtr<FAvaStaggerTool> Tool = WeakTool.Pin())
	{
		return Tool->CanAlignToPlayhead();
	}
	return false;
}

FReply SAvaStaggerTool::OnAlignToPlayhead()
{
	if (const TSharedPtr<FAvaStaggerTool> Tool = WeakTool.Pin())
	{
		Tool->AlignToPlayhead();
	}
	return FReply::Handled();
}

void SAvaStaggerTool::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* const InPropertyChanged)
{
	OnSettingChange.ExecuteIfBound(InPropertyChanged->GetFName());
}

#undef LOCTEXT_NAMESPACE
