// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformMisc.h"
#include "Misc/Attribute.h"
#include "Misc/NotifyHook.h"
#include "Misc/OptionalFwd.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SCompoundWidget.h"

class FAvaStaggerTool;
class FReply;
class IDetailsView;
class SWidget;

DECLARE_DELEGATE_OneParam(FAvaStaggerToolSettingChange, const FName /*InPropertyName*/)

class SAvaStaggerTool : public SCompoundWidget, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(SAvaStaggerTool) {}
		SLATE_EVENT(FSimpleDelegate, OnResetToDefaults)
		SLATE_EVENT(FAvaStaggerToolSettingChange, OnSettingChange)
		SLATE_EVENT(FSimpleDelegate, OnApply)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<FAvaStaggerTool>& InWeakTool);

protected:
	TSharedRef<SWidget> ConstructDetails();
	TSharedRef<SWidget> ConstructApplyRow();

	FText GetSelectionText() const;
	FText GetSelectionToolTipText() const;
	FSlateColor GetSelectionTextColor() const;

	FReply OnApplyButtonClick();
	FReply OnResetToDefaultsClick();
	void OnToggleAutoUpdateClick(const ECheckBoxState InNewState);

	bool OnCanAlignToPlayhead() const;
	FReply OnAlignToPlayhead();

	//~ Begin FNotifyHook
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* const InPropertyChanged) override;
	//~ End FNotifyHook

	TWeakPtr<FAvaStaggerTool> WeakTool;

	FSimpleDelegate OnResetToDefaults;
	FAvaStaggerToolSettingChange OnSettingChange;
	FSimpleDelegate OnApply;

	TSharedPtr<IDetailsView> DetailsView;
};
