// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagSoftHandle.h"

class FAvaSequenceDefaultTags
{
	FAvaSequenceDefaultTags();

public:
	AVALANCHESEQUENCER_API static const FAvaSequenceDefaultTags& Get();

	FAvaTagSoftHandle In;
	FAvaTagSoftHandle Out;
	FAvaTagSoftHandle Change;
};
