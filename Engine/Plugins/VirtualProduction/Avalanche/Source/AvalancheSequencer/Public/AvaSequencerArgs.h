// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IAvaSequencerController;

struct FAvaSequencerArgs
{
	/** Optional Controller instance to use instead of Ava Sequencer instantiating its own */
	TSharedPtr<IAvaSequencerController> SequencerController;

	/**
	 * Whether Custom Clean Playback Mode should be used,
	 * rather than relying on FSequencer's clean playback that currently only works with level editor viewport clients
	 */
	bool bUseCustomCleanPlaybackMode = false;

	/**
	 * Whether FAvaSequencer is allowed to select to/from the ISequencer instance
	 * This could be false in, for example, when the ISequencer instance is instantiated as being a Level Editor Sequencer (as it processes the selections in FSequencer)
	 */
	bool bCanProcessSequencerSelections = true;
};
