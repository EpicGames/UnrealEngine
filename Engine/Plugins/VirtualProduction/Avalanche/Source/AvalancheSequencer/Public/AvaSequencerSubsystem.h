// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequencerArgs.h"
#include "Subsystems/WorldSubsystem.h"
#include "Templates/SharedPointer.h"
#include "AvaSequencerSubsystem.generated.h"

class IAvaSequencer;
class IAvaSequencerProvider;

/** Subsystem in charge of instancing and keeping reference of the World's Sequencer */
UCLASS()
class AVALANCHESEQUENCER_API UAvaSequencerSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	/**
	 * Gets or Instantiates the World's Sequencer
	 * @param InProvider the sequencer provider
	 * @param InArgs extra arguments to dictate how the ava sequencer is instantiated
	 * @returns an existing or new valid IAvaSequencer instance
	 * @note the subsystem holds a non-owning reference of the sequencer, the provider is what ideally should be holding the owning reference
	 */
	TSharedRef<IAvaSequencer> GetOrCreateSequencer(IAvaSequencerProvider& InProvider, FAvaSequencerArgs&& InArgs);

	TSharedPtr<IAvaSequencer> GetSequencer() const { return SequencerWeak.Pin(); }

	/** A delegate that gets executed then a sequencer is created */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAvaSequencerCreated, TSharedRef<IAvaSequencer>);
	FOnAvaSequencerCreated& OnSequencerCreated() { return OnSequencerCreatedDelegate; }

	//~ Begin UWorldSubsystem
	virtual bool DoesSupportWorldType(const EWorldType::Type InWorldType) const override;
	//~ End UWorldSubsystem

private:
	TWeakPtr<IAvaSequencer> SequencerWeak;

	FOnAvaSequencerCreated OnSequencerCreatedDelegate;
};
