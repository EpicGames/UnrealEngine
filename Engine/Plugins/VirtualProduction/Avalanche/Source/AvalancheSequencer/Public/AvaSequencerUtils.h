// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerModule.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointerFwd.h"
#include "Templates/Function.h"

class IAvaSceneInterface;
class IAvaSequenceProvider;
class IAvaSequencer;
class IAvaSequencerController;
class ISequencer;
class UAvaSceneSubsystem;
class UAvaSequencerSubsystem;
class UWorld;

struct FAvaSequencerUtils
{
	static const FName& GetSequencerModuleName()
	{
		static const FName SequencerModuleName("Sequencer");
		return SequencerModuleName;
	}

	static ISequencerModule& GetSequencerModule()
	{
		return FModuleManager::Get().LoadModuleChecked<ISequencerModule>(GetSequencerModuleName());
	}

	static bool IsSequencerModuleLoaded()
	{
		return FModuleManager::Get().IsModuleLoaded(GetSequencerModuleName());
	}

	AVALANCHESEQUENCER_API static TSharedRef<IAvaSequencerController> CreateSequencerController();

	/** @return Motion Design Sequencer World from an ISequencer */
	static UWorld* GetSequencerWorld(const TSharedRef<ISequencer>& InSequencer);

	/** @return Motion Design Sequencer Subsystem from an ISequencer */
	static UAvaSequencerSubsystem* GetSequencerSubsystem(const TSharedRef<ISequencer>& InSequencer);

	/** @return Motion Design Scene Subsystem from an ISequencer */
	static UAvaSceneSubsystem* GetSceneSubsystem(const TSharedRef<ISequencer>& InSequencer);

	/** @return Motion Design Scene Interface from an ISequencer */
	AVALANCHESEQUENCER_API static IAvaSceneInterface* GetSceneInterface(const TSharedRef<ISequencer>& InSequencer);

	/** @return Motion Design Sequence Provider from an ISequencer */
	AVALANCHESEQUENCER_API static IAvaSequenceProvider* GetSequenceProvider(const TSharedRef<ISequencer>& InSequencer);

	/** @return Motion Design Sequencer from an ISequencer */
	AVALANCHESEQUENCER_API static TSharedPtr<IAvaSequencer> GetAvaSequencer(const TSharedRef<ISequencer>& InSequencer);
};
