// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequence.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringView.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/WeakObjectPtrTemplatesFwd.h"

class AActor;
class FAvaEditorSelection;
class FString;
class FUICommandList;
class IAvaSequencerProvider;
class IPropertyHandle;
class ISequencer;
class SWidget;
class UAvaSequence;
class UObject;
class USequencerSettings;
struct FAvaSequencePreset;

class IAvaSequencer
{
public:
	virtual ~IAvaSequencer() = default;

	virtual const IAvaSequencerProvider& GetProvider() const = 0;

	UE_DEPRECATED(5.7, "Get sequencer by shared reference is deprecated. Use GetSequencerPtr instead.")
	virtual TSharedRef<ISequencer> GetSequencer() const
	{
		return GetSequencerPtr().ToSharedRef();
	}

	/** Gets the sequencer ptr if available. */
	virtual TSharedPtr<ISequencer> GetSequencerPtr() const = 0;

	virtual USequencerSettings* GetSequencerSettings() const = 0;

	/** Sets the Command List that the Sequencer will use to append its CommandList to */
	virtual void SetBaseCommandList(const TSharedPtr<FUICommandList>& InBaseCommandList) = 0;

	/** Get the currently viewed Sequence in Sequencer */
	virtual UAvaSequence* GetViewedSequence() const = 0;

	/** Get the Provider's Default Sequence (e.g. a fallback sequence to view), setting a new valid one if not set */
	virtual UAvaSequence* GetDefaultSequence() const = 0;

	/** Sets the Sequencer to view the provided Sequence */
	virtual void SetViewedSequence(UAvaSequence* InSequenceToView) = 0;

	/** Finds all the sequences the given object is in */
	virtual TArray<UAvaSequence*> GetSequencesForObject(UObject* InObject) const = 0;

	virtual TSharedRef<SWidget> CreateSequenceWidget() = 0;

	/** Should be called when Actors have been copied and give Ava Sequencer opportunity to add to the Buffer to copy the Sequence data for those Actors */
	virtual void OnActorsCopied(FString& InOutCopiedData, TConstArrayView<AActor*> InCopiedActors) = 0;

	/** Should be called when Actors have been pasted to parse the data that was filled in by IAvaSequencer::OnActorsCopied */
	virtual void OnActorsPasted(FStringView InPastedData, const TMap<FName, AActor*>& InPastedActors) = 0;

	/** Should be called when the Editor (non-Sequencer) selection has changed and needs to propagate to the Sequencer Selection */
	virtual void OnEditorSelectionChanged(const FAvaEditorSelection& InEditorSelection) = 0;

	/** Should be called when the UAvaSequence tree has changed, so that FAvaSequencer can trigger a UI refresh */
	virtual void NotifyOnSequenceTreeChanged() = 0;

	/** Gets the list of embedded level sequences */
	virtual const TArray<TWeakObjectPtr<UAvaSequence>>& GetRootSequences() const = 0;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSequenceAdded, UAvaSequence*);
	virtual FOnSequenceAdded& OnSequenceAdded() = 0;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSequenceRemoved, UAvaSequence*);
	virtual FOnSequenceRemoved& OnSequenceRemoved() = 0;

	virtual bool CanAddSequence() const = 0;
	virtual UAvaSequence* AddSequence(UAvaSequence* const InParentSequence = nullptr) = 0;

	/** Creates new sequences based on the given presets */
	virtual uint32 AddSequenceFromPresets(TConstArrayView<const FAvaSequencePreset*> InPresets) = 0;

	virtual void DeleteSequences(const TSet<UAvaSequence*>& InSequences) = 0;
};
