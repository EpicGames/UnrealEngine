// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class ISequencer;

class IAvaSequencerController : public TSharedFromThis<IAvaSequencerController>
{
public:
	virtual ~IAvaSequencerController() = default;

	virtual void SetSequencer(const TSharedPtr<ISequencer>& InSequencer) = 0;
};
