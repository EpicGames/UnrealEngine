// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "MovieSceneFwd.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"

class FEditorModeTools;
class FEditorViewportClient;
class IAvaSequencer;
class IAvaSequencePlaybackObject;
class IAvaSequenceProvider;
class ISequencer;
class IToolkitHost;
class UActorComponent;
class UAvaSequence;
struct FFrameTime;

class IAvaSequencerProvider
{
public:
    virtual ~IAvaSequencerProvider() = default;

	virtual TSharedPtr<IAvaSequencer> GetAvaSequencer() const = 0;

	/** Option for FAvaSequencer to use an externally instanced ISequencer instead of instancing its own */
	virtual TSharedPtr<ISequencer> GetExternalSequencer() const { return nullptr; };

	virtual void OnViewedSequenceChanged(UAvaSequence* InOldSequence, UAvaSequence* InNewSequence) {}

	/** Returns the Sequence Provider (for managing AvaSequences like Adding/Removing Sequences, etc) */
	virtual IAvaSequenceProvider* GetSequenceProvider() const = 0;

	/** Returns the Mode Tools used (e.g. for Selections) */
	virtual FEditorModeTools* GetSequencerModeTools() const = 0;

	virtual IAvaSequencePlaybackObject* GetPlaybackObject() const = 0;

	/** Returns the Toolkit Host to use by Sequencer */
    virtual TSharedPtr<IToolkitHost> GetSequencerToolkitHost() const = 0;

	/** Gets the Context object to use */
	virtual UObject* GetPlaybackContext() const = 0;

	/** Returns whether Sequences can be played in the Sequencer */
	virtual bool CanEditOrPlaySequences() const = 0;

	/** Gets the Editor Viewport Clients to be affected by the Custom Clean View */
	virtual void GetCustomCleanViewViewportClients(TArray<TWeakPtr<FEditorViewportClient>>& OutViewportClients) const {}

	/** Called when a camera cut occurs. */
	virtual void OnUpdateCameraCut(UObject* InCameraObject, bool bInJumpCut) {};

	/** Implementation of converting the Sequences to Level Sequence assets */
	virtual void ExportSequences(TConstArrayView<UAvaSequence*> InSequencesToExport) {}
};
