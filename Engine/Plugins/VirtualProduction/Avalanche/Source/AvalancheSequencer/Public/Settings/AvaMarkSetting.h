// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Marks/AvaMark.h"
#include "MovieSceneMarkedFrame.h"
#include "AvaMarkSetting.generated.h"

USTRUCT()
struct FAvaMarkSetting
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Mark")
	FString Label;

	UPROPERTY(EditAnywhere, Category = "Mark")
	int32 FrameNumber = 0;

	UPROPERTY(EditAnywhere, Category = "Mark")
	FAvaMark Mark;
};
