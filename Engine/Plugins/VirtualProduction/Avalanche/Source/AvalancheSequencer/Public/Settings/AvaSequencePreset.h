// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaMarkSetting.h"
#include "AvaTagSoftHandle.h"
#include "AvaSequencePreset.generated.h"

class UAvaSequence;

/** Simple settings to apply to a sequence */
USTRUCT()
struct FAvaSequencePreset
{
	GENERATED_BODY()

	using FKeyType = FName;

	AVALANCHESEQUENCER_API FAvaSequencePreset() = default;
	AVALANCHESEQUENCER_API virtual ~FAvaSequencePreset() = default;

	AVALANCHESEQUENCER_API explicit FAvaSequencePreset(FName InPresetName)
		: PresetName(InPresetName)
	{
	}

	bool ShouldModifySequence() const;

	bool ShouldModifyMovieScene() const;

	AVALANCHESEQUENCER_API void ApplyPreset(UAvaSequence* InSequence) const;

	bool operator==(const FAvaSequencePreset& InOther) const
	{
		return PresetName == InOther.PresetName;
	}

	bool operator==(const FAvaSequencePreset::FKeyType& InKey) const
	{
		return PresetName == InKey;
	}

	friend uint32 GetTypeHash(const FAvaSequencePreset& InPreset)
	{
		return GetTypeHash(InPreset.PresetName);
	}

	/** Name to identify this preset */
	UPROPERTY(EditAnywhere, Category="Sequence")
	FName PresetName;

	/** If not none, the sequence label to set */
	UPROPERTY(EditAnywhere, Category="Sequence", meta=(EditCondition="bEnableLabel"))
	FName SequenceLabel;

	UPROPERTY(EditAnywhere, Category="Sequence", meta=(EditCondition="bEnableTag"))
	FAvaTagSoftHandle SequenceTag;

	/** Initial end time to set for a sequence, in seconds */
	UPROPERTY(EditAnywhere, Category="Sequence", meta=(EditCondition="bEnableEndTime"))
	double EndTime = 1.0;

	UPROPERTY(EditAnywhere, Category="Sequence", meta=(EditCondition="bEnableMarks"))
	TArray<FAvaMarkSetting> Marks;

	UPROPERTY(meta=(InlineEditConditionToggle))
	bool bEnableLabel = false;

	UPROPERTY(meta=(InlineEditConditionToggle))
	bool bEnableTag = false;

	UPROPERTY(meta=(InlineEditConditionToggle))
    bool bEnableEndTime = false;

	UPROPERTY(meta=(InlineEditConditionToggle))
	bool bEnableMarks = false;
};

/** Represents one or many presets to apply */
USTRUCT()
struct FAvaSequencePresetGroup
{
	GENERATED_BODY()

	using FKeyType = FName;
	
	bool operator==(const FAvaSequencePresetGroup& InOther) const
	{
		return GroupName == InOther.GroupName;
	}

	bool operator==(const FAvaSequencePreset::FKeyType& InKey) const
	{
		return GroupName == InKey;
	}

	friend uint32 GetTypeHash(const FAvaSequencePresetGroup& InPresetGroup)
	{
		return GetTypeHash(InPresetGroup.GroupName);
	}

	/** Identifier of this group */
	UPROPERTY(EditAnywhere, Category="Sequence")
	FName GroupName;

	/** The presets that this group represents */
	UPROPERTY(EditAnywhere, Category="Sequence")
	TArray<FName> PresetNames;
};
