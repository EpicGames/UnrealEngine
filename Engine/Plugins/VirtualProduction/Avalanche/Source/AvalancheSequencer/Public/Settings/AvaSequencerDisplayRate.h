// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "AvaSequencerDisplayRate.generated.h"

USTRUCT(BlueprintType)
struct FAvaSequencerDisplayRate
{
	GENERATED_BODY()

	UPROPERTY()
	FFrameRate FrameRate;
};
