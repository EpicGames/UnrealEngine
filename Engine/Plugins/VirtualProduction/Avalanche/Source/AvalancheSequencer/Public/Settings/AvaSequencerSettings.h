// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequencePreset.h"
#include "AvaSequencerDisplayRate.h"
#include "Engine/DeveloperSettings.h"
#include "Sidebar/SidebarState.h"
#include "UObject/Object.h"
#include "AvaSequencerSettings.generated.h"

#define UE_API AVALANCHESEQUENCER_API

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, meta = (DisplayName = "Sequencer"))
class UAvaSequencerSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	static constexpr const TCHAR* SettingsCategory = TEXT("Motion Design");
	static constexpr const TCHAR* SettingsSection  = TEXT("Sequencer");

	/** Group name for the built-in default groups */
	UE_API static const FLazyName TransitionLogicStarterGroup;
	UE_API static const FLazyName TransitionLogicSubLayerGroup;

	/** Default change sequence names */
	UE_API static const FLazyName DefaultChangeInName;
	UE_API static const FLazyName DefaultChangeOutName;

	UAvaSequencerSettings();

	FFrameRate GetDisplayRate() const
	{
		return DisplayRate.FrameRate;
	}

	double GetStartTime() const
	{
		return StartTime;
	}

	double GetEndTime() const
	{
		return EndTime;
	}

	/** Attempts to find the preset with the given preset name, first searching in the custom presets, then looking at the default presets */
	const FAvaSequencePreset* FindPreset(FName InPresetName) const;

	/** Attempts to find the preset group with the given preset name, first searching in the custom groups, then looking at the default (TL) groups */
	const FAvaSequencePresetGroup* FindPresetGroup(FName InGroupName) const;

	/** Finds all the presets within a preset group */
	UE_API TArray<const FAvaSequencePreset*> GatherPresetsFromGroup(FName InGroupName) const;

	UE_API TConstArrayView<FAvaSequencePreset> GetDefaultSequencePresets() const;

	UE_API TConstArrayView<FAvaSequencePresetGroup> GetTransitionLogicPresetGroups() const;

	const TSet<FAvaSequencePreset>& GetCustomSequencePresets() const
	{
		return CustomSequencePresets;
	}

	const TSet<FAvaSequencePresetGroup>& GetCustomPresetGroups() const
	{
		return CustomPresetGroups;
	}

	UE_DEPRECATED(5.6, "The Motion Design sidebar drawer was moved to the Sequencer sidebar and now stores its state in USequencerSettings. Use USequencerSettings::GetSidebarState() instead.")
	FSidebarState& GetSidebarState()
	{
		return SidebarState_DEPRECATED;
	}

	UE_DEPRECATED(5.6, "The Motion Design sidebar drawer was moved to the Sequencer sidebar and now stores its state in USequencerSettings. Use USequencerSettings::SetSidebarState() instead.")
	void SetSidebarState(const FSidebarState& InSidebarState)
	{
		SidebarState_DEPRECATED = InSidebarState;
	}

private:
	/** The default display rate to use for new sequences */
	UPROPERTY(Config, EditAnywhere, Category = "Playback")
	FAvaSequencerDisplayRate DisplayRate;

	/** The default start time to use for new sequences */
	UPROPERTY(Config, EditAnywhere, Category = "Playback")
	double StartTime = 0.0;

	/** The default end time to use for new sequences */
	UPROPERTY(Config, EditAnywhere, Category = "Playback")
	double EndTime = 2.0;

	/** Sequence Presets that are uniquely identified by their Preset Name */
	UPROPERTY(Config, EditAnywhere, Category = "Sequencer")
	TSet<FAvaSequencePreset> CustomSequencePresets;

	/** Sequence Preset groups that will be served as options when creating sequences */
	UPROPERTY(Config, EditAnywhere, Category = "Sequencer")
	TSet<FAvaSequencePresetGroup> CustomPresetGroups;

	/** The state of a sidebar to be restored when Sequencer is initialized */
	UPROPERTY(Config, meta = (DeprecatedProperty, DeprecationMessage = "The Motion Design sidebar drawer was moved to the Sequencer sidebar and now stores its state in USequencerSettings"))
	FSidebarState SidebarState_DEPRECATED;
};

#undef UE_API
