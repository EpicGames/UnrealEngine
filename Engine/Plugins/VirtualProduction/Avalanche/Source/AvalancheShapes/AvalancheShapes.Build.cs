// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheShapes : ModuleRules
{
	public AvalancheShapes(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AvalancheCore",
				"Avalanche",
				"AvalancheInteractiveToolsRuntime",
				"AvalancheMaterial",
				"Core",
				"CoreUObject",
				"Engine",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ActorModifierCore",
				"DeveloperSettings",
				"DynamicMaterial",
				"DynamicMesh",
				"GeometryCore",
				"GeometryFramework",
				"GeometryScriptingCore",
				"GeometryAlgorithms",
				"InputCore",
				"MovieScene",
				"MovieSceneTracks"
			}
		);

		if (Target.Type == TargetRules.TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"AvalancheEditorCore",
				"DynamicMaterialEditor",
				"TypedElementFramework",
				"UnrealEd",
			});
		}
	}
}
