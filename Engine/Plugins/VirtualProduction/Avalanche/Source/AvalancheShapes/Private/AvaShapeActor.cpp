// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeActor.h"

#include "AvaShapeSettings.h"
#include "Components/DynamicMeshComponent.h"
#include "Engine/CollisionProfile.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"

const FName AAvaShapeActor::ShapeComponentName("Shape Component");

// Sets default values
AAvaShapeActor::AAvaShapeActor()
{
	SetCanBeDamaged(false);
	PrimaryActorTick.bCanEverTick          = false;
	PrimaryActorTick.bStartWithTickEnabled = false;
	PrimaryActorTick.bTickEvenWhenPaused   = false;

	ShapeMeshComponent = CreateDefaultSubobject<UDynamicMeshComponent>(ShapeComponentName);

	// Setup collision
	ShapeMeshComponent->SetCollisionProfileName(UCollisionProfile::BlockAllDynamic_ProfileName);
	ShapeMeshComponent->SetGenerateOverlapEvents(true);
	ShapeMeshComponent->SetCanEverAffectNavigation(false);
	ShapeMeshComponent->SetComplexAsSimpleCollisionEnabled(true);
	ShapeMeshComponent->SetMeshDrawPath(EDynamicMeshDrawPath::StaticDraw);

	SetRootComponent(ShapeMeshComponent);

	bFinishedCreation = false;
}

void AAvaShapeActor::SetDynamicMesh(UAvaShapeDynamicMeshBase* NewDynamicMesh)
{
	DynamicMesh = NewDynamicMesh;
	DynamicMesh->SetFlags(RF_Transactional);

	AddInstanceComponent(DynamicMesh);

	NewDynamicMesh->OnComponentCreated();
	NewDynamicMesh->RegisterComponent();

#if WITH_EDITOR
	RerunConstructionScripts();
#endif
}

#if WITH_EDITOR
FString AAvaShapeActor::GetDefaultActorLabel() const
{
	return DynamicMesh
		? DynamicMesh->GetMeshName()
		: TEXT("Shape");
}

void AAvaShapeActor::PostEditUndo()
{
	// fix to restore dynamic mesh
	if (DynamicMesh && !IsValid(DynamicMesh))
	{
		DynamicMesh->ClearGarbage();
	}
	
	Super::PostEditUndo();
}
#endif

FAvaColorChangeData AAvaShapeActor::GetColorData() const
{
	if (!DynamicMesh)
	{
		return FAvaColorChangeData(EAvaColorStyle::None, FLinearColor::White, FLinearColor::White, true);
	}

	return DynamicMesh->GetActiveColor();
}

void AAvaShapeActor::SetColorData(const FAvaColorChangeData& NewColorData)
{
	if (!DynamicMesh)
	{
		return;
	}

	DynamicMesh->OnColorPicked(NewColorData);
}

void AAvaShapeActor::PostInitProperties()
{
	Super::PostInitProperties();

	if (ShapeMeshComponent && !HasAnyFlags(RF_ClassDefaultObject))
	{
		const UAvaShapeSettings* const Settings = GetDefault<UAvaShapeSettings>();
		if (Settings && Settings->ShouldForceDisableShapeCollision())
		{
			ShapeMeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			ShapeMeshComponent->bEnableComplexCollision = false;
		}
	}
}

TArray<FAvaSnapPoint> AAvaShapeActor::GetLocalSnapPoints() const
{
	if (DynamicMesh)
	{
		return DynamicMesh->GetLocalSnapPoints();
	}

	return {};
}

UObject* AAvaShapeActor::GetModeDetailsObject_Implementation() const
{
	return DynamicMesh.Get();
}

void AAvaShapeActor::RefreshParametricMaterial()
{
	if (DynamicMesh)
	{
		DynamicMesh->RefreshParametricMaterial();
	}
}
