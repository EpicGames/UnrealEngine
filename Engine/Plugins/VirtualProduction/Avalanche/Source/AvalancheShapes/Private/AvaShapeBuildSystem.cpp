// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeBuildSystem.h"

#include "Algo/ForEach.h"
#include "Async/ParallelFor.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMeshes/AvaShape2DArrowDynMesh.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "Modifiers/ActorModifierCoreDefs.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"

#if USING_INSTRUMENTATION
#include "Sanitizer/RaceDetector.h"
#endif

namespace UE::AvaShape::Private
{

#if USING_INSTRUMENTATION
static bool GDetectRaceDuringBuild = false;
static FAutoConsoleVariableRef CVarDetectRaceDuringBuild(
	TEXT("MotionDesign.Shapes.DetectRaceDuringBuild"),
	GDetectRaceDuringBuild,
	TEXT("Activate the race detector when building shapes in parallel"),
	ECVF_Default
);
#endif

} // UE::AvaShape::Private

UAvaShapeBuildSystem* UAvaShapeBuildSystem::FindBuildSystem(TNotNull<const UObject*> InContextObject)
{
	if (UWorld* const World = InContextObject->GetWorld())
	{
		return World->GetSubsystem<UAvaShapeBuildSystem>();
	}
	return nullptr;
}

void UAvaShapeBuildSystem::QueueShapeBuild(UAvaShapeDynamicMeshBase* InShape)
{
	if (!InShape)
	{
		return;
	}

	// Required: shapes need to be added uniquely
	QueuedShapes.AddUnique(InShape);
}

bool UAvaShapeBuildSystem::DoesSupportWorldType(const EWorldType::Type InWorldType) const
{
	return InWorldType != EWorldType::None;
}

bool UAvaShapeBuildSystem::IsTickableInEditor() const
{
	return true;
}

void UAvaShapeBuildSystem::Tick(float InDeltaTime)
{
	BuildShapes();
}

TStatId UAvaShapeBuildSystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UAvaShapeBuildSubsystem, STATGROUP_Tickables);
}

UAvaShapeBuildSystem::FShapeBuild::FShapeBuild(TNotNull<UAvaShapeDynamicMeshBase*> InShapeBuilder, const UActorModifierCoreSubsystem* InModifierSystem)
	: Shape(InShapeBuilder)
	, ModifierLock(InModifierSystem ? InModifierSystem->GetActorModifierStack(InShapeBuilder->GetShapeActor()) : nullptr)
{
}

void UAvaShapeBuildSystem::BuildShapes()
{
	if (QueuedShapes.IsEmpty())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UAvaShapeBuildSubsystem::BuildShapes);

	// shape builds that can be built from different threads.
	TArray<FShapeBuild> ParallelBuilds;
	ParallelBuilds.Reserve(QueuedShapes.Num());

	// shape builds that can only be built in game thread.
	TArray<FShapeBuild> GameThreadBuilds;

	const UActorModifierCoreSubsystem* const ModifierSubsystem = UActorModifierCoreSubsystem::Get();
	for (const TWeakObjectPtr<UAvaShapeDynamicMeshBase>& ShapeWeak : QueuedShapes)
	{
		UAvaShapeDynamicMeshBase* const Shape = ShapeWeak.Get();
		if (!Shape)
		{
			continue;
		}

		if (Shape->CanUpdateMeshInAnyThread())
		{
			ParallelBuilds.Emplace(Shape, ModifierSubsystem);	
		}
		else
		{
			GameThreadBuilds.Emplace(Shape, ModifierSubsystem);
		}
	}
	QueuedShapes.Empty();

	if (GameThreadBuilds.IsEmpty() && ParallelBuilds.IsEmpty())
	{
		return;
	}

	Algo::ForEach(GameThreadBuilds, Projection(&FShapeBuild::Shape, &UAvaShapeDynamicMeshBase::PreUpdateMesh));
	Algo::ForEach(ParallelBuilds  , Projection(&FShapeBuild::Shape, &UAvaShapeDynamicMeshBase::PreUpdateMesh));

	const UAvaShapeDynamicMeshBase::FMeshUpdateContext ParallelUpdateContext
		{
			.bParallel = true
		};

	// Update shapes in parallel
	{
		// Shape building could be highly variable as it depends on the shape being updated
		constexpr EParallelForFlags ParallelForFlags = EParallelForFlags::Unbalanced;

#if USING_INSTRUMENTATION
		UE::Sanitizer::RaceDetector::FRaceDetectorScope RaceDetectorScope(UE::AvaShape::Private::GDetectRaceDuringBuild);
#endif

		ParallelForWithPreWork(ParallelBuilds.Num(),
			[&ParallelBuilds, &ParallelUpdateContext](int32 InBuildIndex)
			{
				ParallelBuilds[InBuildIndex].Shape->UpdateMesh(ParallelUpdateContext);
			},
			[&GameThreadBuilds]()
			{
				const UAvaShapeDynamicMeshBase::FMeshUpdateContext GameThreadUpdateContext
					{
						.bParallel = false
					};
				for (FShapeBuild& Build : GameThreadBuilds)
				{
					Build.Shape->UpdateMesh(GameThreadUpdateContext);
				}
			}, ParallelForFlags);
	}

	Algo::ForEach(GameThreadBuilds, Projection(&FShapeBuild::Shape, &UAvaShapeDynamicMeshBase::PostUpdateMesh));
	Algo::ForEach(ParallelBuilds  , Projection(&FShapeBuild::Shape, &UAvaShapeDynamicMeshBase::PostUpdateMesh));

	// Unlock modifier execution by emptying the builds
	// Done before scope exit for profiling
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UAvaShapeBuildSubsystem::BuildShapes_UnlockModifierExecutions);
		GameThreadBuilds.Empty();
		ParallelBuilds.Empty();
	}
}
