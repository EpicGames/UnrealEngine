// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/ActorModifierCoreDefs.h"
#include "Subsystems/WorldSubsystem.h"
#include "AvaShapeBuildSystem.generated.h"

class UActorModifierCoreSubsystem;
class UAvaShapeDynamicMeshBase;
class UDynamicMeshComponent;

UCLASS()
class UAvaShapeBuildSystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()

public:
	/** Returns the build system for the given context object */
	static UAvaShapeBuildSystem* FindBuildSystem(TNotNull<const UObject*> InContextObject);

	void QueueShapeBuild(UAvaShapeDynamicMeshBase* InShape);

	//~ Begin UWorldSubsystem
	virtual bool DoesSupportWorldType(const EWorldType::Type InWorldType) const override;
	//~ End UWorldSubsystem

	//~ Begin FTickableGameObject
	virtual bool IsTickableInEditor() const override;
	virtual void Tick(float InDeltaTime) override;
	virtual TStatId GetStatId() const override;
	//~ End FTickableGameObject

private:
	struct FShapeBuild
	{
		explicit FShapeBuild(TNotNull<UAvaShapeDynamicMeshBase*> InShapeBuilder, const UActorModifierCoreSubsystem* InModifierSystem);
		/** Shape to build */
		TNotNull<UAvaShapeDynamicMeshBase*> Shape;
		/** Keep the modifier locked while build takes place */
		FActorModifierCoreScopedLock ModifierLock;
	};
	void BuildShapes();

	UPROPERTY()
	TArray<TWeakObjectPtr<UAvaShapeDynamicMeshBase>> QueuedShapes;
};
