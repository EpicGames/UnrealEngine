// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeMaterialBridge.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "MaterialBridge/Context/AvaMaterialBridgeApplyStateContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeReadSlotContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeStoreStateContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeWriteSlotContext.h"
#include "MaterialBridge/Slot/AvaMaterialBridgeReadSlot.h"
#include "MaterialBridge/Slot/CommonFeatures/AvaMaterialBridgeBlendModeFeature.h"

namespace UE::Ava
{

namespace Private
{

EAvaShapeParametricMaterialTranslucency GetTargetTranslucency(EBlendMode InBlendMode)
{
	switch (InBlendMode)
	{
	case BLEND_Masked:
	case BLEND_Translucent:
		return EAvaShapeParametricMaterialTranslucency::Enabled;

	default:
		return EAvaShapeParametricMaterialTranslucency::Auto;
	}
}

} // UE::Ava::Private

FShapeMaterialWriteSlot::FShapeMaterialWriteSlot(TNotNull<UAvaShapeDynamicMeshBase*> InShape, int32 InMeshIndex)
	: FMaterialBridgeWriteSlot(InShape->GetMaterial(InMeshIndex), FSlotId(InMeshIndex))
	, Shape(InShape)
	, MeshIndex(InMeshIndex)
{
}

void FShapeMaterialWriteSlot::OnMaterialChanged()
{
	Shape->SetMaterial(MeshIndex, GetMaterial());
}

bool FShapeMaterialWriteSlot::OnFeatureRequest(TConstStructView<FAvaMaterialBridgeFeature> InFeature)
{
	// Blend mode feature
	if (const FAvaMaterialBridgeBlendModeFeature* BlendModeFeature = InFeature.GetPtr<FAvaMaterialBridgeBlendModeFeature>())
	{
		const EAvaShapeParametricMaterialTranslucency TargetTranslucency = Private::GetTargetTranslucency(BlendModeFeature->BlendMode);

		FAvaShapeParametricMaterial* const ParametricMaterial = Shape->GetParametricMaterialPtr(MeshIndex);
		if (ParametricMaterial && ParametricMaterial->GetTranslucency() != TargetTranslucency && Shape->GetMaterialType(MeshIndex) == EMaterialType::Parametric)
		{
			ParametricMaterial->SetTranslucency(TargetTranslucency);
			SetMaterialInternal(Shape->GetMaterial(MeshIndex));
		}
		return true;
	}

	return false;
}

const UStruct* FShapeMaterialBridge::OnGetBridgedType() const
{
	return UAvaShapeDynamicMeshBase::StaticClass();
}

EControlFlow FShapeMaterialBridge::OnAccessSlots(const FReadSlotContext& InContext, TFunctionRef<EControlFlow(const FReadSlotContext&, const FReadSlot&)> InFunc, const FReadSlotOptions& InOptions) const
{
	const UAvaShapeDynamicMeshBase& Shape = InContext.MaterialContainer.Get<const UAvaShapeDynamicMeshBase>();

	for (const TPair<int32, FAvaShapeMeshData>& Pair : Shape.GetMeshDatas())
	{
		const FReadSlot Slot(Pair.Value.GetMaterial(), FSlotId(Pair.Key));
		if (InFunc(InContext, Slot) == EControlFlow::Break)
		{
			return EControlFlow::Break;
		}
	}

	return EControlFlow::Continue;
}

EControlFlow FShapeMaterialBridge::OnAccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const
{
	UAvaShapeDynamicMeshBase& Shape = InContext.MaterialContainer.GetMutable<UAvaShapeDynamicMeshBase>();

	const TSet<int32> MeshIndices = Shape.GetMeshesIndexes();

	for (int32 MeshIndex : MeshIndices)
	{
		if (FAvaShapeMeshData* MeshData = Shape.GetMeshData(MeshIndex))
		{
			FShapeMaterialWriteSlot Slot(&Shape, MeshIndex);
			if (InFunc(InContext, Slot) == EControlFlow::Break)
			{
				return EControlFlow::Break;
			}
		}
	}

	return EControlFlow::Continue;
}

TSubScriptStructOf<FAvaMaterialContainerState> FShapeMaterialBridge::OnGetContainerStateType() const
{
	return FContainerState::StaticStruct();
}

void FShapeMaterialBridge::OnApplyState(const FApplyStateContext& InContext, TConstStructView<FAvaMaterialContainerState> InContainerState, const FApplyStateOptions& InOptions) const
{
	UAvaShapeDynamicMeshBase& Shape = InContext.MaterialContainer.GetMutable<UAvaShapeDynamicMeshBase>();
	const FContainerState& ContainerState = InContainerState.Get<FContainerState>();

	if (ContainerState.Meshes.IsEmpty())
	{
		return;
	}

	for (const FAvaShapeMaterialContainerStateMesh& Mesh : ContainerState.Meshes)
	{
		if (Mesh.MaterialType == EMaterialType::Parametric)
		{
			if (FAvaShapeParametricMaterial* ParametricMaterial = Shape.GetParametricMaterialPtr(Mesh.Index))
			{
				*ParametricMaterial = Mesh.ParametricMaterial;
			}
		}
		Shape.SetMaterialType(Mesh.Index, Mesh.MaterialType);
		Shape.SetMaterial(Mesh.Index, Mesh.Material);
	}

	Shape.MarkRenderStateDirty();
}

void FShapeMaterialBridge::OnStoreState(const FStoreStateContext& InContext, TStructView<FAvaMaterialContainerState> InContainerState, const FStoreStateOptions& InOptions) const
{
	const UAvaShapeDynamicMeshBase& Shape = InContext.MaterialContainer.Get<const UAvaShapeDynamicMeshBase>();
	FContainerState& StateData = InContainerState.Get<FContainerState>();

	StateData.Meshes.Reset();

	for (int32 MeshIndex : Shape.GetMeshesIndexes())
	{
		if (const FAvaShapeMeshData* MeshData = Shape.GetMeshData(MeshIndex))
		{
			FAvaShapeMaterialContainerStateMesh& MeshState = StateData.Meshes.AddDefaulted_GetRef();
			MeshState.Index = MeshIndex;
			MeshState.Material = MeshData->GetMaterial();
			MeshState.MaterialType = MeshData->GetMaterialType();

			if (MeshState.MaterialType == EMaterialType::Parametric)
			{
				MeshState.ParametricMaterial = MeshData->GetParametricMaterial();	
			}
		}
	}
}

} // UE::Ava
