// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeMesh.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"

void FAvaShapeMesh::Clear()
{
	Vertices.Empty();
	Triangles.Empty();
	Normals.Empty();
	UVs.Empty();
	VertexColours.Empty();
	NewTriangleQueue.Empty();
}

void FAvaShapeMesh::ClearIds()
{
	TriangleIds.Empty();
	VerticeIds.Empty();
	UVIds.Empty();
	NormalIds.Empty();
	ColourIds.Empty();
}

void FAvaShapeMesh::AddTriangle(int32 A, int32 B, int32 C)
{
	Triangles.Add(A);
	Triangles.Add(B);
	Triangles.Add(C);
}

void FAvaShapeMesh::EnqueueTriangleIndex(int32 A)
{
	NewTriangleQueue.Add(A);

	if (NewTriangleQueue.Num() == 3)
	{
		if (NewTriangleQueue[0] != NewTriangleQueue[1]
			&& NewTriangleQueue[0] != NewTriangleQueue[2]
			&& NewTriangleQueue[1] != NewTriangleQueue[2])
		{
			Triangles.Add(NewTriangleQueue[0]);
			Triangles.Add(NewTriangleQueue[1]);
			Triangles.Add(NewTriangleQueue[2]);
		}
		else
		{
			UE_LOGF(LogTemp, Log, "FAvaShapeMesh: Bad Triangle! %d %d %d", NewTriangleQueue[0], NewTriangleQueue[1], NewTriangleQueue[2]);
		}

		NewTriangleQueue.Empty();
	}
}
