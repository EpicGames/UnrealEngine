// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeMeshComponentMaterialBridge.h"
#include "AvaShapeActor.h"
#include "Components/DynamicMeshComponent.h"

namespace UE::Ava
{

const UStruct* FShapeMeshComponentMaterialBridge::OnGetBridgedType() const
{
	return UDynamicMeshComponent::StaticClass();
}

bool FShapeMeshComponentMaterialBridge::OnIsMaterialContainerSupported(FConstDataView InMaterialContainer) const
{
	const UDynamicMeshComponent& MeshComponent = InMaterialContainer.Get<const UDynamicMeshComponent>();

	const AAvaShapeActor* ShapeActor = Cast<AAvaShapeActor>(MeshComponent.GetOuter());

	return ShapeActor && ShapeActor->GetShapeMeshComponent() == &MeshComponent;
}
} // UE::Ava
