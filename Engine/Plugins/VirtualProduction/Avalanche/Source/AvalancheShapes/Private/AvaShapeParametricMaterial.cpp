// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeParametricMaterial.h"

#include "Engine/Texture.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceDynamic.h"

FAvaShapeParametricMaterial::FOnMaterialChanged FAvaShapeParametricMaterial::OnMaterialChangedDelegate;
FAvaShapeParametricMaterial::FOnMaterialParameterChanged FAvaShapeParametricMaterial::OnMaterialParameterChangedDelegate;

const FName FAvaShapeParametricMaterial::StyleParameterName = TEXT("Style");
const FName FAvaShapeParametricMaterial::TextureParameterName = TEXT("Texture");
const FName FAvaShapeParametricMaterial::ColorAParameterName = TEXT("ColorA");
const FName FAvaShapeParametricMaterial::ColorBParameterName = TEXT("ColorB");
const FName FAvaShapeParametricMaterial::GradientOffsetParameterName = TEXT("GradientOffset");
const FName FAvaShapeParametricMaterial::GradientRotationParameterName = TEXT("GradientRotation");

FAvaShapeParametricMaterial::FAvaShapeParametricMaterial(const FAvaShapeParametricMaterial& Other)
{
	bUseUnlitMaterial        = Other.bUseUnlitMaterial;
	Translucency             = Other.Translucency;
	bUseTwoSidedMaterial     = Other.bUseTwoSidedMaterial;
	Texture                  = Other.Texture;
	ColorA                   = Other.ColorA;
	ColorB                   = Other.ColorB;
	GradientOffset           = Other.GradientOffset;
	Style                    = Other.Style;
	ActiveInstanceIndex      = Other.ActiveInstanceIndex;

	DefaultMaterials         = Other.DefaultMaterials;
	InstanceMaterials.SetNum(DefaultMaterials.Num(), EAllowShrinking::No);
}

FAvaShapeParametricMaterial& FAvaShapeParametricMaterial::operator=(const FAvaShapeParametricMaterial& Other)
{
	bUseUnlitMaterial        = Other.bUseUnlitMaterial;
	Translucency             = Other.Translucency;
	bUseTwoSidedMaterial     = Other.bUseTwoSidedMaterial;
	Texture                  = Other.Texture;
	ColorA                   = Other.ColorA;
	ColorB                   = Other.ColorB;
	GradientOffset           = Other.GradientOffset;
	Style                    = Other.Style;
	ActiveInstanceIndex      = Other.ActiveInstanceIndex;

	DefaultMaterials         = Other.DefaultMaterials;
	InstanceMaterials.SetNum(DefaultMaterials.Num(), EAllowShrinking::No);

	return *this;
}

UMaterialInterface* FAvaShapeParametricMaterial::GetDefaultMaterial() const
{
	LoadDefaultMaterials();

	const int32 Index = GetActiveInstanceIndex();
	check(DefaultMaterials.IsValidIndex(Index))
	return DefaultMaterials[Index];
}

void FAvaShapeParametricMaterial::LoadDefaultMaterials() const
{
	if (DefaultMaterials.Num() == MaterialTypeCount)
	{
		return;
	}

	FAvaShapeParametricMaterial* MutableThis = const_cast<FAvaShapeParametricMaterial*>(this);

	if (!MutableThis)
	{
		return;
	}

	MutableThis->DefaultMaterials.Empty(MaterialTypeCount);

	static const TCHAR* LitOpaqueMaterialPath = TEXT("/Avalanche/ToolboxResources/M_Toolbox_Opaque_Lit.M_Toolbox_Opaque_Lit");
	UMaterialInterface* DefaultLitMaterialOpaque = LoadObject<UMaterialInterface>(nullptr, LitOpaqueMaterialPath);
	MutableThis->DefaultMaterials.Add(DefaultLitMaterialOpaque);

	static const TCHAR* LitTranslucentMaterialPath = TEXT("/Avalanche/ToolboxResources/M_Toolbox_Translucent_Lit.M_Toolbox_Translucent_Lit");
	UMaterialInterface* DefaultLitMaterialTranslucent = LoadObject<UMaterialInterface>(nullptr, LitTranslucentMaterialPath);
	MutableThis->DefaultMaterials.Add(DefaultLitMaterialTranslucent);

	static const TCHAR* UnlitOpaqueMaterialPath = TEXT("/Avalanche/ToolboxResources/M_Toolbox_Opaque_Unlit.M_Toolbox_Opaque_Unlit");
	UMaterialInterface* DefaultUnlitMaterialOpaque = LoadObject<UMaterialInterface>(nullptr, UnlitOpaqueMaterialPath);
	MutableThis->DefaultMaterials.Add(DefaultUnlitMaterialOpaque);

	static const TCHAR* UnlitTranslucentMaterialPath = TEXT("/Avalanche/ToolboxResources/M_Toolbox_Translucent_Unlit.M_Toolbox_Translucent_Unlit");
	UMaterialInterface* DefaultUnlitMaterialTranslucent = LoadObject<UMaterialInterface>(nullptr, UnlitTranslucentMaterialPath);
	MutableThis->DefaultMaterials.Add(DefaultUnlitMaterialTranslucent);

	// As above, but one sided materials.
	static const TCHAR* LitOpaqueOneSidedMaterialPath = TEXT("/Avalanche/ToolboxResources/M_Toolbox_Opaque_Lit_Onesided.M_Toolbox_Opaque_Lit_Onesided");
	UMaterialInterface* DefaultLitOneSidedMaterialOpaque = LoadObject<UMaterialInterface>(nullptr, LitOpaqueOneSidedMaterialPath);
	MutableThis->DefaultMaterials.Add(DefaultLitOneSidedMaterialOpaque);

	static const TCHAR* LitTranslucentOneSidedMaterialPath = TEXT("/Avalanche/ToolboxResources/M_Toolbox_Translucent_Lit_Onesided.M_Toolbox_Translucent_Lit_Onesided");
	UMaterialInterface* DefaultLitOneSidedMaterialTranslucent = LoadObject<UMaterialInterface>(nullptr, LitTranslucentOneSidedMaterialPath);
	MutableThis->DefaultMaterials.Add(DefaultLitOneSidedMaterialTranslucent);

	static const TCHAR* UnlitOpaqueOneSidedMaterialPath = TEXT("/Avalanche/ToolboxResources/M_Toolbox_Opaque_Unlit_Onesided.M_Toolbox_Opaque_Unlit_Onesided");
	UMaterialInterface* DefaultUnlitOneSidedMaterialOpaque = LoadObject<UMaterialInterface>(nullptr, UnlitOpaqueOneSidedMaterialPath);
	MutableThis->DefaultMaterials.Add(DefaultUnlitOneSidedMaterialOpaque);

	static const TCHAR* UnlitTranslucentOneSidedMaterialPath = TEXT("/Avalanche/ToolboxResources/M_Toolbox_Translucent_Unlit_Onesided.M_Toolbox_Translucent_Unlit_Onesided");
	UMaterialInterface* DefaultUnlitOneSidedMaterialTranslucent = LoadObject<UMaterialInterface>(nullptr, UnlitTranslucentOneSidedMaterialPath);
	MutableThis->DefaultMaterials.Add(DefaultUnlitOneSidedMaterialTranslucent);

	MutableThis->InstanceMaterials.SetNum(DefaultMaterials.Num(), EAllowShrinking::No);
}

void FAvaShapeParametricMaterial::SetUseTwoSidedMaterial(bool bInUse)
{
	if (bInUse == bUseTwoSidedMaterial)
	{
		return;
	}

	bUseTwoSidedMaterial = bInUse;
	OnMaterialParameterUpdated();
}

void FAvaShapeParametricMaterial::SetMaterialParameterValues(UMaterialInstanceDynamic* InMaterialInstance, bool bInNotifyUpdate) const
{
	if (!IsValid(InMaterialInstance))
	{
		return;
	}

	InMaterialInstance->SetScalarParameterValue(FAvaShapeParametricMaterial::StyleParameterName, static_cast<float>(Style));

	switch (Style)
	{
		case EAvaShapeParametricMaterialStyle::Solid:
			InMaterialInstance->SetVectorParameterValue(FAvaShapeParametricMaterial::ColorAParameterName, ColorA);
		break;
		case EAvaShapeParametricMaterialStyle::LinearGradient:
			InMaterialInstance->SetVectorParameterValue(FAvaShapeParametricMaterial::ColorAParameterName, ColorA);
			InMaterialInstance->SetVectorParameterValue(FAvaShapeParametricMaterial::ColorBParameterName, ColorB);
			InMaterialInstance->SetScalarParameterValue(FAvaShapeParametricMaterial::GradientOffsetParameterName, GradientOffset);
			InMaterialInstance->SetScalarParameterValue(FAvaShapeParametricMaterial::GradientRotationParameterName, GradientRotation);
		break;
		case EAvaShapeParametricMaterialStyle::Texture:
			InMaterialInstance->SetTextureParameterValue(FAvaShapeParametricMaterial::TextureParameterName, Texture);
		break;
	}

	if (bInNotifyUpdate
		&& InstanceMaterials.IsValidIndex(ActiveInstanceIndex)
		&& InstanceMaterials[ActiveInstanceIndex] == InMaterialInstance)
	{
		if (OnMaterialParameterChangedDelegate.IsBound())
		{
			OnMaterialParameterChangedDelegate.Broadcast(*this);
		}
	}
}

void FAvaShapeParametricMaterial::PostEditChange(TConstArrayView<FName> InPropertyNames)
{
	static const TArray<FName> PropertyNames =
	{
		GET_MEMBER_NAME_CHECKED(FAvaShapeParametricMaterial, Style),
		GET_MEMBER_NAME_CHECKED(FAvaShapeParametricMaterial, Texture),
		GET_MEMBER_NAME_CHECKED(FAvaShapeParametricMaterial, ColorA),
		GET_MEMBER_NAME_CHECKED(FAvaShapeParametricMaterial, ColorB),
		GET_MEMBER_NAME_CHECKED(FAvaShapeParametricMaterial, GradientOffset),
		GET_MEMBER_NAME_CHECKED(FAvaShapeParametricMaterial, GradientRotation),
		GET_MEMBER_NAME_CHECKED(FAvaShapeParametricMaterial, bUseUnlitMaterial),
		GET_MEMBER_NAME_CHECKED(FAvaShapeParametricMaterial, bUseTwoSidedMaterial),
		GET_MEMBER_NAME_CHECKED(FAvaShapeParametricMaterial, Translucency),
	};

	const FName MemberPropertyName = !InPropertyNames.IsEmpty() ? InPropertyNames.Last() : NAME_None;

	if (PropertyNames.Contains(MemberPropertyName))
	{
		OnMaterialParameterUpdated();
	}
}

bool FAvaShapeParametricMaterial::ShouldUseTranslucentMaterial() const
{
	if (Translucency == EAvaShapeParametricMaterialTranslucency::Auto)
	{
		return ColorA.A < 1.f || ColorB.A < 1.f;
	}

	return Translucency == EAvaShapeParametricMaterialTranslucency::Enabled;
}

void FAvaShapeParametricMaterial::OnMaterialParameterUpdated()
{
	const int32 PreviousActiveIndex = ActiveInstanceIndex;
	const int32 NewActiveIndex = GetActiveInstanceIndex();

	if (PreviousActiveIndex != NewActiveIndex)
	{
		if (OnMaterialChangedDelegate.IsBound())
		{
			OnMaterialChangedDelegate.Broadcast(*this);
		}
	}

	for (const TObjectPtr<UMaterialInstanceDynamic>& Material : InstanceMaterials)
	{
		SetMaterialParameterValues(Material, true);
	}
}

int32 FAvaShapeParametricMaterial::GetActiveInstanceIndex() const
{
	FAvaShapeParametricMaterial* MutableThis = const_cast<FAvaShapeParametricMaterial*>(this);

	MutableThis->ActiveInstanceIndex = (ShouldUseTranslucentMaterial() ? Translucent : Opaque)
		+ (bUseUnlitMaterial ? Unlit : Lit)
		+ (bUseTwoSidedMaterial ? TwoSided : OneSided);

	return MutableThis->ActiveInstanceIndex;
}

UMaterialInstanceDynamic* FAvaShapeParametricMaterial::CreateMaterialInstance(UObject* InOuter)
{
	if (!IsValid(InOuter))
	{
		return nullptr;
	}

	UMaterialInterface* ParentMaterial = GetDefaultMaterial();

	if (!ParentMaterial)
	{
		return nullptr;
	}

	UMaterialInstanceDynamic* MaterialInstance = UMaterialInstanceDynamic::Create(
		ParentMaterial,
		InOuter
	);

	const int32 Index = GetActiveInstanceIndex();
	InstanceMaterials[Index] = MaterialInstance;

	OnMaterialParameterUpdated();

	return MaterialInstance;
}

bool FAvaShapeParametricMaterial::CopyFromMaterialParameters(UMaterialInstance* InMaterial)
{
	if (!IsParametricMaterial(InMaterial, true))
	{
		return false;
	}

	float InputStyle;
	if (InMaterial->GetScalarParameterValue(FMaterialParameterInfo(FAvaShapeParametricMaterial::StyleParameterName), InputStyle))
	{
		Style = static_cast<EAvaShapeParametricMaterialStyle>(FMath::TruncToInt(InputStyle));
	}

	FLinearColor InputColorA;
	if (InMaterial->GetVectorParameterValue(FMaterialParameterInfo(FAvaShapeParametricMaterial::ColorAParameterName), InputColorA))
	{
		ColorA = InputColorA;
	}

	FLinearColor InputColorB;
	if (InMaterial->GetVectorParameterValue(FMaterialParameterInfo(FAvaShapeParametricMaterial::ColorBParameterName), InputColorB))
	{
		ColorB = InputColorB;
	}

	float InputOffset;
	if (InMaterial->GetScalarParameterValue(FMaterialParameterInfo(FAvaShapeParametricMaterial::GradientOffsetParameterName), InputOffset))
	{
		GradientOffset = InputOffset;
	}

	UTexture* InputTexture;
	if (InMaterial->GetTextureParameterValue(FMaterialParameterInfo(FAvaShapeParametricMaterial::TextureParameterName), InputTexture))
	{
		Texture = InputTexture;
	}

	return true;
}

bool FAvaShapeParametricMaterial::IsParametricMaterial(UMaterialInterface* InMaterial, const bool bCheckIfDefault) const
{
	if (!InMaterial)
	{
		return false;
	}

	auto IsInstance = [this](const UMaterialInterface* Material)
	{
		return InstanceMaterials.Contains(Material);
	};

	auto IsDefault = [this](const UMaterialInterface* Material)
	{
		return DefaultMaterials.Contains(Material);
	};

	auto IsParametric = [IsInstance, IsDefault, bCheckIfDefault](const UMaterialInterface* Material)
	{
		return IsInstance(Material) || (bCheckIfDefault && IsDefault(Material));
	};

	// Direct comparison first
	if (IsParametric(InMaterial))
	{
		return true;
	}

	// Otherwise check parent
	const UMaterialInstance* AsMaterialInstance = Cast<UMaterialInstance>(InMaterial);
	if (IsValid(AsMaterialInstance) && IsValid(AsMaterialInstance->Parent))
	{
		UMaterialInterface* Parent = AsMaterialInstance->Parent;
		while (Parent)
		{
			if (IsParametric(Parent))
			{
				return true;
			}

			if (const UMaterialInstance* ParentAsMaterialInstance = Cast<UMaterialInstance>(Parent))
			{
				Parent = ParentAsMaterialInstance->Parent;
				continue;
			}

			break;
		}
	}

	return false;
}

UMaterialInstanceDynamic* FAvaShapeParametricMaterial::GetMaterial() const
{
	LoadDefaultMaterials();

	const int32 Index = GetActiveInstanceIndex();
	check(InstanceMaterials.IsValidIndex(Index))
	UMaterialInstanceDynamic* Material = InstanceMaterials[Index];

	SetMaterialParameterValues(Material, false);

	return Material;
}

UMaterialInstanceDynamic* FAvaShapeParametricMaterial::GetOrCreateMaterial(UObject* InOuter)
{
	UMaterialInstanceDynamic* Material = GetMaterial();

	/** If active instance material is not set, create and set it up */
	if (Material == nullptr)
	{
		Material = CreateMaterialInstance(InOuter);
	}

	return Material;
}

FAvaShapeParametricMaterial::FOnMaterialParameterChanged::RegistrationType& FAvaShapeParametricMaterial::OnMaterialParameterChanged()
{
	return OnMaterialParameterChangedDelegate;
}

FAvaShapeParametricMaterial::FOnMaterialChanged::RegistrationType& FAvaShapeParametricMaterial::OnMaterialChanged()
{
	return OnMaterialChangedDelegate;
}

void FAvaShapeParametricMaterial::SetMaterial(UMaterialInstanceDynamic* InMaterial)
{
	if (!IsValid(InMaterial))
	{
		return;
	}

	if (!IsParametricMaterial(InMaterial, false))
	{
		return;
	}

	const int32 ActiveIndex = GetActiveInstanceIndex();
	InstanceMaterials[ActiveIndex] = InMaterial;

	OnMaterialParameterUpdated();
}

void FAvaShapeParametricMaterial::SetTranslucency(EAvaShapeParametricMaterialTranslucency InTranslucency)
{
	if (InTranslucency == Translucency)
	{
		return;
	}

	Translucency = InTranslucency;
	OnMaterialParameterUpdated();
}

void FAvaShapeParametricMaterial::SetStyle(EAvaShapeParametricMaterialStyle InStyle)
{
	if (InStyle == Style)
	{
		return;
	}

	Style = InStyle;
	OnMaterialParameterUpdated();
}

void FAvaShapeParametricMaterial::SetTexture(UTexture* InTexture)
{
	if (Texture == InTexture)
	{
		return;
	}

	Texture = InTexture;
	OnMaterialParameterUpdated();
}

void FAvaShapeParametricMaterial::SetPrimaryColor(const FLinearColor& InColor)
{
	if (InColor.Equals(ColorA))
	{
		return;
	}

	ColorA = InColor;
	OnMaterialParameterUpdated();
}

void FAvaShapeParametricMaterial::SetSecondaryColor(const FLinearColor& InColor)
{
	if (InColor.Equals(ColorB))
	{
		return;
	}

	ColorB = InColor;
	OnMaterialParameterUpdated();
}

void FAvaShapeParametricMaterial::SetGradientOffset(float InOffset)
{
	if (FMath::IsNearlyEqual(InOffset, GradientOffset))
	{
		return;
	}

	GradientOffset = InOffset;
	OnMaterialParameterUpdated();
}

void FAvaShapeParametricMaterial::SetGradientRotation(float InRotation)
{
	if (FMath::IsNearlyEqual(InRotation, GradientRotation))
	{
		return;
	}

	GradientRotation = InRotation;
	OnMaterialParameterUpdated();
}

void FAvaShapeParametricMaterial::SetUseUnlitMaterial(bool bInUse)
{
	if (bInUse == bUseUnlitMaterial)
	{
		return;
	}

	bUseUnlitMaterial = bInUse;
	OnMaterialParameterUpdated();
}
