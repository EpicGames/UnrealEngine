// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapePrimitiveFunctions.h"

#include "AvaShapeActor.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "DynamicMeshes/AvaShapeRectangleDynMesh.h"

UAvaShapeRectangleDynamicMesh* UAvaShapeMeshFunctions::SetRectangle(
	AAvaShapeActor* ShapeActor
	, const FVector2D& Size
	, const FTransform& Transform)
{
	UAvaShapeRectangleDynamicMesh* Mesh = SetMesh<UAvaShapeRectangleDynamicMesh>(ShapeActor, Transform);

	if (!ensure(!Size.IsNearlyZero()))
	{
		return nullptr;
	}

	Mesh->SetSize2D(Size);

	return Mesh;
}

UAvaShapeDynamicMeshBase* UAvaShapeMeshFunctions::SetMesh(
	AAvaShapeActor* InShapeActor
	, const TSubclassOf<UAvaShapeDynamicMeshBase>& InMeshClass
	, const FTransform& InTransform)
{
	if (!ensure(InShapeActor))
	{
		return nullptr;
	}

	UAvaShapeDynamicMeshBase* Mesh = NewObject<UAvaShapeDynamicMeshBase>(InShapeActor, InMeshClass);
	if (!ensure(Mesh))
	{
		return nullptr;
	}

	InShapeActor->SetDynamicMesh(Mesh);

	return Mesh;
}
