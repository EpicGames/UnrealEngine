// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeSettings.h"

UAvaShapeSettings::UAvaShapeSettings()
	: ForceDisableShapeCollision(true)
{
	CategoryName = TEXT("Motion Design");
	SectionName  = TEXT("Shapes");
}

bool UAvaShapeSettings::ShouldForceDisableShapeCollision() const
{
	return ForceDisableShapeCollision.load(std::memory_order_relaxed);
}

void UAvaShapeSettings::PostInitProperties()
{
	Super::PostInitProperties();
	ForceDisableShapeCollision.store(bForceDisableShapeCollision, std::memory_order_relaxed);
}

#if WITH_EDITOR
void UAvaShapeSettings::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);
	ForceDisableShapeCollision.store(bForceDisableShapeCollision, std::memory_order_relaxed);
}
#endif
