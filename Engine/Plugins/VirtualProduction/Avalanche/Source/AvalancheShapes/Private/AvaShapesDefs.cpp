// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapesDefs.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"

namespace UE::AvaShapes
{
	FVector2D FindClosestPointOnLine(const FVector2D& LineStart, const FVector2D& LineEnd, const FVector2D& TestPoint)
	{
		const FVector2D LineVector = LineEnd - LineStart;

		const float A = -FVector2D::DotProduct(LineStart - TestPoint, LineVector);
		const float B = LineVector.SizeSquared();
		const float T = FMath::Clamp<float>(A / B, 0.0f, 1.0f);

		// Generate closest point
		return LineStart + T * LineVector;
	}

	bool TransformMeshUVs(UE::Geometry::FDynamicMesh3& InEditMesh, const TArray<int32>& UVIds
		, const FAvaShapeMaterialUVParameters& InParams, const FVector2D& InShapeSize, const FVector2D& InUVOffset
		, const float InUVFixRotation)
	{
		using namespace UE::Geometry;
		
		if (!InEditMesh.HasAttributes() || UVIds.IsEmpty())
		{
			return false;
		}

		FVector2D ShapeScale(1, 1);
		// in uniform: uv should stay 1:1 even if shape is not 1:1
		if (InParams.GetMode() == EAvaShapeUVMode::Uniform)
		{
			if (InShapeSize.X > InShapeSize.Y)
			{
				ShapeScale.Y = InShapeSize.Y / InShapeSize.X;
				ShapeScale.X = 1;
			}
			else
			{
				ShapeScale.X = InShapeSize.X / InShapeSize.Y;
				ShapeScale.Y = 1;
			}
		}
		ShapeScale *= InParams.GetScale();
		const float Rotation = InParams.GetRotation() + InUVFixRotation;
		FDynamicMeshUVOverlay* UVOverlay = InEditMesh.Attributes()->GetUVLayer(0);
		const FVector2f UseScaleOrigin(0, 0);
		const FMatrix2f RotationMat = FMatrix2f::RotationDeg(Rotation);
		// remove rotation fix added previously
		const FVector2D FinalAnchor = InParams.GetAnchor().GetRotated(-InUVFixRotation);
		const FVector2f UseRotateOrigin(FinalAnchor);
		for (int32 Idx = 0; Idx < UVIds.Num(); Idx++)
		{
			const int32 VId = UVIds[Idx];
			if (UVOverlay->IsElement(VId))
			{
				FVector2f UV(0);
				UV = UVOverlay->GetElement(VId);
				// flip
				UV.X *= InParams.GetFlipVertical() ? -1 : 1;
				UV.Y *= InParams.GetFlipHorizontal() ? -1 : 1;
				// scale
				UV = ((UV - UseScaleOrigin) * static_cast<FVector2f>(ShapeScale) + UseScaleOrigin);
				// translate
				UV = (UV + static_cast<FVector2f>(InParams.GetOffset())) + static_cast<FVector2f>(InUVOffset);
				// rotate
				UV = (RotationMat * (UV - UseRotateOrigin) + UseRotateOrigin);
				UVOverlay->SetElement(VId, UV);
			}
		}
		return true;
	}
}
