// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapesModule.h"
#include "AvaShapeMaterialBridge.h"
#include "AvaShapeMeshComponentMaterialBridge.h"
#include "MaterialBridge/AvaMaterialBridgeRegistry.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FAvaShapesModule, AvalancheShapes)

void FAvaShapesModule::StartupModule()
{
	UE::Ava::FMaterialBridgeRegistry& MaterialBridgeRegistry = UE::Ava::FMaterialBridgeRegistry::GetMutable();

	constexpr uint32 DefaultPriority = 0;
	MaterialBridgeRegistry.Register<UE::Ava::FShapeMaterialBridge>(DefaultPriority);

	// Higher than default to be prioritized over registered default material bridges of the same bridged type.
	constexpr uint32 HigherPriority = 100;
	MaterialBridgeRegistry.Register<UE::Ava::FShapeMeshComponentMaterialBridge>(HigherPriority);
}
