// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshes/AvaShape2DArrowDynMesh.h"

#include "AvaShapeVertices.h"

const FString UAvaShape2DArrowDynamicMesh::MeshName = TEXT("2DArrow");

void UAvaShape2DArrowDynamicMesh::SetRatioArrowLine(float InRatio)
{
	if (RatioArrowLine == InRatio)
	{
		return;
	}

	if (InRatio < 0.01f || InRatio > 1.f)
	{
		return;
	}

	RatioArrowLine = InRatio;
	OnRatioArrowLineChanged();
}

void UAvaShape2DArrowDynamicMesh::SetRatioLineHeight(float InRatio)
{
	if (RatioLineHeight == InRatio)
	{
		return;
	}

	if (InRatio < 0.01f || InRatio > 1.f)
	{
		return;
	}

	RatioLineHeight = InRatio;
	OnRatioLineHeightChanged();
}

void UAvaShape2DArrowDynamicMesh::SetRatioArrowY(float InRatio)
{
	if (RatioArrowY == InRatio)
	{
		return;
	}

	if (InRatio < 0.01f || InRatio > 1.f)
	{
		return;
	}

	RatioArrowY = InRatio;
	OnRatioArrowYChanged();
}

void UAvaShape2DArrowDynamicMesh::SetRatioLineY(float InRatio)
{
	if (RatioLineY == InRatio)
	{
		return;
	}

	if (InRatio < 0.01f || InRatio > 1.f)
	{
		return;
	}

	RatioLineY = InRatio;
	OnRatioLineYChanged();
}

void UAvaShape2DArrowDynamicMesh::SetBothSideArrows(bool bBothSide)
{
	if (bBothSideArrows == bBothSide)
	{
		return;
	}

	bBothSideArrows = bBothSide;
	OnBothSideArrowsChanged();
}

#if WITH_EDITOR
void UAvaShape2DArrowDynamicMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static FName RatioArrowLineName = GET_MEMBER_NAME_CHECKED(UAvaShape2DArrowDynamicMesh, RatioArrowLine);
	static FName RatioLineHeightName = GET_MEMBER_NAME_CHECKED(UAvaShape2DArrowDynamicMesh, RatioLineHeight);
	static FName RatioArrowYName = GET_MEMBER_NAME_CHECKED(UAvaShape2DArrowDynamicMesh, RatioArrowY);
	static FName RatioLineYName = GET_MEMBER_NAME_CHECKED(UAvaShape2DArrowDynamicMesh, RatioLineY);
	static FName BothSideArrowsName = GET_MEMBER_NAME_CHECKED(UAvaShape2DArrowDynamicMesh, bBothSideArrows);

	if (PropertyChangedEvent.MemberProperty->GetFName() == RatioArrowLineName)
	{
		OnRatioArrowLineChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == RatioLineHeightName)
	{
		OnRatioLineHeightChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == BothSideArrowsName)
	{
		OnBothSideArrowsChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == RatioArrowYName)
	{
		OnRatioArrowYChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == RatioLineYName)
	{
		OnRatioLineYChanged();
	}
}
#endif

void UAvaShape2DArrowDynamicMesh::OnRatioArrowLineChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShape2DArrowDynamicMesh::OnRatioLineHeightChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShape2DArrowDynamicMesh::OnBothSideArrowsChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShape2DArrowDynamicMesh::OnRatioArrowYChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShape2DArrowDynamicMesh::OnRatioLineYChanged()
{
	MarkAllMeshesDirty();
}

bool UAvaShape2DArrowDynamicMesh::IsMeshVisible(int32 MeshIndex)
{
	switch (MeshIndex)
	{
		case MESH_INDEX_PRIMARY:
			return RatioArrowLine > UE_KINDA_SMALL_NUMBER || RatioLineHeight > UE_KINDA_SMALL_NUMBER;
		default:
			return true;
	}
}

bool UAvaShape2DArrowDynamicMesh::CreateMesh(FAvaShapeMesh& InMesh)
{
	if (InMesh.GetMeshIndex() == MESH_INDEX_PRIMARY)
	{
		// Right Arrow

		const float BothSideRatio = bBothSideArrows ? 0.5f : 1.f;

		const float RatioArrow = RatioArrowLine * BothSideRatio;
		const float RatioLine = 1.f - RatioArrowLine;

		const float SXArrow = Size2D.X - (Size2D.X * RatioArrow);
		const float WXArrow = Size2D.X - SXArrow;
		// last point of arrows
		const float HYArrow = Size2D.Y * RatioArrowY;

		if (RatioArrow > 0.f)
		{
			AddVertexRaw(InMesh, FVector2D(SXArrow, Size2D.Y));
			AddVertexRaw(InMesh, FVector2D(SXArrow, 0));
			AddVertexRaw(InMesh, FVector2D(SXArrow + WXArrow, HYArrow));
		}

		// Line
		if (RatioLine > 0.f)
		{
			const float SXLine = bBothSideArrows ? WXArrow : 0;
			const float WXLine = Size2D.X * RatioLine;

			const float WYLine = (Size2D.Y * RatioLineHeight);
			const float SYLine = FMath::Clamp((RatioLineY * Size2D.Y) - WYLine / 2, 0, Size2D.Y - WYLine);

			const FAvaShapeCachedVertex2D P1 = CacheVertexCreate(InMesh, FVector2D(SXLine + WXLine, SYLine + WYLine));
			const FAvaShapeCachedVertex2D P2 = CacheVertexCreate(InMesh, FVector2D(SXLine, SYLine));

			AddVertexRaw(InMesh, FVector2D(SXLine, SYLine + WYLine));
			AddVertexRaw(InMesh, P2);
			AddVertexRaw(InMesh, P1);

			AddVertexRaw(InMesh, P1);
			AddVertexRaw(InMesh, P2);
			AddVertexRaw(InMesh, FVector2D(SXLine + WXLine, SYLine));
		}

		if (RatioArrow > 0.f && bBothSideArrows)
		{
			// Left Arrow
			AddVertexRaw(InMesh, FVector2D(WXArrow, Size2D.Y));
			AddVertexRaw(InMesh, FVector2D(0, HYArrow));
			AddVertexRaw(InMesh, FVector2D(WXArrow, 0));
		}
	}

	return Super::CreateMesh(InMesh);
}

bool UAvaShape2DArrowDynamicMesh::CanUpdateMeshInAnyThread() const
{
	return GetClass() == UAvaShape2DArrowDynamicMesh::StaticClass();
}
