// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshes/AvaShape2DDynMeshBase.h"
#include "AvaShapeActor.h"
#include "AvaShapeVertices.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMeshEditor.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "Operations/MinimalHoleFiller.h"
#include "Operations/OffsetMeshRegion.h"

void UAvaShape2DDynMeshBase::SetSize3D(const FVector& InSize)
{
	SetSize2D({InSize.Y, InSize.Z});
}

void UAvaShape2DDynMeshBase::PostLoad()
{
	Super::PostLoad();

	Size3D.Y = Size2D.X;
	Size3D.Z = Size2D.Y;

	PreEditSize2D = Size2D;
}

int32 UAvaShape2DDynMeshBase::AddVertexRaw(FAvaShapeMesh& InMesh, const FVector2D& Location, bool bForceNew)
{
	int32 VertexIndex = CacheVertex(InMesh, Location, bForceNew);
	InMesh.EnqueueTriangleIndex(VertexIndex);

	return VertexIndex;
}

FAvaShapeCachedVertex2D UAvaShape2DDynMeshBase::AddVertexCreate(FAvaShapeMesh& InMesh, const FVector2D& Location, bool bForceNew)
{
	return { Location, AddVertexRaw(InMesh, Location, bForceNew) };
}

bool UAvaShape2DDynMeshBase::AddVertex(FAvaShapeMesh& InMesh, const FAvaShapeCachedVertex2D& Vertex, bool bForceNew)
{
	if (Vertex.HasIndex())
	{
		return AddVertex(InMesh, Vertex.Index);
	}

	const_cast<FAvaShapeCachedVertex2D*>(&Vertex)->Index = AddVertexRaw(InMesh, Vertex.Location, bForceNew);

	return Vertex.HasIndex();
}

bool UAvaShape2DDynMeshBase::AddVertex(FAvaShapeMesh& InMesh, int32 VertexIndex)
{
	if (!InMesh.Vertices.IsValidIndex(VertexIndex))
	{
		return false;
	}

	InMesh.EnqueueTriangleIndex(VertexIndex);

	return true;
}

int32 UAvaShape2DDynMeshBase::CacheVertex(FAvaShapeMesh& InMesh,
	const FVector2D& Location, bool bForceNew)
{
	// All normals of 2d-shapes face the camera.
	static const FVector Normal(-1.f, 0.f, 0.f);

	FVector ToAdd = { 0.f, Location.X, Location.Y };

	// Center the mesh and convert from XY to YZ plane
	if (!bDoNotRecenterVertices)
	{
		ToAdd.Y -= Size2D.X / 2.f;
		ToAdd.Z -= Size2D.Y / 2.f;
	}

	int32 VertexIndex = INDEX_NONE;

	if (!bForceNew)
	{
		VertexIndex = InMesh.Vertices.Find(ToAdd);
	}

	if (VertexIndex == INDEX_NONE)
	{
		VertexIndex = InMesh.Vertices.Add(ToAdd);
		InMesh.Normals.Add(Normal);
	}

	return VertexIndex;
}

bool UAvaShape2DDynMeshBase::CacheVertex(FAvaShapeMesh& InMesh, const FAvaShapeCachedVertex2D& Vertex, bool bForceNew)
{
	if (Vertex.HasIndex())
	{
		return true;
	}

	const_cast<FAvaShapeCachedVertex2D*>(&Vertex)->Index = CacheVertex(InMesh, Vertex.Location, bForceNew);

	return Vertex.HasIndex();
}

FAvaShapeCachedVertex2D UAvaShape2DDynMeshBase::CacheVertexCreate(FAvaShapeMesh& InMesh, const FVector2D& Location, bool bForceNew)
{
	return { Location, CacheVertex(InMesh, Location, bForceNew) };
}

void UAvaShape2DDynMeshBase::AddTriangle(FAvaShapeMesh& InMesh, const FAvaShapeCachedVertex2D& A, const FAvaShapeCachedVertex2D& B, const FAvaShapeCachedVertex2D& C)
{
	CacheVertex(InMesh, A);
	CacheVertex(InMesh, B);
	CacheVertex(InMesh, C);
	AddTriangle(InMesh, A.Index, B.Index, C.Index);
}

void UAvaShape2DDynMeshBase::AddTriangle(FAvaShapeMesh& InMesh, int32 A, int32 B, int32 C)
{
	InMesh.AddTriangle(A, B, C);
}

bool UAvaShape2DDynMeshBase::CreateMesh(FAvaShapeMesh& InMesh)
{
	LocalSnapPoints.Empty();

	return Super::CreateMesh(InMesh);
}

void UAvaShape2DDynMeshBase::SetPixelSize2D(const FVector2D& InPixelSize2D)
{
	if (!bAllowEditSize)
	{
		return;
	}

	if (PixelSize2D == InPixelSize2D)
	{
		return;
	}

	if (InPixelSize2D.GetMin() < UAvaShapeDynamicMeshBase::MinSizeValue)
	{
		return;
	}

	PixelSize2D = InPixelSize2D;
	OnPixelSizeChanged();
}

// Special case. We can update more easily via c++ than Details panel!
void UAvaShape2DDynMeshBase::SetSize2D(const FVector2D& InSize2D)
{
	if (!bAllowEditSize)
	{
		return;
	}

	const FVector2D NewSize = FVector2D::Max(FVector2D(UAvaShapeDynamicMeshBase::MinSizeValue), InSize2D);
	if (Size2D.Equals(NewSize))
	{
		return;
	}

	Size2D = NewSize;
	OnSizeChanged();
}

void UAvaShape2DDynMeshBase::OnRegisteredMeshes()
{
	Super::OnRegisteredMeshes();

	PixelSize2D = Size2D;
	PreEditSize2D = Size2D;
}

void UAvaShape2DDynMeshBase::OnPixelSizeChanged()
{
	SetSize2D(PixelSize2D);
}

void UAvaShape2DDynMeshBase::OnScaledSizeChanged()
{
	if (GetShapeMeshComponent())
	{
		GetShapeMeshComponent()->SetRelativeScale3D({1.f, UniformScaledSize, UniformScaledSize});
	}
}

void UAvaShape2DDynMeshBase::OnSizeChanged()
{
	Size2D = FVector2D::Max(Size2D, UAvaShapeDynamicMeshBase::MinSize2D);

	PixelSize2D = Size2D;

	if (Size2D.Equals(PreEditSize2D))
	{
		return;
	}

	if (PreEditSize2D.IsZero())
	{
		PreEditSize2D = Size2D;
	}

	Size3D.Y = Size2D.X;
	Size3D.Z = Size2D.Y;

	const FVector2D Scale = Size2D / PreEditSize2D;
	PreEditSize2D = Size2D;

	ScaleVertices(Scale);
}

bool UAvaShape2DDynMeshBase::CreateUVs(FAvaShapeMesh& InMesh, FAvaShapeMaterialUVParameters& InParams)
{
	if (InMesh.GetMeshIndex() == MESH_INDEX_PRIMARY)
	{
		return ApplyUVsPlanarProjection(InMesh, FRotator(0, 90, 90), Size2D)
		&& ApplyUVsTransform(InMesh, InParams, Size2D, FVector2D(0.5, 0.5), 0.f);
	}
	return Super::CreateUVs(InMesh, InParams);
}

void UAvaShape2DDynMeshBase::GetBounds(FVector& Origin, FVector& BoxExtent, FVector& Pivot) const
{
	GetShapeActor()->GetActorBounds(false, Origin, BoxExtent, false);
	// Origin = GetShapeMeshComponent()->GetLocalBounds().Origin;
	// BoxExtent = GetShapeMeshComponent()->GetLocalBounds().BoxExtent;
	Pivot = FVector(-BoxExtent.X, 0, 0);
}

#if WITH_EDITOR
void UAvaShape2DDynMeshBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static const FName SizeName = GET_MEMBER_NAME_CHECKED(UAvaShape2DDynMeshBase, Size2D);
	static const FName PixelSizeName = GET_MEMBER_NAME_CHECKED(UAvaShape2DDynMeshBase, PixelSize2D);

	if (PropertyChangedEvent.MemberProperty->GetFName() == SizeName)
	{
		OnSizeChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == PixelSizeName)
	{
		OnPixelSizeChanged();
	}
}
#endif
