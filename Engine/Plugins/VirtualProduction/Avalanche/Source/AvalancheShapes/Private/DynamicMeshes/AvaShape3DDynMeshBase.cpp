// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshes/AvaShape3DDynMeshBase.h"
#include "Components/DynamicMeshComponent.h"
#include "AvaShapeActor.h"
#include "AvaShapeVertices.h"

int32 UAvaShape3DDynMeshBase::AddVertexRaw(FAvaShapeMesh& InMesh, const FVector& Location, const FVector& Normal, bool bAddToTriangle)
{
	int32 VertexIndex = InMesh.Vertices.Add(Location);
	InMesh.Normals.Add(Normal);

	if (VertexIndex != INDEX_NONE && bAddToTriangle)
	{
		InMesh.EnqueueTriangleIndex(VertexIndex);
	}

	return VertexIndex;
}

bool UAvaShape3DDynMeshBase::AddVertex(FAvaShapeMesh& InMesh, const FAvaShapeCachedVertex3D& Vertex)
{
	if (Vertex.HasIndex())
	{
		return AddVertex(InMesh, Vertex.Index);
	}

	const_cast<FAvaShapeCachedVertex3D*>(&Vertex)->Index = AddVertexRaw(InMesh, Vertex.Location, Vertex.Normal, true);

	return Vertex.HasIndex();
}

bool UAvaShape3DDynMeshBase::AddVertex(FAvaShapeMesh& InMesh, int32 VertexIndex)
{
	if (!InMesh.Vertices.IsValidIndex(VertexIndex))
	{
		return false;
	}

	InMesh.EnqueueTriangleIndex(VertexIndex);

	return true;
}

FAvaShapeCachedVertex3D UAvaShape3DDynMeshBase::CacheVertexCreate(FAvaShapeMesh& InMesh,
	const FVector& Location, const FVector& Normal, bool bAddToTriangle)
{
	return { Location, Normal, FVector2D(0,0), AddVertexRaw(InMesh, Location, Normal, bAddToTriangle)};
}

bool UAvaShape3DDynMeshBase::CacheVertex(FAvaShapeMesh& InMesh, const FAvaShapeCachedVertex3D& Vertex,
	bool bAddToTriangle)
{
	if (Vertex.HasIndex())
	{
		return true;
	}

	const_cast<FAvaShapeCachedVertex3D*>(&Vertex)->Index = AddVertexRaw(InMesh, Vertex.Location, Vertex.Normal, bAddToTriangle);

	return Vertex.HasIndex();
}

void UAvaShape3DDynMeshBase::AddTriangle(FAvaShapeMesh& InMesh, const FAvaShapeCachedVertex3D& A,
	const FAvaShapeCachedVertex3D& B, const FAvaShapeCachedVertex3D& C)
{
	if (CacheVertex(InMesh, A) && CacheVertex(InMesh, B) && CacheVertex(InMesh, C))
	{
		AddTriangle(InMesh, A.Index, B.Index, C.Index);
	}
}

void UAvaShape3DDynMeshBase::AddTriangle(FAvaShapeMesh& InMesh, int32 A, int32 B, int32 C)
{
	if (InMesh.Vertices.IsValidIndex(A) && InMesh.Vertices.IsValidIndex(B) && InMesh.Vertices.IsValidIndex(C))
	{
		InMesh.AddTriangle(A, B, C);
	}
}

void UAvaShape3DDynMeshBase::GetBounds(FVector& Origin, FVector& BoxExtent, FVector& Pivot) const
{
	GetShapeActor()->GetActorBounds(false, Origin, BoxExtent, false);
	// Origin = GetShapeMeshComponent()->GetLocalBounds().Origin;
	// BoxExtent = GetShapeMeshComponent()->GetLocalBounds().BoxExtent;
	Pivot = FVector(0, 0, 0);
}

void UAvaShape3DDynMeshBase::PostLoad()
{
	Super::PostLoad();

	PreEditSize3D = Size3D;
}

void UAvaShape3DDynMeshBase::SetPixelSize3D(const FVector& InPixelSize)
{
	if (!bAllowEditSize)
	{
		return;
	}

	if (PixelSize3D == InPixelSize)
	{
		return;
	}

	if (InPixelSize.GetMin() < UAvaShapeDynamicMeshBase::MinSizeValue)
	{
		return;
	}

	PixelSize3D = InPixelSize;
	OnPixelSizeChanged();
}

void UAvaShape3DDynMeshBase::SetSize3D(const FVector& InSize)
{
	if (!bAllowEditSize)
	{
		return;
	}

	const FVector NewSize = FVector::Max(FVector(UAvaShapeDynamicMeshBase::MinSizeValue), InSize);
	if (Size3D.Equals(NewSize))
	{
		return;
	}

	Size3D = NewSize;
	OnSizeChanged();
}

void UAvaShape3DDynMeshBase::OnRegisteredMeshes()
{
	Super::OnRegisteredMeshes();

	PixelSize3D = Size3D;
	PreEditSize3D = Size3D;
}

void UAvaShape3DDynMeshBase::OnPixelSizeChanged()
{
	SetSize3D(PixelSize3D);
}

void UAvaShape3DDynMeshBase::OnSizeChanged()
{
	Size3D = FVector::Max(Size3D, UAvaShapeDynamicMeshBase::MinSize3D);

	PixelSize3D = Size3D;

	if (Size3D == PreEditSize3D)
	{
		return;
	}

	if (PreEditSize3D.IsZero())
	{
		PreEditSize3D = Size3D;
	}

	const FVector Scale = Size3D / PreEditSize3D;
	PreEditSize3D = Size3D;

	ScaleVertices(Scale);
}

void UAvaShape3DDynMeshBase::OnScaledSizeChanged()
{
	if (GetShapeMeshComponent())
	{
		GetShapeMeshComponent()->SetRelativeScale3D({UniformScaledSize, UniformScaledSize, UniformScaledSize});
	}
}

#if WITH_EDITOR
void UAvaShape3DDynMeshBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static const FName SizeName = GET_MEMBER_NAME_CHECKED(UAvaShape3DDynMeshBase, Size3D);
	static const FName PixelSizeName = GET_MEMBER_NAME_CHECKED(UAvaShape3DDynMeshBase, PixelSize3D);

	if (PropertyChangedEvent.MemberProperty->GetFName() == SizeName)
	{
		OnSizeChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == PixelSizeName)
	{
		OnPixelSizeChanged();
	}
}
#endif
