// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshes/AvaShapeChevronDynMesh.h"

const FString UAvaShapeChevronDynamicMesh::MeshName = TEXT("Chevron");

#if WITH_EDITOR
void UAvaShapeChevronDynamicMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static FName RatioChevronName = GET_MEMBER_NAME_CHECKED(UAvaShapeChevronDynamicMesh, RatioChevron);

	if (PropertyChangedEvent.MemberProperty->GetFName() == RatioChevronName)
	{
		OnRatioChevronChanged();
	}

}
#endif

void UAvaShapeChevronDynamicMesh::SetRatioChevron(float InRatio)
{
	if (RatioChevron == InRatio)
	{
		return;
	}

	if (InRatio < 0.f || InRatio > 0.99f)
	{
		return;
	}

	RatioChevron = InRatio;
	OnRatioChevronChanged();
}

void UAvaShapeChevronDynamicMesh::OnRatioChevronChanged()
{
	MarkAllMeshesDirty();
}

bool UAvaShapeChevronDynamicMesh::IsMeshVisible(int32 MeshIndex)
{
	switch (MeshIndex)
	{
		case MESH_INDEX_PRIMARY:
			return RatioChevron > UE_KINDA_SMALL_NUMBER;
		default:
			return true;
	}
}

bool UAvaShapeChevronDynamicMesh::CreateMesh(FAvaShapeMesh& InMesh)
{
	if (InMesh.GetMeshIndex() == MESH_INDEX_PRIMARY)
	{
		AddVertexRaw(InMesh, FVector2D(0, 0));
		int32 BottomCenter = AddVertexRaw(InMesh, FVector2D(Size2D.X * RatioChevron, 0));
		int32 LeftCenter = AddVertexRaw(InMesh, FVector2D(Size2D.X * (1 - RatioChevron), Size2D.Y / 2));

		AddVertex(InMesh, LeftCenter);
		AddVertex(InMesh, BottomCenter);
		int32 RightCenter = AddVertexRaw(InMesh, FVector2D(Size2D.X, Size2D.Y / 2));

		AddVertex(InMesh, RightCenter);
		int32 TopCenter = AddVertexRaw(InMesh, FVector2D(Size2D.X * RatioChevron, Size2D.Y));
		AddVertex(InMesh, LeftCenter);

		AddVertex(InMesh, TopCenter);
		AddVertexRaw(InMesh, FVector2D(0, Size2D.Y));
		AddVertex(InMesh, LeftCenter);
	}

	return Super::CreateMesh(InMesh);
}

bool UAvaShapeChevronDynamicMesh::CanUpdateMeshInAnyThread() const
{
	return GetClass() == UAvaShapeChevronDynamicMesh::StaticClass();
}
