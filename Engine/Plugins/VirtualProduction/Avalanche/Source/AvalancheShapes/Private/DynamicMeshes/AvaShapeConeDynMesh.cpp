// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshes/AvaShapeConeDynMesh.h"

#include "AvaShapeVertices.h"
#include "Kismet/KismetMathLibrary.h"

const FString UAvaShapeConeDynamicMesh::MeshName = TEXT("Cone");

void UAvaShapeConeDynamicMesh::SetNumSides(uint8 InNumSides)
{
	if (NumSides == InNumSides)
	{
		return;
	}

	if (InNumSides < UAvaShapeConeDynamicMesh::MinNumSides || InNumSides > UAvaShapeConeDynamicMesh::MaxNumSides)
	{
		return;
	}

	NumSides = InNumSides;
	OnNumSidesChanged();
}

void UAvaShapeConeDynamicMesh::SetTopRadius(float InTopRadius)
{
	if (TopRadius == InTopRadius)
	{
		return;
	}

	if (InTopRadius < 0.f || InTopRadius > 1.f)
	{
		return;
	}

	TopRadius = InTopRadius;
	OnTopRadiusChanged();
}

void UAvaShapeConeDynamicMesh::SetAngleDegree(float InDegree)
{
	if (AngleDegree == InDegree)
	{
		return;
	}

	if (InDegree < 0.f || InDegree > 360.f)
	{
		return;
	}

	AngleDegree = InDegree;
	OnAngleDegreeChanged();
}

void UAvaShapeConeDynamicMesh::SetStartDegree(float InDegree)
{
	if (StartDegree == InDegree)
	{
		return;
	}

	if (InDegree < 0.f || InDegree > 360.f)
	{
		return;
	}

	StartDegree = InDegree;
	OnStartDegreeChanged();
}

#if WITH_EDITOR
void UAvaShapeConeDynamicMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static FName HeightName = GET_MEMBER_NAME_CHECKED(UAvaShapeConeDynamicMesh, Height);
	static FName BaseRadiusName = GET_MEMBER_NAME_CHECKED(UAvaShapeConeDynamicMesh, BaseRadius);
	static FName TopRadiusName = GET_MEMBER_NAME_CHECKED(UAvaShapeConeDynamicMesh, TopRadius);
	static FName NumSidesName = GET_MEMBER_NAME_CHECKED(UAvaShapeConeDynamicMesh, NumSides);
	static FName AngleDegreeName = GET_MEMBER_NAME_CHECKED(UAvaShapeConeDynamicMesh, AngleDegree);
	static FName StartDegreeName = GET_MEMBER_NAME_CHECKED(UAvaShapeConeDynamicMesh, StartDegree);

	if (PropertyChangedEvent.MemberProperty->GetFName() == NumSidesName)
	{
		OnNumSidesChanged();
	}

	else if (PropertyChangedEvent.MemberProperty->GetFName() == HeightName)
	{
		OnHeightChanged();
	}

	else if (PropertyChangedEvent.MemberProperty->GetFName() == BaseRadiusName)
	{
		OnBaseRadiusChanged();
	}

	else if (PropertyChangedEvent.MemberProperty->GetFName() == TopRadiusName)
	{
		OnTopRadiusChanged();
	}

	else if (PropertyChangedEvent.MemberProperty->GetFName() == AngleDegreeName)
	{
		OnAngleDegreeChanged();
	}

	else if (PropertyChangedEvent.MemberProperty->GetFName() == StartDegreeName)
	{
		OnStartDegreeChanged();
	}
}
#endif

void UAvaShapeConeDynamicMesh::RegisterMeshes()
{
	// Cone Mesh above the base mesh
	FAvaShapeMeshData BottomMesh(MESH_INDEX_BOTTOM, TEXT("Bottom"), true);
	// Mesh on top of the cone
	FAvaShapeMeshData TopMesh(MESH_INDEX_TOP, TEXT("Top"), true);
	// Mesh visible when angle degree is below 360 at the start
	FAvaShapeMeshData StartMesh(MESH_INDEX_START, TEXT("Start"), false);
	// Mesh visible when angle degree is below 360 at the end
	FAvaShapeMeshData EndMesh(MESH_INDEX_END, TEXT("End"), false);

	RegisterMesh(BottomMesh);
	RegisterMesh(TopMesh);
	RegisterMesh(StartMesh);
	RegisterMesh(EndMesh);

	Super::RegisterMeshes();
}

bool UAvaShapeConeDynamicMesh::IsMeshVisible(int32 MeshIndex)
{
	if (AngleDegree < UE_KINDA_SMALL_NUMBER)
	{
		return false;
	}

	switch (MeshIndex)
	{
		case(MESH_INDEX_TOP):
			return TopRadius > 0;
		case(MESH_INDEX_START):
			return AngleDegree < 360;
		case(MESH_INDEX_END):
			return AngleDegree < 360;
		default:
			return true;
	}
}

void UAvaShapeConeDynamicMesh::OnHeightChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShapeConeDynamicMesh::OnBaseRadiusChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShapeConeDynamicMesh::OnNumSidesChanged()
{
	NumSides = FMath::Clamp<uint8>(NumSides, UAvaShapeConeDynamicMesh::MinNumSides, UAvaShapeConeDynamicMesh::MaxNumSides);
	MarkAllMeshesDirty();
}

void UAvaShapeConeDynamicMesh::OnAngleDegreeChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShapeConeDynamicMesh::OnStartDegreeChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShapeConeDynamicMesh::OnTopRadiusChanged()
{
	MarkAllMeshesDirty();
}

bool UAvaShapeConeDynamicMesh::CreateBaseUVs(FAvaShapeMesh& BaseMesh, FAvaShapeMaterialUVParameters& InUseParams)
{
	const FVector2D Size(Size3D.X, Size3D.Y);
	return ApplyUVsPlanarProjection(BaseMesh, FRotator(180, 0, 0),  Size * BaseRadius * 2)
		&& ApplyUVsTransform(BaseMesh, InUseParams, Size, FVector2D(0.5, 0.5));
}

bool UAvaShapeConeDynamicMesh::GenerateBaseMeshSections(FAvaShapeMesh& BottomMesh)
{
	// base creation only
	static const FVector BaseNormal(0, 0, -1);
	const float Angle = AngleDegree / NumSides;
	// const float ScaleY = Size3D.Y / Size3D.X;
	const FVector Scale(1.f, Size3D.Y / Size3D.X, Size3D.Z / Size3D.X);
	// bottom base
	const float ConeHeight = Height * Size3D.X;
	const FVector InnerBase = FVector(0, 0, (-ConeHeight / 2));
	// bottom base radius
	const float BottomRadius = (Size3D.X * BaseRadius);
	const FVector OuterBase = FVector(InnerBase.X + BottomRadius, InnerBase.Y, InnerBase.Z);
	// center base vertice cached
	const FAvaShapeCachedVertex3D BaseVertexCached = CacheVertexCreate(BottomMesh, InnerBase * Scale, BaseNormal);
	// next vertice becomes current vertice
	int32 CurVertexIdx = -1;

	for (int32 i = 0; i < NumSides; i++)
	{
		/* base creation */
		const float CurSideAngle = StartDegree + (i * Angle);
		const float NextSideAngle = StartDegree + ((i + 1) * Angle);

		AddVertex(BottomMesh, BaseVertexCached);
		if (CurVertexIdx > -1)
		{
			AddVertex(BottomMesh, CurVertexIdx);
		}
		else
		{
			FVector CurVertice = OuterBase.RotateAngleAxis(CurSideAngle, FVector::ZAxisVector) * Scale;
			AddVertexRaw(BottomMesh, CurVertice, BaseNormal);
		}
		FVector NextVertice = OuterBase.RotateAngleAxis(NextSideAngle, FVector::ZAxisVector) * Scale;
		CurVertexIdx = AddVertexRaw(BottomMesh, NextVertice, BaseNormal);
	}

	return true;
}

bool UAvaShapeConeDynamicMesh::CreateTopUVs(FAvaShapeMesh& TopMesh, FAvaShapeMaterialUVParameters& InUseParams)
{
	const FVector2D Size(Size3D.X, Size3D.Y);
	return ApplyUVsPlanarProjection(TopMesh, FRotator(0), Size * (TopRadius * BaseRadius) * 2)
		&& ApplyUVsTransform(TopMesh, InUseParams, Size, FVector2D(0.5, 0.5));
}

bool UAvaShapeConeDynamicMesh::GenerateTopMeshSections(FAvaShapeMesh& TopMesh)
{
	if (TopRadius <= 0.f)
	{
		return true;
	}

	// top creation only
	static const FVector BaseNormal(0, 0, 1);
	const float Angle = AngleDegree / NumSides;
	// const float ScaleY = Size3D.Y / Size3D.X;
	const FVector Scale(1.f, Size3D.Y / Size3D.X, Size3D.Z / Size3D.X);
	// top base
	const float ConeHeight = Height * Size3D.X;
	const FVector InnerBase = FVector(0, 0, (ConeHeight / 2));
	// top base radius
	const float Radius = (Size3D.X * (TopRadius * BaseRadius));
	const FVector OuterBase = FVector(InnerBase.X + Radius, InnerBase.Y, InnerBase.Z);
	// center base vertice cached
	const FAvaShapeCachedVertex3D BaseVertexCached = CacheVertexCreate(TopMesh, InnerBase * Scale, BaseNormal);
	// next vertice becomes current vertice
	int32 CurVertexIdx = -1;

	for (int32 i = 0; i < NumSides; i++)
	{
		/* top creation */
		const float CurSideAngle = StartDegree + (i * Angle);
		const float NextSideAngle = StartDegree + ((i + 1) * Angle);

		if (CurVertexIdx > -1)
		{
			AddVertex(TopMesh, CurVertexIdx);
		}
		else
		{
			FVector CurVertice = OuterBase.RotateAngleAxis(CurSideAngle, FVector::ZAxisVector) * Scale;
			AddVertexRaw(TopMesh, CurVertice, BaseNormal);
		}
		AddVertex(TopMesh, BaseVertexCached);
		FVector NextVertice = OuterBase.RotateAngleAxis(NextSideAngle, FVector::ZAxisVector) * Scale;
		CurVertexIdx = AddVertexRaw(TopMesh, NextVertice, BaseNormal);
	}

	return true;
}

bool UAvaShapeConeDynamicMesh::GenerateConeMeshSections(FAvaShapeMesh& ConeMesh)
{
	const float Angle = AngleDegree / NumSides;
	// const float ScaleY = Size3D.Y / Size3D.X;
	const FVector Scale(1.f, Size3D.Y / Size3D.X, Size3D.Z / Size3D.X);

	const float ConeHeight = Height * Size3D.X;
	const FVector InnerBase(0, 0, (-ConeHeight / 2));

	const float BottomRadius = (Size3D.X * BaseRadius);
	const FVector OuterBase = FVector(InnerBase.X + BottomRadius, InnerBase.Y, InnerBase.Z);

	const float CapRadius = (Size3D.X * (TopRadius * BaseRadius));
	const FVector TopBase(InnerBase.X + CapRadius, InnerBase.Y, InnerBase.Z + ConeHeight);

	// compute normal based on slope cone
	const FVector BottomTop = TopBase - OuterBase;
	FRotator Rot = FRotationMatrix::MakeFromX(BottomTop).Rotator();
	const FVector Normal = FVector::XAxisVector.RotateAngleAxis(Rot.Pitch - 90.f, FVector::YAxisVector);

	// next vertice becomes current vertice
	int32 CurVertexIdx = -1;

	TArray<int32> CachedVertexIdx;

	for (int32 i = 0; i < NumSides; i++)
	{
		/* base to top creation */
		const float CurSideAngle = StartDegree + (i * Angle);
		const float NextSideAngle = StartDegree + ((i + 1) * Angle);

		if (CurVertexIdx > -1)
		{
			AddVertex(ConeMesh, CurVertexIdx);
		}
		else
		{
			// current vertice
			FVector CurVertice = OuterBase.RotateAngleAxis(CurSideAngle, FVector::ZAxisVector) * Scale;
			FVector CurNormal = Normal.RotateAngleAxis(CurSideAngle, FVector::ZAxisVector);
			AddVertexRaw(ConeMesh, CurVertice, CurNormal);
		}
		// top vertice
		FVector TopVertice = TopBase.RotateAngleAxis(CurSideAngle, FVector::ZAxisVector) * Scale;
		FVector TopNormal = Normal.RotateAngleAxis(CurSideAngle, FVector::ZAxisVector);
		CachedVertexIdx.Add(AddVertexRaw(ConeMesh, TopVertice, TopNormal));
		// next vertice
		if (i != NumSides)
		{
			FVector NextVertice = OuterBase.RotateAngleAxis(NextSideAngle, FVector::ZAxisVector) * Scale;
			FVector NextNormal = Normal.RotateAngleAxis(NextSideAngle, FVector::ZAxisVector);
			CurVertexIdx = AddVertexRaw(ConeMesh, NextVertice, NextNormal);
			CachedVertexIdx.Add(CurVertexIdx);
		}
		else
		{
			CurVertexIdx = 0;
			AddVertex(ConeMesh, CurVertexIdx);
		}

	}

	// Complete tris to close the gaps when TopRadius > 0
	if (TopRadius > 0 && CachedVertexIdx.Num() >= 2)
	{
		for (int32 i = 2; i < CachedVertexIdx.Num(); i += 2)
		{
			AddVertex(ConeMesh, CachedVertexIdx[i - 1]);
			AddVertex(ConeMesh, CachedVertexIdx[i - 2]);
			AddVertex(ConeMesh, CachedVertexIdx[i]);
		}

		const float FinalSideAngle = StartDegree + (NumSides * Angle);

		FVector TopVertice = TopBase.RotateAngleAxis(FinalSideAngle, FVector::ZAxisVector) * Scale;
		FVector TopNormal = Normal.RotateAngleAxis(FinalSideAngle, FVector::ZAxisVector);
		AddVertexRaw(ConeMesh, TopVertice, TopNormal);
		FVector CurVertice = OuterBase.RotateAngleAxis(FinalSideAngle, FVector::ZAxisVector) * Scale;
		FVector CurNormal = Normal.RotateAngleAxis(FinalSideAngle, FVector::ZAxisVector);
		AddVertexRaw(ConeMesh, CurVertice, CurNormal);
		int32 Last = CachedVertexIdx[CachedVertexIdx.Num() - 2];
		AddVertex(ConeMesh, Last);
	}

	return true;
}

bool UAvaShapeConeDynamicMesh::CreateStartUVs(FAvaShapeMesh& StartMesh, FAvaShapeMaterialUVParameters& InUseParams)
{
	// find rotation angle for plane
	const FVector XAxis = FVector(1, 0, 0).RotateAngleAxis(StartDegree, FVector::ZAxisVector);
	FRotator Rot = UKismetMathLibrary::MakeRotFromX(XAxis);
	Rot.Roll = 90;
	const float HSize = FMath::Sqrt(FMath::Pow(XAxis.X * Size3D.X, 2) + FMath::Pow(XAxis.Y * Size3D.Y, 2));

	const FVector2D Size(HSize * BaseRadius, Size3D.Z * Height);
	return ApplyUVsPlanarProjection(StartMesh, Rot, Size)
		&& ApplyUVsTransform(StartMesh, InUseParams, Size, FVector2D(0, 0.5));
}

bool UAvaShapeConeDynamicMesh::GenerateStartMeshSections(FAvaShapeMesh& StartMesh)
{
	if (AngleDegree >= 360.f)
	{
		return true;
	}

	const float StartAngle = StartDegree;
	const FVector StartNormal = FVector(0, -1, 0).RotateAngleAxis(StartAngle, FVector::ZAxisVector);
	const FVector Scale(1.f, Size3D.Y / Size3D.X, Size3D.Z / Size3D.X);

	const float ConeHeight = Height * Size3D.X;
	const FVector InnerBase(0, 0, (-ConeHeight / 2));

	const float BottomRadius = (Size3D.X * BaseRadius);
	const FVector OuterBase = FVector(InnerBase.X + BottomRadius, InnerBase.Y, InnerBase.Z).RotateAngleAxis(StartAngle, FVector::ZAxisVector);

	const float CapRadius = (Size3D.X * (TopRadius * BaseRadius));
	const FVector TopOuterBase = FVector(InnerBase.X + CapRadius, InnerBase.Y, InnerBase.Z + ConeHeight).RotateAngleAxis(StartAngle, FVector::ZAxisVector);
	const FVector TopInnerBase(0, 0, (ConeHeight / 2));

	int32 TopInnerBaseIdx = AddVertexRaw(StartMesh, TopInnerBase * Scale, StartNormal);
	int32 OuterBaseIdx = AddVertexRaw(StartMesh, OuterBase * Scale, StartNormal);
	AddVertexRaw(StartMesh, InnerBase * Scale, StartNormal);

	if (TopRadius > 0.f)
	{
		AddVertexRaw(StartMesh, TopOuterBase * Scale, StartNormal);
		AddVertex(StartMesh, OuterBaseIdx);
		AddVertex(StartMesh, TopInnerBaseIdx);
	}

	return true;
}

bool UAvaShapeConeDynamicMesh::CreateEndUVs(FAvaShapeMesh& EndMesh, FAvaShapeMaterialUVParameters& InUseParams)
{
	// find rotation angle for plane
	const FVector XAxis = FVector(1, 0, 0).RotateAngleAxis(StartDegree + AngleDegree, FVector::ZAxisVector);
	FRotator Rot = UKismetMathLibrary::MakeRotFromX(XAxis);
	Rot.Roll = 90;
	const float HSize = FMath::Sqrt(FMath::Pow(XAxis.X * Size3D.X, 2) + FMath::Pow(XAxis.Y * Size3D.Y, 2));

	const FVector2D Size(HSize * BaseRadius, Size3D.Z * Height);
	return ApplyUVsPlanarProjection(EndMesh, Rot, Size)
		&& ApplyUVsTransform(EndMesh, InUseParams, Size, FVector2D(0, 0.5));
}

bool UAvaShapeConeDynamicMesh::GenerateEndMeshSections(FAvaShapeMesh& EndMesh)
{
	if (AngleDegree >= 360.f)
	{
		return true;
	}

	const float EndAngle = StartDegree + AngleDegree;
	const FVector EndNormal = FVector(0, 1, 0).RotateAngleAxis(EndAngle, FVector::ZAxisVector);
	const FVector Scale(1.f, Size3D.Y / Size3D.X, Size3D.Z / Size3D.X);

	const float ConeHeight = Height * Size3D.X;
	const FVector InnerBase(0, 0, (-ConeHeight / 2));

	const float BottomRadius = (Size3D.X * BaseRadius);
	const FVector OuterBase = FVector(InnerBase.X + BottomRadius, InnerBase.Y, InnerBase.Z).RotateAngleAxis(EndAngle, FVector::ZAxisVector);

	const float CapRadius = (Size3D.X * (TopRadius * BaseRadius));
	const FVector TopOuterBase = FVector(InnerBase.X + CapRadius, InnerBase.Y, InnerBase.Z + ConeHeight).RotateAngleAxis(EndAngle, FVector::ZAxisVector);
	const FVector TopInnerBase(0, 0, (ConeHeight / 2));

	int32 OuterBaseIdx = AddVertexRaw(EndMesh, OuterBase * Scale, EndNormal);
	int32 TopInnerBaseIdx = AddVertexRaw(EndMesh, TopInnerBase * Scale, EndNormal);
	AddVertexRaw(EndMesh, InnerBase * Scale, EndNormal);

	if (TopRadius > 0.f)
	{
		AddVertex(EndMesh, OuterBaseIdx);
		AddVertexRaw(EndMesh, TopOuterBase * Scale, EndNormal);
		AddVertex(EndMesh, TopInnerBaseIdx);
	}

	return true;
}

bool UAvaShapeConeDynamicMesh::CreateConeUVs(FAvaShapeMesh& ConeMesh, FAvaShapeMaterialUVParameters& InUseParams)
{
	const int32 Sides = NumSides + (TopRadius > 0 ? 1 : 0);
	const float Fraction = -1.f / Sides;
	ConeMesh.UVs.Add(FVector2D(Fraction * 0, 1.f));
	for (int32 i = 0; i < Sides; i++)
	{
		ConeMesh.UVs.Add((FVector2D(Fraction * i, 0.f)));
		ConeMesh.UVs.Add((FVector2D(Fraction * (i + 1), 1.f)));
	}

	const float HSize = Height * Size3D.Z;
	const float WSize = 2 * PI * (BaseRadius * Size3D.X / 2);

	return ApplyUVsManually(ConeMesh)
		&& ApplyUVsTransform(ConeMesh, InUseParams, FVector2D(WSize, HSize), FVector2D(0));
}

bool UAvaShapeConeDynamicMesh::CreateUVs(FAvaShapeMesh& InMesh, FAvaShapeMaterialUVParameters& InParams)
{
	if (InMesh.Vertices.Num() == 0)
	{
		return false;
	}

	switch (InMesh.GetMeshIndex())
	{
		case MESH_INDEX_PRIMARY:
			return CreateConeUVs(InMesh, InParams);
		case MESH_INDEX_BOTTOM:
			return CreateBaseUVs(InMesh, InParams);
		case MESH_INDEX_TOP:
			return CreateTopUVs(InMesh, InParams);
		case MESH_INDEX_START:
			return CreateStartUVs(InMesh, InParams);
		case MESH_INDEX_END:
			return CreateEndUVs(InMesh, InParams);
		default: ;
	}

	return Super::CreateUVs(InMesh, InParams);
}

bool UAvaShapeConeDynamicMesh::CanUpdateMeshInAnyThread() const
{
	return GetClass() == UAvaShapeConeDynamicMesh::StaticClass();
}

bool UAvaShapeConeDynamicMesh::CreateMesh(FAvaShapeMesh& InMesh)
{
	if (!Super::CreateMesh(InMesh))
	{
		return false;
	}

	switch (InMesh.GetMeshIndex())
	{
		case MESH_INDEX_PRIMARY:
			return GenerateConeMeshSections(InMesh);
		case MESH_INDEX_BOTTOM:
			return GenerateBaseMeshSections(InMesh);
		case MESH_INDEX_TOP:
			return GenerateTopMeshSections(InMesh);
		case MESH_INDEX_START:
			return GenerateStartMeshSections(InMesh);
		case MESH_INDEX_END:
			return GenerateEndMeshSections(InMesh);
		default: ;
	}

	return true;
}


