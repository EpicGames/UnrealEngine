// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshes/AvaShapeCubeDynMesh.h"

#include "AvaShapeVertices.h"
#include "Kismet/KismetMathLibrary.h"
#include "Kismet/KismetSystemLibrary.h"

const FString UAvaShapeCubeDynamicMesh::MeshName = TEXT("Cube");

void UAvaShapeCubeDynamicMesh::SetSegment(float InSegment)
{
	if (Segment == InSegment)
	{
		return;
	}

	if (InSegment < 0.01f)
	{
		return;
	}

	Segment = InSegment;
	OnSegmentChanged();
}

void UAvaShapeCubeDynamicMesh::SetBevelSizeRatio(float InBevel)
{
	if (BevelSizeRatio == InBevel)
	{
		return;
	}

	if (InBevel < 0 || InBevel > GetMaxBevelSize())
	{
		return;
	}

	BevelSizeRatio = InBevel;
	OnBevelSizeChanged();
}

void UAvaShapeCubeDynamicMesh::SetBevelNum(uint8 InBevel)
{
	if (BevelNum == InBevel)
	{
		return;
	}

	if (InBevel < UAvaShapeCubeDynamicMesh::MinBevelNum || InBevel > UAvaShapeCubeDynamicMesh::MaxBevelNum)
	{
		return;
	}

	BevelNum = InBevel;
	OnBevelNumChanged();
}

#if WITH_EDITOR
void UAvaShapeCubeDynamicMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static FName SegmentName = GET_MEMBER_NAME_CHECKED(UAvaShapeCubeDynamicMesh, Segment);
	static FName BevelSizeName = GET_MEMBER_NAME_CHECKED(UAvaShapeCubeDynamicMesh, BevelSizeRatio);
	static FName BevelNumName = GET_MEMBER_NAME_CHECKED(UAvaShapeCubeDynamicMesh, BevelNum);

	if (PropertyChangedEvent.MemberProperty->GetFName() == SegmentName)
	{
		OnSegmentChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == BevelSizeName)
	{
		OnBevelSizeChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == BevelNumName)
	{
		OnBevelNumChanged();
	}
}
#endif

void UAvaShapeCubeDynamicMesh::RegisterMeshes()
{
	// Mesh on top of the cube
	FAvaShapeMeshData TopMesh(MESH_INDEX_TOP, TEXT("Top"), true);
	// Mesh on bottom of the cube
	FAvaShapeMeshData BottomMesh(MESH_INDEX_BOTTOM, TEXT("Bottom"), true);
	// Mesh on back of the cube
	FAvaShapeMeshData BackMesh(MESH_INDEX_BACK, TEXT("Back"), true);
	// Mesh on left of the cube
	FAvaShapeMeshData LeftMesh(MESH_INDEX_LEFT, TEXT("Left"), true);
	// Mesh on right of the cube
	FAvaShapeMeshData RightMesh(MESH_INDEX_RIGHT, TEXT("Right"), true);

	RegisterMesh(TopMesh);
	RegisterMesh(BottomMesh);
	RegisterMesh(BackMesh);
	RegisterMesh(LeftMesh);
	RegisterMesh(RightMesh);

	Super::RegisterMeshes();
}

bool UAvaShapeCubeDynamicMesh::IsMeshVisible(int32 MeshIndex)
{
	// everything is visible
	return true;
}

void UAvaShapeCubeDynamicMesh::OnSizeChanged()
{
	MarkAllMeshesDirty();
}

bool UAvaShapeCubeDynamicMesh::CanUpdateMeshInAnyThread() const
{
	return GetClass() == UAvaShapeCubeDynamicMesh::StaticClass();
}

void UAvaShapeCubeDynamicMesh::OnSegmentChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShapeCubeDynamicMesh::OnBevelSizeChanged()
{
	if (BevelSizeRatio > GetMaxBevelSize())
	{
		BevelSizeRatio = GetMaxBevelSize();
	}
	MarkAllMeshesDirty();
}

void UAvaShapeCubeDynamicMesh::OnBevelNumChanged()
{
	BevelNum = FMath::Clamp<uint8>(BevelNum, UAvaShapeCubeDynamicMesh::MinBevelNum, UAvaShapeCubeDynamicMesh::MaxBevelNum);
	if (BevelSizeRatio > 0)
	{
		MarkAllMeshesDirty();
	}
}

float UAvaShapeCubeDynamicMesh::GetMaxBevelSize() const
{
	return FMath::Min(FMath::Min(Size3D.X / 2, Size3D.Y / 2), Size3D.Z / 2) - 0.1f;
}

FVector UAvaShapeCubeDynamicMesh::GetBevelSize() const
{
	const float MinSegment = FMath::Min(GetMaxBevelSize(), BevelSizeRatio);
	return FVector( MinSegment, MinSegment, MinSegment);
}

FVector UAvaShapeCubeDynamicMesh::GetSegmentSize() const
{
	return FVector(Segment * (Size3D.X / 2), Segment * (Size3D.Y / 2), Segment * (Size3D.Z / 2));
}

FVector UAvaShapeCubeDynamicMesh::GetScaleSize() const
{
	return FVector::OneVector;
}

void UAvaShapeCubeDynamicMesh::CreateBevelCorner(FAvaShapeMesh& InMesh, const FVector& CornerStartLoc, const FVector& CornerEndLoc, const FVector& StartNormal,
	int32 PrevStartIdx, int32 PrevEndIdx, const FVector& BevelSize, const FVector& Scale, const FVector& CornerStartVtxLoc, const FVector& CornerEndVtxLoc)
{
	const FVector Axis = UKismetMathLibrary::GetDirectionUnitVector(CornerStartLoc, CornerEndLoc);
	const float AngleRot = 45.f / BevelNum;
	// find center of corner sphere (start & end)
	const FVector CornerStart = CornerStartLoc - (StartNormal * BevelSize);
	const FVector CornerEnd = CornerEndLoc - (StartNormal * BevelSize);

	int32 NextEndIdx(-1);
	int32 NextStartIdx(-1);

	for (int32 CurBevel = 0; CurBevel < BevelNum; CurBevel++)
	{
		const FVector NextNormal = StartNormal.RotateAngleAxis(AngleRot * (CurBevel + 1), Axis);

		AddVertex(InMesh, PrevStartIdx);
		AddVertex(InMesh, PrevEndIdx);
		NextEndIdx = AddVertexRaw(InMesh, (CornerEnd + (NextNormal * BevelSize)) * Scale, NextNormal);

		AddVertex(InMesh, PrevStartIdx);
		AddVertex(InMesh, NextEndIdx);
		NextStartIdx = AddVertexRaw(InMesh, (CornerStart + (NextNormal * BevelSize)) * Scale, NextNormal);

		const FVector CurNormal = StartNormal.RotateAngleAxis(AngleRot * CurBevel, Axis);

		// start corner direction
		FVector CornerStartNormal = UKismetMathLibrary::GetDirectionUnitVector(CornerStart, CornerStartVtxLoc);
		FVector CenterStartLoc = (CornerStart + (CornerStartNormal * BevelSize));

		FVector CurStartLoc = CornerStart + (CurNormal * BevelSize);
		FVector NextStartLoc = CornerStart + (NextNormal * BevelSize);

		int32 PrevStartSideIdx(PrevStartIdx);
		int32 NextStartSideIdx(NextStartIdx);

		for (int32 SideBevel = 1; SideBevel < BevelNum; SideBevel++)
		{
			FVector CurLoc = UKismetMathLibrary::VLerp(CurStartLoc, CenterStartLoc, (1.0 * SideBevel) / (BevelNum * 1.0));
			FVector CurSideBevelNormal = UKismetMathLibrary::GetDirectionUnitVector(CornerStart, CurLoc);

			AddVertex(InMesh, PrevStartSideIdx);
			AddVertex(InMesh, NextStartSideIdx);
			PrevStartSideIdx = AddVertexRaw(InMesh, (CornerStart + (CurSideBevelNormal * BevelSize)) * Scale, CurSideBevelNormal);

			FVector NextLoc = UKismetMathLibrary::VLerp(NextStartLoc, CenterStartLoc, (1.0 * SideBevel) / (BevelNum * 1.0));
			FVector NextSideBevelNormal = UKismetMathLibrary::GetDirectionUnitVector(CornerStart, NextLoc);

			AddVertex(InMesh, PrevStartSideIdx);
			AddVertex(InMesh, NextStartSideIdx);
			NextStartSideIdx = AddVertexRaw(InMesh, (CornerStart + (NextSideBevelNormal * BevelSize)) * Scale, NextSideBevelNormal);
		}

		// close start
		AddVertex(InMesh, PrevStartSideIdx);
		AddVertex(InMesh, NextStartSideIdx);
		AddVertexRaw(InMesh, CenterStartLoc * Scale, CornerStartNormal);

		// end corner direction
		FVector CornerEndNormal = UKismetMathLibrary::GetDirectionUnitVector(CornerEnd, CornerEndVtxLoc);
		FVector CenterEndLoc = (CornerEnd + (CornerEndNormal * BevelSize));

		FVector CurEndLoc = CornerEnd + (CurNormal * BevelSize);
		FVector NextEndLoc = CornerEnd + (NextNormal * BevelSize);

		int32 PrevEndSideIdx(PrevEndIdx);
		int32 NextEndSideIdx(NextEndIdx);

		for (int32 SideBevel = 1; SideBevel < BevelNum; SideBevel++)
		{
			FVector CurLoc = UKismetMathLibrary::VLerp(CurEndLoc, CenterEndLoc, (1.0 * SideBevel) / (BevelNum * 1.0));
			FVector CurSideBevelNormal = UKismetMathLibrary::GetDirectionUnitVector(CornerEnd, CurLoc);

			AddVertex(InMesh, NextEndSideIdx);
			AddVertex(InMesh, PrevEndSideIdx);
			PrevEndSideIdx = AddVertexRaw(InMesh, (CornerEnd + CurSideBevelNormal * BevelSize) * Scale, CurSideBevelNormal);

			FVector NextLoc = UKismetMathLibrary::VLerp(NextEndLoc, CenterEndLoc, (1.0 * SideBevel) / (BevelNum * 1.0));
			FVector NextSideBevelNormal = UKismetMathLibrary::GetDirectionUnitVector(CornerEnd, NextLoc);

			AddVertex(InMesh, NextEndSideIdx);
			AddVertex(InMesh, PrevEndSideIdx);
			NextEndSideIdx = AddVertexRaw(InMesh, (CornerEnd + NextSideBevelNormal * BevelSize) * Scale, NextSideBevelNormal);
		}

		// close end
		AddVertex(InMesh, NextEndSideIdx);
		AddVertex(InMesh, PrevEndSideIdx);
		AddVertexRaw(InMesh, CenterEndLoc * Scale, CornerEndNormal);

		PrevEndIdx = NextEndIdx;
		PrevStartIdx = NextStartIdx;
	}
}

bool UAvaShapeCubeDynamicMesh::GenerateBottomMeshSections(FAvaShapeMesh& BottomMesh)
{
	const FVector SegmentSize(GetSegmentSize());
	const FVector BevelSize(GetBevelSize());
	const FVector Scale(GetScaleSize());

	// bottom

	static const FVector BottomNormal(0, 0, -1);
	// 
	const FVector BottomLeftLoc(-(SegmentSize.X - BevelSize.X), -(SegmentSize.Y - BevelSize.Y), -SegmentSize.Z);
	const FAvaShapeCachedVertex3D BottomLeft = CacheVertexCreate(BottomMesh, BottomLeftLoc * Scale, BottomNormal);
	const FVector BottomRightLoc((SegmentSize.X - BevelSize.X), (SegmentSize.Y - BevelSize.Y), -SegmentSize.Z);
	const FAvaShapeCachedVertex3D BottomRight = CacheVertexCreate(BottomMesh, BottomRightLoc * Scale, BottomNormal);

	AddVertex(BottomMesh, BottomLeft);
	const FVector BottomLeftLoc2 = FVector((SegmentSize.X - BevelSize.X), -(SegmentSize.Y - BevelSize.Y), -SegmentSize.Z);
	int32 BottomLeftIdx = AddVertexRaw(BottomMesh, BottomLeftLoc2 * Scale, BottomNormal);
	AddVertex(BottomMesh, BottomRight);

	AddVertex(BottomMesh, BottomRight);
	const FVector BottomRightLoc2 = FVector(-(SegmentSize.X - BevelSize.X), (SegmentSize.Y - BevelSize.Y), -SegmentSize.Z);
	int32 BottomRightIdx = AddVertexRaw(BottomMesh, BottomRightLoc2 * Scale, BottomNormal);
	AddVertex(BottomMesh, BottomLeft);

	if (BevelSizeRatio > 0)
	{
		CreateBevelCorner(BottomMesh, BottomRightLoc2, BottomRightLoc, BottomNormal, BottomRightIdx, BottomRight.Index,
			BevelSize, Scale, FVector(-SegmentSize.X, SegmentSize.Y, -SegmentSize.Z), FVector(SegmentSize.X, SegmentSize.Y, -SegmentSize.Z));

		CreateBevelCorner(BottomMesh, BottomRightLoc, BottomLeftLoc2, BottomNormal, BottomRight.Index, BottomLeftIdx,
			BevelSize, Scale, FVector(SegmentSize.X, SegmentSize.Y, -SegmentSize.Z), FVector(SegmentSize.X, -SegmentSize.Y, -SegmentSize.Z));

		CreateBevelCorner(BottomMesh, BottomLeftLoc2, BottomLeftLoc, BottomNormal, BottomLeftIdx, BottomLeft.Index,
			BevelSize, Scale, FVector(SegmentSize.X, -SegmentSize.Y, -SegmentSize.Z), FVector(-SegmentSize.X, -SegmentSize.Y, -SegmentSize.Z));

		CreateBevelCorner(BottomMesh, BottomLeftLoc, BottomRightLoc2, BottomNormal, BottomLeft.Index, BottomRightIdx,
			BevelSize, Scale, FVector(-SegmentSize.X, -SegmentSize.Y, -SegmentSize.Z), FVector(-SegmentSize.X, SegmentSize.Y, -SegmentSize.Z));
	}

	return true;
}

bool UAvaShapeCubeDynamicMesh::GenerateTopMeshSections(FAvaShapeMesh& TopMesh)
{
	const FVector SegmentSize(GetSegmentSize());
	const FVector BevelSize(GetBevelSize());
	const FVector Scale(GetScaleSize());

	// top

	static const FVector TopNormal(0, 0, 1);
	// bottom left
	const FVector TopLeftLoc(-(SegmentSize.X - BevelSize.X), -(SegmentSize.Y - BevelSize.Y), SegmentSize.Z);
	const FAvaShapeCachedVertex3D TopLeft = CacheVertexCreate(TopMesh, TopLeftLoc * Scale, TopNormal);
	// top right
	const FVector TopRightLoc((SegmentSize.X - BevelSize.X), (SegmentSize.Y - BevelSize.Y), SegmentSize.Z);
	const FAvaShapeCachedVertex3D TopRight = CacheVertexCreate(TopMesh, TopRightLoc * Scale, TopNormal);

	AddVertex(TopMesh, TopLeft);
	AddVertex(TopMesh, TopRight);
	// top left
	const FVector TopLeftLoc2 = FVector((SegmentSize.X - BevelSize.X), -(SegmentSize.Y - BevelSize.Y), SegmentSize.Z);
	int32 TopLeftIdx = AddVertexRaw(TopMesh, TopLeftLoc2 * Scale, TopNormal);

	AddVertex(TopMesh, TopRight);
	AddVertex(TopMesh, TopLeft);
	// bottom right
	const FVector TopRightLoc2 = FVector(-(SegmentSize.X - BevelSize.X), (SegmentSize.Y - BevelSize.Y), SegmentSize.Z);
	int32 TopRightIdx = AddVertexRaw(TopMesh, TopRightLoc2 * Scale, TopNormal);

	if (BevelSizeRatio > 0)
	{
		CreateBevelCorner(TopMesh, TopLeftLoc2, TopRightLoc, TopNormal, TopLeftIdx, TopRight.Index,
			BevelSize, Scale, FVector(SegmentSize.X, -SegmentSize.Y, SegmentSize.Z), FVector(SegmentSize.X, SegmentSize.Y, SegmentSize.Z));

		CreateBevelCorner(TopMesh, TopRightLoc, TopRightLoc2, TopNormal, TopRight.Index, TopRightIdx,
			BevelSize, Scale, FVector(SegmentSize.X, SegmentSize.Y, SegmentSize.Z), FVector(-SegmentSize.X, SegmentSize.Y, SegmentSize.Z));

		CreateBevelCorner(TopMesh, TopRightLoc2, TopLeftLoc, TopNormal, TopRightIdx, TopLeft.Index,
			BevelSize, Scale, FVector(-SegmentSize.X, SegmentSize.Y, SegmentSize.Z), FVector(-SegmentSize.X, -SegmentSize.Y, SegmentSize.Z));

		CreateBevelCorner(TopMesh, TopLeftLoc, TopLeftLoc2, TopNormal, TopLeft.Index, TopLeftIdx,
			BevelSize, Scale, FVector(-SegmentSize.X, -SegmentSize.Y, SegmentSize.Z), FVector(SegmentSize.X, -SegmentSize.Y, SegmentSize.Z));
	}

	return true;
}

bool UAvaShapeCubeDynamicMesh::GenerateFrontMeshSections(FAvaShapeMesh& FrontMesh)
{
	const FVector SegmentSize(GetSegmentSize());
	const FVector BevelSize(GetBevelSize());
	const FVector Scale(GetScaleSize());

	// front

	static const FVector FrontNormal(-1, 0, 0);
	const FVector FrontLeftLoc(-SegmentSize.X, -(SegmentSize.Y - BevelSize.Y), SegmentSize.Z - BevelSize.Z);
	// top left
	const FAvaShapeCachedVertex3D FrontLeft = CacheVertexCreate(FrontMesh, FrontLeftLoc * Scale, FrontNormal);
	// bottom right
	const FVector FrontRightLoc(-SegmentSize.X, SegmentSize.Y - BevelSize.Y, -(SegmentSize.Z - BevelSize.Z));
	const FAvaShapeCachedVertex3D FrontRight = CacheVertexCreate(FrontMesh, FrontRightLoc * Scale, FrontNormal);

	AddVertex(FrontMesh, FrontLeft);
	AddVertex(FrontMesh, FrontRight);
	const FVector TopRightLoc = FVector(-SegmentSize.X, SegmentSize.Y - BevelSize.Y, SegmentSize.Z - BevelSize.Z);
	int32 TopRightIdx = AddVertexRaw(FrontMesh, TopRightLoc * Scale, FrontNormal);

	AddVertex(FrontMesh, FrontRight);
	AddVertex(FrontMesh, FrontLeft);
	const FVector BottomLeftLoc = FVector(-SegmentSize.X, -(SegmentSize.Y - BevelSize.Y), -(SegmentSize.Z - BevelSize.Z));
	int32 BottomLeftIdx = AddVertexRaw(FrontMesh, BottomLeftLoc * Scale, FrontNormal);

	if (BevelSizeRatio > 0)
	{
		CreateBevelCorner(FrontMesh, FrontLeftLoc, TopRightLoc, FrontNormal, FrontLeft.Index, TopRightIdx,
			BevelSize, Scale, FVector(-SegmentSize.X, -SegmentSize.Y, SegmentSize.Z), FVector(-SegmentSize.X, SegmentSize.Y, SegmentSize.Z));

		CreateBevelCorner(FrontMesh, TopRightLoc, FrontRightLoc, FrontNormal, TopRightIdx, FrontRight.Index,
			BevelSize, Scale, FVector(-SegmentSize.X, SegmentSize.Y, SegmentSize.Z), FVector(-SegmentSize.X, SegmentSize.Y, -SegmentSize.Z));

		CreateBevelCorner(FrontMesh, FrontRightLoc, BottomLeftLoc, FrontNormal, FrontRight.Index, BottomLeftIdx,
			BevelSize, Scale, FVector(-SegmentSize.X, SegmentSize.Y, -SegmentSize.Z), FVector(-SegmentSize.X, -SegmentSize.Y, -SegmentSize.Z));

		CreateBevelCorner(FrontMesh, BottomLeftLoc, FrontLeftLoc, FrontNormal, BottomLeftIdx, FrontLeft.Index,
			BevelSize, Scale, FVector(-SegmentSize.X, -SegmentSize.Y, -SegmentSize.Z), FVector(-SegmentSize.X, -SegmentSize.Y, SegmentSize.Z));
	}

	return true;
}

bool UAvaShapeCubeDynamicMesh::GenerateBackMeshSections(FAvaShapeMesh& BackMesh)
{
	const FVector SegmentSize(GetSegmentSize());
	const FVector BevelSize(GetBevelSize());
	const FVector Scale(GetScaleSize());

	// back

	static const FVector BackNormal(1, 0, 0);
	const FVector BackLeftLoc(SegmentSize.X, -(SegmentSize.Y - BevelSize.Y), (SegmentSize.Z - BevelSize.Z));
	const FAvaShapeCachedVertex3D BackLeft = CacheVertexCreate(BackMesh, BackLeftLoc * Scale, BackNormal);
	const FVector BackRightLoc(SegmentSize.X, (SegmentSize.Y - BevelSize.Y), -(SegmentSize.Z - BevelSize.Z));
	const FAvaShapeCachedVertex3D BackRight = CacheVertexCreate(BackMesh, BackRightLoc * Scale, BackNormal);

	AddVertex(BackMesh, BackLeft);
	const FVector BackLeftLoc2(SegmentSize.X, (SegmentSize.Y - BevelSize.Y), (SegmentSize.Z - BevelSize.Z));
	int32 BackLeftIdx = AddVertexRaw(BackMesh, BackLeftLoc2 * Scale, BackNormal);
	AddVertex(BackMesh, BackRight);

	AddVertex(BackMesh, BackRight);
	const FVector BackRightLoc2(SegmentSize.X, -(SegmentSize.Y - BevelSize.Y), -(SegmentSize.Z - BevelSize.Z));
	int32 BackRightIdx = AddVertexRaw(BackMesh, BackRightLoc2 * Scale, BackNormal);
	AddVertex(BackMesh, BackLeft);

	if (BevelSizeRatio > 0)
	{
		CreateBevelCorner(BackMesh, BackLeftLoc2, BackLeftLoc, BackNormal, BackLeftIdx, BackLeft.Index,
			BevelSize, Scale, FVector(SegmentSize.X, SegmentSize.Y, SegmentSize.Z), FVector(SegmentSize.X, -SegmentSize.Y, SegmentSize.Z));

		CreateBevelCorner(BackMesh, BackLeftLoc, BackRightLoc2, BackNormal, BackLeft.Index, BackRightIdx,
			BevelSize, Scale, FVector(SegmentSize.X, -SegmentSize.Y, SegmentSize.Z), FVector(SegmentSize.X, -SegmentSize.Y, -SegmentSize.Z));

		CreateBevelCorner(BackMesh, BackRightLoc2, BackRightLoc, BackNormal, BackRightIdx, BackRight.Index,
			BevelSize, Scale, FVector(SegmentSize.X, -SegmentSize.Y, -SegmentSize.Z), FVector(SegmentSize.X, SegmentSize.Y, -SegmentSize.Z));

		CreateBevelCorner(BackMesh, BackRightLoc, BackLeftLoc2, BackNormal, BackRight.Index, BackLeftIdx,
			BevelSize, Scale, FVector(SegmentSize.X, SegmentSize.Y, -SegmentSize.Z), FVector(SegmentSize.X, SegmentSize.Y, SegmentSize.Z));
	}

	return true;
}

bool UAvaShapeCubeDynamicMesh::GenerateLeftMeshSections(FAvaShapeMesh& LeftMesh)
{
	const FVector SegmentSize(GetSegmentSize());
	const FVector BevelSize(GetBevelSize());
	const FVector Scale(GetScaleSize());

	// left

	static const FVector LeftNormal(0, -1, 0);
	const FVector LeftLeftLoc((SegmentSize.X - BevelSize.X), -SegmentSize.Y, SegmentSize.Z - BevelSize.Z);
	const FAvaShapeCachedVertex3D LeftLeft = CacheVertexCreate(LeftMesh, LeftLeftLoc * Scale, LeftNormal);
	const FVector LeftRightLoc(-(SegmentSize.X - BevelSize.X), -SegmentSize.Y, -(SegmentSize.Z - BevelSize.Z));
	const FAvaShapeCachedVertex3D LeftRight = CacheVertexCreate(LeftMesh, LeftRightLoc * Scale, LeftNormal);

	AddVertex(LeftMesh, LeftLeft);
	AddVertex(LeftMesh, LeftRight);
	const FVector LeftRightLoc2(-(SegmentSize.X - BevelSize.X), -SegmentSize.Y, (SegmentSize.Z - BevelSize.Z));
	int32 LeftRightIdx = AddVertexRaw(LeftMesh, LeftRightLoc2 * Scale, LeftNormal);

	AddVertex(LeftMesh, LeftRight);
	AddVertex(LeftMesh, LeftLeft);
	const FVector LeftLeftLoc2((SegmentSize.X - BevelSize.X), -SegmentSize.Y, -(SegmentSize.Z - BevelSize.Z));
	int32 LeftLeftIdx = AddVertexRaw(LeftMesh, LeftLeftLoc2 * Scale, LeftNormal);

	if (BevelSizeRatio > 0)
	{
		CreateBevelCorner(LeftMesh, LeftLeftLoc, LeftRightLoc2, LeftNormal, LeftLeft.Index, LeftRightIdx,
			BevelSize, Scale, FVector(SegmentSize.X, -SegmentSize.Y, SegmentSize.Z), FVector(-SegmentSize.X, -SegmentSize.Y, SegmentSize.Z));

		CreateBevelCorner(LeftMesh, LeftRightLoc2, LeftRightLoc, LeftNormal, LeftRightIdx, LeftRight.Index,
			BevelSize, Scale, FVector(-SegmentSize.X, -SegmentSize.Y, SegmentSize.Z), FVector(-SegmentSize.X, -SegmentSize.Y, -SegmentSize.Z));

		CreateBevelCorner(LeftMesh, LeftRightLoc, LeftLeftLoc2, LeftNormal, LeftRight.Index, LeftLeftIdx,
			BevelSize, Scale, FVector(-SegmentSize.X, -SegmentSize.Y, -SegmentSize.Z), FVector(SegmentSize.X, -SegmentSize.Y, -SegmentSize.Z));

		CreateBevelCorner(LeftMesh, LeftLeftLoc2, LeftLeftLoc, LeftNormal, LeftLeftIdx, LeftLeft.Index,
			BevelSize, Scale, FVector(SegmentSize.X, -SegmentSize.Y, -SegmentSize.Z), FVector(SegmentSize.X, -SegmentSize.Y, SegmentSize.Z));
	}

	return true;
}

bool UAvaShapeCubeDynamicMesh::GenerateRightMeshSections(FAvaShapeMesh& RightMesh)
{
	const FVector SegmentSize(GetSegmentSize());
	const FVector BevelSize(GetBevelSize());
	const FVector Scale(GetScaleSize());

	// right

	static const FVector RightNormal(0, 1, 0);
	const FVector RightLeftLoc((SegmentSize.X - BevelSize.X), SegmentSize.Y, (SegmentSize.Z - BevelSize.Z));
	const FAvaShapeCachedVertex3D RightLeft = CacheVertexCreate(RightMesh, RightLeftLoc * Scale, RightNormal);
	const FVector RightRightLoc(-(SegmentSize.X - BevelSize.X), SegmentSize.Y, -(SegmentSize.Z - BevelSize.Z));
	const FAvaShapeCachedVertex3D RightRight = CacheVertexCreate(RightMesh, RightRightLoc * Scale, RightNormal);

	AddVertex(RightMesh, RightLeft);
	const FVector RightLeftLoc2(-(SegmentSize.X - BevelSize.X), SegmentSize.Y, (SegmentSize.Z - BevelSize.Z));
	int32 RightLeftIdx = AddVertexRaw(RightMesh, RightLeftLoc2 * Scale, RightNormal);
	AddVertex(RightMesh, RightRight);

	AddVertex(RightMesh, RightRight);
	const FVector RightRightLoc2((SegmentSize.X - BevelSize.X), SegmentSize.Y, -(SegmentSize.Z - BevelSize.Z));
	int32 RightRightIdx = AddVertexRaw(RightMesh, RightRightLoc2 * Scale, RightNormal);
	AddVertex(RightMesh, RightLeft);

	if (BevelSizeRatio > 0)
	{
		CreateBevelCorner(RightMesh, RightLeftLoc2, RightLeftLoc, RightNormal, RightLeftIdx, RightLeft.Index,
			BevelSize, Scale, FVector(-SegmentSize.X, SegmentSize.Y, SegmentSize.Z), FVector(SegmentSize.X, SegmentSize.Y, SegmentSize.Z));

		CreateBevelCorner(RightMesh, RightLeftLoc, RightRightLoc2, RightNormal, RightLeft.Index, RightRightIdx,
			BevelSize, Scale, FVector(SegmentSize.X, SegmentSize.Y, SegmentSize.Z), FVector(SegmentSize.X, SegmentSize.Y, -SegmentSize.Z));

		CreateBevelCorner(RightMesh, RightRightLoc2, RightRightLoc, RightNormal, RightRightIdx, RightRight.Index,
			BevelSize, Scale, FVector(SegmentSize.X, SegmentSize.Y, -SegmentSize.Z), FVector(-SegmentSize.X, SegmentSize.Y, -SegmentSize.Z));

		CreateBevelCorner(RightMesh, RightRightLoc, RightLeftLoc2, RightNormal, RightRight.Index, RightLeftIdx,
			BevelSize, Scale, FVector(-SegmentSize.X, SegmentSize.Y, -SegmentSize.Z), FVector(-SegmentSize.X, SegmentSize.Y, SegmentSize.Z));
	}

	return true;
}

bool UAvaShapeCubeDynamicMesh::CreateFaceUVs(FAvaShapeMesh& InMesh, FAvaShapeMaterialUVParameters& InUseParams, FRotator ProjectionRot, FVector2D UVScale, FVector2D UVOffset)
{
	const FVector2D Size = UVScale * Segment;
	return ApplyUVsPlanarProjection(InMesh, ProjectionRot, Size)
		&& ApplyUVsTransform(InMesh, InUseParams, UVScale, UVOffset, 0.f);
}

bool UAvaShapeCubeDynamicMesh::CreateUVs(FAvaShapeMesh& InMesh, FAvaShapeMaterialUVParameters& InParams)
{
	if (InMesh.Vertices.Num() == 0)
	{
		return false;
	}

	switch (InMesh.GetMeshIndex())
	{
		case MESH_INDEX_PRIMARY:
			return CreateFaceUVs(InMesh, InParams, FRotator(0, 90, 90), FVector2D(Size3D.Y, Size3D.Z), FVector2D(0.5, 0.5));
		case MESH_INDEX_TOP:
			return CreateFaceUVs(InMesh, InParams, FRotator(0, 0, 0), FVector2D(Size3D.X, Size3D.Y), FVector2D(0.5, 0.5));
		case MESH_INDEX_BOTTOM:
			return CreateFaceUVs(InMesh, InParams, FRotator(180, 0, 0), FVector2D(Size3D.X, Size3D.Y), FVector2D(0.5, 0.5));
		case MESH_INDEX_BACK:
			return CreateFaceUVs(InMesh, InParams, FRotator(0, -90, 90), FVector2D(Size3D.Y, Size3D.Z), FVector2D(0.5, 0.5));
		case MESH_INDEX_LEFT:
			return CreateFaceUVs(InMesh, InParams, FRotator(180, 0, -90), FVector2D(Size3D.X, Size3D.Z), FVector2D(0.5, 0.5));
		case MESH_INDEX_RIGHT:
			return CreateFaceUVs(InMesh, InParams, FRotator(0, 0, 90), FVector2D(Size3D.X, Size3D.Z), FVector2D(0.5, 0.5));
		default:
			return false;
	}
}

bool UAvaShapeCubeDynamicMesh::CreateMesh(FAvaShapeMesh& InMesh)
{
	if (!Super::CreateMesh(InMesh))
	{
		return false;
	}

	switch (InMesh.GetMeshIndex())
	{
		case MESH_INDEX_PRIMARY:
			return GenerateFrontMeshSections(InMesh);
		case MESH_INDEX_TOP:
			return GenerateTopMeshSections(InMesh);
		case MESH_INDEX_BOTTOM:
			return GenerateBottomMeshSections(InMesh);
		case MESH_INDEX_BACK:
			return GenerateBackMeshSections(InMesh);
		case MESH_INDEX_LEFT:
			return GenerateLeftMeshSections(InMesh);
		case MESH_INDEX_RIGHT:
			return GenerateRightMeshSections(InMesh);
		default:
			return false;
	}
}
