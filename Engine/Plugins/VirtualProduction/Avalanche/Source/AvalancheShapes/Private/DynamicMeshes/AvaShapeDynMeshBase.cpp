// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshes/AvaShapeDynMeshBase.h"

#include "Async/Async.h"
#include "Async/ParallelFor.h"
#include "AvaSceneSubsystem.h"
#include "AvaShapeActor.h"
#include "AvaShapeBuildSystem.h"
#include "Components/DynamicMeshComponent.h"
#include "Containers/Ticker.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMeshMikkTWrapper.h"
#include "DynamicMeshes/AvaShape2DDynMeshBase.h"
#include "Engine/StaticMesh.h"
#include "Framework/AvaGizmoComponent.h"
#include "GeometryScript/MeshAssetFunctions.h"
#include "GeometryScript/MeshBasicEditFunctions.h"
#include "GeometryScript/MeshNormalsFunctions.h"
#include "InputState.h"
#include "Material/DynamicMaterialInstance.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Modifiers/ActorModifierCoreStack.h"
#include "Modifiers/ActorModifierRenderStateDirtyEvent.h"
#include "Modules/ModuleManager.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "Subsystems/ActorModifierCoreSubsystem.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "DMObjectMaterialProperty.h"
#include "Editor.h"
#include "Elements/Actor/ActorElementEditorViewportInteractionCustomization.h"
#include "IDynamicMaterialEditorModule.h"
#include "LevelEditor/AvaLevelEditorUtils.h"
#include "Material/DynamicMaterialInstanceFactory.h"
#include "PropertyEditorModule.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogAvaDynamicMesh, Log, All);

using namespace UE::Geometry;

namespace UE::Private::AvaShape
{
	/** Edit underlying dynamic mesh */
	bool EditMesh(const UAvaShapeDynamicMeshBase* InContext, const TFunctionRef<void(FDynamicMesh3&)>& InFunctor, EDynamicMeshChangeType InChangeType = EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags InAttributeChangeType = EDynamicMeshAttributeChangeFlags::Unknown)
	{
		checkf(!!InContext, TEXT("Invalid shape context provided to edit mesh"))

		if (UDynamicMeshComponent* MeshComponent = InContext->GetShapeMeshComponent())
		{
			if (UDynamicMesh* Mesh = MeshComponent->GetDynamicMesh())
			{
				const bool bSkipBroadcastChangeEvents = true; // EditMesh never broadcasts any event. This is done by the UAvaShapeDynamicMeshBase on post mesh update
				Mesh->EditMesh(InFunctor, InChangeType, InAttributeChangeType, bSkipBroadcastChangeEvents);
				return true;
			}
		}

		return false;
	}
}

UAvaShapeDynamicMeshBase::UAvaShapeDynamicMeshBase(const FLinearColor& InVertexColor, float InUniformScaledSize, bool bInAllowEditSize)
	: bAllowEditSize(bInAllowEditSize)
	, UniformScaledSize(InUniformScaledSize)
	, bUsePrimaryMaterialEverywhere(true)
	, bMeshRegistered(false)
	, bInCreateMode(false)
	, bAllMeshDirty(true)
	, bAnyMeshDirty(bAllMeshDirty)
	, bVerticesDirty(false)
{
	if (!IsTemplate())
	{
#if WITH_EDITOR
		FEditorDelegates::OnApplyObjectToActor.AddUObject(this, &UAvaShapeDynamicMeshBase::OnAssetDropped);
#endif

		FAvaShapeParametricMaterial::OnMaterialChanged().AddUObject(this, &UAvaShapeDynamicMeshBase::OnParametricMaterialChanged);
		FAvaShapeParametricMaterial::OnMaterialParameterChanged().AddUObject(this, &UAvaShapeDynamicMeshBase::OnParametricMaterialParameterChanged);
	}
}

EMaterialType UAvaShapeDynamicMeshBase::GetMaterialType(int32 InMeshIndex)
{
	if (const FAvaShapeMeshData* MeshData = GetMeshData(InMeshIndex))
	{
		return MeshData->MaterialType;
	}

	return EMaterialType::Default;
}

void UAvaShapeDynamicMeshBase::SetMaterialType(int32 InMeshIndex, EMaterialType InType)
{
	if (FAvaShapeMeshData* MeshData = GetMeshData(InMeshIndex))
	{
		MeshData->MaterialType = InType;
		OnMaterialTypeChanged(InMeshIndex);
	}
}

bool UAvaShapeDynamicMeshBase::IsMaterialType(int32 MeshIndex, EMaterialType Type)
{
	return GetMaterialType(MeshIndex) == Type;
}

void UAvaShapeDynamicMeshBase::SetupMeshes()
{
	if (!bMeshRegistered)
	{
		// setup, done once
		bMeshRegistered = true;

		if (MeshDatas.IsEmpty())
		{
			// handle creating of mesh sections
			MeshDatas.Empty();

			// register main mesh
			FAvaShapeMeshData PrimaryMeshData(MESH_INDEX_PRIMARY, TEXT("Primary"), true);
			RegisterMesh(PrimaryMeshData);

			// register other meshes
			RegisterMeshes();

			const bool bSetPrimaryMaterial = bUsePrimaryMaterialEverywhere;
			const bool bIs2DShape = IsA<UAvaShape2DDynMeshBase>();
			for (const int32& MeshIdx : GetMeshesIndexes())
			{
				FAvaShapeMeshData* MeshData = GetMeshData(MeshIdx);

				// Enable translucency by default
				MeshData->ParametricMaterial.SetTranslucency(bIs2DShape ? EAvaShapeParametricMaterialTranslucency::Enabled : EAvaShapeParametricMaterialTranslucency::Auto);
				OnMaterialChanged(MeshIdx);
			}

			bUsePrimaryMaterialEverywhere = bSetPrimaryMaterial;
		}

		// handle duplication and loading saved asset here
		OnUsePrimaryMaterialEverywhereChanged();

		// meshes setup done
		OnRegisteredMeshes();
	}
}

void UAvaShapeDynamicMeshBase::RefreshParametricMaterial()
{
	if (bUsePrimaryMaterialEverywhere)
	{
		OnParametricMaterialChanged(MESH_INDEX_PRIMARY);
	}
	else
	{
		for (const int32 Index : GetMeshesIndexes())
		{
			if (IsMaterialType(Index, EMaterialType::Parametric))
			{
				OnParametricMaterialChanged(Index);
			}
		}
	}
}

bool UAvaShapeDynamicMeshBase::SetAlignmentSize(AActor* InActor, const FVector& InSizeMultiplier)
{
	if (const AAvaShapeActor* ShapeActor = Cast<AAvaShapeActor>(InActor))
	{
		if (UAvaShapeDynamicMeshBase* DynMesh = ShapeActor->GetDynamicMesh())
		{
			DynMesh->SetSize3D(DynMesh->GetSize3D() * InSizeMultiplier);
		}
	}

	return false;
}

void UAvaShapeDynamicMeshBase::OnRegister()
{
	Super::OnRegister();

	if (bAnyMeshDirty)
	{
		RequestUpdate();
	}
}

void UAvaShapeDynamicMeshBase::OnComponentCreated()
{
	Super::OnComponentCreated();

	SetupMeshes();
}

void UAvaShapeDynamicMeshBase::PostLoad()
{
	Super::PostLoad();

	AActor* ShapeActor = GetOwner();
	if (!IsValid(ShapeActor))
	{
		return;
	}

	UDynamicMeshComponent* ShapeComponent = GetShapeMeshComponent();
	if (!IsValid(ShapeComponent))
	{
		return;
	}

	const bool bSetPrimaryMaterial = bUsePrimaryMaterialEverywhere;
	for (TPair<int32, FAvaShapeMeshData>& MeshData : MeshDatas)
	{
		if (MeshData.Value.MaterialType != EMaterialType::Parametric)
		{
			continue;
		}

		// If we have a valid Material Designer Instance in the parametric material struct, make sure it's set on the mesh
		if (IsValid(MeshData.Value.ParametricMaterial.GetMaterial()))
		{
			SetMaterialDirect(MeshData.Key, MeshData.Value.ParametricMaterial.GetMaterial());
			continue;
		}

		UMaterialInstanceDynamic* ShapeMID = Cast<UMaterialInstanceDynamic>(ShapeComponent->GetMaterial(MeshData.Key));

		// Assume the mesh has the correct Material Designer Instance and set it on the parametric material struct
		if (ShapeMID && ShapeMID->GetMaterial() == MeshData.Value.ParametricMaterial.GetDefaultMaterial())
		{
			MeshData.Value.ParametricMaterial.SetMaterial(ShapeMID);
		}

		// We are in parametric mode but material is not parametric
		// If the shape has no valid Material Designer Instance, create it from the parametric material struct and apply
		SetMaterialDirect(MeshData.Key, MeshData.Value.ParametricMaterial.GetOrCreateMaterial(ShapeActor));
	}

	bUsePrimaryMaterialEverywhere = bSetPrimaryMaterial;
	OnUsePrimaryMaterialEverywhereChanged();
}

void UAvaShapeDynamicMeshBase::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	Super::OnComponentDestroyed(bDestroyingHierarchy);

	if (UDynamicMeshComponent* MeshComponent = GetShapeMeshComponent())
	{
		MeshComponent->TransformUpdated.RemoveAll(this);
	}

	FAvaShapeParametricMaterial::OnMaterialChanged().RemoveAll(this);
	FAvaShapeParametricMaterial::OnMaterialParameterChanged().RemoveAll(this);
}

void UAvaShapeDynamicMeshBase::ToggleGizmo_Implementation(
	const UAvaGizmoComponent* InGizmoComponent
	, const bool bShowAsGizmo)
{
	auto ForEachMeshData = [this](TUniqueFunction<void(FAvaShapeMeshData*, const int32)>&& InFunc)
	{
		for (const int32& MeshIdx : GetMeshesIndexes())
		{
			if (FAvaShapeMeshData* MeshData = GetMeshData(MeshIdx))
			{
				InFunc(MeshData, MeshIdx);
			}
		}
	};

	if (!bShowAsGizmo)
	{
#if WITH_EDITOR
		ForEachMeshData([this](FAvaShapeMeshData* InMeshData, const int32 InMeshIdx)
		{
			// Restore non-gizmo settings
			if (const FAvaShapeMeshData* StoredMeshData = NonGizmoMeshData.Find(InMeshIdx))
			{
				InMeshData->MaterialType = StoredMeshData->MaterialType;
			}
		});
#endif
	}
	// IS showing as gizmo
	else
	{
		// Material is specified
		if (UMaterialInterface* GizmoMaterial = InGizmoComponent->GetMaterial())
		{
			ForEachMeshData([this, GizmoMaterial](FAvaShapeMeshData* InMeshData, const int32 InMeshIdx)
			{
				#if WITH_EDITORONLY_DATA
				// Store existing settings (if in editor)
				NonGizmoMeshData.FindOrAdd(InMeshIdx).MaterialType = InMeshData->MaterialType;
				#endif

				// Set from GizmoComponent
				InMeshData->MaterialType = EMaterialType::Asset;
				SetMaterialDirect(InMeshIdx, GizmoMaterial);
			});
		}
	}
}

void UAvaShapeDynamicMeshBase::UpdateMesh(const FMeshUpdateContext& InUpdateContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAvaShapeDynamicMeshBase::UpdateMesh);

	MeshUpdateContext = InUpdateContext;
	ON_SCOPE_EXIT
	{
		MeshUpdateContext.Reset();
	};

	RestoreCachedMesh(/** Reset */true);

	bool bUpdateMesh = GenerateMesh();

	UpdateVertices();

	// Apply vertices, triangles, normals
	ApplyMeshSectionChanges();

	bUpdateMesh |= GenerateUV();

	if (bUpdateMesh)
	{
		GenerateTangents();
	}

	// Apply Update UV, tangents
	ApplyMeshSectionChanges();

	SaveCachedMesh();
}

void UAvaShapeDynamicMeshBase::OffsetActorToAlign()
{
	// Offset to match alignments
	if (bHasNewMeshRegenWorldLocation)
	{
		bHasNewMeshRegenWorldLocation = false;

		AActor* OwningActor = GetOwner();
		const FVector Delta = MeshRegenWorldLocation - OwningActor->GetRootComponent()->GetComponentLocation();

#if WITH_EDITOR
		FActorElementEditorViewportInteractionCustomization::ApplyDeltaToActor(
			OwningActor, true, &Delta,
			nullptr, nullptr, FVector::ZeroVector, FInputDeviceState()
		); // Pivot is not used for just translation
#else
		OwningActor->ApplyWorldOffset(Delta, false);
#endif
		MeshRegenWorldLocation = FVector::ZeroVector;
	}
}

void UAvaShapeDynamicMeshBase::PreUpdateMesh()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAvaShapeDynamicMeshBase::PreUpdateMesh_GameThread);

	if (!bAllMeshDirty && !bAnyMeshDirty)
	{
		return;
	}

	if (UDynamicMeshComponent* DynamicMeshComponent = GetShapeMeshComponent())
	{
		if (UDynamicMesh* DynamicMesh = DynamicMeshComponent->GetDynamicMesh())
		{
			DynamicMesh->OnPreMeshChanged().Broadcast(DynamicMesh, FDynamicMeshChangeInfo());
		}
	}

	SetupMeshes();

	if (!IsDynamicMeshInitialized())
	{
		InitializeDynamicMesh();
	}
}

void UAvaShapeDynamicMeshBase::PostUpdateMesh()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAvaShapeDynamicMeshBase::PostUpdateMesh);
	OffsetActorToAlign();

	if (UDynamicMeshComponent* MeshComponent = GetShapeMeshComponent())
	{
		int32 MeshIndex = INDEX_NONE;
		UMaterialInterface* MeshMaterial;

		auto UpdateBaseMaterial = [MeshComponent, &MeshIndex, &MeshMaterial]()
			{
				check(MeshIndex >= 0);
				if (MeshIndex >= MeshComponent->BaseMaterials.Num())
				{
					MeshComponent->BaseMaterials.SetNum(MeshIndex + 1, EAllowShrinking::No);
				}
				MeshComponent->BaseMaterials[MeshIndex] = MeshMaterial;
			};

		for (const TPair<int32, FAvaShapeMeshData>& MeshData : MeshDatas)
		{
			const FAvaShapeMesh& ShapeMesh = MeshData.Value.Mesh;
			MeshIndex = ShapeMesh.GetMeshIndex();
			MeshMaterial = GetMaterial(ShapeMesh.GetMeshIndex());
			UpdateBaseMaterial();
		}

		UE::ActorModifierCore::FRenderStateDirtyReasonScope Scope(UE::ActorModifierCore::ERenderStateDirtyReason::Material | UE::ActorModifierCore::ERenderStateDirtyReason::Geometry);

		if (MeshIndex != INDEX_NONE)
		{
			// Call SetMaterial on last index/material to mark render state dirty, update physical materials, etc.
			MeshComponent->SetMaterial(MeshIndex, MeshMaterial);
		}

		// No need to call MeshComponent->NotifyMeshModified(), as this is called by the delegate too (see UDynamicMeshComponent::UDynamicMeshComponent)
		if (UDynamicMesh* Mesh = MeshComponent->GetDynamicMesh())
		{
			Mesh->OnMeshChanged().Broadcast(Mesh, FDynamicMeshChangeInfo());
		}

		MeshUpdateStatus = EAvaDynamicMeshUpdateState::UpToDate;
		OnMeshUpdateFinished();
	}
}

void UAvaShapeDynamicMeshBase::ApplyMeshSectionChanges()
{
	for (const int32 MeshIndex : GetMeshesIndexes())
	{
		FAvaShapeMesh* Mesh = GetMesh(MeshIndex);
		if (Mesh->bUpdateRequired)
		{
			OnMeshSectionUpdated(*Mesh);
		}
	}
}

void UAvaShapeDynamicMeshBase::RunUpdate()
{
	if (MeshUpdateStatus == EAvaDynamicMeshUpdateState::UpdateInProgress)
	{
		return;
	}

	if (MeshUpdateStatus != EAvaDynamicMeshUpdateState::UpdateRequired)
	{
		return;
	}

	MeshUpdateStatus = EAvaDynamicMeshUpdateState::UpdateInProgress;

	auto ExecUpdate = [this](float=0.f)->bool
		{
			if (GetShapeMeshComponent())
			{
				PreUpdateMesh();

				const FMeshUpdateContext UpdateContext
					{
						.bParallel = false
					};
				UpdateMesh(UpdateContext);

				PostUpdateMesh();
			}
			else
			{
				MeshUpdateStatus = EAvaDynamicMeshUpdateState::UpdateRequired;
			}
			return false;
		};

	if (GetRunAsync())
	{
		if (UAvaShapeBuildSystem* const BuildSystem = UAvaShapeBuildSystem::FindBuildSystem(this))
		{
			BuildSystem->QueueShapeBuild(this);
		}
		else
		{
			GetTicker().AddTicker(FTickerDelegate::CreateWeakLambda(this, ExecUpdate));	
		}
	}
	else
	{
		ExecUpdate();
	}
}

void UAvaShapeDynamicMeshBase::RequestUpdate()
{
	// only runs if the mesh was up to date previously
	if (MeshUpdateStatus == EAvaDynamicMeshUpdateState::UpToDate)
	{
		MeshUpdateStatus = EAvaDynamicMeshUpdateState::UpdateRequired;
		RunUpdate();
	}
}

AAvaShapeActor* UAvaShapeDynamicMeshBase::GetShapeActor() const
{
	return Cast<AAvaShapeActor>(GetOuter());
}

UDynamicMeshComponent* UAvaShapeDynamicMeshBase::GetShapeMeshComponent() const
{
	UDynamicMeshComponent* ShapeMeshComponent = CachedComponent.Get();

	// Only resolve shape mesh component when not in an update. Mesh Updates should've already validated this ahead of time in PreUpdateMesh
	if (!ShapeMeshComponent && !MeshUpdateContext.IsSet())
	{
		if (const AAvaShapeActor* ShapeActor = GetShapeActor())
		{
			ShapeMeshComponent = ShapeActor->GetShapeMeshComponent();
			if (ShapeMeshComponent)
			{
				UAvaShapeDynamicMeshBase* const MutableThis = const_cast<UAvaShapeDynamicMeshBase*>(this);
				MutableThis->CachedComponent = ShapeMeshComponent;
				MutableThis->InitializeDynamicMesh();
			}
		}
	}

	return ShapeMeshComponent;
}

const FString& UAvaShapeDynamicMeshBase::GetMeshName() const
{
	static FString MeshName = TEXT("BaseMesh");
	return MeshName;
}

void UAvaShapeDynamicMeshBase::SetSize2D(const FVector2D& InSize2D)
{
	if (!bAllowEditSize)
	{
		return;
	}

	FVector Size3D = GetSize3D();
	Size3D.Y = InSize2D.X;
	Size3D.Z = InSize2D.Y;
	SetSize3D(Size3D);
}

FVector2D UAvaShapeDynamicMeshBase::GetSize2D() const
{
	const FVector Size3D = GetSize3D();
	return FVector2D(Size3D.Y, Size3D.Z);
}

bool UAvaShapeDynamicMeshBase::IsMeshSectionVisible(int32 InIndex) const
{
	if (const FAvaShapeMeshData* MeshData = GetMeshData(InIndex))
	{
		return MeshData->bMeshVisible;
	}

	return false;
}

void UAvaShapeDynamicMeshBase::SetUsePrimaryMaterialEverywhere(bool bInUse)
{
	if (bUsePrimaryMaterialEverywhere == bInUse)
	{
		return;
	}

	bUsePrimaryMaterialEverywhere = bInUse;
	OnUsePrimaryMaterialEverywhereChanged();
}

void UAvaShapeDynamicMeshBase::SetUniformScaledSize(float InSize)
{
	if (UniformScaledSize == InSize)
	{
		return;
	}

	UniformScaledSize = InSize;
	OnScaledSizeChanged();
}

void UAvaShapeDynamicMeshBase::OnUsePrimaryMaterialEverywhereChanged()
{
	if (!IsValidMeshIndex(MESH_INDEX_PRIMARY))
	{
		return;
	}

	if (bUsePrimaryMaterialEverywhere)
	{
		// create a copy
		const EMaterialType PrimaryMaterialType = GetMaterialType(MESH_INDEX_PRIMARY);
		const FAvaShapeParametricMaterial PrimaryParametricMaterial = GetParametricMaterial(MESH_INDEX_PRIMARY);

		const TSet<int32> Keys = GetMeshesIndexes();

		for (const int32 MeshIndex : Keys)
		{
			if (MeshIndex == MESH_INDEX_PRIMARY)
			{
				continue;
			}

			FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex);
			MeshData->MaterialType = PrimaryMaterialType;
			if (PrimaryMaterialType == EMaterialType::Parametric)
			{
				MeshData->ParametricMaterial = PrimaryParametricMaterial;
			}
			MeshData->bUsesPrimaryMaterialParams = true;
			SetMaterialDirect(MeshIndex, GetMaterial(MESH_INDEX_PRIMARY));
		}
	}
	else
	{
		const TSet<int32> Keys = GetMeshesIndexes();

		for (const int32 MeshIndex : Keys)
		{
			if (MeshIndex == MESH_INDEX_PRIMARY)
			{
				continue;
			}

			FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex);
			MeshData->bUsesPrimaryMaterialParams = false;

			// Reset to proper state if it was a parametric material
			if (GetParametricMaterialPtr(MESH_INDEX_PRIMARY)->IsParametricMaterial(GetMaterial(MeshIndex)))
			{
				MeshData->MaterialType = EMaterialType::Parametric;
				MeshData->Material = MeshData->ParametricMaterial.GetOrCreateMaterial(GetOwner());
			}

			OnMaterialChanged(MeshIndex);
		}
	}
}

void UAvaShapeDynamicMeshBase::OnMeshTransformChanged(USceneComponent* InComponent, EUpdateTransformFlags InTransformFlags, ETeleportType InTeleport)
{
	/*
	 * TransformUpdated gets broadcasted when component is registered first time so it will have the correct starting value (see UpdateComponentToWorld in UActorComponent::OnRegister)
	 * Further calls affecting transform value, will have the proper zero scaled status
	 */
	const bool bIsZeroScaled = FMath::IsNearlyZero(InComponent->GetComponentScale().GetMin(), UE_KINDA_SMALL_NUMBER);

	if (bIsZeroScaled)
	{
		bZeroScaled = true;
	}

	// Only update when scale was zero and is not anymore
	if (bZeroScaled && !bIsZeroScaled)
	{
		MarkAllMeshesDirty();
		bZeroScaled = false;
	}
}

void UAvaShapeDynamicMeshBase::SetOverridePrimaryUVParams(int32 InMeshIndex, bool bInOverride)
{
	if (InMeshIndex != MESH_INDEX_PRIMARY)
	{
		if (FAvaShapeMeshData* MeshData = GetMeshData(InMeshIndex))
		{
			MeshData->bOverridePrimaryUVParams = bInOverride;
			OnUsesPrimaryUVParamsChanged(InMeshIndex);
		}
	}
}

bool UAvaShapeDynamicMeshBase::GetOverridePrimaryUVParams(int32 InMeshIndex) const
{
	if (InMeshIndex != MESH_INDEX_PRIMARY)
	{
		if (const FAvaShapeMeshData* MeshData = GetMeshData(InMeshIndex))
		{
			return MeshData->bOverridePrimaryUVParams;
		}
	}

	return false;
}

bool UAvaShapeDynamicMeshBase::IsMeshSizeValid() const
{
	if (const UDynamicMeshComponent* DynMeshComp = GetShapeMeshComponent())
	{
		if (DynMeshComp->GetComponentScale().GetMin() <= UE_KINDA_SMALL_NUMBER)
		{
			return false;
		}
		return true;
	}
	return false;
}

bool UAvaShapeDynamicMeshBase::GenerateMesh()
{
	if (!bAllMeshDirty && !bAnyMeshDirty)
	{
		return false;
	}

	bAllMeshDirty = false;
	bAnyMeshDirty = false;

	if (!ClearMesh())
	{
		return false;
	}

	const TSet<int32> Keys = GetMeshesIndexes();

	for (const int32 MeshIndex : Keys)
	{
		FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex);

		// Check State
		MeshData->bMeshDirty = IsMeshDirty(MeshIndex);

		if (!MeshData->bMeshDirty)
		{
			continue;
		}

		MeshData->bMeshDirty = false;

		// Check Visibility
		MeshData->bMeshVisible = IsMeshVisible(MeshIndex);

		if (!MeshData->bMeshVisible)
		{
			ClearDynamicMeshSection(MeshIndex);
			continue;
		}

		// generates vertices, normals & triangles if size is greater than min
		if (IsMeshSizeValid())
		{
			MeshData->Mesh.bUpdateRequired = CreateMesh(MeshData->Mesh);

			FAvaShapeMaterialUVParameters& InUseParams = *GetInUseMaterialUVParams(MeshIndex);
			InUseParams.bUVsDirty = true;
		}
	}

	bVerticesDirty = false;

	return true;
}

bool UAvaShapeDynamicMeshBase::GenerateUV()
{
	const TSet<int32> Keys = GetMeshesIndexes();

	bool bUVUpdated = false;
	for (const int32 MeshIndex : Keys)
	{
		FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex);

		if (!MeshData->bMeshVisible || MeshData->Mesh.Vertices.Num() == 0)
		{
			continue;
		}

		if (!GetInUseMaterialUVParams(MeshIndex)->bUVsDirty)
		{
			continue;
		}

		MeshData->Mesh.UVs.Empty();
		if (CreateUVs(MeshData->Mesh, *GetInUseMaterialUVParams(MeshIndex)))
		{
			bUVUpdated = true;
			MeshData->Mesh.bUpdateRequired = true;

			if (MeshIndex != MESH_INDEX_PRIMARY)
			{
				MeshData->MaterialUVParams.bUVsDirty = false;
			}
		}
	}

	GetMaterialUVParamsPtr(MESH_INDEX_PRIMARY)->bUVsDirty = false;

	return bUVUpdated;
}

void UAvaShapeDynamicMeshBase::OnMeshUpdateFinished()
{
	if (UActorModifierCoreStack* ShapeModifierStack = UActorModifierCoreSubsystem::Get()->GetActorModifierStack(GetShapeActor()))
	{
		// Set saved mesh otherwise current shape will be overwritten when modifier are restored
		TWeakObjectPtr<UAvaShapeDynamicMeshBase> ThisWeak(this);
		auto RestoreMeshFunction = [ThisWeak]()
		{
			UAvaShapeDynamicMeshBase* This = ThisWeak.Get();
			if (!This)
			{
				return;
			}

			This->RestoreCachedMesh(/** Reset */false);
		};

		// When stack is restored function will be called
		ShapeModifierStack->ProcessFunctionOnRestore(RestoreMeshFunction);

		// Mark stack dirty to re-execute it
		ShapeModifierStack->MarkModifierDirty();
	}
}

FTSTicker& UAvaShapeDynamicMeshBase::GetTicker() const
{
	if (const UWorld* const World = GetWorld())
	{
		if (UAvaSceneSubsystem* const SceneSubsystem = World->GetSubsystem<UAvaSceneSubsystem>())
		{
			return SceneSubsystem->GetTicker();
		}
	}
	UE_LOGF(LogAvaDynamicMesh, Verbose, "No valid Motion Design Scene Subsystem was found to retrieve ticker. Defaulting to Core Ticker.");
	return FTSTicker::GetCoreTicker();
}

FAvaShapeMesh* UAvaShapeDynamicMeshBase::GetMesh(int32 MeshIndex)
{
	if (FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex))
	{
		return &MeshData->Mesh;
	}

	return nullptr;
}

bool UAvaShapeDynamicMeshBase::IsValidMeshIndex(int32 InMeshIndex) const
{
	return MeshDatas.Contains(InMeshIndex);
}

void UAvaShapeDynamicMeshBase::SetMaterial(int32 InMeshIndex, UMaterialInterface* InNewMaterial)
{
	if (InMeshIndex == INDEX_NONE)
	{
		SetUsePrimaryMaterialEverywhere(true);
		InMeshIndex = MESH_INDEX_PRIMARY;
	}
	else if (InMeshIndex != MESH_INDEX_PRIMARY && GetUsePrimaryMaterialEverywhere())
	{
		SetUsePrimaryMaterialEverywhere(false);
	}

	if (SetMaterialDirect(InMeshIndex, InNewMaterial))
	{
		OnMaterialChanged(InMeshIndex);
	}
}

bool UAvaShapeDynamicMeshBase::SetMaterialDirect(int32 MeshIndex, UMaterialInterface* NewMaterial)
{
	if (FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex))
	{
		MeshData->Material = NewMaterial;
		SetShapeComponentMaterial(MeshIndex, NewMaterial);
		return true;
	}

	return false;
}

const FAvaShapeParametricMaterial& UAvaShapeDynamicMeshBase::GetParametricMaterial(int32 InMeshIndex) const
{
	if (const FAvaShapeMeshData* MeshData = GetMeshData(InMeshIndex))
	{
		return MeshData->ParametricMaterial;
	}

	static const FAvaShapeParametricMaterial DefaultParametricMaterial;
	return DefaultParametricMaterial;
}

FAvaShapeParametricMaterial* UAvaShapeDynamicMeshBase::GetParametricMaterialPtr(int32 InMeshIndex)
{
	if (FAvaShapeMeshData* MeshData = GetMeshData(InMeshIndex))
	{
		return &MeshData->ParametricMaterial;
	}

	return nullptr;
}

void UAvaShapeDynamicMeshBase::SetParametricMaterial(int32 InMeshIndex, const FAvaShapeParametricMaterial& InNewMaterialParams)
{
	if (FAvaShapeMeshData* MeshData = GetMeshData(InMeshIndex))
	{
		MeshData->ParametricMaterial = InNewMaterialParams;
		OnParametricMaterialChanged(InMeshIndex);
	}
}

UMaterialInterface* UAvaShapeDynamicMeshBase::GetMaterial(int32 InMeshIndex) const
{
	if (const FAvaShapeMeshData* MeshData = GetMeshData(InMeshIndex))
	{
		return MeshData->Material;
	}

	return nullptr;
}

const FAvaShapeMaterialUVParameters* UAvaShapeDynamicMeshBase::GetInUseMaterialUVParams(int32 MeshIndex) const
{
	if (const FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex))
	{
		if (!MeshData->bOverridePrimaryUVParams)
		{
			return &GetMeshData(MESH_INDEX_PRIMARY)->MaterialUVParams;
		}

		return &MeshData->MaterialUVParams;
	}

	return nullptr;
}

const FAvaShapeMaterialUVParameters& UAvaShapeDynamicMeshBase::GetMaterialUVParams(int32 MeshIndex) const
{
	if (const FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex))
	{
		return MeshData->MaterialUVParams;
	}

	static const FAvaShapeMaterialUVParameters DefaultMaterialUVParams;
	return DefaultMaterialUVParams;
}

FAvaShapeMaterialUVParameters* UAvaShapeDynamicMeshBase::GetInUseMaterialUVParams(int32 MeshIndex)
{
	if (FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex))
	{
		if (!MeshData->bOverridePrimaryUVParams)
		{
			return &GetMeshData(MESH_INDEX_PRIMARY)->MaterialUVParams;
		}

		return &MeshData->MaterialUVParams;
	}

	return nullptr;
}

FAvaShapeMaterialUVParameters* UAvaShapeDynamicMeshBase::GetMaterialUVParamsPtr(int32 MeshIndex)
{
	if (FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex))
	{
		return &MeshData->MaterialUVParams;
	}

	return nullptr;
}

void UAvaShapeDynamicMeshBase::SetMaterialUVParams(int32 InMeshIndex, const FAvaShapeMaterialUVParameters& InParams)
{
	if (FAvaShapeMeshData* MeshData = GetMeshData(InMeshIndex))
	{
		MeshData->MaterialUVParams = InParams;
		OnUVParamsChanged(InMeshIndex);
	}
}

EAvaShapeUVMode UAvaShapeDynamicMeshBase::GetMaterialUVMode(int32 MeshIndex) const
{
	return GetMaterialUVParams(MeshIndex).Mode;
}

bool UAvaShapeDynamicMeshBase::SetMaterialUVMode(int32 MeshIndex, EAvaShapeUVMode InUVMode)
{
	FAvaShapeMaterialUVParameters* Params = GetMaterialUVParamsPtr(MeshIndex);

	if (Params == nullptr)
	{
		return false;
	}

	if (Params->Mode == InUVMode)
	{
		return false;
	}

	Params->Mode = InUVMode;
	OnUVParamsChanged(MeshIndex);

	return true;
}

EAvaAnchors UAvaShapeDynamicMeshBase::GetMaterialUVAnchorPreset(int32 MeshIndex) const
{
	return GetMaterialUVParams(MeshIndex).AnchorPreset;
}

bool UAvaShapeDynamicMeshBase::SetMaterialUVAnchorPreset(int32 MeshIndex,
	EAvaAnchors InUVAnchorPreset)
{
	FAvaShapeMaterialUVParameters* Params = GetMaterialUVParamsPtr(MeshIndex);

	if (Params == nullptr)
	{
		return false;
	}

	if (Params->AnchorPreset == InUVAnchorPreset)
	{
		return false;
	}

	Params->AnchorPreset = InUVAnchorPreset;
	OnUVParamsChanged(MeshIndex);

	return true;
}

float UAvaShapeDynamicMeshBase::GetMaterialUVRotation(int32 MeshIndex) const
{
	return GetMaterialUVParams(MeshIndex).Rotation;
}

bool UAvaShapeDynamicMeshBase::SetMaterialUVRotation(int32 MeshIndex, float InUVRotation)
{
	FAvaShapeMaterialUVParameters* Params = GetMaterialUVParamsPtr(MeshIndex);

	if (Params == nullptr)
	{
		return false;
	}

	if (Params->Rotation == InUVRotation)
	{
		return false;
	}

	if (InUVRotation < -360.f || InUVRotation > 360.f)
	{
		return false;
	}

	Params->Rotation = InUVRotation;
	OnUVParamsChanged(MeshIndex);

	return true;
}

const FVector2D& UAvaShapeDynamicMeshBase::GetMaterialUVAnchor(int32 MeshIndex) const
{
	return GetMaterialUVParams(MeshIndex).Anchor;
}

bool UAvaShapeDynamicMeshBase::SetMaterialUVAnchor(int32 MeshIndex, const FVector2D& InUVAnchor)
{
	FAvaShapeMaterialUVParameters* Params = GetMaterialUVParamsPtr(MeshIndex);

	if (Params == nullptr)
	{
		return false;
	}

	if (Params->Anchor == InUVAnchor)
	{
		return false;
	}

	Params->Anchor = InUVAnchor;
	OnUVParamsChanged(MeshIndex);

	return true;
}

const FVector2D& UAvaShapeDynamicMeshBase::GetMaterialUVScale(int32 MeshIndex) const
{
	return GetMaterialUVParams(MeshIndex).Scale;
}

bool UAvaShapeDynamicMeshBase::SetMaterialUVScale(int32 MeshIndex, const FVector2D& InUVScale)
{
	FAvaShapeMaterialUVParameters* Params = GetMaterialUVParamsPtr(MeshIndex);

	if (Params == nullptr)
	{
		return false;
	}

	if (Params->Scale == InUVScale)
	{
		return false;
	}

	if (InUVScale.X <= 0.f || InUVScale.Y <= 0.f)
	{
		return false;
	}

	Params->Scale = InUVScale;
	OnUVParamsChanged(MeshIndex);

	return true;
}

const FVector2D& UAvaShapeDynamicMeshBase::GetMaterialUVOffset(int32 MeshIndex) const
{
	return GetMaterialUVParams(MeshIndex).Offset;
}

bool UAvaShapeDynamicMeshBase::SetMaterialUVOffset(int32 MeshIndex, const FVector2D& InUVOffset)
{
	FAvaShapeMaterialUVParameters* Params = GetMaterialUVParamsPtr(MeshIndex);

	if (Params == nullptr)
	{
		return false;
	}

	if (Params->Offset == InUVOffset)
	{
		return false;
	}

	Params->Offset = InUVOffset;
	OnUVParamsChanged(MeshIndex);

	return true;
}

bool UAvaShapeDynamicMeshBase::GetMaterialHorizontalFlip(int32 MeshIndex) const
{
	return GetMaterialUVParams(MeshIndex).bFlipHorizontal;
}

bool UAvaShapeDynamicMeshBase::SetMaterialHorizontalFlip(int32 MeshIndex, bool InHorizontalFlip)
{
	FAvaShapeMaterialUVParameters* Params = GetMaterialUVParamsPtr(MeshIndex);

	if (Params == nullptr)
	{
		return false;
	}

	if (Params->bFlipHorizontal == InHorizontalFlip)
	{
		return false;
	}

	Params->bFlipHorizontal = InHorizontalFlip;
	OnUVParamsChanged(MeshIndex);

	return true;
}

bool UAvaShapeDynamicMeshBase::GetMaterialVerticalFlip(int32 MeshIndex) const
{
	return GetMaterialUVParams(MeshIndex).bFlipVertical;
}

bool UAvaShapeDynamicMeshBase::SetMaterialVerticalFlip(int32 MeshIndex, bool InVerticalFlip)
{
	FAvaShapeMaterialUVParameters* Params = GetMaterialUVParamsPtr(MeshIndex);

	if (Params == nullptr)
	{
		return false;
	}

	if (Params->bFlipVertical == InVerticalFlip)
	{
		return false;
	}

	Params->bFlipVertical = InVerticalFlip;
	OnUVParamsChanged(MeshIndex);

	return true;
}

bool UAvaShapeDynamicMeshBase::GenerateTangents()
{
	UDynamicMeshComponent* DynMeshComp = GetShapeMeshComponent();

	if (!DynMeshComp)
	{
		return false;
	}

	// Copied from UGeometryScriptLibrary_MeshNormalsFunctions::ComputeTangents to not broadcast any change
	UE::Private::AvaShape::EditMesh(this,
		[this](FDynamicMesh3& EditMesh)
		{
			constexpr int32 UVLayer = 0;
			if (!EditMesh.HasAttributes() || EditMesh.Attributes()->NumUVLayers() <= UVLayer || EditMesh.Attributes()->NumNormalLayers() == 0)
			{
				return;
			}
			if (EditMesh.Attributes()->HasTangentSpace() == false)
			{
				EditMesh.Attributes()->EnableTangents();
			}

			FComputeTangentsOptions TangentOptions;
			TangentOptions.bAveraged = true;
			TangentOptions.bParallel = !MeshUpdateContext.IsSet() || !MeshUpdateContext->bParallel; // do not do parallel operation if already parallel.

			FMeshTangentsd Tangents(&EditMesh);
			Tangents.ComputeTriVertexTangents(
				EditMesh.Attributes()->PrimaryNormals(),
				EditMesh.Attributes()->GetUVLayer(UVLayer),
				TangentOptions);
			Tangents.CopyToOverlays(EditMesh);
		}
		, EDynamicMeshChangeType::GeneralEdit
		, EDynamicMeshAttributeChangeFlags::Unknown);

	return true;
}

bool UAvaShapeDynamicMeshBase::GenerateNormals()
{
	UDynamicMeshComponent* DynMeshComp = GetShapeMeshComponent();

	if (!DynMeshComp)
	{
		return false;
	}

	const FGeometryScriptCalculateNormalsOptions NormalOptions;

	UGeometryScriptLibrary_MeshNormalsFunctions::RecomputeNormals(DynMeshComp->GetDynamicMesh(), NormalOptions);

	return true;
}

void UAvaShapeDynamicMeshBase::OnUVParamsChanged(int32 MeshIndex)
{
	if (FAvaShapeMaterialUVParameters* UVParams = GetMaterialUVParamsPtr(MeshIndex))
	{
		UVParams->bUVsDirty = true;
		RequestUpdate();
	}
}

void UAvaShapeDynamicMeshBase::OnUsesPrimaryUVParamsChanged(int32 MeshIndex)
{
	FAvaShapeMeshData* PrimaryMeshData = GetMeshData(MESH_INDEX_PRIMARY);
	FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex);

	if (PrimaryMeshData && MeshData)
	{
		if ((MeshData->ParametricMaterial.IsParametricMaterial(MeshData->Material)) && MeshData->bOverridePrimaryUVParams)
		{
			MeshData->MaterialType = EMaterialType::Parametric;
		}

		if (!MeshData->bOverridePrimaryUVParams)
		{
			PrimaryMeshData->MaterialUVParams.bUVsDirty = true;
		}
		else
		{
			MeshData->MaterialUVParams.bUVsDirty = true;
		}
	}
}

void UAvaShapeDynamicMeshBase::OnMeshChanged(int32 MeshIndex)
{
	MarkAllMeshesDirty();
}

void UAvaShapeDynamicMeshBase::OnMaterialChanged(int32 MaterialIndex)
{
	if (!IsValidMeshIndex(MaterialIndex))
	{
		return;
	}

	FAvaShapeMeshData* MeshSection = GetMeshData(MaterialIndex);
	UMaterialInterface* Material = MeshSection->Material;

	// Switch back to parametric material when material is cleared.
	if (Material == nullptr)
	{
		Material = MeshSection->ParametricMaterial.GetOrCreateMaterial(GetShapeActor());
		SetMaterialDirect(MaterialIndex, Material);
	}

	if (MeshSection->ParametricMaterial.IsParametricMaterial(Material))
	{
		MeshSection->MaterialType = EMaterialType::Parametric;
	}
	else if (Cast<UDynamicMaterialInstance>(Material) != nullptr)
	{
		MeshSection->MaterialType = EMaterialType::MaterialDesigner;
	}
	else
	{
		MeshSection->MaterialType = EMaterialType::Asset;
	}

	SetShapeComponentMaterial(MaterialIndex, Material);

	GetInUseMaterialUVParams(MaterialIndex)->bUVsDirty = true;

	if (bUsePrimaryMaterialEverywhere)
	{
		// update all material slots if override enabled
		if (MaterialIndex == MESH_INDEX_PRIMARY)
		{
			OnUsePrimaryMaterialEverywhereChanged();
		}
		// disable override if we change a material different than primary
		else if (Material != GetMaterial(MESH_INDEX_PRIMARY))
		{
			bUsePrimaryMaterialEverywhere = false;
		}
	}
}

void UAvaShapeDynamicMeshBase::OnMaterialTypeChanged(int32 MaterialIndex)
{
	if (const FAvaShapeMeshData* MeshData = GetMeshData(MaterialIndex))
	{
		switch (MeshData->MaterialType)
		{
			case(EMaterialType::Asset):
			{
				UMaterialInterface* CurrentMaterial = GetMaterial(MaterialIndex);

				if (!IsValid(CurrentMaterial) || CurrentMaterial->IsA<UDynamicMaterialInstance>())
				{
					CurrentMaterial = nullptr;
				}

				if (!bUsePrimaryMaterialEverywhere)
				{
					SetMaterialDirect(MaterialIndex, CurrentMaterial);
				}
				else
				{
					for (const int32 SectionIndex : GetMeshesIndexes())
					{
						SetMaterialDirect(SectionIndex, CurrentMaterial);
					}
				}

				break;
			}

			case(EMaterialType::Parametric):
				SetMaterial(MaterialIndex, nullptr);
				break;

			case(EMaterialType::MaterialDesigner):
			{
				if (Cast<UDynamicMaterialInstance>(GetMaterial(MaterialIndex)) == nullptr)
				{
#if WITH_EDITOR
					UDynamicMaterialInstanceFactory* DynamicMaterialInstanceFactory = NewObject<UDynamicMaterialInstanceFactory>();
					check(DynamicMaterialInstanceFactory);

					UDynamicMaterialInstance* NewInstance = Cast<UDynamicMaterialInstance>(DynamicMaterialInstanceFactory->FactoryCreateNew(
						UDynamicMaterialInstance::StaticClass(),
						this,
						NAME_None,
						RF_Transactional,
						nullptr,
						GWarn
					));
#else
					UDynamicMaterialInstance* NewInstance = NewObject<UDynamicMaterialInstance>(this);
#endif
					SetMaterial(MaterialIndex, NewInstance);

#if WITH_EDITOR
					const FDMObjectMaterialProperty ObjectMaterialProperty = FDMObjectMaterialProperty(GetShapeMeshComponent(), MaterialIndex);

					const IDynamicMaterialEditorModule& MaterialDesignerModule = IDynamicMaterialEditorModule::Get();
					MaterialDesignerModule.OpenMaterialObjectProperty(ObjectMaterialProperty, GetWorld(), /* Invoke Tab */ true);
#endif
				}
				break;
			}
		}
	}
}

void UAvaShapeDynamicMeshBase::OnParametricMaterialChanged(int32 MaterialIndex)
{
	if (!IsValidMeshIndex(MaterialIndex))
	{
		return;
	}

	FAvaShapeParametricMaterial* ParametricMaterial = GetParametricMaterialPtr(MaterialIndex);

	UMaterialInterface* ParametricMaterialInstance = ParametricMaterial->GetOrCreateMaterial(GetShapeActor());

	UMaterialInterface* CurrentMaterial = GetMaterial(MaterialIndex);

	// We're using the wrong trans/opaque settings
	if (ParametricMaterial->IsParametricMaterial(CurrentMaterial) && CurrentMaterial != ParametricMaterialInstance)
	{
		SetMaterial(MaterialIndex, ParametricMaterialInstance);
	}

	// We've toggled on our parametric material
	if (IsMaterialType(MaterialIndex, EMaterialType::Parametric) && CurrentMaterial != ParametricMaterialInstance)
	{
		SetMaterial(MaterialIndex, ParametricMaterialInstance);
	}

	if (MaterialIndex == MESH_INDEX_PRIMARY)
	{
		FAvaShapeMeshData* MeshData = GetMeshData(MESH_INDEX_PRIMARY);

		if (MeshData->ParametricMaterial.IsParametricMaterial(GetMaterial(MESH_INDEX_PRIMARY)))
		{
			MeshData->MaterialType = EMaterialType::Parametric;
		}

		if (bUsePrimaryMaterialEverywhere)
		{
			OnUsePrimaryMaterialEverywhereChanged();
		}
	}
	else
	{
		FAvaShapeMeshData* MeshData = GetMeshData(MaterialIndex);

		if (MeshData->ParametricMaterial.IsParametricMaterial(GetMaterial(MaterialIndex)))
		{
			MeshData->MaterialType = EMaterialType::Parametric;
		}
	}
}

bool UAvaShapeDynamicMeshBase::ClearMesh()
{
	const TSet<int32> Keys = GetMeshesIndexes();

	for (const int32 MeshIndex : Keys)
	{
		if (FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex); MeshData->bMeshDirty)
		{
			// we do not clear the section here, we do it after if we detect a topology change
			MeshData->Mesh.Clear();
		}
	}

	return true;
}

bool UAvaShapeDynamicMeshBase::IsDynamicMeshInitialized() const
{
	bool bHasMeshData = false;
	if (UDynamicMeshComponent* DynMeshComp = GetShapeMeshComponent())
	{
		DynMeshComp->GetDynamicMesh()->ProcessMesh([&bHasMeshData](const FDynamicMesh3& InMesh)
		{
			bHasMeshData = InMesh.HasAttributes()
			&& InMesh.Attributes()->NumNormalLayers() > 0
			&& InMesh.Attributes()->NumPolygroupLayers() > 0
			&& InMesh.Attributes()->NumUVLayers() > 0;
		});
	}
	return bHasMeshData;
}

void UAvaShapeDynamicMeshBase::UpdateVertices()
{
	if (!bVerticesDirty)
	{
		return;
	}

	bVerticesDirty = false;

	const TSet<int32> Keys = GetMeshesIndexes();

	for (const int32 MeshIndex : Keys)
	{
		FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex);
		MeshData->Mesh.bUpdateRequired = true;
	}
}

bool UAvaShapeDynamicMeshBase::OnMeshSectionUpdated(FAvaShapeMesh& InMesh)
{
	if (InMesh.Vertices.Num() == 0)
	{
		ClearDynamicMeshSection(InMesh.GetMeshIndex());
		return true;
	}

	if (!MeshUpdateContext.IsSet())
	{
		InMesh.bUpdateRequired = true;
		return true;
	}

	return UpdateDynamicMesh(InMesh);
}

EAvaAnchors UAvaShapeDynamicMeshBase::GetAnchorFromNumerics(const FVector2D& AnchorNumeric)
{
	if (AnchorNumeric.X == 0.f && AnchorNumeric.Y == 0.f)
	{
		return EAvaAnchors::TopLeft;
	}

	if (AnchorNumeric.X == 0.5f && AnchorNumeric.Y == 0.f)
	{
		return EAvaAnchors::Top;
	}

	if (AnchorNumeric.X == 1.f && AnchorNumeric.Y == 0.f)
	{
		return EAvaAnchors::TopRight;
	}

	if (AnchorNumeric.X == 0.0f && AnchorNumeric.Y == 0.5f)
	{
		return EAvaAnchors::Left;
	}

	if (AnchorNumeric.X == 0.5f && AnchorNumeric.Y == 0.5f)
	{
		return EAvaAnchors::Center;
	}

	if (AnchorNumeric.X == 1.f && AnchorNumeric.Y == 0.5f)
	{
		return EAvaAnchors::Right;
	}

	if (AnchorNumeric.X == 0.f && AnchorNumeric.Y == 1.f)
	{
		return EAvaAnchors::BottomLeft;
	}

	if (AnchorNumeric.X == 0.5f && AnchorNumeric.Y == 1.f)
	{
		return EAvaAnchors::Bottom;
	}

	if (AnchorNumeric.X == 1.f && AnchorNumeric.Y == 1.f)
	{
		return EAvaAnchors::BottomRight;
	}

	return EAvaAnchors::Custom;
}

FVector2D UAvaShapeDynamicMeshBase::GetNumericsFromAnchor(EAvaAnchors AnchorEnum)
{
	FVector2D Anchor = FVector2D::ZeroVector;

	switch (ToUnderlyingType<EAvaAnchors>(AnchorEnum) & UE::Ava::AnchorPoints::Horizontal)
	{
	default:
		//falls through

	case UE::Ava::AnchorPoints::Left:
		// nothing to do
		break;

	case UE::Ava::AnchorPoints::HMiddle:
		Anchor.X = 0.5f;
		break;

	case UE::Ava::AnchorPoints::Right:
		Anchor.X = 1.f;
		break;
	}

	switch (ToUnderlyingType<EAvaAnchors>(AnchorEnum) & UE::Ava::AnchorPoints::Vertical)
	{
	default:
		//falls through

	case UE::Ava::AnchorPoints::Top:
		// nothing to do
		break;

	case UE::Ava::AnchorPoints::VMiddle:
		Anchor.Y = 0.5f;
		break;

	case UE::Ava::AnchorPoints::Bottom:
		Anchor.Y = 1.f;
		break;
	}

	return Anchor;
}

bool UAvaShapeDynamicMeshBase::HasSameTopology(FAvaShapeMesh& InMesh)
{
	UDynamicMeshComponent* ProcMesh = GetShapeMeshComponent();

	if (!ProcMesh)
	{
		return false;
	}

	return !(InMesh.GetMeshIndex() >= ProcMesh->GetMesh()->MaxGroupID()
		|| InMesh.Vertices.Num() != InMesh.VerticeIds.Num()
		|| (InMesh.Triangles.Num() / 3) != InMesh.TriangleIds.Num());
}

bool UAvaShapeDynamicMeshBase::CreateDynamicMesh(FAvaShapeMesh& InMesh)
{
	UDynamicMeshComponent* ProcMesh = GetShapeMeshComponent();

	if (!ProcMesh)
	{
		return false;
	}

	// vertices and triangles should not exists if we create them
	if (InMesh.VerticeIds.Num() != 0 || InMesh.TriangleIds.Num() != 0)
	{
		UE_LOGF(LogAvaDynamicMesh, Warning, "CreateDynamicMesh %ls %i : VerticeIds %i, TriangleIds %i should be empty to create mesh", *GetMeshName(), InMesh.GetMeshIndex(), InMesh.VerticeIds.Num(), InMesh.TriangleIds.Num());
		return false;
	}

	// should have same number for vertices
	if (!(InMesh.Vertices.Num() == InMesh.Normals.Num()))
	{
		UE_LOGF(LogAvaDynamicMesh, Warning, "CreateDynamicMesh %ls %i : Arrays Vertices %i, Normals %i should have same length, invalid array given", *GetMeshName(), InMesh.GetMeshIndex(), InMesh.Vertices.Num(), InMesh.Normals.Num());
		return false;
	}

	// should be multiple of 3
	if ((InMesh.Triangles.Num() % 3) != 0)
	{
		UE_LOGF(LogAvaDynamicMesh, Warning, "CreateDynamicMesh %ls %i : Triangles array should be multiple of 3, %i invalid array given", *GetMeshName(), InMesh.GetMeshIndex(), InMesh.Triangles.Num());
		return false;
	}

	InMesh.bUpdateRequired = false;

	UE::Private::AvaShape::EditMesh(this,
		[this, &InMesh](FDynamicMesh3& Mesh)
		{
			// clear ids before adding them
			InMesh.ClearIds();

			FDynamicMeshNormalOverlay* const NormalOverlay = Mesh.Attributes()->PrimaryNormals();
			FDynamicMeshMaterialAttribute* const MaterialAttr = Mesh.Attributes()->GetMaterialID();
			FDynamicMeshPolygroupAttribute* const PolyGroupAttr = Mesh.Attributes()->GetPolygroupLayer(InMesh.GetMeshIndex());

			// process vertices infos
			for (int32 v = 0; v < InMesh.Vertices.Num(); v++)
			{
				int32 VId = Mesh.AppendVertex(InMesh.Vertices[v]);
				InMesh.VerticeIds.Add(VId);

				FVector3f Normal = static_cast<FVector3f>(InMesh.Normals[v]);
				int32 NId = NormalOverlay->AppendElement(Normal);
				InMesh.NormalIds.Add(NId);
			}

			// process triangles
			for (int32 t = 0; t < InMesh.Triangles.Num(); t += 3)
			{
				int32 Idx1 = InMesh.Triangles[t];
				int32 Idx2 = InMesh.Triangles[t + 1];
				int32 Idx3 = InMesh.Triangles[t + 2];

				if (!InMesh.VerticeIds.IsValidIndex(Idx1) ||
					!InMesh.VerticeIds.IsValidIndex(Idx2) ||
					!InMesh.VerticeIds.IsValidIndex(Idx3))
				{
					UE_LOGF(LogAvaDynamicMesh, Warning, "CreateDynamicMesh %ls %i : Invalid Vertice idx for triangle %i %i %i", *GetMeshName(), InMesh.GetMeshIndex(), Idx1, Idx2, Idx3);
					continue;
				}

				// get vertice id from DM
				int32 VId1 = InMesh.VerticeIds[Idx1];
				int32 VId2 = InMesh.VerticeIds[Idx2];
				int32 VId3 = InMesh.VerticeIds[Idx3];
				// create triangle from vertices id
				int32 TId = Mesh.AppendTriangle(VId1, VId2, VId3, InMesh.GetMeshIndex());
				if (TId < 0)
				{
					UE_LOGF(LogAvaDynamicMesh, Warning, "CreateDynamicMesh %ls %i : Invalid Triangle ID for mesh", *GetMeshName(), InMesh.GetMeshIndex());
					continue;
				}
				InMesh.TriangleIds.Add(TId);
				// material
				MaterialAttr->SetValue(TId, InMesh.GetMeshIndex());
				// poly group
				PolyGroupAttr->SetValue(TId, InMesh.GetMeshIndex());
				// normals
				const FIndex3i NormalIds(InMesh.NormalIds[Idx1], InMesh.NormalIds[Idx2], InMesh.NormalIds[Idx3]);
				NormalOverlay->SetTriangle(TId, NormalIds, true);
			}
		}
		, EDynamicMeshChangeType::GeneralEdit
		, EDynamicMeshAttributeChangeFlags::MeshTopology);
	return true;
}

bool UAvaShapeDynamicMeshBase::UpdateDynamicMesh(FAvaShapeMesh& InMesh)
{
	UDynamicMeshComponent* ProcMesh = GetShapeMeshComponent();

	if (!ProcMesh)
	{
		return false;
	}

	// different topology, create mesh section
	if (InMesh.GetMeshIndex() >= ProcMesh->GetMesh()->MaxGroupID() || InMesh.Vertices.Num() != InMesh.VerticeIds.Num())
	{
		// clear previous triangles
		return ClearDynamicMeshSection(InMesh.GetMeshIndex()) && CreateDynamicMesh(InMesh);
	}

	// should have same number for vertices to update
	if (!(InMesh.Vertices.Num() == InMesh.Normals.Num()))
	{
		UE_LOGF(LogAvaDynamicMesh, Warning, "UpdateDynamicMesh %ls %i: Arrays Vertices %i, Normals %i should have same length, invalid array given", *GetMeshName(), InMesh.GetMeshIndex(), InMesh.Vertices.Num(), InMesh.Normals.Num());
		return false;
	}

	InMesh.bUpdateRequired = false;

	// same topology, update vertices only (location, normal, color, uv)
	UE::Private::AvaShape::EditMesh(this,
		[this, InMesh](FDynamicMesh3& EditMesh)
		{
			FDynamicMeshNormalOverlay* const NormalOverlay = EditMesh.Attributes()->PrimaryNormals();

			auto DoWork = [InMesh, &EditMesh, NormalOverlay](int32 Idx)
				{
					// vertices
					int32 VId = InMesh.VerticeIds[Idx];
					EditMesh.SetVertex(VId, InMesh.Vertices[Idx]);
					// normals
					int32 NId = InMesh.NormalIds[Idx];
					NormalOverlay->SetElement(NId, static_cast<FVector3f>(InMesh.Normals[Idx]));
				};

			if (MeshUpdateContext.IsSet() && MeshUpdateContext->bParallel)
			{
				for (int32 Idx = 0; Idx < InMesh.Vertices.Num(); ++Idx)
				{
					DoWork(Idx);
				}
			}
			else
			{
				ParallelFor(InMesh.Vertices.Num(), DoWork);
			}
		}
		, EDynamicMeshChangeType::MeshVertexChange
		, EDynamicMeshAttributeChangeFlags::Unknown);

	return true;
}

#if WITH_EDITOR
void UAvaShapeDynamicMeshBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static const FName ScaledSizeName = GET_MEMBER_NAME_CHECKED(UAvaShapeDynamicMeshBase, UniformScaledSize);
	static const FName UsePrimaryMaterialEverywhereName = GET_MEMBER_NAME_CHECKED(UAvaShapeDynamicMeshBase, bUsePrimaryMaterialEverywhere);

	const FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (MemberPropertyName.IsEqual(ScaledSizeName))
	{
		OnScaledSizeChanged();
	}
	else if (PropertyName.IsEqual(UsePrimaryMaterialEverywhereName))
	{
		OnUsePrimaryMaterialEverywhereChanged();
	}
}

void UAvaShapeDynamicMeshBase::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	auto GetChainPropertyNames = [](const FPropertyChangedChainEvent& InPropertyChangedEvent)->TArray<FName>
	{
		TArray<FName> PropertyNames;
		FEditPropertyChain::TDoubleLinkedListNode* Node = InPropertyChangedEvent.PropertyChain.GetActiveMemberNode();

		while (Node)
		{
			const FProperty* Property = Node->GetValue();

			if (!Property)
			{
				break;
			}

			PropertyNames.Add(Property->GetFName());
			Node = Node->GetNextNode();
		}

		Algo::Reverse(PropertyNames);
		return PropertyNames;
	};

	static const FName MeshDatasName = GET_MEMBER_NAME_CHECKED(UAvaShapeDynamicMeshBase, MeshDatas);

	TArray<FName> PropertyNames = GetChainPropertyNames(PropertyChangedEvent);

	const FName MemberPropertyName = !PropertyNames.IsEmpty() ? PropertyNames.Pop() : NAME_None;

	if (MemberPropertyName == MeshDatasName)
	{
		const int32 MeshDatasIndex = PropertyChangedEvent.GetArrayIndex(MeshDatasName.ToString());

		const TArray<int32> MeshesIndexes = GetMeshesIndexes().Array();
		if (!MeshesIndexes.IsValidIndex(MeshDatasIndex))
		{
			return;
		}

		const int32 MeshIndex = MeshesIndexes[MeshDatasIndex];

		static const FName MaterialTypeName = GET_MEMBER_NAME_CHECKED(FAvaShapeMeshData, MaterialType);
		static const FName MaterialName = GET_MEMBER_NAME_CHECKED(FAvaShapeMeshData, Material);
		static const FName ParametricMaterialName = GET_MEMBER_NAME_CHECKED(FAvaShapeMeshData, ParametricMaterial);
		static const FName UsesPrimaryUVParamsName = GET_MEMBER_NAME_CHECKED(FAvaShapeMeshData, bOverridePrimaryUVParams);
		static const FName MaterialUVParamsName = GET_MEMBER_NAME_CHECKED(FAvaShapeMeshData, MaterialUVParams);

		const FName MeshSectionProperty = !PropertyNames.IsEmpty() ? PropertyNames.Pop() : NAME_None;

		if (MeshSectionProperty == MaterialTypeName)
		{
			OnMaterialTypeChanged(MeshIndex);
		}
		else if (MeshSectionProperty == MaterialName)
		{
			OnMaterialChanged(MeshIndex);
		}
		else if (MeshSectionProperty == ParametricMaterialName)
		{
			MeshDatas[MeshIndex].ParametricMaterial.PostEditChange(PropertyNames);
		}
		else if (MeshSectionProperty == UsesPrimaryUVParamsName)
		{
			OnUsesPrimaryUVParamsChanged(MeshIndex);
		}
		else if (MeshSectionProperty == MaterialUVParamsName)
		{
			static const FName AnchorPresetName = GET_MEMBER_NAME_CHECKED(FAvaShapeMaterialUVParameters, AnchorPreset);
			static const FName AnchorName = GET_MEMBER_NAME_CHECKED(FAvaShapeMaterialUVParameters, Anchor);
			static const FName ScaleName = GET_MEMBER_NAME_CHECKED(FAvaShapeMaterialUVParameters, Scale);

			FAvaShapeMaterialUVParameters* UVParams = GetMaterialUVParamsPtr(MeshIndex);
			const FName PropertyName = PropertyChangedEvent.GetPropertyName();

			if (PropertyName == AnchorPresetName)
			{
				UVParams->Anchor = GetNumericsFromAnchor(UVParams->AnchorPreset);
			}
			else if (PropertyName == AnchorName)
			{
				UVParams->AnchorPreset = GetAnchorFromNumerics(UVParams->Anchor);
			}
			else if (PropertyName == ScaleName)
			{
				if (UVParams->Scale.X == 0)
				{
					UVParams->Scale.X = 1;
				}
				if (UVParams->Scale.Y == 0)
				{
					UVParams->Scale.Y = 1;
				}
			}

			OnUVParamsChanged(MeshIndex);
		}
	}
}

void UAvaShapeDynamicMeshBase::PreEditUndo()
{
	Super::PreEditUndo();
}

void UAvaShapeDynamicMeshBase::PostEditUndo()
{
	Super::PostEditUndo();

	CachedMesh.Reset();
	ClearDynamicMesh();

	CachedComponent.Reset();

	MeshUpdateStatus = EAvaDynamicMeshUpdateState::UpToDate;
	MarkAllMeshesDirty();
}

void UAvaShapeDynamicMeshBase::OnAssetDropped(UObject* DroppedObj, AActor* TargetActor)
{
	if (TargetActor == GetShapeActor())
	{
		if (CheckMaterialSlotChanges())
		{
			// refresh details panel in editor
			const FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

			for (const FName& DetailViewId : FAvaLevelEditorUtils::GetDetailsViewNames())
			{
				if (const TSharedPtr<IDetailsView> DetailView = PropertyModule.FindDetailView(DetailViewId))
				{
					DetailView->ForceRefresh();
				}
			}
		}
	}
}

void UAvaShapeDynamicMeshBase::PostEditImport()
{
	Super::PostEditImport();

	// reset cache component
	CachedComponent.Reset();

	const bool bUseSingleMaterial = bUsePrimaryMaterialEverywhere;

	// Update material options
	for (const int32 MeshIdx : GetMeshesIndexes())
	{
		OnMaterialTypeChanged(MeshIdx);
	}

	SetUsePrimaryMaterialEverywhere(bUseSingleMaterial);
}

void UAvaShapeDynamicMeshBase::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	// reset cache component
	CachedComponent.Reset();
}
#endif

bool UAvaShapeDynamicMeshBase::CheckMaterialSlotChanges()
{
	const UDynamicMeshComponent* DynMeshComp = GetShapeMeshComponent();
	bool bResult = false;

	for (const int32 SectionIdx : GetMeshesIndexes())
	{
		UMaterialInterface* SlotMat = DynMeshComp->GetMaterial(SectionIdx);
		const UMaterialInterface* SectionMat = GetMaterial(SectionIdx);
		if (SlotMat != SectionMat)
		{
			bResult = true;
			SetMaterial(SectionIdx, SlotMat);
		}
	}

	return bResult;
}

void UAvaShapeDynamicMeshBase::SetShapeComponentMaterial(int32 InMaterialIndex, UMaterialInterface* InNewMaterial)
{
	UDynamicMeshComponent* ShapeComponent = GetShapeMeshComponent();

	if (!IsValid(ShapeComponent) || InMaterialIndex >= ShapeComponent->GetNumMaterials())
	{
		return;
	}

	const UMaterialInterface* CurrentMaterial = ShapeComponent->GetMaterial(InMaterialIndex);

	if (InNewMaterial == CurrentMaterial)
	{
		return;
	}

	ShapeComponent->SetMaterial(InMaterialIndex, InNewMaterial);
}

void UAvaShapeDynamicMeshBase::ScaleVertices(const FVector& InScale)
{
	if (InScale.IsNearlyZero())
	{
		return;
	}

	for (const int32 MeshIndex : GetMeshesIndexes())
	{
		FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex);

		for (FVector& Vertex : MeshData->Mesh.Vertices)
		{
			Vertex.X *= InScale.X;
			Vertex.Y *= InScale.Y;
			Vertex.Z *= InScale.Z;
		}

		MeshData->MaterialUVParams.bUVsDirty = true;
	}

	for (FAvaSnapPoint& SnapPoint : LocalSnapPoints)
	{
		SnapPoint.Location.Y *= InScale.Y;
		SnapPoint.Location.Z *= InScale.Z;
	}

	MarkVerticesDirty();
}

void UAvaShapeDynamicMeshBase::ScaleVertices(const FVector2D& InScale)
{
	ScaleVertices(FVector(1.f, InScale.X, InScale.Y));
}

void UAvaShapeDynamicMeshBase::SetMeshRegenWorldLocation(const FVector& NewLocation, bool bImmediateUpdate)
{
	MeshRegenWorldLocation = NewLocation;
	InvalidateSection(bHasNewMeshRegenWorldLocation, bImmediateUpdate);
}

bool UAvaShapeDynamicMeshBase::ApplyUVsManually(FAvaShapeMesh& InMesh)
{
	if (InMesh.TriangleIds.IsEmpty())
	{
		return false;
	}

	if (InMesh.VerticeIds.Num() != InMesh.UVs.Num())
	{
		return false;
	}

	InMesh.UVIds.Empty();

	return UE::Private::AvaShape::EditMesh(this,
		[this, &InMesh](FDynamicMesh3& EditMesh)
		{
			FDynamicMeshUVOverlay* const UVOverlay = EditMesh.Attributes()->GetUVLayer(0);

			// used to quickly find index O(1) instead of O(n)
			TMap<int32, int32> BaseToOverlayVIDMap;
			for (int32 Idx = 0; Idx < InMesh.VerticeIds.Num(); Idx++)
			{
				FVector2f UV = static_cast<FVector2f>(InMesh.UVs[Idx]);
				int32 UId = UVOverlay->AppendElement(UV);
				InMesh.UVIds.Add(UId);
				BaseToOverlayVIDMap.Add(InMesh.VerticeIds[Idx], UId);
			}

			for (int32 TID : InMesh.TriangleIds)
			{
				FIndex3i TriVtx = EditMesh.GetTriangle(TID);
				FIndex3i UVElem;
				for (int32 j = 0; j < 3; ++j)
				{
					const int32* FoundElementID = BaseToOverlayVIDMap.Find(TriVtx[j]);
					if (FoundElementID == nullptr)
					{
						UE_LOGF(LogAvaDynamicMesh, Warning, "ApplyUVsManually %ls %i: vertice id %i for uv is invalid, skipping triangle", *GetMeshName(), InMesh.GetMeshIndex(), TriVtx[j]);
						break;
					}
					else
					{
						UVElem[j] = *FoundElementID;
					}
				}
				UVOverlay->SetTriangle(TID, UVElem);
			}
		}
		, EDynamicMeshChangeType::AttributeEdit
		, EDynamicMeshAttributeChangeFlags::UVs);
}

bool UAvaShapeDynamicMeshBase::ApplyUVsPlanarProjection(FAvaShapeMesh& InMesh, FRotator PlaneRotation, FVector2D PlaneSize)
{
	if (InMesh.TriangleIds.IsEmpty())
	{
		return false;
	}

	return UE::Private::AvaShape::EditMesh(this,
		[this, &InMesh, PlaneRotation, PlaneSize](FDynamicMesh3& EditMesh)
		{
			const FTransform PlaneTransform(PlaneRotation, FVector(0), FVector(PlaneSize.X, PlaneSize.Y, 0));
			FDynamicMeshUVOverlay* UVOverlay = EditMesh.Attributes()->GetUVLayer(0);
			FDynamicMeshUVEditor UVEditor(&EditMesh, UVOverlay);
			const FFrame3d ProjectionFrame(PlaneTransform);
			const FVector Scale = PlaneTransform.GetScale3D();
			const FVector2d Dimensions(Scale.X, Scale.Y);
			FUVEditResult Result;

			UVEditor.SetTriangleUVsFromPlanarProjection(InMesh.TriangleIds, [](const FVector3d& Pos) {
				return Pos;
			}, ProjectionFrame, Dimensions, &Result);

			InMesh.UVIds = Result.NewUVElements;

			// copy generated uv to cache
			InMesh.UVs.Empty();
			for(const int32 UId : InMesh.UVIds)
			{
				InMesh.UVs.Add(static_cast<FVector2D>(UVOverlay->GetElement(UId)));
			}
		}
		, EDynamicMeshChangeType::AttributeEdit
		, EDynamicMeshAttributeChangeFlags::UVs);
}

bool UAvaShapeDynamicMeshBase::ApplyUVsBoxProjection(FAvaShapeMesh& InMesh, FRotator BoxRotation, FVector BoxSize)
{
	if (InMesh.TriangleIds.IsEmpty())
	{
		return false;
	}

	return UE::Private::AvaShape::EditMesh(this,
		[this, &InMesh, BoxRotation, BoxSize](FDynamicMesh3& EditMesh)
		{
			const FTransform PlaneTransform(BoxRotation, FVector(0), BoxSize);
			FDynamicMeshUVOverlay* UVOverlay = EditMesh.Attributes()->GetUVLayer(0);
			FDynamicMeshUVEditor UVEditor(&EditMesh, UVOverlay);
			const FFrame3d ProjectionFrame(PlaneTransform);
			FUVEditResult Result;

			UVEditor.SetTriangleUVsFromBoxProjection(InMesh.TriangleIds, [this](const FVector3d& Pos) {
				return Pos;
			}, ProjectionFrame, BoxSize, 2, &Result);

			InMesh.UVIds = Result.NewUVElements;

			// copy generated uv to cache
			InMesh.UVs.Empty();
			for (const int32 UId : InMesh.UVIds)
			{
				InMesh.UVs.Add(static_cast<FVector2D>(UVOverlay->GetElement(UId)));
			}
		}
		, EDynamicMeshChangeType::AttributeEdit
		, EDynamicMeshAttributeChangeFlags::UVs);
}

bool UAvaShapeDynamicMeshBase::ApplyUVsTransform(FAvaShapeMesh& InMesh, FAvaShapeMaterialUVParameters& InParams, FVector2D ShapeSize, FVector2D UVOffset, float UVFixRotation)
{
	bool bResult = true;
	return UE::Private::AvaShape::EditMesh(this,
		[InMesh, InParams, ShapeSize, UVOffset, UVFixRotation, &bResult](FDynamicMesh3& EditMesh)
		{
			bResult = UE::AvaShapes::TransformMeshUVs(EditMesh, InMesh.UVIds, InParams, ShapeSize, UVOffset, UVFixRotation);
		}
		, EDynamicMeshChangeType::AttributeEdit
		, EDynamicMeshAttributeChangeFlags::UVs) && bResult;
}

void UAvaShapeDynamicMeshBase::SetRunAsync(bool bInAsync)
{
	bRunAsync = bInAsync;
}

void UAvaShapeDynamicMeshBase::OnParametricMaterialChanged(const FAvaShapeParametricMaterial& InMaterial)
{
	for (const int32 Index : GetMeshesIndexes())
	{
		if (GetParametricMaterialPtr(Index) == &InMaterial && GetMaterialType(Index) == EMaterialType::Parametric)
		{
			OnParametricMaterialChanged(Index);
			break;
		}
	}
}

void UAvaShapeDynamicMeshBase::OnParametricMaterialParameterChanged(const FAvaShapeParametricMaterial& InMaterial)
{
	for (const int32 Index : GetMeshesIndexes())
	{
		if (GetParametricMaterialPtr(Index) == &InMaterial && GetMaterialType(Index) == EMaterialType::Parametric)
		{
			OnParametricMaterialChanged(Index);
			break;
		}
	}
}

void UAvaShapeDynamicMeshBase::RestoreCachedMesh(bool bInReset)
{
	if (CachedMesh.IsSet())
	{
		UE::Private::AvaShape::EditMesh(this,
			[this, bInReset](FDynamicMesh3& EditMesh)
			{
				if (bInReset)
				{
					EditMesh = MoveTemp(CachedMesh.GetValue());
					CachedMesh.Reset();
				}
				else
				{
					EditMesh = CachedMesh.GetValue();
				}
			}
		, EDynamicMeshChangeType::GeneralEdit
		, EDynamicMeshAttributeChangeFlags::Unknown);
	}
}

void UAvaShapeDynamicMeshBase::SaveCachedMesh()
{
	if (const UDynamicMeshComponent* DynamicMeshComponent = GetShapeMeshComponent())
	{
		DynamicMeshComponent->ProcessMesh([this](const FDynamicMesh3& Mesh)
		{
			CachedMesh = Mesh;
		});
	}
}

bool UAvaShapeDynamicMeshBase::CreateUVs(FAvaShapeMesh& InMesh, FAvaShapeMaterialUVParameters& InParams)
{
	return ApplyUVsTransform(InMesh, InParams, FVector2D(1, 1), FVector2D(0, 0));
}

void UAvaShapeDynamicMeshBase::InvalidateSection(bool& bInvalidatedSection, bool bRequireUpdate)
{
	bInvalidatedSection = true;

	if (bRequireUpdate)
	{
		RequestUpdate();
	}
}

bool UAvaShapeDynamicMeshBase::RegisterMesh(FAvaShapeMeshData& NewMeshData)
{
	if (MeshDatas.Contains(NewMeshData.GetMeshIndex()))
	{
		UE_LOGF(LogAvaDynamicMesh, Warning, "RegisterMesh %ls %i : Already contains mesh cannot add again", *GetMeshName(), NewMeshData.GetMeshIndex());
		return false;
	}

	MeshDatas.Add(NewMeshData.GetMeshIndex(), NewMeshData);

	OnRegisteredMesh(NewMeshData.GetMeshIndex());

	return true;
}

const FAvaShapeMeshData* UAvaShapeDynamicMeshBase::GetMeshData(int32 InMeshIndex) const
{
	return MeshDatas.Find(InMeshIndex);
}

FAvaShapeMeshData* UAvaShapeDynamicMeshBase::GetMeshData(int32 InMeshIndex)
{
	return MeshDatas.Find(InMeshIndex);
}

bool UAvaShapeDynamicMeshBase::IsMeshVisible(int32 MeshIndex)
{
	if (const FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex))
	{
		return MeshData->bMeshVisible;
	}

	return true;
}

TSet<int32> UAvaShapeDynamicMeshBase::GetMeshesIndexes() const
{
	TSet<int32> MeshIndexes;
	MeshDatas.GetKeys(MeshIndexes);
	return MeshIndexes;
}

bool UAvaShapeDynamicMeshBase::IsMeshDirty(int32 MeshIndex)
{
	if (const FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex))
	{
		return MeshData->bMeshDirty;
	}

	return false;
}

void UAvaShapeDynamicMeshBase::MarkMeshDirty(int32 MeshIndex)
{
	if (bAllMeshDirty && bAnyMeshDirty)
	{
		return;
	}

	if (FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex))
	{
		MeshData->bMeshDirty = true;
		InvalidateSection(bAnyMeshDirty);
	}
}

void UAvaShapeDynamicMeshBase::MarkAllMeshesDirty()
{
	for (TPair<int32, FAvaShapeMeshData>& MeshData : MeshDatas)
	{
		MeshData.Value.bMeshDirty = true;
	}

	InvalidateSection(bAnyMeshDirty, false);
	InvalidateSection(bAllMeshDirty);
}

void UAvaShapeDynamicMeshBase::MarkVerticesDirty()
{
	InvalidateSection(bVerticesDirty);
}

FName UAvaShapeDynamicMeshBase::GetMeshSectionName(int32 InMeshIndex) const
{
	if (const FAvaShapeMeshData* MeshData = GetMeshData(InMeshIndex))
	{
		return MeshData->MeshName;
	}

	return FName();
}

TArray<FName> UAvaShapeDynamicMeshBase::GetMeshSectionNames() const
{
	TArray<FName> Names;
	const TSet<int32> Keys = GetMeshesIndexes();

	for (const int32 MeshIndex : Keys)
	{
		const FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex);
		Names.Add(MeshData->MeshName);
	}

	return Names;
}

const TArray<UMaterialInterface*> UAvaShapeDynamicMeshBase::GetMeshDataMaterials()
{
	TArray<UMaterialInterface*> Materials;
	const TSet<int32> Keys = GetMeshesIndexes();

	for (const int32 MeshIndex : Keys)
	{
		Materials.Add(GetMaterial(MeshIndex));
	}

	return Materials;
}

void UAvaShapeDynamicMeshBase::InitializeDynamicMesh()
{
	UDynamicMeshComponent* const MeshComponent = CachedComponent.Get();
	if (!MeshComponent)
	{
		return;
	}

	UE::Private::AvaShape::EditMesh(this,
		[this](FDynamicMesh3& EditMesh)
		{
			EditMesh.Clear();

			// for sections
			EditMesh.EnableTriangleGroups();

			// for materials, tangents, uv, colors
			EditMesh.EnableAttributes();
			EditMesh.Attributes()->EnableMaterialID();
			EditMesh.Attributes()->GetMaterialID()->SetName(FName(GetMeshName()));
			EditMesh.Attributes()->EnableTangents();

			// set name for PolyGroup layers
			EditMesh.Attributes()->SetNumPolygroupLayers(MeshDatas.Num());
			for (int32 Idx = 0; Idx < MeshDatas.Num(); Idx++)
			{
				const FName DataName = GetMeshSectionName(Idx);
				EditMesh.Attributes()->GetPolygroupLayer(Idx)->SetName(DataName);
			}
			EditMesh.Attributes()->SetNumUVLayers(1);
			EditMesh.Attributes()->EnablePrimaryColors();

			CachedMesh = EditMesh;
		}
		, EDynamicMeshChangeType::GeneralEdit
		, EDynamicMeshAttributeChangeFlags::Unknown);

	MeshComponent->TransformUpdated.AddUObject(this, &UAvaShapeDynamicMeshBase::OnMeshTransformChanged);
}

TArray<FAvaSnapPoint> UAvaShapeDynamicMeshBase::GetLocalSnapPoints() const
{
	TArray<FAvaSnapPoint> SnapPoints;

	if (!GetShapeMeshComponent())
	{
		return SnapPoints;
	}

	AActor* Actor = GetShapeActor();

	if (!Actor)
	{
		return SnapPoints;
	}

	const FTransform ShapeTransform = GetTransform();
	const FVector ShapeScale = ShapeTransform.GetScale3D();

	for (const FAvaSnapPoint& LocalSnapPoint : LocalSnapPoints)
	{
		FAvaSnapPoint SnapPoint = LocalSnapPoint;
		SnapPoint.Outer = Actor;
		SnapPoint.Location *= ShapeScale;
		SnapPoints.Add(SnapPoint);
	}

	const FVector CurSize3D = GetSize3D();

	if (CurSize3D.X != 0)
	{
		for (const FAvaSnapPoint& LocalSnapPoint : LocalSnapPoints)
		{
			FAvaSnapPoint SnapPoint = LocalSnapPoint;
			SnapPoint.Outer = Actor;
			SnapPoint.Location.X = GetSize3D().X / 2.f;
			SnapPoint.Location *= ShapeScale;
			SnapPoints.Add(SnapPoint);
		}

		for (const FAvaSnapPoint& LocalSnapPoint : LocalSnapPoints)
		{
			FAvaSnapPoint SnapPoint = LocalSnapPoint;
			SnapPoint.Outer = Actor;
			SnapPoint.Location.X = GetSize3D().X;
			SnapPoint.Location *= ShapeScale;
			SnapPoints.Add(SnapPoint);
		}
	}

	return SnapPoints;
}

void UAvaShapeDynamicMeshBase::GetLocalSnapPoints(TArray<FAvaSnapPoint>& Points) const
{
	Points.Append(GetLocalSnapPoints());
}

FTransform UAvaShapeDynamicMeshBase::GetTransform() const
{
	if (const USceneComponent* DynMeshComp = GetShapeMeshComponent())
	{
		FTransform MeshTransform = DynMeshComp->GetComponentTransform();

		if (HasMeshRegenWorldLocation())
		{
			MeshTransform.SetLocation(GetMeshRegenWorldLocation());
		}

		return MeshTransform;
	}

	return FTransform::Identity;
}

bool UAvaShapeDynamicMeshBase::ClearDynamicMeshSection(int32 MeshIndex)
{
	FAvaShapeMeshData* MeshData = GetMeshData(MeshIndex);

	if (!MeshData)
	{
		return false;
	}

	return UE::Private::AvaShape::EditMesh(this,
		[this, &MeshData](FDynamicMesh3& EditMesh)
		{
			for (const int32 TId : MeshData->Mesh.TriangleIds)
			{
				if(EditMesh.IsTriangle(TId))
				{
					EditMesh.RemoveTriangle(TId);
				}
			}

			// empty ids arrays
			MeshData->Mesh.ClearIds();
		}
		, EDynamicMeshChangeType::GeneralEdit
		, EDynamicMeshAttributeChangeFlags::MeshTopology);
}

bool UAvaShapeDynamicMeshBase::ClearDynamicMesh()
{
	const bool bResult = UE::Private::AvaShape::EditMesh(this,
		[this](FDynamicMesh3& EditMesh)
		{
			for (const int32 TId : EditMesh.TriangleIndicesItr())
			{
				if(EditMesh.IsTriangle(TId))
				{
					EditMesh.RemoveTriangle(TId);
				}
			}
		}
		, EDynamicMeshChangeType::GeneralEdit
		, EDynamicMeshAttributeChangeFlags::MeshTopology);

	if (!bResult)
	{
		return false;
	}

	for(const int32 MeshIdx : GetMeshesIndexes())
	{
		if(FAvaShapeMeshData* MeshData = GetMeshData(MeshIdx))
		{
			MeshData->Mesh.ClearIds();
		}
	}

	return true;
}

bool UAvaShapeDynamicMeshBase::ExportToStaticMesh(UStaticMesh* DestinationMesh)
{
	if (!DestinationMesh || !GetShapeMeshComponent())
	{
		return false;
	}

	UDynamicMesh* SourceMesh = GetShapeMeshComponent()->GetDynamicMesh();

	// export options
	FGeometryScriptCopyMeshToAssetOptions AssetOptions;
	AssetOptions.bReplaceMaterials = true;
	AssetOptions.bEnableRecomputeNormals = false;
	AssetOptions.bEnableRecomputeTangents = false;
	AssetOptions.bEnableRemoveDegenerates = true;
	AssetOptions.NewMaterialSlotNames = GetMeshSectionNames();

	// Only take asset materials instead of dynamic ones
	UMaterial* DefaultMaterial = LoadObject<UMaterial>(nullptr, TEXT("/Script/Engine.Material'/Engine/EngineMaterials/WorldGridMaterial.WorldGridMaterial'"));
	for (UMaterialInterface* Material : GetMeshDataMaterials())
	{
		if (IsValid(Material) && Material->IsAsset())
		{
			AssetOptions.NewMaterials.Add(Material);
		}
		else
		{
			AssetOptions.NewMaterials.Add(DefaultMaterial);
		}
	}

	// LOD options
	FGeometryScriptMeshWriteLOD TargetLOD;
	TargetLOD.LODIndex = 0;

	EGeometryScriptOutcomePins OutResult;
	UGeometryScriptLibrary_StaticMeshFunctions::CopyMeshToStaticMesh(SourceMesh, DestinationMesh, AssetOptions, TargetLOD, OutResult);
	DestinationMesh->GetBodySetup()->AggGeom = CachedComponent->GetBodySetup()->AggGeom;
	return OutResult == EGeometryScriptOutcomePins::Success;
}

void UAvaShapeDynamicMeshBase::OnColorPicked(const FAvaColorChangeData& InNewColorData)
{
	constexpr int32 MeshIndex = MESH_INDEX_PRIMARY;

	const FAvaShapeMeshData* const MeshData = GetMeshData(MeshIndex);
	if (!MeshData || MeshData->MaterialType != EMaterialType::Parametric)
	{
		return;
	}

	FAvaShapeParametricMaterial* const ParametricMaterial = GetParametricMaterialPtr(MeshIndex);
	if (!ParametricMaterial)
	{
		return;
	}

	EAvaShapeParametricMaterialStyle NewMaterialStyle = ParametricMaterial->GetStyle();

	// TODO: Have these be one enum and directly check with ParametricMaterial->Style whether there's pending change here
	switch (InNewColorData.ColorStyle)
	{
		case EAvaColorStyle::Solid:
			NewMaterialStyle = EAvaShapeParametricMaterialStyle::Solid;
			break;

		case EAvaColorStyle::LinearGradient:
			NewMaterialStyle = EAvaShapeParametricMaterialStyle::LinearGradient;
			break;

		default:
			// Nothing to do
			break;
	}

	const bool bUnlitChanged = ParametricMaterial->GetUseUnlitMaterial() != InNewColorData.bIsUnlit;

	const bool bAnythingToChange = NewMaterialStyle != ParametricMaterial->GetStyle()
		|| ParametricMaterial->GetPrimaryColor() != InNewColorData.PrimaryColor
		|| ParametricMaterial->GetSecondaryColor() != InNewColorData.SecondaryColor
		|| bUnlitChanged;

	if (!bAnythingToChange)
	{
		return;
	}

#if WITH_EDITOR
	Modify();
#endif

	ParametricMaterial->SetPrimaryColor(InNewColorData.PrimaryColor);
	ParametricMaterial->SetSecondaryColor(InNewColorData.SecondaryColor);
	ParametricMaterial->SetStyle(NewMaterialStyle);

	if (bUnlitChanged)
	{
		ParametricMaterial->SetUseUnlitMaterial(InNewColorData.bIsUnlit);

		// switching to a different material (lit/unlit)
		OnParametricMaterialChanged(MeshIndex);
	}

	// This ensures the object in the viewport is rendered with the updated material
	MarkMeshRenderStateDirty();
}

FAvaColorChangeData UAvaShapeDynamicMeshBase::GetActiveColor() const
{
	FAvaColorChangeData ActiveColor = {EAvaColorStyle::None, FLinearColor::White, FLinearColor::White, /* bIsUnlit */ true};
	const FAvaShapeMeshData* MeshData = GetMeshData(MESH_INDEX_PRIMARY);

	if (!MeshData)
	{
		return ActiveColor;
	}

	if (MeshData->MaterialType != EMaterialType::Parametric)
	{
		return ActiveColor;
	}

	switch (MeshData->ParametricMaterial.GetStyle())
	{
		case EAvaShapeParametricMaterialStyle::Solid:
			ActiveColor.ColorStyle = EAvaColorStyle::Solid;
			break;

		case EAvaShapeParametricMaterialStyle::LinearGradient:
			ActiveColor.ColorStyle = EAvaColorStyle::LinearGradient;
			break;

		default:
			ActiveColor.ColorStyle = EAvaColorStyle::None;
			break;
	}

	ActiveColor.PrimaryColor = MeshData->ParametricMaterial.GetPrimaryColor();
	ActiveColor.SecondaryColor = MeshData->ParametricMaterial.GetSecondaryColor();
	ActiveColor.bIsUnlit = MeshData->ParametricMaterial.GetUseUnlitMaterial();

	return ActiveColor;
}

void UAvaShapeDynamicMeshBase::MarkMeshRenderStateDirty() const
{
	if (UDynamicMeshComponent* MeshComponent = GetShapeMeshComponent())
	{
		UE::ActorModifierCore::FRenderStateDirtyReasonScope Scope(UE::ActorModifierCore::ERenderStateDirtyReason::Material);
		MeshComponent->MarkRenderStateDirty();
	}
}
