// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshes/AvaShapeEllipseDynMesh.h"
#include "AvaShapeVertices.h"

const FString UAvaShapeEllipseDynamicMesh::MeshName = TEXT("Ellipse");

void UAvaShapeEllipseDynamicMesh::SetNumSides(uint8 InNumSides)
{
	if (NumSides == InNumSides)
	{
		return;
	}

	if (InNumSides < UAvaShapeEllipseDynamicMesh::MinNumSides || InNumSides > UAvaShapeEllipseDynamicMesh::MaxNumSides)
	{
		return;
	}

	NumSides = InNumSides;
	OnNumSidesChanged();
}

void UAvaShapeEllipseDynamicMesh::SetAngleDegree(float InDegree)
{
	if (AngleDegree == InDegree)
	{
		return;
	}

	if (InDegree < 0.f || InDegree > 360.f)
	{
		return;
	}

	AngleDegree = InDegree;
	OnAngleDegreeChanged();
}

void UAvaShapeEllipseDynamicMesh::SetStartDegree(float InDegree)
{
	if (StartDegree == InDegree)
	{
		return;
	}

	if (InDegree < 0.f || InDegree > 360.f)
	{
		return;
	}

	StartDegree = InDegree;
	OnStartDegreeChanged();
}

#if WITH_EDITOR
void UAvaShapeEllipseDynamicMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static FName NumSidesName = GET_MEMBER_NAME_CHECKED(UAvaShapeEllipseDynamicMesh, NumSides);

	static FName AngleDegreeName = GET_MEMBER_NAME_CHECKED(UAvaShapeEllipseDynamicMesh, AngleDegree);
	static FName StartDegreeName = GET_MEMBER_NAME_CHECKED(UAvaShapeEllipseDynamicMesh, StartDegree);

	if (PropertyChangedEvent.MemberProperty->GetFName() == NumSidesName)
	{
		OnNumSidesChanged();
	}

	else if (PropertyChangedEvent.MemberProperty->GetFName() == AngleDegreeName)
	{
		OnAngleDegreeChanged();
	}

	else if (PropertyChangedEvent.MemberProperty->GetFName() == StartDegreeName)
	{
		OnStartDegreeChanged();
	}
}
#endif

void UAvaShapeEllipseDynamicMesh::OnNumSidesChanged()
{
	NumSides = FMath::Clamp(NumSides, UAvaShapeEllipseDynamicMesh::MinNumSides, UAvaShapeEllipseDynamicMesh::MaxNumSides);
	MarkAllMeshesDirty();
}

void UAvaShapeEllipseDynamicMesh::OnAngleDegreeChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShapeEllipseDynamicMesh::OnStartDegreeChanged()
{
	MarkAllMeshesDirty();
}

bool UAvaShapeEllipseDynamicMesh::IsMeshVisible(int32 MeshIndex)
{
	switch (MeshIndex)
	{
		case MESH_INDEX_PRIMARY:
			return AngleDegree > UE_KINDA_SMALL_NUMBER;
		default:
			return true;
	}
}

bool UAvaShapeEllipseDynamicMesh::CreateMesh(FAvaShapeMesh& InMesh)
{
	if (InMesh.GetMeshIndex() == MESH_INDEX_PRIMARY)
	{
		const FVector2D OffsetBase = FVector2D(0.f, Size2D.Y / 2.f).GetRotated(StartDegree);
		const FAvaShapeCachedVertex2D FirstVertex = CacheVertexCreate(InMesh, FVector2D(0, 0));
		const float ScaleX = Size2D.X / Size2D.Y;
		const float SideAngle = AngleDegree / NumSides;

		FVector2D PreviousVertex = OffsetBase.GetRotated(SideAngle * 0);
		PreviousVertex.X *= ScaleX;
		int32 PreviousIndex = CacheVertex(InMesh, PreviousVertex, true);

		for (int32 SideIdx = 1; SideIdx <= NumSides; ++SideIdx)
		{
			AddVertex(InMesh, FirstVertex);
			AddVertex(InMesh, PreviousIndex);

			FVector2D NextVertex = OffsetBase.GetRotated(SideAngle * SideIdx);
			NextVertex.X *= ScaleX;
			int32 NextIndex = AddVertexRaw(InMesh, NextVertex, true);

			PreviousIndex = NextIndex;
		}

	}

	return Super::CreateMesh(InMesh);
}

bool UAvaShapeEllipseDynamicMesh::CanUpdateMeshInAnyThread() const
{
	return GetClass() == UAvaShapeEllipseDynamicMesh::StaticClass();
}
