// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshes/AvaShapeIrregularPolygonDynMesh.h"
#include "CompGeom/PolygonTriangulation.h"
#include "AvaShapesDefs.h"
#include "AvaShapeActor.h"
#include "DynamicMeshes/AvaShapeRectangleDynMesh.h"

const float UAvaShapeIrregularPolygonDynamicMesh::MinPointDistance = 1.f;

const FString UAvaShapeIrregularPolygonDynamicMesh::MeshName = TEXT("FreehandPolygon");

void UAvaShapeIrregularPolygonDynamicMesh::RecalculateExtent()
{
	if (Points.Num() <= 1)
	{
		Size2D = FVector2D::ZeroVector;
		Size3D.Y = 0.f;
		Size3D.Z = 0.f;
		return;
	}

	FVector2D Min = Points[0].Location;
	FVector2D Max = Points[0].Location;

	for (int32 PointIdx = 1; PointIdx < Points.Num(); ++PointIdx)
	{
		Min.X = FMath::Min(Min.X, Points[PointIdx].Location.X);
		Max.X = FMath::Max(Max.X, Points[PointIdx].Location.X);
		Min.Y = FMath::Min(Min.Y, Points[PointIdx].Location.Y);
		Max.Y = FMath::Max(Max.Y, Points[PointIdx].Location.Y);
	}

	Size2D.X = Max.X - Min.X;
	Size2D.Y = Max.Y - Min.Y;
	Size3D.Y = Size2D.X;
	Size3D.Z = Size2D.Y;
}

void UAvaShapeIrregularPolygonDynamicMesh::RecalculateActorPosition()
{
	FVector2D Min = Points[0].Location;
	FVector2D Max = Points[0].Location;

	for (int32 PointIdx = 1; PointIdx < Points.Num(); ++PointIdx)
	{
		Min.X = FMath::Min(Min.X, Points[PointIdx].Location.X);
		Max.X = FMath::Max(Max.X, Points[PointIdx].Location.X);
		Min.Y = FMath::Min(Min.Y, Points[PointIdx].Location.Y);
		Max.Y = FMath::Max(Max.Y, Points[PointIdx].Location.Y);
	}

	FVector2D Center = (Max + Min) / 2.f; // /2.f twice

	if (GetShapeMeshComponent() && (Center.X != 0.f || Center.Y != 0.f))
	{
		for (FAvaShapeRoundedCorner& Point : Points)
			Point.Location -= Center;

		FVector ActorMove = FVector::ZeroVector;
		ActorMove.Y = Center.X;
		ActorMove.Z = Center.Y;

		FTransform MeshTransform = GetTransform();
		ActorMove = MeshTransform.TransformVector(ActorMove);
		SetMeshRegenWorldLocation(MeshTransform.GetLocation() + ActorMove, false);

		MarkAllMeshesDirty();
	}
}

bool UAvaShapeIrregularPolygonDynamicMesh::CanAddPoint(const FVector2D& InPoint)
{
	if (Points.Num() > 0)
	{
		/*
		 * Point / Point checking
		 */

		 // Check to see if the moved point is too near any other point
		if (IsPointTooCloseToAnotherPoint(InPoint, false))
		{
			return false;
		}

		if (Points.Num() > 1)
		{
			/*
			 * Point / Line checking
			 */

			 // Check to see if the moved point is too near to any lines
			if (IsPointTooCloseToALine(InPoint))
			{
				return false;
			}

			if (IsLineTooCloseToAPoint(Points.Last().Location, InPoint))
			{
				return false;
			}

			if (Points.Num() > 2)
			{
				/*
				 * Line / Line checking
				 */

				if (DoesLineIntersectBorder(Points.Last().Location, InPoint))
				{
					return false;
				}
			}
		}
	}

	return true;
}

bool UAvaShapeIrregularPolygonDynamicMesh::DoLinesIntersect(const FVector2D Origin1, const FVector2D End1,
                                                            const FVector2D Origin2, const FVector2D End2)
{
	FVector2D Vector1 = End1 - Origin1;
	FVector2D Vector2 = End2 - Origin2;

	float Determinant = (-Vector2.X * Vector1.Y + Vector1.X * Vector2.Y);

	if (Determinant == 0)
	{
		return false;
	}

	float Mu = (-Vector1.Y * (Origin1.X - Origin2.X) + Vector1.X * (Origin1.Y - Origin2.Y)) / Determinant;

	if (Mu < 0 || Mu > 1)
	{
		return false;
	}

	Mu = (Vector2.X * (Origin1.Y - Origin2.Y) - Vector2.Y * (Origin1.X - Origin2.X)) / Determinant;

	return (Mu >= 0.f && Mu <= 1.f);
}

bool UAvaShapeIrregularPolygonDynamicMesh::DoesLineIntersectBorder(const FVector2D CheckOrigin, const FVector2D CheckEnd) const
{
	// The last line cannot intersect with the mouse line because they share a vertex.
	for (int32 PointIdx = 1; PointIdx < Points.Num(); ++PointIdx)
	{
		FVector2D Origin = Points[PointIdx - 1].Location;
		FVector2D End = Points[PointIdx].Location;

		// Don't check lines that share points - they cannot intersect each other
		if (CheckOrigin == Origin || CheckOrigin == End || CheckEnd == Origin || CheckEnd == End)
		{
			continue;
		}

		if (DoLinesIntersect(CheckOrigin, CheckEnd, Origin, End))
		{
			return true;
		}
	}

	return false;
}

bool UAvaShapeIrregularPolygonDynamicMesh::IsPointTooCloseToAnotherPoint(const FVector2D& InPoint,
	bool bAllowExactlyOneMatch) const
{
	bool bHasMatch = false;

	// Check if we're too close to other points
	for (const FAvaShapeRoundedCorner& Point : Points)
	{
		if (InPoint == Point.Location)
		{
			if (!bAllowExactlyOneMatch)
			{
				return true;
			}

			if (bHasMatch)
			{
				return true;
			}

			bHasMatch = true;
			continue;
		}

		if ((Point.Location - InPoint).SizeSquared() < UAvaShapeIrregularPolygonDynamicMesh::MinPointDistance)
		{
			return true;
		}
	}

	return false;
}

bool UAvaShapeIrregularPolygonDynamicMesh::IsPointTooCloseToALine(const FVector2D& InPoint) const
{
	// Check if we're too close to other lines
	for (int32 PointIdx = 0; PointIdx < Points.Num(); ++PointIdx)
	{
		int32 PreviousPointIdx = (PointIdx > 0 ? PointIdx - 1 : Points.Num() - 1);

		if (InPoint == Points[PreviousPointIdx].Location || InPoint == Points[PointIdx].Location)
		{
			continue;
		}

		FVector2D ClosestPoint = UE::AvaShapes::FindClosestPointOnLine(
			Points[PreviousPointIdx].Location,
			Points[PointIdx].Location,
			InPoint
		);

		if ((InPoint - ClosestPoint).Size() < UAvaShapeIrregularPolygonDynamicMesh::MinPointDistance)
		{
			return true;
		}
	}

	return false;
}

bool UAvaShapeIrregularPolygonDynamicMesh::IsLineTooCloseToAPoint(const FVector2D& Start, const FVector2D& End) const
{
	for (int32 PointIdx = 0; PointIdx < Points.Num(); ++PointIdx)
	{
		if (Points[PointIdx].Location == Start || Points[PointIdx].Location == End)
		{
			continue;
		}

		FVector2D ClosestPoint = UE::AvaShapes::FindClosestPointOnLine(Start, End, Points[PointIdx].Location);

		if ((ClosestPoint - Points[PointIdx].Location).Size() < UAvaShapeIrregularPolygonDynamicMesh::MinPointDistance)
		{
			return true;
		}
	}

	return false;
}

bool UAvaShapeIrregularPolygonDynamicMesh::CanBeGenerated() const
{
	if (Points.Num() < 3)
	{
		return false;
	}

	return !DoesLineIntersectBorder(Points[0].Location, Points.Last().Location);
}

bool UAvaShapeIrregularPolygonDynamicMesh::AddPoint(const FVector2D& InPoint)
{
	if (!CanAddPoint(InPoint))
	{
		return false;
	}

	Points.Add(InPoint);
	PreEditPoints.Add(FVector2D::ZeroVector);

	FAvaShapeRoundedCornerSettings Settings;
	Settings.BevelSize = GlobalBevelSize;
	Settings.BevelSubdivisions = GlobalBevelSubdivisions;
	Points.Last().Settings = Settings;

	OnPointsUpdated();

	return true;
}

bool UAvaShapeIrregularPolygonDynamicMesh::RemovePoint(int32 PointIdx)
{
	if (!Points.IsValidIndex(PointIdx))
	{
		return false;
	}

	// First and last point can always be removed
	if (Points.Num() > 3 && PointIdx > 0 && PointIdx < (Points.Num() - 1))
	{
		FVector2D PreviousPoint = Points[PointIdx > 0 ? PointIdx - 1 : Points.Num() - 1].Location;
		FVector2D NextPoint = Points[PointIdx < (Points.Num() - 1) ? PointIdx + 1 : 0].Location;

		if (DoesLineIntersectBorder(PreviousPoint, NextPoint))
		{
			return false;
		}
	}

	Points.RemoveAt(PointIdx);
	PreEditPoints.RemoveAt(PointIdx);
	OnPointsUpdated();

	return true;
}

bool UAvaShapeIrregularPolygonDynamicMesh::RemoveFirstPoint()
{
	return RemovePoint(0);
}

bool UAvaShapeIrregularPolygonDynamicMesh::RemoveLastPoint()
{
	return RemovePoint(Points.Num() - 1);
}

bool UAvaShapeIrregularPolygonDynamicMesh::RemoveAllPoints()
{
	if (Points.Num() == 0)
	{
		return true;
	}

	Points.SetNum(0);
	PreEditPoints.SetNum(0);
	OnPointsUpdated();

	return true;
}

void UAvaShapeIrregularPolygonDynamicMesh::BackupPoints()
{
	PreEditPoints.Empty();
	PreEditPoints.Append(Points);
}

void UAvaShapeIrregularPolygonDynamicMesh::RestorePoints()
{
	Points.Empty();
	Points.Append(PreEditPoints);
}

void UAvaShapeIrregularPolygonDynamicMesh::SetPoints(const TArray<FVector2D>& InPoints)
{
	BackupPoints();
	Points.Empty();
	Points.Append(InPoints);

	CheckNewPointsArray();
}

void UAvaShapeIrregularPolygonDynamicMesh::SetPoints(const TArray<FAvaShapeRoundedCorner>& InPoints)
{
	BackupPoints();
	Points.Empty();
	Points.Append(InPoints);

	CheckNewPointsArray();
}

bool UAvaShapeIrregularPolygonDynamicMesh::SetLocation(int32 PointIdx, const FVector2D& InPoint)
{
	if (!Points.IsValidIndex(PointIdx))
	{
		return false;
	}

	if (Points[PointIdx].Location == InPoint)
	{
		return false;
	}

	PreEditPoints[PointIdx] = Points[PointIdx];
	Points[PointIdx].Location = InPoint;

	// force bevel size update to fix artifacts and triangulations
	const float BevelSizeCached = Points[PointIdx].Settings.BevelSize;
	if (Points[PointIdx].Settings.BevelSize == 0)
	{
		SetBevelSize(PointIdx, 0.01f);
	}
	else
	{
		SetBevelSize(PointIdx, 0.0f);
	}
	SetBevelSize(PointIdx, BevelSizeCached);

	OnLocationUpdated(PointIdx);

	return true;
}

float UAvaShapeIrregularPolygonDynamicMesh::GetMaxBevelSizeForPoint(int32 PointIdx) const
{
	if (!Points.IsValidIndex(PointIdx))
	{
		return 0.f;
	}

	// We have no corners...
	if (Points.Num() < 2)
	{
		return 0.f;
	}

	int32 PreviousIdx = (PointIdx > 0 ? PointIdx - 1 : Points.Num() - 1);
	int32 NextIdx = (PointIdx < (Points.Num() - 1) ? PointIdx + 1 : 0);

	float MinDistance = FMath::Min((Points[PreviousIdx].Location - Points[PointIdx].Location).Size(),
		(Points[NextIdx].Location - Points[PointIdx].Location).Size()) / 2.1f;

	for (int32 PointCompareIdx = 0; PointCompareIdx < Points.Num(); ++PointCompareIdx)
	{
		if (PointCompareIdx == PointIdx || PointCompareIdx == PreviousIdx || PointCompareIdx == NextIdx)
		{
			continue;
		}

		float CurrentDistance = (Points[PointCompareIdx].Location - Points[PointIdx].Location).Size() / 1.1f;

		if (MinDistance < 0.f)
		{
			MinDistance = CurrentDistance;
		}
		else
		{
			MinDistance = FMath::Min(MinDistance, CurrentDistance);
		}
	}

	return MinDistance;
}

bool UAvaShapeIrregularPolygonDynamicMesh::BreakSide(int32 InPointIdx)
{
	if (!Points.IsValidIndex(InPointIdx))
	{
		return false;
	}

	if (Points.Num() < 2)
	{
		return false;
	}

	const int32 NextPointIdx = InPointIdx == (Points.Num() - 1) ? 0 : InPointIdx + 1;

	FAvaShapeRoundedCorner NewPoint;
	NewPoint.Location                   = (Points[InPointIdx].Location * 0.5)                   + (Points[NextPointIdx].Location * 0.5);
	NewPoint.Settings.BevelSize         = (Points[InPointIdx].Settings.BevelSize * 0.5)         + (Points[NextPointIdx].Settings.BevelSize * 0.5);
	NewPoint.Settings.BevelSubdivisions = (Points[InPointIdx].Settings.BevelSubdivisions * 0.5) + (Points[NextPointIdx].Settings.BevelSubdivisions * 0.5);

	const float AdjustmentToBreakStraightLine = FMath::Clamp((Points[InPointIdx].Location - Points[NextPointIdx].Location).Size() / 10.f, 1.f, 5.f);
	NewPoint.Location += NewPoint.Location.GetSafeNormal() * AdjustmentToBreakStraightLine;

	BackupPoints();

	// We're breaking the last point, so just add to the end of the array
	if (NextPointIdx == 0)
	{
		Points.Add(NewPoint);
	}
	else
	{
		Points.Insert(NewPoint, InPointIdx + 1);
	}

	return CheckNewPointsArray();
}

bool UAvaShapeIrregularPolygonDynamicMesh::SetBevelSize(int32 PointIdx, float InBevelSize)
{
	if (!Points.IsValidIndex(PointIdx))
	{
		return false;
	}

	if (Points[PointIdx].Settings.BevelSize == InBevelSize)
	{
		return false;
	}

	if (InBevelSize < 0 || InBevelSize > 1)
	{
		return false;
	}

	Points[PointIdx].Settings.BevelSize = InBevelSize;
	OnBevelSizeChanged(PointIdx);

	return true;
}

bool UAvaShapeIrregularPolygonDynamicMesh::SetBevelSubdivisions(int32 PointIdx, uint8 InBevelSubdivisions)
{
	if (!Points.IsValidIndex(PointIdx))
	{
		return false;
	}

	if (Points[PointIdx].Settings.BevelSubdivisions == InBevelSubdivisions)
	{
		return false;
	}

	if (InBevelSubdivisions > UAvaShapeDynamicMeshBase::MaxSubdivisions)
	{
		return false;
	}

	Points[PointIdx].Settings.BevelSubdivisions = InBevelSubdivisions;
	OnBevelSubdivisionsChanged(PointIdx);

	return true;
}

bool UAvaShapeIrregularPolygonDynamicMesh::ShiftPoints(const FVector2D& Amount)
{
	for (FAvaShapeRoundedCorner& Point : Points)
	{
		Point.Location += Amount;
	}

	MarkAllMeshesDirty();

	return true;
}

void UAvaShapeIrregularPolygonDynamicMesh::SetGlobalBevelSize(float InBevelSize)
{
	if (InBevelSize < 0 || InBevelSize > 1)
	{
		return;
	}

	GlobalBevelSize = InBevelSize;
	OnGlobalBevelSizeChanged();
}

void UAvaShapeIrregularPolygonDynamicMesh::SetGlobalBevelSubdivisions(uint8 InBevelSubdivisions)
{
	if (InBevelSubdivisions > UAvaShapeDynamicMeshBase::MaxSubdivisions)
	{
		return;
	}

	GlobalBevelSubdivisions = InBevelSubdivisions;
	OnGlobalBevelSubdivisionsChanged();
}

#if WITH_EDITOR
void UAvaShapeIrregularPolygonDynamicMesh::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (!PropertyAboutToChange)
	{
		return;
	}

	const FName PropertyName = PropertyAboutToChange->GetFName();

	static const FName PointsName = GET_MEMBER_NAME_CHECKED(UAvaShapeIrregularPolygonDynamicMesh, Points);
	static const FName LocationName = GET_MEMBER_NAME_CHECKED(FAvaShapeRoundedCorner, Location);
	static const FName BevelSizeName = GET_MEMBER_NAME_CHECKED(FAvaShapeRoundedCornerSettings, BevelSize);
	static const FName BevelSubdivisionsName = GET_MEMBER_NAME_CHECKED(FAvaShapeRoundedCornerSettings, BevelSubdivisions);
	static const FName XName = FName("X");
	static const FName YName = FName("Y");

	if (PropertyName == PointsName
		|| PropertyName == LocationName
		|| PropertyName == BevelSizeName
		|| PropertyName == BevelSubdivisionsName
		|| PropertyName == XName
		|| PropertyName == YName)
	{
		BackupPoints();
	}
}

void UAvaShapeIrregularPolygonDynamicMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	static FName PointsName = GET_MEMBER_NAME_CHECKED(UAvaShapeIrregularPolygonDynamicMesh, Points);
	static FName LocationName = GET_MEMBER_NAME_CHECKED(FAvaShapeRoundedCorner, Location);
	static FName BevelSizeName = GET_MEMBER_NAME_CHECKED(FAvaShapeRoundedCornerSettings, BevelSize);
	static FName BevelSubdivisionsName = GET_MEMBER_NAME_CHECKED(FAvaShapeRoundedCornerSettings, BevelSubdivisions);
	static FName GlobalBevelSubdivisionsName = GET_MEMBER_NAME_CHECKED(UAvaShapeIrregularPolygonDynamicMesh, GlobalBevelSubdivisions);
	static FName GlobalBevelSizeName = GET_MEMBER_NAME_CHECKED(UAvaShapeIrregularPolygonDynamicMesh, GlobalBevelSize);
	static FName XName = FName("X");
	static FName YName = FName("Y");

	if (MemberName == PointsName)
	{
		if (PropertyName == LocationName
			|| PropertyName == XName
			|| PropertyName == YName)
		{
			OnLocationsUpdated();
		}
		else if (PropertyName == BevelSizeName)
		{
			OnBevelSizeChanged();
		}
		else if (PropertyName == BevelSubdivisionsName)
		{
			OnBevelSubdivisionsChanged();
		}
		else
		{
			OnPointsUpdated();
		}
	}
	else if (MemberName == GlobalBevelSubdivisionsName)
	{
		OnGlobalBevelSubdivisionsChanged();
	}
	else if (MemberName == GlobalBevelSizeName)
	{
		OnGlobalBevelSizeChanged();
	}
}
#endif

bool UAvaShapeIrregularPolygonDynamicMesh::CheckNewPointsArray()
{
	if (Points.Num() < 4)
	{
		OnPointsUpdated();
		return true;
	}

	// When points are updated en masse the only option is to check them one by one.
	TArray<FAvaShapeRoundedCorner> TempPoints;
	TempPoints.Append(Points);
	Points.Empty();

	for (int32 PointIdx = 0; PointIdx < TempPoints.Num(); ++PointIdx)
	{
		if (AddPoint(TempPoints[PointIdx].Location))
		{
			Points[PointIdx].Settings = TempPoints[PointIdx].Settings;
			Points[PointIdx].CornerMetrics = TempPoints[PointIdx].CornerMetrics;
			continue;
		}

		RestorePoints();
		return false;
	}

	RecalculateExtent();

	return true;
}

void UAvaShapeIrregularPolygonDynamicMesh::OnPointsUpdated()
{
	RecalculateExtent();

	if (CanBeGenerated())
	{
		MarkAllMeshesDirty();
	}
}

void UAvaShapeIrregularPolygonDynamicMesh::OnLocationsUpdated()
{
	for (int32 PointIdx = 0; PointIdx < Points.Num(); ++PointIdx)
	{
		if (Points[PointIdx].Location != PreEditPoints[PointIdx].Location)
		{
			OnLocationUpdated(PointIdx);
		}
	}
}

void UAvaShapeIrregularPolygonDynamicMesh::OnLocationUpdated(int32 InPointIdx)
{
	if (!Points.IsValidIndex(InPointIdx))
	{
		return;
	}

	int32 PreviousPointIdx = (InPointIdx > 0 ? InPointIdx - 1 : Points.Num() - 1);
	int32 NextPointIdx = (InPointIdx < (Points.Num() - 1) ? InPointIdx + 1 : 0);

	if (Points.Num() > 0)
	{
		/*
		 * Point / Point checking
		 */

		// Check to see if the moved point is too near any other point
		if (IsPointTooCloseToAnotherPoint(Points[InPointIdx].Location, true))
		{
			Points[InPointIdx].Location = PreEditPoints[InPointIdx].Location;
			return;
		}

		// Has at least 2 lines
		if (Points.Num() > 2)
		{
			/*
			 * Point / Line checking
			 */

			// Check to see if the moved point is too near to any lines
			if (IsPointTooCloseToALine(Points[InPointIdx].Location))
			{
				Points[InPointIdx].Location = PreEditPoints[InPointIdx].Location;
				return;
			}

			if (IsLineTooCloseToAPoint(Points[PreviousPointIdx].Location, Points[InPointIdx].Location))
			{
				Points[InPointIdx] = PreEditPoints[InPointIdx];
				return;
			}

			if (IsLineTooCloseToAPoint(Points[InPointIdx].Location, Points[NextPointIdx].Location))
			{
				Points[InPointIdx] = PreEditPoints[InPointIdx];
				return;
			}

			// Has at least 3 lines
			if (Points.Num() > 3)
			{
				/*
				 * Line / Line checking
				 */

				bool bIsEnd = (InPointIdx == 0 || InPointIdx == (Points.Num() - 1));

				// Check to see if the new lines intersect any other lines
				// First and last point can be freely placed if we're in edit mode
				if (!bIsEnd)
				{
					FVector2D PreviousPoint = Points[InPointIdx > 0 ? InPointIdx - 1 : Points.Num() - 1].Location;
					FVector2D NextPoint = Points[InPointIdx < (Points.Num() - 1) ? InPointIdx + 1 : 0].Location;

					if (DoesLineIntersectBorder(PreviousPoint, Points[InPointIdx].Location)
						|| DoesLineIntersectBorder(Points[InPointIdx].Location, NextPoint))
					{
						Points[InPointIdx] = PreEditPoints[InPointIdx];
						return;
					}
				}
			}

			// At least 3 points, so we invalidate nearby corners!
			Points[PreviousPointIdx].CornerMetrics.bValid = false;
			Points[InPointIdx].CornerMetrics.bValid = false;
			Points[NextPointIdx].CornerMetrics.bValid = false;
		}
	}

	RecalculateExtent();

	if (CanBeGenerated())
	{
		MarkAllMeshesDirty();
	}
}

void UAvaShapeIrregularPolygonDynamicMesh::OnBevelSizeChanged()
{
	for (int32 PointIdx = 0; PointIdx < Points.Num(); ++PointIdx)
	{
		if (Points[PointIdx].Settings.BevelSize != PreEditPoints[PointIdx].Settings.BevelSize)
		{
			OnBevelSizeChanged(PointIdx);
		}
	}
}

void UAvaShapeIrregularPolygonDynamicMesh::OnBevelSizeChanged(int32 PointIdx)
{
	if (!Points.IsValidIndex(PointIdx))
	{
		return;
	}

	if (Points[PointIdx].Settings.BevelSize > 0 && Points[PointIdx].Settings.BevelSubdivisions == 0)
	{
		SetBevelSubdivisions(PointIdx, UAvaShapeDynamicMeshBase::DefaultSubdivisions);
	}

	Points[PointIdx].CornerMetrics.bValid = false;

	if (CanBeGenerated())
	{
		MarkAllMeshesDirty();
	}
}

void UAvaShapeIrregularPolygonDynamicMesh::OnBevelSubdivisionsChanged()
{
	for (int32 PointIdx = 0; PointIdx < Points.Num(); ++PointIdx)
	{
		if (Points[PointIdx].Settings.BevelSubdivisions != PreEditPoints[PointIdx].Settings.BevelSubdivisions)
		{
			OnBevelSubdivisionsChanged(PointIdx);
		}
	}
}

void UAvaShapeIrregularPolygonDynamicMesh::OnBevelSubdivisionsChanged(int32 PointIdx)
{
	if (!Points.IsValidIndex(PointIdx))
	{
		return;
	}

	if (Points[PointIdx].Settings.BevelSubdivisions > 0 && Points[PointIdx].Settings.BevelSize == 0.f)
	{
		SetBevelSize(PointIdx, 0.25f);
	}

	Points[PointIdx].CornerMetrics.bValid = false;

	if (CanBeGenerated())
	{
		MarkAllMeshesDirty();
	}
}

void UAvaShapeIrregularPolygonDynamicMesh::OnGlobalBevelSubdivisionsChanged()
{
	for (int32 PointIdx = 0; PointIdx < Points.Num(); ++PointIdx)
	{
		SetBevelSubdivisions(PointIdx, GlobalBevelSubdivisions);
	}
}

void UAvaShapeIrregularPolygonDynamicMesh::OnGlobalBevelSizeChanged()
{
	for (int32 PointIdx = 0; PointIdx < Points.Num(); ++PointIdx)
	{
		SetBevelSize(PointIdx, GlobalBevelSize);
	}
}

FVector UAvaShapeIrregularPolygonDynamicMesh::ScreenToWorld(const FVector2D& ScreenLocation) const
{
	FTransform ActorTransform = GetTransform();
	FVector WorldLocation = FVector(0.f, ScreenLocation.X, ScreenLocation.Y);
	WorldLocation = ActorTransform.TransformPosition(WorldLocation);

	return WorldLocation;
}

bool UAvaShapeIrregularPolygonDynamicMesh::IsLocationInsideShape(const FVector2D& Location)
{
	// Use "Infinite" ray edge intersections
	FVector2D Start = Location - 9999.f;
	int32 EdgeCrosses = 0;

	for (int32 PointIdx = 0; PointIdx < Points.Num(); ++PointIdx)
	{
		FVector2D EdgeStart = Points[PointIdx == 0 ? Points.Num() - 1 : PointIdx - 1].Location;
		FVector2D EdgeEnd =  Points[PointIdx].Location;

		if (UAvaShapeIrregularPolygonDynamicMesh::DoLinesIntersect(Start, Location, EdgeStart, EdgeEnd))
		{
			++EdgeCrosses;
		}
	}

	return ((EdgeCrosses % 2) == 1);
}

bool UAvaShapeIrregularPolygonDynamicMesh::ClearMesh()
{
	if (!Super::ClearMesh())
	{
		return false;
	}

	for (FAvaShapeRoundedCorner& Point : Points)
	{
		Point.CornerMetrics.End.ClearIndex();
		Point.CornerMetrics.Start.ClearIndex();
		Point.CornerMetrics.VertexAnchor.ClearIndex();
	}

	return true;
}

bool UAvaShapeIrregularPolygonDynamicMesh::CreateMesh(FAvaShapeMesh& InMesh)
{
	if (!CanBeGenerated())
	{
		return false;
	}

	if (InMesh.GetMeshIndex() == MESH_INDEX_PRIMARY)
	{
		AActor* Actor = GetShapeActor();

		for (int32 PointIdx = 0; PointIdx < Points.Num(); ++PointIdx)
		{
			LocalSnapPoints.Add(FAvaSnapPoint::CreateActorCustomSnapPoint(Actor, Points[PointIdx].Location, PointIdx));
		}

		// Calculate corner infos
		for (int32 PointIdx = 0; PointIdx < Points.Num(); ++PointIdx)
		{
			if (Points[PointIdx].CornerMetrics.bValid)
			{
				continue;
			}

			FVector2D PreviousPoint = Points[PointIdx > 0 ? PointIdx - 1 : Points.Num() - 1].Location;
			FVector2D NextPoint = Points[PointIdx < (Points.Num() - 1) ? PointIdx + 1 : 0].Location;
			FVector2D LineIn = Points[PointIdx].Location - PreviousPoint;
			FVector2D LineOut = NextPoint - Points[PointIdx].Location;
			float BevelSize = GetMaxBevelSizeForPoint(PointIdx) * Points[PointIdx].Settings.BevelSize;
			bool bConvex = true;

			if (Points[PointIdx].Settings.BevelSize > 0)
			{
				// Calculate the midpoint of the corner ends
				FVector2D MidCornerPoint = ((Points[PointIdx].Location - LineIn.GetSafeNormal() * BevelSize)
					+ (Points[PointIdx].Location + LineOut.GetSafeNormal() * BevelSize)) / 2.f;

				// Move it closer to the corner (at least > root2)
				MidCornerPoint = (MidCornerPoint + Points[PointIdx].Location) / 2.f;

				bConvex = IsLocationInsideShape(MidCornerPoint);
			}

			Points[PointIdx].CornerMetrics = UAvaShapeRoundedPolygonDynamicMesh::CreateCornerInfo(
				LineIn,
				LineOut,
				Points[PointIdx].Location,
				BevelSize,
				Points[PointIdx].Settings.BevelSubdivisions,
				bConvex
			);
		}

		// Add non-corner vertices and polys.
		TArray< FVector2D > PointsTV2;
		TArray< UE::Geometry::FIndex3i> OutTriangles;

		for (int32 PointIdx = 0; PointIdx < Points.Num(); ++PointIdx)
		{
			if (Points[PointIdx].CornerMetrics.Angle <= 0.f)
			{
				CacheVertex(InMesh, Points[PointIdx].Location);
				PointsTV2.Add({ Points[PointIdx].Location.X, Points[PointIdx].Location.Y });
				continue;
			}

			CacheVertex(InMesh, Points[PointIdx].CornerMetrics.Start);
			PointsTV2.Add({ Points[PointIdx].CornerMetrics.Start.Location.X,
				Points[PointIdx].CornerMetrics.Start.Location.Y });

			if (!Points[PointIdx].CornerMetrics.bConvex)
			{
				CacheVertex(InMesh, Points[PointIdx].CornerMetrics.VertexAnchor);
				PointsTV2.Add(FVector2D{ Points[PointIdx].CornerMetrics.VertexAnchor.Location.X,
					Points[PointIdx].CornerMetrics.VertexAnchor.Location.Y });
			}

			CacheVertex(InMesh, Points[PointIdx].CornerMetrics.End);
			PointsTV2.Add(FVector2D{ Points[PointIdx].CornerMetrics.End.Location.X,
				Points[PointIdx].CornerMetrics.End.Location.Y });
		}

		PolygonTriangulation::TriangulateSimplePolygon<double>(PointsTV2, OutTriangles, true);

		if (OutTriangles.Num() == 0)
		{
			return false;
		}

		FVector Normal = FVector::ZeroVector;;
		FVector::FReal BiggestNormal = 0.0;

		for (int32 TriangleIdx = 0; TriangleIdx < OutTriangles.Num() && Normal.Z > -0.1 && Normal.Z < 0.1; ++TriangleIdx)
		{
			FVector2D AB = { PointsTV2[OutTriangles[TriangleIdx].B].X - PointsTV2[OutTriangles[TriangleIdx].A].X,
				PointsTV2[OutTriangles[TriangleIdx].B].Y - PointsTV2[OutTriangles[TriangleIdx].A].Y };

			FVector2D AC = { PointsTV2[OutTriangles[TriangleIdx].C].X - PointsTV2[OutTriangles[TriangleIdx].A].X,
				PointsTV2[OutTriangles[TriangleIdx].C].Y - PointsTV2[OutTriangles[TriangleIdx].A].Y };

			Normal = FVector(AB.X, AB.Y, 0.f).GetSafeNormal().Cross(FVector(AC.X, AC.Y, 0.f).GetSafeNormal());

			if (FMath::Abs(Normal.Z) > FMath::Abs(BiggestNormal))
			{
				BiggestNormal = Normal.Z;
			}
		}

		bool bReverse = (BiggestNormal < 0);

		for (UE::Geometry::FIndex3i Triangle : OutTriangles)
		{
			if (!bReverse)
			{
				AddTriangle(InMesh, Triangle.A, Triangle.B, Triangle.C);
			}
			else
			{
				AddTriangle(InMesh, Triangle.A, Triangle.C, Triangle.B);
			}
		}

		for (int32 PointIdx = 0; PointIdx < Points.Num(); ++PointIdx)
		{
			if (Points[PointIdx].CornerMetrics.Angle <= 0.f)
			{
				continue;
			}

			UAvaShapeRoundedPolygonDynamicMesh::CreateRoundedCorner(this, InMesh,
				Points[PointIdx].Location, Points[PointIdx].CornerMetrics, true);
		}
	}

	return Super::CreateMesh(InMesh);
}

bool UAvaShapeIrregularPolygonDynamicMesh::CanUpdateMeshInAnyThread() const
{
	return GetClass() == UAvaShapeIrregularPolygonDynamicMesh::StaticClass();
}
