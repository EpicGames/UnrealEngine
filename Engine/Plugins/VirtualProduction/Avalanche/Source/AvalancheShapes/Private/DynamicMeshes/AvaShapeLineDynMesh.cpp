// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshes/AvaShapeLineDynMesh.h"

const FString UAvaShapeLineDynamicMesh::MeshName = TEXT("Line");

void UAvaShapeLineDynamicMesh::SetLineWidth(float InLineWidth)
{
	if (InLineWidth == LineWidth)
	{
		return;
	}

	if (InLineWidth < 0.f)
	{
		return;
	}

	LineWidth = InLineWidth;
	OnLineWidthChanged();
}

void UAvaShapeLineDynamicMesh::SetVector(const FVector2D& InVector)
{
	if (InVector == Vector)
	{
		return;
	}

	if (InVector == FVector2D::ZeroVector)
	{
		return;
	}

	Vector = InVector;
	OnVectorChanged();
}

void UAvaShapeLineDynamicMesh::GenerateBorderVertices(TArray<FVector2D>& BorderVertices)
{
	Super::GenerateBorderVertices(BorderVertices);

	TArray<FVector2D> LineVertices = GenerateLineVertices();
	BorderVertices.Append(LineVertices);
}

bool UAvaShapeLineDynamicMesh::CanUpdateMeshInAnyThread() const
{
	return GetClass() == UAvaShapeLineDynamicMesh::StaticClass();
}

void UAvaShapeLineDynamicMesh::OnLineWidthChanged()
{
	if (LineWidth < 0.f)
	{
		LineWidth = 1.f;
	}

	UpdateExtent();
	MarkAllMeshesDirty();
}

void UAvaShapeLineDynamicMesh::OnVectorChanged()
{
	if (Vector == FVector2D::ZeroVector)
	{
		Vector = FVector2D(1.f, 0.f);
	}

	UpdateExtent();
	MarkAllMeshesDirty();
}

void UAvaShapeLineDynamicMesh::UpdateExtent()
{
	TArray<FVector2D> LineVertices = GenerateLineVertices();

	if (LineVertices.Num() < 4)
	{
		return;
	}

	Size2D.X = FMath::Max(FMath::Max(FMath::Max(LineVertices[0].X, LineVertices[1].X), LineVertices[2].X), LineVertices[3].X)
		- FMath::Min(FMath::Min(FMath::Min(LineVertices[0].X, LineVertices[1].X), LineVertices[2].X), LineVertices[3].X);

	Size2D.Y = FMath::Max(FMath::Max(FMath::Max(LineVertices[0].Y, LineVertices[1].Y), LineVertices[2].Y), LineVertices[3].Y)
		- FMath::Min(FMath::Min(FMath::Min(LineVertices[0].Y, LineVertices[1].Y), LineVertices[2].Y), LineVertices[3].Y);

	Size3D.Y = Size2D.X;
	Size3D.Z = Size2D.Y;
}

TArray<FVector2D> UAvaShapeLineDynamicMesh::GenerateLineVertices() const
{
	TArray<FVector2D> LineVertices;

	if (LineWidth <= 0.f || Vector == FVector2D::ZeroVector)
	{
		return LineVertices;
	}


	FVector2D SideOffset = Vector.GetSafeNormal().GetRotated(90.f) * LineWidth / 2.f;

	LineVertices.Add(-Vector / 2.f - SideOffset);
	LineVertices.Add(-Vector / 2.f + SideOffset);
	LineVertices.Add(Vector / 2.f + SideOffset);
	LineVertices.Add(Vector / 2.f - SideOffset);

	return LineVertices;
}

bool UAvaShapeLineDynamicMesh::IsMeshVisible(int32 MeshIndex)
{
	switch (MeshIndex)
	{
		case MESH_INDEX_PRIMARY:
			return LineWidth > UE_KINDA_SMALL_NUMBER;
		default:
			return true;
	}
}

#if WITH_EDITOR
void UAvaShapeLineDynamicMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static FName LineWidthName = GET_MEMBER_NAME_CHECKED(UAvaShapeLineDynamicMesh, LineWidth);
	static FName VectorName = GET_MEMBER_NAME_CHECKED(UAvaShapeLineDynamicMesh, Vector);

	if (PropertyChangedEvent.MemberProperty->GetFName() == LineWidthName)
	{
		OnLineWidthChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == VectorName)
	{
		OnVectorChanged();
	}
}
#endif
