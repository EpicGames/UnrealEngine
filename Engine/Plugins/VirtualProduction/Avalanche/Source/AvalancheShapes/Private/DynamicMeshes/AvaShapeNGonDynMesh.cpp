// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshes/AvaShapeNGonDynMesh.h"

const FString UAvaShapeNGonDynamicMesh::MeshName = TEXT("RegularPolygon");

void UAvaShapeNGonDynamicMesh::SetNumSides(uint8 InNumSides)
{
	if (NumSides == InNumSides)
	{
		return;
	}

	if (InNumSides < UAvaShapeNGonDynamicMesh::MinNumSides || InNumSides > UAvaShapeNGonDynamicMesh::MaxNumSides)
	{
		return;
	}

	NumSides = InNumSides;
	OnNumSidesChanged();
}

#if WITH_EDITOR
void UAvaShapeNGonDynamicMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static FName NumSidesName = GET_MEMBER_NAME_CHECKED(UAvaShapeNGonDynamicMesh, NumSides);

	if (PropertyChangedEvent.MemberProperty->GetFName() == NumSidesName)
	{
		OnNumSidesChanged();
	}
}
#endif

void UAvaShapeNGonDynamicMesh::OnNumSidesChanged()
{
	NumSides = FMath::Clamp(NumSides, UAvaShapeNGonDynamicMesh::MinNumSides, UAvaShapeNGonDynamicMesh::MaxNumSides);
	MarkAllMeshesDirty();
}

void UAvaShapeNGonDynamicMesh::GenerateBorderVertices(TArray<FVector2D>& BorderVertices)
{
	Super::GenerateBorderVertices(BorderVertices);

	const FVector2D ZeroRotation = FVector2D(0.f, Size2D.Y / 2.f);
	const FVector2D VScale = {
		Size2D.X >= Size2D.Y ? Size2D.X / Size2D.Y : 1.f,
		Size2D.Y >= Size2D.X ? Size2D.Y / Size2D.X : 1.f
	};

	for (uint8 SideIdx = 0; SideIdx < NumSides; ++SideIdx)
	{
		FVector2D RotatedZero = ZeroRotation.GetRotated((360.f / NumSides) * -SideIdx);
		RotatedZero *= VScale;
		BorderVertices.Add(RotatedZero);
	}
}

bool UAvaShapeNGonDynamicMesh::CanUpdateMeshInAnyThread() const
{
	return GetClass() == UAvaShapeNGonDynamicMesh::StaticClass();
}
