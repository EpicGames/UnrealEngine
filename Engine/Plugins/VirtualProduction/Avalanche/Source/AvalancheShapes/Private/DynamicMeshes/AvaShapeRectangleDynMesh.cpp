// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshes/AvaShapeRectangleDynMesh.h"
#include "AvaShapesDefs.h"
#include "DynamicMeshes/AvaShapeRoundedPolygonDynMesh.h"

const FString UAvaShapeRectangleDynamicMesh::MeshName = TEXT("Rectangle");

void UAvaShapeRectangleDynamicMesh::SetHorizontalAlignment(EAvaHorizontalAlignment InHorizontalAlignment)
{
	if (HorizontalAlignment == InHorizontalAlignment)
	{
		return;
	}

	HorizontalAlignment = InHorizontalAlignment;
	OnAlignmentChanged();
}

void UAvaShapeRectangleDynamicMesh::SetVerticalAlignment(EAvaVerticalAlignment InVerticalAlignment)
{
	if (VerticalAlignment == InVerticalAlignment)
	{
		return;
	}

	VerticalAlignment = InVerticalAlignment;
	OnAlignmentChanged();
}

void UAvaShapeRectangleDynamicMesh::SetBottomRightBevelSubdivisions(uint8 InBevelSubdivisions)
{
	if (BottomRight.BevelSubdivisions == InBevelSubdivisions)
	{
		return;
	}

	if (InBevelSubdivisions > UAvaShapeDynamicMeshBase::MaxSubdivisions)
	{
		return;
	}

	BottomRight.BevelSubdivisions = InBevelSubdivisions;
	OnBottomRightBevelSubdivisionsChanged();
}

float UAvaShapeRectangleDynamicMesh::GetMaximumBevelSize() const
{
	return (FMath::Min(Size2D.X, Size2D.Y) / 2.f) - CornerMinMargin;
}

bool UAvaShapeRectangleDynamicMesh::IsSlantAngleValid() const
{
	const FVector2D TopLeftCorner(LeftSlant < 0.f ? 0.f : Size2D.Y * FMath::Tan(FMath::DegreesToRadians(LeftSlant)), Size2D.Y);
	const FVector2D TopRightCorner(RightSlant >= 0.f ? Size2D.X : Size2D.X + Size2D.Y * FMath::Tan(FMath::DegreesToRadians(RightSlant)), Size2D.Y);
	const FVector2D BottomLeftCorner(LeftSlant >= 0.f ? 0.f : -Size2D.Y * FMath::Tan(FMath::DegreesToRadians(LeftSlant)), 0.f);
	const FVector2D BottomRightCorner(RightSlant < 0.f ? Size2D.X : Size2D.X - Size2D.Y * FMath::Tan(FMath::DegreesToRadians(RightSlant)), 0.f);

	if (TopRightCorner.X + UE_KINDA_SMALL_NUMBER <= TopLeftCorner.X)
	{
		return false;
	}
	if (BottomRightCorner.X + UE_KINDA_SMALL_NUMBER <= BottomLeftCorner.X)
	{
		return false;
	}

	return true;
}

FVector2D UAvaShapeRectangleDynamicMesh::GetValidRangeLeftSlantAngle() const
{
	const float MaxX = FMath::Clamp(Size2D.Y, 0, Size2D.X);

	// Flat slant = 0 deg
	FVector2D TopLeftCorner(0.f, Size2D.Y);
	FVector2D BottomLeftCorner(0.f, 0.f);

	// Max Slant
	TopLeftCorner.X = MaxX;
	BottomLeftCorner.X = 0.f;

	const float MaxLeftSlant = 90 - FMath::RadiansToDegrees(FMath::Atan2(TopLeftCorner.Y - BottomLeftCorner.Y, TopLeftCorner.X - BottomLeftCorner.X));

	// Min Slant
	TopLeftCorner.X = 0;
	BottomLeftCorner.X = -MaxX;

	const float MinLeftSlant = -90 + FMath::RadiansToDegrees(FMath::Atan2(TopLeftCorner.Y - BottomLeftCorner.Y, TopLeftCorner.X - BottomLeftCorner.X));

	return FVector2D(MinLeftSlant, MaxLeftSlant);
}

FVector2D UAvaShapeRectangleDynamicMesh::GetValidRangeRightSlantAngle() const
{
	const float MaxX = FMath::Clamp(Size2D.Y, 0, Size2D.X);

	// Flat slant = 0 deg
	FVector2D TopRightCorner(Size2D.X, Size2D.Y);
	FVector2D BottomRightCorner(Size2D.X, 0.f);

	// Max Slant
	TopRightCorner.X = Size2D.X + MaxX;
	BottomRightCorner.X = Size2D.X;

	const float MaxLeftSlant = 90 - FMath::RadiansToDegrees(FMath::Atan2(TopRightCorner.Y - BottomRightCorner.Y, TopRightCorner.X - BottomRightCorner.X));

	// Min Slant
	TopRightCorner.X = Size2D.X;
	BottomRightCorner.X = Size2D.X - MaxX;

	const float MinLeftSlant = -90 + FMath::RadiansToDegrees(FMath::Atan2(TopRightCorner.Y - BottomRightCorner.Y, TopRightCorner.X - BottomRightCorner.X));

	return FVector2D(MinLeftSlant, MaxLeftSlant);
}

void UAvaShapeRectangleDynamicMesh::GetValidSlantAngle(float& OutLeftSlant, float& OutRightSlant) const
{
	const FVector2D OutRangeLeft = GetValidRangeLeftSlantAngle();
	const FVector2D OutRangeRight = GetValidRangeRightSlantAngle();

	// Clamp base on mesh size
	OutLeftSlant = FMath::Clamp(LeftSlant, OutRangeLeft.X, OutRangeLeft.Y);
	OutRightSlant = FMath::Clamp(RightSlant, OutRangeRight.X, OutRangeRight.Y);

	if (OutLeftSlant == 0 || OutRightSlant == 0)
	{
		return;
	}

	const float MaxSizeX = Size2D.X;

	// check corners do not exceed max size
	const float LeftSlantSizeX = Size2D.Y * FMath::Tan(FMath::DegreesToRadians(OutLeftSlant));
	const float RightSlantSizeX = -Size2D.Y * FMath::Tan(FMath::DegreesToRadians(OutRightSlant));

	const float SizeX = FMath::Abs(LeftSlantSizeX + RightSlantSizeX);

	if (SizeX >= MaxSizeX)
	{
		const float SizeRatio = MaxSizeX / SizeX;

		// Calculate the new left and right slant angles
		OutLeftSlant = FMath::RadiansToDegrees(FMath::Atan(LeftSlantSizeX * SizeRatio / Size2D.Y));
		OutRightSlant = FMath::Sign(LeftSlant * RightSlant) * FMath::RadiansToDegrees(FMath::Atan(RightSlantSizeX * SizeRatio / Size2D.Y));
	}
}

void UAvaShapeRectangleDynamicMesh::SetLeftSlant(float InSlant)
{
	if (LeftSlant == InSlant)
	{
		return;
	}

	if (InSlant < MinSlantAngle || InSlant > MaxSlantAngle)
	{
		return;
	}

	LeftSlant = InSlant;
	OnLeftSlantChanged();
}

void UAvaShapeRectangleDynamicMesh::SetRightSlant(float InSlant)
{
	if (RightSlant == InSlant)
	{
		return;
	}

	if (InSlant < MinSlantAngle || InSlant > MaxSlantAngle)
	{
		return;
	}

	RightSlant = InSlant;
	OnRightSlantChanged();
}

void UAvaShapeRectangleDynamicMesh::SetGlobalBevelSize(float InBevelSize)
{
	if (InBevelSize < 0 || InBevelSize > GetMaximumBevelSize())
	{
		return;
	}

	GlobalBevelSize = InBevelSize;
	OnGlobalBevelSizeChanged();
}

void UAvaShapeRectangleDynamicMesh::SetGlobalBevelSubdivisions(uint8 InGlobalBevelSubdivisions)
{
	if (InGlobalBevelSubdivisions > UAvaShapeDynamicMeshBase::MaxSubdivisions)
	{
		return;
	}

	GlobalBevelSubdivisions = InGlobalBevelSubdivisions;
	OnGlobalBevelSubdivisionsChanged();
}

void UAvaShapeRectangleDynamicMesh::SetTopLeft(const FAvaShapeRectangleCornerSettings& InCornerSettings)
{
	SetTopLeftCornerType(InCornerSettings.Type);
	SetTopLeftBevelSize(InCornerSettings.BevelSize);
	SetTopLeftBevelSubdivisions(InCornerSettings.BevelSubdivisions);
}

void UAvaShapeRectangleDynamicMesh::SetTopRight(const FAvaShapeRectangleCornerSettings& InCornerSettings)
{
	SetTopRightCornerType(InCornerSettings.Type);
	SetTopRightBevelSize(InCornerSettings.BevelSize);
	SetTopRightBevelSubdivisions(InCornerSettings.BevelSubdivisions);
}

void UAvaShapeRectangleDynamicMesh::SetBottomLeft(const FAvaShapeRectangleCornerSettings& InCornerSettings)
{
	SetBottomLeftCornerType(InCornerSettings.Type);
	SetBottomLeftBevelSize(InCornerSettings.BevelSize);
	SetBottomLeftBevelSubdivisions(InCornerSettings.BevelSubdivisions);
}

void UAvaShapeRectangleDynamicMesh::SetBottomRight(const FAvaShapeRectangleCornerSettings& InCornerSettings)
{
	SetBottomRightCornerType(InCornerSettings.Type);
	SetBottomRightBevelSize(InCornerSettings.BevelSize);
	SetBottomRightBevelSubdivisions(InCornerSettings.BevelSubdivisions);
}

void UAvaShapeRectangleDynamicMesh::SetTopLeftCornerType(EAvaShapeCornerType InType)
{
	if (TopLeft.Type == InType)
	{
		return;
	}

	TopLeft.Type = InType;
	OnTopLeftCornerTypeChanged();
}

void UAvaShapeRectangleDynamicMesh::SetTopLeftBevelSize(float InSize)
{
	if (InSize == TopLeft.BevelSize)
	{
		return;
	}

	if (InSize < 0 || InSize > GetMaximumBevelSize())
	{
		return;
	}

	TopLeft.BevelSize = InSize;
	OnTopLeftBevelSizeChanged();
}

void UAvaShapeRectangleDynamicMesh::SetTopLeftBevelSubdivisions(uint8 InBevelSubdivisions)
{
	if (TopLeft.BevelSubdivisions == InBevelSubdivisions)
	{
		return;
	}

	if (InBevelSubdivisions > UAvaShapeDynamicMeshBase::MaxSubdivisions)
	{
		return;
	}

	TopLeft.BevelSubdivisions = InBevelSubdivisions;
	OnTopLeftBevelSubdivisionsChanged();
}

void UAvaShapeRectangleDynamicMesh::SetTopRightCornerType(EAvaShapeCornerType InType)
{
	if (TopRight.Type == InType)
	{
		return;
	}

	TopRight.Type = InType;
	OnTopRightCornerTypeChanged();
}

void UAvaShapeRectangleDynamicMesh::SetTopRightBevelSize(float InSize)
{
	if (InSize == TopRight.BevelSize)
	{
		return;
	}

	if (InSize < 0 || InSize > GetMaximumBevelSize())
	{
		return;
	}

	TopRight.BevelSize = InSize;
	OnTopRightBevelSizeChanged();
}

void UAvaShapeRectangleDynamicMesh::SetTopRightBevelSubdivisions(uint8 InBevelSubdivisions)
{
	if (TopRight.BevelSubdivisions == InBevelSubdivisions)
	{
		return;
	}

	if (InBevelSubdivisions > UAvaShapeDynamicMeshBase::MaxSubdivisions)
	{
		return;
	}

	TopRight.BevelSubdivisions = InBevelSubdivisions;
	OnTopRightBevelSubdivisionsChanged();
}

void UAvaShapeRectangleDynamicMesh::SetBottomLeftCornerType(EAvaShapeCornerType InType)
{
	if (BottomLeft.Type == InType)
	{
		return;
	}

	BottomLeft.Type = InType;
	OnBottomLeftCornerTypeChanged();
}

void UAvaShapeRectangleDynamicMesh::SetBottomLeftBevelSize(float InSize)
{
	if (InSize == BottomLeft.BevelSize)
	{
		return;
	}

	if (InSize < 0 || InSize > GetMaximumBevelSize())
	{
		return;
	}

	BottomLeft.BevelSize = InSize;
	OnBottomLeftBevelSizeChanged();
}

void UAvaShapeRectangleDynamicMesh::SetBottomLeftBevelSubdivisions(uint8 InBevelSubdivisions)
{
	if (BottomLeft.BevelSubdivisions == InBevelSubdivisions)
	{
		return;
	}

	if (InBevelSubdivisions > UAvaShapeDynamicMeshBase::MaxSubdivisions)
	{
		return;
	}

	BottomLeft.BevelSubdivisions = InBevelSubdivisions;
	OnBottomLeftBevelSubdivisionsChanged();
}

void UAvaShapeRectangleDynamicMesh::SetBottomRightCornerType(EAvaShapeCornerType InType)
{
	if (BottomRight.Type == InType)
	{
		return;
	}

	BottomRight.Type = InType;
	OnBottomRightCornerTypeChanged();
}

void UAvaShapeRectangleDynamicMesh::SetBottomRightBevelSize(float InSize)
{
	if (InSize == BottomRight.BevelSize)
	{
		return;
	}

	if (InSize < 0 || InSize > GetMaximumBevelSize())
	{
		return;
	}

	BottomRight.BevelSize = InSize;
	OnBottomRightBevelSizeChanged();
}

void UAvaShapeRectangleDynamicMesh::OnAlignmentChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShapeRectangleDynamicMesh::OnLeftSlantChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShapeRectangleDynamicMesh::OnRightSlantChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShapeRectangleDynamicMesh::OnGlobalBevelSizeChanged()
{
	GlobalBevelSize = FMath::Clamp(GlobalBevelSize, 0, GetMaximumBevelSize());
	SetTopLeftBevelSize(GlobalBevelSize);
	SetTopRightBevelSize(GlobalBevelSize);
	SetBottomLeftBevelSize(GlobalBevelSize);
	SetBottomRightBevelSize(GlobalBevelSize);
}

void UAvaShapeRectangleDynamicMesh::OnGlobalBevelSubdivisionsChanged()
{
	SetTopLeftBevelSubdivisions(GlobalBevelSubdivisions);
	SetTopRightBevelSubdivisions(GlobalBevelSubdivisions);
	SetBottomLeftBevelSubdivisions(GlobalBevelSubdivisions);
	SetBottomRightBevelSubdivisions(GlobalBevelSubdivisions);
}

void UAvaShapeRectangleDynamicMesh::OnTopLeftBevelSizeChanged()
{
	TopLeft.BevelSize = FMath::Clamp(TopLeft.BevelSize, 0.f, GetMaximumBevelSize());

	if (TopLeft.BevelSize > 0.f)
	{
		if (TopLeft.Type == EAvaShapeCornerType::Point)
		{
			TopLeft.Type = EAvaShapeCornerType::CurveIn;
		}
	}

	MarkAllMeshesDirty();
}

void UAvaShapeRectangleDynamicMesh::OnTopLeftCornerTypeChanged()
{
	if (TopLeft.Type != EAvaShapeCornerType::Point)
	{
		if (TopLeft.BevelSize == 0.f)
		{
			TopLeft.BevelSize = GetMaximumBevelSize() / 4.f;
		}
	}

	MarkAllMeshesDirty();
}

void UAvaShapeRectangleDynamicMesh::OnTopLeftBevelSubdivisionsChanged()
{
	if (TopLeft.BevelSubdivisions != 0)
	{
		if (TopLeft.Type == EAvaShapeCornerType::Point)
		{
			TopLeft.Type = EAvaShapeCornerType::CurveIn;
		}

		if (TopLeft.BevelSize == 0.f)
		{
			TopLeft.BevelSize = GetMaximumBevelSize() / 4.f;
		}
	}

	MarkAllMeshesDirty();
}

void UAvaShapeRectangleDynamicMesh::OnBottomLeftBevelSizeChanged()
{
	BottomLeft.BevelSize = FMath::Clamp(BottomLeft.BevelSize, 0.f, GetMaximumBevelSize());

	if (BottomLeft.BevelSize > 0.f)
	{
		if (BottomLeft.Type == EAvaShapeCornerType::Point)
		{
			BottomLeft.Type = EAvaShapeCornerType::CurveIn;
		}
	}

	MarkAllMeshesDirty();
}

void UAvaShapeRectangleDynamicMesh::OnBottomLeftCornerTypeChanged()
{
	if (BottomLeft.Type != EAvaShapeCornerType::Point)
	{
		if (BottomLeft.BevelSize == 0.f)
		{
			BottomLeft.BevelSize = GetMaximumBevelSize() / 4.f;
		}
	}

	MarkAllMeshesDirty();
}

void UAvaShapeRectangleDynamicMesh::OnBottomLeftBevelSubdivisionsChanged()
{
	if (BottomLeft.BevelSubdivisions != 0)
	{
		if (BottomLeft.Type == EAvaShapeCornerType::Point)
		{
			BottomLeft.Type = EAvaShapeCornerType::CurveIn;
		}

		if (BottomLeft.BevelSize == 0.f)
		{
			BottomLeft.BevelSize = GetMaximumBevelSize() / 4.f;
		}
	}

	MarkAllMeshesDirty();
}

void UAvaShapeRectangleDynamicMesh::OnTopRightBevelSizeChanged()
{
	TopRight.BevelSize = FMath::Clamp(TopRight.BevelSize, 0.f, GetMaximumBevelSize());

	if (TopRight.BevelSize > 0.f)
	{
		if (TopRight.Type == EAvaShapeCornerType::Point)
		{
			TopRight.Type = EAvaShapeCornerType::CurveIn;
		}
	}

	MarkAllMeshesDirty();
}

void UAvaShapeRectangleDynamicMesh::OnTopRightCornerTypeChanged()
{
	if (TopRight.Type != EAvaShapeCornerType::Point)
	{
		if (TopRight.BevelSize == 0.f)
		{
			TopRight.BevelSize = GetMaximumBevelSize() / 4.f;
		}
	}

	MarkAllMeshesDirty();
}

void UAvaShapeRectangleDynamicMesh::OnTopRightBevelSubdivisionsChanged()
{
	if (TopRight.BevelSubdivisions != 0)
	{
		if (TopRight.Type == EAvaShapeCornerType::Point)
		{
			TopRight.Type = EAvaShapeCornerType::CurveIn;
		}

		if (TopRight.BevelSize == 0.f)
		{
			TopRight.BevelSize = GetMaximumBevelSize() / 4.f;
		}
	}

	MarkAllMeshesDirty();
}

void UAvaShapeRectangleDynamicMesh::OnBottomRightBevelSizeChanged()
{
	BottomRight.BevelSize = FMath::Clamp(BottomRight.BevelSize, 0.f, GetMaximumBevelSize());

	if (BottomRight.BevelSize > 0.f)
	{
		if (BottomRight.Type == EAvaShapeCornerType::Point)
		{
			BottomRight.Type = EAvaShapeCornerType::CurveIn;
		}
	}

	MarkAllMeshesDirty();
}

void UAvaShapeRectangleDynamicMesh::OnBottomRightCornerTypeChanged()
{
	if (BottomRight.Type != EAvaShapeCornerType::Point)
	{
		if (BottomRight.BevelSize == 0.f)
		{
			BottomRight.BevelSize = GetMaximumBevelSize() / 4.f;
		}
	}

	MarkAllMeshesDirty();
}

void UAvaShapeRectangleDynamicMesh::OnBottomRightBevelSubdivisionsChanged()
{
	if (BottomRight.BevelSubdivisions != 0)
	{
		if (BottomRight.Type == EAvaShapeCornerType::Point)
		{
			BottomRight.Type = EAvaShapeCornerType::CurveIn;
		}

		if (BottomRight.BevelSize == 0.f)
		{
			BottomRight.BevelSize = GetMaximumBevelSize() / 4.f;
		}
	}

	MarkAllMeshesDirty();
}

bool UAvaShapeRectangleDynamicMesh::GenerateBaseMeshSections(FAvaShapeMesh& PrimaryMesh)
{
	const static FVector2D PlusX = FVector2D(1.f, 0.f);
	const static FVector2D MinuxX = FVector2D(-1.f, 0.f);

	FVector2D AnchorOffset = FVector2D::ZeroVector;

	switch (HorizontalAlignment)
	{
		case EAvaHorizontalAlignment::Left:
			AnchorOffset.X = Size2D.X / 2.f;
			break;

		case EAvaHorizontalAlignment::Right:
			AnchorOffset.X = -Size2D.X / 2.f;
			break;

		default:
			// Do nothing
			break;
	}

	switch (VerticalAlignment)
	{
		case EAvaVerticalAlignment::Top:
			AnchorOffset.Y = -Size2D.Y / 2.f;
			break;

		case EAvaVerticalAlignment::Bottom:
			AnchorOffset.Y = Size2D.Y / 2.f;
			break;

		default:
			// Do nothing
			break;
	}

	const float MaxBevelSize = GetMaximumBevelSize();

	FAvaShapeRectangleCornerSettings TopLeftCopy = TopLeft;
	FAvaShapeRectangleCornerSettings TopRightCopy = TopRight;
	FAvaShapeRectangleCornerSettings BottomLeftCopy = BottomLeft;
	FAvaShapeRectangleCornerSettings BottomRightCopy = BottomRight;

	TopLeftCopy.BevelSize = FMath::Min(TopLeftCopy.BevelSize, MaxBevelSize);
	TopRightCopy.BevelSize = FMath::Min(TopRightCopy.BevelSize, MaxBevelSize);
	BottomLeftCopy.BevelSize = FMath::Min(BottomLeftCopy.BevelSize, MaxBevelSize);
	BottomRightCopy.BevelSize = FMath::Min(BottomRightCopy.BevelSize, MaxBevelSize);

	float LeftSlantCopy;
	float RightSlantCopy;
	GetValidSlantAngle(LeftSlantCopy, RightSlantCopy);

	FAvaShapeCachedVertex2D TopLeftCorner = FVector2D(LeftSlantCopy < 0.f ? 0.f : Size2D.Y * FMath::Tan(FMath::DegreesToRadians(LeftSlantCopy)), Size2D.Y);
	FAvaShapeCachedVertex2D TopRightCorner = FVector2D(RightSlantCopy >= 0.f ? Size2D.X : Size2D.X + Size2D.Y * FMath::Tan(FMath::DegreesToRadians(RightSlantCopy)), Size2D.Y);
	FAvaShapeCachedVertex2D BottomLeftCorner = FVector2D(LeftSlantCopy >= 0.f ? 0.f : -Size2D.Y * FMath::Tan(FMath::DegreesToRadians(LeftSlantCopy)), 0.f);
	FAvaShapeCachedVertex2D BottomRightCorner = FVector2D(RightSlantCopy < 0.f ? Size2D.X : Size2D.X - Size2D.Y * FMath::Tan(FMath::DegreesToRadians(RightSlantCopy)), 0.f);

	const bool bTopLeftCornerRounded = TopLeftCopy.IsBeveled();
	const bool bTopRightCornerRounded = TopRightCopy.IsBeveled();
	const bool bBottomLeftCornerRounded = BottomLeftCopy.IsBeveled();
	const bool bBottomRightCornerRounded = BottomRightCopy.IsBeveled();

	// We're a simple quadrilateral !
	if (!bTopLeftCornerRounded && !bBottomLeftCornerRounded && !bTopRightCornerRounded
		&& !bBottomRightCornerRounded)
	{
		TopLeftCorner.Location += AnchorOffset;
		TopRightCorner.Location += AnchorOffset;
		BottomLeftCorner.Location += AnchorOffset;
		BottomRightCorner.Location += AnchorOffset;

		AddTriangle(PrimaryMesh, TopLeftCorner, BottomLeftCorner, TopRightCorner);
		AddTriangle(PrimaryMesh, BottomLeftCorner, BottomRightCorner, TopRightCorner);

		TopLeftCopy.CornerPositionCache = AnchorOffset + FVector2D(-Size2D.X / 2.f, Size2D.Y / 2.f);
		TopRightCopy.CornerPositionCache = AnchorOffset + FVector2D(Size2D.X / 2.f, Size2D.Y / 2.f);
		BottomLeftCopy.CornerPositionCache = AnchorOffset + FVector2D(-Size2D.X / 2.f, -Size2D.Y / 2.f);
		BottomRightCopy.CornerPositionCache = AnchorOffset + FVector2D(Size2D.X / 2.f, -Size2D.Y / 2.f);

		// Add snap points
		LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::TopLeft,
			TopLeftCorner.Location - Size2D / 2.f, static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::Point)));
		LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::TopRight,
			TopRightCorner.Location - Size2D / 2.f, static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::Point)));
		LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::BottomLeft,
			BottomLeftCorner.Location - Size2D / 2.f, static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::Point)));
		LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::BottomRight,
			BottomRightCorner.Location - Size2D / 2.f, static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::Point)));

		return true;
	}

	const bool bTopLeftCornerCurveOut = (bTopLeftCornerRounded && TopLeftCopy.Type == EAvaShapeCornerType::CurveOut);
	const bool bTopRightCornerCurveOut = (bTopRightCornerRounded && TopRightCopy.Type == EAvaShapeCornerType::CurveOut);
	const bool bBottomLeftCornerCurveOut = (bBottomLeftCornerRounded && BottomLeftCopy.Type == EAvaShapeCornerType::CurveOut);
	const bool bBottomRightCornerCurveOut = (bBottomRightCornerRounded && BottomRightCopy.Type == EAvaShapeCornerType::CurveOut);

	if (bTopLeftCornerCurveOut)
	{
		TopLeftCorner.Location.X += TopLeftCopy.BevelSize;

		if (!bBottomLeftCornerCurveOut)
		{
			BottomLeftCorner.Location.X += TopLeftCopy.BevelSize;
		}
	}

	if (bTopRightCornerCurveOut)
	{
		TopRightCorner.Location.X -= TopRightCopy.BevelSize;

		if (!bBottomRightCornerCurveOut)
		{
			BottomRightCorner.Location.X -= TopRightCopy.BevelSize;
		}
	}

	if (bBottomLeftCornerCurveOut)
	{
		BottomLeftCorner.Location.X += BottomLeftCopy.BevelSize;

		if (!bTopLeftCornerCurveOut)
		{
			TopLeftCorner.Location.X += BottomLeftCopy.BevelSize;
		}
	}

	if (bBottomRightCornerCurveOut)
	{
		BottomRightCorner.Location.X -= BottomRightCopy.BevelSize;

		if (!bTopRightCornerCurveOut)
		{
			TopRightCorner.Location.X -= BottomRightCopy.BevelSize;
		}
	}

	// Make sure we don't go over half way and get disturbed by the other side's shapes
	TopLeftCorner.Location.X = FMath::Min(
		TopLeftCorner.Location.X,
		Size2D.X / 2.f - CornerMinMargin - (bTopLeftCornerCurveOut ? 0 : TopLeftCopy.BevelSize)
	);

	// Update bevel size to the maximum computed value
	TopLeftCopy.BevelSize = FMath::Min3<float>(
		TopLeftCopy.BevelSize,
		Size2D.X / 2.f - CornerMinMargin - TopLeftCorner.Location.X,
		Size2D.Y / 2.f - CornerMinMargin
	);

	TopRightCorner.Location.X = FMath::Max(
		TopRightCorner.Location.X,
		Size2D.X / 2.f + CornerMinMargin + (bTopRightCornerCurveOut ? 0 : TopRightCopy.BevelSize)
	);

	// Update bevel size to the maximum computed value
	TopRightCopy.BevelSize = FMath::Min3<float>(
		TopRightCopy.BevelSize,
		TopRightCorner.Location.X - Size2D.X / 2.f - CornerMinMargin,
		Size2D.Y / 2.f - CornerMinMargin
	);

	BottomLeftCorner.Location.X = FMath::Min(
		BottomLeftCorner.Location.X,
		Size2D.X / 2.f - CornerMinMargin - (bBottomLeftCornerCurveOut ? 0 : BottomLeftCopy.BevelSize)
	);

	// Update bevel size to the maximum computed value
	BottomLeftCopy.BevelSize = FMath::Min3<float>(
		BottomLeftCopy.BevelSize,
		Size2D.X / 2.f - CornerMinMargin - BottomLeftCorner.Location.X,
		Size2D.Y / 2.f - CornerMinMargin
	);

	BottomRightCorner.Location.X = FMath::Max(
		BottomRightCorner.Location.X,
		Size2D.X / 2.f + CornerMinMargin + (bBottomRightCornerCurveOut ? 0 : BottomRightCopy.BevelSize)
	);

	// Update bevel size to the maximum computed value
	BottomRightCopy.BevelSize = FMath::Min3<float>(
		BottomRightCopy.BevelSize,
		BottomRightCorner.Location.X - Size2D.X / 2.f - CornerMinMargin,
		Size2D.Y / 2.f - CornerMinMargin
	);

	float LeftMin = FMath::Min(
		TopLeftCorner.Location.X - (bTopLeftCornerCurveOut ? TopLeftCopy.BevelSize : 0.f),
		BottomLeftCorner.Location.X - (bBottomLeftCornerCurveOut ? BottomLeftCopy.BevelSize : 0.f)
	);

	if (LeftMin > 0)
	{
		TopLeftCorner.Location.X -= LeftMin;
		BottomLeftCorner.Location.X -= LeftMin;
	}

	float RightMax = FMath::Max(
		TopRightCorner.Location.X + (bTopRightCornerCurveOut ? TopRightCopy.BevelSize : 0.f),
		BottomRightCorner.Location.X + (bBottomRightCornerCurveOut ? BottomRightCopy.BevelSize : 0.f)
	);

	if (RightMax < Size2D.X)
	{
		float RightMod = Size2D.X - RightMax;
		TopRightCorner.Location.X += RightMod;
		BottomRightCorner.Location.X += RightMod;
	}

	TopLeftCorner.Location += AnchorOffset;
	TopRightCorner.Location += AnchorOffset;
	BottomLeftCorner.Location += AnchorOffset;
	BottomRightCorner.Location += AnchorOffset;

	TopLeftCopy.CornerPositionCache = TopLeftCorner.Location;
	TopRightCopy.CornerPositionCache = TopRightCorner.Location;
	BottomLeftCopy.CornerPositionCache = BottomLeftCorner.Location;
	BottomRightCopy.CornerPositionCache = BottomRightCorner.Location;

	const FAvaShapeCachedVertex2D TopLeftCornerRoundedInsetTop = TopLeftCorner
		+ (TopLeftCopy.Type == EAvaShapeCornerType::CurveIn
			? TopLeftCopy.BevelSize
			: MaxBevelSize * 0.01f) * PlusX;

	const FAvaShapeCachedVertex2D TopLeftCornerRoundedInsetSide = TopLeftCorner
		+ (BottomLeftCorner - TopLeftCorner).GetSafeNormal() * TopLeftCopy.BevelSize;

	const FAvaShapeCachedVertex2D TopRightCornerRoundedInsetTop = TopRightCorner
		- (TopRightCopy.Type == EAvaShapeCornerType::CurveIn
			? TopRightCopy.BevelSize
			: MaxBevelSize * 0.01f) * PlusX;

	const FAvaShapeCachedVertex2D TopRightCornerRoundedInsetSide = TopRightCorner
		+ (BottomRightCorner - TopRightCorner).GetSafeNormal() * TopRightCopy.BevelSize;

	const FAvaShapeCachedVertex2D BottomLeftCornerRoundedInsetBottom = BottomLeftCorner
		+ (BottomLeftCopy.Type == EAvaShapeCornerType::CurveIn
			? BottomLeftCopy.BevelSize
			: MaxBevelSize * 0.01f) * PlusX;

	const FAvaShapeCachedVertex2D BottomLeftCornerRoundedInsetSide = BottomLeftCorner
		+ (TopLeftCorner - BottomLeftCorner).GetSafeNormal() * BottomLeftCopy.BevelSize;

	const FAvaShapeCachedVertex2D BottomRightCornerRoundedInsetBottom = BottomRightCorner
		- (BottomRightCopy.Type == EAvaShapeCornerType::CurveIn
			? BottomRightCopy.BevelSize
			: MaxBevelSize * 0.01f) * PlusX;

	const FAvaShapeCachedVertex2D BottomRightCornerRoundedInsetSide = BottomRightCorner
		+ (TopRightCorner - BottomRightCorner).GetSafeNormal() * BottomRightCopy.BevelSize;

	const FAvaShapeCachedVertex2D Center = Size2D / 2.f + AnchorOffset;

	// Calc side verts and local snap positions
	TArray<FAvaShapeCachedVertex2D> LeftVerts;
	TArray<FAvaShapeCachedVertex2D> RightVerts;

	// Left side
	if (!bTopLeftCornerRounded)
	{
		LeftVerts.Add(TopLeftCorner);

		LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::TopLeft,
			TopLeftCorner.Location - Size2D / 2.f, static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::Point)));
	}
	else
	{
		LeftVerts.Add(TopLeftCornerRoundedInsetTop);
		LeftVerts.Add(TopLeftCornerRoundedInsetSide);

		LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::TopLeft,
			TopLeftCornerRoundedInsetTop.Location - Size2D / 2.f, static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::InsetVertical)));
		LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::TopLeft,
			TopLeftCornerRoundedInsetSide.Location - Size2D / 2.f, static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::InsetHorizontal)));

		if (bTopLeftCornerCurveOut)
		{
			LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::TopLeft,
				TopLeftCorner.Location - Size2D / 2.f - FVector2D(TopLeftCopy.BevelSize, 0.f), static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::Point)));
		}
	}

	if (!bBottomLeftCornerRounded)
	{
		LeftVerts.Add(BottomLeftCorner);

		LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::BottomLeft,
			BottomLeftCorner.Location - Size2D / 2.f, static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::Point)));
	}
	else
	{
		LeftVerts.Add(BottomLeftCornerRoundedInsetSide);
		LeftVerts.Add(BottomLeftCornerRoundedInsetBottom);

		LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::BottomLeft,
			BottomLeftCornerRoundedInsetSide.Location - Size2D / 2.f, static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::InsetHorizontal)));
		LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::BottomLeft,
			BottomLeftCornerRoundedInsetBottom.Location - Size2D / 2.f, static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::InsetVertical)));

		if (bTopLeftCornerCurveOut)
		{
			LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::BottomLeft,
				BottomLeftCorner.Location - Size2D / 2.f - FVector2D(TopLeftCopy.BevelSize, 0.f), static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::Point)));
		}
	}

	// Right side
	if (!bTopRightCornerRounded)
	{
		RightVerts.Add(TopRightCorner);

		LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::TopRight,
			TopRightCorner.Location - Size2D / 2.f, static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::Point)));
	}
	else
	{
		RightVerts.Add(TopRightCornerRoundedInsetTop);
		RightVerts.Add(TopRightCornerRoundedInsetSide);

		LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::TopRight,
			TopRightCornerRoundedInsetTop.Location - Size2D / 2.f, static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::InsetVertical)));
		LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::TopRight,
			TopRightCornerRoundedInsetSide.Location - Size2D / 2.f, static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::InsetHorizontal)));

		if (bTopRightCornerCurveOut)
		{
			LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::TopRight,
				TopRightCorner.Location - Size2D / 2.f + FVector2D(TopLeftCopy.BevelSize, 0.f), static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::Point)));
		}
	}

	if (!bBottomRightCornerRounded)
	{
		RightVerts.Add(BottomRightCorner);

		LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::BottomRight,
			BottomRightCorner.Location - Size2D / 2.f, static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::Point)));
	}
	else
	{
		RightVerts.Add(BottomRightCornerRoundedInsetSide);
		RightVerts.Add(BottomRightCornerRoundedInsetBottom);

		LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::BottomRight,
			BottomRightCornerRoundedInsetSide.Location - Size2D / 2.f, static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::InsetHorizontal)));
		LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::BottomRight,
			BottomRightCornerRoundedInsetBottom.Location - Size2D / 2.f, static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::InsetVertical)));

		if (bBottomRightCornerCurveOut)
		{
			LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorIndexedSnapPoint(EAvaAnchors::BottomRight,
				BottomRightCorner.Location - Size2D / 2.f + FVector2D(TopLeftCopy.BevelSize, 0.f), static_cast<int32>(EAvaShapeRectangleCornerTypeIndex::Point)));
		}
	}

	AddVertex(PrimaryMesh, Center);
	AddVertex(PrimaryMesh, RightVerts[0]);
	AddVertex(PrimaryMesh, LeftVerts[0]);

	for (int32 LeftIdx = 1; LeftIdx < LeftVerts.Num(); ++LeftIdx)
	{
		AddVertex(PrimaryMesh, Center);
		AddVertex(PrimaryMesh, LeftVerts[LeftIdx - 1]);
		AddVertex(PrimaryMesh, LeftVerts[LeftIdx]);
	}

	AddVertex(PrimaryMesh, Center);
	AddVertex(PrimaryMesh, LeftVerts.Last());
	AddVertex(PrimaryMesh, RightVerts.Last());

	for (int32 RightIdx = 1; RightIdx < RightVerts.Num(); ++RightIdx)
	{
		AddVertex(PrimaryMesh, Center);
		AddVertex(PrimaryMesh, RightVerts[RightIdx]);
		AddVertex(PrimaryMesh, RightVerts[RightIdx - 1]);
	}

	// Rounded corners
	if (bTopLeftCornerRounded)
	{
		uint8 Subdivisions = TopLeftCopy.BevelSubdivisions;

		if (TopLeftCopy.Type == EAvaShapeCornerType::CurveIn)
		{
			FAvaShapeRoundedCornerMetrics CornerInfo = UAvaShapeRoundedPolygonDynamicMesh::CreateCornerInfo(
				TopLeftCornerRoundedInsetTop - TopRightCornerRoundedInsetTop,
				BottomLeftCornerRoundedInsetSide - TopLeftCornerRoundedInsetSide,
				TopLeftCorner,
				TopLeftCopy.BevelSize,
				Subdivisions,
				true
			);

			CornerInfo.Start = TopLeftCornerRoundedInsetTop;
			CornerInfo.End = TopLeftCornerRoundedInsetSide;

			UAvaShapeRoundedPolygonDynamicMesh::CreateRoundedCorner(this, PrimaryMesh, TopLeftCorner,
				CornerInfo, true);
		}
		else if (TopLeftCopy.Type == EAvaShapeCornerType::CurveOut)
		{
			FAvaShapeRoundedCornerMetrics CornerInfo = UAvaShapeRoundedPolygonDynamicMesh::CreateCornerInfo(
				TopRightCornerRoundedInsetTop - TopLeftCornerRoundedInsetTop,
				BottomLeftCornerRoundedInsetSide - TopLeftCornerRoundedInsetSide,
				TopLeftCorner,
				TopLeftCopy.BevelSize,
				Subdivisions,
				false
			);

			CornerInfo.End = TopLeftCornerRoundedInsetSide;
			CornerInfo.VertexAnchor = TopLeftCornerRoundedInsetTop;

			UAvaShapeRoundedPolygonDynamicMesh::CreateRoundedCorner(this, PrimaryMesh, TopLeftCorner, CornerInfo);
		}
	}

	if (bTopRightCornerRounded)
	{
		uint8 Subdivisions = TopRightCopy.BevelSubdivisions;

		if (TopRightCopy.Type == EAvaShapeCornerType::CurveIn)
		{
			FAvaShapeRoundedCornerMetrics CornerInfo = UAvaShapeRoundedPolygonDynamicMesh::CreateCornerInfo(
				TopRightCornerRoundedInsetTop - TopLeftCornerRoundedInsetTop,
				BottomRightCornerRoundedInsetSide - TopRightCornerRoundedInsetSide,
				TopRightCorner,
				TopRightCopy.BevelSize,
				Subdivisions,
				true
			);

			CornerInfo.Start = TopRightCornerRoundedInsetTop;
			CornerInfo.End = TopRightCornerRoundedInsetSide;

			UAvaShapeRoundedPolygonDynamicMesh::CreateRoundedCorner(this, PrimaryMesh, TopRightCorner,
				CornerInfo, true);
		}
		else if (TopRightCopy.Type == EAvaShapeCornerType::CurveOut)
		{
			FAvaShapeRoundedCornerMetrics CornerInfo = UAvaShapeRoundedPolygonDynamicMesh::CreateCornerInfo(
				TopLeftCornerRoundedInsetTop - TopRightCornerRoundedInsetTop,
				BottomRightCornerRoundedInsetSide - TopRightCornerRoundedInsetSide,
				TopRightCorner,
				TopRightCopy.BevelSize,
				Subdivisions,
				false
			);

			CornerInfo.End = TopRightCornerRoundedInsetSide;
			CornerInfo.VertexAnchor = TopRightCornerRoundedInsetTop;

			UAvaShapeRoundedPolygonDynamicMesh::CreateRoundedCorner(this, PrimaryMesh, TopRightCorner, CornerInfo);
		}
	}

	if (bBottomLeftCornerRounded)
	{
		uint8 Subdivisions = BottomLeftCopy.BevelSubdivisions;

		if (BottomLeftCopy.Type == EAvaShapeCornerType::CurveIn)
		{
			FAvaShapeRoundedCornerMetrics CornerInfo = UAvaShapeRoundedPolygonDynamicMesh::CreateCornerInfo(
				BottomLeftCornerRoundedInsetBottom - BottomRightCornerRoundedInsetBottom,
				TopLeftCornerRoundedInsetSide - BottomLeftCornerRoundedInsetSide,
				BottomLeftCorner,
				BottomLeftCopy.BevelSize,
				Subdivisions,
				true
			);

			CornerInfo.Start = BottomLeftCornerRoundedInsetBottom;
			CornerInfo.End = BottomLeftCornerRoundedInsetSide;

			UAvaShapeRoundedPolygonDynamicMesh::CreateRoundedCorner(this, PrimaryMesh, BottomLeftCorner,
				CornerInfo, true);
		}
		else if (BottomLeftCopy.Type == EAvaShapeCornerType::CurveOut)
		{
			FAvaShapeRoundedCornerMetrics CornerInfo = UAvaShapeRoundedPolygonDynamicMesh::CreateCornerInfo(
				BottomRightCornerRoundedInsetBottom - BottomLeftCornerRoundedInsetBottom,
				TopLeftCornerRoundedInsetSide - BottomLeftCornerRoundedInsetSide,
				BottomLeftCorner,
				BottomLeftCopy.BevelSize,
				Subdivisions,
				false
			);

			CornerInfo.End = BottomLeftCornerRoundedInsetSide;
			CornerInfo.VertexAnchor = BottomLeftCornerRoundedInsetBottom;

			UAvaShapeRoundedPolygonDynamicMesh::CreateRoundedCorner(this, PrimaryMesh, BottomLeftCorner, CornerInfo);
		}
	}

	if (bBottomRightCornerRounded)
	{
		uint8 Subdivisions = BottomRightCopy.BevelSubdivisions;

		if (BottomRightCopy.Type == EAvaShapeCornerType::CurveIn)
		{
			FAvaShapeRoundedCornerMetrics CornerInfo = UAvaShapeRoundedPolygonDynamicMesh::CreateCornerInfo(
				BottomRightCornerRoundedInsetBottom - BottomLeftCornerRoundedInsetBottom,
				TopRightCornerRoundedInsetSide - BottomRightCornerRoundedInsetSide,
				BottomRightCorner,
				BottomRightCopy.BevelSize,
				Subdivisions,
				true
			);

			CornerInfo.Start = BottomRightCornerRoundedInsetBottom;
			CornerInfo.End = BottomRightCornerRoundedInsetSide;

			UAvaShapeRoundedPolygonDynamicMesh::CreateRoundedCorner(this, PrimaryMesh, BottomRightCorner,
				CornerInfo, true);
		}
		else if (BottomRightCopy.Type == EAvaShapeCornerType::CurveOut)
		{
			FAvaShapeRoundedCornerMetrics CornerInfo = UAvaShapeRoundedPolygonDynamicMesh::CreateCornerInfo(
				BottomLeftCornerRoundedInsetBottom - BottomRightCornerRoundedInsetBottom,
				TopRightCornerRoundedInsetSide - BottomRightCornerRoundedInsetSide,
				BottomRightCorner,
				BottomRightCopy.BevelSize,
				Subdivisions,
				false
			);

			CornerInfo.End = BottomRightCornerRoundedInsetSide;
			CornerInfo.VertexAnchor = BottomRightCornerRoundedInsetBottom;

			UAvaShapeRoundedPolygonDynamicMesh::CreateRoundedCorner(this, PrimaryMesh, BottomRightCorner, CornerInfo);
		}
	}

	return true;
}

#if WITH_EDITOR
void UAvaShapeRectangleDynamicMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	static const FName HorizontalAlignmentName = GET_MEMBER_NAME_CHECKED(UAvaShapeRectangleDynamicMesh, HorizontalAlignment);
	static const FName VerticalAlignmentName = GET_MEMBER_NAME_CHECKED(UAvaShapeRectangleDynamicMesh, VerticalAlignment);
	static const FName LeftSlantName = GET_MEMBER_NAME_CHECKED(UAvaShapeRectangleDynamicMesh, LeftSlant);
	static const FName RightSlantName = GET_MEMBER_NAME_CHECKED(UAvaShapeRectangleDynamicMesh, RightSlant);
	static const FName GlobalBevelSizeName = GET_MEMBER_NAME_CHECKED(UAvaShapeRectangleDynamicMesh, GlobalBevelSize);
	static const FName GlobalBevelSubdivisionsName = GET_MEMBER_NAME_CHECKED(UAvaShapeRectangleDynamicMesh, GlobalBevelSubdivisions);
	static const FName TopLeftName = GET_MEMBER_NAME_CHECKED(UAvaShapeRectangleDynamicMesh, TopLeft);
	static const FName BottomLeftName = GET_MEMBER_NAME_CHECKED(UAvaShapeRectangleDynamicMesh, BottomLeft);
	static const FName TopRightName = GET_MEMBER_NAME_CHECKED(UAvaShapeRectangleDynamicMesh, TopRight);
	static const FName BottomRightName = GET_MEMBER_NAME_CHECKED(UAvaShapeRectangleDynamicMesh, BottomRight);
	static const FName CornerTypeName = GET_MEMBER_NAME_CHECKED(FAvaShapeRectangleCornerSettings, Type);
	static const FName BevelSizeName = GET_MEMBER_NAME_CHECKED(FAvaShapeRectangleCornerSettings, BevelSize);
	static const FName BevelSubdivisionsName = GET_MEMBER_NAME_CHECKED(FAvaShapeRectangleCornerSettings, BevelSubdivisions);

	if (MemberName == HorizontalAlignmentName
		|| MemberName == VerticalAlignmentName)
	{
		OnAlignmentChanged();
	}
	else if (MemberName == LeftSlantName)
	{
		OnLeftSlantChanged();
	}
	else if (MemberName == RightSlantName)
	{
		OnRightSlantChanged();
	}
	else if (MemberName == GlobalBevelSizeName)
	{
		OnGlobalBevelSizeChanged();
	}
	else if (MemberName == GlobalBevelSubdivisionsName)
	{
		OnGlobalBevelSubdivisionsChanged();
	}
	else if (MemberName == TopLeftName &&
		PropertyName == CornerTypeName)
	{
		OnTopLeftCornerTypeChanged();
	}
	else if (MemberName == TopLeftName &&
		PropertyName == BevelSizeName)
	{
		OnTopLeftBevelSizeChanged();
	}
	else if (MemberName == TopLeftName &&
		PropertyName == BevelSubdivisionsName)
	{
		OnTopLeftBevelSubdivisionsChanged();
	}
	else if (MemberName == TopRightName &&
		PropertyName == CornerTypeName)
	{
		OnTopRightCornerTypeChanged();
	}
	else if (MemberName == TopRightName &&
		PropertyName == BevelSizeName)
	{
		OnTopRightBevelSizeChanged();
	}
	else if (MemberName == TopRightName &&
		PropertyName == BevelSubdivisionsName)
	{
		OnTopRightBevelSubdivisionsChanged();
	}
	else if (MemberName == BottomLeftName &&
		PropertyName == CornerTypeName)
	{
		OnBottomLeftCornerTypeChanged();
	}
	else if (MemberName == BottomLeftName &&
		PropertyName == BevelSizeName)
	{
		OnBottomLeftBevelSizeChanged();
	}
	else if (MemberName == BottomLeftName &&
		PropertyName == BevelSubdivisionsName)
	{
		OnBottomLeftBevelSubdivisionsChanged();
	}
	else if (MemberName == BottomRightName &&
		PropertyName == CornerTypeName)
	{
		OnBottomRightCornerTypeChanged();
	}
	else if (MemberName == BottomRightName &&
		PropertyName == BevelSubdivisionsName)
	{
		OnBottomRightBevelSubdivisionsChanged();
	}
	else if (MemberName == BottomRightName &&
		PropertyName == BevelSizeName)
	{
		OnBottomRightBevelSizeChanged();
	}
}
#endif

void UAvaShapeRectangleDynamicMesh::OnSizeChanged()
{
	Super::OnSizeChanged();

	//Only Mark Meshes Dirty any of the corners has bevels
	if (TopLeft.IsBeveled()
		|| TopRight.IsBeveled()
		|| BottomLeft.IsBeveled()
		|| BottomRight.IsBeveled()
		|| LeftSlant != 0.f
		|| RightSlant != 0.f)
	{
		MarkAllMeshesDirty();
	}
}

bool UAvaShapeRectangleDynamicMesh::CanUpdateMeshInAnyThread() const
{
	return GetClass() == UAvaShapeRectangleDynamicMesh::StaticClass();
}

bool UAvaShapeRectangleDynamicMesh::CreateMesh(FAvaShapeMesh& InMesh)
{
	if (InMesh.GetMeshIndex() == MESH_INDEX_PRIMARY)
	{
		GenerateBaseMeshSections(InMesh);
	}

	return Super::CreateMesh(InMesh);
}

