// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshes/AvaShapeRingDynMesh.h"

const FString UAvaShapeRingDynamicMesh::MeshName = TEXT("Ring");

void UAvaShapeRingDynamicMesh::SetNumSides(uint8 InNumSides)
{
	InNumSides = FMath::Clamp(InNumSides, UAvaShapeRingDynamicMesh::MinNumSides, UAvaShapeRingDynamicMesh::MaxNumSides);
	if (NumSides == InNumSides)
	{
		return;
	}

	NumSides = InNumSides;
	OnNumSidesChanged();
}

void UAvaShapeRingDynamicMesh::SetInnerSize(float InInnerSize)
{
	InInnerSize = FMath::Clamp(InInnerSize, 0.f, 1.f);
	if (FMath::IsNearlyEqual(InnerSize, InInnerSize))
	{
		return;
	}

	InnerSize = InInnerSize;
	OnInnerSizeChanged();
}

void UAvaShapeRingDynamicMesh::SetAngleDegree(float InDegree)
{
	InDegree = FMath::Clamp(InDegree, 0.f, 360.f);
	if (FMath::IsNearlyEqual(AngleDegree, InDegree))
	{
		return;
	}

	AngleDegree = InDegree;
	OnAngleDegreeChanged();
}

void UAvaShapeRingDynamicMesh::SetStartDegree(float InDegree)
{
	InDegree = FMath::Clamp(InDegree, 0.f, 360.f);
	if (FMath::IsNearlyEqual(StartDegree, InDegree))
	{
		return;
	}

	StartDegree = InDegree;
	OnStartDegreeChanged();
}

#if WITH_EDITOR
void UAvaShapeRingDynamicMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static FName NumSidesName = GET_MEMBER_NAME_CHECKED(UAvaShapeRingDynamicMesh, NumSides);
	static FName InnerSizeName = GET_MEMBER_NAME_CHECKED(UAvaShapeRingDynamicMesh, InnerSize);
	static FName StartDegreeName = GET_MEMBER_NAME_CHECKED(UAvaShapeRingDynamicMesh, StartDegree);
	static FName AngleDegreeName = GET_MEMBER_NAME_CHECKED(UAvaShapeRingDynamicMesh, AngleDegree);

	if (PropertyChangedEvent.MemberProperty->GetFName() == NumSidesName)
	{
		OnNumSidesChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == InnerSizeName)
	{
		OnInnerSizeChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == StartDegreeName)
	{
		OnStartDegreeChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == AngleDegreeName)
	{
		OnAngleDegreeChanged();
	}
}
#endif

void UAvaShapeRingDynamicMesh::OnNumSidesChanged()
{
	NumSides = FMath::Clamp(NumSides, UAvaShapeRingDynamicMesh::MinNumSides, UAvaShapeRingDynamicMesh::MaxNumSides);
	MarkAllMeshesDirty();
}

void UAvaShapeRingDynamicMesh::OnInnerSizeChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShapeRingDynamicMesh::OnAngleDegreeChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShapeRingDynamicMesh::OnStartDegreeChanged()
{
	MarkAllMeshesDirty();
}

bool UAvaShapeRingDynamicMesh::IsMeshVisible(int32 MeshIndex)
{
	switch (MeshIndex)
	{
		case MESH_INDEX_PRIMARY:
			return AngleDegree > UE_KINDA_SMALL_NUMBER && InnerSize < (1.f - UE_KINDA_SMALL_NUMBER);
		default:
		return true;
	}
}

bool UAvaShapeRingDynamicMesh::CreateMesh(FAvaShapeMesh& InMesh)
{
	if (InMesh.GetMeshIndex() == MESH_INDEX_PRIMARY)
	{
		const FVector2D OffsetBaseOuter = FVector2D(0.f, Size2D.Y / 2.f).GetRotated(StartDegree);
		const FVector2D OffsetBaseInner = FVector2D(0.f, Size2D.Y * InnerSize / 2.f).GetRotated(StartDegree);
		const FVector2D Scale(Size2D.X / Size2D.Y, 1.f);
		const float SideAngle = AngleDegree / NumSides;

		int32 PreviousOuterIndex = CacheVertex(InMesh, OffsetBaseOuter * Scale);
		int32 PreviousInnerIndex = CacheVertex(InMesh, OffsetBaseInner * Scale);

		for (uint8 SideIdx = 1; SideIdx <= NumSides; ++SideIdx)
		{
			const float CurrentAngle = SideAngle * SideIdx;

			FVector2D NextOuterVertex = OffsetBaseOuter.GetRotated(CurrentAngle) * Scale;
			int32 NextOuterIndex = CacheVertex(InMesh, NextOuterVertex);

			FVector2D NextInnerVertex = OffsetBaseInner.GetRotated(CurrentAngle) * Scale;
			int32 NextInnerIndex = CacheVertex(InMesh, NextInnerVertex);

			AddVertex(InMesh, PreviousInnerIndex);
			AddVertex(InMesh, PreviousOuterIndex);
			AddVertex(InMesh, NextOuterIndex);

			AddVertex(InMesh, PreviousInnerIndex);
			AddVertex(InMesh, NextOuterIndex);
			AddVertex(InMesh, NextInnerIndex);

			PreviousOuterIndex = NextOuterIndex;
			PreviousInnerIndex = NextInnerIndex;
		}
	}

	return Super::CreateMesh(InMesh);
}

bool UAvaShapeRingDynamicMesh::CanUpdateMeshInAnyThread() const
{
	return GetClass() == UAvaShapeRingDynamicMesh::StaticClass();
}
