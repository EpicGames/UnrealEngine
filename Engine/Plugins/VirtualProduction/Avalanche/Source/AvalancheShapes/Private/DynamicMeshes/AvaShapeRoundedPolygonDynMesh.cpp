// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshes/AvaShapeRoundedPolygonDynMesh.h"
#include "AvaShapesDefs.h"
#include "DynamicMeshes/AvaShapeRectangleDynMesh.h"
#include "Kismet/KismetMathLibrary.h"

void UAvaShapeRoundedPolygonDynamicMesh::SetBevelSize(float InBevelSize)
{
	if (BevelSize == InBevelSize)
	{
		return;
	}

	if (InBevelSize < 0.f || InBevelSize >= FMath::Min(Size2D.X, Size2D.Y))
	{
		return;
	}

	BevelSize = InBevelSize;
	OnRoundedRadiusChanged();
}

void UAvaShapeRoundedPolygonDynamicMesh::SetBevelSubdivisions(uint8 InBevelSubdivisions)
{
	if (BevelSubdivisions == InBevelSubdivisions)
	{
		return;
	}

	if (InBevelSubdivisions > UAvaShapeDynamicMeshBase::MaxSubdivisions)
	{
		return;
	}

	BevelSubdivisions = InBevelSubdivisions;
	OnBevelSubdivisionsChanged();
}

void UAvaShapeRoundedPolygonDynamicMesh::OnRoundedRadiusChanged()
{
	if (BevelSize > 0.f && BevelSubdivisions == 0)
	{
		BevelSubdivisions = UAvaShapeDynamicMeshBase::DefaultSubdivisions;
	}

	MarkAllMeshesDirty();
}

void UAvaShapeRoundedPolygonDynamicMesh::OnBevelSubdivisionsChanged()
{
	if (BevelSubdivisions > 0 && BevelSize == 0.f)
	{
		BevelSize = 10.f;
	}

	MarkAllMeshesDirty();
}

#if WITH_EDITOR
void UAvaShapeRoundedPolygonDynamicMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	static const FName BevelSizeName = GET_MEMBER_NAME_CHECKED(UAvaShapeRoundedPolygonDynamicMesh, BevelSize);
	static const FName BevelSubdivisionsName = GET_MEMBER_NAME_CHECKED(UAvaShapeRoundedPolygonDynamicMesh, BevelSubdivisions);

	if (MemberName == BevelSizeName)
	{
		OnRoundedRadiusChanged();
	}
	else if (MemberName == BevelSubdivisionsName)
	{
		OnBevelSubdivisionsChanged();
	}
}
#endif

void UAvaShapeRoundedPolygonDynamicMesh::OnSizeChanged()
{
	Super::OnSizeChanged();

	// Only Mark Meshes Dirty if we have bevels on it
	if (BevelSize > 0.f && BevelSubdivisions > 0)
	{
		MarkAllMeshesDirty();
	}
}

bool UAvaShapeRoundedPolygonDynamicMesh::CreateMesh(FAvaShapeMesh& InMesh)
{
	if (InMesh.GetMeshIndex() == MESH_INDEX_PRIMARY)
	{
		TArray<FVector2D> Vertices;
		GenerateBorderVertices(Vertices);
		int32 VertexCount = Vertices.Num();

		auto CycleIndex = [&](int32 VertexIdx)->int32
		{
			while (VertexIdx < 0)
			{
				VertexIdx += VertexCount;
			}

			if (VertexIdx >= VertexCount)
			{
				VertexIdx %= VertexCount;
			}

			return VertexIdx;
		};

		if (VertexCount == 0)
		{
			return false;
		}

		FVector2D VMin = Vertices[0];
		FVector2D VMax = VMin;
		FAvaShapeCachedVertex2D Center = FVector2D::ZeroVector;

		for (FVector2D& Vertex : Vertices)
		{
			VMin.X = FMath::Min(VMin.X, Vertex.X);
			VMax.X = FMath::Max(VMax.X, Vertex.X);
			VMin.Y = FMath::Min(VMin.Y, Vertex.Y);
			VMax.Y = FMath::Max(VMax.Y, Vertex.Y);
			Center.Location += Vertex / Vertices.Num();
		}

		FVector2D VRange = VMax - VMin;
		FVector2D VScale = Size2D / VRange;
		double MinScale = FMath::Min(VScale.X, VScale.Y);

		auto MapDouble = [MinScale](double& InOutValue, double InMin, double InMax, double InSize, double InRange)
			{
				const double HalfSize  = 0.5 * InSize;
				const double HalfRange = 0.5 * InRange * MinScale;

				InOutValue = FMath::GetMappedRangeValueUnclamped(FVector2D(InMin, InMax)
					, FVector2D(HalfSize - HalfRange, HalfSize + HalfRange)
					, InOutValue);
			};

		auto MapAxis = [this, &VMin, &VMax, &VRange, &MapDouble](FVector2D& InOutVector)
			{
				MapDouble(InOutVector.X, VMin.X, VMax.X, Size2D.X, VRange.X);
				MapDouble(InOutVector.Y, VMin.Y, VMax.Y, Size2D.Y, VRange.Y);
			};

		// Recenter vertices and add snap points
		for (int32 VertexIdx = 0; VertexIdx < Vertices.Num(); ++VertexIdx)
		{
			MapAxis(Vertices[VertexIdx]);
			LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorCustomSnapPoint(Vertices[VertexIdx] - Size2D / 2.0, VertexIdx));
		}

		MapAxis(Center.Location);

		if (BevelSize == 0 || BevelSubdivisions == 0)
		{
			TArray<FAvaShapeCachedVertex2D> CachedVertices;

			for (int32 VertexIdx = 0; VertexIdx < Vertices.Num(); ++VertexIdx)
				CachedVertices.Add(Vertices[VertexIdx]);

			// Generate regular polygon
			if (UseCenteredVertex())
			{
				for (int32 VertexIdx = 0; VertexIdx < VertexCount; ++VertexIdx)
				{
					AddVertex(InMesh, Center);
					AddVertexRaw(InMesh, CachedVertices[CycleIndex(VertexIdx + 1)]);
					AddVertexRaw(InMesh, CachedVertices[VertexIdx]);
				}
			}

			else
			{
				for (int32 VertexIdx = 0; VertexIdx < (VertexCount - 2); ++VertexIdx)
				{
					AddVertex(InMesh, CachedVertices[0]);
					AddVertexRaw(InMesh, CachedVertices[CycleIndex(VertexIdx + 2)]);
					AddVertexRaw(InMesh, CachedVertices[CycleIndex(VertexIdx + 1)]);
				}
			}

			return true;
		}

		// Generate rounded polygon
		TArray<FAvaShapeRoundedCornerMetrics> CornerInfos;

		for (int32 VertexIdx = 0; VertexIdx < Vertices.Num(); ++VertexIdx)
		{
			FVector2D LineIn = Vertices[VertexIdx] - Vertices[CycleIndex(VertexIdx - 1)];
			FVector2D LineOut = Vertices[CycleIndex(VertexIdx + 1)] - Vertices[VertexIdx];
			float MaxRadius = FMath::Min(BevelSize, FMath::Min(LineIn.Size() / 2.f, LineOut.Size() / 2.f));
			FVector2D Start = Vertices[VertexIdx] - (LineIn.GetSafeNormal() * MaxRadius);
			FVector2D End = Vertices[VertexIdx] + (LineOut.GetSafeNormal() * MaxRadius);
			FVector2D MidPoint = (Start + End) / 2.f;
			bool bConvex = (Center - MidPoint).SizeSquared() <= (Center - Vertices[VertexIdx]).SizeSquared();

			FAvaShapeRoundedCornerMetrics NewCornerInfo = CreateCornerInfo(LineIn, LineOut,
				Vertices[VertexIdx], MaxRadius, BevelSubdivisions, bConvex);

			CornerInfos.Add(NewCornerInfo);

			LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorCustomSnapPoint(Start - Size2D / 2.f, VertexIdx + Vertices.Num()));
			LocalSnapPoints.Add(FAvaSnapPoint::CreateLocalActorCustomSnapPoint(End - Size2D / 2.f, VertexIdx + Vertices.Num() * 2));
		}

		// Add inner triangles
		if (UseCenteredVertex())
		{
			for (int32 VertexIdx = 0; VertexIdx < Vertices.Num(); ++VertexIdx)
			{
				FAvaShapeCachedVertex2D ThisStart = (CornerInfos[VertexIdx].Angle > 0
					? CornerInfos[VertexIdx].Start
					: CornerInfos[VertexIdx].VertexAnchor
					);

				FAvaShapeCachedVertex2D ThisEnd = (CornerInfos[VertexIdx].Angle > 0
					? CornerInfos[VertexIdx].End
					: CornerInfos[VertexIdx].VertexAnchor
					);

				FAvaShapeCachedVertex2D NextStart = (CornerInfos[CycleIndex(VertexIdx + 1)].Angle > 0
					? CornerInfos[CycleIndex(VertexIdx + 1)].Start
					: CornerInfos[CycleIndex(VertexIdx + 1)].VertexAnchor
					);

				if (CornerInfos[VertexIdx].Angle > 0.f)
				{
					AddVertex(InMesh, Center);
					AddVertex(InMesh, CornerInfos[VertexIdx].VertexAnchor);
					AddVertex(InMesh, ThisStart);

					AddVertex(InMesh, Center);
					AddVertex(InMesh, ThisEnd);
					AddVertex(InMesh, CornerInfos[VertexIdx].VertexAnchor);

					AddVertex(InMesh, Center);
					AddVertex(InMesh, NextStart);
					AddVertex(InMesh, ThisEnd);
				}

				else if (NextStart != ThisEnd)
				{
					AddVertex(InMesh, Center);
					AddVertex(InMesh, NextStart);
					AddVertex(InMesh, ThisEnd);
				}
			}
		}

		else
		{
			FAvaShapeCachedVertex2D Start = (CornerInfos[0].Angle > 0 ? CornerInfos[0].Start : CornerInfos[0].VertexAnchor);

			for (int32 VertexIdx = 0; VertexIdx < Vertices.Num(); ++VertexIdx)
			{
				// 2 unrounded corners
				if (CornerInfos[VertexIdx].Angle == 0.f && CornerInfos[CycleIndex(VertexIdx + 1)].Angle == 0.f)
				{
					AddVertex(InMesh, CornerInfos[0].VertexAnchor);
					AddVertex(InMesh, CornerInfos[CycleIndex(VertexIdx + 2)].VertexAnchor);
					AddVertex(InMesh, CornerInfos[CycleIndex(VertexIdx + 1)].VertexAnchor);

					if (VertexIdx == (Vertices.Num() - 2))
					{
						break;
					}

					continue;
				}

				// This corner is rounded
				if (CornerInfos[VertexIdx].Angle > 0.f)
				{
					FAvaShapeCachedVertex2D ThisStart = (CornerInfos[VertexIdx].Angle > 0
						? CornerInfos[VertexIdx].Start
						: CornerInfos[VertexIdx].VertexAnchor
						);

					FAvaShapeCachedVertex2D ThisEnd = (CornerInfos[VertexIdx].Angle > 0
						? CornerInfos[VertexIdx].End
						: CornerInfos[VertexIdx].VertexAnchor
						);

					FAvaShapeCachedVertex2D NextStart = (CornerInfos[CycleIndex(VertexIdx + 1)].Angle > 0
						? CornerInfos[CycleIndex(VertexIdx + 1)].Start
						: CornerInfos[CycleIndex(VertexIdx + 1)].VertexAnchor
						);

					if (VertexIdx == 0)
					{
						AddVertex(InMesh, Start);
						AddVertex(InMesh, ThisEnd);
						AddVertex(InMesh, CornerInfos[VertexIdx].VertexAnchor);
					}

					else
					{
						AddVertex(InMesh, Start);
						AddVertex(InMesh, CornerInfos[VertexIdx].VertexAnchor);
						AddVertex(InMesh, ThisStart);

						// Corner radius causing the last 2 corners to meet.
						if (CornerInfos[0].Start != CornerInfos[VertexIdx].End)
						{
							AddVertex(InMesh, Start);
							AddVertex(InMesh, ThisEnd);
							AddVertex(InMesh, CornerInfos[VertexIdx].VertexAnchor);
						}
					}

					// Corner radius causing these 2 corners to meet.
					if (NextStart != ThisEnd && Start != NextStart)
					{
						AddVertex(InMesh, Start);
						AddVertex(InMesh, NextStart);
						AddVertex(InMesh, ThisEnd);
					}
				}
			}
		}

		// Add rounded corners
		for (int32 VertexIdx = 0; VertexIdx < Vertices.Num(); ++VertexIdx)
		{
			CreateRoundedCorner(this, InMesh, Vertices[VertexIdx],
				CornerInfos[VertexIdx], false);
		}
	}

	return Super::CreateMesh(InMesh);
}

FAvaShapeRoundedCornerMetrics UAvaShapeRoundedPolygonDynamicMesh::CreateCornerInfo(
	const FVector2D& LineIn, const FVector2D LineOut, const FVector2D& CornerVertex, float Radius, uint8 BevelSubdivisions,
	bool bConvex)
{
	// Perform no rounding
	if (BevelSubdivisions == 0 || Radius <= 0)
	{
		return {
			0.f,
			CornerVertex - LineIn.GetSafeNormal() * Radius,
			CornerVertex + LineOut.GetSafeNormal() * Radius,
			CornerVertex,
			bConvex,
			FVector2D::ZeroVector,
			BevelSubdivisions,
			true
		};
	}

	float Angle = FMath::RadiansToDegrees(FMath::Acos(
		FVector2D::DotProduct(LineIn, LineOut) / (LineIn.Size() * LineOut.Size())
	));

	FVector2D Start = CornerVertex - (LineIn.GetSafeNormal() * Radius);
	FVector2D End = CornerVertex + (LineOut.GetSafeNormal() * Radius);
	FVector2D MidPoint = (Start + End) / 2.f;

	FVector2D CenterOfRotationDirection = (MidPoint - CornerVertex).GetSafeNormal();
	float HalfMidpointDistance = (End - Start).Size() / 2.f;
	float DistanceToCenterOfRotationMultiplier = FMath::Tan(FMath::DegreesToRadians(Angle / 2.f));
	FVector2D CenterOfRotation = MidPoint +
		(CenterOfRotationDirection * HalfMidpointDistance / DistanceToCenterOfRotationMultiplier);

	// Concave corners can use the original corner as an anchor.
	FVector2D VertexAnchor;

	// There must be a small triangle inside the corner to avoid parallel vector triangles.
	// Scale the outer vertex it so it is half way between the midpoint and the rotation length.
	if (bConvex)
	{
		float MidpointCenterRotation = (MidPoint - CenterOfRotation).Size() ;
		float RotationLength = (Start - CenterOfRotation).Size();
		VertexAnchor = (MidPoint - CenterOfRotation).GetSafeNormal() * (MidpointCenterRotation + RotationLength) / 2.f;
		VertexAnchor += CenterOfRotation;
	}

	else
	{
		VertexAnchor = CornerVertex;
	}

	return {
		Angle,
		Start,
		End,
		VertexAnchor,
		bConvex,
		CenterOfRotation,
		BevelSubdivisions,
		true
	};
}

void UAvaShapeRoundedPolygonDynamicMesh::CreateRoundedCorner(
	UAvaShape2DDynMeshBase* InDynamicMesh, FAvaShapeMesh& InMesh,
	const FVector2D& CornerVertex, const FAvaShapeRoundedCornerMetrics& CornerInfo,
	bool bAddConcaveCornerInsert)
{
	const FVector StartUnit = UKismetMathLibrary::GetDirectionUnitVector(FVector(CornerInfo.VertexAnchor.Location, 0), FVector(CornerInfo.Start.Location, 0));
	const FVector EndUnit = UKismetMathLibrary::GetDirectionUnitVector(FVector(CornerInfo.VertexAnchor.Location, 0), FVector(CornerInfo.End.Location, 0));

	const FVector Normal = StartUnit.Cross(EndUnit);

	if (Normal.Z == 0)
	{
		return;
	}

	const bool bReverse = (Normal.Z < 0);

	if (CornerInfo.Angle <= 0)
	{
		if (!CornerInfo.bConvex)
		{
			InDynamicMesh->AddVertex(InMesh, CornerInfo.VertexAnchor);

			if (bReverse)
			{
				InDynamicMesh->AddVertex(InMesh, CornerInfo.End);
			}

			InDynamicMesh->AddVertex(InMesh, CornerInfo.Start);

			if (!bReverse)
			{
				InDynamicMesh->AddVertex(InMesh, CornerInfo.End);
			}
		}

		return;
	}

	FVector2D InitialVector = CornerInfo.Start - CornerInfo.CenterOfRotation;
	float AnglePerFace = CornerInfo.Angle / static_cast<float>(CornerInfo.BevelSubdivisions);

	// Convex
	if (CornerInfo.bConvex)
	{
		if (bAddConcaveCornerInsert)
		{
			InDynamicMesh->AddVertex(InMesh, CornerInfo.VertexAnchor);

			if (!bReverse)
			{
				InDynamicMesh->AddVertex(InMesh, CornerInfo.Start);
				InDynamicMesh->AddVertex(InMesh, CornerInfo.End);
			}

			else
			{
				InDynamicMesh->AddVertex(InMesh, CornerInfo.End);
				InDynamicMesh->AddVertex(InMesh, CornerInfo.Start);
			}
		}
	}

	TArray<FAvaShapeCachedVertex2D> CornerVertices;
	CornerVertices.Add(CornerInfo.Start);

	FVector2D HalfwayRotation = InitialVector.GetRotated(CornerInfo.Angle / 2.f) + CornerInfo.CenterOfRotation;
	float HalfwayDistance = (CornerVertex - HalfwayRotation).SizeSquared();
	float StartDistance = (CornerVertex - CornerInfo.Start).SizeSquared();
	float Multiplier = (HalfwayDistance < StartDistance) ? 1.f : -1.f;

	for (int32 FaceIdx = 1; FaceIdx < CornerInfo.BevelSubdivisions; ++FaceIdx)
	{
		FVector2D RotatedLine = InitialVector.GetRotated(AnglePerFace * Multiplier * FaceIdx);
		RotatedLine += CornerInfo.CenterOfRotation;
		CornerVertices.Add(RotatedLine);
	}

	CornerVertices.Add(CornerInfo.End);

	for (int32 FaceIdx = 0; FaceIdx < CornerInfo.BevelSubdivisions; ++FaceIdx)
	{
		InDynamicMesh->AddVertex(InMesh, CornerInfo.VertexAnchor);

		if (bReverse != CornerInfo.bConvex)
		{
			InDynamicMesh->AddVertex(InMesh, CornerVertices[FaceIdx + 1]);
			InDynamicMesh->AddVertex(InMesh, CornerVertices[FaceIdx]);
		}

		else
		{
			InDynamicMesh->AddVertex(InMesh, CornerVertices[FaceIdx]);
			InDynamicMesh->AddVertex(InMesh, CornerVertices[FaceIdx + 1]);
		}
	}
}

