// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshes/AvaShapeSphereDynMesh.h"

#include "AvaShapeVertices.h"
#include "Kismet/KismetMathLibrary.h"

const FString UAvaShapeSphereDynamicMesh::MeshName = TEXT("Sphere");

void UAvaShapeSphereDynamicMesh::SetNumSides(uint8 InNumSides)
{
	if (NumSides == InNumSides)
	{
		return;
	}

	if (InNumSides < UAvaShapeSphereDynamicMesh::MinNumSides || InNumSides > UAvaShapeSphereDynamicMesh::MaxNumSides)
	{
		return;
	}

	NumSides = InNumSides;
	OnNumSidesChanged();
}

void UAvaShapeSphereDynamicMesh::SetStartLatitude(float InDegree)
{
	if (StartLatitude == InDegree)
	{
		return;
	}

	if (InDegree < 0.0f || InDegree > 360.f)
	{
		return;
	}

	StartLatitude = InDegree;
	OnStartLatitudeChanged();
}

void UAvaShapeSphereDynamicMesh::SetLatitudeDegree(float InDegree)
{
	if (LatitudeDegree == InDegree)
	{
		return;
	}

	if (InDegree < 0.0f || InDegree > 360.f)
	{
		return;
	}

	LatitudeDegree = InDegree;
	OnLatitudeDegreeChanged();
}

void UAvaShapeSphereDynamicMesh::SetStartLongitude(float InDegree)
{
	if (StartLongitude == InDegree)
	{
		return;
	}

	if (InDegree < 0.0f || InDegree > 180.f)
	{
		return;
	}

	if (InDegree > EndLongitude)
	{
		return;
	}

	PreEditStartLongitude = StartLongitude;
	StartLongitude = InDegree;
	OnStartLongitudeChanged();
}

void UAvaShapeSphereDynamicMesh::SetEndLongitude(float InDegree)
{
	if (EndLongitude == InDegree)
	{
		return;
	}

	if (InDegree < 0.01f || InDegree > 180.f)
	{
		return;
	}

	if (InDegree < StartLongitude)
	{
		return;
	}

	PreEditEndLongitude = EndLongitude;
	EndLongitude = InDegree;
	OnEndLongitudeChanged();
}

#if WITH_EDITOR
void UAvaShapeSphereDynamicMesh::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	PreEditEndLongitude = EndLongitude;
	PreEditStartLongitude = StartLongitude;
}

void UAvaShapeSphereDynamicMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static FName NumSidesName = GET_MEMBER_NAME_CHECKED(UAvaShapeSphereDynamicMesh, NumSides);
	static FName RadiusName = GET_MEMBER_NAME_CHECKED(UAvaShapeSphereDynamicMesh, Radius);
	static FName StartLatitudeName = GET_MEMBER_NAME_CHECKED(UAvaShapeSphereDynamicMesh, StartLatitude);
	static FName LatitudeDegreeName = GET_MEMBER_NAME_CHECKED(UAvaShapeSphereDynamicMesh, LatitudeDegree);
	static FName StartLongitudeName = GET_MEMBER_NAME_CHECKED(UAvaShapeSphereDynamicMesh, StartLongitude);
	static FName EndLongitudeName = GET_MEMBER_NAME_CHECKED(UAvaShapeSphereDynamicMesh, EndLongitude);

	if (PropertyChangedEvent.MemberProperty->GetFName() == NumSidesName)
	{
		OnNumSidesChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == RadiusName)
	{
		OnRadiusChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == StartLatitudeName)
	{
		OnStartLatitudeChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == LatitudeDegreeName)
	{
		OnLatitudeDegreeChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == StartLongitudeName)
	{
		OnStartLongitudeChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == EndLongitudeName)
	{
		OnEndLongitudeChanged();
	}
}
#endif

void UAvaShapeSphereDynamicMesh::RegisterMeshes()
{
	// Mesh visible at the top when longitude is below 180 degrees
	FAvaShapeMeshData TopMesh(MESH_INDEX_TOP, TEXT("Top"), false);
	// Mesh visible at the bottom when longitude is below 180 degrees
	FAvaShapeMeshData BottomMesh(MESH_INDEX_BOTTOM, TEXT("Bottom"), false);
	// Mesh visible at the start when latitude is below 360 degrees
	FAvaShapeMeshData StartMesh(MESH_INDEX_START, TEXT("Start"), false);
	// Mesh visible at the end when latitude is below 360 degrees
	FAvaShapeMeshData EndMesh(MESH_INDEX_END, TEXT("End"), false);

	RegisterMesh(TopMesh);
	RegisterMesh(BottomMesh);
	RegisterMesh(StartMesh);
	RegisterMesh(EndMesh);

	return Super::RegisterMeshes();
}

bool UAvaShapeSphereDynamicMesh::IsMeshVisible(int32 MeshIndex)
{
	if (LatitudeDegree < UE_KINDA_SMALL_NUMBER)
	{
		return false;
	}

	switch (MeshIndex)
	{
		case(MESH_INDEX_TOP):
			return StartLongitude > 0.f;
		case(MESH_INDEX_BOTTOM):
			return EndLongitude < 180.f;
		case(MESH_INDEX_START):
		case(MESH_INDEX_END):
			return LatitudeDegree < 360.f;
		default: ;
	}

	return true;
}

void UAvaShapeSphereDynamicMesh::OnNumSidesChanged()
{
	NumSides = FMath::Clamp(NumSides, UAvaShapeSphereDynamicMesh::MinNumSides, UAvaShapeSphereDynamicMesh::MaxNumSides);
	MarkAllMeshesDirty();
}

void UAvaShapeSphereDynamicMesh::OnRadiusChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShapeSphereDynamicMesh::OnStartLatitudeChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShapeSphereDynamicMesh::OnLatitudeDegreeChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShapeSphereDynamicMesh::OnStartLongitudeChanged()
{
	if (StartLongitude > EndLongitude)
	{
		StartLongitude = PreEditStartLongitude;
		return;
	}
	MarkAllMeshesDirty();
}

void UAvaShapeSphereDynamicMesh::OnEndLongitudeChanged()
{
	if (EndLongitude < StartLongitude)
	{
		EndLongitude = PreEditEndLongitude;
		return;
	}
	MarkAllMeshesDirty();
}

bool UAvaShapeSphereDynamicMesh::CreateBaseUVs(FAvaShapeMesh& BaseMesh, FAvaShapeMaterialUVParameters& InUseParams)
{
	const float Fraction = (1.f / NumSides);

	for (int32 LonIdx = 0; LonIdx < NumSides; LonIdx++)
	{
		for (int32 LatIdx = 0; LatIdx < NumSides; LatIdx++)
		{
			BaseMesh.UVs.Add(FVector2D((Fraction * LonIdx), (Fraction * LatIdx)));
			BaseMesh.UVs.Add(FVector2D((Fraction * LonIdx), (Fraction * (LatIdx + 1))));
			BaseMesh.UVs.Add(FVector2D((Fraction * (LonIdx + 1)), (Fraction * LatIdx)));
			BaseMesh.UVs.Add(FVector2D((Fraction * (LonIdx + 1)), (Fraction * (LatIdx + 1))));
		}
	}

	return Super::ApplyUVsManually(BaseMesh)
		&& Super::ApplyUVsTransform(BaseMesh, InUseParams, FVector2D(1, 1), FVector2D(0), 90.f);
}

bool UAvaShapeSphereDynamicMesh::GenerateBaseMeshSections(FAvaShapeMesh& BaseMesh)
{
	// total degree
	const float LonDegree = FMath::Max(EndLongitude - StartLongitude, 1.0);
	const float LatDegree = LatitudeDegree;
	// around Z axis
	const float LatAngle = LatDegree / NumSides;
	const FVector LatAxis = FVector::ZAxisVector;
	// around Y axis
	const float LonAngle = LonDegree / NumSides;
	const FVector LonAxis = FVector::YAxisVector;
	// sphere radius
	const float SphereRadius = Radius * (Size3D.Z / 2.f);
	// scale sphere
	const float ScaleX = Size3D.X / Size3D.Z;
	const float ScaleY = Size3D.Y / Size3D.Z;
	const FVector Scale(ScaleX, ScaleY, 1.0f);
	// start location
	const FVector InitialNormal = FVector(0, 0, 1).RotateAngleAxis(StartLongitude, LonAxis);
	const FVector InitialLocation = FVector(0, 0, SphereRadius).RotateAngleAxis(StartLongitude, LonAxis);
	for (int32 LonIdx = 0; LonIdx < NumSides; LonIdx++)
	{
		const float CurLonAngle = LonAngle * LonIdx;
		const float NextLonAngle = LonAngle * (LonIdx + 1);
		// apply longitude rotation for current vertice
		const FVector CurLonLocation = InitialLocation.RotateAngleAxis(CurLonAngle, LonAxis);
		const FVector CurLonNormal = InitialNormal.RotateAngleAxis(CurLonAngle, LonAxis);
		// apply longitude rotation for next vertice
		const FVector NextLonLocation = InitialLocation.RotateAngleAxis(NextLonAngle, LonAxis);
		const FVector NextLonNormal = InitialNormal.RotateAngleAxis(NextLonAngle, LonAxis);

		for (int32 LatIdx = 0; LatIdx < NumSides; LatIdx++)
		{
			const float CurLatAngle = StartLatitude + (LatAngle * LatIdx);
			const float NextLatAngle = StartLatitude + (LatAngle * (LatIdx + 1));
			// current vertice
			const FVector CurLocation = CurLonLocation.RotateAngleAxis(CurLatAngle, LatAxis) * Scale;
			const FVector CurNormal = CurLonNormal.RotateAngleAxis(CurLatAngle, LatAxis);
			AddVertexRaw(BaseMesh, CurLocation, CurNormal);
			// next vertice
			const FVector NextLocation = CurLonLocation.RotateAngleAxis(NextLatAngle, LatAxis) * Scale;
			const FVector NextNormal = CurLonNormal.RotateAngleAxis(NextLatAngle, LatAxis);
			int32 NextIdx = AddVertexRaw(BaseMesh, NextLocation, NextNormal);
			// offset vertice
			const FVector OffLocation = NextLonLocation.RotateAngleAxis(CurLatAngle, LatAxis) * Scale;
			const FVector OffNormal = NextLonNormal.RotateAngleAxis(CurLatAngle, LatAxis);
			int32 OffIdx = AddVertexRaw(BaseMesh, OffLocation, OffNormal);
			// close the quad with next offset vertice
			const FVector NextOffLocation = NextLonLocation.RotateAngleAxis(NextLatAngle, LatAxis) * Scale;
			const FVector NextOffNormal = NextLonNormal.RotateAngleAxis(NextLatAngle, LatAxis);
			int32 NextOffIdx = AddVertexRaw(BaseMesh, NextOffLocation, NextOffNormal);
			AddVertex(BaseMesh, OffIdx);
			AddVertex(BaseMesh, NextIdx);
		}
	}

	return true;
}

bool UAvaShapeSphereDynamicMesh::CreateUVs(FAvaShapeMesh& InMesh, FAvaShapeMaterialUVParameters& InParams)
{
	if (InMesh.Vertices.Num() == 0)
	{
		return false;
	}

	switch (InMesh.GetMeshIndex())
	{
		case MESH_INDEX_PRIMARY:
			return CreateBaseUVs(InMesh, InParams);
		case MESH_INDEX_TOP:
			return CreateTopUVs(InMesh, InParams);
		case MESH_INDEX_BOTTOM:
			return CreateBottomUVs(InMesh, InParams);
		case MESH_INDEX_START:
			return CreateStartUVs(InMesh, InParams);
		case MESH_INDEX_END:
			return CreateEndUVs(InMesh, InParams);
		default: ;
	}

	return Super::CreateUVs(InMesh, InParams);
}

bool UAvaShapeSphereDynamicMesh::CanUpdateMeshInAnyThread() const
{
	return GetClass() == UAvaShapeSphereDynamicMesh::StaticClass();
}

bool UAvaShapeSphereDynamicMesh::CreateMesh(FAvaShapeMesh& InMesh)
{
	if (!Super::CreateMesh(InMesh))
	{
		return false;
	}

	switch (InMesh.GetMeshIndex())
	{
		case MESH_INDEX_PRIMARY:
			return GenerateBaseMeshSections(InMesh);
		case MESH_INDEX_TOP:
			return GenerateTopMeshSections(InMesh);
		case MESH_INDEX_BOTTOM:
			return GenerateBottomMeshSections(InMesh);
		case MESH_INDEX_START:
			return GenerateStartMeshSections(InMesh);
		case MESH_INDEX_END:
			return GenerateEndMeshSections(InMesh);
		default: ;
	}

	return true;
}

bool UAvaShapeSphereDynamicMesh::CreateTopUVs(FAvaShapeMesh& TopMesh, FAvaShapeMaterialUVParameters& InUseParams)
{
	const float RatioX = FVector(0, 0, 1).RotateAngleAxis(StartLongitude, FVector::YAxisVector).X;
	const float RatioY = FVector(0, 0, 1).RotateAngleAxis(StartLongitude, FVector::XAxisVector).Y;
	const FVector2D Size = FVector2D(Size3D.X, Size3D.Y) * Radius * FVector2D(RatioX, RatioY);

	return ApplyUVsPlanarProjection(TopMesh, FRotator(180, 180, 0), Size)
		&& ApplyUVsTransform(TopMesh, InUseParams, FVector2D(Size3D.X, Size3D.Y), FVector2D(0.5, 0.5));
}

bool UAvaShapeSphereDynamicMesh::GenerateTopMeshSections(FAvaShapeMesh& TopMesh)
{
	// total degree
	const float LatDegree = LatitudeDegree;
	// around Z axis
	const float LatAngle = LatDegree / NumSides;
	const FVector LatAxis = FVector::ZAxisVector;
	// around Y axis
	const FVector LonAxis = FVector::YAxisVector;
	// sphere radius
	const float SphereRadius = Radius * (Size3D.Z / 2.f);
	// scale sphere
	const float ScaleX = Size3D.X / Size3D.Z;
	const float ScaleY = Size3D.Y / Size3D.Z;
	const FVector Scale(ScaleX, ScaleY, 1.0f);
	// start location
	const FVector InitialNormal = FVector(0, 0, 1);
	const FVector InitialLocation = FVector(0, 0, SphereRadius).RotateAngleAxis(StartLongitude, LonAxis);
	// avoid duplicating current and next vertice since next becomes current
	int32 NextCurIdx = INDEX_NONE;
	const float CenterLocZ = ((InitialLocation + InitialLocation.RotateAngleAxis(180.f, LatAxis)) / 2).Z;
	// center base vertice cached
	const FAvaShapeCachedVertex3D BaseVertexCached = CacheVertexCreate(TopMesh, FVector(0, 0, CenterLocZ), InitialNormal);
	for (int32 LatIdx = 0; LatIdx < NumSides; LatIdx++)
	{
		const float CurLatAngle = StartLatitude + (LatAngle * LatIdx);
		const float NextLatAngle = StartLatitude + (LatAngle * (LatIdx + 1));
		// current vertice
		if (NextCurIdx != INDEX_NONE)
		{
			AddVertex(TopMesh, NextCurIdx);
		}
		else
		{
			const FVector CurLocation = InitialLocation.RotateAngleAxis(CurLatAngle, LatAxis) * Scale;
			AddVertexRaw(TopMesh, CurLocation, InitialNormal);
		}
		// center vertice
		AddVertex(TopMesh, BaseVertexCached);
		// next vertice
		const FVector NextLocation = InitialLocation.RotateAngleAxis(NextLatAngle, LatAxis) * Scale;
		NextCurIdx = AddVertexRaw(TopMesh, NextLocation, InitialNormal);
	}

	return true;
}

bool UAvaShapeSphereDynamicMesh::CreateBottomUVs(FAvaShapeMesh& BottomMesh, FAvaShapeMaterialUVParameters& InUseParams)
{
	const float RatioX = FVector(0, 0, 1).RotateAngleAxis(EndLongitude, FVector::YAxisVector).X;
	const float RatioY = FVector(0, 0, 1).RotateAngleAxis(EndLongitude, FVector::XAxisVector).Y;
	const FVector2D Size = FVector2D(Size3D.X, Size3D.Y) * Radius * FVector2D(RatioX, RatioY);

	return ApplyUVsPlanarProjection(BottomMesh, FRotator(180, 0, 180), Size)
		&& ApplyUVsTransform(BottomMesh, InUseParams, FVector2D(Size3D.X, Size3D.Y), FVector2D(0.5, 0.5));
}

bool UAvaShapeSphereDynamicMesh::GenerateBottomMeshSections(FAvaShapeMesh& BottomMesh)
{
	// total degree
	const float LatDegree = LatitudeDegree;
	// around Z axis
	const float LatAngle = LatDegree / NumSides;
	const FVector LatAxis = FVector::ZAxisVector;
	// around Y axis
	const FVector LonAxis = FVector::YAxisVector;
	// sphere radius
	const float SphereRadius = Radius * (Size3D.Z / 2.f);
	// scale sphere
	const float ScaleX = Size3D.X / Size3D.Z;
	const float ScaleY = Size3D.Y / Size3D.Z;
	const FVector Scale(ScaleX, ScaleY, 1.0f);
	// start location
	const FVector InitialNormal = FVector(0, 0, -1);
	const FVector InitialLocation = FVector(0, 0, SphereRadius).RotateAngleAxis(EndLongitude, LonAxis);
	// avoid duplicating current and next vertice since next becomes current
	int32 NextCurIdx = INDEX_NONE;
	const float CenterLocZ = ((InitialLocation + InitialLocation.RotateAngleAxis(180.f, LatAxis)) / 2).Z;
	// center base vertice cached
	const FAvaShapeCachedVertex3D BaseVertexCached = CacheVertexCreate(BottomMesh, FVector(0, 0, CenterLocZ), InitialNormal);
	for (int32 LatIdx = 0; LatIdx < NumSides; LatIdx++)
	{
		const float CurLatAngle = StartLatitude + (LatAngle * LatIdx);
		const float NextLatAngle = StartLatitude + (LatAngle * (LatIdx + 1));
		// current vertice
		if (NextCurIdx != INDEX_NONE)
		{
			AddVertex(BottomMesh, NextCurIdx);
		}
		else
		{
			const FVector CurLocation = InitialLocation.RotateAngleAxis(CurLatAngle, LatAxis) * Scale;
			AddVertexRaw(BottomMesh, CurLocation, InitialNormal);
		}
		// next vertice
		const FVector NextLocation = InitialLocation.RotateAngleAxis(NextLatAngle, LatAxis) * Scale;
		NextCurIdx = AddVertexRaw(BottomMesh, NextLocation, InitialNormal);
		// center vertice
		AddVertex(BottomMesh, BaseVertexCached);
	}

	return true;
}

bool UAvaShapeSphereDynamicMesh::CreateStartUVs(FAvaShapeMesh& StartMesh, FAvaShapeMaterialUVParameters& InUseParams)
{
	const FVector Axis = FVector(1, 0, 0).RotateAngleAxis(StartLatitude, FVector::ZAxisVector);

	// find rotation angle for plane
	FRotator Rot = UKismetMathLibrary::MakeRotFromX(Axis);
	Rot.Roll = 90;

	const float HSize = FMath::Sqrt(FMath::Pow(Axis.X * Size3D.X, 2) + FMath::Pow(Axis.Y * Size3D.Y, 2));
	const FVector2D Size(HSize * Radius, Size3D.Z * Radius);

	return ApplyUVsPlanarProjection(StartMesh, Rot, Size)
		&& ApplyUVsTransform(StartMesh, InUseParams, Size, FVector2D(0, 0.5));
}

bool UAvaShapeSphereDynamicMesh::GenerateStartMeshSections(FAvaShapeMesh& StartMesh)
{
	// total degree
	const float LonDegree = FMath::Max(EndLongitude - StartLongitude, 1.0);
	// around Y axis
	const float LonAngle = LonDegree / NumSides;
	const FVector LonAxis = FVector::YAxisVector;
	const FVector LatAxis = FVector::ZAxisVector;
	// sphere radius
	const float SphereRadius = Radius * (Size3D.Z / 2.f);
	// scale sphere
	const float ScaleX = Size3D.X / Size3D.Z;
	const float ScaleY = Size3D.Y / Size3D.Z;
	const FVector Scale(ScaleX, ScaleY, 1.0f);
	// start location
	const FVector InitialNormal = FVector(0, -1, 0).RotateAngleAxis(StartLongitude, LonAxis);
	const FVector InitialLocation = FVector(0, 0, SphereRadius).RotateAngleAxis(StartLongitude, LonAxis);
	// avoid duplicating current and next vertice since next becomes current
	int32 NextCurIdx = INDEX_NONE;
	// compute center for base
	const FVector TopLoc = InitialLocation;
	const float CenterTopLoc = ((TopLoc + TopLoc.RotateAngleAxis(180.f, LatAxis)) / 2).Z;
	const FVector BottomLoc = InitialLocation.RotateAngleAxis(EndLongitude - StartLongitude, LonAxis);
	const float CenterBottomLoc = ((BottomLoc + BottomLoc.RotateAngleAxis(180.f, LatAxis)) / 2).Z;
	const float CenterLocZ = (CenterTopLoc + CenterBottomLoc) / 2;
	// center base vertice cached
	const FAvaShapeCachedVertex3D BaseVertexCached = CacheVertexCreate(StartMesh, FVector(0, 0, CenterLocZ), InitialNormal.RotateAngleAxis(StartLatitude, LatAxis));
	int32 FirstVertexIdx = INDEX_NONE;
	for (int32 LonIdx = 0; LonIdx < NumSides; LonIdx++)
	{
		const float CurLonAngle = LonAngle * LonIdx;
		const float NextLonAngle = LonAngle * (LonIdx + 1);
		// current vertice
		if (NextCurIdx != INDEX_NONE)
		{
			AddVertex(StartMesh, NextCurIdx);
		}
		else
		{
			const FVector CurLocation = InitialLocation.RotateAngleAxis(CurLonAngle, LonAxis).RotateAngleAxis(StartLatitude, LatAxis) * Scale;
			const FVector CurNormal = InitialNormal.RotateAngleAxis(CurLonAngle, LonAxis).RotateAngleAxis(StartLatitude, LatAxis);
			FirstVertexIdx = AddVertexRaw(StartMesh, CurLocation, CurNormal);
		}
		// next vertice
		const FVector NextLocation = InitialLocation.RotateAngleAxis(NextLonAngle, LonAxis).RotateAngleAxis(StartLatitude, LatAxis) * Scale;
		const FVector NextNormal = InitialNormal.RotateAngleAxis(NextLonAngle, LonAxis).RotateAngleAxis(StartLatitude, LatAxis);
		NextCurIdx = AddVertexRaw(StartMesh, NextLocation, NextNormal);
		// center vertice
		AddVertex(StartMesh, BaseVertexCached);
	}

	// top close with first
	if (StartLongitude > 0.f)
	{
		AddVertex(StartMesh, BaseVertexCached);
		AddVertexRaw(StartMesh, FVector(0, 0, CenterTopLoc), InitialNormal.RotateAngleAxis(StartLatitude, LatAxis));
		AddVertex(StartMesh, FirstVertexIdx);
	}

	// bottom close with last
	if (EndLongitude < 180.f)
	{
		AddVertex(StartMesh, BaseVertexCached);
		AddVertex(StartMesh, NextCurIdx);
		AddVertexRaw(StartMesh, FVector(0, 0, CenterBottomLoc), InitialNormal.RotateAngleAxis(StartLatitude, LatAxis));
	}

	return true;
}


bool UAvaShapeSphereDynamicMesh::CreateEndUVs(FAvaShapeMesh& EndMesh, FAvaShapeMaterialUVParameters& InUseParams)
{
	const FVector Axis = FVector(1, 0, 0).RotateAngleAxis(StartLatitude + LatitudeDegree, FVector::ZAxisVector);

	// find rotation angle for plane
	FRotator Rot = UKismetMathLibrary::MakeRotFromX(Axis);
	Rot.Roll = 90;

	const float HSize = FMath::Sqrt(FMath::Pow(Axis.X * Size3D.X, 2) + FMath::Pow(Axis.Y * Size3D.Y, 2));
	const FVector2D Size(HSize * Radius, Size3D.Z * Radius);

	return ApplyUVsPlanarProjection(EndMesh, Rot, Size)
		&& ApplyUVsTransform(EndMesh, InUseParams, Size, FVector2D(0, 0.5));
}

bool UAvaShapeSphereDynamicMesh::GenerateEndMeshSections(FAvaShapeMesh& EndMesh)
{
	// total degree
	const float LonDegree = FMath::Max(EndLongitude - StartLongitude, 1.0);
	// around Y axis
	const float LonAngle = LonDegree / NumSides;
	const FVector LonAxis = FVector::YAxisVector;
	const FVector LatAxis = FVector::ZAxisVector;
	// sphere radius
	const float SphereRadius = Radius * (Size3D.Z / 2.f);
	// scale sphere
	const float ScaleX = Size3D.X / Size3D.Z;
	const float ScaleY = Size3D.Y / Size3D.Z;
	const FVector Scale(ScaleX, ScaleY, 1.0f);
	// start location
	const FVector InitialNormal = FVector(0, 1, 0).RotateAngleAxis(StartLongitude, LonAxis);
	const FVector InitialLocation = FVector(0, 0, SphereRadius).RotateAngleAxis(StartLongitude, LonAxis);
	// avoid duplicating current and next vertice since next becomes current
	int32 NextCurIdx = INDEX_NONE;
	// compute center for base
	const FVector TopLoc = InitialLocation;
	const float CenterTopLoc = ((TopLoc + TopLoc.RotateAngleAxis(180.f, LatAxis)) / 2).Z;
	const FVector BottomLoc = InitialLocation.RotateAngleAxis(EndLongitude - StartLongitude, LonAxis);
	const float CenterBottomLoc = ((BottomLoc + BottomLoc.RotateAngleAxis(180.f, LatAxis)) / 2).Z;
	const float CenterLocZ = (CenterTopLoc + CenterBottomLoc) / 2;
	// center base vertice cached
	const FAvaShapeCachedVertex3D BaseVertexCached = CacheVertexCreate(EndMesh, FVector(0, 0, CenterLocZ), InitialNormal.RotateAngleAxis(StartLatitude + LatitudeDegree, LatAxis));
	int32 FirstVertexIdx = INDEX_NONE;
	for (int32 LonIdx = 0; LonIdx < NumSides; LonIdx++)
	{
		const float CurLonAngle = LonAngle * LonIdx;
		const float NextLonAngle = LonAngle * (LonIdx + 1);
		// current vertice
		if (NextCurIdx != INDEX_NONE)
		{
			AddVertex(EndMesh, NextCurIdx);
		}
		else
		{
			const FVector CurLocation = InitialLocation.RotateAngleAxis(CurLonAngle, LonAxis).RotateAngleAxis(StartLatitude + LatitudeDegree, LatAxis) * Scale;
			const FVector CurNormal = InitialNormal.RotateAngleAxis(CurLonAngle, LonAxis).RotateAngleAxis(StartLatitude + LatitudeDegree, LatAxis);
			FirstVertexIdx = AddVertexRaw(EndMesh, CurLocation, CurNormal);
		}
		// center vertice
		AddVertex(EndMesh, BaseVertexCached);
		// next vertice
		const FVector NextLocation = InitialLocation.RotateAngleAxis(NextLonAngle, LonAxis).RotateAngleAxis(StartLatitude + LatitudeDegree, LatAxis) * Scale;
		const FVector NextNormal = InitialNormal.RotateAngleAxis(NextLonAngle, LonAxis).RotateAngleAxis(StartLatitude + LatitudeDegree, LatAxis);
		NextCurIdx = AddVertexRaw(EndMesh, NextLocation, NextNormal);
	}

	// top close with first
	if (StartLongitude > 0.f)
	{
		AddVertex(EndMesh, BaseVertexCached);
		AddVertex(EndMesh, FirstVertexIdx);
		AddVertexRaw(EndMesh, FVector(0, 0, CenterTopLoc), InitialNormal.RotateAngleAxis(StartLatitude + LatitudeDegree, LatAxis));
	}

	// bottom close with last
	if (EndLongitude < 180.f)
	{
		AddVertex(EndMesh, BaseVertexCached);
		AddVertexRaw(EndMesh, FVector(0, 0, CenterBottomLoc), InitialNormal.RotateAngleAxis(StartLatitude + LatitudeDegree, LatAxis));
		AddVertex(EndMesh, NextCurIdx);
	}

	return true;
}
