// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshes/AvaShapeStarDynMesh.h"

const FString UAvaShapeStarDynamicMesh::MeshName = TEXT("Star");

void UAvaShapeStarDynamicMesh::SetNumPoints(uint8 InNumPoints)
{
	if (NumPoints == InNumPoints)
	{
		return;
	}

	if (InNumPoints < UAvaShapeStarDynamicMesh::MinNumPoints || InNumPoints > UAvaShapeStarDynamicMesh::MaxNumPoints)
	{
		return;
	}

	NumPoints = InNumPoints;
	OnNumSidesChanged();
}

void UAvaShapeStarDynamicMesh::SetInnerSize(float InInnerSize)
{
	if (InnerSize == InInnerSize)
	{
		return;
	}

	if (InInnerSize < 0.0f || InInnerSize > 0.99f)
	{
		return;
	}

	InnerSize = InInnerSize;
	OnInnerSizeChanged();
}

#if WITH_EDITOR
void UAvaShapeStarDynamicMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static FName NumPointsName = GET_MEMBER_NAME_CHECKED(UAvaShapeStarDynamicMesh, NumPoints);
	static FName InnerSizeName = GET_MEMBER_NAME_CHECKED(UAvaShapeStarDynamicMesh, InnerSize);

	if (PropertyChangedEvent.MemberProperty->GetFName() == NumPointsName)
	{
		OnNumSidesChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == InnerSizeName)
	{
		OnInnerSizeChanged();
	}
}
#endif

void UAvaShapeStarDynamicMesh::OnNumSidesChanged()
{
	NumPoints = FMath::Clamp(NumPoints, UAvaShapeStarDynamicMesh::MinNumPoints, UAvaShapeStarDynamicMesh::MaxNumPoints);
	MarkAllMeshesDirty();
}

void UAvaShapeStarDynamicMesh::OnInnerSizeChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShapeStarDynamicMesh::GenerateBorderVertices(TArray<FVector2D>& BorderVertices)
{
	Super::GenerateBorderVertices(BorderVertices);

	const FVector2D ZeroRotation = FVector2D(0.f, Size2D.Y / 2.f);
	const FVector2D ZeroRotationInner = FVector2D(0.f, Size2D.Y * InnerSize / 2.f);
	const FVector2D VScale = {
		Size2D.X >= Size2D.Y ? Size2D.X / Size2D.Y : 1.f,
		Size2D.Y >= Size2D.X ? Size2D.Y / Size2D.X : 1.f
	};

	for (uint8 SideIdx = 0; SideIdx < NumPoints; ++SideIdx)
	{
		FVector2D RotatedZero = ZeroRotation.GetRotated((360.f / NumPoints) * -static_cast<float>(SideIdx));
		RotatedZero *= VScale;
		BorderVertices.Add(RotatedZero);

		FVector2D RotatedZeroInner = ZeroRotationInner.GetRotated((360.f / NumPoints) * -(static_cast<float>(SideIdx) + 0.5f));
		RotatedZeroInner *= VScale;
		BorderVertices.Add(RotatedZeroInner);
	}
}

bool UAvaShapeStarDynamicMesh::IsMeshVisible(int32 MeshIndex)
{
	switch (MeshIndex)
	{
		case MESH_INDEX_PRIMARY:
			return InnerSize > UE_KINDA_SMALL_NUMBER;
		default:
			return true;
	}
}

bool UAvaShapeStarDynamicMesh::CanUpdateMeshInAnyThread() const
{
	return GetClass() == UAvaShapeStarDynamicMesh::StaticClass();
}
