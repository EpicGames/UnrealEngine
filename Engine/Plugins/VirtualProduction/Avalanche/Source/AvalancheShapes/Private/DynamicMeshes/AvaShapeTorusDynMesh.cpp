// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshes/AvaShapeTorusDynMesh.h"

#include "AvaShapeVertices.h"
#include "Kismet/KismetMathLibrary.h"

const FString UAvaShapeTorusDynamicMesh::MeshName = TEXT("Torus");

void UAvaShapeTorusDynamicMesh::SetNumSides(uint8 InNumSides)
{
	if (NumSides == InNumSides)
	{
		return;
	}

	if (InNumSides < UAvaShapeTorusDynamicMesh::MinNumSides || InNumSides > UAvaShapeTorusDynamicMesh::MaxNumSides)
	{
		return;
	}

	NumSides = InNumSides;
	OnNumSidesChanged();
}

void UAvaShapeTorusDynamicMesh::SetNumSlices(uint8 InNumSlices)
{
	if (NumSlices == InNumSlices)
	{
		return;
	}

	if (InNumSlices < UAvaShapeTorusDynamicMesh::MinNumSlices || InNumSlices > UAvaShapeTorusDynamicMesh::MaxNumSlices)
	{
		return;
	}

	NumSlices = InNumSlices;
	OnNumSlicesChanged();
}

void UAvaShapeTorusDynamicMesh::SetInnerSize(float InInnerSize)
{
	if (InnerSize == InInnerSize)
	{
		return;
	}

	if (InInnerSize < 0.5f || InInnerSize > 0.99f)
	{
		return;
	}

	InnerSize = InInnerSize;
	OnInnerSizeChanged();
}

void UAvaShapeTorusDynamicMesh::SetAngleDegree(float InAngleDegree)
{
	if (AngleDegree == InAngleDegree)
	{
		return;
	}

	if (InAngleDegree < 0.f || InAngleDegree > 360.f)
	{
		return;
	}

	AngleDegree = InAngleDegree;
	OnAngleDegreeChanged();
}

void UAvaShapeTorusDynamicMesh::SetStartDegree(float InStartDegree)
{
	if (StartDegree == InStartDegree)
	{
		return;
	}

	if (InStartDegree < 0.f || InStartDegree > 360.f)
	{
		return;
	}

	StartDegree = InStartDegree;
	OnStartDegreeChanged();
}

#if WITH_EDITOR
void UAvaShapeTorusDynamicMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	static FName NumSidesName = GET_MEMBER_NAME_CHECKED(UAvaShapeTorusDynamicMesh, NumSides);
	static FName NumSlicesName = GET_MEMBER_NAME_CHECKED(UAvaShapeTorusDynamicMesh, NumSlices);
	static FName InnerSizeName = GET_MEMBER_NAME_CHECKED(UAvaShapeTorusDynamicMesh, InnerSize);
	static FName AngleDegreeName = GET_MEMBER_NAME_CHECKED(UAvaShapeTorusDynamicMesh, AngleDegree);
	static FName StartDegreeName = GET_MEMBER_NAME_CHECKED(UAvaShapeTorusDynamicMesh, StartDegree);

	if (PropertyChangedEvent.MemberProperty->GetFName() == NumSidesName)
	{
		OnNumSidesChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == NumSlicesName)
	{
		OnNumSlicesChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == InnerSizeName)
	{
		OnInnerSizeChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == AngleDegreeName)
	{
		OnAngleDegreeChanged();
	}
	else if (PropertyChangedEvent.MemberProperty->GetFName() == StartDegreeName)
	{
		OnStartDegreeChanged();
	}
}
#endif

void UAvaShapeTorusDynamicMesh::RegisterMeshes()
{
	// Mesh visible when angledegree is less than 360, at the start of the tube
	FAvaShapeMeshData StartMesh(MESH_INDEX_START, TEXT("Start"), false);
	// Mesh visible when angledegree is less than 360, at the end of the tube
	FAvaShapeMeshData EndMesh(MESH_INDEX_END, TEXT("End"), false);

	RegisterMesh(StartMesh);
	RegisterMesh(EndMesh);

	Super::RegisterMeshes();
}

bool UAvaShapeTorusDynamicMesh::IsMeshVisible(int32 MeshIndex)
{
	if (AngleDegree < UE_KINDA_SMALL_NUMBER)
	{
		return false;
	}

	switch (MeshIndex)
	{
		case(MESH_INDEX_START):
		case(MESH_INDEX_END):
			return AngleDegree < 360;
		default:
			return true;
	}
}

bool UAvaShapeTorusDynamicMesh::CreateStartUVs(FAvaShapeMesh& StartMesh, FAvaShapeMaterialUVParameters& InUseParams)
{
	const float AngleCircle = -360.f / NumSides;
	static const FVector2D Multiplier(1, -1);
	static const FVector2D Initial(0, 0.5);
	// base
	StartMesh.UVs.Add(FVector2D(0.5, 0.5));
	for (int32 i = 0; i < NumSides + 1; i++)
	{
		constexpr float Offset = 0.5;
		// current
		const FVector2D Current = Initial.GetRotated(AngleCircle * i) * Multiplier;
		StartMesh.UVs.Add(FVector2D(Current.X, Current.Y) + Offset);
	}

	const FVector Axis = FVector(0, 0, 1).RotateAngleAxis(StartDegree, FVector::XAxisVector);
	const float WSize = Size3D.X;
	const float HSize = FMath::Sqrt(FMath::Pow(Axis.Y * Size3D.Y, 2) + FMath::Pow(Axis.Z * Size3D.Z, 2));

	return ApplyUVsManually(StartMesh)
		&& ApplyUVsTransform(StartMesh, InUseParams, FVector2D(WSize, HSize), FVector2D(0, 0));
}

bool UAvaShapeTorusDynamicMesh::GenerateStartMeshSections(FAvaShapeMesh& StartMesh)
{
	const float AngleCircle = 360.f / NumSides;
	// radius torus
	const float InnerRadius = InnerSize * (Size3D.Z / 2.f);
	const float SliceRadius = (Size3D.Z / 2.f) - InnerRadius;
	// scale torus
	const float ScaleX = Size3D.X / Size3D.Z;
	const float ScaleY = Size3D.Y / Size3D.Z;
	const FVector Scale(ScaleX, ScaleY, 1.0f);

	static const FVector InitialNormal(0, 1, 0);
	const FVector InitialLocation(0, 0, SliceRadius);
	const FVector InitialDirection(0, 0, 1);

	const FVector SliceLoc = InitialLocation.RotateAngleAxis(StartDegree, FVector::XAxisVector);
	const FVector SliceNormal = InitialNormal.RotateAngleAxis(StartDegree, FVector::XAxisVector);
	const FVector CircleAxis = FVector::YAxisVector.RotateAngleAxis(StartDegree, FVector::XAxisVector);
	const FVector SliceDirection = InitialDirection.RotateAngleAxis(StartDegree, FVector::XAxisVector);

	// avoid duplicating current and next vertice since next becomes current
	int32 NextCurIdx = INDEX_NONE;

	// center base vertice cached
	const FAvaShapeCachedVertex3D BaseVertexCached = CacheVertexCreate(StartMesh, (SliceDirection * InnerRadius) * Scale, SliceNormal);

	for (int32 IdxCircle = 0; IdxCircle < NumSides; IdxCircle++)
	{
		// current
		if (NextCurIdx != INDEX_NONE)
		{
			AddVertex(StartMesh, NextCurIdx);
		}
		else
		{
			FVector CurrentLoc = SliceLoc.RotateAngleAxis(AngleCircle * IdxCircle, CircleAxis) + SliceDirection * InnerRadius;
			CurrentLoc *= Scale;
			AddVertexRaw(StartMesh, CurrentLoc, SliceNormal);
		}
		// base
		AddVertex(StartMesh, BaseVertexCached);
		// next
		FVector NextLoc = SliceLoc.RotateAngleAxis(AngleCircle * (IdxCircle+1), CircleAxis) + SliceDirection * InnerRadius;
		NextLoc *= Scale;
		NextCurIdx = AddVertexRaw(StartMesh, NextLoc, SliceNormal);
	}

	return true;
}

bool UAvaShapeTorusDynamicMesh::CreateEndUVs(FAvaShapeMesh& EndMesh, FAvaShapeMaterialUVParameters& InUseParams)
{
	const float AngleCircle = -360.f / NumSides;
	static const FVector2D Multiplier(1, -1);
	static const FVector2D Initial(0, 0.5);
	// base
	EndMesh.UVs.Add(FVector2D(0.5, 0.5));
	for (int32 i = 0; i < NumSides + 1; i++)
	{
		constexpr float Offset = 0.5;
		// current
		const FVector2D Current = Initial.GetRotated(AngleCircle * i) * Multiplier;
		EndMesh.UVs.Add(FVector2D(Current.X, Current.Y) + Offset);
	}

	const FVector Axis = FVector(0, 0, 1).RotateAngleAxis(StartDegree + AngleDegree, FVector::XAxisVector);
	const float WSize = Size3D.X;
	const float HSize = FMath::Sqrt(FMath::Pow(Axis.Y * Size3D.Y, 2) + FMath::Pow(Axis.Z * Size3D.Z, 2));

	return ApplyUVsManually(EndMesh)
		&& ApplyUVsTransform(EndMesh, InUseParams, FVector2D(WSize, HSize), FVector2D(0, 0));
}

bool UAvaShapeTorusDynamicMesh::GenerateEndMeshSections(FAvaShapeMesh& EndMesh)
{
	const float AngleSlice = AngleDegree / NumSlices;
	const float AngleCircle = 360.f / NumSides;
	// radius torus
	const float InnerRadius = InnerSize * (Size3D.Z / 2.f);
	const float SliceRadius = (Size3D.Z / 2.f) - InnerRadius;
	// scale torus
	const float ScaleX = Size3D.X / Size3D.Z;
	const float ScaleY = Size3D.Y / Size3D.Z;
	const FVector Scale(ScaleX, ScaleY, 1.0f);

	static const FVector InitialNormal(0, -1, 0);
	const FVector InitialLocation(0, 0, SliceRadius);
	const FVector InitialDirection(0, 0, 1);

	const FVector SliceLoc = InitialLocation.RotateAngleAxis(StartDegree + (AngleSlice * NumSlices), FVector::XAxisVector);
	const FVector SliceNormal = InitialNormal.RotateAngleAxis(StartDegree + (AngleSlice * NumSlices), FVector::XAxisVector);
	const FVector CircleAxis = FVector::YAxisVector.RotateAngleAxis(StartDegree + (AngleSlice * NumSlices), FVector::XAxisVector);
	const FVector SliceDirection = InitialDirection.RotateAngleAxis(StartDegree + (AngleSlice * NumSlices), FVector::XAxisVector);

	// avoid duplicating current and next vertice since next becomes current
	int32 NextCurIdx = INDEX_NONE;

	// center base vertice cached
	const FAvaShapeCachedVertex3D BaseVertexCached = CacheVertexCreate(EndMesh, (SliceDirection * InnerRadius) * Scale, SliceNormal);

	for (int32 IdxCircle = 0; IdxCircle < NumSides; IdxCircle++)
	{
		// base
		AddVertex(EndMesh, BaseVertexCached);
		// current
		if (NextCurIdx != INDEX_NONE)
		{
			AddVertex(EndMesh, NextCurIdx);
		}
		else
		{
			FVector CurrentLoc = SliceLoc.RotateAngleAxis(AngleCircle * IdxCircle, CircleAxis) + SliceDirection * InnerRadius;
			CurrentLoc *= Scale;
			AddVertexRaw(EndMesh, CurrentLoc, SliceNormal);
		}
		// next
		FVector NextLoc = SliceLoc.RotateAngleAxis(AngleCircle * (IdxCircle + 1), CircleAxis) + SliceDirection * InnerRadius;
		NextLoc *= Scale;
		NextCurIdx = AddVertexRaw(EndMesh, NextLoc, SliceNormal);
	}

	return true;
}

bool UAvaShapeTorusDynamicMesh::CreateBaseUVs(FAvaShapeMesh& BaseMesh, FAvaShapeMaterialUVParameters& InUseParams)
{
	const float FractionSides = (1.f / NumSides);
	const float FractionSlice = (1.f / NumSlices);

	for (int32 SliceIdx = 0; SliceIdx < NumSlices; SliceIdx++)
	{
		for (int32 SideIdx = 0; SideIdx < NumSides; SideIdx++)
		{
			if (SideIdx == 0)
			{
				BaseMesh.UVs.Add(FVector2D(FractionSides * SideIdx, FractionSlice * (SliceIdx + 1)));
				BaseMesh.UVs.Add(FVector2D(FractionSides * SideIdx, FractionSlice * SliceIdx));
			}
			BaseMesh.UVs.Add(FVector2D(FractionSides * (SideIdx + 1), FractionSlice * SliceIdx));
			BaseMesh.UVs.Add(FVector2D(FractionSides * (SideIdx + 1), FractionSlice * (SliceIdx + 1)));
		}
	}

	return ApplyUVsManually(BaseMesh)
		&& ApplyUVsTransform(BaseMesh, InUseParams, FVector2D(1), FVector2D(0, 0));
}

bool UAvaShapeTorusDynamicMesh::GenerateBaseMeshSections(FAvaShapeMesh& BaseMesh)
{
	const float AngleSlice = AngleDegree / NumSlices;
	const float AngleCircle = 360.f / NumSides;
	// radius torus
	const float InnerRadius = InnerSize * (Size3D.Z / 2.f);
	const float SliceRadius = (Size3D.Z / 2.f) - InnerRadius;
	// scale torus
	const float ScaleX = Size3D.X / Size3D.Z;
	const float ScaleY = Size3D.Y / Size3D.Z;
	const FVector Scale(ScaleX, ScaleY, 1.0f);

	const FVector InitialNormal = FVector(0, 0, 1).RotateAngleAxis(StartDegree, FVector::XAxisVector);
	const FVector InitialLocation = FVector(0, 0, SliceRadius).RotateAngleAxis(StartDegree, FVector::XAxisVector);
	const FVector InitialDirection = FVector(0, 0, 1).RotateAngleAxis(StartDegree, FVector::XAxisVector);
	const FVector InitialCircleAxis = FVector::YAxisVector.RotateAngleAxis(StartDegree, FVector::XAxisVector);

	for (int32 IdxSlice = 0; IdxSlice < NumSlices; IdxSlice++)
	{
		const float CurSliceAngle = AngleSlice * IdxSlice;
		const float NextSliceAngle = AngleSlice * (IdxSlice + 1);

		const FVector SliceLoc = InitialLocation.RotateAngleAxis(CurSliceAngle, FVector::XAxisVector);
		const FVector SliceNormal = InitialNormal.RotateAngleAxis(CurSliceAngle, FVector::XAxisVector);
		const FVector CircleAxis = InitialCircleAxis.RotateAngleAxis(CurSliceAngle, FVector::XAxisVector);

		FVector NextSliceLoc = InitialLocation.RotateAngleAxis(NextSliceAngle, FVector::XAxisVector);
		FVector NextSliceNormal = InitialNormal.RotateAngleAxis(NextSliceAngle, FVector::XAxisVector);
		FVector NextCircleAxis = InitialCircleAxis.RotateAngleAxis(NextSliceAngle, FVector::XAxisVector);

		const FVector SliceDirection = InitialDirection.RotateAngleAxis(CurSliceAngle, FVector::XAxisVector);
		FVector NextSliceDirection = InitialDirection.RotateAngleAxis(NextSliceAngle, FVector::XAxisVector);

		// avoid duplicating current and next vertice since next becomes current
		int32 NextCurIdx = INDEX_NONE;
		// avoid duplicating offset and next offset vertice since next offset becomes current offset
		int32 NextOffIdx = INDEX_NONE;

		for (int32 IdxCircle = 0; IdxCircle < NumSides; IdxCircle++)
		{
			const float CurCircleAngle = AngleCircle * IdxCircle;
			const float NextCircleAngle = AngleCircle * (IdxCircle + 1);
			// offset location
			if (NextOffIdx != INDEX_NONE)
			{
				AddVertex(BaseMesh, NextOffIdx);
			}
			else
			{
				FVector OffsetCircleLoc = NextSliceLoc.RotateAngleAxis(CurCircleAngle, NextCircleAxis) + NextSliceDirection * InnerRadius;
				OffsetCircleLoc *= Scale;
				const FVector OffsetCircleNormal = NextSliceNormal.RotateAngleAxis(CurCircleAngle, NextCircleAxis);
				NextOffIdx = AddVertexRaw(BaseMesh, OffsetCircleLoc, OffsetCircleNormal);
			}
			// Current location
			if (NextCurIdx != INDEX_NONE)
			{
				AddVertex(BaseMesh, NextCurIdx);
			}
			else
			{
				FVector CircleLoc = SliceLoc.RotateAngleAxis(CurCircleAngle, CircleAxis) + SliceDirection * InnerRadius;
				CircleLoc *= Scale;
				const FVector CircleNormal = SliceNormal.RotateAngleAxis(CurCircleAngle, CircleAxis);
				AddVertexRaw(BaseMesh, CircleLoc, CircleNormal);
			}
			// Next location
			FVector NextCircleLoc = SliceLoc.RotateAngleAxis(NextCircleAngle, CircleAxis) + SliceDirection * InnerRadius;
			NextCircleLoc *= Scale;
			const FVector NextCircleNormal = SliceNormal.RotateAngleAxis(NextCircleAngle, CircleAxis);
			NextCurIdx = AddVertexRaw(BaseMesh, NextCircleLoc, NextCircleNormal);
			// complete quad with next offset location
			AddVertex(BaseMesh, NextOffIdx);
			AddVertex(BaseMesh, NextCurIdx);
			FVector NextOffsetCircleLoc = NextSliceLoc.RotateAngleAxis(NextCircleAngle, NextCircleAxis) + NextSliceDirection * InnerRadius;
			NextOffsetCircleLoc *= Scale;
			const FVector NextOffsetCircleNormal = NextSliceNormal.RotateAngleAxis(NextCircleAngle, NextCircleAxis);
			NextOffIdx = AddVertexRaw(BaseMesh, NextOffsetCircleLoc, NextOffsetCircleNormal);
		}
	}

	return true;
}

void UAvaShapeTorusDynamicMesh::OnNumSidesChanged()
{
	NumSides = FMath::Clamp(NumSides, UAvaShapeTorusDynamicMesh::MinNumSides, UAvaShapeTorusDynamicMesh::MaxNumSides);
	MarkAllMeshesDirty();
}

void UAvaShapeTorusDynamicMesh::OnNumSlicesChanged()
{
	NumSlices = FMath::Clamp(NumSlices, UAvaShapeTorusDynamicMesh::MinNumSlices, UAvaShapeTorusDynamicMesh::MaxNumSlices);
	MarkAllMeshesDirty();
}

void UAvaShapeTorusDynamicMesh::OnInnerSizeChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShapeTorusDynamicMesh::OnAngleDegreeChanged()
{
	MarkAllMeshesDirty();
}

void UAvaShapeTorusDynamicMesh::OnStartDegreeChanged()
{
	MarkAllMeshesDirty();
}

bool UAvaShapeTorusDynamicMesh::CreateUVs(FAvaShapeMesh& InMesh, FAvaShapeMaterialUVParameters& InParams)
{
	if (InMesh.Vertices.Num() == 0)
	{
		return false;
	}

	switch (InMesh.GetMeshIndex())
	{
		case MESH_INDEX_PRIMARY:
			return CreateBaseUVs(InMesh, InParams);
		case MESH_INDEX_START:
			return CreateStartUVs(InMesh, InParams);
		case MESH_INDEX_END:
			return CreateEndUVs(InMesh, InParams);
		default: ;
	}

	return Super::CreateUVs(InMesh, InParams);
}

bool UAvaShapeTorusDynamicMesh::CanUpdateMeshInAnyThread() const
{
	return GetClass() == UAvaShapeTorusDynamicMesh::StaticClass();
}

bool UAvaShapeTorusDynamicMesh::CreateMesh(FAvaShapeMesh& InMesh)
{
	if (!Super::CreateMesh(InMesh))
	{
		return false;
	}

	switch (InMesh.GetMeshIndex())
	{
		case MESH_INDEX_PRIMARY:
			return GenerateBaseMeshSections(InMesh);
		case MESH_INDEX_START:
			return GenerateStartMeshSections(InMesh);
		case MESH_INDEX_END:
			return GenerateEndMeshSections(InMesh);
		default: ;
	}

	return true;
}
