// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieScene/MovieSceneAvaShapeRectCornerSection.h"
#include "AvaShapesDefs.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "DynamicMeshes/AvaShapeRectangleDynMesh.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "UObject/SequencerObjectVersion.h"

#define LOCTEXT_NAMESPACE "MovieSceneAvaShapeRectCornerSection"

#if WITH_EDITOR
namespace UE::Ava::Private
{
	struct FShapeRectCornerChannelMetaData
	{
		FShapeRectCornerChannelMetaData()
		{		
			BevelSizeMetaData.SetIdentifiers("BevelSize", LOCTEXT("BevelSize", "BevelSize"));
			BevelSizeMetaData.SubPropertyPath = BevelSizeMetaData.Name;
			BevelSizeMetaData.SortOrder = 0;
			BevelSizeMetaData.bCanCollapseToTrack = false;

			BevelSubdivisionsMetaData.SetIdentifiers("BevelSubdivisions", LOCTEXT("BevelSubdivisions", "BevelSubdivisions"));
			BevelSubdivisionsMetaData.SubPropertyPath = BevelSubdivisionsMetaData.Name;
			BevelSubdivisionsMetaData.SortOrder = 1;
			BevelSubdivisionsMetaData.bCanCollapseToTrack = false;

			TypeMetaData.SetIdentifiers("Type", LOCTEXT("Type", "Type"));
			TypeMetaData.SubPropertyPath = TypeMetaData.Name;
			TypeMetaData.SortOrder = 2;
			TypeMetaData.bCanCollapseToTrack = true;
		}

		FMovieSceneChannelMetaData TypeMetaData;
		FMovieSceneChannelMetaData BevelSizeMetaData;
		FMovieSceneChannelMetaData BevelSubdivisionsMetaData;
	};
}
#endif

UMovieSceneAvaShapeRectCornerSection::UMovieSceneAvaShapeRectCornerSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	EvalOptions.EnableAndSetCompletionMode(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault
		? EMovieSceneCompletionMode::RestoreState
		: EMovieSceneCompletionMode::ProjectDefault);
	
	BlendType = EMovieSceneBlendType::Absolute;
	
	Type.SetEnum(StaticEnum<EAvaShapeCornerType>());
	
	// Initialize this section's channel proxy
	FMovieSceneChannelProxyData Channels;

#if WITH_EDITOR
	using namespace UE::Ava::Private;
	static const FShapeRectCornerChannelMetaData MetaData;
	Channels.Add(BevelSize,         MetaData.BevelSizeMetaData,          TMovieSceneExternalValue<float>());
	Channels.Add(BevelSubdivisions, MetaData.BevelSubdivisionsMetaData,  TMovieSceneExternalValue<uint8>());
	Channels.Add(Type,              MetaData.TypeMetaData,               TMovieSceneExternalValue<uint8>());
#else
	Channels.Add(BevelSize);
	Channels.Add(BevelSubdivisions);
	Channels.Add(Type);
#endif

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
}

#undef LOCTEXT_NAMESPACE
