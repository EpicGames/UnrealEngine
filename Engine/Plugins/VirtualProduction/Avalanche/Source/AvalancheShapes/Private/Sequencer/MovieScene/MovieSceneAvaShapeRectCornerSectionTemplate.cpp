// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneAvaShapeRectCornerSectionTemplate.h"
#include "Sequencer/MovieScene/MovieSceneAvaShapeRectCornerSection.h"
#include "DynamicMeshes/AvaShapeRectangleDynMesh.h"
#include "Tracks/MovieScenePropertyTrack.h"

void UE::MovieScene::MultiChannelFromData(const FAvaShapeRectangleCornerSettings& In
	, FMultiChannelShapeRectCorner& Out)
{
	Out.BevelSize         = { In.BevelSize };
	Out.BevelSubdivisions = In.BevelSubdivisions;
	Out.Type              = In.Type;
}

void UE::MovieScene::ResolveChannelsToData(const FMultiChannelShapeRectCorner& In
	, FAvaShapeRectangleCornerSettings& Out)
{
	Out.BevelSize         = In[0];
	Out.BevelSubdivisions = In.BevelSubdivisions;
	Out.Type              = In.Type;
}

void UE::MovieScene::BlendValue(FBlendableShapeRectCorner& OutBlend
	, const FMultiChannelShapeRectCorner& InValue
	, float Weight
	, EMovieSceneBlendType BlendType
	, TMovieSceneInitialValueStore<FAvaShapeRectangleCornerSettings>& InitialValueStore)
{
	if (BlendType == EMovieSceneBlendType::Absolute || BlendType == EMovieSceneBlendType::Relative)
	{
		if (BlendType == EMovieSceneBlendType::Relative)
		{
			const FAvaShapeRectangleCornerSettings InitialValue = InitialValueStore.GetInitialValue();
			OutBlend.Absolute.BevelSize.Set(0, (InitialValue.BevelSize + InValue.BevelSize.Get(0, 0.f)) * Weight);
		}
		else
		{
			OutBlend.Absolute.BevelSize.Set(0, InValue.BevelSize.Get(0, 0.f) * Weight);
		}
		
		OutBlend.AbsoluteWeights[0] = OutBlend.AbsoluteWeights[0] + Weight;
		OutBlend.Absolute.BevelSubdivisions = InValue.BevelSubdivisions;
		OutBlend.Absolute.Type = InValue.Type;
	}
	else if (BlendType == EMovieSceneBlendType::Additive)
	{
		OutBlend.Additive.BevelSize.Set(0, InValue.BevelSize.Get(0, 0.f) * Weight);
		OutBlend.Additive.BevelSubdivisions = InValue.BevelSubdivisions;
		OutBlend.Additive.Type = InValue.Type;
	}
}

template<> FMovieSceneAnimTypeID GetBlendingDataType<FAvaShapeRectangleCornerSettings>()
{
	static FMovieSceneAnimTypeID TypeID = FMovieSceneAnimTypeID::Unique();
	return TypeID;
}

FMovieSceneAvaShapeRectCornerSectionTemplate::FMovieSceneAvaShapeRectCornerSectionTemplate(const UMovieSceneAvaShapeRectCornerSection& Section
	, const UMovieScenePropertyTrack& Track)
	: FMovieScenePropertySectionTemplate(Track.GetPropertyName(), Track.GetPropertyPath().ToString())
	, BlendType(Section.GetBlendType().Get())
{
	BevelSize         = Section.BevelSize;
	BevelSubdivisions = Section.BevelSubdivisions;
	Type              = Section.Type;
}

void FMovieSceneAvaShapeRectCornerSectionTemplate::Evaluate(const FMovieSceneEvaluationOperand& Operand
	, const FMovieSceneContext& Context
	, const FPersistentEvaluationData& PersistentData
	, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	using namespace UE::MovieScene;
	
	const FFrameTime Time = Context.GetTime();

	FMultiChannelShapeRectCorner RectCorner;
		
	float BevelSizeValue;
	if (BevelSize.Evaluate(Time, BevelSizeValue))
	{
		RectCorner.BevelSize.Set(0, BevelSizeValue);
	}
	
	uint8 BevelSubdivisionsValue;
	if (BevelSubdivisions.Evaluate(Time, BevelSubdivisionsValue))
	{
		RectCorner.BevelSubdivisions = BevelSubdivisionsValue;
	}

	uint8 TypeValue;
	if (Type.Evaluate(Time, TypeValue))
	{
		RectCorner.Type = static_cast<EAvaShapeCornerType>(TypeValue);
	}

	if (!RectCorner.BevelSize.IsEmpty())
	{
		const FMovieSceneBlendingActuatorID ActuatorTypeID = EnsureActuator<FAvaShapeRectangleCornerSettings>(ExecutionTokens.GetBlendingAccumulator());
	
		// Add the blendable to the accumulator
		const float Weight = EvaluateEasing(Context.GetTime());
		ExecutionTokens.BlendToken(ActuatorTypeID
			, TBlendableToken<FAvaShapeRectangleCornerSettings>(RectCorner
			, BlendType
			, Weight));
	}
}
