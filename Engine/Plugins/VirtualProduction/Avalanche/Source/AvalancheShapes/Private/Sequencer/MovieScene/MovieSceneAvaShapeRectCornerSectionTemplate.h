// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Evaluation/MovieScenePropertyTemplate.h"
#include "DynamicMeshes/AvaShapeRectangleDynMesh.h"
#include "MovieSceneAvaShapeRectCornerSectionTemplate.generated.h"

namespace UE::MovieScene
{
	struct FMultiChannelShapeRectCorner
	{
		FMultiChannelShapeRectCorner() = default;

		float operator[](int32 Index) const
		{
			return BevelSize[Index];
		}

		UE::MovieScene::TMultiChannelValue<float, 1> BevelSize;
		uint8 BevelSubdivisions = 0;
		EAvaShapeCornerType Type = EAvaShapeCornerType::Point;
	};
	
	void MultiChannelFromData(const FAvaShapeRectangleCornerSettings& In
		, FMultiChannelShapeRectCorner& Out);

	void ResolveChannelsToData(const FMultiChannelShapeRectCorner& In
		, FAvaShapeRectangleCornerSettings& Out);

	struct FBlendableShapeRectCorner
	{
		FBlendableShapeRectCorner()
		{
			// All weights start at 0
			FMemory::Memzero(&AbsoluteWeights, sizeof(AbsoluteWeights));
		}

		TOptional<FMultiChannelShapeRectCorner> InitialValue;
		FMultiChannelShapeRectCorner Absolute;
		FMultiChannelShapeRectCorner Additive;
		
		float AbsoluteWeights[1];
		
		/** Resolve this structure's data into a final value to pass to the actuator */
		FAvaShapeRectangleCornerSettings Resolve(TMovieSceneInitialValueStore<FAvaShapeRectangleCornerSettings>& InitialValueStore)
		{
			TOptional<FMultiChannelShapeRectCorner> CurrentValue;
			FMultiChannelShapeRectCorner Result;

			// Iterate through each channel
			for (uint8 Channel = 0; Channel < 1; ++Channel)
			{
				// Any animated channels with a weight of 0 should match the object's *initial* position. Exclusively additive channels are based implicitly off the initial value
				const bool bUseInitialValue = (Absolute.BevelSize.IsSet(Channel) && AbsoluteWeights[Channel] == 0.f)
					|| (!Absolute.BevelSize.IsSet(Channel) && Additive.BevelSize.IsSet(Channel));
				
				if (bUseInitialValue)
				{
					if (!InitialValue.IsSet())
					{
						InitialValue.Emplace();
						UE::MovieScene::MultiChannelFromData(InitialValueStore.GetInitialValue(), InitialValue.GetValue());
					}
					
					Result.BevelSize.Set(Channel, InitialValue.GetValue()[Channel]);
					Result.BevelSubdivisions = InitialValue.GetValue().BevelSubdivisions;
					Result.Type = InitialValue.GetValue().Type;
				}
				else if (Absolute.BevelSize.IsSet(Channel))
				{
					// If it has a non-zero weight, divide by it, and apply the absolute total to the result
					Result.BevelSize.Set(Channel, Absolute[Channel] / AbsoluteWeights[Channel]);
					Result.BevelSubdivisions = Absolute.BevelSubdivisions;
					Result.Type = Absolute.Type;
				}

				// If it has any additive values in the channel, add those on
				if (Additive.BevelSize.IsSet(Channel))
				{
					Result.BevelSize.Increment(Channel, Additive[Channel]);
					Result.BevelSubdivisions = Additive.BevelSubdivisions;
					Result.Type = Additive.Type;
				}

				// If the channel has not been animated at all, set it to the *current* value
				if (!Result.BevelSize.IsSet(Channel))
				{
					if (!CurrentValue.IsSet())
					{
						CurrentValue.Emplace();
						UE::MovieScene::MultiChannelFromData(InitialValueStore.RetrieveCurrentValue(), CurrentValue.GetValue());
					}

					Result.BevelSize.Set(Channel, CurrentValue.GetValue()[Channel]);
					Result.BevelSubdivisions = CurrentValue.GetValue().BevelSubdivisions;
					Result.Type = CurrentValue.GetValue().Type;
				}
			}

			ensureMsgf(Result.BevelSize.IsFull()
				, TEXT("Attempting to apply a compound data type with some channels uninitialized."));

			// Resolve the final channel data into the correct data type for the actuator
			FAvaShapeRectangleCornerSettings FinalResult;
			UE::MovieScene::ResolveChannelsToData(Result, FinalResult);
			return FinalResult;
		}
	};
	
	void BlendValue(FBlendableShapeRectCorner& OutBlend
		, const FMultiChannelShapeRectCorner& InValue
		, float Weight
		, EMovieSceneBlendType BlendType
		, TMovieSceneInitialValueStore<FAvaShapeRectangleCornerSettings>& InitialValueStore);
}

template<> FMovieSceneAnimTypeID GetBlendingDataType<FAvaShapeRectangleCornerSettings>();

template<> struct TBlendableTokenTraits<FAvaShapeRectangleCornerSettings>
{
	using WorkingDataType = UE::MovieScene::FBlendableShapeRectCorner;
};

USTRUCT()
struct FMovieSceneAvaShapeRectCornerSectionTemplate : public FMovieScenePropertySectionTemplate
{
public:
	
	GENERATED_BODY()

	FMovieSceneAvaShapeRectCornerSectionTemplate()
		: BlendType(EMovieSceneBlendType::Absolute)
	{
	}
	
	FMovieSceneAvaShapeRectCornerSectionTemplate(const class UMovieSceneAvaShapeRectCornerSection& Section
		, const UMovieScenePropertyTrack& Track);

private:
	
	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	virtual void SetupOverrides() override { EnableOverrides(RequiresSetupFlag); }
	virtual void Evaluate(const FMovieSceneEvaluationOperand& Operand
		, const FMovieSceneContext& Context
		, const FPersistentEvaluationData& PersistentData
		, FMovieSceneExecutionTokens& ExecutionTokens) const override;

	UPROPERTY()
	FMovieSceneFloatChannel BevelSize;

	UPROPERTY()
	FMovieSceneByteChannel BevelSubdivisions;

	UPROPERTY()
	FMovieSceneByteChannel Type;
	
	UPROPERTY()
	EMovieSceneBlendType BlendType;	
};
