// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequencer/MovieScene/MovieSceneAvaShapeRectCornerTrack.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "MovieSceneAvaShapeRectCornerSectionTemplate.h"
#include "Sequencer/MovieScene/MovieSceneAvaShapeRectCornerSection.h"

UMovieSceneAvaShapeRectCornerTrack::UMovieSceneAvaShapeRectCornerTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(174, 201, 255, 65);
#endif
	SupportedBlendTypes = FMovieSceneBlendTypeField::All();
}

bool UMovieSceneAvaShapeRectCornerTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneAvaShapeRectCornerSection::StaticClass();
}

UMovieSceneSection* UMovieSceneAvaShapeRectCornerTrack::CreateNewSection()
{
	return NewObject<UMovieSceneAvaShapeRectCornerSection>(this, NAME_None, RF_Transactional);
}

FMovieSceneEvalTemplatePtr UMovieSceneAvaShapeRectCornerTrack::CreateTemplateForSection(
	const UMovieSceneSection& InSection) const
{
	return FMovieSceneAvaShapeRectCornerSectionTemplate(*CastChecked<const UMovieSceneAvaShapeRectCornerSection>(&InSection)
		, *this);
}
