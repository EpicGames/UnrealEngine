// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "IAvaInteractiveToolsModeDetailsObjectProvider.h"
#include "Viewport/Interaction/IAvaSnapPointGenerator.h"
#include "AvaShapeActor.generated.h"

class UAvaShapeDynamicMeshBase;
class UDynamicMeshComponent;

UCLASS()
class AVALANCHESHAPES_API AAvaShapeActor
	: public AActor
	, public IAvaSnapPointGenerator
	, public IAvaInteractiveToolsModeDetailsObjectProvider
{
	GENERATED_BODY()

public:
	AAvaShapeActor();

	bool HasFinishedCreation() const { return bFinishedCreation; }
	void FinishCreation() { bFinishedCreation = true; }

	UAvaShapeDynamicMeshBase* GetDynamicMesh() const { return DynamicMesh; }
	void SetDynamicMesh(UAvaShapeDynamicMeshBase* NewDynamicMesh);

	UDynamicMeshComponent* GetShapeMeshComponent() const { return ShapeMeshComponent; }

#if WITH_EDITOR
	virtual FString GetDefaultActorLabel() const override;
	virtual void PostEditUndo() override;
#endif

	FAvaColorChangeData GetColorData() const;
	void SetColorData(const FAvaColorChangeData& NewColorData);

	//~ Begin UObject
	virtual void PostInitProperties() override;
	//~ End UObject

	//~ Begin IAvaSnapPointGenerator
	virtual TArray<FAvaSnapPoint> GetLocalSnapPoints() const override;
	//~ End IAvaSnapPointGenerator

	//~ Begin IAvaInteractiveToolsModeDetailsObjectProvider
	virtual UObject* GetModeDetailsObject_Implementation() const override;
	//~ End IAvaInteractiveToolsModeDetailsObjectProvider

	/** Used to trigger a refresh of the parametric material if needed through RC or blueprint */
	UFUNCTION(BlueprintCallable, CallInEditor, Category="Material")
	void RefreshParametricMaterial();

protected:
	static const FName ShapeComponentName;

	UPROPERTY()
	bool bFinishedCreation;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design")
	TObjectPtr<UAvaShapeDynamicMeshBase> DynamicMesh = nullptr;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design")
	TObjectPtr<UDynamicMeshComponent> ShapeMeshComponent = nullptr;
};
