// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaShapeParametricMaterial.h"
#include "AvaShapeUVParameters.h"
#include "AvaShapesDefs.h"
#include "MaterialBridge/AvaMaterialBridge.h"
#include "MaterialBridge/AvaMaterialContainerState.h"
#include "MaterialBridge/Slot/AvaMaterialBridgeWriteSlot.h"
#include "AvaShapeMaterialBridge.generated.h"

#define UE_API AVALANCHESHAPES_API

/** Shape mesh state data. Only the items available through FAvaShapeMeshData getters are available */
USTRUCT()
struct FAvaShapeMaterialContainerStateMesh
{
	GENERATED_BODY()

	UPROPERTY()
	int32 Index = INDEX_NONE;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> Material;

	UPROPERTY()
	EMaterialType MaterialType = EMaterialType::Default;

	UPROPERTY()
	FAvaShapeParametricMaterial ParametricMaterial;
};

/** Material container state for UAvaShapeDynamicMeshBase components */
USTRUCT()
struct FAvaShapeMaterialContainerState : public FAvaMaterialContainerState
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FAvaShapeMaterialContainerStateMesh> Meshes;
};

namespace UE::Ava
{

/** Represents a slot in Motion Design Shape Components when accessing the slot with write access */
class FShapeMaterialWriteSlot : public FMaterialBridgeWriteSlot
{
public:
	UE_API explicit FShapeMaterialWriteSlot(TNotNull<UAvaShapeDynamicMeshBase*> InShape, int32 InMeshIndex);

protected:
	//~ Begin FMaterialBridgeSlot
	UE_API virtual void OnMaterialChanged() override;
	UE_API virtual bool OnFeatureRequest(TConstStructView<FAvaMaterialBridgeFeature> InFeature) override;
	//~ End FMaterialBridgeSlot

	/** The dynamic mesh shape component owning the slot */
	TNotNull<TObjectPtr<UAvaShapeDynamicMeshBase>> Shape;

	/** The mesh index of the slot */
	int32 MeshIndex;
};

/** Material Bridge for Motion Design Shape Components */
class FShapeMaterialBridge : public FMaterialBridge
{
public:
	using FContainerState = FAvaShapeMaterialContainerState;

protected:
	//~ Begin FMaterialBridge
	UE_API virtual const UStruct* OnGetBridgedType() const override;
	UE_API virtual EControlFlow OnAccessSlots(const FReadSlotContext& InContext, TFunctionRef<EControlFlow(const FReadSlotContext&, const FReadSlot&)> InFunc, const FReadSlotOptions& InOptions) const override;
	UE_API virtual EControlFlow OnAccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const override;
	UE_API virtual TSubScriptStructOf<FAvaMaterialContainerState> OnGetContainerStateType() const override;
	UE_API virtual void OnApplyState(const FApplyStateContext& InContext, TConstStructView<FAvaMaterialContainerState> InContainerState, const FApplyStateOptions& InOptions) const override;
	UE_API virtual void OnStoreState(const FStoreStateContext& InContext, TStructView<FAvaMaterialContainerState> InContainerState, const FStoreStateOptions& InOptions) const override;
	//~ End FMaterialBridge
};

} // UE::Ava

#undef UE_API
