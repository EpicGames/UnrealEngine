// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaShapeParametricMaterial.h"
#include "AvaShapesDefs.h"
#include "AvaShapeUVParameters.h"
#include "Containers/Array.h"
#include "Materials/MaterialInterface.h"
#include "Math/MathFwd.h"
#include "AvaShapeMesh.generated.h"

/** Represents a mesh section */
USTRUCT()
struct AVALANCHESHAPES_API FAvaShapeMesh
{
	friend class UAvaShapeDynamicMeshBase;
	friend struct FAvaShapeMeshData;

	GENERATED_BODY()

	explicit FAvaShapeMesh()
		: FAvaShapeMesh(INDEX_NONE)
	{}

	explicit FAvaShapeMesh(int32 InMeshIndex)
		: MeshIndex(InMeshIndex)
		, bUpdateRequired(false)
	{}

	// clear the cached data only
	void Clear();

	// clear the mesh ids only
	void ClearIds();

	void AddTriangle(int32 A, int32 B, int32 C);

	void EnqueueTriangleIndex(int32 A);

	int32 GetMeshIndex() const
	{
		return MeshIndex;
	}

	// used to quickly clear the mesh section and avoid looping through all triangles, set automatically
	TArray<int32> TriangleIds;

	// reflects id from the dynamic mesh to quickly match with mesh ids for updates, set automatically
	TArray<int32> VerticeIds;

	// reflects id from the dynamic mesh to quickly match with mesh ids for updates, set automatically
	TArray<int32> UVIds;

	// reflects id from the dynamic mesh to quickly match with mesh ids for updates, set automatically
	TArray<int32> NormalIds;

	// reflects id from the dynamic mesh to quickly match with mesh ids for updates, set automatically
	TArray<int32> ColourIds;

	// cached version to regenerate section quickly
	TArray<FVector> Vertices;

	TArray<int32> Triangles;

	TArray<FVector> Normals;

	TArray<FVector2D> UVs;

	TArray<FLinearColor> VertexColours;

	TArray<int32> NewTriangleQueue;

protected:
	// readonly should not be changed
	UPROPERTY()
	int32 MeshIndex;

	bool bUpdateRequired;
};

/** Represents a mesh section with its material, uv data */
USTRUCT(BlueprintType)
struct FAvaShapeMeshData
{
	friend class UAvaShapeDynamicMeshBase;
	friend class FAvaMeshesDetailCustomization;
	friend class FAvaShapeDynamicMeshVisualizer;

	GENERATED_BODY()

	FAvaShapeMeshData(int32 InMeshIndex, FName InMeshName, bool InbMeshVisible)
		: bMeshVisible(InbMeshVisible)
		, bUsesPrimaryMaterialParams(InMeshIndex != 0)
		, MaterialType(EMaterialType::Parametric)
		, Material(nullptr)
		, ParametricMaterial(FAvaShapeParametricMaterial())
		, bOverridePrimaryUVParams(InMeshIndex == 0)
		, MaterialUVParams(FAvaShapeMaterialUVParameters())
		, MeshIndex(InMeshIndex)
		, MeshName(InMeshName)
		, Mesh(FAvaShapeMesh(InMeshIndex))
		, bMeshDirty(true)
	{}

	FAvaShapeMeshData()
		: FAvaShapeMeshData(0, TEXT("PRIMARY"), true)
	{}

	int32 GetMeshIndex() const
	{
		return Mesh.GetMeshIndex();
	}

	FName GetMeshName() const
	{
		return MeshName;
	}

	EMaterialType GetMaterialType() const
	{
		return MaterialType;
	}

	UMaterialInterface* GetMaterial() const
	{
		return Material;
	}

	const FAvaShapeParametricMaterial& GetParametricMaterial() const
	{
		return ParametricMaterial;
	}

protected:
	// whether the mesh is currently visible or not and should be editable
	UPROPERTY(EditDefaultsOnly, Category="Shape")
	bool bMeshVisible;

	// mesh uses same material as primary
	UPROPERTY(Transient)
	bool bUsesPrimaryMaterialParams;

	// mesh material type
	UPROPERTY(EditInstanceOnly, Category="Material", meta=(NoResetToDefault, DisplayName="Material Type", DisplayAfter="bUsePrimaryMaterialEverywhere", EditCondition="bMeshVisible && !bUsesPrimaryMaterialParams", EditConditionHides, AllowPrivateAccess = "true"))
	EMaterialType MaterialType;

	// mesh material
	UPROPERTY(EditInstanceOnly, Category="Material", meta=(NoResetToDefault, DisplayName="Material", DisplayAfter="MaterialType", EditCondition="bMeshVisible && !bUsesPrimaryMaterialParams && MaterialType != EMaterialType::Parametric", EditConditionHides, AllowPrivateAccess = "true"))
	TObjectPtr<UMaterialInterface> Material;

	// parametric material settings
	UPROPERTY(EditInstanceOnly, Category="Material", meta=(NoResetToDefault, DisplayName="Material", DisplayAfter="Material", EditCondition="bMeshVisible && !bUsesPrimaryMaterialParams && MaterialType == EMaterialType::Parametric", EditConditionHides, AllowPrivateAccess = "true"))
	FAvaShapeParametricMaterial ParametricMaterial;

	// mesh uses same uv params as primary
	UPROPERTY(EditInstanceOnly, Category="Material", meta=(NoResetToDefault, DisplayAfter="ParametricMaterial", EditCondition="bMeshVisible && MeshIndex != 0", EditConditionHides, AllowPrivateAccess = "true"))
	bool bOverridePrimaryUVParams;

	// mesh material UV params
	UPROPERTY(EditInstanceOnly, Category="Material", meta=(NoResetToDefault, DisplayName="UV Params", DisplayAfter="bOverridePrimaryUVParams", EditCondition="bMeshVisible && bOverridePrimaryUVParams", EditConditionHides, AllowPrivateAccess = "true"))
	FAvaShapeMaterialUVParameters MaterialUVParams;

	// do not change this once assigned, 0 means primary
	UPROPERTY()
	int32 MeshIndex;

	// display name of the mesh index for the widget
	UPROPERTY()
	FName MeshName;

	UPROPERTY()
	FAvaShapeMesh Mesh;

	// Flag to regenerate this mesh again once updated
	bool bMeshDirty;
};
