// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialBridge/AvaMaterialBridge.h"

#define UE_API AVALANCHESHAPES_API

namespace UE::Ava
{
/**
 * Material Bridge for shape mesh components that are managed by the shape builders 
 * Slot accessing and material container state are not implemented by design as the material slot access for these is managed by the shape builder component. 
 * FShapeMaterialBridge should be used instead.
 */
class FShapeMeshComponentMaterialBridge : public FMaterialBridge
{
protected:
	//~ Begin FMaterialBridge
	UE_API virtual const UStruct* OnGetBridgedType() const override;
	UE_API virtual bool OnIsMaterialContainerSupported(FConstDataView InMaterialContainer) const override;
	//~ End FMaterialBridge
};

} // UE::Ava

#undef UE_API
