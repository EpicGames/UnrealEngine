// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaShapesDefs.h"
#include "Math/Color.h"
#include "Math/MathFwd.h"
#include "UObject/ObjectPtr.h"
#include "AvaShapeParametricMaterial.generated.h"

class UMaterial;
class UMaterialInstance;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class UObject;
class UTexture;
struct FAvaShapeParametricMaterial;

UENUM(BlueprintType)
enum class EAvaShapeParametricMaterialTranslucency : uint8
{
	/** Switches when parameters have opacity < 1 */
	Auto,
	/** Uses opaque material regardless of opacity parameters */
	Disabled,
	/** Uses translucent material regardless of opacity parameters */
	Enabled
};

USTRUCT(BlueprintType)
struct FAvaShapeParametricMaterial
{
	GENERATED_BODY()

	enum EMaterialType : int32
	{
		Opaque = 0,
		Translucent = 1 << 0,

		Lit = 0,
		Unlit = 1 << 1,

		TwoSided = 0,
		OneSided = 1 << 2,

		MaterialTypeCount = Translucent + Unlit + OneSided + 1
	};

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMaterialChanged, const FAvaShapeParametricMaterial&)
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMaterialParameterChanged, const FAvaShapeParametricMaterial&)

	static const FName StyleParameterName;
	static const FName TextureParameterName;
	static const FName ColorAParameterName;
	static const FName ColorBParameterName;
	static const FName GradientOffsetParameterName;
	static const FName GradientRotationParameterName;
	
	/** Called when active material changed */
	static FOnMaterialChanged::RegistrationType& OnMaterialChanged();

	/** Called when active material parameters changed */
	static AVALANCHESHAPES_API FOnMaterialParameterChanged::RegistrationType& OnMaterialParameterChanged();

	AVALANCHESHAPES_API FAvaShapeParametricMaterial() = default;
	AVALANCHESHAPES_API FAvaShapeParametricMaterial(const FAvaShapeParametricMaterial& Other);
	AVALANCHESHAPES_API FAvaShapeParametricMaterial& operator=(const FAvaShapeParametricMaterial& Other);

	AVALANCHESHAPES_API bool CopyFromMaterialParameters(UMaterialInstance* InMaterial);

	/** Check if input material is a parametric material */
	bool IsParametricMaterial(UMaterialInterface* InMaterial, const bool bCheckIfDefault = false) const;

	/** Get default parent material currently active */
	AVALANCHESHAPES_API UMaterialInterface* GetDefaultMaterial() const;

	/** Get active up to date material instance */
	AVALANCHESHAPES_API UMaterialInstanceDynamic* GetMaterial() const;

	/** Get active up to date material instance or creates it */
	AVALANCHESHAPES_API UMaterialInstanceDynamic* GetOrCreateMaterial(UObject* InOuter);

	/** Set instanced material currently active */
	void SetMaterial(UMaterialInstanceDynamic* InMaterial);

	AVALANCHESHAPES_API void SetTranslucency(EAvaShapeParametricMaterialTranslucency InTranslucency);

	EAvaShapeParametricMaterialTranslucency GetTranslucency() const
	{
		return Translucency;
	}

	AVALANCHESHAPES_API bool ShouldUseTranslucentMaterial() const;

	EAvaShapeParametricMaterialStyle GetStyle() const
	{
		return Style;
	}

	AVALANCHESHAPES_API void SetStyle(EAvaShapeParametricMaterialStyle InStyle);

	const FLinearColor& GetPrimaryColor() const
	{
		return ColorA;
	}

	AVALANCHESHAPES_API void SetTexture(UTexture* InTexture);

	UTexture* GetTexture() const
	{
		return Texture;
	}

	AVALANCHESHAPES_API void SetPrimaryColor(const FLinearColor& InColor);

	const FLinearColor& GetSecondaryColor() const
	{
		return ColorB;
	}

	AVALANCHESHAPES_API void SetSecondaryColor(const FLinearColor& InColor);

	float GetGradientOffset() const
	{
		return GradientOffset;
	}

	AVALANCHESHAPES_API void SetGradientOffset(float InOffset);

	float GetGradientRotation() const
	{
		return GradientRotation;
	}

	AVALANCHESHAPES_API void SetGradientRotation(float InRotation);

	bool GetUseUnlitMaterial() const
	{
		return bUseUnlitMaterial;
	}

	AVALANCHESHAPES_API void SetUseUnlitMaterial(bool bInUse);

	bool GetUseTwoSidedMaterial() const
	{
		return bUseTwoSidedMaterial;
	}

	AVALANCHESHAPES_API void SetUseTwoSidedMaterial(bool bInUse);

	/** Set parameter values on a material instance */
	AVALANCHESHAPES_API void SetMaterialParameterValues(UMaterialInstanceDynamic* InMaterialInstance, bool bInNotifyUpdate = false) const;

	void PostEditChange(TConstArrayView<FName> InPropertyNames);

protected:
	/** Default style for the material */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Material", meta=(NoResetToDefault, DisplayName="Material Style", AllowPrivateAccess = "true"))
	EAvaShapeParametricMaterialStyle Style = EAvaShapeParametricMaterialStyle::Solid;

	/** Simple texture for the material */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Material", meta=(NoResetToDefault, DisplayName="Texture", EditCondition="Style == EAvaShapeParametricMaterialStyle::Texture", EditConditionHides, AllowPrivateAccess = "true"))
	TObjectPtr<UTexture> Texture = nullptr;

	/** Primary colour for the material */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Material", meta=(NoResetToDefault, DisplayName="Primary Color", EditCondition="Style != EAvaShapeParametricMaterialStyle::Texture", EditConditionHides, AllowPrivateAccess = "true"))
	FLinearColor ColorA = FLinearColor::White;

	/** Secondary colour for the material */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Material", meta=(NoResetToDefault, DisplayName="Secondary Color", EditCondition="Style == EAvaShapeParametricMaterialStyle::LinearGradient", EditConditionHides, AllowPrivateAccess = "true"))
	FLinearColor ColorB = FLinearColor::Black;

	/** Offset for gradient style material */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Material", meta=(NoResetToDefault, ClampMin="0", ClampMax="1", DisplayName="Gradient Offset", EditCondition = "Style == EAvaShapeParametricMaterialStyle::LinearGradient", EditConditionHides, AllowPrivateAccess = "true"))
	float GradientOffset = 0.5f;

	/** Rotation for gradient style material */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Material", meta=(NoResetToDefault, ClampMin="0", ClampMax="1", DisplayName="Gradient Rotation", EditCondition = "Style == EAvaShapeParametricMaterialStyle::LinearGradient", EditConditionHides, AllowPrivateAccess = "true"))
	float GradientRotation = 0.f;

	/** whether the material is unlit or default lit */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Material", meta=(NoResetToDefault, DisplayName="Use Unlit Material", AllowPrivateAccess = "true"))
	bool bUseUnlitMaterial = false;

	/** whether the material is one sided or two sided */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category = "Material", meta=(NoResetToDefault, DisplayName = "Use Two Sided Material", AllowPrivateAccess = "true"))
	bool bUseTwoSidedMaterial = false;

	/** How to handle translucency for the underlying material */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Material", meta=(NoResetToDefault))
	EAvaShapeParametricMaterialTranslucency Translucency = EAvaShapeParametricMaterialTranslucency::Auto;

private:
	static FOnMaterialChanged OnMaterialChangedDelegate;
	static FOnMaterialParameterChanged OnMaterialParameterChangedDelegate;

	/** Parent material
	 * 1. Opaque Lit
	 * 2. Translucent Lit
	 * 3. Opaque Unlit
	 * 4. Translucent Unlit
	 * ...
	 */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UMaterialInterface>> DefaultMaterials;

	/** Material instances corresponding to the parent material instanced
	 * 1. Opaque Lit
	 * 2. Translucent Lit
	 * 3. Opaque Unlit
	 * 4. Translucent Unlit
	 * ...
	 */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UMaterialInstanceDynamic>> InstanceMaterials;

	int32 ActiveInstanceIndex = INDEX_NONE;

	/** Load parents materials to create instance materials */
	void LoadDefaultMaterials() const;

	/** Create an instance material based on the current active parent */
	UMaterialInstanceDynamic* CreateMaterialInstance(UObject* InOuter);

	/** Update parameter values on all instance materials */
	void OnMaterialParameterUpdated();

	/** Get the active material index */
	int32 GetActiveInstanceIndex() const;
};
