// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMeshes/AvaShapeRectangleDynMesh.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "AvaShapePrimitiveFunctions.generated.h"

class AAvaShapeActor;

/** FunctionLibrary to Create Ava Shape Meshes and apply them to a Shape Actor. */
UCLASS(MinimalAPI)
class UAvaShapeMeshFunctions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Sets the Shape Actor mesh to Rectangle. */
	UFUNCTION(BlueprintCallable, Category = "Shapes", meta = (ScriptMethod))
	static AVALANCHESHAPES_API UPARAM(DisplayName = "Mesh") UAvaShapeRectangleDynamicMesh* SetRectangle(AAvaShapeActor* ShapeActor
		, const FVector2D& Size
		, const FTransform& Transform);

private:
	static UAvaShapeDynamicMeshBase* SetMesh(AAvaShapeActor* InShapeActor, const TSubclassOf<UAvaShapeDynamicMeshBase>& InMeshClass, const FTransform& InTransform = FTransform::Identity);

	template <typename InMeshType
		UE_REQUIRES(TIsDerivedFrom<InMeshType, UAvaShapeDynamicMeshBase>::Value)>
	static InMeshType* SetMesh(AAvaShapeActor* InShapeActor, const FTransform& InTransform = FTransform::Identity)
	{
		return Cast<InMeshType>(SetMesh(InShapeActor, InMeshType::StaticClass(), InTransform));
	}
};
