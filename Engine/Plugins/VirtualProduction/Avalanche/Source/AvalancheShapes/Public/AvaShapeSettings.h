// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "AvaShapeSettings.generated.h"

#define UE_API AVALANCHESHAPES_API

UCLASS(MinimalAPI, config=Engine, meta=(DisplayName="Motion Design Shapes"))
class UAvaShapeSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UAvaShapeSettings();

	/** Returns whether shape collision should be force disabled */
	UE_API bool ShouldForceDisableShapeCollision() const;

	//~ Begin UObject
	virtual void PostInitProperties() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

private:
	/**
	 * Option to disable shape collisions globally, as shapes with collisions can have a significant hit on perf.
	 * Enabled by default as Motion Design workflows most often do not require collisions.
	 * NOTE: Enabling this and saving levels with shapes will leave them in the 'No Collision' state.
	 */
	UPROPERTY(Config, EditAnywhere, Category="Motion Design")
	bool bForceDisableShapeCollision = true;

	/** Mirror atomic version of bForceDisableShapeCollision */
	std::atomic<bool> ForceDisableShapeCollision;
};

#undef UE_API
