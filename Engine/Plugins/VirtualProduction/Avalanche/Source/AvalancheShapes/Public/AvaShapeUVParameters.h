// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaShapeActor.h"
#include "AvaShapesDefs.h"
#include "AvaShapeUVParameters.generated.h"

USTRUCT(BlueprintType)
struct FAvaShapeMaterialUVParameters
{
	friend class UAvaShapeDynamicMeshBase;
	friend class FAvaShapeDynamicMeshVisualizer;

	GENERATED_BODY()

	FAvaShapeMaterialUVParameters()
		: Mode(EAvaShapeUVMode::Uniform)
		, AnchorPreset(EAvaAnchors::Center)
		, Anchor(FVector2D(0.5f, 0.5f))
		, Scale(FVector2D(1.f, 1.f))
		, Offset(FVector2D::ZeroVector)
		, Rotation(0.f)
		, bFlipHorizontal(false)
		, bFlipVertical(false)
		, bUVsDirty(false)
	{}

	EAvaShapeUVMode GetMode() const
	{
		return Mode;
	}

	EAvaAnchors GetAnchorPreset() const
	{
		return AnchorPreset;
	}

	FVector2D GetAnchor() const
	{
		return Anchor;
	}

	FVector2D GetScale() const
	{
		return Scale;
	}

	FVector2D GetOffset() const
	{
		return Offset;
	}

	float GetRotation() const
	{
		return Rotation;
	}

	bool GetFlipHorizontal() const
	{
		return bFlipHorizontal;
	}

	bool GetFlipVertical() const
	{
		return bFlipVertical;
	}

protected:
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape", meta=(NoResetToDefault, AllowPrivateAccess = "true"))
	EAvaShapeUVMode Mode;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape", meta=(NoResetToDefault, AllowPrivateAccess = "true"))
	EAvaAnchors AnchorPreset;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape", meta=(NoResetToDefault, ClampMin="0.0", ClampMax="1.0", EditCondition="AnchorPreset == EAvaAnchors::Custom", AllowPrivateAccess = "true"))
	FVector2D Anchor;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape", meta=(NoResetToDefault, ClampMin="0.01", MotionDesignVectorWidget, AllowPreserveRatio="XY", Delta="0.05", AllowPrivateAccess = "true"))
	FVector2D Scale;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape", meta=(NoResetToDefault, Delta="0.05", AllowPrivateAccess = "true"))
	FVector2D Offset;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape", meta=(NoResetToDefault, ClampMin="-360.0", ClampMax="360.0", AllowPrivateAccess = "true"))
	float Rotation;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape", meta=(NoResetToDefault, AllowPrivateAccess = "true"))
	bool bFlipHorizontal;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape", meta=(NoResetToDefault, AllowPrivateAccess = "true"))
	bool bFlipVertical;

	bool bUVsDirty;
};