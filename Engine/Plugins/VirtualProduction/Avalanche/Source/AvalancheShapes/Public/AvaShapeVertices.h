// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Misc/CoreMiscDefines.h"

struct FAvaShapeCachedVertex2D
{
	FVector2D Location;
	int32 Index = INDEX_NONE;

	FAvaShapeCachedVertex2D()
		: FAvaShapeCachedVertex2D(FVector2D::ZeroVector)
	{
	}

	FAvaShapeCachedVertex2D(const FVector2D& InLocation, int32 InIndex = INDEX_NONE)
	{
		Location = InLocation;
		Index    = InIndex;
	}

	bool operator==(const FAvaShapeCachedVertex2D& CachedVertex) const
	{
		return Location == CachedVertex.Location;
	}

	bool operator!=(const FAvaShapeCachedVertex2D& CachedVertex) const
	{
		return Location != CachedVertex.Location;
	}

	FAvaShapeCachedVertex2D& operator=(const FVector2D& InLocation)
	{
		Location = InLocation;
		Index    = INDEX_NONE;

		return *this;
	}

	FVector2D operator+(const FAvaShapeCachedVertex2D& Value) const { return Location + Value.Location; }
	FVector2D operator+(const FVector2D& Value) const { return Location + Value; }
	FVector2D operator+(const float& Value) const { return Location + Value; }
	FVector2D operator-(const FAvaShapeCachedVertex2D& Value) const { return Location - Value.Location; }
	FVector2D operator-(const FVector2D& Value) const { return Location - Value; }
	FVector2D operator-(const float& Value) const { return Location - Value; }
	FVector2D operator*(const FAvaShapeCachedVertex2D& Value) const { return Location * Value.Location; }
	FVector2D operator*(const FVector2D& Value) const { return Location * Value; }
	FVector2D operator*(const float& Value) const { return Location * Value; }
	FVector2D operator/(const FAvaShapeCachedVertex2D& Value) const { return Location / Value.Location; }
	FVector2D operator/(const FVector2D& Value) const { return Location / Value; }
	FVector2D operator/(const float& Value) const { return Location / Value; }

	operator FVector2D() const
	{
		return Location;
	}

	bool HasIndex() const { return Index != INDEX_NONE; }
	void ClearIndex() { Index = INDEX_NONE; }
};

struct FAvaShapeCachedVertex3D
{
	FVector Location;
	FVector Normal;
	FVector2D UV;
	int32 Index = INDEX_NONE;

	FAvaShapeCachedVertex3D(const FVector& InLocation, const FVector& InNormal, const FVector2D& InUV, int32 InIndex = INDEX_NONE)
	{
		Location = InLocation;
		Normal   = InNormal;
		UV       = InUV;
		Index    = InIndex;
	}

	FAvaShapeCachedVertex3D(const FAvaShapeCachedVertex2D& CachedVertex, int32 InIndex = INDEX_NONE)
	{
		Location = {0.f, CachedVertex.Location.X, CachedVertex.Location.Y};
		// All normals of 2d-shapes face the camera
		Normal = {-1.f, 0.f, 0.f};
		Index  = InIndex;
	}

	bool operator==(const FAvaShapeCachedVertex3D& CachedVertex) const
	{
		return Location == CachedVertex.Location && Normal == CachedVertex.Normal && UV == CachedVertex.UV;
	}

	bool operator!=(const FAvaShapeCachedVertex3D& CachedVertex) const
	{
		return Location != CachedVertex.Location || Normal != CachedVertex.Normal || UV != CachedVertex.UV;
	}

	FAvaShapeCachedVertex3D& operator=(const FAvaShapeCachedVertex3D& CachedVertex)
	{
		Location = CachedVertex.Location;
		Normal   = CachedVertex.Normal;
		UV       = CachedVertex.UV;
		Index    = CachedVertex.Index;

		return *this;
	}

	FAvaShapeCachedVertex3D& operator=(const FVector& InLocation)
	{
		Location = InLocation;
		// All normals of 2d-shapes face the camera
		Normal = {-1.f, 0.f, 0.f};
		Index  = INDEX_NONE;

		return *this;
	}

	FVector operator+(const FAvaShapeCachedVertex3D& Value) const { return Location + Value.Location; }
	FVector operator+(const FVector& Value) const { return Location + Value; }
	FVector operator+(const float& Value) const { return Location + Value; }
	FVector operator-(const FAvaShapeCachedVertex3D& Value) const { return Location - Value.Location; }
	FVector operator-(const FVector& Value) const { return Location - Value; }
	FVector operator-(const float& Value) const { return Location - Value; }
	FVector operator*(const FAvaShapeCachedVertex3D& Value) const { return Location * Value.Location; }
	FVector operator*(const FVector& Value) const { return Location * Value; }
	FVector operator*(const float& Value) const { return Location * Value; }
	FVector operator/(const FAvaShapeCachedVertex3D& Value) const { return Location / Value.Location; }
	FVector operator/(const FVector& Value) const { return Location / Value; }
	FVector operator/(const float& Value) const { return Location / Value; }

	operator FVector() const
	{
		return Location;
	}

	bool HasIndex() const { return Index != INDEX_NONE; }
};
