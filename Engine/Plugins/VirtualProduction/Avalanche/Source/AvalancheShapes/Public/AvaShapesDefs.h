// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "Types/SlateEnums.h"
#include "AvaShapesDefs.generated.h"

class UAvaShapeDynamicMeshBase;
class UCameraComponent;
class UMaterial;
class UMaterialInstanceDynamic;
class UMaterialInterface;
struct FAvaShapeMaterialUVParameters;

namespace UE::AvaShapes
{
	FVector2D FindClosestPointOnLine(const FVector2D& LineStart, const FVector2D& LineEnd, const FVector2D& TestPoint);

	bool AVALANCHESHAPES_API TransformMeshUVs(UE::Geometry::FDynamicMesh3& InEditMesh
		, const TArray<int32>& UVIds
		, const FAvaShapeMaterialUVParameters& InParams
		, const FVector2D& InShapeSize
		, const FVector2D& InUVOffset
		, float InUVFixRotation);
};

enum class EAvaShapeRectangleCornerTypeIndex : uint8
{
	Point = 1,
	InsetVertical = 2,
	InsetHorizontal = 3
};

UENUM()
enum class EAvaShapeUVMode : uint8
{
	// uvs are stretched to fit the shape size
	Stretch		UMETA(DisplayName="Stretch to Fit"),
	// uvs are uniform to not cause any distortion
	Uniform		UMETA(DisplayName="Preserve Ratio"),
};

UENUM()
enum class EAvaShapeCornerType : uint8
{
	Point,
	CurveIn,
	CurveOut
};

UENUM(BlueprintType)
enum class EAvaShapeParametricMaterialStyle : uint8
{
	Solid,
	LinearGradient,
	Texture
};

UENUM(BlueprintType)
enum class EMaterialType : uint8
{
	Asset = 0,
	Parametric = 1 UMETA(DisplayName = "Simple"),
	MaterialDesigner = 2,

	Default = Asset UMETA(Hidden)
};

UENUM(BlueprintType)
enum class ESizeType : uint8
{
	UnrealUnit = 0, // cm
	Pixel = 1
};

enum class EAvaDynamicMeshUpdateState : uint8
{
	/** we need to update the mesh */
	UpdateRequired,
	/** the update is in progress */
	UpdateInProgress,
	/** the mesh is now up to date with latest changes */
	UpToDate
};
