// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMeshes/AvaShape2DDynMeshBase.h"
#include "AvaShape2DArrowDynMesh.generated.h"

UCLASS(MinimalAPI, ClassGroup="Shape", BlueprintType, CustomConstructor, Within=AvaShapeActor)
class UAvaShape2DArrowDynamicMesh : public UAvaShape2DDynMeshBase
{
	GENERATED_BODY()

public:
	static const FString MeshName;

	UAvaShape2DArrowDynamicMesh()
		: UAvaShape2DArrowDynamicMesh(FVector2D(50.f, 50.f))
	{}

	UAvaShape2DArrowDynamicMesh(const FVector2D& Size2D, const FLinearColor& InVertexColor = FLinearColor::White)
		: UAvaShape2DDynMeshBase(Size2D, InVertexColor)
		, RatioArrowLine(0.5f)
		, RatioLineHeight(0.5f)
		, RatioArrowY(0.5f)
		, RatioLineY(0.5f)
		, bBothSideArrows(false)
	{}

	virtual const FString& GetMeshName() const override { return MeshName; }

	AVALANCHESHAPES_API void SetRatioArrowLine(float InRatio);
	float GetRatioArrowLine() const { return RatioArrowLine; }

	AVALANCHESHAPES_API void SetRatioLineHeight(float InRatio);
	float GetRatioLineHeight() const { return RatioLineHeight; }

	AVALANCHESHAPES_API void SetRatioArrowY(float InRatio);
	float GetRatioArrowY() const { return RatioArrowY; }

	AVALANCHESHAPES_API void SetRatioLineY(float InRatio);
	float GetRatioLineY() const { return RatioLineY; }

	AVALANCHESHAPES_API void SetBothSideArrows(bool bBothSide);
	bool IsBothSideArrows() const { return bBothSideArrows; }

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	virtual void OnRatioArrowLineChanged();
	virtual void OnRatioLineHeightChanged();
	virtual void OnBothSideArrowsChanged();
	virtual void OnRatioArrowYChanged();
	virtual void OnRatioLineYChanged();

	virtual bool IsMeshVisible(int32 MeshIndex) override;
	virtual bool CreateMesh(FAvaShapeMesh& InMesh) override;
	virtual bool CanUpdateMeshInAnyThread() const override;

	// represents the ratio for the arrow and the line, 0.6 means 60% arrow and 40% line
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape", Setter, Getter, meta=(ClampMin="0.0", ClampMax="1.0", DisplayName="Ratio Arrow Line", AllowPrivateAccess="true"))
	float RatioArrowLine;

	// represents the ratio for the line height, 1 means 100% of the height available
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape", Setter, Getter, meta=(ClampMin="0.0", ClampMax="1.0", DisplayName="Ratio Line Height", AllowPrivateAccess="true"))
	float RatioLineHeight;

	// represents the ratio for the arrow end, 0 means arrow point will be at bottom, 1 means arrow point will be at top
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape", Setter, Getter, meta=(ClampMin="0.0", ClampMax="1.0", DisplayName="Ratio Arrow Y", AllowPrivateAccess="true"))
	float RatioArrowY;

	// represents the ratio for the arrow end, 0 means arrow point will be at bottom, 1 means arrow point will be at top
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape", Setter, Getter, meta=(ClampMin="0.0", ClampMax="1.0", DisplayName="Ratio Line Y", AllowPrivateAccess="true"))
	float RatioLineY;

	// whether there should be an arrow on both side, arrows will have same ratio
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape", Setter="SetBothSideArrows", Getter="IsBothSideArrows", meta=(DisplayName="Both Side Arrows", AllowPrivateAccess="true"))
	bool bBothSideArrows;
};
