// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaShapesDefs.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "AvaShape2DDynMeshBase.generated.h"

struct FAvaShapeCachedVertex2D;

UCLASS(MinimalAPI, ClassGroup="Shape", Abstract, BlueprintType, CustomConstructor)
class UAvaShape2DDynMeshBase : public UAvaShapeDynamicMeshBase
{
	GENERATED_BODY()

	friend class UAvaShapeRoundedPolygonDynamicMesh;
	friend class FAvaShape2DDynamicMeshVisualizer;
	friend class FAvaVectorPropertyTypeCustomization;

public:
	UAvaShape2DDynMeshBase()
	: UAvaShape2DDynMeshBase(FVector2D(50.f, 50.f))
	{}

	UAvaShape2DDynMeshBase(const FVector2D& InExtent,
		const FLinearColor& InVertexColor = FLinearColor::White)
		: UAvaShapeDynamicMeshBase(InVertexColor)
		, bDoNotRecenterVertices(false)
		, Size2D(InExtent)
		, Size3D(FVector(0.f, InExtent.X, InExtent.Y))
	{}

	virtual ~UAvaShape2DDynMeshBase() override = default;

	AVALANCHESHAPES_API void SetPixelSize2D(const FVector2D& InPixelSize2D);
	const FVector2D& GetPixelSize2D() const
	{
		return PixelSize2D;
	}

	AVALANCHESHAPES_API virtual void SetSize2D(const FVector2D& InSize2D) override;
	virtual FVector2D GetSize2D() const override
	{
		return Size2D;
	}

	AVALANCHESHAPES_API virtual void SetSize3D(const FVector& InSize) override;
	virtual FVector GetSize3D() const override
	{
		return Size3D;
	}

	virtual void GetBounds(FVector& Origin, FVector& BoxExtent, FVector& Pivot) const override;

protected:
	//~ Begin UObject
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	virtual void OnRegisteredMeshes() override;
	virtual void OnPixelSizeChanged() override;
	virtual void OnScaledSizeChanged() override;
	virtual void OnSizeChanged() override;

	// For every 3rd vertex added, a new triangle is added. To add existing vertices to the queue,
	// add it via the index. This is skipped if bAddToTriangle is false.
	int32 AddVertexRaw(FAvaShapeMesh& InMesh, const FVector2D& Location, bool bForceNew = false);

	FAvaShapeCachedVertex2D AddVertexCreate(FAvaShapeMesh& InMesh, const FVector2D& Location,
		bool bForceNew = false);

	bool AddVertex(FAvaShapeMesh& InMesh, const FAvaShapeCachedVertex2D& Vertex,
		bool bForceNew = false);

	// Adds an already existing vertex to the triangle queue.
	bool AddVertex(FAvaShapeMesh& InMesh, int32 VertexIndex);

	/*
	 * Adds a vertex to the Mesh. Converted from the XY to YZ plane.
	 * X and Y should be in the range 0->Width/Height, they will be changed to be centered around 0.
	 * UVs are generated. Normals point towards the negative X direction.
	 *
	 * Vertices are checked for duplicates. This is skipped if bForceNew is true.
	 * Returns the new vertex index
	 */
	int32 CacheVertex(FAvaShapeMesh& InMesh, const FVector2D& Location, bool bForceNew = false);

	bool CacheVertex(FAvaShapeMesh& InMesh, const FAvaShapeCachedVertex2D& Vertex,
		bool bForceNew = false);

	FAvaShapeCachedVertex2D CacheVertexCreate(FAvaShapeMesh& InMesh,
		const FVector2D& Location, bool bForceNew = false);

	void AddTriangle(FAvaShapeMesh& InMesh, const FAvaShapeCachedVertex2D& A,
		const FAvaShapeCachedVertex2D& B, const FAvaShapeCachedVertex2D& C);

	void AddTriangle(FAvaShapeMesh& InMesh, int32 A, int32 B, int32 C);

	// Creates the mesh and updates the local snap points
	virtual bool CreateMesh(FAvaShapeMesh& InMesh) override;

	virtual bool CreateUVs(FAvaShapeMesh& InMesh, FAvaShapeMaterialUVParameters& InParams) override;

	UPROPERTY()
	bool bDoNotRecenterVertices;

	// pixel size of the mesh, will only be available in editor
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Transient, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", DisplayName="Pixel Size", DisplayAfter="SizeType", MotionDesignVectorWidget, AllowPreserveRatio="XY", EditCondition="bAllowEditSize && SizeType == ESizeType::Pixel", EditConditionHides, AllowPrivateAccess="true"))
	FVector2D PixelSize2D = FVector2D::ZeroVector;

	// total size in 2D from 0 to mesh size and not origin
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", DisplayName="Mesh Size", DisplayAfter="SizeType", MotionDesignVectorWidget, AllowPreserveRatio="XY", Units="Centimeters", EditCondition="bAllowEditSize && SizeType == ESizeType::UnrealUnit", EditConditionHides, AllowPrivateAccess="true"))
	FVector2D Size2D = FVector2D::ZeroVector;

	UPROPERTY(Transient)
	FVector2D PreEditSize2D = FVector2D::ZeroVector;

	// total size in 3D from 0 to mesh size and not origin, XY remapped to YZ in 3D
	UPROPERTY()
	FVector	Size3D = FVector::ZeroVector;
};
