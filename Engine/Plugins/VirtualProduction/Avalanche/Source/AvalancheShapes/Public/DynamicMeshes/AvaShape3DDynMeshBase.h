// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "AvaShape3DDynMeshBase.generated.h"

struct FAvaShapeCachedVertex3D;

UCLASS(MinimalAPI, ClassGroup="Shape", Abstract, BlueprintType, CustomConstructor)
class UAvaShape3DDynMeshBase : public UAvaShapeDynamicMeshBase
{
	GENERATED_BODY()

	friend class FAvaShape3DDynamicMeshVisualizer;
	friend class FAvaVectorPropertyTypeCustomization;

public:
	UAvaShape3DDynMeshBase()
	: UAvaShape3DDynMeshBase(FVector(50.f, 50.f, 50.f))
	{}

	UAvaShape3DDynMeshBase(const FVector& InSize, const FLinearColor& InVertexColor = FLinearColor::White)
		: UAvaShapeDynamicMeshBase(InVertexColor)
		, Size3D(InSize)
	{}

	AVALANCHESHAPES_API void SetPixelSize3D(const FVector& InPixelSize);
	const FVector& GetPixelSize3D() const
	{
		return PixelSize3D;
	}

	AVALANCHESHAPES_API virtual void SetSize3D(const FVector& InSize) override;
	virtual FVector GetSize3D() const override
	{
		return Size3D;
	}

	virtual void GetBounds(FVector& Origin, FVector& BoxExtent, FVector& Pivot) const override;

protected:
	//~ Begin UObject
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	virtual void OnRegisteredMeshes() override;
	virtual void OnPixelSizeChanged() override;
	virtual void OnSizeChanged() override;
	virtual void OnScaledSizeChanged() override;
	/*
	 * Adds a vertex to the Mesh even if it already exists, X, Y and Z should be centered around 0. Normal and UV need to be specified. Adds it to triangle queue by default
	 */
	int32 AddVertexRaw(FAvaShapeMesh& InMesh, const FVector& Location, const FVector& Normal, bool bAddToTriangle = true);
	/*
	 * Adds an already existing cached vertex by its index to the triangle queue.
	 */
	bool AddVertex(FAvaShapeMesh& InMesh, const FAvaShapeCachedVertex3D& Vertex);
	/*
	 * Adds an already existing vertex by its index to the triangle queue.
	 */
	bool AddVertex(FAvaShapeMesh& InMesh, int32 VertexIndex);
	/*
	 * Creates a cache vertex that can be reused to create another triangle, does not add the vertice to the triangle by default
	 */
	FAvaShapeCachedVertex3D CacheVertexCreate(FAvaShapeMesh& InMesh,
		const FVector& Location, const FVector& Normal, bool bAddToTriangle = false);
	/*
	 *	Adds a cache vertex that can be reused to create another triangle, does not add the vertice to the triangle by default
	 */
	bool CacheVertex(FAvaShapeMesh& InMesh, const FAvaShapeCachedVertex3D& Vertex,
		bool bAddToTriangle = false);
	/*
	 * Adds a triangle based on valid cached vertex
	 */
	void AddTriangle(FAvaShapeMesh& InMesh, const FAvaShapeCachedVertex3D& A,
		const FAvaShapeCachedVertex3D& B, const FAvaShapeCachedVertex3D& C);
	/*
	 * Adds a triangle based on valid indexes
	 */
	void AddTriangle(FAvaShapeMesh& InMesh, int32 A, int32 B, int32 C);

	// pixel size of the mesh, will only be available in editor
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Transient, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", DisplayName="Pixel Size", DisplayAfter="SizeType", MotionDesignVectorWidget, AllowPreserveRatio="XYZ", EditCondition="bAllowEditSize && SizeType == ESizeType::Pixel", EditConditionHides, AllowPrivateAccess="true"))
	FVector PixelSize3D = FVector::ZeroVector;

	/*
	 * Corresponds to the total size from 0 to mesh size
	 */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", DisplayName="Mesh Size", DisplayAfter="SizeType", MotionDesignVectorWidget, AllowPreserveRatio="XYZ", Units="Centimeters", EditCondition="bAllowEditSize && SizeType == ESizeType::UnrealUnit", EditConditionHides, AllowPrivateAccess="true"))
	FVector Size3D = FVector::ZeroVector;

	UPROPERTY(Transient)
	FVector PreEditSize3D = FVector::ZeroVector;
};
