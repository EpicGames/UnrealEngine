// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMeshes/AvaShape2DDynMeshBase.h"
#include "AvaShapeChevronDynMesh.generated.h"

UCLASS(MinimalAPI, ClassGroup="Shape", BlueprintType, CustomConstructor, Within=AvaShapeActor)
class UAvaShapeChevronDynamicMesh : public UAvaShape2DDynMeshBase
{
	GENERATED_BODY()

public:
	static const FString MeshName;

	UAvaShapeChevronDynamicMesh()
		: UAvaShapeChevronDynamicMesh(FVector2D(50.f, 50.f))
	{}

	UAvaShapeChevronDynamicMesh(
		const FVector2D& Size2D,
		const FLinearColor& InVertexColor = FLinearColor::White,
		const float InRatioChevron = 0.5)
		: UAvaShape2DDynMeshBase(Size2D, InVertexColor)
		, RatioChevron(InRatioChevron)
	{}

	virtual const FString& GetMeshName() const override
	{
		return MeshName;
	}

	AVALANCHESHAPES_API void SetRatioChevron(float InRatio);
	float GetRatioChevron() const
	{
		return RatioChevron;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	virtual void OnRatioChevronChanged();

	virtual bool IsMeshVisible(int32 MeshIndex) override;
	virtual bool CreateMesh(FAvaShapeMesh& InMesh) override;
	virtual bool CanUpdateMeshInAnyThread() const override;

	// represents the ratio for the chevron size
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", ClampMax="0.99", DisplayName="Ratio Chevron", AllowPrivateAccess="true"))
	float RatioChevron;
};
