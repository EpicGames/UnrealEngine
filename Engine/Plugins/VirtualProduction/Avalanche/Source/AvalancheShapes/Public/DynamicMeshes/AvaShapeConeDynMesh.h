// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaShape3DDynMeshBase.h"
#include "AvaShapeConeDynMesh.generated.h"

UCLASS(MinimalAPI, ClassGroup="Shape", BlueprintType, CustomConstructor, Within=AvaShapeActor)
class UAvaShapeConeDynamicMesh : public UAvaShape3DDynMeshBase
{
	GENERATED_BODY()

	friend class FAvaShapeConeDynamicMeshVisualizer;

public:
	static const FString MeshName;

	static constexpr uint8 MinNumSides = 3;
	static constexpr uint8 MaxNumSides = 128;

	static constexpr int32 MESH_INDEX_BOTTOM = 1;
	static constexpr int32 MESH_INDEX_TOP = 2;
	static constexpr int32 MESH_INDEX_START = 3;
	static constexpr int32 MESH_INDEX_END = 4;

	UAvaShapeConeDynamicMesh()
		: UAvaShapeConeDynamicMesh(FVector(50.f, 50.f, 50.f))
	{}

	UAvaShapeConeDynamicMesh(
		const FVector& InSize,
		const FLinearColor& InVertexColor = FLinearColor::White,
		float InHeight = 1,
		float InBaseRadius = 0.5,
		float InTopRadius = 0,
		uint8 InNumSides = 32,
		float InAngleDegree = 360.f,
		float InStartDegree = 0.f)
		: UAvaShape3DDynMeshBase(InSize, InVertexColor)
		, NumSides(InNumSides)
		, TopRadius(InTopRadius)
		, AngleDegree(InAngleDegree)
		, StartDegree(InStartDegree)
		, Height(InHeight)
		, BaseRadius(InBaseRadius)
	{}

	virtual const FString& GetMeshName() const override
	{
		return MeshName;
	}

	AVALANCHESHAPES_API void SetNumSides(uint8 InNumSides);
	uint8 GetNumSides() const
	{
		return NumSides;
	}

	AVALANCHESHAPES_API void SetTopRadius(float InTopRadius);
	float GetTopRadius() const
	{
		return TopRadius;
	}

	AVALANCHESHAPES_API void SetAngleDegree(float InDegree);
	float GetAngleDegree() const
	{
		return AngleDegree;
	}

	AVALANCHESHAPES_API void SetStartDegree(float InDegree);
	float GetStartDegree() const
	{
		return StartDegree;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	virtual void OnHeightChanged();
	virtual void OnBaseRadiusChanged();
	virtual void OnTopRadiusChanged();
	virtual void OnNumSidesChanged();
	virtual void OnAngleDegreeChanged();
	virtual void OnStartDegreeChanged();

	virtual bool CreateBaseUVs(FAvaShapeMesh& BaseMesh, FAvaShapeMaterialUVParameters& InUseParams);
	virtual bool GenerateBaseMeshSections(FAvaShapeMesh& BottomMesh);

	virtual bool CreateTopUVs(FAvaShapeMesh& TopMesh, FAvaShapeMaterialUVParameters& InUseParams);
	virtual bool GenerateTopMeshSections(FAvaShapeMesh& TopMesh);

	virtual bool CreateConeUVs(FAvaShapeMesh& ConeMesh, FAvaShapeMaterialUVParameters& InUseParams);
	virtual bool GenerateConeMeshSections(FAvaShapeMesh& ConeMesh);

	virtual bool CreateStartUVs(FAvaShapeMesh& StartMesh, FAvaShapeMaterialUVParameters& InUseParams);
	virtual bool GenerateStartMeshSections(FAvaShapeMesh& StartMesh);

	virtual bool CreateEndUVs(FAvaShapeMesh& EndMesh, FAvaShapeMaterialUVParameters& InUseParams);
	virtual bool GenerateEndMeshSections(FAvaShapeMesh& EndMesh);

	virtual void RegisterMeshes() override;
	virtual bool IsMeshVisible(int32 MeshIndex) override;
	virtual bool CreateMesh(FAvaShapeMesh& InMesh) override;
	virtual bool CreateUVs(FAvaShapeMesh& InMesh, FAvaShapeMaterialUVParameters& InParams) override;
	virtual bool CanUpdateMeshInAnyThread() const override;

	// The number of sides around the base of the cone
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="3.0", ClampMax="128.0", DisplayName="Sides", AllowPrivateAccess="true"))
	uint8 NumSides;

	// the ratio for the radius of the cone top
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", ClampMax="1.0", DisplayName="Top radius ratio", AllowPrivateAccess="true"))
	float TopRadius;

	// represents the base angle in degree for the cone
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", ClampMax="360.0", DisplayName="Angle degree", AllowPrivateAccess="true"))
	float AngleDegree;

	// represents the starting angle in degree for the cone
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", ClampMax="360.0", DisplayName="Start Degree", AllowPrivateAccess="true"))
	float StartDegree;

private:
	// The height of the cone from the base to the top
	UPROPERTY()
	float Height;

	// the radius of the cone base
	UPROPERTY()
	float BaseRadius;
};
