// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMeshes/AvaShape3DDynMeshBase.h"
#include "AvaShapeCubeDynMesh.generated.h"

UCLASS(MinimalAPI, ClassGroup="Shape", BlueprintType, CustomConstructor, Within=AvaShapeActor)
class UAvaShapeCubeDynamicMesh : public UAvaShape3DDynMeshBase
{
	GENERATED_BODY()

	friend class FAvaShapeCubeDynamicMeshVisualizer;

public:
	static const FString MeshName;

	static constexpr uint8 MinBevelNum = 1;
	static constexpr uint8 MaxBevelNum = 8;

	static constexpr int32 MESH_INDEX_TOP = 1;
	static constexpr int32 MESH_INDEX_BOTTOM = 2;
	static constexpr int32 MESH_INDEX_BACK = 3;
	static constexpr int32 MESH_INDEX_LEFT = 4;
	static constexpr int32 MESH_INDEX_RIGHT = 5;

	UAvaShapeCubeDynamicMesh()
		: UAvaShapeCubeDynamicMesh(FVector(50.f, 50.f, 50.f))
	{}

	UAvaShapeCubeDynamicMesh(
		const FVector& InSize,
		const FLinearColor& InVertexColor = FLinearColor::White,
		float InSegment = 1.f,
		float InBevel = 0.f,
		uint8 InBevelNum = 1)
		: UAvaShape3DDynMeshBase(InSize, InVertexColor)
		, BevelSizeRatio(InBevel)
		, BevelNum(InBevelNum)
		, Segment(InSegment)
	{}

	virtual const FString& GetMeshName() const override
	{
		return MeshName;
	}

	AVALANCHESHAPES_API void SetSegment(float InSegment);
	float GetSegment() const
	{
		return Segment;
	}

	AVALANCHESHAPES_API void SetBevelSizeRatio(float InBevel);
	float GetBevelSizeRatio() const
	{
		return BevelSizeRatio;
	}

	AVALANCHESHAPES_API void SetBevelNum(uint8 InBevel);
	uint8 GetBevelNum() const
	{
		return BevelNum;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	virtual void OnSegmentChanged();
	virtual void OnBevelSizeChanged();
	virtual void OnBevelNumChanged();

	virtual float GetMaxBevelSize() const;
	virtual FVector GetBevelSize() const;
	virtual FVector GetSegmentSize() const;
	virtual FVector GetScaleSize() const;

	virtual void CreateBevelCorner(FAvaShapeMesh& InMesh, const FVector& CornerStartLoc, const FVector& CornerEndLoc, const FVector& StartNormal,
		int32 PrevStartIdx, int32 PrevEndIdx, const FVector& BevelSize, const FVector& Scale, const FVector& CornerStartVtxLoc, const FVector& CornerEndVtxLoc);

	virtual void RegisterMeshes() override;
	virtual bool IsMeshVisible(int32 MeshIndex) override;
	virtual void OnSizeChanged() override;
	virtual bool CanUpdateMeshInAnyThread() const override;

	virtual bool GenerateBottomMeshSections(FAvaShapeMesh& BottomMesh);

	virtual bool GenerateTopMeshSections(FAvaShapeMesh& TopMesh);

	virtual bool GenerateFrontMeshSections(FAvaShapeMesh& FrontMesh);

	virtual bool GenerateBackMeshSections(FAvaShapeMesh& BackMesh);

	virtual bool GenerateLeftMeshSections(FAvaShapeMesh& LeftMesh);

	virtual bool GenerateRightMeshSections(FAvaShapeMesh& RightMesh);

	virtual bool CreateFaceUVs(FAvaShapeMesh& InMesh, FAvaShapeMaterialUVParameters& InUseParams, FRotator ProjectionDirection, FVector2D UVScale, FVector2D UVOffset);

	virtual bool CreateMesh(FAvaShapeMesh& InMesh) override;

	virtual bool CreateUVs(FAvaShapeMesh& InMesh, FAvaShapeMaterialUVParameters& InParams) override;

	// represents the bevel size applied on each face of the cube, 0 means no bevels
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", DisplayName="Bevel Size", AllowPrivateAccess="true"))
	float BevelSizeRatio;

	// represents the bevel number of division, only valid when bevel size is greater than zero
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="1.0", ClampMax="8.0", DisplayName="Bevel Num", AllowPrivateAccess="true"))
	uint8 BevelNum;

private:
	// segment size ratio to multiply with mesh size
	UPROPERTY()
	float Segment;
};
