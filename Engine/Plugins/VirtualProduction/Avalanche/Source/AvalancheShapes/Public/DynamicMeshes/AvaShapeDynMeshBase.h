// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaDefs.h"
#include "AvaShapeMesh.h"
#include "AvaShapesDefs.h"
#include "AvaShapeUVParameters.h"
#include "Components/ActorComponent.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "IAvaInteractiveToolsModeDetailsObject.h"
#include "Viewport/Interaction/AvaSnapPoint.h"
#include "Viewport/Interaction/IAvaGizmoObject.h"

#include "AvaShapeDynMeshBase.generated.h"

class AAvaShapeActor;
class FTSTicker;
class UDynamicMeshComponent;
class UStaticMesh;
struct FAvaShapeParametricMaterial;

DECLARE_MULTICAST_DELEGATE_OneParam(FMaskEnabledDelegate, AActor* /** NewMaskActor */);
DECLARE_MULTICAST_DELEGATE_OneParam(FMaskDisabledDelegate, AActor* /** OldMaskActor */);
DECLARE_MULTICAST_DELEGATE_TwoParams(FMaskVisibilityDelegate, const UWorld* /** CurrentWorld */, bool /** bMaskActorVisible */);

UCLASS(MinimalAPI, ClassGroup="Shape", Abstract, BlueprintType, CustomConstructor, EditInlineNew, DefaultToInstanced, HideCategories=(Tags, Activation, Cooking, AssetUserData, Navigation))
class UAvaShapeDynamicMeshBase
	: public UActorComponent
	, public IAvaInteractiveToolsModeDetailsObject
	, public IAvaGizmoObjectInterface
{
	GENERATED_BODY()

	friend class AAvaShapeActor;
	friend class FAvaDynMeshBaseDetailCustomization;
	friend class FAvaMeshesDetailCustomization;
	friend class FAvaShapeDynamicMeshVisualizer;
	friend class UAvaShapeBuildSystem;

public:
	static constexpr float MinSizeValue = UE_SMALL_NUMBER;
	static inline const FVector2D MinSize2D = FVector2D(MinSizeValue);
	static inline const FVector MinSize3D = FVector(MinSizeValue);

	static constexpr uint8 DefaultSubdivisions = 8;
	static constexpr uint8 MaxSubdivisions = 64;

	static constexpr int32 MESH_INDEX_NONE = -1;
	static constexpr int32 MESH_INDEX_ALL = INT_MAX;
	static constexpr int32 MESH_INDEX_PRIMARY = 0;

	static inline FMaskEnabledDelegate OnMaskEnabled;
	static inline FMaskDisabledDelegate OnMaskDisabled;
	static inline FMaskVisibilityDelegate OnMaskVisibility;

	UAvaShapeDynamicMeshBase(const FObjectInitializer& ObjectInitializer)
		: UAvaShapeDynamicMeshBase(FLinearColor::White)
	{}

	UAvaShapeDynamicMeshBase(const FLinearColor& InVertexColor = FLinearColor::White,
	                         float InUniformScaledSize = 1.f, bool bInAllowEditSize = true);

	// Converts the Outer of this object to a AAvaShapeActor.
	AVALANCHESHAPES_API AAvaShapeActor* GetShapeActor() const;

	// Gets the Dynamic Mesh component from the Shape actor, this will load it if its nullptr
	AVALANCHESHAPES_API UDynamicMeshComponent* GetShapeMeshComponent() const;

	/** Get the name of the shape */
	virtual const FString& GetMeshName() const;

	/** Returns the name of the sections composing this mesh */
	UFUNCTION(BlueprintCallable, Category="Shape")
	AVALANCHESHAPES_API TArray<FName> GetMeshSectionNames() const;

	/** Can we change the size of this shape */
	UFUNCTION(BlueprintPure, Category="Shape")
	bool GetAllowEditSize() const
	{
		return bAllowEditSize;
	}

	UFUNCTION(BlueprintCallable, Category="Shape")
	virtual void SetSize3D(const FVector& InSize) {}

	UFUNCTION(BlueprintPure, Category="Shape")
	virtual FVector GetSize3D() const
	{
		return FVector::ZeroVector;
	}

	UFUNCTION(BlueprintCallable, Category="Shape")
	AVALANCHESHAPES_API virtual void SetSize2D(const FVector2D& InSize2D);

	UFUNCTION(BlueprintPure, Category="Shape")
	AVALANCHESHAPES_API virtual FVector2D GetSize2D() const;

	/** Checks if a mesh section is currently visible */
	UFUNCTION(BlueprintPure, Category="Shape")
	bool IsMeshSectionVisible(int32 InIndex) const;

	/** Set Primary material override */
	UFUNCTION(BlueprintCallable, Category="Shape")
	AVALANCHESHAPES_API void SetUsePrimaryMaterialEverywhere(bool bInUse);

	UFUNCTION(BlueprintPure, Category="Shape")
	bool GetUsePrimaryMaterialEverywhere() const
	{
		return bUsePrimaryMaterialEverywhere;
	}

	UFUNCTION(BlueprintCallable, Category="Shape")
	AVALANCHESHAPES_API void SetUniformScaledSize(float InSize);

	UFUNCTION(BlueprintPure, Category="Shape")
	float GetUniformScaledSize() const
	{
		return UniformScaledSize;
	}

	/** Checks if the mesh section is valid */
	UFUNCTION(BlueprintPure, Category="Shape")
	AVALANCHESHAPES_API bool IsValidMeshIndex(int32 InMeshIndex) const;

	UFUNCTION(BlueprintCallable, Category="Shape")
	void SetMaterialType(int32 InMeshIndex, EMaterialType InType);

	UFUNCTION(BlueprintPure, Category="Shape")
	EMaterialType GetMaterialType(int32 InMeshIndex);

	AVALANCHESHAPES_API bool IsMaterialType(int32 MeshIndex, EMaterialType Type);

	/** Get the material of a mesh section */
	UFUNCTION(BlueprintPure, Category="Shape")
	AVALANCHESHAPES_API UMaterialInterface* GetMaterial(int32 InMeshIndex) const;

	/** Set the material of a mesh section */
	UFUNCTION(BlueprintCallable, Category="Shape")
	AVALANCHESHAPES_API void SetMaterial(int32 InMeshIndex, UMaterialInterface* InNewMaterial);

	/** Set parametric material of a section, internal materials are not copied over, only settings */
	UFUNCTION(BlueprintCallable, Category="Shape")
	AVALANCHESHAPES_API void SetParametricMaterial(int32 InMeshIndex, const FAvaShapeParametricMaterial& InNewMaterialParams);

	/** Get parametric material of a mesh section */
	UFUNCTION(BlueprintPure, Category="Shape")
	const FAvaShapeParametricMaterial& GetParametricMaterial(int32 InMeshIndex) const;

	AVALANCHESHAPES_API FAvaShapeParametricMaterial* GetParametricMaterialPtr(int32 InMeshIndex);

	/** Use custom uv params per mesh section instead of using the primary one */
	UFUNCTION(BlueprintCallable, Category="Shape")
	void SetOverridePrimaryUVParams(int32 InMeshIndex, bool bInOverride);

	UFUNCTION(BlueprintPure, Category="Shape")
	bool GetOverridePrimaryUVParams(int32 InMeshIndex) const;

	UFUNCTION(BlueprintCallable, Category="Shape")
	AVALANCHESHAPES_API void SetMaterialUVParams(int32 InMeshIndex, const FAvaShapeMaterialUVParameters& InParams);

	UFUNCTION(BlueprintPure, Category="Shape")
	AVALANCHESHAPES_API const FAvaShapeMaterialUVParameters& GetMaterialUVParams(int32 InMeshIndex) const;

	FAvaShapeMaterialUVParameters* GetMaterialUVParamsPtr(int32 MeshIndex);

	// Helper function to quickly get what you need instead of using GetMeshData()

	AVALANCHESHAPES_API EAvaShapeUVMode GetMaterialUVMode(int32 MeshIndex) const;
	AVALANCHESHAPES_API bool SetMaterialUVMode(int32 MeshIndex, EAvaShapeUVMode InUVMode);

	EAvaAnchors GetMaterialUVAnchorPreset(int32 MeshIndex) const;
	bool SetMaterialUVAnchorPreset(int32 MeshIndex, EAvaAnchors InUVAnchorPreset);

	AVALANCHESHAPES_API float GetMaterialUVRotation(int32 MeshIndex) const;
	AVALANCHESHAPES_API bool SetMaterialUVRotation(int32 MeshIndex, float InUVRotation);

	AVALANCHESHAPES_API const FVector2D& GetMaterialUVAnchor(int32 MeshIndex) const;
	AVALANCHESHAPES_API bool SetMaterialUVAnchor(int32 MeshIndex, const FVector2D& InUVAnchor);

	AVALANCHESHAPES_API const FVector2D& GetMaterialUVScale(int32 MeshIndex) const;
	AVALANCHESHAPES_API bool SetMaterialUVScale(int32 MeshIndex, const FVector2D& InUVScale);

	AVALANCHESHAPES_API const FVector2D& GetMaterialUVOffset(int32 MeshIndex) const;
	AVALANCHESHAPES_API bool SetMaterialUVOffset(int32 MeshIndex, const FVector2D& InUVOffset);

	AVALANCHESHAPES_API bool GetMaterialHorizontalFlip(int32 MeshIndex) const;
	AVALANCHESHAPES_API bool SetMaterialHorizontalFlip(int32 MeshIndex, bool InHorizontalFlip);

	AVALANCHESHAPES_API bool GetMaterialVerticalFlip(int32 MeshIndex) const;
	AVALANCHESHAPES_API bool SetMaterialVerticalFlip(int32 MeshIndex, bool InVerticalFlip);

	AVALANCHESHAPES_API const FAvaShapeMaterialUVParameters* GetInUseMaterialUVParams(int32 MeshIndex) const;
	AVALANCHESHAPES_API FAvaShapeMaterialUVParameters* GetInUseMaterialUVParams(int32 MeshIndex);

	bool HasMeshRegenWorldLocation() const { return bHasNewMeshRegenWorldLocation; }
	const FVector& GetMeshRegenWorldLocation() const { return MeshRegenWorldLocation; }
	AVALANCHESHAPES_API void SetMeshRegenWorldLocation(const FVector& NewLocation, bool bImmediateUpdate = false);

	// Generate a list of 3d-space snap points for this shape
	virtual TArray<FAvaSnapPoint> GetLocalSnapPoints() const;
	void GetLocalSnapPoints(TArray<FAvaSnapPoint>& Points) const;

	AVALANCHESHAPES_API FTransform GetTransform() const;

	/** Clear the dynamic mesh section with a specific index */
	bool ClearDynamicMeshSection(int32 MeshIndex);

	/** Clear the whole dynamic mesh */
	bool ClearDynamicMesh();

	/** Converts dynamic mesh to static mesh */
	AVALANCHESHAPES_API bool ExportToStaticMesh(UStaticMesh* DestinationMesh);

	// Gets the bounds of the shape, override this in child classes for custom bounds
	// Origin is the center of the box, BoxExtent is half the size, pivot is the default location of the pivot for this shape
	virtual void GetBounds(FVector& Origin, FVector& BoxExtent, FVector& Pivot) const {}

	virtual void OnColorPicked(const FAvaColorChangeData& InNewColorData);
	virtual FAvaColorChangeData GetActiveColor() const;

	/** Get the registered meshes indexes */
	AVALANCHESHAPES_API TSet<int32> GetMeshesIndexes() const;

	/** find a registered mesh and gets a pointer to it */
	AVALANCHESHAPES_API FAvaShapeMeshData* GetMeshData(int32 InMeshIndex);

	/** find a registered mesh and gets a pointer to it */
	const FAvaShapeMeshData* GetMeshData(int32 InMeshIndex) const;

	const TMap<int32, FAvaShapeMeshData>& GetMeshDatas() const
	{
		return MeshDatas;
	}

	// apply scale, offset, rotation on dynamic mesh section
	bool ApplyUVsTransform(FAvaShapeMesh& InMesh, FAvaShapeMaterialUVParameters& InParams, FVector2D ShapeSize, FVector2D UVFixOffset, float UVFixRotation = 0.f);

	/** Set whether mesh update should run async or not */
	AVALANCHESHAPES_API void SetRunAsync(bool bInAsync);
	bool GetRunAsync() const
	{
		return bRunAsync;
	}

	/** Updates parametric material by reapplying parameters */
	AVALANCHESHAPES_API void RefreshParametricMaterial();

protected:
	static bool SetAlignmentSize(AActor* InActor, const FVector& InSizeMultiplier);
	static EAvaAnchors GetAnchorFromNumerics(const FVector2D& AnchorNumeric);
	static FVector2D GetNumericsFromAnchor(EAvaAnchors AnchorEnum);

	//~ Begin UActorComponent
	virtual void OnRegister() override;
	virtual void OnComponentCreated() override;
	virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	//~ End UActorComponent

	//~ Begin UObject
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	virtual void PreEditUndo() override;
	virtual void PostEditUndo() override;
	virtual void PostEditImport() override;
	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	//~ End UObject

	// triggers when we drop an asset (material) onto our actor
	void OnAssetDropped(UObject* DroppedObj, AActor* TargetActor);
#endif

	// ~Begin IAvaGizmoObjectInterface
	virtual void ToggleGizmo_Implementation(const UAvaGizmoComponent* InGizmoComponent, const bool bShowAsGizmo) override;
	// ~End IAvaGizmoObjectInterface

	EAvaDynamicMeshUpdateState GetMeshUpdateStatus() const { return MeshUpdateStatus; }

	/** Checks if shape size is render-able and not minimal */
	bool IsMeshSizeValid() const;

	/** Flag all mesh sections dirty to regenerate them */
	void MarkAllMeshesDirty();

	/** Only update vertices from mesh */
	void MarkVerticesDirty();

	/** Register all meshes once, calls RegisterMeshes() */
	void SetupMeshes();

	/** override this in child classes, to register new meshes */
	virtual void RegisterMeshes() {}

	/** Call to register meshes used for this shape, call this inside RegisterMeshes() */
	bool RegisterMesh(FAvaShapeMeshData& NewMeshData);

	/** Called when a new mesh has been registered */
	virtual void OnRegisteredMesh(int32 MeshIndex) {}

	/** Called when all the meshes have been registered and the setup is done */
	virtual void OnRegisteredMeshes() {}

	/** Returns whether the mesh is dirty, called before creating mesh if visible */
	virtual bool IsMeshDirty(int32 MeshIndex);

	/** override this in child classes, called before creating mesh if visible */
	virtual bool IsMeshVisible(int32 MeshIndex);

	/** Returns the name of the mesh */
	FName GetMeshSectionName(int32 InMeshIndex) const;

	/** Returns the materials used by every mesh */
	const TArray<UMaterialInterface*> GetMeshDataMaterials();

	/** Checks to see if attributes are initialized and mesh is ready */
	bool IsDynamicMeshInitialized() const;

	/** Clears everything and reinitialize the DM to use it properly */
	void InitializeDynamicMesh();

	bool SetMaterialDirect(int32 MeshIndex, UMaterialInterface* NewMaterial);

	// called once the mesh is done updating
	virtual void OnMeshUpdateFinished();

	// called once the pixel size has changed, only in editor
	virtual void OnPixelSizeChanged() {}

	virtual void OnMeshChanged(int32 MeshIndex);
	virtual void OnMaterialChanged(int32 MaterialIndex);
	virtual void OnMaterialTypeChanged(int32 MaterialIndex);
	virtual void OnParametricMaterialChanged(int32 MaterialIndex);
	virtual void OnUVParamsChanged(int32 MeshIndex);
	virtual void OnUsesPrimaryUVParamsChanged(int32 MeshIndex);

	virtual bool OnMeshSectionUpdated(FAvaShapeMesh& InMesh);

	// override this in child classes
	virtual void OnScaledSizeChanged() {}

	// override this in child classes, for 2D and 3D shape to update the scale
	virtual void OnSizeChanged() {}

	// Clears the mesh sections that are dirty
	virtual bool ClearMesh();

	// Special case when the mesh is scaled, rather than reconfigured.
	virtual void UpdateVertices();

	// sets the uvs manually on a dynamic mesh section
	bool ApplyUVsManually(FAvaShapeMesh& InMesh);

	// apply a planar projection on a dynamic mesh section
	bool ApplyUVsPlanarProjection(FAvaShapeMesh& InMesh, FRotator PlaneRotation, FVector2D PlaneSize);

	// apply a box projection on a dynamic mesh section
	bool ApplyUVsBoxProjection(FAvaShapeMesh& InMesh, FRotator BoxRotation, FVector BoxSize);

	// override this in child classes when you create UV for a specific mesh
	virtual bool CreateUVs(FAvaShapeMesh& InMesh, FAvaShapeMaterialUVParameters& InParams);

	// override this in child classes to create the mesh if it's visible
	virtual bool CreateMesh(FAvaShapeMesh& InMesh) { return true; }

	// checks whether the topology of the mesh has changed (different vertices count and different triangles count than current)
	bool HasSameTopology(FAvaShapeMesh& InMesh);

	// Creates a section in the dynamic mesh component
	bool CreateDynamicMesh(FAvaShapeMesh& InMesh);

	// Updates a section in the dynamic mesh component
	bool UpdateDynamicMesh(FAvaShapeMesh& InMesh);

	// Invalidates a section and runs the mesh update if necessary
	void InvalidateSection(bool& bInvalidatedSection, bool bRequireUpdate = true);

	/** Flag a mesh section dirty to regenerate it */
	void MarkMeshDirty(int32 MeshIndex);

	/**
	 * Determines whether shape mesh can be done from a different thread.
	 * All built-in meshes are safe to build in a different thread. This virtual would need to be overriden for any derived class that ensures thread safety.
	 */
	virtual bool CanUpdateMeshInAnyThread() const
	{
		return false;
	}

	struct FMeshUpdateContext
	{
		/** Determines whether the update is taking place in parallel (potentially not the game thread) */
		bool bParallel = false;
	};
	/** Takes all dirty flags, regenerate whatever is needed and updates the DMC, called by RunUpdate or RequireUpdate */
	void UpdateMesh(const FMeshUpdateContext& InUpdateContext);

	/** Offsets actor to account for the mesh generated location */
	void OffsetActorToAlign();

	/** Part of update flow that can only be performed safely on the game thread. Should be called before UpdateMesh_AnyThread */
	void PreUpdateMesh();

	/** Part of update flow that can only be performed safely on the game thread. Should be called after UpdateMesh_AnyThread */
	void PostUpdateMesh();

	/** Pushed mesh section data updates to dynamic mesh and creates/updates geometry data */
	void ApplyMeshSectionChanges();

	// runs the update for this mesh, async by default
	void RequestUpdate();

	// used to check for material changes in slot of DMC to match with our current material settings
	bool CheckMaterialSlotChanges();

	// Changes the material on the shape component and keeps the MDI up to date.
	void SetShapeComponentMaterial(int32 InMaterialIndex, UMaterialInterface* InNewMaterial);

	/** Scale the vertices on all sections of the mesh */
	void ScaleVertices(const FVector& InScale);
	void ScaleVertices(const FVector2D& InScale);

	/** Is mesh size editing allowed */
	UPROPERTY(Getter="GetAllowEditSize")
	bool bAllowEditSize;

	/** the type of size you want to handle */
	UPROPERTY(EditInstanceOnly, Category="Shape", Transient, meta=(DisplayName="Size Type", HideEditConditionToggle, EditCondition="bAllowEditSize", EditConditionHides, AllowPrivateAccess="true"))
	ESizeType SizeType = ESizeType::UnrealUnit;

	/** Uniform scaled size of the mesh */
	UPROPERTY(EditInstanceOnly, Transient, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", DisplayName="Uniform Scaled Size", Units="times", AllowPrivateAccess="true"))
	float UniformScaledSize = 1.f;

	// use primary material for every slot available
	UPROPERTY(EditInstanceOnly, Setter="SetUsePrimaryMaterialEverywhere", Getter="GetUsePrimaryMaterialEverywhere", Category="Material", meta=(DisplayName="Use Single Material", AllowPrivateAccess="true"))
	bool bUsePrimaryMaterialEverywhere;

	// Meshes used for the current shape sections
	UPROPERTY(EditInstanceOnly, Category="Material", meta=(EditFixedOrder, EditFixedSize, AllowPrivateAccess="true"))
	TMap<int32, FAvaShapeMeshData> MeshDatas;

	TArray<FAvaSnapPoint> LocalSnapPoints;

private:
	/** Gets the ticker to use for shapes */
	FTSTicker& GetTicker() const;

	FAvaShapeMesh* GetMesh(int32 MeshIndex);

	/** Update render state (materials) */
	void MarkMeshRenderStateDirty() const;

	/** Generates the dirty mesh sections */
	bool GenerateMesh();

	/** Generates the uvs for the mesh (all sections) */
	bool GenerateUV();

	/** Generates the tangents for the mesh (all sections) */
	bool GenerateTangents();

	/** Generates the normals for the mesh (all sections), needs the uv set on the mesh */
	bool GenerateNormals();

	/** runs the update for this mesh only if it's required (mesh is not up to date) */
	void RunUpdate();

	/** Restore the existing saved cached mesh on the component */
	void RestoreCachedMesh(bool bInReset);

	/** Save the existing component mesh into a saved cached mesh */
	void SaveCachedMesh();

	/** Apply primary material everywhere */
	void OnUsePrimaryMaterialEverywhereChanged();

	void OnMeshTransformChanged(USceneComponent* InComponent, EUpdateTransformFlags InTransformFlags, ETeleportType InTeleport);

	/** When a parametric material is updated, eg: opaque -> translucent */
	void OnParametricMaterialChanged(const FAvaShapeParametricMaterial& InMaterial);

	/** When a parametric material parameter is updated */
	void OnParametricMaterialParameterChanged(const FAvaShapeParametricMaterial& InMaterial);

	// used to quickly restore mesh that has been modified after OnMeshUpdateFinished was called (eg extrude, masking)
	TOptional<UE::Geometry::FDynamicMesh3> CachedMesh;

	/** the context while mesh is being built */
	TOptional<FMeshUpdateContext> MeshUpdateContext;

	/** Do not access directly use getter to load it if nullptr */
	TWeakObjectPtr<UDynamicMeshComponent> CachedComponent;

	UPROPERTY()
	FVector MeshRegenWorldLocation;

#if WITH_EDITORONLY_DATA
	/** Stores select settings to allow restoration when Gizmo mode turned off */
	UPROPERTY()
	TMap<int32, FAvaShapeMeshData> NonGizmoMeshData;
#endif

	// only change this value from the base class to avoid thread lock, use RunUpdate() or RequireUpdate()
	EAvaDynamicMeshUpdateState MeshUpdateStatus = EAvaDynamicMeshUpdateState::UpToDate;

	bool bHasNewMeshRegenWorldLocation;

	// flag to register meshes only once
	bool bMeshRegistered;

	// whether we are in the tool shape mode and creating this shape
	bool bInCreateMode;

	/** true if all meshes are marked as dirty, false otherwise, use MarkAllMeshesDirty() instead of modifying this */
	bool bAllMeshDirty;

	/** true if at least one meshes is marked as dirty, false otherwise, use MarkMeshDirty(MeshIndex) instead of modifying this */
	bool bAnyMeshDirty;

	// Should only be used when the shape's vertices have changed, but the overall shape has not changed.
	bool bVerticesDirty;

	/** Is the mesh scaled to zero ? */
	bool bZeroScaled = false;

	/** Run mesh updates async */
	bool bRunAsync = true;
};
