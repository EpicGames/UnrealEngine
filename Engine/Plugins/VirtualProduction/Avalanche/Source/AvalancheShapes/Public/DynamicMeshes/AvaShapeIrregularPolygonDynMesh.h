// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaShapeRoundedPolygonDynMesh.h"
#include "DynamicMeshes/AvaShape2DDynMeshBase.h"
#include "AvaShapeIrregularPolygonDynMesh.generated.h"

USTRUCT(BlueprintType)
struct FAvaShapeRoundedCornerSettings
{
	GENERATED_BODY()

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape", meta=(ClampMin="0.0",ClampMax="1.0"))
	float BevelSize = 0.f;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape", meta=(ClampMax="64.0"))
	uint8 BevelSubdivisions = UAvaShapeDynamicMeshBase::DefaultSubdivisions;
};

USTRUCT(BlueprintType)
struct FAvaShapeRoundedCorner
{
	GENERATED_BODY()

	FAvaShapeRoundedCorner()
		: FAvaShapeRoundedCorner(FVector2D::ZeroVector)
	{}

	FAvaShapeRoundedCorner(const FVector2D& InLocation)
	{
		Location = InLocation;
		CornerMetrics = { 0.f, FVector2D::ZeroVector, FVector2D::ZeroVector,
			FVector2D::ZeroVector, true, FVector2D::ZeroVector, false };
		Settings = { 0.f, 0 };
	}

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape")
	FVector2D Location;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape", meta=(DisplayName="Corner"))
	FAvaShapeRoundedCornerSettings Settings;

	FAvaShapeRoundedCornerMetrics CornerMetrics;
};

UCLASS(MinimalAPI, ClassGroup="Shape", BlueprintType, CustomConstructor, Within=AvaShapeActor)
class UAvaShapeIrregularPolygonDynamicMesh : public UAvaShape2DDynMeshBase
{
	GENERATED_BODY()

	friend class FAvaShapeIrregularPolygonDynamicMeshVisualizer;

public:
	static const FString MeshName;
	static const float MinPointDistance;

	static bool DoLinesIntersect(const FVector2D Origin1, const FVector2D End1, const FVector2D Origin2, const FVector2D End2);

	UAvaShapeIrregularPolygonDynamicMesh(const FObjectInitializer& ObjectInitializer)
     		: UAvaShapeIrregularPolygonDynamicMesh(FLinearColor::White)
	{}

	UAvaShapeIrregularPolygonDynamicMesh(const FLinearColor& InVertexColor)
		: UAvaShape2DDynMeshBase(FVector2D::Zero(), InVertexColor)
	{
		bDoNotRecenterVertices = true;
		bAllowEditSize = false;
	}

	virtual const FString& GetMeshName() const override
	{
		return MeshName;
	}

	AVALANCHESHAPES_API void SetGlobalBevelSize(float InBevelSize);
	float GetGlobalBevelSize() const
	{
		return GlobalBevelSize;
	}

	AVALANCHESHAPES_API void SetGlobalBevelSubdivisions(uint8 InBevelSubdivisions);
	uint8 GetGlobalBevelSubdivisions() const
	{
		return GlobalBevelSubdivisions;
	}

	AVALANCHESHAPES_API void SetPoints(const TArray<FVector2D>& InPoints);
	AVALANCHESHAPES_API void SetPoints(const TArray<FAvaShapeRoundedCorner>& InPoints);
	const TArray<FAvaShapeRoundedCorner>& GetPoints() const
	{
		return Points;
	}

	const FAvaShapeRoundedCorner& GetPoint(int32 PointIdx) const
	{
		return Points[PointIdx];
	}

	int32 GetNumPoints() const
	{
		return Points.Num();
	}

	AVALANCHESHAPES_API bool CanAddPoint(const FVector2D& InPoint);
	AVALANCHESHAPES_API bool AddPoint(const FVector2D& InPoint);
	bool RemovePoint(int32 PointIdx);
	bool RemoveFirstPoint();
	bool RemoveLastPoint();
	bool RemoveAllPoints();
	AVALANCHESHAPES_API void RecalculateActorPosition();

protected:
	// Begin UObject
#if WITH_EDITOR
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	void RecalculateExtent();

	AVALANCHESHAPES_API bool SetLocation(int32 PointIdx, const FVector2D& InPoint);
	AVALANCHESHAPES_API bool SetBevelSize(int32 PointIdx, float InBevelSize);
	bool SetBevelSubdivisions(int32 PointIdx, uint8 InBevelSubdivisions);

	bool ShiftPoints(const FVector2D& Amount);

	bool DoesLineIntersectBorder(const FVector2D CheckOrigin, const FVector2D CheckEnd) const;

	// Checks to see if this point is too near to other points.
	bool IsPointTooCloseToAnotherPoint(const FVector2D& InPoint, bool bAllowExactlyOneMatch = false) const;
	bool IsPointTooCloseToALine(const FVector2D& InPoint) const;
	bool IsLineTooCloseToAPoint(const FVector2D& Start, const FVector2D& End) const;
	bool CanBeGenerated() const;

	bool IsLocationInsideShape(const FVector2D& Location);

	AVALANCHESHAPES_API void BackupPoints();

	AVALANCHESHAPES_API float GetMaxBevelSizeForPoint(int32 PointIdx) const;

	// Breaks the line, adding a new point, in between InPointIdx and InPointIdx+1
	AVALANCHESHAPES_API bool BreakSide(int32 InPointIdx);

	void RestorePoints();
	bool CheckNewPointsArray();
	void OnPointsUpdated();
	void OnLocationsUpdated();
	void OnBevelSizeChanged();
	void OnBevelSubdivisionsChanged();
	void OnGlobalBevelSizeChanged();
	void OnGlobalBevelSubdivisionsChanged();

	void OnLocationUpdated(int32 InPointIdx);
	void OnBevelSizeChanged(int32 PointIdx);
	void OnBevelSubdivisionsChanged(int32 PointIdx);

	FVector ScreenToWorld(const FVector2D& ScreenLocation) const;

	virtual bool ClearMesh() override;
	virtual bool CreateMesh(FAvaShapeMesh& InMesh) override;
	virtual bool CanUpdateMeshInAnyThread() const override;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0",ClampMax="1.0", AllowPrivateAccess="true"))
	float GlobalBevelSize = 0.f;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMax="64.0", AllowPrivateAccess="true"))
	uint8 GlobalBevelSubdivisions = DefaultSubdivisions;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(EditFixedOrder, AllowPrivateAccess="true"))
	TArray<FAvaShapeRoundedCorner> Points;
	TArray<FAvaShapeRoundedCorner> PreEditPoints;
};
