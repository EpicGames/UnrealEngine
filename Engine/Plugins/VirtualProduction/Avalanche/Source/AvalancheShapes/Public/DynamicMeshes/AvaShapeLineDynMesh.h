// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaShapeRoundedPolygonDynMesh.h"
#include "AvaShapeLineDynMesh.generated.h"

UCLASS(MinimalAPI, ClassGroup="Shape", BlueprintType, CustomConstructor, Within=AvaShapeActor)
class UAvaShapeLineDynamicMesh : public UAvaShapeRoundedPolygonDynamicMesh
{
	GENERATED_BODY()

	friend class FAvaShapeLineDynamicMeshVisualizer;

public:
	static const FString MeshName;

	UAvaShapeLineDynamicMesh()
		: UAvaShapeLineDynamicMesh(FVector2D(50.f, 1.f))
	{}

	UAvaShapeLineDynamicMesh(const FVector2D& Size2D, const FLinearColor& InVertexColor = FLinearColor::White,
		float InBevelSize = 0.f, uint8 InBevelSubdivisions = 0, float InLineWidth = 5.f)
		: UAvaShapeRoundedPolygonDynamicMesh(Size2D, InVertexColor, InBevelSize, InBevelSubdivisions)
		, LineWidth(InLineWidth)
		, Vector(FVector2D(50.f, 0.f))
	{
		bAllowEditSize = false;
	}

	virtual const FString& GetMeshName() const override
	{
		return MeshName;
	}

	AVALANCHESHAPES_API void SetLineWidth(float InLineWidth);
	float GetLineWidth() const
	{
		return LineWidth;
	}

	AVALANCHESHAPES_API void SetVector(const FVector2D& InVector);
	const FVector2D& GetVector() const
	{
		return Vector;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	void OnLineWidthChanged();
	void OnVectorChanged();

	void UpdateExtent();

	TArray<FVector2D> GenerateLineVertices() const;

	virtual bool IsMeshVisible(int32 MeshIndex) override;
	virtual void GenerateBorderVertices(TArray<FVector2D>& BorderVertices) override;
	virtual bool CanUpdateMeshInAnyThread() const override;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", AllowPrivateAccess="true"))
	float LineWidth;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(AllowPrivateAccess="true"))
	FVector2D Vector;
};
