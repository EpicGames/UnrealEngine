// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaShapeRoundedPolygonDynMesh.h"
#include "AvaShapeNGonDynMesh.generated.h"

UCLASS(MinimalAPI, ClassGroup="Shape", BlueprintType, CustomConstructor, Within=AvaShapeActor)
class UAvaShapeNGonDynamicMesh : public UAvaShapeRoundedPolygonDynamicMesh
{
	GENERATED_BODY()

	friend class FAvaShapeNGonDynamicMeshVisualizer;

public:
	static const FString MeshName;
	static inline constexpr float MinNumSides = 3.f;
	static inline constexpr float MaxNumSides = 128.f;

	UAvaShapeNGonDynamicMesh()
		: UAvaShapeNGonDynamicMesh(FVector2D(50.f, 50.f))
	{}

	UAvaShapeNGonDynamicMesh(const FVector2D& Size2D, const FLinearColor& InVertexColor = FLinearColor::White,
		float InBevelSize = 0.f, uint8 InBevelSubdivisions = 0, uint8 InNumSides = 5)
		: UAvaShapeRoundedPolygonDynamicMesh(Size2D, InVertexColor, InBevelSize, InBevelSubdivisions)
		, NumSides(InNumSides)
	{}

	virtual const FString& GetMeshName() const override
	{
		return MeshName;
	}

	AVALANCHESHAPES_API void SetNumSides(uint8 InNumSides);
	uint8 GetNumSides() const
	{
		return NumSides;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	virtual void OnNumSidesChanged();
	virtual void GenerateBorderVertices(TArray<FVector2D>& BorderVertices) override;
	virtual bool CanUpdateMeshInAnyThread() const override;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="3.0", ClampMax="128.0", DisplayName="Sides", AllowPrivateAccess="true"))
	uint8 NumSides;
};
