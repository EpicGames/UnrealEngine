// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMeshes/AvaShape2DDynMeshBase.h"
#include "AvaShapeRectangleDynMesh.generated.h"

USTRUCT(BlueprintType)
struct FAvaShapeRectangleCornerSettings
{
	GENERATED_BODY()

	bool IsBeveled() const
	{
		return Type != EAvaShapeCornerType::Point && BevelSize > 0.f;
	}

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape")
	EAvaShapeCornerType Type = EAvaShapeCornerType::Point;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape", meta=(ClampMin="0.0",DisplayName="Size"))
	float BevelSize = 0.f;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Category="Shape", meta=(ClampMax="64.0", DisplayName="Subdivisions"))
	uint8 BevelSubdivisions = UAvaShapeDynamicMeshBase::DefaultSubdivisions;

	UPROPERTY()
	FVector2D CornerPositionCache = FVector2D::ZeroVector;
};

UCLASS(MinimalAPI, ClassGroup="Shape", BlueprintType, CustomConstructor, Within=AvaShapeActor)
class UAvaShapeRectangleDynamicMesh : public UAvaShape2DDynMeshBase
{
	GENERATED_BODY()

	friend class FAvaShapeRectangleDynamicMeshVisualizer;

public:
	static const FString MeshName;
	static inline constexpr float MinSlantAngle = -45.f;
	static inline constexpr float MaxSlantAngle = 45.f;
	static inline constexpr float CornerMinMargin = 0.1f;

	UAvaShapeRectangleDynamicMesh()
		: UAvaShapeRectangleDynamicMesh(FVector2D(50.f, 50.f))
	{}

    explicit UAvaShapeRectangleDynamicMesh(const FVector2D& Size2D, const FLinearColor& InVertexColor = FLinearColor::White)
		: UAvaShape2DDynMeshBase(Size2D, InVertexColor)
	{}

	virtual const FString& GetMeshName() const override
	{
		return MeshName;
	}

	AVALANCHESHAPES_API void SetHorizontalAlignment(EAvaHorizontalAlignment InHorizontalAlignment);
	EAvaHorizontalAlignment GetHorizontalAlignment() const
	{
		return HorizontalAlignment;
	}

	AVALANCHESHAPES_API void SetVerticalAlignment(EAvaVerticalAlignment InVerticalAlignment);
	EAvaVerticalAlignment GetVerticalAlignment() const
	{
		return VerticalAlignment;
	}

	AVALANCHESHAPES_API void SetLeftSlant(float InSlant);
	float GetLeftSlant() const
	{
		return LeftSlant;
	}

	AVALANCHESHAPES_API void SetRightSlant(float InSlant);
	float GetRightSlant() const
	{
		return RightSlant;
	}

	AVALANCHESHAPES_API void SetGlobalBevelSize(float InBevelSize);
	float GetGlobalBevelSize() const
	{
		return GlobalBevelSize;
	}

	/** Controls all corners bevel subdivisions */
	AVALANCHESHAPES_API void SetGlobalBevelSubdivisions(uint8 InGlobalBevelSubdivisions);
	uint8 GetGlobalBevelSubdivisions() const
	{
		return GlobalBevelSubdivisions;
	}

	AVALANCHESHAPES_API void SetTopLeft(const FAvaShapeRectangleCornerSettings& InCornerSettings);
	const FAvaShapeRectangleCornerSettings& GetTopLeft() const
	{
		return TopLeft;
	}

	AVALANCHESHAPES_API void SetTopRight(const FAvaShapeRectangleCornerSettings& InCornerSettings);
	const FAvaShapeRectangleCornerSettings& GetTopRight() const
	{
		return TopRight;
	}

	AVALANCHESHAPES_API void SetBottomLeft(const FAvaShapeRectangleCornerSettings& InCornerSettings);
	const FAvaShapeRectangleCornerSettings& GetBottomLeft() const
	{
		return BottomLeft;
	}

	AVALANCHESHAPES_API void SetBottomRight(const FAvaShapeRectangleCornerSettings& InCornerSettings);
	const FAvaShapeRectangleCornerSettings& GetBottomRight() const
	{
		return BottomRight;
	}

	AVALANCHESHAPES_API void SetTopLeftCornerType(EAvaShapeCornerType InType);
	EAvaShapeCornerType GetTopLeftCornerType() const
	{
		return TopLeft.Type;
	}

	AVALANCHESHAPES_API void SetTopLeftBevelSize(float InSize);
	float GetTopLeftBevelSize() const
	{
		return TopLeft.BevelSize;
	}

	/** Bevel subdivisions for top left corner */
	AVALANCHESHAPES_API void SetTopLeftBevelSubdivisions(uint8 InBevelSubdivisions);
	uint8 GetTopLeftBevelSubdivisions() const
	{
		return TopLeft.BevelSubdivisions;
	}

	AVALANCHESHAPES_API void SetBottomLeftCornerType(EAvaShapeCornerType InType);
	EAvaShapeCornerType GetBottomLeftCornerType() const
	{
		return BottomLeft.Type;
	}

	AVALANCHESHAPES_API void SetBottomLeftBevelSize(float InSize);
	float GetBottomLeftBevelSize() const
	{
		return BottomLeft.BevelSize;
	}

	/** Bevel subdivisions for bottom left corner */
	AVALANCHESHAPES_API void SetBottomLeftBevelSubdivisions(uint8 InBevelSubdivisions);
	uint8 GetBottomLeftBevelSubdivisions() const
	{
		return BottomLeft.BevelSubdivisions;
	}

	AVALANCHESHAPES_API void SetTopRightCornerType(EAvaShapeCornerType InType);
	EAvaShapeCornerType GetTopRightCornerType() const
	{
		return TopRight.Type;
	}

	AVALANCHESHAPES_API void SetTopRightBevelSize(float InSize);
	float GetTopRightBevelSize() const
	{
		return TopRight.BevelSize;
	}

	/** Bevel subdivisions for top right corner */
	AVALANCHESHAPES_API void SetTopRightBevelSubdivisions(uint8 InBevelSubdivisions);
	uint8 GetTopRightBevelSubdivisions() const
	{
		return TopRight.BevelSubdivisions;
	}

	AVALANCHESHAPES_API void SetBottomRightCornerType(EAvaShapeCornerType InType);
	EAvaShapeCornerType GetBottomRightCornerType() const
	{
		return BottomRight.Type;
	}

	AVALANCHESHAPES_API void SetBottomRightBevelSize(float InSize);
	float GetBottomRightBevelSize() const
	{
		return BottomRight.BevelSize;
	}

	/** Bevel subdivisions for bottom right corner */
	AVALANCHESHAPES_API void SetBottomRightBevelSubdivisions(uint8 InBevelSubdivisions);
	uint8 GetBottomRightBevelSubdivisions() const
	{
		return BottomRight.BevelSubdivisions;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	float GetMaximumBevelSize() const;

	bool IsSlantAngleValid() const;
	FVector2D GetValidRangeLeftSlantAngle() const;
	FVector2D GetValidRangeRightSlantAngle() const;
	void GetValidSlantAngle(float& OutLeftSlant, float& OutRightSlant) const;

	void OnAlignmentChanged();

	void OnLeftSlantChanged();
	void OnRightSlantChanged();

	void OnGlobalBevelSizeChanged();
	void OnGlobalBevelSubdivisionsChanged();

	void OnTopLeftCornerTypeChanged();
	void OnTopLeftBevelSizeChanged();
	void OnTopLeftBevelSubdivisionsChanged();

	void OnBottomLeftCornerTypeChanged();
	void OnBottomLeftBevelSizeChanged();
	void OnBottomLeftBevelSubdivisionsChanged();

	void OnTopRightCornerTypeChanged();
	void OnTopRightBevelSizeChanged();
	void OnTopRightBevelSubdivisionsChanged();

	void OnBottomRightCornerTypeChanged();
	void OnBottomRightBevelSizeChanged();
	void OnBottomRightBevelSubdivisionsChanged();

	bool GenerateBaseMeshSections(FAvaShapeMesh& BaseMesh);

	virtual bool CreateMesh(FAvaShapeMesh& InMesh) override;
	virtual void OnSizeChanged() override;
	virtual bool CanUpdateMeshInAnyThread() const override;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(AllowPrivateAccess="true"))
	EAvaHorizontalAlignment HorizontalAlignment = EAvaHorizontalAlignment::Center;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(AllowPrivateAccess="true"))
	EAvaVerticalAlignment VerticalAlignment = EAvaVerticalAlignment::Center;

	/** Angle in degrees for the left slant of the rectangle */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="-45.0",ClampMax="45.0", AllowPrivateAccess="true"))
	float LeftSlant = 0.f;

	/** Angle in degrees for the right slant of the rectangle */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="-45.0",ClampMax="45.0", AllowPrivateAccess="true"))
	float RightSlant = 0.f;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", AllowPrivateAccess="true"))
	float GlobalBevelSize = 0.f;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMax="128.0", AllowPrivateAccess="true"))
	uint8 GlobalBevelSubdivisions = DefaultSubdivisions;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(AllowPrivateAccess="true"))
	FAvaShapeRectangleCornerSettings TopLeft;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(AllowPrivateAccess="true"))
	FAvaShapeRectangleCornerSettings TopRight;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(AllowPrivateAccess="true"))
	FAvaShapeRectangleCornerSettings BottomLeft;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(AllowPrivateAccess="true"))
	FAvaShapeRectangleCornerSettings BottomRight;
};
