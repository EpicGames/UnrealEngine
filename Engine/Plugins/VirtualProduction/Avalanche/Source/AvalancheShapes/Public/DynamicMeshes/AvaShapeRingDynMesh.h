// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMeshes/AvaShape2DDynMeshBase.h"
#include "AvaShapeRingDynMesh.generated.h"

UCLASS(MinimalAPI, ClassGroup="Shape", BlueprintType, CustomConstructor, Within=AvaShapeActor)
class UAvaShapeRingDynamicMesh : public UAvaShape2DDynMeshBase
{
	GENERATED_BODY()

	friend class FAvaShapeRingDynamicMeshVisualizer;

public:
	static const FString MeshName;
	static inline constexpr uint8 MinNumSides = 3;
	static inline constexpr uint8 MaxNumSides = 128;

	UAvaShapeRingDynamicMesh()
		: UAvaShapeRingDynamicMesh(FVector2D(50.f, 50.f))
	{}

	UAvaShapeRingDynamicMesh(const FVector2D& Size2D, const FLinearColor& InVertexColor = FLinearColor::White,
		uint8 InNumSides = 64, float InInnerSize = 0.8f, float InAngleDegree = 360.f, float InStartDegree = 0.f)
		: UAvaShape2DDynMeshBase(Size2D, InVertexColor)
		, NumSides(InNumSides)
		, InnerSize(InInnerSize)
		, AngleDegree(InAngleDegree)
		, StartDegree (InStartDegree)
	{
		bDoNotRecenterVertices = true;
	}

	virtual const FString& GetMeshName() const override
	{
		return MeshName;
	}

	AVALANCHESHAPES_API void SetNumSides(uint8 InNumSides);
	uint8 GetNumSides() const
	{
		return NumSides;
	}

	AVALANCHESHAPES_API void SetInnerSize(float InInnerSize);
	float GetInnerSize() const
	{
		return InnerSize;
	}

	AVALANCHESHAPES_API void SetAngleDegree(float InDegree);
	float GetAngleDegree() const
	{
		return AngleDegree;
	}

	AVALANCHESHAPES_API void SetStartDegree(float InDegree);
	float GetStartDegree() const
	{
		return StartDegree;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	virtual void OnNumSidesChanged();
	virtual void OnInnerSizeChanged();
	virtual void OnAngleDegreeChanged();
	virtual void OnStartDegreeChanged();

	virtual bool IsMeshVisible(int32 MeshIndex) override;
	virtual bool CreateMesh(FAvaShapeMesh& InMesh) override;
	virtual bool CanUpdateMeshInAnyThread() const override;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="3.0", ClampMax="128.0", DisplayName="Sides", AllowPrivateAccess="true"))
	uint8 NumSides;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", ClampMax="1.0",DisplayName="Inner Size", AllowPrivateAccess="true"))
	float InnerSize;

	// represents the angle in degree for the ring
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", ClampMax="360.0", DisplayName="Angle degree", AllowPrivateAccess="true"))
	float AngleDegree;

	// represents the starting angle in degree for the ring
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", ClampMax="360.0", DisplayName="Start Degree", AllowPrivateAccess="true"))
	float StartDegree;
};
