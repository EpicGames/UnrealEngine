// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaShapeVertices.h"
#include "DynamicMeshes/AvaShape2DDynMeshBase.h"
#include "AvaShapeRoundedPolygonDynMesh.generated.h"

struct FAvaShapeRoundedCornerMetrics
{
	float Angle;
	FAvaShapeCachedVertex2D Start;
	FAvaShapeCachedVertex2D End;
	FAvaShapeCachedVertex2D VertexAnchor;
	bool bConvex;
	FVector2D CenterOfRotation;
	uint8 BevelSubdivisions;
	bool bValid = false;
};

UCLASS(MinimalAPI, ClassGroup="Shape", Abstract, BlueprintType, CustomConstructor)
class UAvaShapeRoundedPolygonDynamicMesh : public UAvaShape2DDynMeshBase
{
	GENERATED_BODY()

	friend class FAvaShapeRoundedPolygonDynamicMeshVisualizer;

public:
	static FAvaShapeRoundedCornerMetrics CreateCornerInfo(const FVector2D& LineIn, const FVector2D LineOut,
		const FVector2D& CornerVertex, float Radius, uint8 BevelSubdivisions , bool bConvex);

	static void CreateRoundedCorner(UAvaShape2DDynMeshBase* InDynamicMesh,
		FAvaShapeMesh& InMesh, const FVector2D& CornerVertex,
		const FAvaShapeRoundedCornerMetrics& CornerInfo, bool bAddConcaveCornerInsert = true);

	UAvaShapeRoundedPolygonDynamicMesh()
		: UAvaShapeRoundedPolygonDynamicMesh(FVector2D(50.f, 50.f))
	{}

	UAvaShapeRoundedPolygonDynamicMesh(const FVector2D& Size2D,
		const FLinearColor& InVertexColor = FLinearColor::White, float InBevelSize = 0.f, uint8 InBevelSubdivisions = 0)
		: UAvaShape2DDynMeshBase(Size2D, InVertexColor)
		, BevelSize(InBevelSize)
		, BevelSubdivisions(InBevelSubdivisions)
	{}

	AVALANCHESHAPES_API void SetBevelSize(float InBevelSize);
	float GetBevelSize() const
	{
		return BevelSize;
	}

	AVALANCHESHAPES_API void SetBevelSubdivisions(uint8 InBevelSubdivisions);
	uint8 GetBevelSubdivisions() const
	{
		return BevelSubdivisions;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	virtual void OnSizeChanged() override;

	virtual void OnRoundedRadiusChanged();
	virtual void OnBevelSubdivisionsChanged();

	// Must be generated by a inheriting class. Cannot have complex shapes. Vertices cannot block each other's
	// path to the center. Example: Triangle, rectangle, pentagram.
	// Vertices must go in a clockwise direction.
	// Inheritors should also adds local snap points
	virtual void GenerateBorderVertices(TArray<FVector2D>& BorderVertices) {}

	virtual bool UseCenteredVertex() { return false; }
	virtual bool CreateMesh(FAvaShapeMesh& InMesh) override;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", AllowPrivateAccess="true"))
	float BevelSize = 0.f;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMax="64.0", AllowPrivateAccess="true"))
	uint8 BevelSubdivisions = UAvaShapeDynamicMeshBase::DefaultSubdivisions;
};
