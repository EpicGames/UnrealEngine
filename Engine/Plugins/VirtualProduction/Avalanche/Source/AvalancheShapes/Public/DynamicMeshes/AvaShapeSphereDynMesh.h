// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMeshes/AvaShape3DDynMeshBase.h"
#include "AvaShapeSphereDynMesh.generated.h"

UCLASS(MinimalAPI, ClassGroup="Shape", BlueprintType, CustomConstructor, Within=AvaShapeActor)
class UAvaShapeSphereDynamicMesh : public UAvaShape3DDynMeshBase
{
	GENERATED_BODY()

	friend class FAvaShapeSphereDynamicMeshVisualizer;

public:
	static const FString MeshName;
	static constexpr uint8 MinNumSides = 4;
	static constexpr uint8 MaxNumSides = 128;

	static constexpr int32 MESH_INDEX_TOP = 1;
	static constexpr int32 MESH_INDEX_BOTTOM = 2;
	static constexpr int32 MESH_INDEX_START = 3;
	static constexpr int32 MESH_INDEX_END = 4;

	UAvaShapeSphereDynamicMesh()
		: UAvaShapeSphereDynamicMesh(FVector(50.f, 50.f, 50.f))
	{}

	UAvaShapeSphereDynamicMesh(
		const FVector& InSize,
		const FLinearColor& InVertexColor = FLinearColor::White,
		float InStartLongitude = 0.f,
		float InEndLongitude = 180.f,
		float InStartLatitude = 0.f,
		float InLatitudeDegree = 360.f,
		uint8 InNumSides = 32,
		float InRadius = 1.f)
		: UAvaShape3DDynMeshBase(InSize, InVertexColor)
		, StartLongitude(InStartLongitude)
		, PreEditStartLongitude(0)
		, EndLongitude(InEndLongitude)
		, PreEditEndLongitude(0)
		, StartLatitude(InStartLatitude)
		, LatitudeDegree(InLatitudeDegree)
		, NumSides(InNumSides)
		, Radius(InRadius)
	{}

	virtual const FString& GetMeshName() const override
	{
		return MeshName;
	}

	AVALANCHESHAPES_API void SetNumSides(uint8 InNumSides);
	uint8 GetNumSides() const
	{
		return NumSides;
	}

	AVALANCHESHAPES_API void SetStartLatitude(float InDegree);
	float GetStartLatitude() const
	{
		return StartLatitude;
	}

	AVALANCHESHAPES_API void SetLatitudeDegree(float InDegree);
	float GetLatitudeDegree() const
	{
		return LatitudeDegree;
	}

	AVALANCHESHAPES_API void SetStartLongitude(float InDegree);
	float GetStartLongitude() const
	{
		return StartLongitude;
	}

	AVALANCHESHAPES_API void SetEndLongitude(float InDegree);
	float GetEndLongitude() const
	{
		return EndLongitude;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	virtual void OnNumSidesChanged();
	virtual void OnRadiusChanged();
	virtual void OnStartLatitudeChanged();
	virtual void OnLatitudeDegreeChanged();
	virtual void OnStartLongitudeChanged();
	virtual void OnEndLongitudeChanged();

	virtual bool CreateBaseUVs(FAvaShapeMesh& BaseMesh, FAvaShapeMaterialUVParameters& InUseParams);
	virtual bool GenerateBaseMeshSections(FAvaShapeMesh& BaseMesh);

	virtual bool CreateTopUVs(FAvaShapeMesh& TopMesh, FAvaShapeMaterialUVParameters& InUseParams);
	virtual bool GenerateTopMeshSections(FAvaShapeMesh& TopMesh);

	virtual bool CreateBottomUVs(FAvaShapeMesh& BottomMesh, FAvaShapeMaterialUVParameters& InUseParams);
	virtual bool GenerateBottomMeshSections(FAvaShapeMesh& BottomMesh);

	virtual bool CreateStartUVs(FAvaShapeMesh& StartMesh, FAvaShapeMaterialUVParameters& InUseParams);
	virtual bool GenerateStartMeshSections(FAvaShapeMesh& StartMesh);

	virtual bool CreateEndUVs(FAvaShapeMesh& EndMesh, FAvaShapeMaterialUVParameters& InUseParams);
	virtual bool GenerateEndMeshSections(FAvaShapeMesh& EndMesh);

	virtual void RegisterMeshes() override;
	virtual bool IsMeshVisible(int32 MeshIndex) override;
	virtual bool CreateMesh(FAvaShapeMesh& InMesh) override;
	virtual bool CreateUVs(FAvaShapeMesh& InMesh, FAvaShapeMaterialUVParameters& InParams) override;
	virtual bool CanUpdateMeshInAnyThread() const override;

	// represents the longitude (Z) angle in degree for the sphere at the start
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", ClampMax="180.0", DisplayName="Start Longitude degree", AllowPrivateAccess="true"))
	float StartLongitude;
	float PreEditStartLongitude;

	// represents the longitude (Z) angle in degree for the sphere at the end
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", ClampMax="180.0", DisplayName="End Longitude degree", AllowPrivateAccess="true"))
	float EndLongitude;
	float PreEditEndLongitude;

	// represents the latitude (Y) angle in degree for the sphere at the start
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", ClampMax="360.0", DisplayName="Start Latitude degree", AllowPrivateAccess="true"))
	float StartLatitude;

	// represents the total latitude (Y) angle in degree for the sphere
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", ClampMax="360.0", DisplayName="Latitude degree", AllowPrivateAccess="true"))
	float LatitudeDegree;

	// represents the precision of the sphere mesh
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="4.0", ClampMax="128.0", DisplayName="Sides", AllowPrivateAccess="true"))
	uint8 NumSides;

private:
	// represents the radius ratio of the sphere
	UPROPERTY()
	float Radius;
};
