// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaShapeRoundedPolygonDynMesh.h"
#include "AvaShapeStarDynMesh.generated.h"

UCLASS(MinimalAPI, ClassGroup="Shape", BlueprintType, CustomConstructor, Within=AvaShapeActor)
class UAvaShapeStarDynamicMesh : public UAvaShapeRoundedPolygonDynamicMesh
{
	GENERATED_BODY()

	friend class FAvaShapeStarDynamicMeshVisualizer;

public:
	static const FString MeshName;
	static inline constexpr uint8 MinNumPoints = 2;
	static inline constexpr uint8 MaxNumPoints = 128;

	UAvaShapeStarDynamicMesh()
		: UAvaShapeStarDynamicMesh(FVector2D(50.f, 50.f))
	{}

	UAvaShapeStarDynamicMesh(const FVector2D& Size2D, const FLinearColor& InVertexColor = FLinearColor::White,
		float InBevelSize = 0.f, uint8 InBevelSubdivisions = 0, uint8 InNumPoints = 5, float InInnerSize = 0.5f)
		: UAvaShapeRoundedPolygonDynamicMesh(Size2D, InVertexColor, InBevelSize, InBevelSubdivisions)
		, NumPoints(InNumPoints)
		, InnerSize(InInnerSize)
	{}

	virtual const FString& GetMeshName() const override
	{
		return MeshName;
	}

	AVALANCHESHAPES_API void SetNumPoints(uint8 InNumPoints);
	uint8 GetNumPoints() const
	{
		return NumPoints;
	}

	AVALANCHESHAPES_API void SetInnerSize(float InInnerSize);
	float GetInnerSize() const
	{
		return InnerSize;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	virtual void OnNumSidesChanged();
	virtual void OnInnerSizeChanged();

	virtual void GenerateBorderVertices(TArray<FVector2D>& BorderVertices) override;

	virtual bool UseCenteredVertex() override { return true; }
	virtual bool IsMeshVisible(int32 MeshIndex) override;
	virtual bool CanUpdateMeshInAnyThread() const override;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="2.0", ClampMax="128.0", DisplayName="Points", AllowPrivateAccess="true"))
	uint8 NumPoints = 0;

	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", ClampMax="0.99", DisplayName="Inner Size Fraction", AllowPrivateAccess="true"))
	float InnerSize = 0.f;
};
