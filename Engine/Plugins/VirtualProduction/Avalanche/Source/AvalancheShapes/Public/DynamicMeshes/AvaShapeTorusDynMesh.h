// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaShape3DDynMeshBase.h"
#include "AvaShapeTorusDynMesh.generated.h"

UCLASS(MinimalAPI, ClassGroup="Shape", BlueprintType, CustomConstructor, Within=AvaShapeActor)
class UAvaShapeTorusDynamicMesh : public UAvaShape3DDynMeshBase
{
	GENERATED_BODY()

	friend class FAvaShapeTorusDynamicMeshVisualizer;

public:
	static const FString MeshName;
	static constexpr uint8 MinNumSlices = 3;
	static constexpr uint8 MaxNumSlices = 128;
	static constexpr uint8 MinNumSides = 4;
	static constexpr uint8 MaxNumSides = 128;

	static constexpr int32 MESH_INDEX_START = 1;
	static constexpr int32 MESH_INDEX_END = 2;

	UAvaShapeTorusDynamicMesh()
		: UAvaShapeTorusDynamicMesh(FVector(50.f, 50.f, 50.f))
	{}

	UAvaShapeTorusDynamicMesh(
		const FVector& InSize,
		const FLinearColor& InVertexColor = FLinearColor::White,
		uint8 InNumSlices = 32,
		uint8 InNumSides = 32,
		float InInnerSize = 0.75,
		float InAngleDegree = 360.f,
		float InStartDegree = 0.f)
		: UAvaShape3DDynMeshBase(InSize, InVertexColor)
		, NumSlices(InNumSlices)
		, NumSides(InNumSides)
		, InnerSize(InInnerSize)
		, AngleDegree(InAngleDegree)
		, StartDegree(InStartDegree)
	{}

	virtual const FString& GetMeshName() const override
	{
		return MeshName;
	}

	AVALANCHESHAPES_API void SetNumSides(uint8 InNumSides);
	uint8 GetNumSides() const
	{
		return NumSides;
	}

	AVALANCHESHAPES_API void SetNumSlices(uint8 InNumSlices);
	uint8 GetNumSlices() const
	{
		return NumSlices;
	}

	AVALANCHESHAPES_API void SetInnerSize(float InInnerSize);
	float GetInnerSize() const
	{
		return InnerSize;
	}

	AVALANCHESHAPES_API void SetAngleDegree(float InAngleDegree);
	float GetAngleDegree() const
	{
		return AngleDegree;
	}

	AVALANCHESHAPES_API void SetStartDegree(float InStartDegree);
	float GetStartDegree() const
	{
		return StartDegree;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	virtual bool CreateBaseUVs(FAvaShapeMesh& BaseMesh, FAvaShapeMaterialUVParameters& InUseParams);
	virtual bool GenerateBaseMeshSections(FAvaShapeMesh& BaseMesh);

	virtual bool CreateStartUVs(FAvaShapeMesh& StartMesh, FAvaShapeMaterialUVParameters& InUseParams);
	virtual bool GenerateStartMeshSections(FAvaShapeMesh& StartMesh);

	virtual bool CreateEndUVs(FAvaShapeMesh& EndMesh, FAvaShapeMaterialUVParameters& InUseParams);
	virtual bool GenerateEndMeshSections(FAvaShapeMesh& EndMesh);

	virtual void OnNumSidesChanged();
	virtual void OnNumSlicesChanged();
	virtual void OnInnerSizeChanged();
	virtual void OnAngleDegreeChanged();
	virtual void OnStartDegreeChanged();

	virtual void RegisterMeshes() override;
	virtual bool IsMeshVisible(int32 MeshIndex) override;
	virtual bool CreateMesh(FAvaShapeMesh& InMesh) override;
	virtual bool CreateUVs(FAvaShapeMesh& InMesh, FAvaShapeMaterialUVParameters& InParams) override;
	virtual bool CanUpdateMeshInAnyThread() const override;

	// represents the number of slices composing the tube
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="3.0", ClampMax="128.0", DisplayName="Slices", AllowPrivateAccess="true"))
	uint8 NumSlices;

	// represents the precision of each circle composing a slice
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="4.0", ClampMax="128.0", DisplayName="Sides", AllowPrivateAccess="true"))
	uint8 NumSides;

	// represents the size ratio available inside the torus
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.5", ClampMax="0.99", DisplayName="Inner Size", AllowPrivateAccess="true"))
	float InnerSize;

	// represents the tube angle in degree for the torus
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", ClampMax="360.0", DisplayName="Angle degree", AllowPrivateAccess="true"))
	float AngleDegree;

	// represents the starting angle in degree for the torus
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter, Getter, Category="Shape", meta=(ClampMin="0.0", ClampMax="360.0", DisplayName="Start Degree", AllowPrivateAccess="true"))
	float StartDegree;
};
