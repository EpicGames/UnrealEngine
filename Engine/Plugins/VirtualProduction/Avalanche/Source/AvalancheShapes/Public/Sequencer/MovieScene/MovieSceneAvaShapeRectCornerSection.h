// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "MovieSceneSection.h"
#include "DynamicMeshes/AvaShapeRectangleDynMesh.h"
#include "MovieSceneAvaShapeRectCornerSection.generated.h"

UCLASS(MinimalAPI)
class UMovieSceneAvaShapeRectCornerSection : public UMovieSceneSection
{
	GENERATED_BODY()

public:
	UMovieSceneAvaShapeRectCornerSection(const FObjectInitializer& ObjectInitializer);

	UPROPERTY()
	FMovieSceneByteChannel Type;

	UPROPERTY()
	FMovieSceneFloatChannel BevelSize;

	UPROPERTY()
	FMovieSceneByteChannel BevelSubdivisions;
};
