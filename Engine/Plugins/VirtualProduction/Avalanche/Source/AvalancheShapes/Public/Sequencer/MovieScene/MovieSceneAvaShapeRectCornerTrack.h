// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "MovieSceneAvaShapeRectCornerTrack.generated.h"

/**
* Track for animating FAvaShapeRectangleCornerSettings properties
*/
UCLASS(MinimalAPI)
class UMovieSceneAvaShapeRectCornerTrack
	: public UMovieScenePropertyTrack
	, public IMovieSceneTrackTemplateProducer
{
	GENERATED_BODY()

public:
	UMovieSceneAvaShapeRectCornerTrack(const FObjectInitializer& ObjectInitializer);

	// UMovieSceneTrack Interface
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;
	//~UMovieSceneTrack Interface
};
