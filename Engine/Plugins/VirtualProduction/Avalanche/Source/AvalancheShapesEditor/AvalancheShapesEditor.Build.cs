// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheShapesEditor : ModuleRules
{
	public AvalancheShapesEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Avalanche",
				"AvalancheComponentVisualizers",
				"AvalancheEditorCore",
				"AvalancheInteractiveTools",
				"AvalancheLevelViewport",
				"AvalancheShapes",
				"AvalancheViewport",
				"ComponentVisualizers",
				"CoreUObject",
				"DeveloperSettings",
				"EditorFramework",
				"Engine",
				"GeometryCore",
				"GeometryFramework",
				"InputCore",
				"InteractiveToolsFramework",
				"MeshConversion",
				"MovieScene",
				"MovieSceneTools",
				"MovieSceneTracks",
				"Projects",
				"Sequencer",
				"Slate",
				"SlateCore",
				"UnrealEd",
				"WidgetRegistration"
			}
		);
	}
}
