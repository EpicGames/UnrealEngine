// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeFactory.h"
#include "AvaShapeActor.h"
#include "DynamicMeshes/AvaShape2DDynMeshBase.h"
#include "DynamicMeshes/AvaShape3DDynMeshBase.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "EngineAnalytics.h"
#include "Subsystems/PlacementSubsystem.h"
#include "Tools/AvaShapesEditorShapeToolBase.h"

UAvaShapeFactory::UAvaShapeFactory()
{
	NewActorClass = AAvaShapeActor::StaticClass();
	MeshSize = UAvaShapesEditorShapeToolBase::DefaultParameters.Size;
}

void UAvaShapeFactory::SetMeshClass(TSubclassOf<UAvaShapeDynamicMeshBase> InMeshClass)
{
	MeshClass = InMeshClass;
}

void UAvaShapeFactory::SetMeshSize(const FVector& InMeshSize)
{
	MeshSize = InMeshSize;
}

void UAvaShapeFactory::SetMeshFunction(TFunction<void(UAvaShapeDynamicMeshBase*)> InFunction)
{
	MeshFunction = InFunction;
}

void UAvaShapeFactory::SetMeshNameOverride(const TOptional<FString>& InMeshNameOverride)
{
	MeshNameOverride = InMeshNameOverride;
}

bool UAvaShapeFactory::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (!MeshClass)
	{
		return false;
	}

	if (const UClass* AssetClass = AssetData.GetClass(EResolveClass::Yes))
	{
		return AssetClass->IsChildOf(NewActorClass);
	}

	return false;
}

AActor* UAvaShapeFactory::GetDefaultActor(const FAssetData& AssetData)
{
	return Cast<AActor>(AAvaShapeActor::StaticClass()->GetDefaultObject());
}

AActor* UAvaShapeFactory::SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams)
{
	UWorld* World = InLevel->GetWorld();

	if (!World)
	{
		return nullptr;
	}

	FActorSpawnParameters Params;
	Params.OverrideLevel = InLevel;
	Params.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	Params.bNoFail = true;
	Params.ObjectFlags = RF_Transactional;

	AAvaShapeActor* ShapeActor = World->SpawnActor<AAvaShapeActor>(AAvaShapeActor::StaticClass(), InTransform, Params);

	UAvaShapeDynamicMeshBase* MeshBase = NewObject<UAvaShapeDynamicMeshBase>(ShapeActor, MeshClass.Get());
	MeshBase->SetSize3D(MeshSize);

	const FAvaShapeParametricMaterial DefaultParametricMaterial;
	MeshBase->SetParametricMaterial(UAvaShapeDynamicMeshBase::MESH_INDEX_PRIMARY, DefaultParametricMaterial);

	if (MeshFunction.IsSet())
	{
		MeshFunction(MeshBase);
	}

	ShapeActor->SetDynamicMesh(MeshBase);

	return ShapeActor;
}

void UAvaShapeFactory::PostSpawnActor(UObject* InAsset, AActor* InNewActor)
{
	Super::PostSpawnActor(InAsset, InNewActor);

	if (AAvaShapeActor* NewShapeActor = Cast<AAvaShapeActor>(InNewActor))
	{
		FActorLabelUtilities::RenameExistingActor(NewShapeActor, MeshNameOverride.Get(NewShapeActor->GetDefaultActorLabel()), /** Unique*/true);
	}
}

void UAvaShapeFactory::PostPlaceAsset(TArrayView<const FTypedElementHandle> InHandle, const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions)
{
	Super::PostPlaceAsset(InHandle, InPlacementInfo, InPlacementOptions);

	if (!InPlacementOptions.bIsCreatingPreviewElements && FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Reserve(3);
		Attributes.Emplace(TEXT("ToolClass"), GetNameSafe(GetClass()));
		Attributes.Emplace(TEXT("ActorClass"), GetNameSafe(NewActorClass));
		Attributes.Emplace(TEXT("SubobjectClass"), GetNameSafe(MeshClass));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MotionDesign.PlaceActor"), Attributes);
	}
}

FString UAvaShapeFactory::GetDefaultActorLabel(UObject* InAsset) const
{
	if (const UClass* AssetClass = Cast<UClass>(InAsset))
	{
		if (const AAvaShapeActor* CDO = AssetClass->GetDefaultObject<AAvaShapeActor>())
		{
			return MeshNameOverride.Get(CDO->GetDefaultActorLabel());
		}
	}

	return Super::GetDefaultActorLabel(InAsset);
}
