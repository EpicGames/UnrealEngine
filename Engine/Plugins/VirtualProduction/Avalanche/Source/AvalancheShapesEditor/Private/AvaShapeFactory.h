// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "Misc/Optional.h"
#include "AvaShapeFactory.generated.h"

class UAvaShapeDynamicMeshBase;

UCLASS()
class UAvaShapeFactory : public UActorFactory
{
	GENERATED_BODY()

public:
	UAvaShapeFactory();

	void SetMeshClass(TSubclassOf<UAvaShapeDynamicMeshBase> InMeshClass);
	void SetMeshSize(const FVector& InMeshSize);
	void SetMeshFunction(TFunction<void(UAvaShapeDynamicMeshBase*)> InFunction);
	void SetMeshNameOverride(const TOptional<FString>& InMeshNameOverride);

protected:
	//~ Begin UActorFactory
	virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	virtual AActor* GetDefaultActor(const FAssetData& AssetData) override;
	virtual AActor* SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams) override;
	virtual void PostSpawnActor(UObject* InAsset, AActor* InNewActor) override;
	virtual void PostPlaceAsset(TArrayView<const FTypedElementHandle> InHandle, const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions) override;
	virtual FString GetDefaultActorLabel(UObject* InAsset) const override;
	//~ End UActorFactory

	TSubclassOf<UAvaShapeDynamicMeshBase> MeshClass;
	FVector MeshSize = FVector::ZeroVector;
	TFunction<void(UAvaShapeDynamicMeshBase*)> MeshFunction;
	TOptional<FString> MeshNameOverride;
};
