// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapesEditorCommands.h"
#include "AvaShapesEditorStyle.h"

#define LOCTEXT_NAMESPACE "AvaShapesEditorCommands"

FAvaShapesEditorCommands::FAvaShapesEditorCommands()
	: TCommands<FAvaShapesEditorCommands>(
		TEXT("AvaShapesEditor")
		, LOCTEXT("MotionDesignShapesEditor", "Motion Design Shapes Editor")
		, NAME_None
		, FAvaShapesEditorStyle::Get().GetStyleSetName()
	)
{
}

void FAvaShapesEditorCommands::RegisterCommands()
{
	Register2DCommands();
	Register3DCommands();
}

void FAvaShapesEditorCommands::Register2DCommands()
{
	UI_COMMAND(Tool_Shape_2DArrow
		, "Arrow"
		, "Create a 2D Arrow Shape in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Tool_Shape_Chevron
		, "Chevron"
		, "Create a 2D Chevron Shape in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Tool_Shape_Ellipse
		, "Ellipse"
		, "Create an 2D Ellipse Shape in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Tool_Shape_IrregularPoly
		, "Freehand Polygon"
		, "Create a 2D Freehand Polygon Shape in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Tool_Shape_Line
		, "Line"
		, "Create a 2D Line Shape in the viewport.\n\n- Holding shift will snap the line to 15 degree increments."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Tool_Shape_NGon
		, "Regular Polygon"
		, "Create a 2D Regular Polygon Shape in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Tool_Shape_Rectangle
		, "Rectangle"
		, "Create a 2D Rectangle Shape in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Tool_Shape_Square
		, "Square"
		, "Create a 2D Square Shape in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Tool_Shape_Ring
		, "Ring"
		, "Create a 2D Ring Shape in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Tool_Shape_Star
		, "Star"
		, "Create a 2D Star Shape in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());
}

void FAvaShapesEditorCommands::Register3DCommands()
{
	UI_COMMAND(Tool_Shape_Cone
		, "Cone"
		, "Create a Cone Shape in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Tool_Shape_Cylinder
		, "Cylinder"
		, "Create a Cylinder Shape in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Tool_Shape_Cube
		, "Cube"
		, "Create a Cube Shape in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Tool_Shape_Sphere
		, "Sphere"
		, "Create a Sphere Shape in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Tool_Shape_Torus
		, "Torus"
		, "Create a Torus Shape in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());
}

#undef LOCTEXT_NAMESPACE
