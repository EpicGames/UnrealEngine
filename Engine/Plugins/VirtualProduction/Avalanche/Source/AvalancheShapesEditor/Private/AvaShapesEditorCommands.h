// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FAvaShapesEditorCommands : public TCommands<FAvaShapesEditorCommands>
{
public:
	FAvaShapesEditorCommands();

	//~ Begin TCommands
	virtual void RegisterCommands() override;
	//~ End TCommands

	void Register2DCommands();
	void Register3DCommands();

	// 2D Tools
	TSharedPtr<FUICommandInfo> Tool_Shape_2DArrow;
	TSharedPtr<FUICommandInfo> Tool_Shape_Chevron;
	TSharedPtr<FUICommandInfo> Tool_Shape_Ellipse;
	TSharedPtr<FUICommandInfo> Tool_Shape_IrregularPoly;
	TSharedPtr<FUICommandInfo> Tool_Shape_Line;
	TSharedPtr<FUICommandInfo> Tool_Shape_NGon;
	TSharedPtr<FUICommandInfo> Tool_Shape_Rectangle;
	TSharedPtr<FUICommandInfo> Tool_Shape_Square;
	TSharedPtr<FUICommandInfo> Tool_Shape_Ring;
	TSharedPtr<FUICommandInfo> Tool_Shape_Star;

	// 3D Tools
	TSharedPtr<FUICommandInfo> Tool_Shape_Cone;
	TSharedPtr<FUICommandInfo> Tool_Shape_Cylinder;
	TSharedPtr<FUICommandInfo> Tool_Shape_Cube;
	TSharedPtr<FUICommandInfo> Tool_Shape_Sphere;
	TSharedPtr<FUICommandInfo> Tool_Shape_Torus;
};
