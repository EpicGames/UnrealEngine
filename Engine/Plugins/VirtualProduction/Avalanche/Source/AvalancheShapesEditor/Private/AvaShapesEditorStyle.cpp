// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapesEditorStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Math/MathFwd.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

FAvaShapesEditorStyle::FAvaShapesEditorStyle()
	: FSlateStyleSet(TEXT("AvaShapesEditor"))
{
	const FVector2f Icon16x16(16.0f, 16.0f);

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	SetContentRoot(Plugin->GetBaseDir() / TEXT("Resources"));

	// Class Icons
	Set("ClassIcon.AvaShape2DArrowDynamicMesh",          new IMAGE_BRUSH("Icons/ToolboxIcons/arrow", Icon16x16));
	Set("ClassIcon.AvaShapeChevronDynamicMesh",          new IMAGE_BRUSH("Icons/ToolboxIcons/chevron", Icon16x16));
	Set("ClassIcon.AvaShapeConeDynamicMesh",             new IMAGE_BRUSH("Icons/ToolboxIcons/cone", Icon16x16));
	Set("ClassIcon.AvaShapeCubeDynamicMesh",             new IMAGE_BRUSH("Icons/ToolboxIcons/cube", Icon16x16));
	Set("ClassIcon.AvaShapeRoundedPolygonDynamicMesh",   new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/Toolbar_Primitive", Icon16x16));
	Set("ClassIcon.AvaShapeSphereDynamicMesh",           new IMAGE_BRUSH("Icons/ToolboxIcons/sphere", Icon16x16));
	Set("ClassIcon.AvaShapeTorusDynamicMesh",            new IMAGE_BRUSH("Icons/ToolboxIcons/torus", Icon16x16));
	Set("ClassIcon.AvaShapeStarDynamicMesh",             new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/Favorite", Icon16x16));
	Set("ClassIcon.AvaShapeLineDynamicMesh",             new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/minus", Icon16x16));
	Set("ClassIcon.AvaShapeRectangleDynamicMesh",        new IMAGE_BRUSH("Icons/ToolboxIcons/rectangle", Icon16x16));
	Set("ClassIcon.AvaShapeNGonDynamicMesh",             new IMAGE_BRUSH("Icons/ToolboxIcons/regularpolygon", Icon16x16));
	Set("ClassIcon.AvaShapeIrregularPolygonDynamicMesh", new IMAGE_BRUSH("Icons/ToolboxIcons/irregularpolygon", Icon16x16));
	Set("ClassIcon.AvaShapeEllipseDynamicMesh",          new IMAGE_BRUSH("Icons/ToolboxIcons/circle", Icon16x16));
	Set("ClassIcon.AvaShapeRingDynamicMesh",             new IMAGE_BRUSH("Icons/ToolboxIcons/ring", Icon16x16));
	Set("ClassIcon.AvaShapeShapeActor",                  new IMAGE_BRUSH("Icons/ToolboxIcons/regularpolygon", Icon16x16));

	// 2D Tools Commands
	Set("AvaShapesEditor.Tool_Shape_2DArrow",       new IMAGE_BRUSH("Icons/ToolboxIcons/arrow", Icon16x16));
	Set("AvaShapesEditor.Tool_Shape_Chevron",       new IMAGE_BRUSH("Icons/ToolboxIcons/chevron", Icon16x16));
	Set("AvaShapesEditor.Tool_Shape_Ellipse",       new IMAGE_BRUSH("Icons/ToolboxIcons/circle", Icon16x16));
	Set("AvaShapesEditor.Tool_Shape_IrregularPoly", new IMAGE_BRUSH("Icons/ToolboxIcons/irregularpolygon", Icon16x16));
	Set("AvaShapesEditor.Tool_Shape_Line",          new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/minus", Icon16x16));
	Set("AvaShapesEditor.Tool_Shape_NGon",          new IMAGE_BRUSH("Icons/ToolboxIcons/regularpolygon", Icon16x16));
	Set("AvaShapesEditor.Tool_Shape_Rectangle",     new IMAGE_BRUSH("Icons/ToolboxIcons/rectangle", Icon16x16));
	Set("AvaShapesEditor.Tool_Shape_Square",        new IMAGE_BRUSH("Icons/ToolboxIcons/roundedsquare", Icon16x16));
	Set("AvaShapesEditor.Tool_Shape_Ring",          new IMAGE_BRUSH("Icons/ToolboxIcons/ring", Icon16x16));
	Set("AvaShapesEditor.Tool_Shape_Star",          new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/Favorite", Icon16x16));

	// 3D Tools Commands
	Set("AvaShapesEditor.Tool_Shape_Cone",       new IMAGE_BRUSH("Icons/ToolboxIcons/cone", Icon16x16));
	Set("AvaShapesEditor.Tool_Shape_Cylinder",   new IMAGE_BRUSH("Icons/ToolboxIcons/cylinder", Icon16x16));
	Set("AvaShapesEditor.Tool_Shape_Cube",       new IMAGE_BRUSH("Icons/ToolboxIcons/cube", Icon16x16));
	Set("AvaShapesEditor.Tool_Shape_Sphere",     new IMAGE_BRUSH("Icons/ToolboxIcons/sphere", Icon16x16));
	Set("AvaShapesEditor.Tool_Shape_Torus",      new IMAGE_BRUSH("Icons/ToolboxIcons/torus", Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FAvaShapesEditorStyle::~FAvaShapesEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
