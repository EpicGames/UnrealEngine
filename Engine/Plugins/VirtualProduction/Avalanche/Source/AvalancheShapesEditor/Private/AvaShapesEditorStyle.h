// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FAvaShapesEditorStyle final : public FSlateStyleSet
{
public:
	static FAvaShapesEditorStyle& Get()
	{
		static FAvaShapesEditorStyle Instance;
		return Instance;
	}

	FAvaShapesEditorStyle();
	virtual ~FAvaShapesEditorStyle() override;
};
