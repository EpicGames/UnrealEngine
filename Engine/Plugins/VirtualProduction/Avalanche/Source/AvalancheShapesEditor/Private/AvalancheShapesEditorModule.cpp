// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvalancheShapesEditorModule.h"
#include "AvaInteractiveToolsDelegates.h"
#include "AvaShapeActor.h"
#include "AvaShapeSprites.h"
#include "AvaShapesEditorCommands.h"
#include "ComponentVisualizers.h"
#include "Engine/Texture2D.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "IDetailsView.h"
#include "ISequencerModule.h"
#include "TrackEditors/AvaShapeRectCornerTrackEditor.h"

// Color Picker
#include "ColorPicker/AvaViewportColorPickerActorClassRegistry.h"
#include "ColorPicker/AvaViewportColorPickerAdapter.h"

// Meshes
#include "DynamicMeshes/AvaShape2DArrowDynMesh.h"
#include "DynamicMeshes/AvaShapeChevronDynMesh.h"
#include "DynamicMeshes/AvaShapeConeDynMesh.h"
#include "DynamicMeshes/AvaShapeCubeDynMesh.h"
#include "DynamicMeshes/AvaShapeEllipseDynMesh.h"
#include "DynamicMeshes/AvaShapeIrregularPolygonDynMesh.h"
#include "DynamicMeshes/AvaShapeLineDynMesh.h"
#include "DynamicMeshes/AvaShapeNGonDynMesh.h"
#include "DynamicMeshes/AvaShapeRectangleDynMesh.h"
#include "DynamicMeshes/AvaShapeRingDynMesh.h"
#include "DynamicMeshes/AvaShapeSphereDynMesh.h"
#include "DynamicMeshes/AvaShapeStarDynMesh.h"
#include "DynamicMeshes/AvaShapeTorusDynMesh.h"

// Tools
#include "Tools/AvaShapesEditorShapeTool2DArrow.h"
#include "Tools/AvaShapesEditorShapeToolChevron.h"
#include "Tools/AvaShapesEditorShapeToolCone.h"
#include "Tools/AvaShapesEditorShapeToolCube.h"
#include "Tools/AvaShapesEditorShapeToolEllipse.h"
#include "Tools/AvaShapesEditorShapeToolIrregularPoly.h"
#include "Tools/AvaShapesEditorShapeToolLine.h"
#include "Tools/AvaShapesEditorShapeToolNGon.h"
#include "Tools/AvaShapesEditorShapeToolRectangle.h"
#include "Tools/AvaShapesEditorShapeToolRing.h"
#include "Tools/AvaShapesEditorShapeToolSphere.h"
#include "Tools/AvaShapesEditorShapeToolStar.h"
#include "Tools/AvaShapesEditorShapeToolTorus.h"

// Visualizers
#include "Visualizers/AvaShapeConeDynMeshVis.h"
#include "Visualizers/AvaShapeCubeDynMeshVis.h"
#include "Visualizers/AvaShapeEllipseDynMeshVis.h"
#include "Visualizers/AvaShapeIrregularPolygonDynMeshVis.h"
#include "Visualizers/AvaShapeLineDynMeshVis.h"
#include "Visualizers/AvaShapeNGonDynMeshVis.h"
#include "Visualizers/AvaShapeRectangleDynMeshVis.h"
#include "Visualizers/AvaShapeRingDynMeshVis.h"
#include "Visualizers/AvaShapeSphereDynMeshVis.h"
#include "Visualizers/AvaShapeStarDynMeshVis.h"
#include "Visualizers/AvaShapeTorusDynMeshVis.h"

#define LOCTEXT_NAMESPACE "AvalancheShapesEditor"

void FAvalancheShapesEditorModule::StartupModule()
{
	FAvaShapesEditorCommands::Register();

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FAvalancheShapesEditorModule::RegisterVisualizers);

	IAvaComponentVisualizersSettings* AvaVisSettings = IAvalancheComponentVisualizersModule::Get().GetSettings();

	auto RegisterDefaultSprite = [AvaVisSettings](FName InName, const TCHAR* InPath)
	{
		UTexture2D* Sprite = LoadObject<UTexture2D>(nullptr, InPath);
		AvaVisSettings->SetDefaultVisualizerSprite(InName, Sprite);
	};

	RegisterDefaultSprite(UE::AvaShapes::BevelSprite, TEXT("Texture2D'/Avalanche/EditorResources/NewBevelHandle.NewBevelHandle'"));
	RegisterDefaultSprite(UE::AvaShapes::BreakSideSprite, TEXT("Texture2D'/Engine/EditorResources/S_Emitter.S_Emitter'"));
	RegisterDefaultSprite(UE::AvaShapes::ColorSelectionSprite, TEXT("Texture2D'/Engine/EditorResources/S_ReflActorIcon.S_ReflActorIcon'"));
	RegisterDefaultSprite(UE::AvaShapes::CornerSprite, TEXT("Texture2D'/Avalanche/EditorResources/Bevel.Bevel'"));
	RegisterDefaultSprite(UE::AvaShapes::DepthSprite, TEXT("Texture2D'/Engine/EditorResources/S_Terrain.S_Terrain'"));
	RegisterDefaultSprite(UE::AvaShapes::InnerSizeSprite, TEXT("Texture2D'/Engine/EditorResources/S_RadForce.S_RadForce'"));
	RegisterDefaultSprite(UE::AvaShapes::LinearGradientSprite, TEXT("Texture2D'/Avalanche/EditorResources/LinearGradient.LinearGradient'"));
	RegisterDefaultSprite(UE::AvaShapes::NumPointsSprite, TEXT("Texture2D'/Engine/EditorResources/S_Emitter.S_Emitter'"));
	RegisterDefaultSprite(UE::AvaShapes::NumSidesSprite, TEXT("Texture2D'/Engine/EditorResources/S_Emitter.S_Emitter'"));
	RegisterDefaultSprite(UE::AvaShapes::SizeSprite, TEXT("Texture2D'/Avalanche/EditorResources/NewSizeHandle.NewSizeHandle'"));
	RegisterDefaultSprite(UE::AvaShapes::SlantSprite, TEXT("Texture2D'/Avalanche/EditorResources/Slant.Slant'"));
	RegisterDefaultSprite(UE::AvaShapes::TextMaxHeightSprite, TEXT("Texture2D'/Engine/EngineResources/Cursors/SplitterVert.SplitterVert'"));
	RegisterDefaultSprite(UE::AvaShapes::TextMaxWidthSprite, TEXT("Texture2D'/Engine/EngineResources/Cursors/SplitterHorz.SplitterHorz'"));
	RegisterDefaultSprite(UE::AvaShapes::TextScaleProportionallySprite, TEXT("Texture2D'/Engine/EditorResources/S_TextRenderActorIcon.S_TextRenderActorIcon'"));
	RegisterDefaultSprite(UE::AvaShapes::UVSprite, TEXT("Texture2D'/Engine/EditorResources/MatInstActSprite.MatInstActSprite'"));

	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
	TrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FAvaShapeRectCornerTrackEditor::CreateTrackEditor));

	FAvaViewportColorPickerActorClassRegistry::RegisterDefaultClassAdapter<AAvaShapeActor>();
}

void FAvalancheShapesEditorModule::ShutdownModule()
{
	FAvaShapesEditorCommands::Unregister();

	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	if (FModuleManager::Get().IsModuleLoaded("Sequencer"))
	{
		ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
		SequencerModule.UnRegisterTrackEditor(TrackEditorHandle);
		TrackEditorHandle.Reset();
	}
}

void FAvalancheShapesEditorModule::RegisterVisualizers()
{
	// 2D Visualizers
	IAvalancheComponentVisualizersModule::RegisterComponentVisualizer<UAvaShape2DArrowDynamicMesh, FAvaShape2DDynamicMeshVisualizer>(&Visualizers);
	IAvalancheComponentVisualizersModule::RegisterComponentVisualizer<UAvaShapeChevronDynamicMesh, FAvaShape2DDynamicMeshVisualizer>(&Visualizers);
	IAvalancheComponentVisualizersModule::RegisterComponentVisualizer<UAvaShapeEllipseDynamicMesh, FAvaShapeEllipseDynamicMeshVisualizer>(&Visualizers);
	IAvalancheComponentVisualizersModule::RegisterComponentVisualizer<UAvaShapeIrregularPolygonDynamicMesh, FAvaShapeIrregularPolygonDynamicMeshVisualizer>(&Visualizers);
	IAvalancheComponentVisualizersModule::RegisterComponentVisualizer<UAvaShapeLineDynamicMesh, FAvaShapeLineDynamicMeshVisualizer>(&Visualizers);
	IAvalancheComponentVisualizersModule::RegisterComponentVisualizer<UAvaShapeNGonDynamicMesh, FAvaShapeNGonDynamicMeshVisualizer>(&Visualizers);
	IAvalancheComponentVisualizersModule::RegisterComponentVisualizer<UAvaShapeRectangleDynamicMesh, FAvaShapeRectangleDynamicMeshVisualizer>(&Visualizers);
	IAvalancheComponentVisualizersModule::RegisterComponentVisualizer<UAvaShapeRingDynamicMesh, FAvaShapeRingDynamicMeshVisualizer>(&Visualizers);
	IAvalancheComponentVisualizersModule::RegisterComponentVisualizer<UAvaShapeStarDynamicMesh, FAvaShapeStarDynamicMeshVisualizer>(&Visualizers);

	// 3D Visualizers
	IAvalancheComponentVisualizersModule::RegisterComponentVisualizer<UAvaShapeConeDynamicMesh, FAvaShapeConeDynamicMeshVisualizer>(&Visualizers);
	IAvalancheComponentVisualizersModule::RegisterComponentVisualizer<UAvaShapeCubeDynamicMesh, FAvaShapeCubeDynamicMeshVisualizer>(&Visualizers);
	IAvalancheComponentVisualizersModule::RegisterComponentVisualizer<UAvaShapeSphereDynamicMesh, FAvaShapeSphereDynamicMeshVisualizer>(&Visualizers);
	IAvalancheComponentVisualizersModule::RegisterComponentVisualizer<UAvaShapeTorusDynamicMesh, FAvaShapeTorusDynamicMeshVisualizer>(&Visualizers);
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FAvalancheShapesEditorModule, AvalancheShapesEditor)