// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Containers/Array.h"
#include "Delegates/IDelegateInstance.h"
#include "Templates/SharedPointer.h"

class FComponentVisualizer;
class IAvalancheInteractiveToolsModule;

class FAvalancheShapesEditorModule : public IModuleInterface
{
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

	TArray<TSharedPtr<FComponentVisualizer>> Visualizers;

	void RegisterVisualizers();

	FDelegateHandle TrackEditorHandle;
};
