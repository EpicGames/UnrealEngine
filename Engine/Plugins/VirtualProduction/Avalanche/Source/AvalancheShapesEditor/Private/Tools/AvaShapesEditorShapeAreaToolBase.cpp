// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaShapesEditorShapeAreaToolBase.h"
#include "AvaShapeActor.h"
#include "AvaViewportUtils.h"
#include "AvaVisibleArea.h"
#include "Containers/Ticker.h"
#include "Editor/EditorEngine.h"
#include "InteractiveToolManager.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "GameFramework/Actor.h"
#include "Planners/AvaInteractiveToolsToolViewportAreaPlanner.h"

UAvaShapesEditorShapeAreaToolBase::UAvaShapesEditorShapeAreaToolBase()
{
	ViewportPlannerClass = UAvaInteractiveToolsToolViewportAreaPlanner::StaticClass();
}

AActor* UAvaShapesEditorShapeAreaToolBase::SpawnActor(TSubclassOf<AActor> InActorClass, EAvaViewportStatus InViewportStatus,
	const FVector2f& InViewportPosition, bool bInPreview, FString* InActorLabelOverride)
{
	AActor* NewActor = Super::SpawnActor(InActorClass, InViewportStatus, InViewportPosition, bInPreview, InActorLabelOverride);

	if (!bPerformingDefaultAction)
	{
		UpdateShapeSize(Cast<AAvaShapeActor>(NewActor));
	}

	return NewActor;
}

void UAvaShapesEditorShapeAreaToolBase::OnViewportPlannerUpdate()
{
	Super::OnViewportPlannerUpdate();

	if (UAvaInteractiveToolsToolViewportAreaPlanner* AreaPlanner = Cast<UAvaInteractiveToolsToolViewportAreaPlanner>(ViewportPlanner))
	{
		if (AreaPlanner->HasStarted())
		{
			if (!PreviewActor)
			{
				PreviewActor = SpawnActor(ActorClass, true);
			}

			if (PreviewActor)
			{
				const FVector2f ViewportPosition = AreaPlanner->GetCenterPosition();

				UWorld* World;
				FVector NewLocation;
				FRotator NewRotation;

				if (ViewportPositionToWorldPositionAndOrientation(EAvaViewportStatus::Focused, ViewportPosition, World, NewLocation, NewRotation))
				{
					PreviewActor->SetActorRotation(NewRotation);

					if (AAvaShapeActor* ShapeActor = Cast<AAvaShapeActor>(PreviewActor))
					{
						if (UAvaShapeDynamicMeshBase* DynMesh = ShapeActor->GetDynamicMesh())
						{
							DynMesh->SetMeshRegenWorldLocation(NewLocation);
						}
					}
				}

				UpdateShapeSize(Cast<AAvaShapeActor>(PreviewActor));
			}
		}
	}
}

void UAvaShapesEditorShapeAreaToolBase::OnViewportPlannerComplete()
{
	if (UAvaInteractiveToolsToolViewportAreaPlanner* AreaPlanner = Cast<UAvaInteractiveToolsToolViewportAreaPlanner>(ViewportPlanner))
	{
		SpawnedActor = SpawnActor(ActorClass, EAvaViewportStatus::Focused, AreaPlanner->GetCenterPosition(), /* Preview */ false);
	}

	Super::OnViewportPlannerComplete();
}

void UAvaShapesEditorShapeAreaToolBase::UpdateShapeSize(AAvaShapeActor* InShapeActor) const
{
	if (!InShapeActor)
	{
		return;
	}

	if (UAvaInteractiveToolsToolViewportAreaPlanner* AreaPlanner = Cast<UAvaInteractiveToolsToolViewportAreaPlanner>(ViewportPlanner))
	{
		if (AreaPlanner->HasStarted())
		{
			SetShapeSize(InShapeActor, AreaPlanner->GetWorldSize());
		}
	}
}
