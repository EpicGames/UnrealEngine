// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaShapesEditorShapeToolBase.h"
#include "AvaShapesEditorShapeAreaToolBase.generated.h"

class AActor;
class AAvaShapeActor;

UCLASS(Abstract)
class UAvaShapesEditorShapeAreaToolBase : public UAvaShapesEditorShapeToolBase
{
	GENERATED_BODY()

	using UAvaShapesEditorShapeToolBase::SpawnActor;

public:
	UAvaShapesEditorShapeAreaToolBase();

	//~ Begin UAvaInteractiveToolsToolBase
	virtual void OnViewportPlannerUpdate() override;
	virtual void OnViewportPlannerComplete() override;
	//~ End UAvaInteractiveToolsToolBase

protected:
	void UpdateShapeSize(AAvaShapeActor* InShapeActor) const;

	//~ Begin UAvaInteractiveToolsToolBase
	virtual AActor* SpawnActor(TSubclassOf<AActor> InActorClass, EAvaViewportStatus InViewportStatus, const FVector2f& InViewportPosition, bool bInPreview, FString* InActorLabelOverride = nullptr) override;
	//~ End UAvaInteractiveToolsToolBase
};
