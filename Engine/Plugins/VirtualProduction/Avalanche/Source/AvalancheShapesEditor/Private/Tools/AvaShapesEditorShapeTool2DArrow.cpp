// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaShapesEditorShapeTool2DArrow.h"
#include "AvaShapesEditorCommands.h"
#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "DynamicMeshes/AvaShape2DArrowDynMesh.h"
#include "UObject/ConstructorHelpers.h"

UAvaShapesEditorShapeTool2DArrow::UAvaShapesEditorShapeTool2DArrow()
{
	ShapeClass = UAvaShape2DArrowDynamicMesh::StaticClass();
}

void UAvaShapesEditorShapeTool2DArrow::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	FAvaInteractiveToolsToolParameters ToolParameters = CreateDefaultToolParameters();
	ToolParameters.UICommand = FAvaShapesEditorCommands::Get().Tool_Shape_2DArrow;
	ToolParameters.Priority = 7000;
	ToolParameters.ActorFactory.Set<TObjectPtr<UActorFactory>>(CreateFactory<UAvaShape2DArrowDynamicMesh>());

	InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryName2D, MoveTemp(ToolParameters));
}
