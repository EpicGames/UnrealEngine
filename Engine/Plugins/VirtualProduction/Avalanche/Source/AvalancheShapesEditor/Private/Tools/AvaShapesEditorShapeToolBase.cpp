// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaShapesEditorShapeToolBase.h"
#include "AvaShapeActor.h"
#include "AvaInteractiveToolsSettings.h"
#include "AvaViewportUtils.h"
#include "DynamicMeshes/AvaShape2DDynMeshBase.h"
#include "DynamicMeshes/AvaShape3DDynMeshBase.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "GameFramework/Actor.h"
#include "InteractiveToolManager.h"
#include "ToolContextInterfaces.h"

const UAvaShapesEditorShapeToolBase::FShapeFactoryParameters UAvaShapesEditorShapeToolBase::DefaultParameters = FShapeFactoryParameters();

UAvaShapesEditorShapeToolBase::UAvaShapesEditorShapeToolBase()
{
	ActorClass = AAvaShapeActor::StaticClass();
}

bool UAvaShapesEditorShapeToolBase::OnBegin()
{
	if (ShapeClass == nullptr)
	{
		return false;
	}

	return Super::OnBegin();
}

void UAvaShapesEditorShapeToolBase::OnActorSpawned(AActor* InActor)
{
	Super::OnActorSpawned(InActor);

	if (const AAvaShapeActor* ShapeActor = Cast<AAvaShapeActor>(InActor))
	{
		SetToolkitSettingsObject(ShapeActor->GetDynamicMesh());
	}
}

void UAvaShapesEditorShapeToolBase::SetShapeSize(AAvaShapeActor* InShapeActor, const FVector2D& InShapeSize) const
{
	if (!InShapeActor)
	{
		return;
	}

	UAvaShapeDynamicMeshBase* MeshBase = InShapeActor->GetDynamicMesh();

	if (!MeshBase)
	{
		return;
	}

	if (UAvaShape2DDynMeshBase* Mesh2D = Cast<UAvaShape2DDynMeshBase>(MeshBase))
	{
		Mesh2D->SetSize2D(InShapeSize);
	}
	else if (UAvaShape3DDynMeshBase* Mesh3D = Cast<UAvaShape3DDynMeshBase>(MeshBase))
	{
		const FVector Size3D = Mesh3D->GetSize3D();
		Mesh3D->SetSize3D({Size3D.X, InShapeSize.X, InShapeSize.Y});
	}
	else
	{
		checkNoEntry();
	}
}

bool UAvaShapesEditorShapeToolBase::UseIdentityLocation() const
{
	return bPerformingDefaultAction;
}
