// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaInteractiveToolsActorToolBase.h"
#include "AvaShapeFactory.h"
#include "Templates/SubclassOf.h"
#include "AvaShapesEditorShapeToolBase.generated.h"

class AAvaShapeActor;
class UAvaShapeDynamicMeshBase;

UCLASS(Abstract)
class UAvaShapesEditorShapeToolBase : public UAvaInteractiveToolsActorToolBase
{
	GENERATED_BODY()

public:
	using UAvaInteractiveToolsActorToolBase::SpawnActor;

	struct FShapeFactoryParameters
	{
		FVector Size = FVector(100);
		TFunction<void(UAvaShapeDynamicMeshBase*)> Functor = [](UAvaShapeDynamicMeshBase*){};
		TOptional<FString> NameOverride;
	};

	static const FShapeFactoryParameters DefaultParameters;

	UAvaShapesEditorShapeToolBase();

protected:
	template<typename InMeshClass
		UE_REQUIRES(std::derived_from<InMeshClass, UAvaShapeDynamicMeshBase>)>
	static UAvaShapeFactory* CreateFactory(const FShapeFactoryParameters& InParameters = DefaultParameters)
	{
		UAvaShapeFactory* Factory = CreateActorFactory<UAvaShapeFactory>();
		Factory->SetMeshClass(InMeshClass::StaticClass());
		Factory->SetMeshSize(InParameters.Size);
		Factory->SetMeshFunction(InParameters.Functor);
		Factory->SetMeshNameOverride(InParameters.NameOverride);
		return Factory;
	}

	//~ Begin UAvaInteractiveToolsToolBase
	virtual bool OnBegin() override;
	//~ End UAvaInteractiveToolsToolBase
	
	//~ Begin UAvaInteractiveToolsToolBase
	virtual bool UseIdentityLocation() const override;
	virtual void OnActorSpawned(AActor* InActor) override;
	//~ End UAvaInteractiveToolsToolBase

	virtual void SetShapeSize(AAvaShapeActor* InShapeActor, const FVector2D& InShapeSize) const;

	UPROPERTY()
	TSubclassOf<UAvaShapeDynamicMeshBase> ShapeClass = nullptr;
};
