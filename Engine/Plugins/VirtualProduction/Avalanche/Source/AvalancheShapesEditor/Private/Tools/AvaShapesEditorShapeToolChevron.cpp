// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaShapesEditorShapeToolChevron.h"
#include "AvaShapesEditorCommands.h"
#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "DynamicMeshes/AvaShapeChevronDynMesh.h"
#include "UObject/ConstructorHelpers.h"

UAvaShapesEditorShapeToolChevron::UAvaShapesEditorShapeToolChevron()
{
	ShapeClass = UAvaShapeChevronDynamicMesh::StaticClass();
}

void UAvaShapesEditorShapeToolChevron::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	FAvaInteractiveToolsToolParameters ToolParameters = CreateDefaultToolParameters();
	ToolParameters.UICommand = FAvaShapesEditorCommands::Get().Tool_Shape_Chevron;
	ToolParameters.Priority = 8000;
	ToolParameters.ActorFactory.Set<TObjectPtr<UActorFactory>>(CreateFactory<UAvaShapeChevronDynamicMesh>());

	InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryName2D, MoveTemp(ToolParameters));
}
