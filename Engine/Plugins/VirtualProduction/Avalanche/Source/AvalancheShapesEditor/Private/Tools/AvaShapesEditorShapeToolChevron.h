// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaShapesEditorShapeAreaToolBase.h"
#include "AvaShapesEditorShapeToolChevron.generated.h"

UCLASS()
class UAvaShapesEditorShapeToolChevron : public UAvaShapesEditorShapeAreaToolBase
{
	GENERATED_BODY()

public:
	UAvaShapesEditorShapeToolChevron();

protected:
	//~ Begin UAvaInteractiveToolsToolBase
	virtual void OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule) override;
	//~ End UAvaInteractiveToolsToolBase
};
