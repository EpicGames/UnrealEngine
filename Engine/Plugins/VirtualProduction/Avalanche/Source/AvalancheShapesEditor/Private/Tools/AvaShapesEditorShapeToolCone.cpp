// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaShapesEditorShapeToolCone.h"
#include "AvaShapesEditorCommands.h"
#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "DynamicMeshes/AvaShapeConeDynMesh.h"
#include "UObject/ConstructorHelpers.h"

UAvaShapesEditorShapeToolCone::UAvaShapesEditorShapeToolCone()
{
	ShapeClass = UAvaShapeConeDynamicMesh::StaticClass();
}

void UAvaShapesEditorShapeToolCone::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	FAvaInteractiveToolsToolParameters ConeToolParameters = CreateDefaultToolParameters();
	ConeToolParameters.UICommand = FAvaShapesEditorCommands::Get().Tool_Shape_Cone;
	ConeToolParameters.Priority = 3000;
	ConeToolParameters.ActorFactory.Set<TObjectPtr<UActorFactory>>(CreateFactory<UAvaShapeConeDynamicMesh>());
	ConeToolParameters.ToolIdentifier = TEXT("Parametric Cone Tool");

	InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryName3D, MoveTemp(ConeToolParameters));

	const FShapeFactoryParameters CylinderFactoryParameters =
	{
		.Functor = [](UAvaShapeDynamicMeshBase* InMesh)
		{
			UAvaShapeConeDynamicMesh* Cone = Cast<UAvaShapeConeDynamicMesh>(InMesh);
			Cone->SetTopRadius(1.f);
		},
		.NameOverride = FString("Cylinder"),
	};

	FAvaInteractiveToolsToolParameters CylinderToolParameters = CreateDefaultToolParameters();
	CylinderToolParameters.UICommand = FAvaShapesEditorCommands::Get().Tool_Shape_Cylinder;
	CylinderToolParameters.Priority = 3000;
	CylinderToolParameters.ActorFactory.Set<TObjectPtr<UActorFactory>>(CreateFactory<UAvaShapeConeDynamicMesh>(CylinderFactoryParameters));
	CylinderToolParameters.ToolIdentifier = TEXT("Parametric Cylinder Tool");

	InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryName3D, MoveTemp(CylinderToolParameters));
}
