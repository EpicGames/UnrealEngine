// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaShapesEditorShapeToolCube.h"
#include "AvaShapesEditorCommands.h"
#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "DynamicMeshes/AvaShapeCubeDynMesh.h"
#include "UObject/ConstructorHelpers.h"

UAvaShapesEditorShapeToolCube::UAvaShapesEditorShapeToolCube()
{
	ShapeClass = UAvaShapeCubeDynamicMesh::StaticClass();
}

void UAvaShapesEditorShapeToolCube::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	FAvaInteractiveToolsToolParameters ToolParameters = CreateDefaultToolParameters();
	ToolParameters.UICommand = FAvaShapesEditorCommands::Get().Tool_Shape_Cube;
	ToolParameters.Priority = 1000;
	ToolParameters.ActorFactory.Set<TObjectPtr<UActorFactory>>(CreateFactory<UAvaShapeCubeDynamicMesh>());
	ToolParameters.ToolIdentifier = TEXT("Parametric Cube Tool");

	InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryName3D, MoveTemp(ToolParameters));
}
