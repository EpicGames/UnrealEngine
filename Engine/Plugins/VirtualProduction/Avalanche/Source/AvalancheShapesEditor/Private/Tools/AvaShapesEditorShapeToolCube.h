// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaShapesEditorShapeAreaToolBase.h"
#include "AvaShapesEditorShapeToolCube.generated.h"

UCLASS()
class UAvaShapesEditorShapeToolCube : public UAvaShapesEditorShapeAreaToolBase
{
	GENERATED_BODY()

public:
	UAvaShapesEditorShapeToolCube();

protected:
	//~ Begin UAvaInteractiveToolsToolBase
	virtual void OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule) override;
	//~ End UAvaInteractiveToolsToolBase
};
