// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaShapesEditorShapeToolEllipse.h"
#include "AvaShapesEditorCommands.h"
#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "DynamicMeshes/AvaShapeEllipseDynMesh.h"
#include "UObject/ConstructorHelpers.h"

UAvaShapesEditorShapeToolEllipse::UAvaShapesEditorShapeToolEllipse()
{
	ShapeClass = UAvaShapeEllipseDynamicMesh::StaticClass();
}

void UAvaShapesEditorShapeToolEllipse::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	FAvaInteractiveToolsToolParameters ToolParameters = CreateDefaultToolParameters();
	ToolParameters.UICommand = FAvaShapesEditorCommands::Get().Tool_Shape_Ellipse;
	ToolParameters.Priority = 2000;
	ToolParameters.ActorFactory.Set<TObjectPtr<UActorFactory>>(CreateFactory<UAvaShapeEllipseDynamicMesh>());
	
	InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryName2D, MoveTemp(ToolParameters));
}
