// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaShapesEditorShapeToolIrregularPoly.h"
#include "AvaShapeActor.h"
#include "AvaShapesEditorCommands.h"
#include "AvaViewportUtils.h"
#include "AvaVisibleArea.h"
#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "DynamicMeshes/AvaShapeIrregularPolygonDynMesh.h"
#include "Framework/Application/SlateApplication.h"
#include "Planners/AvaInteractiveToolsToolViewportPointListPlanner.h"
#include "SceneView.h"
#include "Styling/StyleColors.h"
#include "ViewportClient/IAvaViewportClient.h"

UAvaShapesEditorShapeToolIrregularPoly::UAvaShapesEditorShapeToolIrregularPoly()
{
	ViewportPlannerClass = UAvaInteractiveToolsToolViewportPointListPlanner::StaticClass();
	ShapeClass = UAvaShapeIrregularPolygonDynamicMesh::StaticClass();
}

void UAvaShapesEditorShapeToolIrregularPoly::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	FAvaInteractiveToolsToolParameters ToolParameters = CreateDefaultToolParameters();
	ToolParameters.UICommand = FAvaShapesEditorCommands::Get().Tool_Shape_IrregularPoly;
	ToolParameters.Priority = 4000;
	ToolParameters.ActorFactory.Set<TObjectPtr<UActorFactory>>(CreateFactory<UAvaShapeIrregularPolygonDynamicMesh>());

	InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryName2D, MoveTemp(ToolParameters));
}

void UAvaShapesEditorShapeToolIrregularPoly::DrawHUD(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI)
{
	Super::DrawHUD(InCanvas, InRenderAPI);

	UAvaInteractiveToolsToolViewportPointListPlanner* PointListPlanner = Cast<UAvaInteractiveToolsToolViewportPointListPlanner>(ViewportPlanner);

	if (!PointListPlanner)
	{
		return;
	}

	TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAvaViewportClient(GetViewport(EAvaViewportStatus::Focused));

	if (!AvaViewportClient.IsValid())
	{
		return;
	}

	const FAvaVisibleArea VisibleArea = AvaViewportClient->GetZoomedVisibleArea();
	const FVector2f Offset = AvaViewportClient->GetViewportOffset();
	const FVector2f& LastPosition = PointListPlanner->GetCurrentViewportPosition();
	const TArray<FVector2f>& ViewportPositions = PointListPlanner->GetViewportPositions();
	const int32 OverlappedIndex = FindOverlappingPointIndex(LastPosition);

	static const FLinearColor NeutralColor = FStyleColors::AccentBlue.GetSpecifiedColor();
	static const FLinearColor CompleteColor = FStyleColors::AccentGreen.GetSpecifiedColor();
	static const FLinearColor RemoveColor = FStyleColors::AccentRed.GetSpecifiedColor();
	static const FVector2D PointSize(5.0, 5.0);
	const float AppScale = FSlateApplication::Get().GetApplicationScale();

	for (int32 PointIdx = 0; PointIdx < ViewportPositions.Num(); ++PointIdx)
	{
		const FVector2f VisiblePosition = (VisibleArea.GetVisiblePosition(ViewportPositions[PointIdx]) * AppScale) + Offset;
		FCanvasNGonItem PointItem(static_cast<FVector2D>(VisiblePosition), PointSize, 8, NeutralColor);

		if (PointIdx == OverlappedIndex)
		{
			if (PointIdx == 0 && ViewportPositions.Num() > 2)
			{
				PointItem.SetColor(CompleteColor);
			}
			else
			{
				PointItem.SetColor(RemoveColor);
			}
		}

		InCanvas->DrawItem(PointItem);
	}
}

void UAvaShapesEditorShapeToolIrregularPoly::OnViewportPlannerUpdate()
{
	Super::OnViewportPlannerUpdate();

	if (UAvaInteractiveToolsToolViewportPointListPlanner* PointListPlanner = Cast<UAvaInteractiveToolsToolViewportPointListPlanner>(ViewportPlanner))
	{
		const TArray<FVector2f>& Positions = PointListPlanner->GetViewportPositions();

		if (Positions.IsEmpty())
		{
			return;
		}

		if (!PreviewActor)
		{
			PreviewActor = SpawnActor(ActorClass, /* Preview */ true);

			if (!PreviewActor)
			{
				return;
			}
		}

		if (AAvaShapeActor* ShapeActor = Cast<AAvaShapeActor>(PreviewActor))
		{
			if (UAvaShapeIrregularPolygonDynamicMesh* IrregularPoly = Cast<UAvaShapeIrregularPolygonDynamicMesh>(ShapeActor->GetDynamicMesh()))
			{
				while (IrregularPoly->GetPoints().Num() < Positions.Num())
				{
					OnNewPointAdded();
				}

				UpdateTransientLineColor();
			}
		}
	}
}

int32 UAvaShapesEditorShapeToolIrregularPoly::FindOverlappingPointIndex(const FVector2f& InPosition) const
{
	int32 Index = INDEX_NONE;

	if (UAvaInteractiveToolsToolViewportPointListPlanner* PointListPlanner = Cast<UAvaInteractiveToolsToolViewportPointListPlanner>(ViewportPlanner))
	{
		const TArray<FVector2f>& ViewportPositions = PointListPlanner->GetViewportPositions();
		constexpr float MinDimSq = MinDim * MinDim;

		for (int32 PointIdx = 0; PointIdx < ViewportPositions.Num(); ++PointIdx)
		{
			if ((ViewportPositions[PointIdx] - InPosition).SizeSquared() < MinDimSq)
			{
				return PointIdx;
			}
		}
	}

	return Index;
}

void UAvaShapesEditorShapeToolIrregularPoly::OnNewPointAdded()
{
	if (UAvaInteractiveToolsToolViewportPointListPlanner* PointListPlanner = Cast<UAvaInteractiveToolsToolViewportPointListPlanner>(ViewportPlanner))
	{
		if (AAvaShapeActor* ShapeActor = Cast<AAvaShapeActor>(PreviewActor))
		{
			if (UAvaShapeIrregularPolygonDynamicMesh* IrregularPoly = Cast<UAvaShapeIrregularPolygonDynamicMesh>(ShapeActor->GetDynamicMesh()))
			{
				const TArray<FVector2f>& ViewportPositions = PointListPlanner->GetViewportPositions();
				const TArray<FAvaShapeRoundedCorner>& Corners = IrregularPoly->GetPoints();

				if (Corners.Num() < ViewportPositions.Num())
				{
					if (ViewportPositions.Num() > 1)
					{
						const FVector2f& LastPosition = ViewportPositions.Last();
						const int32 OverlappedIndex = FindOverlappingPointIndex(LastPosition);
						const bool bOverlapped = OverlappedIndex != INDEX_NONE && OverlappedIndex < ViewportPositions.Num() - 1;

						if (bOverlapped)
						{
							// Complete shape
							if (OverlappedIndex == 0 && Corners.Num() > 2)
							{
								// Make sure to exit the loop
								PointListPlanner->GetViewportPositions().SetNum(Corners.Num());

								AddFinishedShape();								
							}
							// Reset shape to earlier state
							else
							{
								TArray<FVector2D> NewPoints;
								NewPoints.Reserve(OverlappedIndex);

								for (int32 PointIdx = 0; PointIdx < OverlappedIndex; ++PointIdx)
								{
									NewPoints.Add(Corners[PointIdx].Location);
								}

								IrregularPoly->SetPoints(NewPoints);
								PointListPlanner->GetViewportPositions().SetNum(Corners.Num());
							}

							return;
						}
					}

					UWorld* TempWorld;
					FVector TempVector;
					FRotator TempRotator;

					if (ViewportPositionToWorldPositionAndOrientation(EAvaViewportStatus::Focused, ViewportPositions[Corners.Num()], TempWorld, TempVector, TempRotator))
					{
						TempVector -= ShapeActor->GetActorLocation();
						TempVector = TempRotator.UnrotateVector(TempVector);

						const bool bAddedSuccessfully = IrregularPoly->AddPoint({TempVector.Y, TempVector.Z});

						if (!bAddedSuccessfully)
						{
							PointListPlanner->GetViewportPositions().SetNum(Corners.Num());
						}
					}
				}
			}
		}
	}
}

void UAvaShapesEditorShapeToolIrregularPoly::UpdateTransientLineColor()
{
	if (UAvaInteractiveToolsToolViewportPointListPlanner* PointListPlanner = Cast<UAvaInteractiveToolsToolViewportPointListPlanner>(ViewportPlanner))
	{
		if (AAvaShapeActor* ShapeActor = Cast<AAvaShapeActor>(PreviewActor))
		{
			if (UAvaShapeIrregularPolygonDynamicMesh* IrregularPoly = Cast<UAvaShapeIrregularPolygonDynamicMesh>(ShapeActor->GetDynamicMesh()))
			{
				const TArray<FVector2f>& ViewportPositions = PointListPlanner->GetViewportPositions();
				const FVector2f& LastPosition = PointListPlanner->GetCurrentViewportPosition();
				const int32 OverlappedIndex = FindOverlappingPointIndex(LastPosition);
				const bool bOverlapped = OverlappedIndex != INDEX_NONE;

				if (bOverlapped)
				{
					PointListPlanner->SetLineStatus(EAvaInteractiveToolsToolViewportPointListPlannerLineStatus::Neutral);
					return;
				}

				UWorld* World;
				FVector Vector;
				FRotator Rotator;

				if (ViewportPositionToWorldPositionAndOrientation(EAvaViewportStatus::Focused, PointListPlanner->GetCurrentViewportPosition(), World, Vector, Rotator))
				{
					Vector -= ShapeActor->GetActorLocation();
					Vector = Rotator.UnrotateVector(Vector);

					if (IrregularPoly->CanAddPoint({Vector.Y, Vector.Z}))
					{
						PointListPlanner->SetLineStatus(EAvaInteractiveToolsToolViewportPointListPlannerLineStatus::Allowed);
						return;
					}
				}
			}
		}

		PointListPlanner->SetLineStatus(EAvaInteractiveToolsToolViewportPointListPlannerLineStatus::Disallowed);
	}
}

void UAvaShapesEditorShapeToolIrregularPoly::AddFinishedShape()
{
	if (UAvaInteractiveToolsToolViewportPointListPlanner* PointListPlanner = Cast<UAvaInteractiveToolsToolViewportPointListPlanner>(ViewportPlanner))
	{
		if (AAvaShapeActor* PreviewShapeActor = Cast<AAvaShapeActor>(PreviewActor))
		{
			if (UAvaShapeIrregularPolygonDynamicMesh* PreviewIrregularPoly = Cast<UAvaShapeIrregularPolygonDynamicMesh>(PreviewShapeActor->GetDynamicMesh()))
			{
				if (AAvaShapeActor* NewShapeActor = Cast<AAvaShapeActor>(SpawnActor(ActorClass, /* Preview */ false)))
				{
					if (UAvaShapeIrregularPolygonDynamicMesh* NewIrregularPoly = Cast<UAvaShapeIrregularPolygonDynamicMesh>(NewShapeActor->GetDynamicMesh()))
					{
						NewShapeActor->SetActorTransform(PreviewShapeActor->GetTransform());

						const TArray<FAvaShapeRoundedCorner>& Corners = PreviewIrregularPoly->GetPoints();
						TArray<FVector2D> NewPoints;
						NewPoints.Reserve(Corners.Num());
						FVector2D MinWorld = Corners[0].Location;
						FVector2D MaxWorld = Corners[0].Location;

						for (const FAvaShapeRoundedCorner& Corner : Corners)
						{
							NewPoints.Add(Corner.Location);

							MinWorld.X = FMath::Min(MinWorld.X, Corner.Location.X);
							MinWorld.Y = FMath::Min(MinWorld.Y, Corner.Location.Y);
							MaxWorld.X = FMath::Max(MaxWorld.X, Corner.Location.X);
							MaxWorld.Y = FMath::Max(MaxWorld.Y, Corner.Location.Y);
						}

						// Readjust so the shape is geometrically centered.
						FVector2D CenterWorld = MinWorld * 0.5 + MaxWorld * 0.5;

						for (FVector2D& Point : NewPoints)
						{
							Point -= CenterWorld;
						}

						NewIrregularPoly->SetPoints(NewPoints);

						const TArray<FVector2f>& ViewportPositions = PointListPlanner->GetViewportPositions();
						FVector2f MinViewport = ViewportPositions[0];
						FVector2f MaxViewport = ViewportPositions[0];

						for (const FVector2f& Position : ViewportPositions)
						{
							MinViewport.X = FMath::Min(MinViewport.X, Position.X);
							MinViewport.Y = FMath::Min(MinViewport.Y, Position.Y);
							MaxViewport.X = FMath::Max(MaxViewport.X, Position.X);
							MaxViewport.Y = FMath::Max(MaxViewport.Y, Position.Y);
						}

						const FVector2f CenterViewport = static_cast<FVector2f>(MinViewport * 0.5 + MaxViewport * 0.5);

						UWorld* World;
						FVector Vector;
						FRotator Rotator;

						if (ViewportPositionToWorldPositionAndOrientation(EAvaViewportStatus::Focused, CenterViewport, World, Vector, Rotator))
						{
							NewShapeActor->SetActorLocation(Vector);
						}

						RequestShutdown(EToolShutdownType::Completed);
						return;
					}
				}
			}
		}
	}

	RequestShutdown(EToolShutdownType::Cancel);
}
