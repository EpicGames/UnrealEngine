// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaShapesEditorShapeToolBase.h"
#include "AvaShapesEditorShapeToolIrregularPoly.generated.h"

UCLASS()
class UAvaShapesEditorShapeToolIrregularPoly : public UAvaShapesEditorShapeToolBase
{
	GENERATED_BODY()

public:
	UAvaShapesEditorShapeToolIrregularPoly();

protected:
	// The minimum dimension
	static constexpr float MinDim = 5;
	
	//~ Begin UAvaInteractiveToolsToolBase
	virtual void OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule) override;
	virtual void DrawHUD(FCanvas* InCanvas, IToolsContextRenderAPI* InRenderAPI) override;
	virtual void OnViewportPlannerUpdate() override;
	virtual void OnViewportPlannerComplete() override {} // Does nothing
	//~ End UAvaInteractiveToolsToolBase

	//~ Begin UAvaShapesEditorShapeToolBase
	virtual void SetShapeSize(AAvaShapeActor* InShapeActor, const FVector2D& InShapeSize) const override {}
	//~ End UAvaShapesEditorShapeToolBase

	int32 FindOverlappingPointIndex(const FVector2f& InPosition) const;

	void OnNewPointAdded();
	void UpdateTransientLineColor();
	void AddFinishedShape();
};
