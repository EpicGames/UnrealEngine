// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaShapesEditorShapeToolLine.h"
#include "AvaShapeActor.h"
#include "AvaShapesEditorCommands.h"
#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "DynamicMeshes/AvaShapeLineDynMesh.h"
#include "Framework/Application/SlateApplication.h"
#include "Planners/AvaInteractiveToolsToolViewportPointListPlanner.h"

namespace UE::AvaShapesEditor::Private
{
	constexpr float LineSnapAngle = 15.f;

	// In angle in degrees
	FVector2f SnapToNearestAngle(const FVector2f& InStartPosition, const FVector2f& InEndPosition, float InSnapAngle)
	{
		if (InSnapAngle < 1 || InSnapAngle > 90)
		{
			return InEndPosition;
		}

		static const FVector2f UpVector = FVector2f(1, 0);

		const FVector2f Vector = InEndPosition - InStartPosition;
		const FVector2f UnitVector = Vector.GetSafeNormal();
		const float DotProduct = UpVector.Dot(UnitVector);
		float Angle = FMath::RadiansToDegrees(FMath::Acos(DotProduct));
		const float Mod = FMath::Fmod(Angle, InSnapAngle);

		if (FMath::IsNearlyZero(Mod))
		{
			return InEndPosition;
		}

		Angle -= Mod;

		if (FMath::Abs(Mod) >= (InSnapAngle *  0.5f))
		{
			Angle += InSnapAngle;
		}

		const float AngleRadians = FMath::DegreesToRadians(Angle);

		// Create snapped vector in correct quadrant
		// May fail if snap angle is not a factor of 180. So let's not do that?
		FVector2f SnappedVector = {
			FMath::Cos(AngleRadians),
			FMath::Sin(AngleRadians) * (Vector.Y >= 0 ? 1.f : -1.f)
		};

		return InStartPosition + (SnappedVector * Vector.Length());
	}
}

UAvaShapesEditorShapeToolLine::UAvaShapesEditorShapeToolLine()
{
	ViewportPlannerClass = UAvaInteractiveToolsToolViewportPointListPlanner::StaticClass();
	ShapeClass = UAvaShapeLineDynamicMesh::StaticClass();
}

void UAvaShapesEditorShapeToolLine::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	const FShapeFactoryParameters LineFactoryParameters =
	{
		.Functor = [](UAvaShapeDynamicMeshBase* InMesh)
		{
			UAvaShapeLineDynamicMesh* Line = Cast<UAvaShapeLineDynamicMesh>(InMesh);
			Line->SetLineWidth(3.f);
		}
	};

	FAvaInteractiveToolsToolParameters ToolParameters = CreateDefaultToolParameters();
	ToolParameters.UICommand = FAvaShapesEditorCommands::Get().Tool_Shape_Line;
	ToolParameters.Priority = 9000;
	ToolParameters.ActorFactory.Set<TObjectPtr<UActorFactory>>(CreateFactory<UAvaShapeLineDynamicMesh>(LineFactoryParameters));

	InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryName2D, MoveTemp(ToolParameters));
}

void UAvaShapesEditorShapeToolLine::OnViewportPlannerUpdate()
{
	Super::OnViewportPlannerUpdate();

	if (UAvaInteractiveToolsToolViewportPointListPlanner* PointListPlanner = Cast<UAvaInteractiveToolsToolViewportPointListPlanner>(ViewportPlanner))
	{
		const TArray<FVector2f>& Positions = PointListPlanner->GetViewportPositions();

		switch (Positions.Num())
		{
			case 0:
				return;

			case 1:
				if (!PreviewActor)
				{
					PreviewActor = SpawnActor(ActorClass, true);
				}

				if (PreviewActor)
				{
					LineEndLocation = PointListPlanner->GetCurrentViewportPosition();

					if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
					{
						using namespace UE::AvaShapesEditor::Private;

						LineEndLocation = SnapToNearestAngle(Positions[0], LineEndLocation, LineSnapAngle);
						PointListPlanner->OverrideCurrentViewportPosition(LineEndLocation);
					}

					SetLineEnds(Cast<AAvaShapeActor>(PreviewActor), Positions[0], LineEndLocation);
				}
				break;
			
			case 2:
			{
				const FVector2f& Center = Positions[0] * 0.5f + Positions[1] * 0.5f;
				SpawnedActor = SpawnActor(ActorClass, EAvaViewportStatus::Focused, Center, /* Preview */ false);
				SetLineEnds(Cast<AAvaShapeActor>(SpawnedActor), Positions[0], Positions[1]);
				RequestShutdown(EToolShutdownType::Completed);
				break;
			}
			default:
				checkNoEntry();
		}
	}
}

void UAvaShapesEditorShapeToolLine::SetShapeSize(AAvaShapeActor* InShapeActor, const FVector2D& InShapeSize) const
{
	Super::SetShapeSize(InShapeActor, InShapeSize);

	if (InShapeActor)
	{
		if (UAvaShapeLineDynamicMesh* LineMesh = Cast<UAvaShapeLineDynamicMesh>(InShapeActor->GetDynamicMesh()))
		{
			LineMesh->SetVector({InShapeSize.X, 0});
		}
	}
}

void UAvaShapesEditorShapeToolLine::SetLineEnds(AAvaShapeActor* InActor, const FVector2f& Start, const FVector2f& End)
{
	if (!InActor)
	{
		return;
	}

	FVector2f ActualEnd = End;

	if ((FMath::Abs(Start.X - End.X) + FMath::Abs(Start.Y - End.Y)) < (MinDim * 2))
	{
		for (int32 Component = 0; Component < 2; ++Component)
		{
			if (FMath::Abs(Start[Component] - End[Component]) < MinDim)
			{
				if (Start[Component] < End[Component])
				{
					ActualEnd[Component] = Start[Component] + MinDim;
				}
				else
				{
					ActualEnd[Component] = Start[Component] - MinDim;
				}
			}
			else
			{
				ActualEnd[Component] = End[Component];
			}
		}
	}

	const FVector2f& Center = Start * 0.5 + ActualEnd * 0.5;
	UWorld* TempWorld;
	FVector TempVector;
	FRotator TempRotator;
	bool bValid = true;

	bValid &= ViewportPositionToWorldPositionAndOrientation(EAvaViewportStatus::Focused, Center, TempWorld, TempVector, TempRotator);
	FVector CenterWorld = TempVector;

	bValid &= ViewportPositionToWorldPositionAndOrientation(EAvaViewportStatus::Focused, Start, TempWorld, TempVector, TempRotator);
	FVector StartWorld = TempVector;

	bValid &= ViewportPositionToWorldPositionAndOrientation(EAvaViewportStatus::Focused, ActualEnd, TempWorld, TempVector, TempRotator);
	FVector EndWorld = TempVector;

	if (bValid)
	{
		if (UAvaShapeLineDynamicMesh* LineMesh = Cast<UAvaShapeLineDynamicMesh>(InActor->GetDynamicMesh()))
		{
			FVector Vector = EndWorld - StartWorld;
			Vector = TempRotator.UnrotateVector(Vector);

			LineMesh->SetMeshRegenWorldLocation(CenterWorld);
			LineMesh->SetVector({Vector.Y, Vector.Z});
		}
	}
}
