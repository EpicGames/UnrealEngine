// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/AvaShapesEditorShapeToolBase.h"
#include "AvaShapesEditorShapeToolLine.generated.h"

UCLASS()
class UAvaShapesEditorShapeToolLine : public UAvaShapesEditorShapeToolBase
{
	GENERATED_BODY()

public:
	UAvaShapesEditorShapeToolLine();

protected:
	// The minimum dimension
	static constexpr float MinDim = 5;

	//~ Begin UAvaInteractiveToolsToolBase
	virtual void OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule) override;
	virtual void OnViewportPlannerUpdate() override;
	virtual void OnViewportPlannerComplete() override {} // Does nothing
	//~ End UAvaInteractiveToolsToolBase
	
	//~ Begin UAvaShapesEditorShapeToolBase
	virtual void SetShapeSize(AAvaShapeActor* InShapeActor, const FVector2D& InShapeSize) const override;
	//~ End UAvaShapesEditorShapeToolBase

	void SetLineEnds(AAvaShapeActor* InShapeActor, const FVector2f& Start, const FVector2f& End);

	FVector2f LineEndLocation = FVector2f::ZeroVector;
};
