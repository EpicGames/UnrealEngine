// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaShapesEditorShapeToolNGon.h"
#include "AvaShapesEditorCommands.h"
#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "DynamicMeshes/AvaShapeNGonDynMesh.h"
#include "UObject/ConstructorHelpers.h"

UAvaShapesEditorShapeToolNGon::UAvaShapesEditorShapeToolNGon()
{
	ShapeClass = UAvaShapeNGonDynamicMesh::StaticClass();
}

void UAvaShapesEditorShapeToolNGon::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	FAvaInteractiveToolsToolParameters ToolParameters = CreateDefaultToolParameters();
	ToolParameters.UICommand = FAvaShapesEditorCommands::Get().Tool_Shape_NGon;
	ToolParameters.Priority = 3000;
	ToolParameters.ActorFactory.Set<TObjectPtr<UActorFactory>>(CreateFactory<UAvaShapeNGonDynamicMesh>());
	
	InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryName2D, MoveTemp(ToolParameters));
}
