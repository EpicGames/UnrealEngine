// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaShapesEditorShapeToolRectangle.h"
#include "AvaShapesEditorCommands.h"
#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "DynamicMeshes/AvaShapeRectangleDynMesh.h"
#include "UObject/ConstructorHelpers.h"

#define LOCTEXT_NAMESPACE "AvaShapesEditorShapeToolRectangle"

UAvaShapesEditorShapeToolRectangle::UAvaShapesEditorShapeToolRectangle()
{
	ShapeClass = UAvaShapeRectangleDynamicMesh::StaticClass();
}

void UAvaShapesEditorShapeToolRectangle::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	FShapeFactoryParameters RectangleFactoryParameters =
	{
		.Size = FVector(0, 160, 90),
		.Functor = [](UAvaShapeDynamicMeshBase* InMesh)
		{
			InMesh->SetMaterialUVMode(UAvaShapeDynamicMeshBase::MESH_INDEX_PRIMARY, EAvaShapeUVMode::Stretch);
		}
	};

	FAvaInteractiveToolsToolParameters RectangleToolParameters = CreateDefaultToolParameters();
	RectangleToolParameters.UICommand = FAvaShapesEditorCommands::Get().Tool_Shape_Rectangle;
	RectangleToolParameters.Priority = 1000;
	RectangleToolParameters.ActorFactory.Set<TObjectPtr<UActorFactory>>(CreateFactory<UAvaShapeRectangleDynamicMesh>(RectangleFactoryParameters));
	RectangleToolParameters.ToolIdentifier = TEXT("Parametric Rectangle Tool");

	InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryName2D, MoveTemp(RectangleToolParameters));

	FShapeFactoryParameters SquareFactoryParameters =
	{
		.NameOverride = FString("Square")
	};

	FAvaInteractiveToolsToolParameters SquareToolParameters = CreateDefaultToolParameters();
	SquareToolParameters.UICommand = FAvaShapesEditorCommands::Get().Tool_Shape_Square;
	SquareToolParameters.Priority = 1100;
	SquareToolParameters.ActorFactory.Set<TObjectPtr<UActorFactory>>(CreateFactory<UAvaShapeRectangleDynamicMesh>(SquareFactoryParameters));
	SquareToolParameters.ToolIdentifier = TEXT("Parametric Square Tool");

	InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryName2D, MoveTemp(SquareToolParameters));
}

#undef LOCTEXT_NAMESPACE
