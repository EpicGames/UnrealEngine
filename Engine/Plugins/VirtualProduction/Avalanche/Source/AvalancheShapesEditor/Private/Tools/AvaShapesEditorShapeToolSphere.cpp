// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaShapesEditorShapeToolSphere.h"
#include "AvaShapesEditorCommands.h"
#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "DynamicMeshes/AvaShapeSphereDynMesh.h"
#include "UObject/ConstructorHelpers.h"

UAvaShapesEditorShapeToolSphere::UAvaShapesEditorShapeToolSphere()
{
	ShapeClass = UAvaShapeSphereDynamicMesh::StaticClass();
}

void UAvaShapesEditorShapeToolSphere::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	FAvaInteractiveToolsToolParameters ToolParameters = CreateDefaultToolParameters();
	ToolParameters.UICommand = FAvaShapesEditorCommands::Get().Tool_Shape_Sphere;
	ToolParameters.Priority = 2000;
	ToolParameters.ActorFactory.Set<TObjectPtr<UActorFactory>>(CreateFactory<UAvaShapeSphereDynamicMesh>());
	
	InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryName3D, MoveTemp(ToolParameters));
}
