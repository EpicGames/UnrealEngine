// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/AvaShapesEditorShapeToolTorus.h"
#include "AvaShapesEditorCommands.h"
#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "DynamicMeshes/AvaShapeTorusDynMesh.h"
#include "UObject/ConstructorHelpers.h"

UAvaShapesEditorShapeToolTorus::UAvaShapesEditorShapeToolTorus()
{
	ShapeClass = UAvaShapeTorusDynamicMesh::StaticClass();
}

void UAvaShapesEditorShapeToolTorus::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	FAvaInteractiveToolsToolParameters ToolParameters = CreateDefaultToolParameters();
	ToolParameters.UICommand = FAvaShapesEditorCommands::Get().Tool_Shape_Torus;
	ToolParameters.Priority = 4000;
	ToolParameters.ActorFactory.Set<TObjectPtr<UActorFactory>>(CreateFactory<UAvaShapeTorusDynamicMesh>());

	InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryName3D, MoveTemp(ToolParameters));
}
