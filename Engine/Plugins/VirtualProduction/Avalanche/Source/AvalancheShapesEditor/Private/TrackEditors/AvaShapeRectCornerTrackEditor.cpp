// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeRectCornerTrackEditor.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneFloatChannel.h"

#define LOCTEXT_NAMESPACE "AvaShapeRectCornerTrackEditor"

TSharedRef<ISequencerTrackEditor> FAvaShapeRectCornerTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InOwningSequencer)
{
	return MakeShared<FAvaShapeRectCornerTrackEditor>(InOwningSequencer);
}

FText FAvaShapeRectCornerTrackEditor::GetDisplayName() const
{
	return LOCTEXT("AvaShapeRectCornerTrackEditor_DisplayName", "Shape Rect Corner");
}

void FAvaShapeRectCornerTrackEditor::GenerateKeysFromPropertyChanged(const FPropertyChangedParams& InPropertyChangedParams
	, UMovieSceneSection* SectionToKey
	, FGeneratedTrackKeys& OutGeneratedKeys)
{
	const FPropertyPath& StructPath = InPropertyChangedParams.StructPathToKey;
	
	const FName ChannelName = StructPath.GetNumProperties() != 0
		? StructPath.GetLeafMostProperty().Property->GetFName()
		: NAME_None;

	FAvaShapeRectangleCornerSettings CornerSettings = InPropertyChangedParams.GetPropertyValue<FAvaShapeRectangleCornerSettings>();

	const bool bKeyType         = ChannelName == GET_MEMBER_NAME_CHECKED(FAvaShapeRectangleCornerSettings, Type)                || ChannelName == NAME_None;
	const bool bKeyBevelSize    = ChannelName == GET_MEMBER_NAME_CHECKED(FAvaShapeRectangleCornerSettings, BevelSize)           || ChannelName == NAME_None;
	const bool bKeySubdivisions = ChannelName == GET_MEMBER_NAME_CHECKED(FAvaShapeRectangleCornerSettings, BevelSubdivisions) || ChannelName == NAME_None;

	int32 ChannelIndex = 0;
	
	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneFloatChannel>(ChannelIndex++
		, CornerSettings.BevelSize
		, bKeyBevelSize));

	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneByteChannel>(ChannelIndex++
		, CornerSettings.BevelSubdivisions
		, bKeySubdivisions));

	OutGeneratedKeys.Add(FMovieSceneChannelValueSetter::Create<FMovieSceneByteChannel>(ChannelIndex++
		, static_cast<uint8>(CornerSettings.Type)
		, bKeyType));
}

#undef LOCTEXT_NAMESPACE
