// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyTrackEditor.h"
#include "Sequencer/MovieScene/MovieSceneAvaShapeRectCornerTrack.h"
#include "DynamicMeshes/AvaShapeRectangleDynMesh.h"

/**
 * A Property track editor for FAvaShapeRectangleCornerSettings.
 */
class FAvaShapeRectCornerTrackEditor : public FPropertyTrackEditor<UMovieSceneAvaShapeRectCornerTrack>
{
public:
	FAvaShapeRectCornerTrackEditor(TSharedRef<ISequencer> InSequencer)
		: FPropertyTrackEditor(InSequencer, GetAnimatedPropertyTypes())
	{
	}
	
	/**
	 * Retrieve a list of all property types that this track editor animates
	 */
	static TArray<FAnimatedPropertyKey, TInlineAllocator<1>> GetAnimatedPropertyTypes()
	{
		UStruct* const StaticStruct = FAvaShapeRectangleCornerSettings::StaticStruct();
		return TArray<FAnimatedPropertyKey, TInlineAllocator<1>>({ FAnimatedPropertyKey::FromStructType(StaticStruct) });
	}
	
	/**
	 * Creates an instance of this class (called by a sequence).
	 * @param InOwningSequencer The sequencer instance to be used by this tool.
	 * @return The new instance of this class.
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> InOwningSequencer);
	
protected:
	//~ Begin FPropertyTrackEditor
	virtual FText GetDisplayName() const override;

	virtual void GenerateKeysFromPropertyChanged(const FPropertyChangedParams& InPropertyChangedParams
		, UMovieSceneSection* InSectionToKey
		, FGeneratedTrackKeys& OutGeneratedKeys) override;
	//~ End FPropertyTrackEditor
};
