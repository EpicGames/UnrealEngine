// Copyright Epic Games, Inc. All Rights Reserved.

#include "Visualizers/AvaShape2DDynMeshVis.h"
#include "AvaField.h"
#include "AvaShapeActor.h"
#include "DynamicMeshes/AvaShape2DDynMeshBase.h"
#include "EditorViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "Interaction/AvaSnapOperation.h"
#include "ScopedTransaction.h"

class FAvaLevelViewportClient;

bool FAvaShape2DDynamicMeshVisualizer::IsEditing() const
{
	if (UVSectionIdx != INDEX_NONE || SizeDragAnchor != INDEX_NONE)
	{
		return true;
	}

	return Super::IsEditing();
}

FAvaShape2DDynamicMeshVisualizer::FAvaShape2DDynamicMeshVisualizer()
	: FAvaShapeDynamicMeshVisualizer()
	, SizeDragAnchor(INDEX_NONE)
	, UVSectionIdx(INDEX_NONE)
	, bEditingUVAnchor(false)
	, InitialPrimaryUVRotation(0)
{
	using namespace UE::AvaCore;
	SizeProperty = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, Size2D));
}

TMap<UObject*, TArray<FProperty*>> FAvaShape2DDynamicMeshVisualizer::GatherEditableProperties(UObject* InObject) const
{
	TMap<UObject*, TArray<FProperty*>> Properties = Super::GatherEditableProperties(InObject);

	if (FMeshType* DynMesh = Cast<FMeshType>(InObject))
	{
		Properties.FindOrAdd(DynMesh).Add(SizeProperty);
	}

	return Properties;
}

bool FAvaShape2DDynamicMeshVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient,
	HComponentVisProxy* InVisProxy, const FViewportClick& InClick)
{
	if (InClick.GetKey() != EKeys::LeftMouseButton)
	{
		EndEditing();
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	FMeshType* DynMesh = Cast<FMeshType>(const_cast<UActorComponent*>(InVisProxy->Component.Get()));

	if (DynMesh == nullptr)
	{
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	if (InVisProxy->IsA(HAvaShapeSizeHitProxy::StaticGetType()))
	{
		if (!DynMesh->GetAllowEditSize())
		{
			return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
		}

		EndEditing();
		SizeDragAnchor = static_cast<HAvaShapeSizeHitProxy*>(InVisProxy)->DragAnchor;
		StartEditing(InViewportClient, DynMesh);

		return true;
	}

	if (InVisProxy->IsA(HAvaShapeUVHitProxy::StaticGetType()))
	{
		const int32 SectionIdx          = static_cast<HAvaShapeUVHitProxy*>(InVisProxy)->SectionIdx;
		const bool bEditingUVAnchorTemp = !bEditingUVAnchor && DynamicMeshComponent.Get() == DynMesh && UVSectionIdx == SectionIdx;

		EndEditing();
		UVSectionIdx = SectionIdx;
		bEditingUVAnchor = bEditingUVAnchorTemp;
		StartEditing(InViewportClient, DynMesh);

		return true;
	}

	return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
}

void FAvaShape2DDynamicMeshVisualizer::DrawVisualizationNotEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationNotEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawUVButton(DynMesh, InView, InPDI, InOutIconIndex, 0, GetIconColor(false));
	++InOutIconIndex;

	if (DynMesh->GetAllowEditSize())
	{
		DrawSizeButtons(DynMesh, InView, InPDI);
	}
}

void FAvaShape2DDynamicMeshVisualizer::DrawVisualizationEditing(const UActorComponent* Component, const FSceneView* View,
	FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationEditing(Component, View, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(Component);

	if (!DynMesh)
	{
		return;
	}

	DrawUVButton(DynMesh, View, InPDI, InOutIconIndex, 0, GetIconColor(UVSectionIdx == 0, bEditingUVAnchor));
	++InOutIconIndex;

	if (DynMesh->GetAllowEditSize())
	{
		DrawSizeButtons(DynMesh, View, InPDI);
	}
}

void FAvaShape2DDynamicMeshVisualizer::DrawSizeButtons(const FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI)
{
	for (const AvaAlignment Alignment : SupportedAlignments)
	{
		if (SizeDragAnchor != Alignment)
		{
			DrawSizeButton(InDynMesh, InView, InPDI, Alignment);
		}
	}
}

bool FAvaShape2DDynamicMeshVisualizer::GetWidgetLocation(const FEditorViewportClient* InViewportClient, FVector& OutLocation) const
{
	const FMeshType* DynMesh2D = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh2D)
	{
		return Super::GetWidgetLocation(InViewportClient, OutLocation);
	}

	if (UVSectionIdx == UAvaShapeDynamicMeshBase::MESH_INDEX_PRIMARY)
	{
		const FVector2D Size2D = DynMesh2D->GetSize2D();

		if (!bEditingUVAnchor)
		{
			const FVector2D UVOffset = -DynMesh2D->GetMaterialUVOffset(UVSectionIdx);
			OutLocation = FVector(-1.f, UVOffset.X * Size2D.X, -UVOffset.Y * Size2D.Y);
			OutLocation = DynMesh2D->GetShapeActor()->GetTransform().TransformPosition(OutLocation);
		}
		else
		{
			const FVector2D UVAnchor = DynMesh2D->GetMaterialUVAnchor(UVSectionIdx);
			OutLocation = FVector(-1.f, UVAnchor.X * Size2D.X - Size2D.X / 2.f, -UVAnchor.Y * Size2D.Y + Size2D.Y / 2.f);
			OutLocation = DynMesh2D->GetShapeActor()->GetTransform().TransformPosition(OutLocation);
		}

		return true;
	}

	if (SizeDragAnchor != INDEX_NONE)
	{
		/*const FVector Size3D = DynMesh2D->GetSize3D();
		OutLocation            = FVector::ZeroVector;
		OutLocation = GetLocationFromAlignment(SizeDragAnchor, Size3D);
		OutLocation = DynMesh2D->GetShapeActor()->GetTransform().TransformPosition(OutLocation);*/
		OutLocation = GetFinalAnchorLocation(DynMesh2D, SizeDragAnchor);
		return true;
	}

	return Super::GetWidgetLocation(InViewportClient, OutLocation);
}

bool FAvaShape2DDynamicMeshVisualizer::GetWidgetMode(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode& OutMode) const
{
	if (UVSectionIdx != INDEX_NONE && bEditingUVAnchor)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Translate;
		return true;
	}

	if (SupportedAlignments.Contains(SizeDragAnchor))
	{
		OutMode = UE::Widget::EWidgetMode::WM_Translate;
		return true;
	}

	return Super::GetWidgetMode(InViewportClient, OutMode);
}

bool FAvaShape2DDynamicMeshVisualizer::GetWidgetAxisList(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (UVSectionIdx != INDEX_NONE)
	{
		if (!bEditingUVAnchor && InWidgetMode == UE::Widget::WM_Rotate)
		{
			OutAxisList = EAxisList::Type::X;
			return true;
		}

		OutAxisList = EAxisList::Type::YZ;
		return true;
	}

	if (SizeDragAnchor != INDEX_NONE)
	{
		const EAvaDepthAlignment Depth = GetDAlignment(SizeDragAnchor);
		const EAvaHorizontalAlignment Horizontal = GetHAlignment(SizeDragAnchor);
		const EAvaVerticalAlignment Vertical = GetVAlignment(SizeDragAnchor);

		uint8 AxisAllowed = EAxisList::Type::None;
		// Y
		if (Horizontal != EAvaHorizontalAlignment::Center)
		{
			AxisAllowed |= EAxisList::Type::Y;
		}
		// Z
		if (Vertical != EAvaVerticalAlignment::Center)
		{
			AxisAllowed |= EAxisList::Type::Z;
		}

		OutAxisList = static_cast<EAxisList::Type>(AxisAllowed);

		return true;
	}

	return Super::GetWidgetAxisList(InViewportClient, InWidgetMode, OutAxisList);
}

void FAvaShape2DDynamicMeshVisualizer::StoreInitialValues()
{
	Super::StoreInitialValues();

	const FMeshType* DynMesh2D = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh2D)
	{
		return;
	}

	InitialSize              = DynMesh2D->GetSize2D();
	InitialPrimaryUVOffset   = DynMesh2D->GetMaterialUVOffset(FMeshType::MESH_INDEX_PRIMARY);
	InitialPrimaryUVScale    = DynMesh2D->GetMaterialUVScale(FMeshType::MESH_INDEX_PRIMARY);
	InitialPrimaryUVRotation = DynMesh2D->GetMaterialUVRotation(FMeshType::MESH_INDEX_PRIMARY);
	InitialPrimaryUVAnchor   = DynMesh2D->GetMaterialUVAnchor(FMeshType::MESH_INDEX_PRIMARY);
}

void FAvaShape2DDynamicMeshVisualizer::SnapLocation2D(FEditorViewportClient* ViewportClient,
	const FTransform& InActorTransform, FVector2D& OutLocation2D) const
{
	if (!SnapOperation.IsValid())
	{
		return;
	}

	if (!this->DynamicMeshComponent.IsValid())
	{
		return;
	}

	if (!this->DynamicMeshComponent->GetShapeActor())
	{
		return;
	}

	const FVector OriginalLocation3D = InActorTransform.TransformPosition(FVector(0.f, OutLocation2D.X, OutLocation2D.Y));
	FVector Location3D               = OriginalLocation3D;

	SnapOperation->SnapLocation(Location3D);FVector SnapOffset             = Location3D - OriginalLocation3D;
	const FMatrix WidgetTransform  = ViewportClient->GetWidgetCoordSystem();
	const EAxisList::Type AxisList = GetViewportWidgetAxisList(ViewportClient);

	SnapOffset = WidgetTransform.InverseTransformVector(SnapOffset);

	if (!(AxisList & EAxisList::X))
	{
		SnapOffset.X = 0.f;
	}

	if (!(AxisList & EAxisList::Y))
	{
		SnapOffset.Y = 0.f;
		SnapOperation->SetSnappedToX(false);
	}

	if (!(AxisList & EAxisList::Z))
	{
		SnapOffset.Z = 0.f;
		SnapOperation->SetSnappedToY(false);
	}

	SnapOffset = WidgetTransform.TransformVector(SnapOffset);

	Location3D = OriginalLocation3D + SnapOffset;
	Location3D = InActorTransform.InverseTransformPosition(Location3D);

	OutLocation2D.X = Location3D.Y;
	OutLocation2D.Y = Location3D.Z;
}

bool FAvaShape2DDynamicMeshVisualizer::IsExtendBothSidesEnabled(FMeshType* InDynMesh) const
{
	return FSlateApplication::Get().GetModifierKeys().IsAltDown();
}

bool FAvaShape2DDynamicMeshVisualizer::HandleInputDeltaInternal(FEditorViewportClient* InViewportClient,
	FViewport* InViewport, const FVector& InAccumulatedTranslation, const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale)
{
	if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
	{
		return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
	}

	FMeshType* DynMesh2D = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh2D)
	{
		EndEditing();
		return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
	}

	if (UVSectionIdx == FMeshType::MESH_INDEX_PRIMARY)
	{
		if (!bEditingUVAnchor)
		{
			// multiply by 0.01 to allow for more precision in the panning
        	constexpr float UVPanDeltaScale = 1.f;

			// control uniform scale when pressing modifier key
			const bool bUniformScale = FSlateApplication::Get().GetModifierKeys().IsShiftDown();

			const FVector2D& Size2D = DynMesh2D->GetSize2D();

			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::YZ)
				{
					FVector2D UVOffset = InitialPrimaryUVOffset;

					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
					{
						UVOffset.X -= InAccumulatedTranslation.Y * UVPanDeltaScale / Size2D.X;
					}

					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
					{
						UVOffset.Y += InAccumulatedTranslation.Z * UVPanDeltaScale / Size2D.Y;
					}

					DynMesh2D->Modify();
					if (DynMesh2D->SetMaterialUVOffset(UVSectionIdx, UVOffset))
					{
						bHasBeenModified = true;
						NotifyPropertyChainModified(DynMesh2D, UVOffsetProperty, EPropertyChangeType::Interactive, UVSectionIdx, {UVParamsProperty, MeshDataProperty});
					}
				}
			}

			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Rotate)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::X)
				{
					float UVRotation = InitialPrimaryUVRotation;
					UVRotation -= InAccumulatedRotation.Roll;

					DynMesh2D->Modify();
					if (DynMesh2D->SetMaterialUVRotation(UVSectionIdx, UVRotation))
					{
						bHasBeenModified = true;
						NotifyPropertyChainModified(DynMesh2D, UVRotationProperty, EPropertyChangeType::Interactive, UVSectionIdx, {UVParamsProperty, MeshDataProperty});
					}
				}
			}

			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Scale)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::YZ)
				{
					FVector2D UVScale = InitialPrimaryUVScale;

					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
					{
						UVScale.X -= InAccumulatedScale.Y * UVPanDeltaScale;

						if (bUniformScale)
						{
							UVScale.Y -= InAccumulatedScale.Y * UVPanDeltaScale;
						}
					}

					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
					{
						UVScale.Y -= InAccumulatedScale.Z * UVPanDeltaScale;

						if (bUniformScale)
						{
							UVScale.X -= InAccumulatedScale.Z * UVPanDeltaScale;
						}
					}

					DynMesh2D->Modify();
					if (DynMesh2D->SetMaterialUVScale(UVSectionIdx, UVScale))
					{
						bHasBeenModified = true;
						NotifyPropertyChainModified(DynMesh2D, UVScaleProperty, EPropertyChangeType::Interactive, UVSectionIdx, {UVParamsProperty, MeshDataProperty});
					}
				}
			}
		}
		else
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::YZ)
				{
					FVector2D Size2D   = DynMesh2D->GetSize2D();
					FVector2D UVAnchor = InitialPrimaryUVAnchor;

					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
					{
						UVAnchor.X = FMath::Clamp(UVAnchor.X + InAccumulatedTranslation.Y / Size2D.X, 0.f, 1.f);
					}

					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
					{
						UVAnchor.Y = FMath::Clamp(UVAnchor.Y - InAccumulatedTranslation.Z / Size2D.Y, 0.f, 1.f);
					}

					DynMesh2D->Modify();
					if (DynMesh2D->SetMaterialUVAnchor(UVSectionIdx, UVAnchor))
					{
						bHasBeenModified = true;
						NotifyPropertyChainModified(DynMesh2D, UVAnchorProperty, EPropertyChangeType::Interactive, UVSectionIdx, {UVParamsProperty, MeshDataProperty});
					}
				}
			}
		}

		return true;
	}

	// ALT + drag
	const bool bExtendBothSide = IsExtendBothSidesEnabled(DynMesh2D);

	if (SizeDragAnchor != INDEX_NONE)
	{
		if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
		{
			if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::YZ)
			{
				FVector2D Size2D       = InitialSize;
				const FTransform Transform   = InitialTransform;
				FVector Location       = FVector::ZeroVector;

				const FVector SizePosition3D = GetLocationFromAlignment(SizeDragAnchor, FVector(0.f, InitialSize.X, InitialSize.Y));
				FVector2D SizePosition = FVector2D(SizePosition3D.Y, SizePosition3D.Z);
				const FVector2D InitialSizePosition = SizePosition;

				const EAvaDepthAlignment Depth = GetDAlignment(SizeDragAnchor);
				const EAvaHorizontalAlignment Horizontal = GetHAlignment(SizeDragAnchor);
				const EAvaVerticalAlignment Vertical = GetVAlignment(SizeDragAnchor);

				// Update translation

				if (Horizontal != EAvaHorizontalAlignment::Center)
				{
					// Y
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
					{
						SizePosition.X += InAccumulatedTranslation.Y;
					}
				}

				if (Vertical != EAvaVerticalAlignment::Center)
				{
					// Z
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
					{
						SizePosition.Y += InAccumulatedTranslation.Z;
					}
				}

				SnapLocation2D(InViewportClient, InitialTransform, SizePosition);

				FVector2D SizePositionChange = SizePosition - InitialSizePosition;

				// Update mesh Size2D

				if (Horizontal != EAvaHorizontalAlignment::Center)
				{
					// Y
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
					{
						Size2D.X += SizePositionChange.X * (Horizontal == EAvaHorizontalAlignment::Left ? -1 : 1);
					}
				}

				if (Vertical != EAvaVerticalAlignment::Center)
				{
					// Z
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
					{
						Size2D.Y += SizePositionChange.Y * (Vertical == EAvaVerticalAlignment::Bottom ? -1 : 1);
					}
				}

				// Update mesh location

				if (!bExtendBothSide)
				{
					FVector2D SizeChange = (Size2D - InitialSize) / 2.f;

					if (Horizontal != EAvaHorizontalAlignment::Center)
					{
						// Y
						if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
						{
							Location.Y += SizeChange.X * (Horizontal == EAvaHorizontalAlignment::Left ? -1 : 1);
						}
					}

					if (Vertical != EAvaVerticalAlignment::Center)
					{
						// Z
						if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
						{
							Location.Z += SizeChange.Y * (Vertical == EAvaVerticalAlignment::Bottom ? -1 : 1);
						}
					}
				}

				Size2D = FVector2D::Max(Size2D, UAvaShapeDynamicMeshBase::MinSize2D);
				DynMesh2D->Modify();
				DynMesh2D->SetSize2D(Size2D);

				if (DynMesh2D->GetSize2D() != PrevShapeSize)
				{
					PrevShapeSize = Size2D;
					if (!bExtendBothSide)
					{
						Location = Transform.TransformPosition(Location);
						DynMesh2D->SetMeshRegenWorldLocation(Location, true);
						bHasBeenModified = true;
						NotifyPropertiesModified(DynMesh2D, {MeshRegenWorldLocationProperty}, EPropertyChangeType::Interactive);
					}
				}
			}
		}

		return true;
	}

	return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
}

bool FAvaShape2DDynamicMeshVisualizer::HandleModifiedClick(FEditorViewportClient* InViewportClient,
	HHitProxy* InHitProxy, const FViewportClick& InClick)
{
	if (!InHitProxy->IsA(HAvaShapeUVHitProxy::StaticGetType()))
	{
		return Super::HandleModifiedClick(InViewportClient, InHitProxy, InClick);
	}

	// Only do anything if we have either control or shift down... and not alt
	if (InClick.IsAltDown() || InClick.IsControlDown() == InClick.IsShiftDown())
	{
		return Super::HandleModifiedClick(InViewportClient, InHitProxy, InClick);
	}

	const HAvaShapeUVHitProxy* UVHitProxy = static_cast<HAvaShapeUVHitProxy*>(InHitProxy);
	FMeshType* HitProxyDynamicMesh    = Cast<FMeshType>(const_cast<UActorComponent*>(UVHitProxy->Component.Get()));

	if (HitProxyDynamicMesh && UVHitProxy->SectionIdx != INDEX_NONE)
	{
		if (InClick.IsControlDown())
		{
			const EAvaShapeUVMode UVMode = HitProxyDynamicMesh->GetMaterialUVMode(UVHitProxy->SectionIdx);

			switch (UVMode)
			{
				case EAvaShapeUVMode::Stretch:
					HitProxyDynamicMesh->Modify();
					HitProxyDynamicMesh->SetMaterialUVMode(UVHitProxy->SectionIdx,
						EAvaShapeUVMode::Uniform);
					break;

				case EAvaShapeUVMode::Uniform:
					HitProxyDynamicMesh->Modify();
					HitProxyDynamicMesh->SetMaterialUVMode(UVHitProxy->SectionIdx, EAvaShapeUVMode::Stretch);
					break;

				default:
					// Do nothing
					break;
			}

			NotifyPropertyChainModified(HitProxyDynamicMesh, UVModeProperty, EPropertyChangeType::Interactive, UVHitProxy->SectionIdx, {UVParamsProperty, MeshDataProperty});
		}
		// Toggle flip flags
		else if (InClick.IsShiftDown())
		{
			const bool bHorizFlip    = HitProxyDynamicMesh->GetMaterialHorizontalFlip(UVHitProxy->SectionIdx);
			const bool bVerticalFlip = HitProxyDynamicMesh->GetMaterialVerticalFlip(UVHitProxy->SectionIdx);

			if (!bHorizFlip && !bVerticalFlip)
			{
				HitProxyDynamicMesh->Modify();
				HitProxyDynamicMesh->SetMaterialHorizontalFlip(UVHitProxy->SectionIdx, true);
				NotifyPropertyChainModified(HitProxyDynamicMesh, UVHorizFlipProperty, EPropertyChangeType::Interactive, UVHitProxy->SectionIdx, {UVParamsProperty, MeshDataProperty});
			}
			else if (bHorizFlip && !bVerticalFlip)
			{
				HitProxyDynamicMesh->Modify();
				HitProxyDynamicMesh->SetMaterialVerticalFlip(UVHitProxy->SectionIdx, true);
				NotifyPropertyChainModified(HitProxyDynamicMesh, UVVertFlipProperty, EPropertyChangeType::Interactive, UVHitProxy->SectionIdx, {UVParamsProperty, MeshDataProperty});
			}
			else if (bHorizFlip && bVerticalFlip)
			{
				HitProxyDynamicMesh->Modify();
				HitProxyDynamicMesh->SetMaterialHorizontalFlip(UVHitProxy->SectionIdx, false);
				NotifyPropertyChainModified(HitProxyDynamicMesh, UVHorizFlipProperty, EPropertyChangeType::Interactive, UVHitProxy->SectionIdx, {UVParamsProperty, MeshDataProperty});
			}
			else
			{
				HitProxyDynamicMesh->Modify();
				HitProxyDynamicMesh->SetMaterialVerticalFlip(UVHitProxy->SectionIdx, false);
				NotifyPropertyChainModified(HitProxyDynamicMesh, UVVertFlipProperty, EPropertyChangeType::Interactive, UVHitProxy->SectionIdx, {UVParamsProperty, MeshDataProperty});
			}
		}
	}

	return true;
}

bool FAvaShape2DDynamicMeshVisualizer::ResetValue(FEditorViewportClient* InViewportClient,
	HHitProxy* InHitProxy)
{
	if (!InHitProxy->IsA(HAvaShapeUVHitProxy::StaticGetType()))
	{
		return Super::ResetValue(InViewportClient, InHitProxy);
	}

	const HAvaShapeUVHitProxy* UVHitProxy = static_cast<HAvaShapeUVHitProxy*>(InHitProxy);

	if (UVHitProxy->SectionIdx != UVSectionIdx)
	{
		return true;
	}

	FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(const_cast<UActorComponent*>(UVHitProxy->Component.Get()));

	if (!HitProxyDynamicMesh)
	{
		return false;
	}

	FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
	HitProxyDynamicMesh->SetFlags(RF_Transactional);
	HitProxyDynamicMesh->Modify();

	if (bEditingUVAnchor)
	{
		HitProxyDynamicMesh->SetMaterialUVAnchor(UVHitProxy->SectionIdx, FVector2D::ZeroVector);
		NotifyPropertyChainModified(HitProxyDynamicMesh, UVAnchorProperty, EPropertyChangeType::ValueSet, UVHitProxy->SectionIdx, {UVParamsProperty, MeshDataProperty});
	}
	else
	{
		switch (InViewportClient->GetWidgetMode())
		{
			case UE::Widget::WM_Translate:
				HitProxyDynamicMesh->SetMaterialUVOffset(UVHitProxy->SectionIdx, FVector2D::ZeroVector);
				NotifyPropertyChainModified(HitProxyDynamicMesh, UVOffsetProperty, EPropertyChangeType::ValueSet, UVHitProxy->SectionIdx, {UVParamsProperty, MeshDataProperty});
				break;

			case UE::Widget::WM_Rotate:
				HitProxyDynamicMesh->SetMaterialUVRotation(UVHitProxy->SectionIdx, 0.f);
				NotifyPropertyChainModified(HitProxyDynamicMesh, UVRotationProperty, EPropertyChangeType::ValueSet, UVHitProxy->SectionIdx, {UVParamsProperty, MeshDataProperty});
				break;

			case UE::Widget::WM_Scale:
				HitProxyDynamicMesh->SetMaterialUVScale(UVHitProxy->SectionIdx, FVector2D(1.f, 1.f));
				NotifyPropertyChainModified(HitProxyDynamicMesh, UVScaleProperty, EPropertyChangeType::ValueSet, UVHitProxy->SectionIdx, {UVParamsProperty, MeshDataProperty});
				break;

			default:
				// Nothing
				break;
		}
	}

	return true;
}

void FAvaShape2DDynamicMeshVisualizer::EndEditing()
{
	Super::EndEditing();

	UVSectionIdx     = INDEX_NONE;
	bEditingUVAnchor = false;
	SizeDragAnchor   = INDEX_NONE;
}
