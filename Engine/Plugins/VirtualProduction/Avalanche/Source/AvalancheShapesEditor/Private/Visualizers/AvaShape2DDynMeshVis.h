// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaShapeDynMeshVis.h"
#include "DynamicMeshes/AvaShape2DDynMeshBase.h"

class UAvaShape2DDynMeshBase;

class FAvaShape2DDynamicMeshVisualizer : public FAvaShapeDynamicMeshVisualizer
{
public:
	using Super = FAvaShapeDynamicMeshVisualizer;
	using FMeshType = UAvaShape2DDynMeshBase;

	FAvaShape2DDynamicMeshVisualizer();

	//~ Begin FAvaVisualizerBase
	virtual TMap<UObject*, TArray<FProperty*>> GatherEditableProperties(UObject* InObject) const override;
	virtual bool VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* InVisProxy,
		const FViewportClick& InClick) override;
	virtual bool GetWidgetLocation(const FEditorViewportClient* InViewportClient, FVector& OutLocation) const override;
	virtual bool GetWidgetMode(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode& OutMode) const override;
	virtual bool GetWidgetAxisList(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode,
		EAxisList::Type& OutAxisList) const override;
	virtual bool HandleModifiedClick(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy, const FViewportClick& InClick) override;
	virtual bool ResetValue(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy) override;
	virtual bool IsEditing() const override;
	virtual void EndEditing() override;
	//~ End FAvaVisualizerBase

protected:
	FProperty* SizeProperty;

	AvaAlignment SizeDragAnchor; // XYZ
	FVector2D InitialSize;
	
	// Supported alignments
	// Center alignments removed on request.
	static inline const TArray<AvaAlignment> SupportedAlignments {
		MakeAlignment(EAvaDepthAlignment::Back, EAvaHorizontalAlignment::Left, EAvaVerticalAlignment::Top), // EAvaAnchors::TopLeft
		MakeAlignment(EAvaDepthAlignment::Back, EAvaHorizontalAlignment::Right, EAvaVerticalAlignment::Top), //  EAvaAnchors::TopRight
		MakeAlignment(EAvaDepthAlignment::Back, EAvaHorizontalAlignment::Left, EAvaVerticalAlignment::Bottom), // EAvaAnchors::BottomLeft
		MakeAlignment(EAvaDepthAlignment::Back, EAvaHorizontalAlignment::Right, EAvaVerticalAlignment::Bottom), // EAvaAnchors::BottomRight
	};

	int32 UVSectionIdx;
	bool bEditingUVAnchor;
	FVector2D InitialPrimaryUVOffset;
	FVector2D InitialPrimaryUVScale;
	float InitialPrimaryUVRotation;
	FVector2D InitialPrimaryUVAnchor;
	FVector2D PrevShapeSize;

	virtual void DrawVisualizationNotEditing(const UActorComponent* InComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;
	virtual void DrawVisualizationEditing(const UActorComponent* Component, const FSceneView* View,
		FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;

	virtual void DrawSizeButtons(const FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI);

	virtual void StoreInitialValues() override;

	virtual bool HandleInputDeltaInternal(FEditorViewportClient* InViewportClient, FViewport* InViewport, const FVector& InAccumulatedTranslation,
		const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale) override;

	void SnapLocation2D(FEditorViewportClient* InViewportClient, const FTransform& InActorTransform, FVector2D& OutLocation2D) const;

	bool IsExtendBothSidesEnabled(FMeshType* InDynMesh) const;
};
