// Copyright Epic Games, Inc. All Rights Reserved.

#include "Visualizers/AvaShape3DDynMeshVis.h"
#include "AvaField.h"
#include "AvaShapeActor.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMeshes/AvaShape3DDynMeshBase.h"
#include "EditorViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "Interaction/AvaSnapOperation.h"

FAvaShape3DDynamicMeshVisualizer::FAvaShape3DDynamicMeshVisualizer()
	: FAvaShapeDynamicMeshVisualizer()
	, SizeDragAnchor(INDEX_NONE)
{
	using namespace UE::AvaCore;
	SizeProperty = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, Size3D));
}

TMap<UObject*, TArray<FProperty*>> FAvaShape3DDynamicMeshVisualizer::GatherEditableProperties(UObject* InObject) const
{
	TMap<UObject*, TArray<FProperty*>> Properties = Super::GatherEditableProperties(InObject);

	if (FMeshType* DynMesh = Cast<FMeshType>(InObject))
	{
		Properties.FindOrAdd(DynMesh).Add(SizeProperty);
	}

	return Properties;
}

bool FAvaShape3DDynamicMeshVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* InVisProxy,
	const FViewportClick& InClick)
{
	if (InClick.GetKey() != EKeys::LeftMouseButton)
	{ 
		EndEditing();
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	FMeshType* DynMesh = Cast<FMeshType>(const_cast<UActorComponent*>(InVisProxy->Component.Get()));

	if (DynMesh == nullptr)
	{
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	// size
	if (InVisProxy->IsA(HAvaShapeSizeHitProxy::StaticGetType()))
	{
		if (!DynMesh->GetAllowEditSize())
		{
			return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
		}

		EndEditing();
		SizeDragAnchor = static_cast<HAvaShapeSizeHitProxy*>(InVisProxy)->DragAnchor;
		StartEditing(InViewportClient, DynMesh);

		return true;
	}
	
	return FAvaShapeDynamicMeshVisualizer::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
}

bool FAvaShape3DDynamicMeshVisualizer::GetWidgetLocation(const FEditorViewportClient* InViewportClient, FVector& OutLocation) const
{
	const FMeshType* DynMesh3D = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh3D)
	{
		return Super::GetWidgetLocation(InViewportClient, OutLocation);
	}

	if (SizeDragAnchor != INDEX_NONE)
	{
		/*const FBox Box = DynMesh3D->GetShapeMeshComponent()->GetLocalBounds().GetBox();
		const FVector Size3D = Box.GetSize(); //  DynMesh3D->GetSize3D();
		OutLocation = FVector::ZeroVector;
		OutLocation = GetLocationFromAlignment(SizeDragAnchor, Size3D);
		OutLocation = DynMesh3D->GetShapeActor()->GetTransform().TransformPosition(OutLocation);*/
		OutLocation = GetFinalAnchorLocation(DynMesh3D, SizeDragAnchor);
		return true;
	}

	return Super::GetWidgetLocation(InViewportClient, OutLocation);
}

bool FAvaShape3DDynamicMeshVisualizer::GetWidgetMode(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode& OutMode) const
{
	return FAvaShapeDynamicMeshVisualizer::GetWidgetMode(InViewportClient, OutMode);
}

bool FAvaShape3DDynamicMeshVisualizer::GetWidgetAxisList(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (SizeDragAnchor != INDEX_NONE)
	{
		const EAvaDepthAlignment Depth = GetDAlignment(SizeDragAnchor);
		const EAvaHorizontalAlignment Horizontal = GetHAlignment(SizeDragAnchor);
		const EAvaVerticalAlignment Vertical = GetVAlignment(SizeDragAnchor);

		uint8 AxisAllowed = EAxisList::Type::None;
		// X
		if (Depth != EAvaDepthAlignment::Center)
		{
			AxisAllowed |= EAxisList::Type::X;
		}
		// Y
		if (Horizontal != EAvaHorizontalAlignment::Center)
		{
			AxisAllowed |= EAxisList::Type::Y;
		}
		// Z
		if (Vertical != EAvaVerticalAlignment::Center)
		{
			AxisAllowed |= EAxisList::Type::Z;
		}

		OutAxisList = static_cast<EAxisList::Type>(AxisAllowed);

		return true;
	}

	return Super::GetWidgetAxisList(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShape3DDynamicMeshVisualizer::HandleModifiedClick(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy,
	const FViewportClick& InClick)
{
	return FAvaShapeDynamicMeshVisualizer::HandleModifiedClick(InViewportClient, InHitProxy, InClick);
}

bool FAvaShape3DDynamicMeshVisualizer::ResetValue(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy)
{
	return FAvaShapeDynamicMeshVisualizer::ResetValue(InViewportClient, InHitProxy);
}

bool FAvaShape3DDynamicMeshVisualizer::IsEditing() const
{
	if (SizeDragAnchor != INDEX_NONE)
	{
		return true;
	}

	return Super::IsEditing();
}

void FAvaShape3DDynamicMeshVisualizer::EndEditing()
{
	Super::EndEditing();
	
	SizeDragAnchor = INDEX_NONE;
}

void FAvaShape3DDynamicMeshVisualizer::DrawVisualizationNotEditing(const UActorComponent* InComponent, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationNotEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	if (DynMesh->GetAllowEditSize())
	{
		for (const AvaAlignment Alignment : SupportedAlignments)
		{
			DrawSizeButton(DynMesh, InView, InPDI, Alignment);
		}
	}
}

void FAvaShape3DDynamicMeshVisualizer::DrawVisualizationEditing(const UActorComponent* InComponent, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	if (DynMesh->GetAllowEditSize())
	{
		for (const AvaAlignment Alignment : SupportedAlignments)
		{
			DrawSizeButton(DynMesh, InView, InPDI, Alignment);
		}
	}
}

void FAvaShape3DDynamicMeshVisualizer::StoreInitialValues()
{
	Super::StoreInitialValues();

	const FMeshType* DynMesh3D = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh3D)
	{
		return;
	}
	
	InitialSize = DynMesh3D->GetSize3D();
}

bool FAvaShape3DDynamicMeshVisualizer::HandleInputDeltaInternal(FEditorViewportClient* InViewportClient, FViewport* InViewport,
	const FVector& InAccumulatedTranslation, const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale)
{
	if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
	{
		return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
	}

	FMeshType* DynMesh3D = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh3D)
	{
		EndEditing();
		return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
	}

	// ALT + drag
	bool bExtendBothSide = false;

	if (FSlateApplication::Get().GetModifierKeys().IsAltDown())
	{
		bExtendBothSide = true;
	}

	if (SizeDragAnchor != INDEX_NONE)
	{
		if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
		{
			if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::XYZ)
			{
				FVector Size3D		   = InitialSize;
				const FTransform Transform   = InitialTransform;
				FVector Location       = FVector::ZeroVector;
				
				FVector SizePosition = GetLocationFromAlignment(SizeDragAnchor, InitialSize);
				const FVector InitialSizePosition = SizePosition;

				const EAvaDepthAlignment Depth = GetDAlignment(SizeDragAnchor);
				const EAvaHorizontalAlignment Horizontal = GetHAlignment(SizeDragAnchor);
				const EAvaVerticalAlignment Vertical = GetVAlignment(SizeDragAnchor);

				// Update translation

				if (Depth != EAvaDepthAlignment::Center)
				{
					// X
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::X)
					{
						SizePosition.X += InAccumulatedTranslation.X;
					}
				}
				
				if (Horizontal != EAvaHorizontalAlignment::Center)
				{
					// Y
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
					{
						SizePosition.Y += InAccumulatedTranslation.Y;
					}
				}
				
				if (Vertical != EAvaVerticalAlignment::Center)
				{
					// Z
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
					{
						SizePosition.Z += InAccumulatedTranslation.Z;
					}
				}

				SnapLocation3D(InViewportClient, InitialTransform, SizePosition);

				const FVector SizePositionChange = SizePosition - InitialSizePosition;

				// Update mesh Size3D
				
				if (Depth != EAvaDepthAlignment::Center)
				{
					// X
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::X)
					{
						Size3D.X += SizePositionChange.X * (Depth == EAvaDepthAlignment::Back ? -1 : 1);
					}
				}

				if (Horizontal != EAvaHorizontalAlignment::Center)
				{
					// Y
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
					{
						Size3D.Y += SizePositionChange.Y * (Horizontal == EAvaHorizontalAlignment::Left ? -1 : 1);
					}
				}

				if (Vertical != EAvaVerticalAlignment::Center)
				{
					// Z
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
					{
						Size3D.Z += SizePositionChange.Z * (Vertical == EAvaVerticalAlignment::Bottom ? -1 : 1);
					}
				}

				// Update mesh location

				if (!bExtendBothSide)
				{
					const FVector SizeChange = (Size3D - InitialSize) / 2.f;
					
					if (Depth != EAvaDepthAlignment::Center)
					{
						// X
						if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::X)
						{
							Location.X += SizeChange.X * (Depth == EAvaDepthAlignment::Back ? -1 : 1);
						}
					}

					if (Horizontal != EAvaHorizontalAlignment::Center)
					{
						// Y
						if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
						{
							Location.Y += SizeChange.Y * (Horizontal == EAvaHorizontalAlignment::Left ? -1 : 1);
						}
					}

					if (Vertical != EAvaVerticalAlignment::Center)
					{
						// Z
						if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
						{
							Location.Z += SizeChange.Z * (Vertical == EAvaVerticalAlignment::Bottom ? -1 : 1);
						}
					}
				}

				Size3D = FVector::Max(Size3D, UAvaShapeDynamicMeshBase::MinSize3D);
				DynMesh3D->Modify();
				DynMesh3D->SetSize3D(Size3D);

				if (!bExtendBothSide)
				{
					Location = Transform.TransformPosition(Location);
					DynMesh3D->SetMeshRegenWorldLocation(Location, true);
					bHasBeenModified = true;
					NotifyPropertiesModified(DynMesh3D, {MeshRegenWorldLocationProperty}, EPropertyChangeType::Interactive);
				}
			}
		}

		return true;
	}

	return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
}

void FAvaShape3DDynamicMeshVisualizer::SnapLocation3D(FEditorViewportClient* InViewportClient,
	const FTransform& InActorTransform, FVector& OutLocation) const
{
	if (!SnapOperation.IsValid())
	{
		return;
	}

	if (!this->DynamicMeshComponent.IsValid())
	{
		return;
	}

	if (!this->DynamicMeshComponent->GetShapeActor())
	{
		return;
	}

	const FVector OriginalLocation3D = InActorTransform.TransformPosition(OutLocation);
	FVector Location3D               = OriginalLocation3D;

	SnapOperation->SnapLocation(Location3D);

	FVector SnapOffset             = Location3D - OriginalLocation3D;
	const FMatrix WidgetTransform  = InViewportClient->GetWidgetCoordSystem();
	const EAxisList::Type AxisList = GetViewportWidgetAxisList(InViewportClient);

	SnapOffset = WidgetTransform.InverseTransformVector(SnapOffset);

	if (!(AxisList & EAxisList::X))
	{
		SnapOffset.X = 0.f;
	}

	if (!(AxisList & EAxisList::Y))
	{
		SnapOffset.Y = 0.f;
		SnapOperation->SetSnappedToX(false);
	}

	if (!(AxisList & EAxisList::Z))
	{
		SnapOffset.Z = 0.f;
		SnapOperation->SetSnappedToY(false);
	}

	SnapOffset = WidgetTransform.TransformVector(SnapOffset);

	Location3D = OriginalLocation3D + SnapOffset;
	Location3D = InActorTransform.InverseTransformPosition(Location3D);

	OutLocation = Location3D;
}
