// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaShapeDynMeshVis.h"

class UAvaShape3DDynMeshBase;

class FAvaShape3DDynamicMeshVisualizer : public FAvaShapeDynamicMeshVisualizer
{
public:
	using Super = FAvaShapeDynamicMeshVisualizer;
	using FMeshType = UAvaShape3DDynMeshBase;
	
	FAvaShape3DDynamicMeshVisualizer();

	//~ Begin FAvaVisualizerBase
	virtual TMap<UObject*, TArray<FProperty*>> GatherEditableProperties(UObject* InObject) const override;
	virtual bool VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* InVisProxy,
		const FViewportClick& InClick) override;
	virtual bool GetWidgetLocation(const FEditorViewportClient* InViewportClient, FVector& OutLocation) const override;
	virtual bool GetWidgetMode(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode& OutMode) const override;
	virtual bool GetWidgetAxisList(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode,
		EAxisList::Type& OutAxisList) const override;
	virtual bool HandleModifiedClick(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy, const FViewportClick& InClick) override;
	virtual bool ResetValue(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy) override;
	virtual bool IsEditing() const override;
	virtual void EndEditing() override;
	//~ End FAvaVisualizerBase

protected:

	// direct access to size property to notify about update
	FProperty* SizeProperty;
	// this gives information about current handle XYZ we are updating
	AvaAlignment SizeDragAnchor;
	// size before any update with visualizer
	FVector InitialSize;
	
	// Supported alignments
	// Center alignments removed on request.
	static inline const TArray<AvaAlignment> SupportedAlignments {
		/* back */
		MakeAlignment(EAvaDepthAlignment::Back, EAvaHorizontalAlignment::Left, EAvaVerticalAlignment::Top),
		MakeAlignment(EAvaDepthAlignment::Back, EAvaHorizontalAlignment::Right, EAvaVerticalAlignment::Top),
		MakeAlignment(EAvaDepthAlignment::Back, EAvaHorizontalAlignment::Left, EAvaVerticalAlignment::Bottom),
		MakeAlignment(EAvaDepthAlignment::Back, EAvaHorizontalAlignment::Right, EAvaVerticalAlignment::Bottom),
		/* front */
		MakeAlignment(EAvaDepthAlignment::Front, EAvaHorizontalAlignment::Left, EAvaVerticalAlignment::Top),
		MakeAlignment(EAvaDepthAlignment::Front, EAvaHorizontalAlignment::Right, EAvaVerticalAlignment::Top),
		MakeAlignment(EAvaDepthAlignment::Front, EAvaHorizontalAlignment::Left, EAvaVerticalAlignment::Bottom),
		MakeAlignment(EAvaDepthAlignment::Front, EAvaHorizontalAlignment::Right, EAvaVerticalAlignment::Bottom),
	};
	
	virtual void DrawVisualizationNotEditing(const UActorComponent* InComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;
	virtual void DrawVisualizationEditing(const UActorComponent* InComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;
	virtual void StoreInitialValues() override;
	virtual bool HandleInputDeltaInternal(FEditorViewportClient* InViewportClient, FViewport* InViewport, const FVector& InAccumulatedTranslation,
		const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale) override;
	
	void SnapLocation3D(FEditorViewportClient* InViewportClient, const FTransform& InActorTransform, FVector& OutLocation) const;
};
