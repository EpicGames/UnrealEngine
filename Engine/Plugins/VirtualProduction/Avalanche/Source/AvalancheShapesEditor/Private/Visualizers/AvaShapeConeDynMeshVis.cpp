// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeConeDynMeshVis.h"
#include "AvaField.h"
#include "AvaShapeActor.h"
#include "AvaShapeSprites.h"
#include "DynamicMeshes/AvaShapeConeDynMesh.h"
#include "EditorViewportClient.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "IAvaComponentVisualizersSettings.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "ScopedTransaction.h"
#include "TextureResource.h"

FAvaShapeConeDynamicMeshVisualizer::FAvaShapeConeDynamicMeshVisualizer()
	: FAvaShape3DDynamicMeshVisualizer()
	, bEditingTopRadius(false)
	, InitialTopRadius(0)
	, bEditingNumSides(false)
	, InitialNumSides(0)
	, bEditingAngleDegree(false)
	, InitialAngleDegree(0)
{
	using namespace UE::AvaCore;
	TopRadiusProperty   = GetProperty<UAvaShapeConeDynamicMesh>(GET_MEMBER_NAME_CHECKED(UAvaShapeConeDynamicMesh, TopRadius));
	NumSidesProperty    = GetProperty<UAvaShapeConeDynamicMesh>(GET_MEMBER_NAME_CHECKED(UAvaShapeConeDynamicMesh, NumSides));
	AngleDegreeProperty = GetProperty<UAvaShapeConeDynamicMesh>(GET_MEMBER_NAME_CHECKED(UAvaShapeConeDynamicMesh, AngleDegree));
}

TMap<UObject*, TArray<FProperty*>> FAvaShapeConeDynamicMeshVisualizer::GatherEditableProperties(UObject* InObject) const
{
	TMap<UObject*, TArray<FProperty*>> Properties = Super::GatherEditableProperties(InObject);

	if (FMeshType* DynMesh = Cast<FMeshType>(InObject))
	{
		Properties.FindOrAdd(DynMesh).Add(TopRadiusProperty);
		Properties.FindOrAdd(DynMesh).Add(NumSidesProperty);
		Properties.FindOrAdd(DynMesh).Add(AngleDegreeProperty);
	}

	return Properties;
}

bool FAvaShapeConeDynamicMeshVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient,
	HComponentVisProxy* InVisProxy, const FViewportClick& InClick)
{
	if (InClick.GetKey() != EKeys::LeftMouseButton)
	{
		EndEditing();
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	FMeshType* DynMesh = Cast<FMeshType>(const_cast<UActorComponent*>(InVisProxy->Component.Get()));

	if (DynMesh == nullptr)
	{
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	// num sides
	if (InVisProxy->IsA(HAvaShapeNumSidesHitProxy::StaticGetType()))
	{
		EndEditing();
		bEditingNumSides    = true;
		StartEditing(InViewportClient, DynMesh);
		return true;
	}
	// top radius
	if (InVisProxy->IsA(HAvaShapeInnerSizeHitProxy::StaticGetType()))
	{
		EndEditing();
		bEditingTopRadius    = true;
		StartEditing(InViewportClient, DynMesh);
		return true;
	}
	// angle degree
	if (InVisProxy->IsA(HAvaShapeAngleDegreeHitProxy::StaticGetType()))
	{
		EndEditing();
		bEditingAngleDegree    = true;
		StartEditing(InViewportClient, DynMesh);
		return true;
	}

	return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
}

bool FAvaShapeConeDynamicMeshVisualizer::GetWidgetLocation(const FEditorViewportClient* InViewportClient,
	FVector& OutLocation) const
{
	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (DynMesh)
	{
		if (bEditingNumSides)
		{
			OutLocation = DynMesh->GetShapeActor()->GetActorLocation();
			return true;
		}

		if (bEditingTopRadius)
		{
			const float OffsetY  = DynMesh->GetSize3D().X * DynMesh->GetTopRadius() / 2.f;
			const float OffsetZ  = DynMesh->GetSize3D().Z / 2.f;
			const FVector Offset = FVector(0.f, OffsetY, OffsetZ);
			OutLocation          = DynamicMeshComponent->GetShapeActor()->GetTransform().TransformPosition(Offset);
			return true;
		}

		if(bEditingAngleDegree)
		{
			const float OffsetZ  = DynMesh->GetSize3D().Z / 2.f;
			const FVector Offset = FVector(0.f, 0.f, -OffsetZ);
			OutLocation          = DynamicMeshComponent->GetShapeActor()->GetTransform().TransformPosition(Offset);
			return true;
		}
	}

	return Super::GetWidgetLocation(InViewportClient, OutLocation);
}

bool FAvaShapeConeDynamicMeshVisualizer::GetWidgetMode(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode& OutMode) const
{
	if (bEditingNumSides)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Scale;
		return true;
	}

	if (bEditingTopRadius)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Translate;
		return true;
	}

	if (bEditingAngleDegree)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Rotate;
		return true;
	}

	return Super::GetWidgetMode(InViewportClient, OutMode);
}

bool FAvaShapeConeDynamicMeshVisualizer::GetWidgetAxisList(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingNumSides)
	{
		OutAxisList = EAxisList::Type::Y;
		return true;
	}

	if (bEditingTopRadius)
	{
		OutAxisList = EAxisList::Type::Screen;
		return true;
	}

	if (bEditingAngleDegree)
	{
		OutAxisList = EAxisList::Type::Z;
		return true;
	}

	return Super::GetWidgetAxisList(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShapeConeDynamicMeshVisualizer::GetWidgetAxisListDragOverride(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingTopRadius)
	{
		OutAxisList = EAxisList::Type::Y;
		return true;
	}

	return Super::GetWidgetAxisListDragOverride(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShapeConeDynamicMeshVisualizer::ResetValue(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy)
{
	if (!InHitProxy->IsA(HAvaShapeNumSidesHitProxy::StaticGetType()) &&
		!InHitProxy->IsA(HAvaShapeInnerSizeHitProxy::StaticGetType()) &&
			!InHitProxy->IsA(HAvaShapeAngleDegreeHitProxy::StaticGetType()) )
	{
		return Super::ResetValue(InViewportClient, InHitProxy);
	}

	// num side
	if (InHitProxy->IsA(HAvaShapeNumSidesHitProxy::StaticGetType()))
	{
		const HAvaShapeNumSidesHitProxy* NumSidesHitProxy = static_cast<HAvaShapeNumSidesHitProxy*>(InHitProxy);

		if (NumSidesHitProxy->Component.IsValid())
		{
			FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(
				const_cast<UActorComponent*>(NumSidesHitProxy->Component.Get()));

			if (HitProxyDynamicMesh)
			{
				FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
				HitProxyDynamicMesh->SetFlags(RF_Transactional);
				HitProxyDynamicMesh->Modify();
				HitProxyDynamicMesh->SetNumSides(64);
				NotifyPropertyModified(HitProxyDynamicMesh, NumSidesProperty, EPropertyChangeType::ValueSet);
			}
		}
	}
	// top radius
	else if (InHitProxy->IsA(HAvaShapeInnerSizeHitProxy::StaticGetType()))
	{
		const HAvaShapeInnerSizeHitProxy* TopRadiusHitProxy = static_cast<HAvaShapeInnerSizeHitProxy*>(InHitProxy);

		if (TopRadiusHitProxy->Component.IsValid())
		{
			FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(
				const_cast<UActorComponent*>(TopRadiusHitProxy->Component.Get()));

			if (HitProxyDynamicMesh)
			{
				FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
				HitProxyDynamicMesh->SetFlags(RF_Transactional);
				HitProxyDynamicMesh->Modify();
				HitProxyDynamicMesh->SetTopRadius(0.f);
				NotifyPropertyModified(HitProxyDynamicMesh, TopRadiusProperty, EPropertyChangeType::ValueSet);
			}
		}
	}
	// angle degree
	else if (InHitProxy->IsA(HAvaShapeAngleDegreeHitProxy::StaticGetType()))
	{
		const HAvaShapeAngleDegreeHitProxy* AngleDegreeHitProxy = static_cast<HAvaShapeAngleDegreeHitProxy*>(InHitProxy);

		if (AngleDegreeHitProxy->Component.IsValid())
		{
			FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(
				const_cast<UActorComponent*>(AngleDegreeHitProxy->Component.Get()));

			if (HitProxyDynamicMesh)
			{
				FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
				HitProxyDynamicMesh->SetFlags(RF_Transactional);
				HitProxyDynamicMesh->Modify();
				HitProxyDynamicMesh->SetAngleDegree(360.f);
				NotifyPropertyModified(HitProxyDynamicMesh, AngleDegreeProperty, EPropertyChangeType::ValueSet);
			}
		}
	}

	return true;
}

bool FAvaShapeConeDynamicMeshVisualizer::IsEditing() const
{
	if (bEditingAngleDegree || bEditingNumSides || bEditingTopRadius)
	{
		return true;
	}

	return Super::IsEditing();
}

void FAvaShapeConeDynamicMeshVisualizer::EndEditing()
{
	Super::EndEditing();

	bEditingAngleDegree = false;
	bEditingNumSides = false;
	bEditingTopRadius = false;
}

void FAvaShapeConeDynamicMeshVisualizer::DrawVisualizationNotEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationNotEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawNumSidesButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive);
	++InOutIconIndex;

	DrawTopRadiusButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive);
	++InOutIconIndex;

	DrawAngleDegreeButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive);
	++InOutIconIndex;
}

void FAvaShapeConeDynamicMeshVisualizer::DrawVisualizationEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawNumSidesButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingNumSides ? Active : Inactive);
	++InOutIconIndex;

	DrawTopRadiusButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingTopRadius ? Active : Inactive);
	++InOutIconIndex;

	DrawAngleDegreeButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingAngleDegree ? Active : Inactive);
	++InOutIconIndex;
}

bool FAvaShapeConeDynamicMeshVisualizer::HandleInputDeltaInternal(FEditorViewportClient* InViewportClient,
	FViewport* InViewport, const FVector& InAccumulatedTranslation, const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale)
{
	if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
	{
		return false;
	}

	FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (DynMesh)
	{
		if (bEditingNumSides)
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Scale)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
				{
					int32 NumSides = InitialNumSides;
					NumSides = FMath::Clamp(NumSides + static_cast<int32>(InAccumulatedScale.Y), 1, 128);
					DynMesh->Modify();
					DynMesh->SetNumSides(NumSides);

					bHasBeenModified = true;
					NotifyPropertyModified(DynMesh, NumSidesProperty, EPropertyChangeType::Interactive);
				}
			}
			return true;
		}
		if (bEditingTopRadius)
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
				{
					float TopRadius = InitialTopRadius;
					TopRadius = FMath::Clamp(TopRadius + (InAccumulatedTranslation.Y / DynMesh->GetSize3D().X * 2.f), 0.0, 1.f);
					DynMesh->Modify();
					DynMesh->SetTopRadius(TopRadius);

                    bHasBeenModified = true;
                    NotifyPropertyModified(DynMesh, TopRadiusProperty, EPropertyChangeType::Interactive);
				}
			}
			return true;
		}
		if (bEditingAngleDegree)
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Rotate)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
				{
					float AngleDegree = InitialAngleDegree;
					AngleDegree = FMath::Clamp(AngleDegree + static_cast<int32>(InAccumulatedRotation.Yaw), 0.f, 360.f);
					DynMesh->Modify();
					DynMesh->SetAngleDegree(AngleDegree);

					bHasBeenModified = true;
					NotifyPropertyModified(DynMesh, AngleDegreeProperty, EPropertyChangeType::Interactive);
				}
			}
		}
	}
	else
	{
		EndEditing();
	}

	return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
}

void FAvaShapeConeDynamicMeshVisualizer::StoreInitialValues()
{
	Super::StoreInitialValues();

	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh)
	{
		return;
	}

	InitialNumSides = DynMesh->GetNumSides();
	InitialTopRadius = DynMesh->GetTopRadius();
	InitialAngleDegree = DynMesh->GetAngleDegree();
}

void FAvaShapeConeDynamicMeshVisualizer::DrawTopRadiusButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	UTexture2D* CornerSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::CornerSprite);

	if (!CornerSprite || !CornerSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeInnerSizeHitProxy(InDynMesh));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, CornerSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaShapeConeDynamicMeshVisualizer::DrawNumSidesButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	UTexture2D* NumSidesSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::NumSidesSprite);

	if (!NumSidesSprite || !NumSidesSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeNumSidesHitProxy(InDynMesh));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, NumSidesSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaShapeConeDynamicMeshVisualizer::DrawAngleDegreeButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	UTexture2D* InnerSizeSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::InnerSizeSprite);

	if (!InnerSizeSprite || !InnerSizeSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeAngleDegreeHitProxy(InDynMesh, MakeAlignment(EAvaDepthAlignment::Center, EAvaHorizontalAlignment::Center, EAvaVerticalAlignment::Center)));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, InnerSizeSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

