// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeCubeDynMeshVis.h"
#include "AvaField.h"
#include "AvaShapeActor.h"
#include "AvaShapeSprites.h"
#include "DynamicMeshes/AvaShapeCubeDynMesh.h"
#include "EditorViewportClient.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "IAvaComponentVisualizersSettings.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "ScopedTransaction.h"
#include "TextureResource.h"

FAvaShapeCubeDynamicMeshVisualizer::FAvaShapeCubeDynamicMeshVisualizer()
	: FAvaShape3DDynamicMeshVisualizer()
	, bEditingBevelSizeRatio(false)
	, InitialBevelSizeRatio(0)
	, bEditingBevelNum(false)
	, InitialBevelNum(0)
{
	using namespace UE::AvaCore;
	BevelNumProperty       = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, BevelNum));
	BevelSizeRatioProperty = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, BevelSizeRatio));
}

TMap<UObject*, TArray<FProperty*>> FAvaShapeCubeDynamicMeshVisualizer::GatherEditableProperties(UObject* InObject) const
{
	TMap<UObject*, TArray<FProperty*>> Properties = Super::GatherEditableProperties(InObject);

	if (FMeshType* DynMesh = Cast<FMeshType>(InObject))
	{
		Properties.FindOrAdd(DynMesh).Add(BevelNumProperty);
		Properties.FindOrAdd(DynMesh).Add(BevelSizeRatioProperty);
	}

	return Properties;
}

bool FAvaShapeCubeDynamicMeshVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient,
	HComponentVisProxy* InVisProxy, const FViewportClick& InClick)
{
	if (InClick.GetKey() != EKeys::LeftMouseButton)
	{
		EndEditing();
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	FMeshType* DynMesh = Cast<FMeshType>(const_cast<UActorComponent*>(InVisProxy->Component.Get()));

	if (DynMesh == nullptr)
	{
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	if (InVisProxy->IsA(HAvaShapeNumSidesHitProxy::StaticGetType()))
	{
		EndEditing();
		bEditingBevelNum = true;
		StartEditing(InViewportClient, DynMesh);
		return true;
	}

	if (InVisProxy->IsA(HAvaShapeCornersHitProxy::StaticGetType()))
	{
		EndEditing();
		bEditingBevelSizeRatio = true;
		StartEditing(InViewportClient, DynMesh);
		return true;
	}

	return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
}

bool FAvaShapeCubeDynamicMeshVisualizer::GetWidgetLocation(const FEditorViewportClient* InViewportClient,
	FVector& OutLocation) const
{
	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (DynMesh)
	{
		if (bEditingBevelNum)
		{
			OutLocation = DynMesh->GetShapeActor()->GetActorLocation();
			return true;
		}

		if (bEditingBevelSizeRatio)
		{
			const float OffsetY  = DynMesh->GetBevelSizeRatio();
			const FVector Offset = FVector(0.f, OffsetY, 0.f);
			OutLocation          = DynamicMeshComponent->GetShapeActor()->GetTransform().TransformPosition(Offset);
			return true;
		}
	}

	return Super::GetWidgetLocation(InViewportClient, OutLocation);
}

bool FAvaShapeCubeDynamicMeshVisualizer::GetWidgetMode(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode& OutMode) const
{
	if (bEditingBevelNum)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Scale;
		return true;
	}

	if (bEditingBevelSizeRatio)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Translate;
		return true;
	}

	return Super::GetWidgetMode(InViewportClient, OutMode);
}

bool FAvaShapeCubeDynamicMeshVisualizer::GetWidgetAxisList(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingBevelNum)
	{
		OutAxisList = EAxisList::Type::Y;
		return true;
	}

	if (bEditingBevelSizeRatio)
	{
		OutAxisList = EAxisList::Type::Screen;
		return true;
	}

	return Super::GetWidgetAxisList(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShapeCubeDynamicMeshVisualizer::GetWidgetAxisListDragOverride(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingBevelSizeRatio)
	{
		OutAxisList = EAxisList::Type::Y;
		return true;
	}

	return Super::GetWidgetAxisListDragOverride(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShapeCubeDynamicMeshVisualizer::ResetValue(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy)
{
	if (!InHitProxy->IsA(HAvaShapeNumSidesHitProxy::StaticGetType()) &&
		!InHitProxy->IsA(HAvaShapeCornersHitProxy::StaticGetType()))
	{
		return Super::ResetValue(InViewportClient, InHitProxy);
	}

	if (InHitProxy->IsA(HAvaShapeNumSidesHitProxy::StaticGetType()))
	{
		const HAvaShapeNumSidesHitProxy* NumBevelHitProxy = static_cast<HAvaShapeNumSidesHitProxy*>(InHitProxy);

		if (NumBevelHitProxy->Component.IsValid())
		{
			FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(
				const_cast<UActorComponent*>(NumBevelHitProxy->Component.Get()));

			if (HitProxyDynamicMesh)
			{
				FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
				HitProxyDynamicMesh->SetFlags(RF_Transactional);
				HitProxyDynamicMesh->Modify();
				HitProxyDynamicMesh->SetBevelNum(1);
				NotifyPropertyModified(HitProxyDynamicMesh, BevelNumProperty, EPropertyChangeType::ValueSet);
			}
		}
	}
	else if (InHitProxy->IsA(HAvaShapeCornersHitProxy::StaticGetType()))
	{
		const HAvaShapeCornersHitProxy* BevelSizeHitProxy = static_cast<HAvaShapeCornersHitProxy*>(InHitProxy);

		if (BevelSizeHitProxy->Component.IsValid())
		{
			FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(
				const_cast<UActorComponent*>(BevelSizeHitProxy->Component.Get()));

			if (HitProxyDynamicMesh)
			{
				FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
				HitProxyDynamicMesh->SetFlags(RF_Transactional);
				HitProxyDynamicMesh->Modify();
				HitProxyDynamicMesh->SetBevelSizeRatio(0.f);
				NotifyPropertyModified(HitProxyDynamicMesh, BevelSizeRatioProperty, EPropertyChangeType::ValueSet);
			}
		}
	}

	return true;
}

bool FAvaShapeCubeDynamicMeshVisualizer::IsEditing() const
{
	if (bEditingBevelNum || bEditingBevelSizeRatio)
	{
		return true;
	}

	return Super::IsEditing();
}

void FAvaShapeCubeDynamicMeshVisualizer::EndEditing()
{
	Super::EndEditing();

	bEditingBevelNum = false;
	bEditingBevelSizeRatio = false;
}

void FAvaShapeCubeDynamicMeshVisualizer::DrawVisualizationNotEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationNotEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawBevelSizeRatioButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive);
	++InOutIconIndex;
	
	DrawBevelNumButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive);
	++InOutIconIndex;
}

void FAvaShapeCubeDynamicMeshVisualizer::DrawVisualizationEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawBevelSizeRatioButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingBevelSizeRatio ? Active : Inactive);
	++InOutIconIndex;

	DrawBevelNumButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingBevelNum ? Active : Inactive);
	++InOutIconIndex;
}

bool FAvaShapeCubeDynamicMeshVisualizer::HandleInputDeltaInternal(FEditorViewportClient* InViewportClient,
	FViewport* InViewport, const FVector& InAccumulatedTranslation, const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale)
{
	if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
	{
		return false;
	}

	FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (DynMesh)
	{
		if (bEditingBevelNum)
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Scale)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
				{
					int32 BevelNum = InitialBevelNum;
					BevelNum = FMath::Clamp(BevelNum + static_cast<int32>(InAccumulatedScale.Y), 1, 255);
					DynMesh->Modify();
					DynMesh->SetBevelNum(BevelNum);

					bHasBeenModified = true;
					NotifyPropertyModified(DynMesh, BevelNumProperty, EPropertyChangeType::Interactive);
				}
			}

			return true;
		}

		if (bEditingBevelSizeRatio)
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
				{
					float BevelSize = InitialBevelSizeRatio;
					BevelSize = FMath::Clamp(BevelSize + InAccumulatedTranslation.Y, 0.f, DynMesh->GetMaxBevelSize());
					DynMesh->Modify();
					DynMesh->SetBevelSizeRatio(BevelSize);

					bHasBeenModified = true;
					NotifyPropertyModified(DynMesh, BevelSizeRatioProperty, EPropertyChangeType::Interactive);
				}
			}

			return true;
		}
	}
	else
	{
		EndEditing();
	}

	return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
}

void FAvaShapeCubeDynamicMeshVisualizer::StoreInitialValues()
{
	Super::StoreInitialValues();

	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh)
	{
		return;
	}

	InitialBevelNum = DynMesh->GetBevelNum();
	InitialBevelSizeRatio = DynMesh->GetBevelSizeRatio();
}

void FAvaShapeCubeDynamicMeshVisualizer::DrawBevelSizeRatioButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	UTexture2D* CornerSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::CornerSprite);

	if (!CornerSprite || !CornerSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeCornersHitProxy(InDynMesh));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, CornerSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaShapeCubeDynamicMeshVisualizer::DrawBevelNumButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	UTexture2D* NumSidesSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::NumSidesSprite);

	if (!NumSidesSprite || !NumSidesSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeNumSidesHitProxy(InDynMesh));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, NumSidesSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}
