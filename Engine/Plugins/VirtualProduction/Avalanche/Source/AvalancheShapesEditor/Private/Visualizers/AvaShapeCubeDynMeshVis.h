// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Visualizers/AvaShape3DDynMeshVis.h"

class UAvaShapeCubeDynamicMesh;

class FAvaShapeCubeDynamicMeshVisualizer : public FAvaShape3DDynamicMeshVisualizer
{
public:
	using Super = FAvaShape3DDynamicMeshVisualizer;
	using FMeshType = UAvaShapeCubeDynamicMesh;
	
	FAvaShapeCubeDynamicMeshVisualizer();

	//~ Begin FAvaVisualizerBase
	virtual TMap<UObject*, TArray<FProperty*>> GatherEditableProperties(UObject* InObject) const override;
	virtual bool VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* InVisProxy,
		const FViewportClick& InClick) override;
	virtual bool GetWidgetLocation(const FEditorViewportClient* InViewportClient, FVector& OutLocation) const override;
	virtual bool GetWidgetMode(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode& OutMode) const override;
	virtual bool GetWidgetAxisList(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode,
		EAxisList::Type& OutAxisList) const override;
	virtual bool GetWidgetAxisListDragOverride(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode,
		EAxisList::Type& OutAxisList) const override;
	virtual bool ResetValue(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy) override;
	virtual bool IsEditing() const override;
	virtual void EndEditing() override;
	//~ End FAvaVisualizerBase

protected:
	FProperty* BevelSizeRatioProperty;
	FProperty* BevelNumProperty;

	bool bEditingBevelSizeRatio;
	float InitialBevelSizeRatio;
	bool bEditingBevelNum;
	uint8 InitialBevelNum;

	virtual void DrawVisualizationNotEditing(const UActorComponent* InComponent, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;
	virtual void DrawVisualizationEditing(const UActorComponent* InComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;
	virtual bool HandleInputDeltaInternal(FEditorViewportClient* InViewportClient, FViewport* InViewport, const FVector& InAccumulatedTranslation,
		const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale) override;
	virtual void StoreInitialValues() override;

	void DrawBevelSizeRatioButton(const FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		int32 InIconIndex, const FLinearColor& InColor) const;
	void DrawBevelNumButton(const FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		int32 InIconIndex, const FLinearColor& InColor) const;

};
