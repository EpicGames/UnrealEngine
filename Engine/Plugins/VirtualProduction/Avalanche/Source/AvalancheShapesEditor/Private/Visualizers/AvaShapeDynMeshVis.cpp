// Copyright Epic Games, Inc. All Rights Reserved.

#include "Visualizers/AvaShapeDynMeshVis.h"
#include "AvaActorUtils.h"
#include "AvaField.h"
#include "AvaShapeActor.h"
#include "AvaShapeSprites.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "Engine/Texture2D.h"
#include "IAvaComponentVisualizersSettings.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "Kismet/KismetMathLibrary.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "TextureResource.h"

IMPLEMENT_HIT_PROXY(HAvaShapeSizeHitProxy, HAvaHitProxy);
IMPLEMENT_HIT_PROXY(HAvaShapeUVHitProxy, HAvaHitProxy);
IMPLEMENT_HIT_PROXY(HAvaShapeNumSidesHitProxy, HAvaHitProxy);
IMPLEMENT_HIT_PROXY(HAvaShapeNumPointsHitProxy, HAvaHitProxy);
IMPLEMENT_HIT_PROXY(HAvaShapeInnerSizeHitProxy, HAvaHitProxy);
IMPLEMENT_HIT_PROXY(HAvaShapeCornersHitProxy, HAvaHitProxy);
IMPLEMENT_HIT_PROXY(HAvaShapeAngleDegreeHitProxy, HAvaHitProxy);

UActorComponent* FAvaShapeDynamicMeshVisualizer::GetEditedComponent() const
{
	return DynamicMeshComponent.Get();
}

TMap<UObject*, TArray<FProperty*>> FAvaShapeDynamicMeshVisualizer::GatherEditableProperties(UObject* InObject) const
{
	return {};
}

const USceneComponent* FAvaShapeDynamicMeshVisualizer::GetEditedSceneComponent() const
{
	return GetEditedSceneComponent(DynamicMeshComponent.Get());
}

const USceneComponent* FAvaShapeDynamicMeshVisualizer::GetEditedSceneComponent(const UActorComponent* InComponent) const
{
	const UAvaShapeDynamicMeshBase* DynMesh = Cast<UAvaShapeDynamicMeshBase>(InComponent);

	if (DynMesh)
	{
		return DynMesh->GetShapeMeshComponent();
	}

	return nullptr;
}

UAvaShapeDynamicMeshBase* FAvaShapeDynamicMeshVisualizer::GetDynamicMesh() const
{
	return Cast<UAvaShapeDynamicMeshBase>(DynamicMeshComponent.Get());
}

void FAvaShapeDynamicMeshVisualizer::StartEditing(FEditorViewportClient* InViewportClient, UActorComponent* InEditedComponent)
{
	FAvaVisualizerBase::StartEditing(InViewportClient, InEditedComponent);

	DynamicMeshComponent = Cast<UAvaShapeDynamicMeshBase>(InEditedComponent);
}

FBox FAvaShapeDynamicMeshVisualizer::GetComponentBounds(const UActorComponent* InComponent) const
{
	if (const UAvaShapeDynamicMeshBase* DynMesh = Cast<UAvaShapeDynamicMeshBase>(InComponent))
	{
		return FAvaActorUtils::GetComponentLocalBoundingBox(DynMesh->GetShapeMeshComponent());
	}

	return Super::GetComponentBounds(InComponent);
}

FTransform FAvaShapeDynamicMeshVisualizer::GetComponentTransform(const UActorComponent* InComponent) const
{
	const UAvaShapeDynamicMeshBase* DynMesh = Cast<UAvaShapeDynamicMeshBase>(InComponent);

	if (DynMesh)
	{
		if (DynMesh->HasMeshRegenWorldLocation())
		{
			FTransform Transform = DynMesh->GetTransform();
			Transform.SetLocation(DynMesh->GetMeshRegenWorldLocation());
			return Transform;
		}

		return DynMesh->GetTransform();
	}

	return Super::GetComponentTransform(InComponent);
}

FAvaShapeDynamicMeshVisualizer::FAvaShapeDynamicMeshVisualizer()
	: FAvaVisualizerBase()
	, DynamicMeshComponent(nullptr)
{
	using namespace UE::AvaCore;

	MeshRegenWorldLocationProperty = GetProperty<UAvaShapeDynamicMeshBase>(
		GET_MEMBER_NAME_CHECKED(UAvaShapeDynamicMeshBase, MeshRegenWorldLocation));

	MeshDataProperty    = GetProperty<UAvaShapeDynamicMeshBase>(GET_MEMBER_NAME_CHECKED(UAvaShapeDynamicMeshBase, MeshDatas));
	UVParamsProperty    = GetProperty<FAvaShapeMeshData>(GET_MEMBER_NAME_CHECKED(FAvaShapeMeshData, MaterialUVParams));
	UVOffsetProperty    = GetProperty<FAvaShapeMaterialUVParameters>(GET_MEMBER_NAME_CHECKED(FAvaShapeMaterialUVParameters, Offset));
	UVScaleProperty     = GetProperty<FAvaShapeMaterialUVParameters>(GET_MEMBER_NAME_CHECKED(FAvaShapeMaterialUVParameters, Scale));
	UVRotationProperty  = GetProperty<FAvaShapeMaterialUVParameters>(GET_MEMBER_NAME_CHECKED(FAvaShapeMaterialUVParameters, Rotation));
	UVAnchorProperty    = GetProperty<FAvaShapeMaterialUVParameters>(GET_MEMBER_NAME_CHECKED(FAvaShapeMaterialUVParameters, Anchor));
	UVModeProperty      = GetProperty<FAvaShapeMaterialUVParameters>(GET_MEMBER_NAME_CHECKED(FAvaShapeMaterialUVParameters, Mode));
	UVHorizFlipProperty = GetProperty<FAvaShapeMaterialUVParameters>(GET_MEMBER_NAME_CHECKED(FAvaShapeMaterialUVParameters, bFlipHorizontal));
	UVVertFlipProperty  = GetProperty<FAvaShapeMaterialUVParameters>(GET_MEMBER_NAME_CHECKED(FAvaShapeMaterialUVParameters, bFlipVertical));
}

void FAvaShapeDynamicMeshVisualizer::StoreInitialValues()
{
	Super::StoreInitialValues();

	if (GetEditedComponent() == nullptr)
	{
		return;
	}

	InitialTransform = DynamicMeshComponent.IsValid() ? DynamicMeshComponent->GetTransform() : FTransform::Identity;
}

void FAvaShapeDynamicMeshVisualizer::EndEditing()
{
	Super::EndEditing();

	DynamicMeshComponent.Reset();
}

void FAvaShapeDynamicMeshVisualizer::DrawSizeButton(const UAvaShapeDynamicMeshBase* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, AvaAlignment InSizeDragAnchor) const
{
	static const float BaseSize = 1.f;
	
	UTexture2D* SizeSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::SizeSprite);

	if (!SizeSprite || !SizeSprite->GetResource())
	{
		return;
	}

	const FVector AnchorLocation = GetFinalAnchorLocation(InDynMesh, InSizeDragAnchor);
	
	// skip if handle not reachable
	if (!IsAnchorReachable(InDynMesh, InView, AnchorLocation))
	{
		return;
	}

	const float IconSize = BaseSize * GetIconSizeScale(InView, AnchorLocation);

	InPDI->SetHitProxy(new HAvaShapeSizeHitProxy(InDynMesh, InSizeDragAnchor));
	InPDI->DrawSprite(AnchorLocation, IconSize, IconSize, SizeSprite->GetResource(), FLinearColor::White,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaShapeDynamicMeshVisualizer::DrawUVButton(const UAvaShapeDynamicMeshBase* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, int32 InSectionIdx, const FLinearColor& InColor) const
{
	UTexture2D* UVSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::UVSprite);

	if (!UVSprite || !UVSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeUVHitProxy(InDynMesh, InSectionIdx));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, UVSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

bool FAvaShapeDynamicMeshVisualizer::IsAnchorReachable(const UAvaShapeDynamicMeshBase* InDynMesh, const FSceneView* InView, const FVector& InAnchorLocation) const
{
	FVector Origin;
	FVector Extent;
	FVector Pivot;
	InDynMesh->GetBounds(Origin, Extent, Pivot);
	if (Extent.X == 0)
	{
		return true;
	}
	
	const FVector ActorLocation = InDynMesh->GetShapeActor()->GetActorLocation();
	const FVector ViewLoc = InView->ViewLocation;
	const float OriginDistance = UKismetMathLibrary::Vector_Distance(ViewLoc, ActorLocation);
	const float AnchorDistance = UKismetMathLibrary::Vector_Distance(ViewLoc, InAnchorLocation);

	const FVector OriginToAnchor = UKismetMathLibrary::GetDirectionUnitVector(ActorLocation, InAnchorLocation);
	const FVector ViewDir = InView->GetViewDirection();
	const float DotProduct = UKismetMathLibrary::Dot_VectorVector(OriginToAnchor, ViewDir);

	const bool bMaxDistanceFromOrigin = AnchorDistance > OriginDistance;
	const bool bFacingView = DotProduct < 0.25f;

	return !(bMaxDistanceFromOrigin && !bFacingView);
}

FVector FAvaShapeDynamicMeshVisualizer::GetFinalAnchorLocation(const UAvaShapeDynamicMeshBase* InDynMesh, AvaAlignment InSizeDragAnchor) const
{
	FVector Origin;
	FVector Extent;
	FVector Pivot;
	InDynMesh->GetBounds(Origin, Extent, Pivot);

	// get bbox of component
	const FBox BBox = FAvaActorUtils::GetComponentLocalBoundingBox(InDynMesh->GetShapeMeshComponent());
	FVector AnchorLocation = GetLocationFromAlignment(InSizeDragAnchor, BBox.GetSize());

	// apply transform on anchor for rotation, scale, translate
	FTransform ComponentTransform = GetComponentTransform(InDynMesh);

	if (InDynMesh->HasMeshRegenWorldLocation())
	{
		ComponentTransform.SetLocation(InDynMesh->GetMeshRegenWorldLocation());
	}

	AnchorLocation = ComponentTransform.TransformPositionNoScale(AnchorLocation);

	// check difference between actor center and box center
	FVector ActorLocation = ComponentTransform.GetLocation();
	const FVector DiffLocation = Origin - ActorLocation;
	AnchorLocation = DiffLocation + AnchorLocation;

	return AnchorLocation;
}
