// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaDefs.h"
#include "AvaVisBase.h"

class UAvaShapeDynamicMeshBase;

struct HAvaShapeSizeHitProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaShapeSizeHitProxy(
		const UActorComponent* InComponent, 
		AvaAlignment InAlignment)
		: HAvaHitProxy(InComponent)
		, DragAnchor(InAlignment) {}

	AvaAlignment DragAnchor; // XYZ
};

struct HAvaShapeUVHitProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaShapeUVHitProxy(const UActorComponent* InComponent, int32 InSectionIdx)
		: HAvaHitProxy(InComponent)
		, SectionIdx(InSectionIdx)
	{}

	int32 SectionIdx;
};

struct HAvaShapeNumSidesHitProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaShapeNumSidesHitProxy(const UActorComponent* InComponent)
		: HAvaHitProxy(InComponent)
	{}
};

struct HAvaShapeNumPointsHitProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaShapeNumPointsHitProxy(const UActorComponent* InComponent)
		: HAvaHitProxy(InComponent)
	{}
};

struct HAvaShapeInnerSizeHitProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaShapeInnerSizeHitProxy(const UActorComponent* InComponent)
		: HAvaHitProxy(InComponent)
	{}
};

struct HAvaShapeCornersHitProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaShapeCornersHitProxy(const UActorComponent* InComponent)
		: HAvaHitProxy(InComponent)
	{}
};

struct HAvaShapeAngleDegreeHitProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaShapeAngleDegreeHitProxy(
		const UActorComponent* InComponent,
		AvaAlignment InAlignment)
		: HAvaHitProxy(InComponent)
		, DragAnchor(InAlignment)
	{}

	AvaAlignment DragAnchor; // XYZ
};

class FAvaShapeDynamicMeshVisualizer : public FAvaVisualizerBase
{
public:
	using Super = FAvaVisualizerBase;

	UAvaShapeDynamicMeshBase* GetDynamicMesh() const;

	//~ Begin FAvaVisualizerBase
	virtual UActorComponent* GetEditedComponent() const override;
	virtual TMap<UObject*, TArray<FProperty*>> GatherEditableProperties(UObject* InObject) const override;
	virtual const USceneComponent* GetEditedSceneComponent() const override;
	virtual const USceneComponent* GetEditedSceneComponent(const UActorComponent* InComponent) const override;
	virtual void StartEditing(FEditorViewportClient* InViewportClient, UActorComponent* InEditedComponent) override;
	virtual void EndEditing() override;
	//~ End FAvaVisualizerBase

protected:
	TWeakObjectPtr<UAvaShapeDynamicMeshBase> DynamicMeshComponent;
	FTransform InitialTransform;

	FProperty* MeshRegenWorldLocationProperty;
	FProperty* MeshDataProperty;
	FProperty* UVParamsProperty;
	FProperty* UVOffsetProperty;
	FProperty* UVScaleProperty;
	FProperty* UVRotationProperty;
	FProperty* UVAnchorProperty;
	FProperty* UVModeProperty;
	FProperty* UVHorizFlipProperty;
	FProperty* UVVertFlipProperty;

	FAvaShapeDynamicMeshVisualizer();

	virtual void StoreInitialValues() override;
	virtual FBox GetComponentBounds(const UActorComponent* InComponent) const override;
	virtual FTransform GetComponentTransform(const UActorComponent* InComponent) const override;

	// draw size button hit proxy with icon at specific alignment location
	void DrawSizeButton(const UAvaShapeDynamicMeshBase* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		AvaAlignment InSizeDragAnchor) const;

	// draw uv button hit proxy with icon for section index
	void DrawUVButton(const UAvaShapeDynamicMeshBase* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI, 
		int32 InIconIndex, int32 InSectionIdx, const FLinearColor& InColor) const;

	// checks whether an anchor location is on the correct size of the view and can be reachable
	bool IsAnchorReachable(const UAvaShapeDynamicMeshBase* InDynMesh, const FSceneView* InView, const FVector& InAnchorLocation) const;

	FVector GetFinalAnchorLocation(const UAvaShapeDynamicMeshBase* InDynMesh, AvaAlignment InSizeDragAnchor) const;
};
