// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeEllipseDynMeshVis.h"
#include "AvaField.h"
#include "AvaShapeActor.h"
#include "AvaShapeSprites.h"
#include "DynamicMeshes/AvaShapeEllipseDynMesh.h"
#include "EditorViewportClient.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "ScopedTransaction.h"
#include "TextureResource.h"

FAvaShapeEllipseDynamicMeshVisualizer::FAvaShapeEllipseDynamicMeshVisualizer()
	: FAvaShape2DDynamicMeshVisualizer()
	, bEditingNumSides(false)
	, InitialNumSides(0)
{
	using namespace UE::AvaCore;
	NumSidesProperty = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, NumSides));
}

TMap<UObject*, TArray<FProperty*>> FAvaShapeEllipseDynamicMeshVisualizer::GatherEditableProperties(UObject* InObject) const
{
	TMap<UObject*, TArray<FProperty*>> Properties = Super::GatherEditableProperties(InObject);

	if (FMeshType* DynMesh = Cast<FMeshType>(InObject))
	{
		Properties.FindOrAdd(DynMesh).Add(NumSidesProperty);
	}

	return Properties;
}

bool FAvaShapeEllipseDynamicMeshVisualizer::VisProxyHandleClick(
		FEditorViewportClient* InViewportClient, HComponentVisProxy* InVisProxy, const FViewportClick& InClick
	)
{
	if (InClick.GetKey() != EKeys::LeftMouseButton)
	{
		EndEditing();
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	FMeshType* DynMesh = Cast<FMeshType>(const_cast<UActorComponent*>(InVisProxy->Component.Get()));

	if (DynMesh == nullptr)
	{
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	if (InVisProxy->IsA(HAvaShapeNumSidesHitProxy::StaticGetType()))
	{
		EndEditing();
		bEditingNumSides     = true;
		StartEditing(InViewportClient, DynMesh);
		return true;
	}

	return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
}

void FAvaShapeEllipseDynamicMeshVisualizer::DrawVisualizationNotEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationNotEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawNumSidesButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive);
	++InOutIconIndex;
}

void FAvaShapeEllipseDynamicMeshVisualizer::DrawVisualizationEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawNumSidesButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingNumSides ? Active : Inactive);
	++InOutIconIndex;
}

void FAvaShapeEllipseDynamicMeshVisualizer::DrawNumSidesButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	UTexture2D* NumSidesSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::NumSidesSprite);

	if (!NumSidesSprite || !NumSidesSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeNumSidesHitProxy(InDynMesh));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, NumSidesSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

bool FAvaShapeEllipseDynamicMeshVisualizer::GetWidgetLocation(const FEditorViewportClient* InViewportClient,
	FVector& OutLocation) const
{
	if (bEditingNumSides && DynamicMeshComponent.IsValid())
	{
		OutLocation = DynamicMeshComponent->GetShapeActor()->GetActorLocation();
		return true;
	}

	return Super::GetWidgetLocation(InViewportClient, OutLocation);
}

bool FAvaShapeEllipseDynamicMeshVisualizer::GetWidgetMode(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode& OutMode) const
{
	if (bEditingNumSides)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Scale;
		return true;
	}

	return Super::GetWidgetMode(InViewportClient, OutMode);
}

bool FAvaShapeEllipseDynamicMeshVisualizer::GetWidgetAxisList(
	const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingNumSides)
	{
		OutAxisList = static_cast<EAxisList::Type>(EAxisList::Type::Y | EAxisList::Type::Screen);
		return true;
	}

	return Super::GetWidgetAxisList(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShapeEllipseDynamicMeshVisualizer::HandleInputDeltaInternal(FEditorViewportClient* InViewportClient,
	FViewport* InViewport, const FVector& InAccumulatedTranslation, const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale)
{
	if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
	{
		return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
	}

	if (bEditingNumSides)
	{
		if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Scale)
		{
			if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
			{
				if (DynamicMeshComponent.IsValid())
				{
					FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

					if (DynMesh)
					{
						int32 NumSides = InitialNumSides;
						NumSides = FMath::Clamp(NumSides + static_cast<int32>(InAccumulatedScale.Y), 3, 128);
						DynMesh->Modify();
						DynMesh->SetNumSides(NumSides);

						bHasBeenModified = true;
						NotifyPropertyModified(DynMesh, NumSidesProperty, EPropertyChangeType::Interactive);
					}
				}
				else
				{
					EndEditing();
				}
			}
		}

		return true;
	}

	return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
}

void FAvaShapeEllipseDynamicMeshVisualizer::StoreInitialValues()
{
	Super::StoreInitialValues();

	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh)
	{
		return;
	}

	InitialNumSides = DynMesh->GetNumSides();
}

bool FAvaShapeEllipseDynamicMeshVisualizer::ResetValue(FEditorViewportClient* InViewportClient,
	HHitProxy* InHitProxy)
{
	if (!InHitProxy->IsA(HAvaShapeNumSidesHitProxy::StaticGetType()))
	{
		return Super::ResetValue(InViewportClient, InHitProxy);
	}

	const HAvaShapeNumSidesHitProxy* NumSidesHitProxy = static_cast<HAvaShapeNumSidesHitProxy*>(InHitProxy);

	if (NumSidesHitProxy->Component.IsValid())
	{
		FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(const_cast<UActorComponent*>(NumSidesHitProxy->Component.Get()));

		if (HitProxyDynamicMesh)
		{
			FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
			HitProxyDynamicMesh->SetFlags(RF_Transactional);
			HitProxyDynamicMesh->Modify();
			HitProxyDynamicMesh->SetNumSides(5);
			NotifyPropertyModified(HitProxyDynamicMesh, NumSidesProperty, EPropertyChangeType::ValueSet);
		}
	}

	return true;
}

bool FAvaShapeEllipseDynamicMeshVisualizer::IsEditing() const
{
	if (bEditingNumSides)
	{
		return true;
	}

	return Super::IsEditing();
}

void FAvaShapeEllipseDynamicMeshVisualizer::EndEditing()
{
	Super::EndEditing();

	bEditingNumSides = false;
}
