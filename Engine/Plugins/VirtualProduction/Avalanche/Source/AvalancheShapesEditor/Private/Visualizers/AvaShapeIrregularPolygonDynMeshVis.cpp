// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeIrregularPolygonDynMeshVis.h"
#include "AvaField.h"
#include "AvaShapeActor.h"
#include "AvaShapeSprites.h"
#include "DynamicMeshes/AvaShapeIrregularPolygonDynMesh.h"
#include "EditorViewportClient.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "IAvaComponentVisualizersSettings.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "Interaction/AvaSnapOperation.h"
#include "ScopedTransaction.h"
#include "Templates/SharedPointer.h"
#include "TextureResource.h"
#include "Viewport/Interaction/AvaSnapPoint.h"

IMPLEMENT_HIT_PROXY(HAvaShapeIrregularPolygonPointHitProxy, HAvaHitProxy)
IMPLEMENT_HIT_PROXY(HAvaShapeIrregularPolygonBevelHitProxy, HAvaHitProxy)
IMPLEMENT_HIT_PROXY(HAvaShapeIrregularPolygonBreakHitProxy, HAvaHitProxy)

#define LOCTEXT_NAMESPACE "AvaShapeIrregularPolygonDynamicMeshVisualizer"

FAvaShapeIrregularPolygonDynamicMeshVisualizer::FAvaShapeIrregularPolygonDynamicMeshVisualizer()
	: FAvaShape2DDynamicMeshVisualizer()
	, bEditingGlobalBevelSize(false)
	, InitialGlobalBevelSize(0)
	, EditingPointIdx(INDEX_NONE)
	, EditingBevelIdx(INDEX_NONE)
	, InitialPointBevelSize(0)
{
	using namespace UE::AvaCore;

	GlobalBevelSizeProperty = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, GlobalBevelSize));
	PointsProperty          = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, Points));

	LocationProperty  = GetProperty<FAvaShapeRoundedCorner>(GET_MEMBER_NAME_CHECKED(FAvaShapeRoundedCorner, Location));
	BevelSizeProperty = GetProperty<FAvaShapeRoundedCornerSettings>(GET_MEMBER_NAME_CHECKED(FAvaShapeRoundedCornerSettings, BevelSize));
}

TMap<UObject*, TArray<FProperty*>> FAvaShapeIrregularPolygonDynamicMeshVisualizer::GatherEditableProperties(UObject* InObject) const
{
	TMap<UObject*, TArray<FProperty*>> Properties = Super::GatherEditableProperties(InObject);

	if (FMeshType* DynMesh = Cast<FMeshType>(InObject))
	{
		Properties.FindOrAdd(DynMesh).Add(GlobalBevelSizeProperty);
	}

	return Properties;
}

bool FAvaShapeIrregularPolygonDynamicMeshVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient,
	HComponentVisProxy* InVisProxy, const FViewportClick& InClick)
{
	if (InVisProxy == nullptr)
	{
		return false;
	}
	
	if (InClick.GetKey() != EKeys::LeftMouseButton)
	{
		EndEditing();
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	FMeshType* DynMesh = Cast<FMeshType>(const_cast<UActorComponent*>(InVisProxy->Component.Get()));

	if (DynMesh == nullptr)
	{
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	if (InVisProxy->IsA(HAvaShapeCornersHitProxy::StaticGetType()))
	{
		const HAvaShapeCornersHitProxy* CornerHitProxy = static_cast<HAvaShapeCornersHitProxy*>(
			InVisProxy);

		if (CornerHitProxy)
		{
			EndEditing();
			bEditingGlobalBevelSize = true;
			StartEditing(InViewportClient, DynMesh);
			return true;
		}
	}

	if (InVisProxy->IsA(HAvaShapeIrregularPolygonPointHitProxy::StaticGetType()))
	{
		EndEditing();
		EditingPointIdx = static_cast<HAvaShapeIrregularPolygonPointHitProxy*>(InVisProxy)->PointIdx;
		StartEditing(InViewportClient, DynMesh);
		return true;
	}

	if (InVisProxy->IsA(HAvaShapeIrregularPolygonBevelHitProxy::StaticGetType()))
	{
		EndEditing();
		EditingBevelIdx = static_cast<HAvaShapeIrregularPolygonBevelHitProxy*>(InVisProxy)->PointIdx;
		StartEditing(InViewportClient, DynMesh);
		return true;
	}

	if (InVisProxy->IsA(HAvaShapeIrregularPolygonBreakHitProxy::StaticGetType()))
	{
		EndEditing();
		FScopedTransaction Transaction(LOCTEXT("VisualizerBreakSide", "Visualizer Break Side"));
		DynMesh->BreakSide(static_cast<HAvaShapeIrregularPolygonBreakHitProxy*>(InVisProxy)->PointIdx);
		return true;
	}

	return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
}

void FAvaShapeIrregularPolygonDynamicMeshVisualizer::DrawVisualizationNotEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationNotEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawGlobalBevelButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive);
	++InOutIconIndex;

	if (ShouldDrawExtraHandles(InComponent, InView))
	{
		for (int32 PointIterIdx = 0; PointIterIdx < DynMesh->GetNumPoints(); ++PointIterIdx)
		{
			DrawPointButton(DynMesh, InView, InPDI, Inactive, PointIterIdx);
			DrawBevelButton(DynMesh, InView, InPDI, Inactive, PointIterIdx);
			DrawBreakButton(DynMesh, InView, InPDI, Inactive, PointIterIdx);
		}
	}
}

void FAvaShapeIrregularPolygonDynamicMeshVisualizer::DrawVisualizationEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawGlobalBevelButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingGlobalBevelSize ? Active : Inactive);
	++InOutIconIndex;

	for (int32 PointIterIdx = 0; PointIterIdx < DynMesh->GetNumPoints(); ++PointIterIdx)
	{
		DrawPointButton(DynMesh, InView, InPDI, Inactive, PointIterIdx);
		DrawBevelButton(DynMesh, InView, InPDI, Inactive, PointIterIdx);
		DrawBreakButton(DynMesh, InView, InPDI, Inactive, PointIterIdx);
	}
}

FVector FAvaShapeIrregularPolygonDynamicMeshVisualizer::GetGlobalBevelLocation(const FMeshType* InDynMesh) const
{
	const FVector2D Size2D = InDynMesh->GetSize2D();
	const float LeftOffset = FMath::Min(Size2D.X / 10.f, 20.f);

	FVector LeftPosition = FVector::ZeroVector;
	LeftPosition.Y       = -Size2D.X / 2.f + LeftOffset + InDynMesh->GetGlobalBevelSize() * Size2D.X / 2.f;

	const FTransform ComponentTransform = InDynMesh->GetTransform();

	return ComponentTransform.TransformPosition(LeftPosition);
}

FVector FAvaShapeIrregularPolygonDynamicMeshVisualizer::GetPointLocation(const FMeshType* InDynMesh, int32 InPointIdx) const
{
	const FTransform ComponentTransform   = InDynMesh->GetTransform();
	const FAvaShapeRoundedCorner& Point = InDynMesh->GetPoint(InPointIdx);
	const FVector OutVector               = ComponentTransform.TransformPosition({0.f, Point.Location.X, Point.Location.Y});

	return OutVector;
}

FVector FAvaShapeIrregularPolygonDynamicMeshVisualizer::GetBevelLocation(const FMeshType* InDynMesh, int32 InPointIdx) const
{
	// TODO fix this
	const FTransform ComponentTransform   = InDynMesh->GetTransform();
	const FAvaShapeRoundedCorner& Point = InDynMesh->GetPoint(InPointIdx);

	const FVector2D BevelVector2D = GetCornerBevelDirection2D(InDynMesh, InPointIdx).GetSafeNormal(0.f);
	const float MaxBevelSize      = InDynMesh->GetMaxBevelSizeForPoint(InPointIdx);

	const FVector2d BevelLocation2D = Point.Location + MaxBevelSize / 4.f * BevelVector2D
		+ Point.Settings.BevelSize * MaxBevelSize * 0.75f * BevelVector2D;

	const FVector OutVector = ComponentTransform.TransformPosition(FVector(0.f, BevelLocation2D.X, BevelLocation2D.Y));

	return OutVector;
}

FVector FAvaShapeIrregularPolygonDynamicMeshVisualizer::GetBreakLocation(const FMeshType* InDynMesh, int32 InPointIdx) const
{
	if (InDynMesh->GetPoints().Num() < 2 || !InDynMesh->GetPoints().IsValidIndex(InPointIdx))
	{
		AActor* Owner = InDynMesh->GetOwner();

		if (IsValid(Owner))
		{
			return Owner->GetActorLocation();
		}

		return FVector::ZeroVector;
	}

	const int32 NextPointIdx = InPointIdx == (InDynMesh->GetPoints().Num() - 1) ? 0 : InPointIdx + 1;

	const FVector PointOneLocation = GetPointLocation(InDynMesh, InPointIdx);
	const FVector PointTwoLocation = GetPointLocation(InDynMesh, NextPointIdx);

	return (PointOneLocation * 0.5) + (PointTwoLocation * 0.5f);
}

void FAvaShapeIrregularPolygonDynamicMeshVisualizer::GenerateContextSensitiveSnapPoints()
{
	Super::GenerateContextSensitiveSnapPoints();

	if (!DynamicMeshComponent.IsValid())
	{
		return;
	}

	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh)
	{
		return;
	}

	const AActor* Actor = DynMesh->GetOwner();

	if (!Actor)
	{
		return;
	}

	if (EditingPointIdx == INDEX_NONE || EditingPointIdx >= DynMesh->GetNumPoints())
	{
		return;
	}

	const FTransform MeshTransform = DynMesh->GetTransform();
	const TArray<FAvaShapeRoundedCorner>& Points = DynMesh->GetPoints();

	for (int32 PointIdx = 0; PointIdx < Points.Num(); ++PointIdx)
	{
		FVector PointLocation = MeshTransform.TransformPosition({0.f, Points[PointIdx].Location.X, Points[PointIdx].Location.Y});

		SnapOperation->AddActorSnapPoint(FAvaSnapPoint::CreateActorCustomSnapPoint(
				Actor,
				PointLocation,
				PointIdx
			));
	}
}

void FAvaShapeIrregularPolygonDynamicMeshVisualizer::DrawGlobalBevelButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	UTexture2D* UVSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::CornerSprite);

	if (!UVSprite || !UVSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeCornersHitProxy(InDynMesh));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, UVSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaShapeIrregularPolygonDynamicMeshVisualizer::DrawPointButton(const FMeshType* InDynMesh,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, const FLinearColor& InColor, int32 InPointIdx) const
{
	if (InPointIdx >= InDynMesh->GetNumPoints())
	{
		return;
	}

	static float BaseSize = 1.f;

	UTexture2D* SizeSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::SizeSprite);

	if (!SizeSprite || !SizeSprite->GetResource())
	{
		return;
	}

	FVector IconLocation = GetPointLocation(InDynMesh, InPointIdx);
	const float IconSize = BaseSize * GetIconSizeScale(InView, IconLocation);
	IconLocation += InDynMesh->GetTransform().TransformVector(
		{-1.f, 0.f, 0.f});

	InPDI->SetHitProxy(new HAvaShapeIrregularPolygonPointHitProxy(InDynMesh, InPointIdx));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, SizeSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaShapeIrregularPolygonDynamicMeshVisualizer::DrawBevelButton(const FMeshType* InDynMesh,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, const FLinearColor& InColor, int32 InPointIdx) const
{
	if (InPointIdx >= InDynMesh->GetNumPoints())
	{
		return;
	}

	static float BaseSize = 1.f;

	UTexture2D* SizeSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::BevelSprite);

	if (!SizeSprite || !SizeSprite->GetResource())
	{
		return;
	}

	FVector IconLocation = GetBevelLocation(InDynMesh, InPointIdx);
	const float IconSize = BaseSize * GetIconSizeScale(InView, IconLocation);
	IconLocation += InDynMesh->GetTransform().TransformVector(
		{-1.f, 0.f, 0.f});

	InPDI->SetHitProxy(new HAvaShapeIrregularPolygonBevelHitProxy(InDynMesh, InPointIdx));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, SizeSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaShapeIrregularPolygonDynamicMeshVisualizer::DrawBreakButton(const FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI, const FLinearColor& InColor, int32 InPointIdx) const
{
	if (InPointIdx >= InDynMesh->GetNumPoints())
	{
		return;
	}

	static float BaseSize = 1.f;

	UTexture2D* BreakSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::BreakSideSprite);

	if (!BreakSprite || !BreakSprite->GetResource())
	{
		return;
	}

	FVector IconLocation = GetBreakLocation(InDynMesh, InPointIdx);
	const float IconSize = BaseSize * GetIconSizeScale(InView, IconLocation);
	IconLocation += InDynMesh->GetTransform().TransformVector(
		{-1.f, 0.f, 0.f});

	InPDI->SetHitProxy(new HAvaShapeIrregularPolygonBreakHitProxy(InDynMesh, InPointIdx));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, BreakSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

bool FAvaShapeIrregularPolygonDynamicMeshVisualizer::GetWidgetMode(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode& OutMode) const
{
	if (bEditingGlobalBevelSize)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Translate;
	}
	else if (EditingPointIdx != INDEX_NONE)
	{
		OutMode = UE::Widget::WM_Translate;
	}
	else if (EditingBevelIdx != INDEX_NONE)
	{
		OutMode = UE::Widget::WM_Translate;
	}

	return true;
}

bool FAvaShapeIrregularPolygonDynamicMeshVisualizer::GetWidgetLocation(const FEditorViewportClient* InViewportClient,
	FVector& OutLocation) const
{
	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh)
	{
		return Super::GetWidgetLocation(InViewportClient, OutLocation);
	}

	if (bEditingGlobalBevelSize)
	{
		OutLocation = GetGlobalBevelLocation(DynMesh);
		return true;
	}

	if (EditingPointIdx != INDEX_NONE)
	{
		OutLocation = GetPointLocation(DynMesh, EditingPointIdx);
		return true;
	}

	if (EditingBevelIdx != INDEX_NONE)
	{
		OutLocation = GetBevelLocation(DynMesh, EditingBevelIdx);
		return true;
	}

	return Super::GetWidgetLocation(InViewportClient, OutLocation);
}

bool FAvaShapeIrregularPolygonDynamicMeshVisualizer::GetWidgetAxisList(
	const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingGlobalBevelSize)
	{
		OutAxisList = EAxisList::Type::Screen;
		return true;
	}

	if (EditingPointIdx != INDEX_NONE)
	{
		OutAxisList = EAxisList::Type::YZ;
		return true;
	}

	if (EditingBevelIdx != INDEX_NONE)
	{
		OutAxisList = EAxisList::Type::Screen;
		return true;
	}

	return Super::GetWidgetAxisList(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShapeIrregularPolygonDynamicMeshVisualizer::GetWidgetAxisListDragOverride(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingGlobalBevelSize)
	{
		OutAxisList = EAxisList::Type::Y;
		return true;
	}

	if (EditingBevelIdx != INDEX_NONE)
	{
		OutAxisList = EAxisList::Type::Y;
		return true;
	}

	return Super::GetWidgetAxisListDragOverride(InViewportClient, InWidgetMode, OutAxisList);
}

FVector2D FAvaShapeIrregularPolygonDynamicMeshVisualizer::GetCornerBevelDirection2D(const FMeshType* InDynMesh, int32 InPointIdx) const
{
	int32 PreviousIdx = InPointIdx - 1;

	if (PreviousIdx < 0)
	{
		PreviousIdx = InDynMesh->GetNumPoints() - 1;
	}

	int32 NextIdx = InPointIdx + 1;

	if (NextIdx == InDynMesh->GetNumPoints())
	{
		NextIdx = 0;
	}

	const FVector2D OutVectorA = InDynMesh->GetPoint(PreviousIdx).Location - InDynMesh->GetPoint(InPointIdx).Location;
	const FVector2D OutVectorB = InDynMesh->GetPoint(NextIdx).Location - InDynMesh->GetPoint(InPointIdx).Location;

	return (OutVectorA.GetSafeNormal(0.f) + OutVectorB.GetSafeNormal(0.f)) / 2.f;
}

bool FAvaShapeIrregularPolygonDynamicMeshVisualizer::GetCustomInputCoordinateSystem(const FEditorViewportClient* InViewportClient,
	FMatrix& OutMatrix) const
{
	if (EditingBevelIdx != INDEX_NONE)
	{
		if (DynamicMeshComponent.IsValid())
		{
			const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

			if (DynMesh && DynMesh->GetNumPoints() >= 3 && DynMesh->GetShapeActor())
			{
				const FVector2D BevelVector2D = GetCornerBevelDirection2D(DynMesh, EditingBevelIdx);

				FVector2D Polar;
				FMath::CartesianToPolar(BevelVector2D, Polar);

				OutMatrix = FRotationMatrix::Make(FQuat(DynMesh->GetShapeActor()->GetActorForwardVector(), Polar.Y));

				return true;
			}
		}
	}

	return Super::GetCustomInputCoordinateSystem(InViewportClient, OutMatrix);
}

bool FAvaShapeIrregularPolygonDynamicMeshVisualizer::HandleInputDeltaInternal(FEditorViewportClient* InViewportClient,
	FViewport* InViewport, const FVector& InAccumulatedTranslation, const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale)
{
	if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
	{
		return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
	}

	if (DynamicMeshComponent.IsValid())
	{
		FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

		if (DynMesh)
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
			{
				if (bEditingGlobalBevelSize)
				{
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
					{
						float BevelSize = InitialGlobalBevelSize;
						BevelSize = FMath::Max(BevelSize + InAccumulatedTranslation.Y / DynMesh->GetSize2D().X * 2.f, 0.f);
						DynMesh->Modify();
						DynMesh->SetGlobalBevelSize(BevelSize);

						bHasBeenModified = true;
						NotifyPropertyModified(DynMesh, GlobalBevelSizeProperty, EPropertyChangeType::Interactive);
					}
				}
				else if (EditingPointIdx != INDEX_NONE)
				{
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::YZ)
					{
						FVector2D NewPointLocation = InitialPointLocation;

						if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
						{
							NewPointLocation.X += InAccumulatedTranslation.Y;
						}

						if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
						{
							NewPointLocation.Y += InAccumulatedTranslation.Z;
						}

						SnapLocation2D(InViewportClient, InitialTransform, NewPointLocation);

						DynMesh->Modify();
						if (DynMesh->SetLocation(EditingPointIdx, NewPointLocation))
						{
							bHasBeenModified = true;
							NotifyPropertyModified(DynMesh, LocationProperty, EPropertyChangeType::Interactive, PointsProperty);
						}
					}
				}
				else if (EditingBevelIdx != INDEX_NONE)
				{
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
					{
						const bool bSetGlobalBevelCorner = FSlateApplication::Get().GetModifierKeys().IsAltDown();
						const float MaxBevelSize = DynMesh->GetMaxBevelSizeForPoint(EditingBevelIdx);
						float BevelSize = InitialPointBevelSize;

						BevelSize = FMath::Clamp(BevelSize + InAccumulatedTranslation.Y / (MaxBevelSize * 0.75f), 0.f, 1.f);

						if (bSetGlobalBevelCorner)
						{
							DynMesh->Modify();
							DynMesh->SetGlobalBevelSize(BevelSize);

							bHasBeenModified = true;
							NotifyPropertyModified(DynMesh, GlobalBevelSizeProperty, EPropertyChangeType::Interactive);
						}
						else
						{
							DynMesh->Modify();
							if (DynMesh->SetBevelSize(EditingBevelIdx, BevelSize))
							{
								bHasBeenModified = true;
								NotifyPropertyModified(DynMesh, BevelSizeProperty, EPropertyChangeType::Interactive, PointsProperty);
							}
						}
					}
				}

				return true;
			}
		}
	}
	else
	{
		EndEditing();
	}

	return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
}

void FAvaShapeIrregularPolygonDynamicMeshVisualizer::StoreInitialValues()
{
	Super::StoreInitialValues();

	FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh)
	{
		return;
	}

	if (!bEditingGlobalBevelSize && EditingPointIdx == INDEX_NONE && EditingBevelIdx == INDEX_NONE)
	{
		return;
	}

	if (bEditingGlobalBevelSize)
	{
		InitialGlobalBevelSize = DynMesh->GetGlobalBevelSize();
	}

	if (EditingPointIdx != INDEX_NONE)
	{
		const FAvaShapeRoundedCorner& Point = DynMesh->GetPoint(EditingPointIdx);
		InitialPointLocation                  = Point.Location;
	}
	else if (EditingBevelIdx != INDEX_NONE)
	{
		const FAvaShapeRoundedCorner& Point = DynMesh->GetPoint(EditingBevelIdx);
		InitialPointBevelSize                 = Point.Settings.BevelSize;
	}

	DynMesh->BackupPoints();
}

bool FAvaShapeIrregularPolygonDynamicMeshVisualizer::ResetValue(FEditorViewportClient* InViewportClient,
	HHitProxy* InHitProxy)
{
	if (!InHitProxy->IsA(HAvaShapeCornersHitProxy::StaticGetType())
		&& InHitProxy->IsA(HAvaShapeIrregularPolygonBevelHitProxy::StaticGetType()))
	{
		return Super::ResetValue(InViewportClient, InHitProxy);
	}

	if (InHitProxy->IsA(HAvaShapeCornersHitProxy::StaticGetType()))
	{
		const HAvaShapeCornersHitProxy* BevelsHitProxy = static_cast<HAvaShapeCornersHitProxy*>(InHitProxy);
		FMeshType* HitProxyDynamicMesh                  = Cast<FMeshType>(const_cast<UActorComponent*>(BevelsHitProxy->Component.Get()));

		if (HitProxyDynamicMesh)
		{
			FScopedTransaction Transaction(LOCTEXT("VisualizerResetValue", "Visualizer Reset Value"));
			HitProxyDynamicMesh->SetFlags(RF_Transactional);
			HitProxyDynamicMesh->Modify();
			HitProxyDynamicMesh->SetGlobalBevelSize(0.f);
			NotifyPropertyModified(HitProxyDynamicMesh, GlobalBevelSizeProperty, EPropertyChangeType::ValueSet);
		}
	}
	else if (InHitProxy->IsA(HAvaShapeIrregularPolygonBevelHitProxy::StaticGetType()))
	{
		const HAvaShapeIrregularPolygonBevelHitProxy* BevelHitProxy = static_cast<HAvaShapeIrregularPolygonBevelHitProxy*>(InHitProxy);

		FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(const_cast<UActorComponent*>(BevelHitProxy->Component.Get()));

		if (HitProxyDynamicMesh)
		{
			FScopedTransaction Transaction(LOCTEXT("VisualizerResetValue", "Visualizer Reset Value"));
			HitProxyDynamicMesh->SetFlags(RF_Transactional);
			HitProxyDynamicMesh->Modify();
			HitProxyDynamicMesh->SetBevelSize(BevelHitProxy->PointIdx, 0.f);
			NotifyPropertyModified(HitProxyDynamicMesh, BevelSizeProperty, EPropertyChangeType::ValueSet, PointsProperty);
		}
	}

	return true;
}

bool FAvaShapeIrregularPolygonDynamicMeshVisualizer::IsEditing() const
{
	if (bEditingGlobalBevelSize || EditingPointIdx != INDEX_NONE || EditingBevelIdx != INDEX_NONE)
	{
		return true;
	}

	return Super::IsEditing();
}

void FAvaShapeIrregularPolygonDynamicMeshVisualizer::EndEditing()
{
	Super::EndEditing();

	bEditingGlobalBevelSize = false;
	EditingPointIdx         = INDEX_NONE;
	EditingBevelIdx         = INDEX_NONE;
}

void FAvaShapeIrregularPolygonDynamicMeshVisualizer::TrackingStopped(FEditorViewportClient* InViewportClient, bool bInDidMove)
{
	if (DynamicMeshComponent.IsValid() && EditingPointIdx != INDEX_NONE)
	{
		FMeshType* Mesh = Cast<FMeshType>(DynamicMeshComponent.Get());
		Mesh->RecalculateActorPosition();
	}

	FAvaShape2DDynamicMeshVisualizer::TrackingStopped(InViewportClient, bInDidMove);
}

#undef LOCTEXT_NAMESPACE
