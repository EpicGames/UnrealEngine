// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Visualizers/AvaShape2DDynMeshVis.h"

struct HAvaShapeIrregularPolygonPointHitProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaShapeIrregularPolygonPointHitProxy(const UActorComponent* InComponent, int32 InPointIdx)
		: HAvaHitProxy(InComponent)
		, PointIdx(InPointIdx)
	{}

	int32 PointIdx;
};

struct HAvaShapeIrregularPolygonBevelHitProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaShapeIrregularPolygonBevelHitProxy(const UActorComponent* InComponent, int32 InPointIdx)
		: HAvaHitProxy(InComponent)
		, PointIdx(InPointIdx)
	{}

	int32 PointIdx;
};

struct HAvaShapeIrregularPolygonBreakHitProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaShapeIrregularPolygonBreakHitProxy(const UActorComponent* InComponent, int32 InPointIdx)
		: HAvaHitProxy(InComponent)
		, PointIdx(InPointIdx)
	{}

	int32 PointIdx;
};

class UAvaShapeIrregularPolygonDynamicMesh;

class FAvaShapeIrregularPolygonDynamicMeshVisualizer : public FAvaShape2DDynamicMeshVisualizer
{
public:

	using Super = FAvaShape2DDynamicMeshVisualizer;
	using FMeshType = UAvaShapeIrregularPolygonDynamicMesh;

	FAvaShapeIrregularPolygonDynamicMeshVisualizer();

	//~ Begin FAvaVisualizerBase
	virtual TMap<UObject*, TArray<FProperty*>> GatherEditableProperties(UObject* InObject) const override;
	virtual bool VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* InVisProxy,
		const FViewportClick& InClick) override;
	virtual bool GetWidgetMode(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode& OutMode) const override;
	virtual bool GetWidgetLocation(const FEditorViewportClient* InViewportClient, FVector& OutLocation) const override;
	virtual bool GetWidgetAxisList(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode,
		EAxisList::Type& OutAxisList) const override;
	virtual bool GetWidgetAxisListDragOverride(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode,
		EAxisList::Type& OutAxisList) const override;
	virtual bool GetCustomInputCoordinateSystem(const FEditorViewportClient* InViewportClient, FMatrix& OutMatrix) const override;
	virtual bool ResetValue(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy) override;
	virtual bool IsEditing() const override;
	virtual void EndEditing() override;
	virtual void TrackingStopped(FEditorViewportClient* InViewportClient, bool bInDidMove) override;
	//~ End FAvaVisualizerBase

protected:
	FProperty* GlobalBevelSizeProperty;
	FProperty* PointsProperty;
	FProperty* LocationProperty;
	FProperty* BevelSizeProperty;

	bool bEditingGlobalBevelSize;
	float InitialGlobalBevelSize;
	int32 EditingPointIdx;
	int32 EditingBevelIdx;
	FVector2D InitialPointLocation;
	float InitialPointBevelSize;

	virtual void DrawVisualizationNotEditing(const UActorComponent* InComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;
	virtual void DrawVisualizationEditing(const UActorComponent* InComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;

	virtual bool HandleInputDeltaInternal(FEditorViewportClient* InViewportClient, FViewport* InViewport, const FVector& InAccumulatedTranslation,
		const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale) override;

	virtual void StoreInitialValues() override;

	FVector2D GetCornerBevelDirection2D(const FMeshType* InDynMesh, int32 InPointIdx) const;

	void DrawGlobalBevelButton(const FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		int32 InIconIndex, const FLinearColor& InColor) const;

	void DrawPointButton(const FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		const FLinearColor& InColor, int32 InPointIdx) const;

	void DrawBevelButton(const FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		const FLinearColor& InColor, int32 InPointIdx) const;

	void DrawBreakButton(const FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		const FLinearColor& InColor, int32 InPointIdx) const;

	FVector GetGlobalBevelLocation(const FMeshType* InDynMesh) const;
	FVector GetPointLocation(const FMeshType* InDynMesh, int32 InPointIdx) const;
	FVector GetBevelLocation(const FMeshType* InDynMesh, int32 InPointIdx) const;
	
	// Returns the half way point between the given point index and the one following it (0 for the last point.)
	FVector GetBreakLocation(const FMeshType* InDynMesh, int32 InPointIdx) const;

	virtual void GenerateContextSensitiveSnapPoints() override;
};
