// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeLineDynMeshVis.h"
#include "AvaField.h"
#include "AvaShapeActor.h"
#include "AvaShapeSprites.h"
#include "DynamicMeshes/AvaShapeLineDynMesh.h"
#include "EditorViewportClient.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "Interaction/AvaSnapOperation.h"
#include "Templates/SharedPointer.h"
#include "TextureResource.h"
#include "Viewport/Interaction/AvaSnapPoint.h"

IMPLEMENT_HIT_PROXY(HAvaShapeLineEndHitProxy, HAvaHitProxy)

FAvaShapeLineDynamicMeshVisualizer::FAvaShapeLineDynamicMeshVisualizer()
	: FAvaShapeRoundedPolygonDynamicMeshVisualizer()
	, bEditingStart(false)
	, bEditingEnd(false)
{
	using namespace UE::AvaCore;
	VectorProperty = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, Vector));
}

TMap<UObject*, TArray<FProperty*>> FAvaShapeLineDynamicMeshVisualizer::GatherEditableProperties(UObject* InObject) const
{
	TMap<UObject*, TArray<FProperty*>> Properties = Super::GatherEditableProperties(InObject);

	if (UAvaShapeDynamicMeshBase* DynMesh = Cast<UAvaShapeDynamicMeshBase>(InObject))
	{
		Properties.FindOrAdd(DynMesh).Add(VectorProperty);
	}

	return Properties;
}

bool FAvaShapeLineDynamicMeshVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient,
	HComponentVisProxy* InVisProxy, const FViewportClick& InClick)
{
	if (InClick.GetKey() != EKeys::LeftMouseButton)
	{
		EndEditing();
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	FMeshType* DynMesh = Cast<FMeshType>(const_cast<UActorComponent*>(InVisProxy->Component.Get()));

	if (DynMesh == nullptr)
	{
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	if (InVisProxy->IsA(HAvaShapeLineEndHitProxy::StaticGetType()))
	{
		EndEditing();
		bEditingStart        = static_cast<HAvaShapeLineEndHitProxy*>(InVisProxy)->bIsStart;
		bEditingEnd          = !bEditingStart;
		StartEditing(InViewportClient, DynMesh);
		return true;
	}

	return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
}

void FAvaShapeLineDynamicMeshVisualizer::DrawVisualizationNotEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationNotEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	if (ShouldDrawExtraHandles(InComponent, InView))
	{
		DrawStartButton(DynMesh, InView, InPDI, Inactive);
		DrawEndButton(DynMesh, InView, InPDI, Inactive);
	}
}

void FAvaShapeLineDynamicMeshVisualizer::DrawVisualizationEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawStartButton(DynMesh, InView, InPDI, Inactive);
	DrawEndButton(DynMesh, InView, InPDI, Inactive);
}

FVector FAvaShapeLineDynamicMeshVisualizer::GetStartWorldLocation(const FMeshType* InDynMesh) const
{
	FVector2D StartPosition = -(InDynMesh->GetVector() / 2.f);
	const FTransform ComponentTransform = InDynMesh->GetTransform();
	const FVector OutVector = ComponentTransform.TransformPosition({0.f, StartPosition.X, StartPosition.Y});

	return OutVector;
}

FVector FAvaShapeLineDynamicMeshVisualizer::GetEndWorldLocation(const FMeshType* InDynMesh) const
{
	FVector2D EndPosition = InDynMesh->GetVector() / 2.f;
	const FTransform ComponentTransform = InDynMesh->GetTransform();
	const FVector OutVector = ComponentTransform.TransformPosition({0.f, EndPosition.X, EndPosition.Y});

	return OutVector;
}

void FAvaShapeLineDynamicMeshVisualizer::GenerateContextSensitiveSnapPoints()
{
	Super::GenerateContextSensitiveSnapPoints();

	if (!DynamicMeshComponent.IsValid())
	{
		return;
	}

	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh)
	{
		return;
	}

	const AActor* Actor = DynMesh->GetOwner();

	if (!Actor)
	{
		return;
	}

	const FVector EndLocation = GetStartWorldLocation(DynMesh);

	SnapOperation->AddActorSnapPoint(FAvaSnapPoint::CreateActorCustomSnapPoint(
			Actor,
			EndLocation,
			1
		));

	const FVector StartLocation = GetEndWorldLocation(DynMesh);

	SnapOperation->AddActorSnapPoint(FAvaSnapPoint::CreateActorCustomSnapPoint(
			Actor,
			StartLocation,
			0
		));
}

void FAvaShapeLineDynamicMeshVisualizer::DrawStartButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, const FLinearColor& InColor) const
{
	static float BaseSize = 1.f;

	UTexture2D* SizeSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::SizeSprite);

	if (!SizeSprite || !SizeSprite->GetResource())
	{
		return;
	}

	FVector IconLocation = GetStartWorldLocation(InDynMesh);
	const float IconSize = BaseSize * GetIconSizeScale(InView, IconLocation);
	IconLocation += InDynMesh->GetTransform().TransformVector({-1.f, 0.f, 0.f});

	InPDI->SetHitProxy(new HAvaShapeLineEndHitProxy(InDynMesh, true));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, SizeSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaShapeLineDynamicMeshVisualizer::DrawEndButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, const FLinearColor& InColor) const
{
	static float BaseSize = 1.f;

	UTexture2D* SizeSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::SizeSprite);

	if (!SizeSprite || !SizeSprite->GetResource())
	{
		return;
	}

	FVector IconLocation = GetEndWorldLocation(InDynMesh);
	const float IconSize = BaseSize * GetIconSizeScale(InView, IconLocation);
	IconLocation += InDynMesh->GetTransform().TransformVector({-1.f, 0.f, 0.f});

	InPDI->SetHitProxy(new HAvaShapeLineEndHitProxy(InDynMesh, false));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, SizeSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

bool FAvaShapeLineDynamicMeshVisualizer::GetWidgetLocation(const FEditorViewportClient* InViewportClient,
	FVector& OutLocation) const
{
	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh)
	{
		return Super::GetWidgetLocation(InViewportClient, OutLocation);
	}

	if (bEditingStart)
	{
		OutLocation = GetStartWorldLocation(DynMesh);
		return true;
	}

	if (bEditingEnd)
	{
		OutLocation = GetEndWorldLocation(DynMesh);
		return true;
	}

	return Super::GetWidgetLocation(InViewportClient, OutLocation);
}

bool FAvaShapeLineDynamicMeshVisualizer::GetWidgetMode(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode& OutMode) const
{
	if (bEditingStart)
	{
		OutMode = UE::Widget::WM_Translate;
		return true;
	}

	if (bEditingEnd)
	{
		OutMode = UE::Widget::WM_Translate;
		return true;
	}

	return Super::GetWidgetMode(InViewportClient, OutMode);
}

bool FAvaShapeLineDynamicMeshVisualizer::GetWidgetAxisList(
	const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingStart)
	{
		OutAxisList = EAxisList::Type::YZ;
		return true;
	}

	if (bEditingEnd)
	{
		OutAxisList = EAxisList::Type::YZ;
		return true;
	}

	return Super::GetWidgetAxisList(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShapeLineDynamicMeshVisualizer::HandleInputDeltaInternal(FEditorViewportClient* InViewportClient,
	FViewport* InViewport, const FVector& InAccumulatedTranslation, const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale)
{
	if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
	{
		return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
	}

	if (DynamicMeshComponent.IsValid())
	{
		FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

		if (DynMesh)
		{
			if (bEditingStart)
			{
				if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
				{
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::YZ)
					{
						FVector2D StartPosition         = -InitialVector / 2.f;
						FVector2D OriginalStartPosition = StartPosition;

						if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
						{
							StartPosition.X += InAccumulatedTranslation.Y;
						}

						if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
						{
							StartPosition.Y += InAccumulatedTranslation.Z;
						}

						FVector2D PreSnappedStartPosition = StartPosition;
						SnapLocation2D(InViewportClient, InitialTransform, StartPosition);

						const FVector2D NewVector = InitialVector / 2.f - StartPosition;

						if (NewVector.SizeSquared() > 1.f)
						{
							const FVector2D NewStartPosition = StartPosition + NewVector * 0.5;
							const FVector NewLocation = InitialTransform.TransformPosition({0.f, NewStartPosition.X, NewStartPosition.Y});
							DynMesh->Modify();
							DynMesh->SetVector(NewVector);
							DynMesh->SetMeshRegenWorldLocation(NewLocation, false);
							NotifyPropertiesModified(DynMesh, {VectorProperty, MeshRegenWorldLocationProperty},
								EPropertyChangeType::Interactive);
						}
					}
				}

				return true;
			}

			if (bEditingEnd)
			{
				if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
				{
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::YZ)
					{
						FVector2D EndPosition = InitialVector / 2.f;

						if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
						{
							EndPosition.X += InAccumulatedTranslation.Y;
						}

						if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
						{
							EndPosition.Y += InAccumulatedTranslation.Z;
						}

						SnapLocation2D(InViewportClient, InitialTransform, EndPosition);

						const FVector2D NewVector = InitialVector / 2.f + EndPosition;

						if (NewVector.SizeSquared() > 1.f)
						{
							const FVector2D NewEndPosition = EndPosition - NewVector * 0.5;
							const FVector NewLocation = InitialTransform.TransformPosition({0.f, NewEndPosition.X, NewEndPosition.Y});
							DynMesh->Modify();
							DynMesh->SetVector(NewVector);
							DynMesh->SetMeshRegenWorldLocation(NewLocation, false);
							NotifyPropertiesModified(DynMesh, {VectorProperty, MeshRegenWorldLocationProperty},
								EPropertyChangeType::Interactive);
						}
					}
				}

				return true;
			}
		}
	}
	else
	{
		EndEditing();
	}

	return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
}

void FAvaShapeLineDynamicMeshVisualizer::StoreInitialValues()
{
	Super::StoreInitialValues();

	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh)
	{
		return;
	}

	InitialStart  = DynMesh->GetVector() / -2.f;
	InitialVector = DynMesh->GetVector();
}

bool FAvaShapeLineDynamicMeshVisualizer::IsEditing() const
{
	if (bEditingStart || bEditingEnd)
	{
		return true;
	}

	return Super::IsEditing();
}

void FAvaShapeLineDynamicMeshVisualizer::EndEditing()
{
	Super::EndEditing();

	bEditingStart = false;
	bEditingEnd   = false;
}
