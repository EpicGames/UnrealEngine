// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaShapeRoundedPolygonDynMeshVis.h"

struct HAvaShapeLineEndHitProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaShapeLineEndHitProxy(const UActorComponent* InComponent, bool bInIsStart)
		: HAvaHitProxy(InComponent)
		, bIsStart(bInIsStart)
	{}

	bool bIsStart;
};

class UAvaShapeLineDynamicMesh;

class FAvaShapeLineDynamicMeshVisualizer : public FAvaShapeRoundedPolygonDynamicMeshVisualizer
{
public:
	using Super = FAvaShapeRoundedPolygonDynamicMeshVisualizer;
	using FMeshType = UAvaShapeLineDynamicMesh;

	FAvaShapeLineDynamicMeshVisualizer();

	//~ Begin FAvaVisualizerBase
	virtual TMap<UObject*, TArray<FProperty*>> GatherEditableProperties(UObject* InObject) const override;
	virtual bool VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* InVisProxy,
		const FViewportClick& InClick) override;
	virtual bool GetWidgetLocation(const FEditorViewportClient* InViewportClient, FVector& OutLocation) const override;
	virtual bool GetWidgetMode(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode& OutMode) const override;
	virtual bool GetWidgetAxisList(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode,
		EAxisList::Type& OutAxisList) const override;
	virtual bool IsEditing() const override;
	virtual void EndEditing() override;
	//~ End FAvaVisualizerBase

protected:
	FProperty* VectorProperty;

	bool bEditingStart;
	FVector2D InitialStart;
	bool bEditingEnd;
	FVector2D InitialVector;

	virtual void DrawVisualizationNotEditing(const UActorComponent* InComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;
	virtual void DrawVisualizationEditing(const UActorComponent* InComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;

	virtual bool HandleInputDeltaInternal(FEditorViewportClient* InViewportClient, FViewport* InViewport, const FVector& InAccumulatedTranslation,
		const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale) override;

	virtual void StoreInitialValues() override;

	void DrawStartButton(const FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		const FLinearColor& InColor) const;

	void DrawEndButton(const FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		const FLinearColor& InColor) const;

	FVector GetStartWorldLocation(const FMeshType* InDynMesh) const;
	FVector GetEndWorldLocation(const FMeshType* InDynMesh) const;

	virtual void GenerateContextSensitiveSnapPoints() override;
};
