// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeNGonDynMeshVis.h"
#include "AvaField.h"
#include "AvaShapeActor.h"
#include "AvaShapeSprites.h"
#include "DynamicMeshes/AvaShapeNGonDynMesh.h"
#include "EditorViewportClient.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "RenderResource.h"
#include "ScopedTransaction.h"
#include "TextureResource.h"

FAvaShapeNGonDynamicMeshVisualizer::FAvaShapeNGonDynamicMeshVisualizer()
	: FAvaShapeRoundedPolygonDynamicMeshVisualizer()
	, bEditingNumSides(false)
	, InitialNumSides(0)
{
	using namespace UE::AvaCore;
	NumSidesProperty = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, NumSides));
}

TMap<UObject*, TArray<FProperty*>> FAvaShapeNGonDynamicMeshVisualizer::GatherEditableProperties(UObject* InObject) const
{
	TMap<UObject*, TArray<FProperty*>> Properties = Super::GatherEditableProperties(InObject);

	if (FMeshType* DynMesh = Cast<FMeshType>(InObject))
	{
		Properties.FindOrAdd(DynMesh).Add(NumSidesProperty);
	}

	return Properties;
}

bool FAvaShapeNGonDynamicMeshVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient,
	HComponentVisProxy* InVisProxy, const FViewportClick& InClick)
{
	if (InClick.GetKey() != EKeys::LeftMouseButton)
	{
		EndEditing();
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	FMeshType* DynMesh = Cast<FMeshType>(const_cast<UActorComponent*>(InVisProxy->Component.Get()));

	if (DynMesh == nullptr)
	{
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	if (InVisProxy->IsA(HAvaShapeNumSidesHitProxy::StaticGetType()))
	{
		EndEditing();
		bEditingNumSides = true;
		StartEditing(InViewportClient, DynMesh);
		return true;
	}

	return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
}

void FAvaShapeNGonDynamicMeshVisualizer::DrawVisualizationNotEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationNotEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawNumSidesButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive);
	++InOutIconIndex;
}

void FAvaShapeNGonDynamicMeshVisualizer::DrawVisualizationEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawNumSidesButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingNumSides ? Active : Inactive);
	++InOutIconIndex;
}

void FAvaShapeNGonDynamicMeshVisualizer::DrawNumSidesButton(const FMeshType* InDynMesh,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	UTexture2D* NumSidesSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::NumSidesSprite);

	if (!NumSidesSprite || !NumSidesSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeNumSidesHitProxy(InDynMesh));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, NumSidesSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

bool FAvaShapeNGonDynamicMeshVisualizer::GetWidgetLocation(const FEditorViewportClient* InViewportClient,
	FVector& OutLocation) const
{
	if (bEditingNumSides)
	{
		if (DynamicMeshComponent.IsValid())
		{
			OutLocation = DynamicMeshComponent->GetShapeActor()->GetActorLocation();
			return true;
		}
	}

	return Super::GetWidgetLocation(InViewportClient, OutLocation);
}

bool FAvaShapeNGonDynamicMeshVisualizer::GetWidgetMode(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode& OutMode) const
{
	if (bEditingNumSides)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Scale;
		return true;
	}

	return Super::GetWidgetMode(InViewportClient, OutMode);
}

bool FAvaShapeNGonDynamicMeshVisualizer::GetWidgetAxisList(
	const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingNumSides)
	{
		OutAxisList = EAxisList::Type::Y;
		return true;
	}

	return Super::GetWidgetAxisList(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShapeNGonDynamicMeshVisualizer::HandleInputDeltaInternal(FEditorViewportClient* InViewportClient,
	FViewport* InViewport, const FVector& InAccumulatedTranslation, const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale)
{
	if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
	{
		return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
	}

	if (bEditingNumSides)
	{
		if (DynamicMeshComponent.IsValid())
		{
			FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

			if (DynMesh)
			{
				if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Scale)
				{
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
					{
						int32 NumSides = InitialNumSides;
						NumSides = FMath::Clamp(NumSides + static_cast<int32>(InAccumulatedScale.Y), 3, 128);
						DynMesh->Modify();
						DynMesh->SetNumSides(NumSides);

						bHasBeenModified = true;
						NotifyPropertyModified(DynMesh, NumSidesProperty, EPropertyChangeType::Interactive);
					}
				}
			}
		}
		else
		{
			EndEditing();
		}

		return true;
	}

	return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
}

void FAvaShapeNGonDynamicMeshVisualizer::StoreInitialValues()
{
	Super::StoreInitialValues();

	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh)
	{
		return;
	}

	InitialNumSides = DynMesh->GetNumSides();
}

bool FAvaShapeNGonDynamicMeshVisualizer::ResetValue(FEditorViewportClient* InViewportClient,
	HHitProxy* InHitProxy)
{
	if (!InHitProxy->IsA(HAvaShapeNumSidesHitProxy::StaticGetType()))
	{
		return Super::ResetValue(InViewportClient, InHitProxy);
	}

	const HAvaShapeNumSidesHitProxy* NumSidesHitProxy = static_cast<HAvaShapeNumSidesHitProxy*>(InHitProxy);

	if (NumSidesHitProxy->Component.IsValid())
	{
		FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(
			const_cast<UActorComponent*>(NumSidesHitProxy->Component.Get()));

		if (HitProxyDynamicMesh)
		{
			FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
			HitProxyDynamicMesh->SetFlags(RF_Transactional);
			HitProxyDynamicMesh->Modify();
			HitProxyDynamicMesh->SetNumSides(5);
			NotifyPropertyModified(HitProxyDynamicMesh, NumSidesProperty, EPropertyChangeType::ValueSet);
		}
	}

	return true;
}

bool FAvaShapeNGonDynamicMeshVisualizer::IsEditing() const
{
	if (bEditingNumSides)
	{
		return true;
	}

	return Super::IsEditing();
}

void FAvaShapeNGonDynamicMeshVisualizer::EndEditing()
{
	Super::EndEditing();

	bEditingNumSides = false;
}
