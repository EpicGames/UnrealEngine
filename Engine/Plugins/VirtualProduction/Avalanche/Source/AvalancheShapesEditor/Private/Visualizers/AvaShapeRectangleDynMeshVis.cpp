// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeRectangleDynMeshVis.h"
#include "AvaField.h"
#include "AvaShapeActor.h"
#include "AvaShapeSprites.h"
#include "DynamicMeshes/AvaShapeRectangleDynMesh.h"
#include "EditorViewportClient.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "RenderResource.h"
#include "ScopedTransaction.h"
#include "TextureResource.h"

IMPLEMENT_HIT_PROXY(HAvaShapeRectangleCornerHitProxy, HAvaHitProxy)
IMPLEMENT_HIT_PROXY(HAvaShapeRectangleSlantHitProxy, HAvaHitProxy)

FAvaShapeRectangleDynamicMeshVisualizer::FAvaShapeRectangleDynamicMeshVisualizer()
	: FAvaShape2DDynamicMeshVisualizer()
	, bEditingGlobalBevelSize(false)
	, InitialGlobalBevelSize(0)
	, Corner(EAvaAnchors::None)
	, InitialTopLeftBevelSize(0)
	, InitialTopRightBevelSize(0)
	, InitialBottomLeftBevelSize(0)
	, InitialBottomRightBevelSize(0)
	, SlantSide(EAvaAnchors::None)
	, InitialLeftSlant(0)
	, InitialRightSlant(0)
{
	using namespace UE::AvaCore;

	GlobalBevelSizeProperty           = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, GlobalBevelSize));
	TopLeftCornerSettingsProperty     = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, TopLeft));
	TopRightCornerSettingsProperty    = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, TopRight));
	BottomLeftCornerSettingsProperty  = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, BottomLeft));
	BottomRightCornerSettingsProperty = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, BottomRight));
	LeftSlantProperty                 = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, LeftSlant));
	RightSlantProperty                = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, RightSlant));

	BevelSizeProperty  = GetProperty<FAvaShapeRectangleCornerSettings>(GET_MEMBER_NAME_CHECKED(FAvaShapeRectangleCornerSettings, BevelSize));
	CornerTypeProperty = GetProperty<FAvaShapeRectangleCornerSettings>(GET_MEMBER_NAME_CHECKED(FAvaShapeRectangleCornerSettings, Type));
}

TMap<UObject*, TArray<FProperty*>> FAvaShapeRectangleDynamicMeshVisualizer::GatherEditableProperties(UObject* InObject) const
{
	TMap<UObject*, TArray<FProperty*>> Properties = Super::GatherEditableProperties(InObject);

	if (FMeshType* DynMesh = Cast<FMeshType>(InObject))
	{
		Properties.FindOrAdd(DynMesh).Add(GlobalBevelSizeProperty);
		Properties.FindOrAdd(DynMesh).Add(LeftSlantProperty);
		Properties.FindOrAdd(DynMesh).Add(RightSlantProperty);
		Properties.FindOrAdd(DynMesh).Add(TopLeftCornerSettingsProperty);
		Properties.FindOrAdd(DynMesh).Add(TopRightCornerSettingsProperty);
		Properties.FindOrAdd(DynMesh).Add(BottomLeftCornerSettingsProperty);
		Properties.FindOrAdd(DynMesh).Add(BottomRightCornerSettingsProperty);
	}

	return Properties;
}

bool FAvaShapeRectangleDynamicMeshVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient, 
	HComponentVisProxy* InVisProxy, const FViewportClick& InClick)
{
	if (InClick.GetKey() != EKeys::LeftMouseButton || !InVisProxy)
	{
		EndEditing();
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	FMeshType* DynMesh = Cast<FMeshType>(const_cast<UActorComponent*>(InVisProxy->Component.Get()));

	if (DynMesh == nullptr)
	{
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	if (InVisProxy->IsA(HAvaShapeCornersHitProxy::StaticGetType()))
	{
		const HAvaShapeCornersHitProxy* CornerHitProxy = static_cast<HAvaShapeCornersHitProxy*>(
			InVisProxy);

		if (CornerHitProxy)
		{
			EndEditing();
			bEditingGlobalBevelSize = true;
			StartEditing(InViewportClient, DynMesh);
			return true;
		}
	}

	if (InVisProxy->IsA(HAvaShapeRectangleCornerHitProxy::StaticGetType()))
	{
		const HAvaShapeRectangleCornerHitProxy* CornerHitProxy = static_cast<HAvaShapeRectangleCornerHitProxy*>(
			InVisProxy);

		if (CornerHitProxy)
		{
			EndEditing();
			Corner = CornerHitProxy->Corner;
			StartEditing(InViewportClient, DynMesh);
			return true;
		}
	}
	else if (InVisProxy->IsA(HAvaShapeRectangleSlantHitProxy::StaticGetType()))
	{
		const HAvaShapeRectangleSlantHitProxy* SlantHitProxy = static_cast<HAvaShapeRectangleSlantHitProxy*>(InVisProxy);

		if (SlantHitProxy)
		{
			EndEditing();
			SlantSide = SlantHitProxy->Side;
			StartEditing(InViewportClient, DynMesh);
			return true;
		}
	}

	return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
}

void FAvaShapeRectangleDynamicMeshVisualizer::DrawVisualizationNotEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationNotEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawGlobalBevelButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive);
	++InOutIconIndex;

	if (ShouldDrawExtraHandles(InComponent, InView))
	{
		DrawBevelButton(DynMesh, InView, InPDI, Inactive, EAvaAnchors::TopLeft);
		DrawBevelButton(DynMesh, InView, InPDI, Inactive, EAvaAnchors::TopRight);
		DrawBevelButton(DynMesh, InView, InPDI, Inactive, EAvaAnchors::BottomLeft);
		DrawBevelButton(DynMesh, InView, InPDI, Inactive, EAvaAnchors::BottomRight);
		DrawSlantButton(DynMesh, InView, InPDI, Inactive, EAvaAnchors::Left);
		DrawSlantButton(DynMesh, InView, InPDI, Inactive, EAvaAnchors::Right);
	}
}

void FAvaShapeRectangleDynamicMeshVisualizer::DrawVisualizationEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(
		InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawGlobalBevelButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingGlobalBevelSize ? Active : Inactive);
	++InOutIconIndex;

	if (ShouldDrawExtraHandles(InComponent, InView))
	{
		DrawBevelButton(DynMesh, InView, InPDI, Inactive, EAvaAnchors::TopLeft);
		DrawBevelButton(DynMesh, InView, InPDI, Inactive, EAvaAnchors::TopRight);
		DrawBevelButton(DynMesh, InView, InPDI, Inactive, EAvaAnchors::BottomLeft);
		DrawBevelButton(DynMesh, InView, InPDI, Inactive, EAvaAnchors::BottomRight);
		DrawSlantButton(DynMesh, InView, InPDI, Inactive, EAvaAnchors::Left);
		DrawSlantButton(DynMesh, InView, InPDI, Inactive, EAvaAnchors::Right);
	}
}

void FAvaShapeRectangleDynamicMeshVisualizer::DrawSizeButtons(const Super::FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI)
{
	const FMeshType* RectangleMesh = Cast<FMeshType>(InDynMesh);

	if (!RectangleMesh)
	{
		return;
	}

	const bool bIsCenterHorizontalAlignment = RectangleMesh->GetHorizontalAlignment() != EAvaHorizontalAlignment::Center;
	const bool bIsCenterVerticalAlignment = RectangleMesh->GetVerticalAlignment() != EAvaVerticalAlignment::Center;

	for (const AvaAlignment Alignment : SupportedAlignments)
	{
		if (SizeDragAnchor == Alignment)
		{
			continue;
		}

		if (bIsCenterHorizontalAlignment && RectangleMesh->GetHorizontalAlignment() == GetHAlignment(Alignment))
		{
			continue;
		}

		if (bIsCenterVerticalAlignment && RectangleMesh->GetVerticalAlignment() == GetVAlignment(Alignment))
		{
			continue;
		}

		DrawSizeButton(RectangleMesh, InView, InPDI, Alignment);
	}
}

FVector FAvaShapeRectangleDynamicMeshVisualizer::GetGlobalBevelLocation(const FMeshType* InDynMesh)
{
	const FVector2D Size2D = InDynMesh->GetSize2D();
	const float LeftOffset = FMath::Min(Size2D.X / 10.f, 20.f);

	FVector LeftPosition = FVector::ZeroVector;
	LeftPosition.Y       = -Size2D.X / 2.f + LeftOffset + InDynMesh->GetGlobalBevelSize() / 2.f;

	const FTransform ComponentTransform = InDynMesh->GetTransform();

	return ComponentTransform.TransformPosition(LeftPosition);
}

FVector FAvaShapeRectangleDynamicMeshVisualizer::GetCornerLocation(const FMeshType* InDynMesh,
	EAvaAnchors InCorner)
{
	const FVector2D Size2D          = InDynMesh->GetSize2D();
	const float MinDim              = FMath::Min(Size2D.X, Size2D.Y);
	const float DistanceFromCornerY = MinDim / 4.f;
	const float DistanceFromCornerZ = MinDim / 4.f;

	FVector CornerPosition = FVector::ZeroVector;

	switch (InCorner)
	{
		case EAvaAnchors::TopLeft:
			CornerPosition.Y = -Size2D.X / 2.f + DistanceFromCornerY;
			CornerPosition.Z = Size2D.Y / 2.f - DistanceFromCornerZ;
			break;

		case EAvaAnchors::TopRight:
			CornerPosition.Y = Size2D.X / 2.f - DistanceFromCornerY;
			CornerPosition.Z = Size2D.Y / 2.f - DistanceFromCornerZ;
			break;

		case EAvaAnchors::BottomLeft:
			CornerPosition.Y = -Size2D.X / 2.f + DistanceFromCornerY;
			CornerPosition.Z = -Size2D.Y / 2.f + DistanceFromCornerZ;
			break;

		case EAvaAnchors::BottomRight:
			CornerPosition.Y = Size2D.X / 2.f - DistanceFromCornerY;
			CornerPosition.Z = -Size2D.Y / 2.f + DistanceFromCornerZ;
			break;

		default:
			return FVector::ZeroVector;
	}

	return InDynMesh->GetTransform().TransformPosition(CornerPosition);
}

FVector FAvaShapeRectangleDynamicMeshVisualizer::GetSlantLocation(const FMeshType* InDynMesh,
	EAvaAnchors InSide)
{
	const FVector2D Size2D       = InDynMesh->GetSize2D();
	const float MinDim           = FMath::Min(Size2D.X, Size2D.Y);
	const float DistanceFromSide = MinDim / 4.f;
	FVector SlantPosition        = FVector::ZeroVector;

	switch (InSide)
	{
		case EAvaAnchors::Left:
			SlantPosition.Y = -Size2D.X / 2.f + DistanceFromSide;
			break;

		case EAvaAnchors::Right:
			SlantPosition.Y = Size2D.X / 2.f - DistanceFromSide;
			break;

		default:
			return FVector::ZeroVector;
	}

	const FTransform ComponentTransform = InDynMesh->GetTransform();

	return ComponentTransform.TransformPosition(SlantPosition);
}

FVector FAvaShapeRectangleDynamicMeshVisualizer::GetCornerDragLocation(const FMeshType* InDynMesh,
	EAvaAnchors InCorner)
{
	const FVector2D Size2D   = InDynMesh->GetSize2D();
	FVector CornerPosition   = FVector::ZeroVector;
	const float CornerOffset = FMath::Min(FMath::Min(Size2D.X, Size2D.Y) / 10.f, 20.f);

	switch (InCorner)
	{
		case EAvaAnchors::TopLeft:
			CornerPosition.Y = -Size2D.X / 2.f + CornerOffset + InDynMesh->GetTopLeftBevelSize() / 2.f;
			CornerPosition.Z = Size2D.Y / 2.f - CornerOffset - InDynMesh->GetTopLeftBevelSize() / 2.f;
			break;

		case EAvaAnchors::TopRight:
			CornerPosition.Y = Size2D.X / 2.f - CornerOffset - InDynMesh->GetTopRightBevelSize() / 2.f;
			CornerPosition.Z = Size2D.Y / 2.f - CornerOffset - InDynMesh->GetTopRightBevelSize() / 2.f;
			break;

		case EAvaAnchors::BottomLeft:
			CornerPosition.Y = -Size2D.X / 2.f + CornerOffset + InDynMesh->GetBottomLeftBevelSize() / 2.f;
			CornerPosition.Z = -Size2D.Y / 2.f + CornerOffset + InDynMesh->GetBottomLeftBevelSize() / 2.f;
			break;

		case EAvaAnchors::BottomRight:
			CornerPosition.Y = Size2D.X / 2.f - CornerOffset - InDynMesh->GetBottomRightBevelSize() / 2.f;
			CornerPosition.Z = -Size2D.Y / 2.f + CornerOffset + InDynMesh->GetBottomRightBevelSize() / 2.f;
			break;

		default:
			return FVector::ZeroVector;
	}

	const FTransform ComponentTransform = InDynMesh->GetTransform();

	return ComponentTransform.TransformPosition(CornerPosition) + GetAlignmentOffset(InDynMesh);
}

void FAvaShapeRectangleDynamicMeshVisualizer::DrawGlobalBevelButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	UTexture2D* UVSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::CornerSprite);

	if (!UVSprite || !UVSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeCornersHitProxy(InDynMesh));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, UVSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaShapeRectangleDynamicMeshVisualizer::DrawBevelButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, const FLinearColor& InColor, EAvaAnchors InCorner) const
{
	static const float BaseSize = 1.f;

	UTexture2D* NumSidesSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::BevelSprite);

	if (!NumSidesSprite || !NumSidesSprite->GetResource())
	{
		return;
	}

	const FVector CornerLocation = GetCornerDragLocation(InDynMesh, InCorner);
	const float IconSize         = BaseSize * GetIconSizeScale(InView, CornerLocation);

	InPDI->SetHitProxy(new HAvaShapeRectangleCornerHitProxy(InDynMesh, InCorner));
	InPDI->DrawSprite(CornerLocation, IconSize, IconSize, NumSidesSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaShapeRectangleDynamicMeshVisualizer::DrawSlantButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, const FLinearColor& InColor, EAvaAnchors InSide) const
{
	static const float BaseSize = 1.f;

	UTexture2D* NumSidesSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::SlantSprite);

	if (!NumSidesSprite || !NumSidesSprite->GetResource())
	{
		return;
	}

	const FVector Slantocation = GetSlantLocation(InDynMesh, InSide) + GetAlignmentOffset(InDynMesh);
	const float IconSize       = BaseSize * GetIconSizeScale(InView, Slantocation);

	InPDI->SetHitProxy(new HAvaShapeRectangleSlantHitProxy(InDynMesh, InSide));
	InPDI->DrawSprite(Slantocation, IconSize, IconSize, NumSidesSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

bool FAvaShapeRectangleDynamicMeshVisualizer::GetWidgetLocation(const FEditorViewportClient* InViewportClient,
	FVector& OutLocation) const
{
	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh)
	{
		if (Super::GetWidgetLocation(InViewportClient, OutLocation))
		{
			OutLocation += GetAlignmentOffset(DynMesh);
			return true;
		}

		return false;
	}

	if (bEditingGlobalBevelSize)
	{
		OutLocation = GetGlobalBevelLocation(DynMesh) + GetAlignmentOffset(DynMesh);
		return true;
	}

	if (Corner != EAvaAnchors::None)
	{
		OutLocation = GetCornerDragLocation(DynMesh, Corner);
		return true;
	}

	if (SlantSide != EAvaAnchors::None)
	{
		OutLocation = GetSlantLocation(DynMesh, SlantSide) + GetAlignmentOffset(DynMesh);
		return true;
	}

	if (Super::GetWidgetLocation(InViewportClient, OutLocation))
	{
		if (SizeDragAnchor == INDEX_NONE)
		{
			OutLocation += GetAlignmentOffset(DynMesh);
		}

		return true;
	}

	return false;
}

bool FAvaShapeRectangleDynamicMeshVisualizer::GetWidgetMode(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode& OutMode) const
{
	if (bEditingGlobalBevelSize)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Translate;
		return true;
	}

	if (Corner != EAvaAnchors::None)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Translate;
		return true;
	}

	if (SlantSide != EAvaAnchors::None)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Translate;
		return true;
	}

	return Super::GetWidgetMode(InViewportClient, OutMode);
}

bool FAvaShapeRectangleDynamicMeshVisualizer::GetWidgetAxisList(
	const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingGlobalBevelSize)
	{
		OutAxisList = EAxisList::Type::Screen;
		return true;
	}

	if (Corner != EAvaAnchors::None)
	{
		OutAxisList = EAxisList::Type::Screen;
		return true;
	}

	if (SlantSide != EAvaAnchors::None)
	{
		OutAxisList = EAxisList::Type::Y;
		return true;
	}

	return Super::GetWidgetAxisList(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShapeRectangleDynamicMeshVisualizer::GetWidgetAxisListDragOverride(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingGlobalBevelSize)
	{
		OutAxisList = EAxisList::Type::Y;
		return true;
	}

	if (Corner != EAvaAnchors::None)
	{
		OutAxisList = EAxisList::Type::Y;
		return true;
	}

	return Super::GetWidgetAxisListDragOverride(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShapeRectangleDynamicMeshVisualizer::GetCustomInputCoordinateSystem(const FEditorViewportClient* InViewportClient,
	FMatrix& OutMatrix) const
{
	if (Corner != EAvaAnchors::None)
	{
		const FQuat ActorRotation = DynamicMeshComponent.Get()->GetShapeActor()->GetTransform().GetRotation();
		float Rotation            = 0.f;

		switch (Corner)
		{
			case EAvaAnchors::TopLeft:
				Rotation = PI / -4.f;
				break;

			case EAvaAnchors::BottomLeft:
				Rotation = PI / 4.f;
				break;

			case EAvaAnchors::TopRight:
				Rotation = -PI / 2.f + PI / -4.f;
				break;

			case EAvaAnchors::BottomRight:
				Rotation = PI / 2.f + PI / 4.f;
				break;

			default:
				// Can never happen
				return Super::GetCustomInputCoordinateSystem(InViewportClient, OutMatrix);
		}

		OutMatrix = FRotationMatrix::Make(FQuat(FVector(1.f, 0.f, 0.f), Rotation)) * FRotationMatrix::Make(ActorRotation);

		return true;
	}

	return Super::GetCustomInputCoordinateSystem(InViewportClient, OutMatrix);
}

bool FAvaShapeRectangleDynamicMeshVisualizer::HandleInputDeltaInternal(FEditorViewportClient* InViewportClient,
	FViewport* InViewport, const FVector& InAccumulatedTranslation, const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale)
{
	if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
	{
		return false;
	}

	if (bEditingGlobalBevelSize)
	{
		if (DynamicMeshComponent.IsValid())
		{
			FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

			if (DynMesh)
			{
				if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
				{
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
					{
						float BevelSize = InitialGlobalBevelSize;
						BevelSize = FMath::Max(BevelSize + InAccumulatedTranslation.Y * 2.f, 0.f);
						DynMesh->Modify();
						DynMesh->SetGlobalBevelSize(BevelSize);

						bHasBeenModified = true;
						NotifyPropertyModified(DynMesh, GlobalBevelSizeProperty, EPropertyChangeType::Interactive);
					}
				}
			}
		}
		else
		{
			EndEditing();
		}

		return true;
	}

	if (Corner != EAvaAnchors::None)
	{
		if (DynamicMeshComponent.IsValid())
		{
			FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

			if (DynMesh)
			{
				if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
				{
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
					{
						const static float Sqrt2 = sqrt(2.0);

						float BevelSize;
						const bool bSetGlobalBevelCorner = FSlateApplication::Get().GetModifierKeys().IsAltDown();

						switch (Corner)
						{
							case EAvaAnchors::TopLeft:
								BevelSize = InitialTopLeftBevelSize;
								BevelSize = FMath::Max(BevelSize + InAccumulatedTranslation.Y * Sqrt2, 0.f);

								if (bSetGlobalBevelCorner)
								{
									DynMesh->Modify();
									DynMesh->SetGlobalBevelSize(BevelSize);

									bHasBeenModified = true;
									NotifyPropertyModified(DynMesh, GlobalBevelSizeProperty, EPropertyChangeType::Interactive);
								}
								else
								{
									DynMesh->Modify();
									DynMesh->SetTopLeftBevelSize(BevelSize);

									bHasBeenModified = true;
									NotifyPropertyModified(DynMesh, BevelSizeProperty, EPropertyChangeType::Interactive, TopLeftCornerSettingsProperty);
								}

								break;

							case EAvaAnchors::TopRight:
								BevelSize = InitialTopRightBevelSize;
								BevelSize = FMath::Max(BevelSize + InAccumulatedTranslation.Y * Sqrt2, 0.f);

								if (bSetGlobalBevelCorner)
								{
									DynMesh->Modify();
									DynMesh->SetGlobalBevelSize(BevelSize);

									bHasBeenModified = true;
									NotifyPropertyModified(DynMesh, GlobalBevelSizeProperty, EPropertyChangeType::Interactive);
								}
								else
								{
									DynMesh->Modify();
									DynMesh->SetTopRightBevelSize(BevelSize);

									bHasBeenModified = true;
									NotifyPropertyModified(DynMesh, BevelSizeProperty, EPropertyChangeType::Interactive, TopRightCornerSettingsProperty);
								}

								break;

							case EAvaAnchors::BottomLeft:
								BevelSize = InitialBottomLeftBevelSize;
								BevelSize = FMath::Max(BevelSize + InAccumulatedTranslation.Y * Sqrt2, 0.f);

								if (bSetGlobalBevelCorner)
								{
									DynMesh->Modify();
									DynMesh->SetGlobalBevelSize(BevelSize);

									bHasBeenModified = true;
									NotifyPropertyModified(DynMesh, GlobalBevelSizeProperty, EPropertyChangeType::Interactive);
								}
								else
								{
									DynMesh->Modify();
									DynMesh->SetBottomLeftBevelSize(BevelSize);

									bHasBeenModified = true;
									NotifyPropertyModified(DynMesh, BevelSizeProperty, EPropertyChangeType::Interactive, BottomLeftCornerSettingsProperty);
								}

								break;

							case EAvaAnchors::BottomRight:
								BevelSize = InitialBottomRightBevelSize;
								BevelSize = FMath::Max(BevelSize + InAccumulatedTranslation.Y * Sqrt2, 0.f);

								if (bSetGlobalBevelCorner)
								{
									DynMesh->Modify();
									DynMesh->SetGlobalBevelSize(BevelSize);

									bHasBeenModified = true;
									NotifyPropertyModified(DynMesh, GlobalBevelSizeProperty, EPropertyChangeType::Interactive);
								}
								else
								{
									DynMesh->Modify();
									DynMesh->SetBottomRightBevelSize(BevelSize);

									bHasBeenModified = true;
									NotifyPropertyModified(DynMesh, BevelSizeProperty, EPropertyChangeType::Interactive, BottomRightCornerSettingsProperty);
								}

								break;

							default:
								// Nothing to do
								break;
						}
					}
				}
			}
		}
		else
		{
			EndEditing();
		}

		return true;
	}

	if (SlantSide != EAvaAnchors::None)
	{
		if (DynamicMeshComponent.IsValid())
		{
			FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

			if (DynMesh)
			{
				if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
				{
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
					{
						const float Delta = InAccumulatedTranslation.Y * 0.005f;
						float Slant;

						switch (SlantSide)
						{
							case EAvaAnchors::Left:
								Slant = InitialLeftSlant;
								Slant = FMath::Clamp(Slant + Delta * UAvaShapeRectangleDynamicMesh::MaxSlantAngle, UAvaShapeRectangleDynamicMesh::MinSlantAngle, UAvaShapeRectangleDynamicMesh::MaxSlantAngle);
								DynMesh->Modify();
								DynMesh->SetLeftSlant(Slant);

								bHasBeenModified = true;
								NotifyPropertyModified(DynMesh, LeftSlantProperty, EPropertyChangeType::Interactive);

								break;

							case EAvaAnchors::Right:
								Slant = InitialRightSlant;
								Slant = FMath::Clamp(Slant + Delta * UAvaShapeRectangleDynamicMesh::MaxSlantAngle, UAvaShapeRectangleDynamicMesh::MinSlantAngle, UAvaShapeRectangleDynamicMesh::MaxSlantAngle);
								DynMesh->Modify();
								DynMesh->SetRightSlant(Slant);

								bHasBeenModified = true;
								NotifyPropertyModified(DynMesh, RightSlantProperty, EPropertyChangeType::Interactive);

								break;
							default:
								// Nothing to do
								break;
						}
					}
				}
			}
		}
		else
		{
			EndEditing();
		}

		return true;
	}
	
	const bool Result = Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);

	// Fix size drag anchor alignment offset
	if (SizeDragAnchor != INDEX_NONE)
	{
		if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
		{
			if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::YZ)
			{
				if (FMeshType* RectMesh = Cast<FMeshType>(DynamicMeshComponent.Get()))
				{
					const EAvaHorizontalAlignment HAlignment = RectMesh->GetHorizontalAlignment();
					const EAvaVerticalAlignment VAlignment = RectMesh->GetVerticalAlignment();
					
					if (HAlignment != EAvaHorizontalAlignment::Center || VAlignment != EAvaVerticalAlignment::Center)
					{
						if (!IsExtendBothSidesEnabled(RectMesh))
						{
							const FVector Offset = InitialTransform.TransformPosition(FVector::ZeroVector);
							RectMesh->SetMeshRegenWorldLocation(Offset, true);
						}
					}
				}
			}
		}
	}

	return Result;
}

void FAvaShapeRectangleDynamicMeshVisualizer::StoreInitialValues()
{
	Super::StoreInitialValues();

	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh)
	{
		return;
	}

	InitialLeftSlant            = DynMesh->GetLeftSlant();
	InitialRightSlant           = DynMesh->GetRightSlant();
	InitialGlobalBevelSize      = DynMesh->GetGlobalBevelSize();
	InitialTopLeftBevelSize     = DynMesh->GetTopLeftBevelSize();
	InitialTopRightBevelSize    = DynMesh->GetTopRightBevelSize();
	InitialBottomLeftBevelSize  = DynMesh->GetBottomLeftBevelSize();
	InitialBottomRightBevelSize = DynMesh->GetBottomRightBevelSize();
}

FVector FAvaShapeRectangleDynamicMeshVisualizer::GetAlignmentOffset(const FMeshType* InDynMesh)
{
	FVector AlignmentOffset = FVector::ZeroVector;

	if (IsValid(InDynMesh))
	{
		const FVector2D& Size2D = InDynMesh->GetSize2D();
		
		switch (InDynMesh->GetHorizontalAlignment())
		{
			case EAvaHorizontalAlignment::Left:
				AlignmentOffset.Y += Size2D.X / 2.f;
				break;

			case EAvaHorizontalAlignment::Right:
				AlignmentOffset.Y -= Size2D.X / 2.f;
				break;

			default:
				// Do nothing
				break;
		}

		switch (InDynMesh->GetVerticalAlignment())
		{
			case EAvaVerticalAlignment::Top:
				AlignmentOffset.Z -= Size2D.Y / 2.f;
				break;

			case EAvaVerticalAlignment::Bottom:
				AlignmentOffset.Z += Size2D.Y / 2.f;
				break;

			default:
				// Do nothing
				break;
		}
	}

	return InDynMesh->GetTransform().TransformVector(AlignmentOffset);
}

bool FAvaShapeRectangleDynamicMeshVisualizer::HandleModifiedClick(FEditorViewportClient* InViewportClient,
	HHitProxy* InHitProxy, const FViewportClick& InClick)
{
	if (!InHitProxy->IsA(HAvaShapeRectangleCornerHitProxy::StaticGetType()))
	{
		return Super::HandleModifiedClick(InViewportClient, InHitProxy, InClick);
	}

	// Only do anything if we have control down, not alt or shift
	if (InClick.IsAltDown() || InClick.IsShiftDown() || !InClick.IsControlDown())
	{
		return Super::HandleModifiedClick(InViewportClient, InHitProxy, InClick);
	}

	const HAvaShapeRectangleCornerHitProxy* CornerHitProxy = static_cast<HAvaShapeRectangleCornerHitProxy*>(InHitProxy);

	FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(
		const_cast<UActorComponent*>(CornerHitProxy->Component.Get()));

	if (HitProxyDynamicMesh && CornerHitProxy->Corner != EAvaAnchors::None)
	{
		auto GetNextCornerMode = [](EAvaShapeCornerType Type)->EAvaShapeCornerType
		{
			if (Type == EAvaShapeCornerType::Point)
			{
				return EAvaShapeCornerType::CurveIn;
			}

			if (Type == EAvaShapeCornerType::CurveIn)
			{
				return EAvaShapeCornerType::CurveOut;
			}

			return EAvaShapeCornerType::Point;
		};

		FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerChange", "Visualizer Change"));
		HitProxyDynamicMesh->Modify();

		switch (CornerHitProxy->Corner)
		{
			case EAvaAnchors::TopLeft:
				HitProxyDynamicMesh->SetTopLeftCornerType(
					GetNextCornerMode(HitProxyDynamicMesh->GetTopLeftCornerType()));
				NotifyPropertyModified(HitProxyDynamicMesh, CornerTypeProperty, EPropertyChangeType::ValueSet,
					TopLeftCornerSettingsProperty);
				break;

			case EAvaAnchors::TopRight:
				HitProxyDynamicMesh->SetTopRightCornerType(
					GetNextCornerMode(HitProxyDynamicMesh->GetTopRightCornerType()));
				NotifyPropertyModified(HitProxyDynamicMesh, CornerTypeProperty, EPropertyChangeType::ValueSet,
					TopRightCornerSettingsProperty);
				break;

			case EAvaAnchors::BottomLeft:
				HitProxyDynamicMesh->SetBottomLeftCornerType(
					GetNextCornerMode(HitProxyDynamicMesh->GetBottomLeftCornerType()));
				NotifyPropertyModified(HitProxyDynamicMesh, CornerTypeProperty, EPropertyChangeType::ValueSet,
					BottomLeftCornerSettingsProperty);
				break;

			case EAvaAnchors::BottomRight:
				HitProxyDynamicMesh->SetBottomRightCornerType(
					GetNextCornerMode(HitProxyDynamicMesh->GetBottomRightCornerType()));
				NotifyPropertyModified(HitProxyDynamicMesh, CornerTypeProperty, EPropertyChangeType::ValueSet,
					BottomRightCornerSettingsProperty);
				break;

			default:
				// Nothing to do
				break;
		}
	}

	return true;
}

bool FAvaShapeRectangleDynamicMeshVisualizer::ResetValue(FEditorViewportClient* InViewportClient,
	HHitProxy* InHitProxy)
{
	if (!InHitProxy->IsA(HAvaShapeCornersHitProxy::StaticGetType())
		&& !InHitProxy->IsA(HAvaShapeRectangleCornerHitProxy::StaticGetType())
		&& !InHitProxy->IsA(HAvaShapeRectangleSlantHitProxy::StaticGetType()))
	{
		return Super::ResetValue(InViewportClient, InHitProxy);
	}

	if (InHitProxy->IsA(HAvaShapeCornersHitProxy::StaticGetType()))
	{
		const HAvaShapeCornersHitProxy* CornersHitProxy = static_cast<HAvaShapeCornersHitProxy*>(InHitProxy);
		FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(const_cast<UActorComponent*>(CornersHitProxy->Component.Get()));

		if (HitProxyDynamicMesh)
		{
			FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
			HitProxyDynamicMesh->SetFlags(RF_Transactional);
			HitProxyDynamicMesh->Modify();
			HitProxyDynamicMesh->SetGlobalBevelSize(0.f);
			NotifyPropertyModified(HitProxyDynamicMesh, GlobalBevelSizeProperty, EPropertyChangeType::ValueSet);
		}
	}
	else if (InHitProxy->IsA(HAvaShapeRectangleCornerHitProxy::StaticGetType()))
	{
		const HAvaShapeRectangleCornerHitProxy* CornerHitProxy = static_cast<HAvaShapeRectangleCornerHitProxy*>(InHitProxy);
		FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(const_cast<UActorComponent*>(CornerHitProxy->Component.Get()));

		if (HitProxyDynamicMesh)
		{
			FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
			HitProxyDynamicMesh->SetFlags(RF_Transactional);
			HitProxyDynamicMesh->Modify();

			switch (CornerHitProxy->Corner)
			{
				case EAvaAnchors::TopLeft:
					HitProxyDynamicMesh->SetTopLeftBevelSize(0.f);
					HitProxyDynamicMesh->SetTopLeftCornerType(EAvaShapeCornerType::Point);
					NotifyPropertiesModified(HitProxyDynamicMesh, {BevelSizeProperty, CornerTypeProperty}, EPropertyChangeType::ValueSet,
						TopLeftCornerSettingsProperty);
					break;

				case EAvaAnchors::TopRight:
					HitProxyDynamicMesh->SetTopRightBevelSize(0.f);
					HitProxyDynamicMesh->SetTopRightCornerType(EAvaShapeCornerType::Point);
					NotifyPropertiesModified(HitProxyDynamicMesh, {BevelSizeProperty, CornerTypeProperty}, EPropertyChangeType::ValueSet,
						TopRightCornerSettingsProperty);
					break;

				case EAvaAnchors::BottomLeft:
					HitProxyDynamicMesh->SetBottomLeftBevelSize(0.f);
					HitProxyDynamicMesh->SetBottomLeftCornerType(EAvaShapeCornerType::Point);
					NotifyPropertiesModified(HitProxyDynamicMesh, {BevelSizeProperty, CornerTypeProperty}, EPropertyChangeType::ValueSet,
						BottomLeftCornerSettingsProperty);
					break;

				case EAvaAnchors::BottomRight:
					HitProxyDynamicMesh->SetBottomRightBevelSize(0.f);
					HitProxyDynamicMesh->SetBottomRightCornerType(EAvaShapeCornerType::Point);
					NotifyPropertiesModified(HitProxyDynamicMesh, {BevelSizeProperty, CornerTypeProperty}, EPropertyChangeType::ValueSet,
						BottomRightCornerSettingsProperty);
					break;

				default:
					// Nothing to do
					break;
			}
		}
	}
	else if (InHitProxy->IsA(HAvaShapeRectangleSlantHitProxy::StaticGetType()))
	{
		const HAvaShapeRectangleSlantHitProxy* CornerHitProxy = static_cast<HAvaShapeRectangleSlantHitProxy*>(InHitProxy);

		if (CornerHitProxy->Component.IsValid())
		{
			FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(const_cast<UActorComponent*>(CornerHitProxy->Component.Get()));

			if (HitProxyDynamicMesh)
			{
				FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
				HitProxyDynamicMesh->SetFlags(RF_Transactional);
				HitProxyDynamicMesh->Modify();

				switch (CornerHitProxy->Side)
				{
					case EAvaAnchors::Left:
						HitProxyDynamicMesh->SetLeftSlant(0.f);
						NotifyPropertyModified(HitProxyDynamicMesh, LeftSlantProperty, EPropertyChangeType::ValueSet);
						break;

					case EAvaAnchors::Right:
						HitProxyDynamicMesh->SetRightSlant(0.f);
						NotifyPropertyModified(HitProxyDynamicMesh, RightSlantProperty, EPropertyChangeType::ValueSet);
						break;

					default:
						// Nothing to do
						break;
				}
			}
		}
	}

	return true;
}

bool FAvaShapeRectangleDynamicMeshVisualizer::IsEditing() const
{
	if (bEditingGlobalBevelSize || Corner != EAvaAnchors::None || SlantSide != EAvaAnchors::None)
	{
		return true;
	}

	return Super::IsEditing();
}

void FAvaShapeRectangleDynamicMeshVisualizer::EndEditing()
{
	Super::EndEditing();

	bEditingGlobalBevelSize = false;
	Corner = EAvaAnchors::None;
	SlantSide = EAvaAnchors::None;
}
