// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaDefs.h"
#include "Visualizers/AvaShape2DDynMeshVis.h"

struct HAvaShapeRectangleCornerHitProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaShapeRectangleCornerHitProxy(const UActorComponent* InComponent, EAvaAnchors InCorner)
		: HAvaHitProxy(InComponent)
		, Corner(InCorner)
	{}

	EAvaAnchors Corner;
};

struct HAvaShapeRectangleSlantHitProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaShapeRectangleSlantHitProxy(const UActorComponent* InComponent, EAvaAnchors InSide)
		: HAvaHitProxy(InComponent)
		, Side(InSide)
	{}

	EAvaAnchors Side;
};

class UAvaShapeRectangleDynamicMesh;

class FAvaShapeRectangleDynamicMeshVisualizer : public FAvaShape2DDynamicMeshVisualizer
{
public:

	using Super = FAvaShape2DDynamicMeshVisualizer;
	using FMeshType = UAvaShapeRectangleDynamicMesh;

	FAvaShapeRectangleDynamicMeshVisualizer();

	//~ Begin FAvaVisualizerBase
	virtual TMap<UObject*, TArray<FProperty*>> GatherEditableProperties(UObject* InObject) const override;
	virtual bool VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* InVisProxy,
		const FViewportClick& InClick) override;
	virtual bool GetWidgetLocation(const FEditorViewportClient* InViewportClient, FVector& OutLocation) const override;
	virtual bool GetWidgetMode(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode& OutMode) const override;
	virtual bool GetWidgetAxisList(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode,
		EAxisList::Type& OutAxisList) const override;
	virtual bool GetWidgetAxisListDragOverride(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode,
		EAxisList::Type& OutAxisList) const override;
	virtual bool GetCustomInputCoordinateSystem(const FEditorViewportClient* InViewportClient, FMatrix& OutMatrix) const override;
	virtual bool HandleModifiedClick(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy, const FViewportClick& InClick) override;
	virtual bool ResetValue(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy) override;
	virtual bool IsEditing() const override;
	virtual void EndEditing() override;
	//~ End FAvaVisualizerBase

protected:
	static FVector GetGlobalBevelLocation(const FMeshType* InDynMesh);
	static FVector GetCornerLocation(const FMeshType* InDynMesh, EAvaAnchors InCorner);
	static FVector GetSlantLocation(const FMeshType* InDynMesh, EAvaAnchors InSide);
	static FVector GetCornerDragLocation(const FMeshType* InDynMesh, EAvaAnchors InCorner);

	static FVector GetAlignmentOffset(const FMeshType* InDynMesh);

	FProperty* GlobalBevelSizeProperty;
	FProperty* TopLeftCornerSettingsProperty;
	FProperty* TopRightCornerSettingsProperty;
	FProperty* BottomLeftCornerSettingsProperty;
	FProperty* BottomRightCornerSettingsProperty;
	FProperty* BevelSizeProperty;
	FProperty* CornerTypeProperty;
	FProperty* LeftSlantProperty;
	FProperty* RightSlantProperty;

	bool bEditingGlobalBevelSize;
	float InitialGlobalBevelSize;
	EAvaAnchors Corner;
	float InitialTopLeftBevelSize;
	float InitialTopRightBevelSize;
	float InitialBottomLeftBevelSize;
	float InitialBottomRightBevelSize;

	EAvaAnchors SlantSide;
	float InitialLeftSlant;
	float InitialRightSlant;

	virtual void DrawVisualizationNotEditing(const UActorComponent* InComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;
	virtual void DrawVisualizationEditing(const UActorComponent* InComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;

	virtual void DrawSizeButtons(const Super::FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI) override;

	virtual bool HandleInputDeltaInternal(FEditorViewportClient* InViewportClient, FViewport* InViewport, const FVector& InAccumulatedTranslation,
		const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale) override;

	virtual void StoreInitialValues() override;

	void DrawGlobalBevelButton(const FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		int32 InIconIndex, const FLinearColor& InColor) const;

	void DrawBevelButton(const FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		const FLinearColor& InColor, EAvaAnchors InCorner) const;

	void DrawSlantButton(const FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		const FLinearColor& InColor, EAvaAnchors InSide) const;
};
