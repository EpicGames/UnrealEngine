// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeRoundedPolygonDynMeshVis.h"
#include "AvaField.h"
#include "AvaShapeSprites.h"
#include "DynamicMeshes/AvaShapeRoundedPolygonDynMesh.h"
#include "EditorViewportClient.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "RenderResource.h"
#include "ScopedTransaction.h"
#include "TextureResource.h"

FAvaShapeRoundedPolygonDynamicMeshVisualizer::FAvaShapeRoundedPolygonDynamicMeshVisualizer()
	: FAvaShape2DDynamicMeshVisualizer()
	, bEditingCorners(false)
	, InitialBevelSize(0)
{
	using namespace UE::AvaCore;
	BevelSizeProperty = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, BevelSize));
}

TMap<UObject*, TArray<FProperty*>> FAvaShapeRoundedPolygonDynamicMeshVisualizer::GatherEditableProperties(UObject* InObject) const
{
	TMap<UObject*, TArray<FProperty*>> Properties = Super::GatherEditableProperties(InObject);

	if (FMeshType* DynMesh = Cast<FMeshType>(InObject))
	{
		Properties.FindOrAdd(DynMesh).Add(BevelSizeProperty);
	}

	return Properties;
}

bool FAvaShapeRoundedPolygonDynamicMeshVisualizer::VisProxyHandleClick(
		FEditorViewportClient* InViewportClient, HComponentVisProxy* InVisProxy, const FViewportClick& InClick
	)
{
	if (InClick.GetKey() != EKeys::LeftMouseButton)
	{
		EndEditing();
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	FMeshType* DynMesh = Cast<FMeshType>(const_cast<UActorComponent*>(InVisProxy->Component.Get()));

	if (DynMesh == nullptr)
	{
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	if (InVisProxy->IsA(HAvaShapeCornersHitProxy::StaticGetType()))
	{
		const HAvaShapeCornersHitProxy* CornerHitProxy = static_cast<HAvaShapeCornersHitProxy*>(
			InVisProxy);

		if (CornerHitProxy)
		{
			EndEditing();
			bEditingCorners = true;
			StartEditing(InViewportClient, DynMesh);
			return true;
		}
	}

	return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
}

void FAvaShapeRoundedPolygonDynamicMeshVisualizer::DrawVisualizationNotEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationNotEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawBevelButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive);
	++InOutIconIndex;
}

void FAvaShapeRoundedPolygonDynamicMeshVisualizer::DrawVisualizationEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawBevelButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingCorners ? Active : Inactive);
	++InOutIconIndex;
}

void FAvaShapeRoundedPolygonDynamicMeshVisualizer::DrawBevelButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	UTexture2D* UVSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::CornerSprite);

	if (!UVSprite || !UVSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeCornersHitProxy(InDynMesh));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, UVSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

bool FAvaShapeRoundedPolygonDynamicMeshVisualizer::GetWidgetLocation(const FEditorViewportClient* InViewportClient,
	FVector& OutLocation) const
{
	if (bEditingCorners)
	{
		const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

		if (!DynMesh)
		{
			return Super::GetWidgetLocation(InViewportClient, OutLocation);
		}

		const FVector2D Size2D = DynMesh->GetSize2D();
		const float LeftOffset = FMath::Min(Size2D.X / 10.f, 20.f);

		FVector LefPosition = FVector::ZeroVector;
		LefPosition.Y       = -Size2D.X / 2.f + LeftOffset + DynMesh->GetBevelSize() / 2.f;

		const FTransform ComponentTransform = DynMesh->GetTransform();
		OutLocation                         = ComponentTransform.TransformPosition(LefPosition);

		return true;
	}

	return Super::GetWidgetLocation(InViewportClient, OutLocation);
}

bool FAvaShapeRoundedPolygonDynamicMeshVisualizer::GetWidgetMode(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode& OutMode) const
{
	if (bEditingCorners)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Translate;
		return true;
	}

	return Super::GetWidgetMode(InViewportClient, OutMode);
}

bool FAvaShapeRoundedPolygonDynamicMeshVisualizer::GetWidgetAxisList(
	const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingCorners)
	{
		OutAxisList = EAxisList::Type::Screen;
		return true;
	}

	return Super::GetWidgetAxisList(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShapeRoundedPolygonDynamicMeshVisualizer::GetWidgetAxisListDragOverride(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingCorners)
	{
		OutAxisList = EAxisList::Type::Y;
		return true;
	}

	return Super::GetWidgetAxisListDragOverride(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShapeRoundedPolygonDynamicMeshVisualizer::HandleInputDeltaInternal(FEditorViewportClient* InViewportClient,
	FViewport* InViewport, const FVector& InAccumulatedTranslation, const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale)
{
	if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
	{
		return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
	}

	if (bEditingCorners)
	{
		if (DynamicMeshComponent.IsValid())
		{
			FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

			if (DynMesh)
			{
				if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
				{
					if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
					{
						float BevelSize = InitialBevelSize;
						BevelSize = FMath::Max(BevelSize + InAccumulatedTranslation.Y * 1.f, 0.f);
						DynMesh->Modify();
						DynMesh->SetBevelSize(BevelSize);

						bHasBeenModified = true;
						NotifyPropertyModified(DynMesh, BevelSizeProperty, EPropertyChangeType::Interactive);
					}
				}
			}
		}
		else
		{
			EndEditing();
		}

		return true;
	}

	return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
}

void FAvaShapeRoundedPolygonDynamicMeshVisualizer::StoreInitialValues()
{
	Super::StoreInitialValues();

	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh)
	{
		return;
	}

	InitialBevelSize = DynMesh->GetBevelSize();
}

bool FAvaShapeRoundedPolygonDynamicMeshVisualizer::ResetValue(FEditorViewportClient* InViewportClient,
	HHitProxy* InHitProxy)
{
	if (!InHitProxy->IsA(HAvaShapeCornersHitProxy::StaticGetType()))
	{
		return Super::ResetValue(InViewportClient, InHitProxy);
	}

	const HAvaShapeCornersHitProxy* CornerHitProxy = static_cast<HAvaShapeCornersHitProxy*>(InHitProxy);

	if (CornerHitProxy->Component.IsValid())
	{
		FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(
			const_cast<UActorComponent*>(CornerHitProxy->Component.Get()));

		if (HitProxyDynamicMesh)
		{
			FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
			HitProxyDynamicMesh->SetFlags(RF_Transactional);
			HitProxyDynamicMesh->Modify();
			HitProxyDynamicMesh->SetBevelSize(0.f);
			NotifyPropertyModified(HitProxyDynamicMesh, BevelSizeProperty, EPropertyChangeType::ValueSet);
		}
	}

	return true;
}

bool FAvaShapeRoundedPolygonDynamicMeshVisualizer::IsEditing() const
{
	if (bEditingCorners)
	{
		return true;
	}

	return Super::IsEditing();
}

void FAvaShapeRoundedPolygonDynamicMeshVisualizer::EndEditing()
{
	Super::EndEditing();

	bEditingCorners = false;
}
