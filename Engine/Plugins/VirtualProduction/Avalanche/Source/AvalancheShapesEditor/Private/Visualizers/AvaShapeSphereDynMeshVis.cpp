// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeSphereDynMeshVis.h"
#include "AvaField.h"
#include "AvaShapeActor.h"
#include "AvaShapeSprites.h"
#include "DynamicMeshes/AvaShapeSphereDynMesh.h"
#include "EditorViewportClient.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "ScopedTransaction.h"
#include "TextureResource.h"

FAvaShapeSphereDynamicMeshVisualizer::FAvaShapeSphereDynamicMeshVisualizer()
	: FAvaShape3DDynamicMeshVisualizer()
	, bEditingNumSides(false)
	, InitialNumSides(0)
	, bEditingLatitudeDegree(false)
	, InitialLatitudeDegree(0)
	, bEditingStartLongitude(false)
	, InitialStartRatio(0)
	, bEditingEndLongitude(false)
	, InitialEndRatio(0)
{
	using namespace UE::AvaCore;

	NumSidesProperty       = GetProperty<UAvaShapeSphereDynamicMesh>(GET_MEMBER_NAME_CHECKED(UAvaShapeSphereDynamicMesh, NumSides));
	LatitudeDegreeProperty = GetProperty<UAvaShapeSphereDynamicMesh>(GET_MEMBER_NAME_CHECKED(UAvaShapeSphereDynamicMesh, LatitudeDegree));
	StartLongitudeProperty = GetProperty<UAvaShapeSphereDynamicMesh>(GET_MEMBER_NAME_CHECKED(UAvaShapeSphereDynamicMesh, StartLongitude));
	EndLongitudeProperty   = GetProperty<UAvaShapeSphereDynamicMesh>(GET_MEMBER_NAME_CHECKED(UAvaShapeSphereDynamicMesh, EndLongitude));
}

TMap<UObject*, TArray<FProperty*>> FAvaShapeSphereDynamicMeshVisualizer::GatherEditableProperties(UObject* InObject) const
{
	TMap<UObject*, TArray<FProperty*>> Properties = Super::GatherEditableProperties(InObject);

	if (UAvaShapeDynamicMeshBase* DynMesh = Cast<UAvaShapeDynamicMeshBase>(InObject))
	{
		Properties.FindOrAdd(DynMesh).Add(NumSidesProperty);
		Properties.FindOrAdd(DynMesh).Add(LatitudeDegreeProperty);
		Properties.FindOrAdd(DynMesh).Add(StartLongitudeProperty);
		Properties.FindOrAdd(DynMesh).Add(EndLongitudeProperty);
	}

	return Properties;
}

bool FAvaShapeSphereDynamicMeshVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient,
	HComponentVisProxy* InVisProxy, const FViewportClick& InClick)
{
	if (InClick.GetKey() != EKeys::LeftMouseButton)
	{
		EndEditing();
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	FMeshType* DynMesh = Cast<FMeshType>(const_cast<UActorComponent*>(InVisProxy->Component.Get()));

	if (DynMesh == nullptr)
	{
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	// num slices
	if (InVisProxy->IsA(HAvaShapeNumSidesHitProxy::StaticGetType()))
	{
		EndEditing();
		bEditingNumSides = true;
		StartEditing(InViewportClient, DynMesh);
		return true;
	}
	// num sides
	if (InVisProxy->IsA(HAvaShapeAngleDegreeHitProxy::StaticGetType()))
	{
		const HAvaShapeAngleDegreeHitProxy* AngleDegreeHitProxy = static_cast<HAvaShapeAngleDegreeHitProxy*>(InVisProxy);
		const EAvaVerticalAlignment VAlignment = GetVAlignment(AngleDegreeHitProxy->DragAnchor);
		// start longitude
		if (VAlignment == EAvaVerticalAlignment::Top)
		{
			EndEditing();
			bEditingStartLongitude = true;
			StartEditing(InViewportClient, DynMesh);
			return true;
		}
		// end longitude
		else if (VAlignment == EAvaVerticalAlignment::Bottom)
		{
			EndEditing();
			bEditingEndLongitude = true;
			StartEditing(InViewportClient, DynMesh);
			return true;
		}
		// latitude degree
		else if (VAlignment == EAvaVerticalAlignment::Center)
		{
			EndEditing();
			bEditingLatitudeDegree = true;
			StartEditing(InViewportClient, DynMesh);
			return true;
		}
	}

	return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
}

bool FAvaShapeSphereDynamicMeshVisualizer::GetWidgetLocation(const FEditorViewportClient* InViewportClient,
	FVector& OutLocation) const
{
	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (DynMesh)
	{
		if (bEditingNumSides || bEditingLatitudeDegree)
		{
			OutLocation = DynMesh->GetShapeActor()->GetActorLocation();
			return true;
		}

		if (bEditingStartLongitude)
		{
			const float OffsetZ = FVector(0, 0, 1).RotateAngleAxis(DynMesh->GetStartLongitude(), FVector::XAxisVector).Z * DynMesh->GetSize3D().Z/2;
			const FVector Offset = FVector(0.f, 0.f, OffsetZ);
			OutLocation          = DynamicMeshComponent->GetShapeActor()->GetTransform().TransformPosition(Offset);
			return true;
		}

		if (bEditingEndLongitude)
		{
			const float OffsetZ = FVector(0, 0, 1).RotateAngleAxis(DynMesh->GetEndLongitude(), FVector::XAxisVector).Z * DynMesh->GetSize3D().Z/2;
			const FVector Offset = FVector(0.f, 0.f, OffsetZ);
			OutLocation          = DynamicMeshComponent->GetShapeActor()->GetTransform().TransformPosition(Offset);
			return true;
		}
	}

	return Super::GetWidgetLocation(InViewportClient, OutLocation);
}

bool FAvaShapeSphereDynamicMeshVisualizer::GetWidgetMode(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode& OutMode) const
{
	if (bEditingNumSides)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Scale;
		return true;
	}

	if (bEditingStartLongitude || bEditingEndLongitude)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Translate;
		return true;
	}

	if (bEditingLatitudeDegree)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Rotate;
		return true;
	}

	return Super::GetWidgetMode(InViewportClient, OutMode);
}

bool FAvaShapeSphereDynamicMeshVisualizer::GetWidgetAxisList(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingNumSides)
	{
		OutAxisList = EAxisList::Type::Y;
		return true;
	}

	if (bEditingStartLongitude || bEditingEndLongitude)
	{
		OutAxisList = EAxisList::Type::Screen;
		return true;
	}
	
	if (bEditingLatitudeDegree)
	{
		OutAxisList = EAxisList::Type::Z;
		return true;
	}

	return Super::GetWidgetAxisList(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShapeSphereDynamicMeshVisualizer::GetWidgetAxisListDragOverride(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingStartLongitude || bEditingEndLongitude)
	{
		OutAxisList = EAxisList::Type::Z;
		return true;
	}

	return Super::GetWidgetAxisListDragOverride(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShapeSphereDynamicMeshVisualizer::ResetValue(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy)
{
	if (!InHitProxy->IsA(HAvaShapeNumSidesHitProxy::StaticGetType()) &&
		!InHitProxy->IsA(HAvaShapeAngleDegreeHitProxy::StaticGetType()))
	{
		return Super::ResetValue(InViewportClient, InHitProxy);
	}

	// num sides
	if (InHitProxy->IsA(HAvaShapeNumSidesHitProxy::StaticGetType()))
	{
		const HAvaShapeNumSidesHitProxy* NumSidesHitProxy = static_cast<HAvaShapeNumSidesHitProxy*>(InHitProxy);

		if (NumSidesHitProxy->Component.IsValid())
		{
			FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(
				const_cast<UActorComponent*>(NumSidesHitProxy->Component.Get()));

			if (HitProxyDynamicMesh)
			{
				FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
				HitProxyDynamicMesh->SetFlags(RF_Transactional);
				HitProxyDynamicMesh->Modify();
				HitProxyDynamicMesh->SetNumSides(32);
				NotifyPropertyModified(HitProxyDynamicMesh, NumSidesProperty, EPropertyChangeType::ValueSet);
			}
		}
	}
	else if (InHitProxy->IsA(HAvaShapeAngleDegreeHitProxy::StaticGetType()))
	{
		const HAvaShapeAngleDegreeHitProxy* AngleDegreeHitProxy = static_cast<HAvaShapeAngleDegreeHitProxy*>(InHitProxy);

		if (AngleDegreeHitProxy->Component.IsValid())
		{
			FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(
				const_cast<UActorComponent*>(AngleDegreeHitProxy->Component.Get()));

			if (HitProxyDynamicMesh)
			{
				const EAvaVerticalAlignment VAlignment = GetVAlignment(AngleDegreeHitProxy->DragAnchor);
				// start longitude
				if (VAlignment == EAvaVerticalAlignment::Top)
				{
					FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
					HitProxyDynamicMesh->SetFlags(RF_Transactional);
					HitProxyDynamicMesh->Modify();
					HitProxyDynamicMesh->SetStartLongitude(0.f);
					NotifyPropertyModified(HitProxyDynamicMesh, StartLongitudeProperty, EPropertyChangeType::ValueSet);
				}
				// end longitude
				else if (VAlignment == EAvaVerticalAlignment::Bottom)
				{
					FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
					HitProxyDynamicMesh->SetFlags(RF_Transactional);
					HitProxyDynamicMesh->Modify();
					HitProxyDynamicMesh->SetEndLongitude(180.f);
					NotifyPropertyModified(HitProxyDynamicMesh, EndLongitudeProperty, EPropertyChangeType::ValueSet);
				}
				// latitude
				else if(VAlignment == EAvaVerticalAlignment::Center)
				{
					FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
					HitProxyDynamicMesh->SetFlags(RF_Transactional);
					HitProxyDynamicMesh->Modify();
					HitProxyDynamicMesh->SetLatitudeDegree(0.f);
					NotifyPropertyModified(HitProxyDynamicMesh, LatitudeDegreeProperty, EPropertyChangeType::ValueSet);
				}
			}
		}
	}

	return true;
}

bool FAvaShapeSphereDynamicMeshVisualizer::IsEditing() const
{
	if (bEditingLatitudeDegree || bEditingNumSides || bEditingStartLongitude || bEditingEndLongitude)
	{
		return true;
	}

	return Super::IsEditing();
}

void FAvaShapeSphereDynamicMeshVisualizer::EndEditing()
{
	Super::EndEditing();

	bEditingLatitudeDegree = false;
	bEditingNumSides = false;
	bEditingStartLongitude = false;
	bEditingEndLongitude = false;
}

void FAvaShapeSphereDynamicMeshVisualizer::DrawVisualizationNotEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationNotEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawNumSidesButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive);
	++InOutIconIndex;

	DrawAngleDegreeButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive, MakeAlignment(EAvaDepthAlignment::Center, EAvaHorizontalAlignment::Center, EAvaVerticalAlignment::Top));
	++InOutIconIndex;
	
	DrawAngleDegreeButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive, MakeAlignment(EAvaDepthAlignment::Center, EAvaHorizontalAlignment::Center, EAvaVerticalAlignment::Center));
	++InOutIconIndex;

	DrawAngleDegreeButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive, MakeAlignment(EAvaDepthAlignment::Center, EAvaHorizontalAlignment::Center, EAvaVerticalAlignment::Bottom));
	++InOutIconIndex;
}

void FAvaShapeSphereDynamicMeshVisualizer::DrawVisualizationEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawNumSidesButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingNumSides ? Active :  Inactive);
	++InOutIconIndex;

	DrawAngleDegreeButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingStartLongitude ? Active : Inactive, MakeAlignment(EAvaDepthAlignment::Center, EAvaHorizontalAlignment::Center, EAvaVerticalAlignment::Top));
	++InOutIconIndex;
	
	DrawAngleDegreeButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingLatitudeDegree ? Active : Inactive, MakeAlignment(EAvaDepthAlignment::Center, EAvaHorizontalAlignment::Center, EAvaVerticalAlignment::Center));
	++InOutIconIndex;

	DrawAngleDegreeButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingEndLongitude ? Active : Inactive, MakeAlignment(EAvaDepthAlignment::Center, EAvaHorizontalAlignment::Center, EAvaVerticalAlignment::Bottom));
	++InOutIconIndex;
}

bool FAvaShapeSphereDynamicMeshVisualizer::HandleInputDeltaInternal(FEditorViewportClient* InViewportClient,
	FViewport* InViewport, const FVector& InAccumulatedTranslation, const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale)
{
	if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
	{
		return false;
	}

	FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (DynMesh)
	{
		if (bEditingNumSides)
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Scale)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
				{
					int32 NumSides = InitialNumSides;
					NumSides = FMath::Clamp(NumSides + static_cast<int32>(InAccumulatedScale.Y), 4, 255);
					DynMesh->Modify();
					DynMesh->SetNumSides(NumSides);

					bHasBeenModified = true;
					NotifyPropertyModified(DynMesh, NumSidesProperty, EPropertyChangeType::Interactive);
				}
			}
			return true;
		}
		if (bEditingEndLongitude)
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
				{
					float EndLongitude = InitialEndRatio;
					// get ratio
					EndLongitude = FMath::Clamp(EndLongitude + ((InAccumulatedTranslation.Z * 2) / DynMesh->GetSize3D().Z), -1.f, 1.f);
					// get angle from ratio
					EndLongitude = FMath::GetMappedRangeValueClamped(FVector2D(-90, 90), FVector2D(180, 0), FMath::RadiansToDegrees(FMath::Asin(EndLongitude)));
					DynMesh->Modify();
					DynMesh->SetEndLongitude(EndLongitude);

					bHasBeenModified = true;
					NotifyPropertyModified(DynMesh, EndLongitudeProperty, EPropertyChangeType::Interactive);
				}
			}
			return true;
		}
		if (bEditingStartLongitude)
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
				{
					float StartLongitude = InitialStartRatio;
					// get ratio
					StartLongitude = FMath::Clamp(StartLongitude + ((InAccumulatedTranslation.Z * 2) / DynMesh->GetSize3D().Z), -1.f, 1.f);
					// get angle from ratio
					StartLongitude = FMath::GetMappedRangeValueClamped(FVector2D(90, -90), FVector2D(0, 180), FMath::RadiansToDegrees(FMath::Asin(StartLongitude)));
					DynMesh->Modify();
					DynMesh->SetStartLongitude(StartLongitude);

					bHasBeenModified = true;
					NotifyPropertyModified(DynMesh, StartLongitudeProperty, EPropertyChangeType::Interactive);
				}
			}
			return true;
		}
		if (bEditingLatitudeDegree)
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Rotate)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
				{
					float LatDegree = InitialLatitudeDegree;
					LatDegree = FMath::Clamp(LatDegree + static_cast<int32>(InAccumulatedRotation.Yaw), 0.f, 360.f);
					DynMesh->Modify();
					DynMesh->SetLatitudeDegree(LatDegree);

					bHasBeenModified = true;
					NotifyPropertyModified(DynMesh, LatitudeDegreeProperty, EPropertyChangeType::Interactive);
				}
			}
		}
	}
	else
	{
		EndEditing();
	}

	return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
}

void FAvaShapeSphereDynamicMeshVisualizer::StoreInitialValues()
{
	Super::StoreInitialValues();

	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh)
	{
		return;
	}
	
	InitialNumSides = DynMesh->GetNumSides();
	InitialLatitudeDegree = DynMesh->GetLatitudeDegree();
	InitialStartRatio = FVector(0, 0, 1).RotateAngleAxis(DynMesh->GetStartLongitude(), FVector::XAxisVector).Z;
	InitialEndRatio = FVector(0, 0, 1).RotateAngleAxis(DynMesh->GetEndLongitude(), FVector::XAxisVector).Z;
}

void FAvaShapeSphereDynamicMeshVisualizer::DrawNumSidesButton(const FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		int32 InIconIndex, const FLinearColor& InColor) const
{
	UTexture2D* NumSidesSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::NumSidesSprite);

	if (!NumSidesSprite || !NumSidesSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeNumSidesHitProxy(InDynMesh));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, NumSidesSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}


void FAvaShapeSphereDynamicMeshVisualizer::DrawAngleDegreeButton(const FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		int32 InIconIndex, const FLinearColor& InColor, AvaAlignment InAngleAnchor) const
{
	UTexture2D* InnerSizeSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::InnerSizeSprite);

	if (!InnerSizeSprite || !InnerSizeSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeAngleDegreeHitProxy(InDynMesh, InAngleAnchor));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, InnerSizeSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}
