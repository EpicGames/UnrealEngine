// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Visualizers/AvaShape3DDynMeshVis.h"

class UAvaShapeSphereDynamicMesh;

class FAvaShapeSphereDynamicMeshVisualizer : public FAvaShape3DDynamicMeshVisualizer
{
public:
	using Super = FAvaShape3DDynamicMeshVisualizer;
	using FMeshType = UAvaShapeSphereDynamicMesh;

	FAvaShapeSphereDynamicMeshVisualizer();

	//~ Begin FAvaVisualizerBase
	virtual TMap<UObject*, TArray<FProperty*>> GatherEditableProperties(UObject* InObject) const override;
	virtual bool VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* InVisProxy,
		const FViewportClick& InClick) override;
	virtual bool GetWidgetLocation(const FEditorViewportClient* InViewportClient, FVector& OutLocation) const override;
	virtual bool GetWidgetMode(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode& OutMode) const override;
	virtual bool GetWidgetAxisList(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode,
		EAxisList::Type& OutAxisList) const override;
	virtual bool GetWidgetAxisListDragOverride(const FEditorViewportClient* InViewportClient, UE::Widget::EWidgetMode InWidgetMode,
		EAxisList::Type& OutAxisList) const override;
	virtual bool ResetValue(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy) override;
	virtual bool IsEditing() const override;
	virtual void EndEditing() override;
	//~ End FAvaVisualizerBase

protected:
	FProperty* NumSidesProperty;
	FProperty* LatitudeDegreeProperty;
	FProperty* StartLongitudeProperty;
	FProperty* EndLongitudeProperty;

	bool bEditingNumSides;
	uint8 InitialNumSides;
	bool bEditingLatitudeDegree;
	float InitialLatitudeDegree;
	bool bEditingStartLongitude;
	float InitialStartRatio;
	bool bEditingEndLongitude;
	float InitialEndRatio;

	virtual void DrawVisualizationNotEditing(const UActorComponent* InComponent, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;
	virtual void DrawVisualizationEditing(const UActorComponent* InComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;
	virtual bool HandleInputDeltaInternal(FEditorViewportClient* InViewportClient, FViewport* InViewport, const FVector& InAccumulatedTranslation,
		const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale) override;
	virtual void StoreInitialValues() override;

	void DrawNumSidesButton(const FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		int32 InIconIndex, const FLinearColor& InColor) const;
	void DrawAngleDegreeButton(const FMeshType* InDynMesh, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		int32 InIconIndex, const FLinearColor& InColor, AvaAlignment InAngleAnchor) const;
};
