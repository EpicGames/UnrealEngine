// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeStarDynMeshVis.h"
#include "AvaField.h"
#include "AvaShapeActor.h"
#include "AvaShapeSprites.h"
#include "DynamicMeshes/AvaShapeStarDynMesh.h"
#include "EditorViewportClient.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "ScopedTransaction.h"
#include "TextureResource.h"

FAvaShapeStarDynamicMeshVisualizer::FAvaShapeStarDynamicMeshVisualizer()
	: FAvaShapeRoundedPolygonDynamicMeshVisualizer()
	, bEditingNumPoints(false)
	, InitialNumPoints(0)
	, bEditingInnerSize(false)
	, InitialInnerSize(0)
{
	using namespace UE::AvaCore;
	NumPointsProperty = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, NumPoints));
	InnerSizeProperty = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, InnerSize));
}

TMap<UObject*, TArray<FProperty*>> FAvaShapeStarDynamicMeshVisualizer::GatherEditableProperties(UObject* InObject) const
{
	TMap<UObject*, TArray<FProperty*>> Properties = Super::GatherEditableProperties(InObject);

	if (FMeshType* DynMesh = Cast<FMeshType>(InObject))
	{
		Properties.FindOrAdd(DynMesh).Add(NumPointsProperty);
		Properties.FindOrAdd(DynMesh).Add(InnerSizeProperty);
	}

	return Properties;
}

bool FAvaShapeStarDynamicMeshVisualizer::VisProxyHandleClick(
		FEditorViewportClient* InViewportClient, HComponentVisProxy* InVisProxy, const FViewportClick& InClick
	)
{
	if (InClick.GetKey() != EKeys::LeftMouseButton)
	{
		EndEditing();
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	FMeshType* DynMesh = Cast<FMeshType>(const_cast<UActorComponent*>(InVisProxy->Component.Get()));

	if (DynMesh == nullptr)
	{
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	if (InVisProxy->IsA(HAvaShapeNumPointsHitProxy::StaticGetType()))
	{
		EndEditing();
		bEditingNumPoints = true;
		StartEditing(InViewportClient, DynMesh);
		return true;
	}

	if (InVisProxy->IsA(HAvaShapeInnerSizeHitProxy::StaticGetType()))
	{
		EndEditing();
		bEditingInnerSize = true;
		StartEditing(InViewportClient, DynMesh);
		return true;
	}

	return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
}

void FAvaShapeStarDynamicMeshVisualizer::DrawVisualizationNotEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationNotEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawNumPointsButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive);
	++InOutIconIndex;

	DrawInnerSizeButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive);
	++InOutIconIndex;
}

void FAvaShapeStarDynamicMeshVisualizer::DrawVisualizationEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawNumPointsButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingNumPoints ? Active : Inactive);
	++InOutIconIndex;

	DrawInnerSizeButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingInnerSize ? Active : Inactive);
	++InOutIconIndex;
}

void FAvaShapeStarDynamicMeshVisualizer::DrawNumPointsButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	UTexture2D* NumPointsSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::NumPointsSprite);

	if (!NumPointsSprite || !NumPointsSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeNumPointsHitProxy(InDynMesh));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, NumPointsSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaShapeStarDynamicMeshVisualizer::DrawInnerSizeButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	UTexture2D* NumPointsSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::InnerSizeSprite);

	if (!NumPointsSprite || !NumPointsSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeInnerSizeHitProxy(InDynMesh));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, NumPointsSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

bool FAvaShapeStarDynamicMeshVisualizer::GetWidgetLocation(const FEditorViewportClient* InViewportClient,
	FVector& OutLocation) const
{
	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (DynMesh)
	{
		if (bEditingNumPoints)
		{
			OutLocation = DynMesh->GetShapeActor()->GetActorLocation();
			return true;
		}

		if (bEditingInnerSize)
		{
			const float OffsetY  = DynMesh->GetSize2D().X * DynMesh->GetInnerSize() / 2.f;
			const FVector Offset = FVector(0.f, -OffsetY, 0.f);
			OutLocation          = DynamicMeshComponent->GetShapeActor()->GetTransform().TransformPosition(Offset);
			return true;
		}
	}

	return Super::GetWidgetLocation(InViewportClient, OutLocation);
}

bool FAvaShapeStarDynamicMeshVisualizer::GetWidgetMode(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode& OutMode) const
{
	if (bEditingNumPoints)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Scale;
		return true;
	}

	if (bEditingInnerSize)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Translate;
		return true;
	}

	return Super::GetWidgetMode(InViewportClient, OutMode);
}

bool FAvaShapeStarDynamicMeshVisualizer::GetWidgetAxisList(const FEditorViewportClient* InViewportClient, 
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingNumPoints)
	{
		OutAxisList = EAxisList::Type::Y;
		return true;
	}

	if (bEditingInnerSize)
	{
		OutAxisList = EAxisList::Type::Screen;
		return true;
	}

	return Super::GetWidgetAxisList(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShapeStarDynamicMeshVisualizer::GetWidgetAxisListDragOverride(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingInnerSize)
	{
		OutAxisList = EAxisList::Type::Y;
		return true;
	}

	return Super::GetWidgetAxisListDragOverride(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShapeStarDynamicMeshVisualizer::HandleInputDeltaInternal(FEditorViewportClient* InViewportClient, FViewport* InViewport, 
	const FVector& InAccumulatedTranslation, const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale)
{
	if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
	{
		return false;
	}

	FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (DynMesh)
	{
		if (bEditingNumPoints)
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Scale)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
				{
					int32 NumPoints = InitialNumPoints;
					NumPoints = FMath::Clamp(NumPoints + static_cast<int32>(InAccumulatedScale.Y), 3, 128);
					DynMesh->Modify();
					DynMesh->SetNumPoints(NumPoints);

					bHasBeenModified = true;
					NotifyPropertyModified(DynMesh, NumPointsProperty, EPropertyChangeType::Interactive);
				}
			}

			return true;
		}

		if (bEditingInnerSize)
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::YZ)
				{
					float InnerSize = InitialInnerSize;
					InnerSize = FMath::Clamp(InnerSize - InAccumulatedTranslation.Y / DynMesh->GetSize2D().X * 2.f, 0.01, 0.99);
					DynMesh->Modify();
					DynMesh->SetInnerSize(InnerSize);

					bHasBeenModified = true;
					NotifyPropertyModified(DynMesh, InnerSizeProperty, EPropertyChangeType::Interactive);
				}
			}

			return true;
		}
	}
	else
	{
		EndEditing();
	}

	return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
}

void FAvaShapeStarDynamicMeshVisualizer::StoreInitialValues()
{
	Super::StoreInitialValues();

	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh)
	{
		return;
	}

	InitialNumPoints = DynMesh->GetNumPoints();
	InitialInnerSize = DynMesh->GetInnerSize();
}

bool FAvaShapeStarDynamicMeshVisualizer::ResetValue(FEditorViewportClient* InViewportClient,
	HHitProxy* InHitProxy)
{
	if (!InHitProxy->IsA(HAvaShapeNumPointsHitProxy::StaticGetType()) &&
		!InHitProxy->IsA(HAvaShapeInnerSizeHitProxy::StaticGetType()))
	{
		return Super::ResetValue(InViewportClient, InHitProxy);
	}

	if (InHitProxy->IsA(HAvaShapeNumPointsHitProxy::StaticGetType()))
	{
		const HAvaShapeNumPointsHitProxy* NumPointsHitProxy = static_cast<HAvaShapeNumPointsHitProxy*>(InHitProxy);

		if (NumPointsHitProxy->Component.IsValid())
		{
			FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(
				const_cast<UActorComponent*>(NumPointsHitProxy->Component.Get()));

			if (HitProxyDynamicMesh)
			{
				FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
				HitProxyDynamicMesh->SetFlags(RF_Transactional);
				HitProxyDynamicMesh->Modify();
				HitProxyDynamicMesh->SetNumPoints(5);
				NotifyPropertyModified(HitProxyDynamicMesh, NumPointsProperty, EPropertyChangeType::ValueSet);
			}
		}
	}
	else if (InHitProxy->IsA(HAvaShapeInnerSizeHitProxy::StaticGetType()))
	{
		const HAvaShapeInnerSizeHitProxy* InnerSizeHitProxy = static_cast<HAvaShapeInnerSizeHitProxy*>(InHitProxy);

		if (InnerSizeHitProxy->Component.IsValid())
		{
			FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(
				const_cast<UActorComponent*>(InnerSizeHitProxy->Component.Get()));

			if (HitProxyDynamicMesh)
			{
				FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
				HitProxyDynamicMesh->SetFlags(RF_Transactional);
				HitProxyDynamicMesh->Modify();
				HitProxyDynamicMesh->SetInnerSize(0.5);
				NotifyPropertyModified(HitProxyDynamicMesh, InnerSizeProperty, EPropertyChangeType::ValueSet);
			}
		}
	}

	return true;
}

bool FAvaShapeStarDynamicMeshVisualizer::IsEditing() const
{
	if (bEditingNumPoints || bEditingInnerSize)
	{
		return true;
	}

	return Super::IsEditing();
}

void FAvaShapeStarDynamicMeshVisualizer::EndEditing()
{
	Super::EndEditing();

	bEditingNumPoints = false;
	bEditingInnerSize = false;
}
