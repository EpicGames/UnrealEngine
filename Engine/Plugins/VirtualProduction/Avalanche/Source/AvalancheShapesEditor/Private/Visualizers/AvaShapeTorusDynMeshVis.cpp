// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaShapeTorusDynMeshVis.h"
#include "AvaField.h"
#include "AvaShapeActor.h"
#include "AvaShapeSprites.h"
#include "DynamicMeshes/AvaShapeTorusDynMesh.h"
#include "EditorViewportClient.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "RenderResource.h"
#include "ScopedTransaction.h"
#include "TextureResource.h"

FAvaShapeTorusDynamicMeshVisualizer::FAvaShapeTorusDynamicMeshVisualizer()
	: FAvaShape3DDynamicMeshVisualizer()
	, bEditingNumSlices(false)
	, InitialNumSlices(0)
	, bEditingNumSides(false)
	, InitialNumSides(0)
	, bEditingInnerSize(false)
	, InitialInnerSize(0)
	, bEditingAngleDegree(false)
	, InitialAngleDegree(0)
{
	using namespace UE::AvaCore;

	NumSidesProperty    = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, NumSides));
	NumSlicesProperty   = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, NumSlices));
	InnerSizeProperty   = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, InnerSize));
	AngleDegreeProperty = GetProperty<FMeshType>(GET_MEMBER_NAME_CHECKED(FMeshType, AngleDegree));
}

TMap<UObject*, TArray<FProperty*>> FAvaShapeTorusDynamicMeshVisualizer::GatherEditableProperties(UObject* InObject) const
{
	TMap<UObject*, TArray<FProperty*>> Properties = Super::GatherEditableProperties(InObject);

	if (FMeshType* DynMesh = Cast<FMeshType>(InObject))
	{
		Properties.FindOrAdd(DynMesh).Add(NumSidesProperty);
		Properties.FindOrAdd(DynMesh).Add(NumSlicesProperty);
		Properties.FindOrAdd(DynMesh).Add(InnerSizeProperty);
		Properties.FindOrAdd(DynMesh).Add(AngleDegreeProperty);
	}

	return Properties;
}

bool FAvaShapeTorusDynamicMeshVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient,
	HComponentVisProxy* InVisProxy, const FViewportClick& InClick)
{
	if (InClick.GetKey() != EKeys::LeftMouseButton)
	{
		EndEditing();
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	FMeshType* DynMesh = Cast<FMeshType>(const_cast<UActorComponent*>(InVisProxy->Component.Get()));

	if (DynMesh == nullptr)
	{
		return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
	}

	// num slices
	if (InVisProxy->IsA(HAvaShapeNumSidesHitProxy::StaticGetType()))
	{
		EndEditing();
		bEditingNumSlices = true;
		StartEditing(InViewportClient, DynMesh);
		return true;
	}
	// num sides
	if (InVisProxy->IsA(HAvaShapeNumPointsHitProxy::StaticGetType()))
	{
		EndEditing();
		bEditingNumSides = true;
		StartEditing(InViewportClient, DynMesh);
		return true;
	}
	// inner size
	if (InVisProxy->IsA(HAvaShapeInnerSizeHitProxy::StaticGetType()))
	{
		EndEditing();
		bEditingInnerSize = true;
		StartEditing(InViewportClient, DynMesh);
		return true;
	}
	// angle degree
	if (InVisProxy->IsA(HAvaShapeAngleDegreeHitProxy::StaticGetType()))
	{
		EndEditing();
		bEditingAngleDegree = true;
		StartEditing(InViewportClient, DynMesh);
		return true;
	}

	return Super::VisProxyHandleClick(InViewportClient, InVisProxy, InClick);
}

bool FAvaShapeTorusDynamicMeshVisualizer::GetWidgetLocation(const FEditorViewportClient* InViewportClient,
	FVector& OutLocation) const
{
	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (DynMesh)
	{
		if (bEditingNumSides || bEditingAngleDegree || bEditingNumSlices)
		{
			OutLocation = DynMesh->GetShapeActor()->GetActorLocation();
			return true;
		}

		if (bEditingInnerSize)
		{
			const float OffsetZ  = DynMesh->GetSize3D().Z * FMath::GetMappedRangeValueClamped(FVector2D(0.5, 1), FVector2D(0, 1), DynMesh->GetInnerSize()) / 2.f;
			const FVector Offset = FVector(0.f, 0.f, OffsetZ);
			OutLocation          = DynamicMeshComponent->GetShapeActor()->GetTransform().TransformPosition(Offset);
			return true;
		}
	}

	return Super::GetWidgetLocation(InViewportClient, OutLocation);
}

bool FAvaShapeTorusDynamicMeshVisualizer::GetWidgetMode(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode& OutMode) const
{
	if (bEditingNumSlices || bEditingNumSides)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Scale;
		return true;
	}

	if (bEditingInnerSize)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Translate;
		return true;
	}

	if (bEditingAngleDegree)
	{
		OutMode = UE::Widget::EWidgetMode::WM_Rotate;
		return true;
	}

	return Super::GetWidgetMode(InViewportClient, OutMode);
}

bool FAvaShapeTorusDynamicMeshVisualizer::GetWidgetAxisList(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingNumSlices)
	{
		OutAxisList = EAxisList::Type::Z;
		return true;
	}

	if (bEditingNumSides)
	{
		OutAxisList = EAxisList::Type::X;
		return true;
	}

	if (bEditingInnerSize)
	{
		OutAxisList = EAxisList::Type::Screen;
		return true;
	}
	
	if (bEditingAngleDegree)
	{
		OutAxisList = EAxisList::Type::X;
		return true;
	}

	return Super::GetWidgetAxisList(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShapeTorusDynamicMeshVisualizer::GetWidgetAxisListDragOverride(const FEditorViewportClient* InViewportClient,
	UE::Widget::EWidgetMode InWidgetMode, EAxisList::Type& OutAxisList) const
{
	if (bEditingInnerSize)
	{
		OutAxisList = EAxisList::Type::Z;
		return true;
	}

	return Super::GetWidgetAxisListDragOverride(InViewportClient, InWidgetMode, OutAxisList);
}

bool FAvaShapeTorusDynamicMeshVisualizer::ResetValue(FEditorViewportClient* InViewportClient, HHitProxy* InHitProxy)
{
	if (!InHitProxy->IsA(HAvaShapeNumSidesHitProxy::StaticGetType()) &&
		!InHitProxy->IsA(HAvaShapeInnerSizeHitProxy::StaticGetType()) &&
		!InHitProxy->IsA(HAvaShapeAngleDegreeHitProxy::StaticGetType()) &&
		!InHitProxy->IsA(HAvaShapeNumPointsHitProxy::StaticGetType()))
	{
		return Super::ResetValue(InViewportClient, InHitProxy);
	}

	// num slices
	if (InHitProxy->IsA(HAvaShapeNumSidesHitProxy::StaticGetType()))
	{
		const HAvaShapeNumSidesHitProxy* NumSlicesHitProxy = static_cast<HAvaShapeNumSidesHitProxy*>(InHitProxy);

		if (NumSlicesHitProxy->Component.IsValid())
		{
			FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(
				const_cast<UActorComponent*>(NumSlicesHitProxy->Component.Get()));

			if (HitProxyDynamicMesh)
			{
				FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
				HitProxyDynamicMesh->SetFlags(RF_Transactional);
				HitProxyDynamicMesh->Modify();
				HitProxyDynamicMesh->SetNumSlices(32);
				NotifyPropertyModified(HitProxyDynamicMesh, NumSlicesProperty, EPropertyChangeType::ValueSet);
			}
		}
	}
	// num sides
	else if (InHitProxy->IsA(HAvaShapeNumPointsHitProxy::StaticGetType()))
	{
		const HAvaShapeNumSidesHitProxy* NumSidesHitProxy = static_cast<HAvaShapeNumSidesHitProxy*>(InHitProxy);

		if (NumSidesHitProxy->Component.IsValid())
		{
			FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(
				const_cast<UActorComponent*>(NumSidesHitProxy->Component.Get()));

			if (HitProxyDynamicMesh)
			{
				FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
				HitProxyDynamicMesh->SetFlags(RF_Transactional);
				HitProxyDynamicMesh->Modify();
				HitProxyDynamicMesh->SetNumSides(32);
				NotifyPropertyModified(HitProxyDynamicMesh, NumSidesProperty, EPropertyChangeType::ValueSet);
			}
		}
	}
	// inner size
	else if (InHitProxy->IsA(HAvaShapeInnerSizeHitProxy::StaticGetType()))
	{
		const HAvaShapeInnerSizeHitProxy* InnerSizeHitProxy = static_cast<HAvaShapeInnerSizeHitProxy*>(InHitProxy);

		if (InnerSizeHitProxy->Component.IsValid())
		{
			FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(
				const_cast<UActorComponent*>(InnerSizeHitProxy->Component.Get()));

			if (HitProxyDynamicMesh)
			{
				FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
				HitProxyDynamicMesh->SetFlags(RF_Transactional);
				HitProxyDynamicMesh->Modify();
				HitProxyDynamicMesh->SetInnerSize(0.75f);
				NotifyPropertyModified(HitProxyDynamicMesh, InnerSizeProperty, EPropertyChangeType::ValueSet);
			}
		}
	}
	// angle degree
	else if (InHitProxy->IsA(HAvaShapeAngleDegreeHitProxy::StaticGetType()))
	{
		const HAvaShapeAngleDegreeHitProxy* AngleDegreeHitProxy = static_cast<HAvaShapeAngleDegreeHitProxy*>(InHitProxy);

		if (AngleDegreeHitProxy->Component.IsValid())
		{
			FMeshType* HitProxyDynamicMesh = Cast<FMeshType>(
				const_cast<UActorComponent*>(AngleDegreeHitProxy->Component.Get()));

			if (HitProxyDynamicMesh)
			{
				FScopedTransaction Transaction(NSLOCTEXT("AvaShapeVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
				HitProxyDynamicMesh->SetFlags(RF_Transactional);
				HitProxyDynamicMesh->Modify();
				HitProxyDynamicMesh->SetAngleDegree(360.f);
				NotifyPropertyModified(HitProxyDynamicMesh, AngleDegreeProperty, EPropertyChangeType::ValueSet);
			}
		}
	}

	return true;
}

bool FAvaShapeTorusDynamicMeshVisualizer::IsEditing() const
{
	if (bEditingAngleDegree || bEditingNumSides || bEditingInnerSize || bEditingNumSlices)
	{
		return true;
	}

	return Super::IsEditing();
}

void FAvaShapeTorusDynamicMeshVisualizer::EndEditing()
{
	Super::EndEditing();

	bEditingAngleDegree = false;
	bEditingNumSides = false;
	bEditingNumSlices = false;
	bEditingInnerSize = false;
}

void FAvaShapeTorusDynamicMeshVisualizer::DrawVisualizationNotEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationNotEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawNumSlicesButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive);
	++InOutIconIndex;

	DrawNumSidesButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive);
	++InOutIconIndex;
	
	DrawInnerSizeButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive);
	++InOutIconIndex;

	DrawAngleDegreeButton(DynMesh, InView, InPDI, InOutIconIndex, Inactive);
	++InOutIconIndex;
}

void FAvaShapeTorusDynamicMeshVisualizer::DrawVisualizationEditing(const UActorComponent* InComponent,
	const FSceneView* InView, FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationNotEditing(InComponent, InView, InPDI, InOutIconIndex);

	const FMeshType* DynMesh = Cast<FMeshType>(InComponent);

	if (!DynMesh)
	{
		return;
	}

	DrawNumSlicesButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingNumSlices ? Active : Inactive);
	++InOutIconIndex;

	DrawNumSidesButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingNumSides ? Active : Inactive);
	++InOutIconIndex;
	
	DrawInnerSizeButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingInnerSize ? Active : Inactive);
	++InOutIconIndex;

	DrawAngleDegreeButton(DynMesh, InView, InPDI, InOutIconIndex, bEditingAngleDegree ? Active : Inactive);
	++InOutIconIndex;
}

bool FAvaShapeTorusDynamicMeshVisualizer::HandleInputDeltaInternal(FEditorViewportClient* InViewportClient,
	FViewport* InViewport, const FVector& InAccumulatedTranslation, const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale)
{
	if (!FSlateApplication::Get().GetPressedMouseButtons().Contains(EKeys::LeftMouseButton))
	{
		return false;
	}

	FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (DynMesh)
	{
		if (bEditingNumSlices)
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Scale)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
				{
					int32 NumSlices = InitialNumSlices;
					NumSlices = FMath::Clamp(NumSlices + static_cast<int32>(InAccumulatedScale.Z), 3, 255);
					DynMesh->Modify();
					DynMesh->SetNumSlices(NumSlices);

					bHasBeenModified = true;
					NotifyPropertyModified(DynMesh, NumSlicesProperty, EPropertyChangeType::Interactive);
				}
			}
			return true;
		}
		if (bEditingNumSides)
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Scale)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::X)
				{
					int32 NumSides = InitialNumSides;
					NumSides = FMath::Clamp(NumSides + static_cast<int32>(InAccumulatedScale.X), 3, 255);
					DynMesh->Modify();
					DynMesh->SetNumSides(NumSides);

					bHasBeenModified = true;
					NotifyPropertyModified(DynMesh, NumSidesProperty, EPropertyChangeType::Interactive);
				}
			}
			return true;
		}
		if (bEditingInnerSize)
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
				{
					float InnerSize = InitialInnerSize;
					InnerSize = FMath::Clamp(InnerSize + InAccumulatedTranslation.Z / DynMesh->GetSize3D().Z, 0.5, 1.f);
					DynMesh->Modify();
					DynMesh->SetInnerSize(InnerSize);

					bHasBeenModified = true;
					NotifyPropertyModified(DynMesh, InnerSizeProperty, EPropertyChangeType::Interactive);
				}
			}
			return true;
		}
		if (bEditingAngleDegree)
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Rotate)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::X)
				{
					float AngleDegree = InitialAngleDegree;
					AngleDegree = FMath::Clamp(AngleDegree + static_cast<int32>(InAccumulatedRotation.Roll), 0.f, 360.f);
					DynMesh->Modify();
					DynMesh->SetAngleDegree(AngleDegree);

					bHasBeenModified = true;
					NotifyPropertyModified(DynMesh, AngleDegreeProperty, EPropertyChangeType::Interactive);
				}
			}
		}
	}
	else
	{
		EndEditing();
	}

	return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
}

void FAvaShapeTorusDynamicMeshVisualizer::StoreInitialValues()
{
	Super::StoreInitialValues();

	const FMeshType* DynMesh = Cast<FMeshType>(DynamicMeshComponent.Get());

	if (!DynMesh)
	{
		return;
	}

	InitialNumSlices = DynMesh->GetNumSlices();
	InitialNumSides = DynMesh->GetNumSides();
	InitialInnerSize = DynMesh->GetInnerSize();
	InitialAngleDegree = DynMesh->GetAngleDegree();
}

void FAvaShapeTorusDynamicMeshVisualizer::DrawNumSlicesButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	UTexture2D* NumSidesSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::NumSidesSprite);

	if (!NumSidesSprite || !NumSidesSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeNumSidesHitProxy(InDynMesh));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, NumSidesSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaShapeTorusDynamicMeshVisualizer::DrawNumSidesButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	UTexture2D* NumPointsSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::NumPointsSprite);

	if (!NumPointsSprite || !NumPointsSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeNumPointsHitProxy(InDynMesh));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, NumPointsSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaShapeTorusDynamicMeshVisualizer::DrawInnerSizeButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	UTexture2D* CornerSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::CornerSprite);

	if (!CornerSprite || !CornerSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeInnerSizeHitProxy(InDynMesh));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, CornerSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaShapeTorusDynamicMeshVisualizer::DrawAngleDegreeButton(const FMeshType* InDynMesh, const FSceneView* InView,
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	UTexture2D* InnerSizeSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::InnerSizeSprite);

	if (!InnerSizeSprite || !InnerSizeSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaShapeAngleDegreeHitProxy(InDynMesh, MakeAlignment(EAvaDepthAlignment::Center, EAvaHorizontalAlignment::Center, EAvaVerticalAlignment::Center)));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, InnerSizeSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}
