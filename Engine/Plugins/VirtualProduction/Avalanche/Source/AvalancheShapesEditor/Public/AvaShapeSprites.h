// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::AvaShapes
{
	constexpr const TCHAR* BevelSprite = TEXT("Bevel");
	constexpr const TCHAR* BreakSideSprite = TEXT("BreakSide");
	constexpr const TCHAR* ColorSelectionSprite = TEXT("ColorSelection");
	constexpr const TCHAR* CornerSprite = TEXT("Corner");
	constexpr const TCHAR* DepthSprite = TEXT("Depth");
	constexpr const TCHAR* InnerSizeSprite = TEXT("InnerSize");
	constexpr const TCHAR* LinearGradientSprite = TEXT("LinearGradient");
	constexpr const TCHAR* NumPointsSprite = TEXT("NumPoints");
	constexpr const TCHAR* NumSidesSprite = TEXT("NumSides");
	constexpr const TCHAR* TextMaxHeightSprite = TEXT("TextMaxHeight");
	constexpr const TCHAR* TextMaxWidthSprite = TEXT("TextMaxWidth");
	constexpr const TCHAR* TextScaleProportionallySprite = TEXT("TextScaleProportionally");
	constexpr const TCHAR* SizeSprite = TEXT("Size");
	constexpr const TCHAR* SlantSprite = TEXT("Slant");
	constexpr const TCHAR* UVSprite = TEXT("UV");
}
