// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagAlias.h"
#include "AvaTag.h"
#include "AvaTagCollection.h"
#include "AvaTagList.h"

#if WITH_EDITOR
void FAvaTagAlias::SetOwner(UAvaTagCollection* InOwner)
{
	OwnerWeak = InOwner;
}

UAvaTagCollection* FAvaTagAlias::GetOwner() const
{
	return OwnerWeak.Get();
}

FString FAvaTagAlias::GetTagsAsString() const
{
	UAvaTagCollection* Owner = GetOwner();
	if (!Owner)
	{
		return FString();
	}

	FString OutString;
	for (const FAvaTagId& TagId : TagIds)
	{
		for (const FAvaTag* Tag : Owner->GetTags(TagId))
		{
			if (!OutString.IsEmpty())
			{
				OutString.Append(TEXT(", "));
			}
			OutString.Append(Tag->ToString());
		}
	}
	return OutString;
}
#endif
