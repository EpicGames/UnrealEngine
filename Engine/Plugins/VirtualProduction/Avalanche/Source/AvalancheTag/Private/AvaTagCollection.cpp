// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagCollection.h"
#include "AvaTagList.h"

FAvaTagList UAvaTagCollection::GetTags(const FAvaTagId& InTagId) const
{
	FAvaTagList TagList;

	if (const FAvaTag* Tag = Tags.Find(InTagId))
	{
		TagList.Tags.Add(Tag);
	}

	if (const FAvaTagAlias* TagAlias = Aliases.Find(InTagId))
	{
		TagList.Tags.Reserve(TagList.Tags.Num() + TagAlias->TagIds.Num());
		for (const FAvaTagId& TagId : TagAlias->TagIds)
		{
			if (const FAvaTag* Tag = Tags.Find(TagId))
			{
				TagList.Tags.Add(Tag);
			}
		}
	}

	return TagList;
}

FName UAvaTagCollection::GetTagName(const FAvaTagId& InTagId) const
{
	if (const FAvaTag* Tag = Tags.Find(InTagId))
	{
		return Tag->TagName;
	}

	if (const FAvaTagAlias* TagAlias = Aliases.Find(InTagId))
	{
		return TagAlias->AliasName;
	}

	return NAME_None;
}

TArray<FAvaTagId> UAvaTagCollection::GetTagIds(bool bInIncludeAliases) const
{
	int32 TagIdCount = Tags.Num();
	if (bInIncludeAliases)
	{
		TagIdCount += Aliases.Num();
	}

	TArray<FAvaTagId> TagIds;
	TagIds.Reserve(TagIdCount);

	for (const TPair<FAvaTagId, FAvaTag>& Pair : Tags)
	{
		TagIds.Add(Pair.Key);
	}

	if (bInIncludeAliases)
	{
		for (const TPair<FAvaTagId, FAvaTagAlias>& Pair : Aliases)
		{
			TagIds.Add(Pair.Key);
		}
	}

	return TagIds;
}

void UAvaTagCollection::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITOR
	UpdateAliasOwner();
#endif
}

#if WITH_EDITOR
void UAvaTagCollection::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UAvaTagCollection, Aliases))
	{
		UpdateAliasOwner();
	}
}
#endif

FName UAvaTagCollection::GetTagMapName()
{
	return GET_MEMBER_NAME_CHECKED(UAvaTagCollection, Tags);
}

FName UAvaTagCollection::GetAliasMapName()
{
	return GET_MEMBER_NAME_CHECKED(UAvaTagCollection, Aliases);
}

#if WITH_EDITOR
void UAvaTagCollection::UpdateAliasOwner()
{
	for (TPair<FAvaTagId, FAvaTagAlias>& Alias : Aliases)
	{
		Alias.Value.SetOwner(this);
	}
}
#endif