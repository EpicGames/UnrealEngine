// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagHandle.h"
#include "AvaTagCollection.h"
#include "AvaTagList.h"

FAvaTagList FAvaTagHandle::GetTags() const
{
	if (::IsValid(Source))
	{
		return Source->GetTags(TagId);
	}
	return FAvaTagList();
}

FString FAvaTagHandle::ToString() const
{
	return ToName().ToString();
}

FString FAvaTagHandle::ToDebugString() const
{
	return FString::Printf(TEXT("TagId: %s, Source: %s"), *TagId.ToString(), *GetNameSafe(Source));
}

FName FAvaTagHandle::ToName() const
{
	if (::IsValid(Source))
	{
		return Source->GetTagName(TagId);
	}
	return NAME_None;
}

void FAvaTagHandle::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsSaving() && TagId.IsValid())
	{
		Ar.MarkSearchableName(FAvaTagId::StaticStruct(), *TagId.ToString());
	}
}

bool FAvaTagHandle::Overlaps(const FAvaTagHandle& InOther) const
{
	if (MatchesExact(InOther))
	{
		return true;
	}

	// No Overlap if this has no Tags
	FAvaTagList ThisTagList = GetTags();
	if (ThisTagList.Tags.IsEmpty())
	{
		return false;
	}

	// No Overlap if other is empty
	FAvaTagList OtherTagList = InOther.GetTags();
	if (OtherTagList.Tags.IsEmpty())
	{
		return false;
	}

	for (const FAvaTag* ThisTag : ThisTagList)
	{
		for (const FAvaTag* OtherTag : OtherTagList)
		{
			if (*ThisTag == *OtherTag)
			{
				return true;
			}
		}
	}

	return false;
}

bool FAvaTagHandle::MatchesExact(const FAvaTagHandle& InOther) const
{
	return Source == InOther.Source && TagId == InOther.TagId;
}
