// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagHandleContainer.h"
#include "AvaTagCollection.h"
#include "AvaTagHandle.h"
#include "AvaTagList.h"

FAvaTagHandleContainer::FAvaTagHandleContainer(const FAvaTagHandle& InTagHandle)
	: Source(InTagHandle.Source)
	, TagIds({ InTagHandle.TagId })
{
}

bool FAvaTagHandleContainer::ContainsTag(const FAvaTagHandle& InTagHandle) const
{
	if (!Source)
	{
		return false;
	}

	if (ContainsTagHandle(InTagHandle))
	{
		return true;
	}

	// Populate the Tag Set with resolved Tags
	TSet<FAvaTag> OtherTagSet;
	{
		FAvaTagList OtherTagList = InTagHandle.GetTags();
		if (OtherTagList.Tags.IsEmpty())
		{
			return false;
		}

		OtherTagSet.Reserve(OtherTagList.Tags.Num());
		for (const FAvaTag* OtherTag : OtherTagList)
		{
			OtherTagSet.Add(*OtherTag);
		}
	}

	for (const FAvaTagId& TagId : TagIds)
	{
		for (const FAvaTag* Tag : Source->GetTags(TagId))
		{
			if (OtherTagSet.Contains(*Tag))
			{
				return true;
			}
		}
	}

	return false;
}

bool FAvaTagHandleContainer::ContainsTagHandle(const FAvaTagHandle& InTagHandle) const
{
	return Source == InTagHandle.Source && TagIds.Contains(InTagHandle.TagId);
}

FString FAvaTagHandleContainer::ToString() const
{
	if (!Source)
	{
		return FString();
	}

	FString OutString;
	for (const FAvaTagId& TagId : TagIds)
	{
		for (const FAvaTag* Tag : Source->GetTags(TagId))
		{
			if (!OutString.IsEmpty())
			{
				OutString.Append(TEXT(", "));
			}
			OutString.Append(Tag->ToString());
		}
	}
	return OutString;
}

void FAvaTagHandleContainer::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsSaving())
	{
		for (const FAvaTagId& TagId : TagIds)
		{
			if (TagId.IsValid())
			{
				Ar.MarkSearchableName(FAvaTagId::StaticStruct(), *TagId.ToString());
			}
		}
	}
}

bool FAvaTagHandleContainer::SerializeFromMismatchedTag(const FPropertyTag& InPropertyTag, FStructuredArchive::FSlot InSlot)
{
	static const FName TagHandleContextName = FAvaTagHandle::StaticStruct()->GetFName();

	if (InPropertyTag.GetType().IsStruct(TagHandleContextName))
	{
		FAvaTagHandle TagHandle;
		FAvaTagHandle::StaticStruct()->SerializeItem(InSlot, &TagHandle, nullptr);

		if (TagHandle.IsValid())
		{
			Source = TagHandle.Source;
			TagIds = { TagHandle.TagId };
		}
		return true;
	}

	return false;
}

bool FAvaTagHandleContainer::AddTagHandle(const FAvaTagHandle& InTagHandle)
{
	// Set Source to latest added tag handle
	if (!Source)
	{
		Source = InTagHandle.Source;
	}

	if (TagIds.Contains(InTagHandle.TagId))
	{
		return false;
	}

	TagIds.Add(InTagHandle.TagId);
	return true;
}

bool FAvaTagHandleContainer::RemoveTagHandle(const FAvaTagHandle& InTagHandle)
{
	return TagIds.Remove(InTagHandle.TagId) > 0;
}

TArray<FAvaTag> FAvaTagHandleContainer::ResolveTags() const
{
	TArray<FAvaTag> Tags;
	if (!Source)
	{
		return Tags;
	}

	Tags.Reserve(TagIds.Num());

	for (const FAvaTagId& TagId : TagIds)
	{
		for (const FAvaTag* Tag : Source->GetTags(TagId))
		{
			Tags.Add(*Tag);
		}
	}

	return Tags;
}
