// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagLibrary.h"
#include "AvaTag.h"
#include "AvaTagHandle.h"
#include "AvaTagHandleContainer.h"
#include "AvaTagList.h"
#include "AvaTagSoftHandle.h"

TArray<FAvaTag> UAvaTagLibrary::ResolveTagHandle(const FAvaTagHandle& InTagHandle)
{
	FAvaTagList TagList = InTagHandle.GetTags();

	TArray<FAvaTag> Tags;
	Tags.Reserve(TagList.Tags.Num());

	for (const FAvaTag* Tag : TagList)
	{
		Tags.Add(*Tag);
	}

	return Tags;
}

TArray<FAvaTag> UAvaTagLibrary::ResolveTagHandles(const FAvaTagHandleContainer& InTagHandleContainer)
{
	return InTagHandleContainer.ResolveTags();
}

FAvaTagHandle UAvaTagLibrary::ResolveTagSoftHandle(const FAvaTagSoftHandle& InTagSoftHandle)
{
	return InTagSoftHandle.MakeTagHandle();
}
