// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagSoftHandle.h"
#include "AvaTagCollection.h"
#include "AvaTagHandle.h"

FAvaTagSoftHandle::FAvaTagSoftHandle(const FAvaTagHandle& InTagHandle)
	: FAvaTagSoftHandle(const_cast<UAvaTagCollection*>(InTagHandle.Source.Get()), InTagHandle.TagId)
{
}

FAvaTagHandle FAvaTagSoftHandle::MakeTagHandle() const
{
	return FAvaTagHandle(Source.LoadSynchronous(), TagId);
}

bool FAvaTagSoftHandle::MatchesExact(const FAvaTagHandle& InTagHandle) const
{
	return Source == InTagHandle.Source && TagId == InTagHandle.TagId;
}

void FAvaTagSoftHandle::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsSaving() && TagId.IsValid())
	{
		Ar.MarkSearchableName(FAvaTagId::StaticStruct(), *TagId.ToString());
	}
}
