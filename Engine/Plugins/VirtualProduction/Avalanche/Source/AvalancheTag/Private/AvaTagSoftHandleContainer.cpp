// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagSoftHandleContainer.h"
#include "AvaTagCollection.h"
#include "AvaTagHandle.h"
#include "AvaTagHandleContainer.h"
#include "AvaTagList.h"
#include "AvaTagSoftHandle.h"

FAvaTagSoftHandleContainer::FAvaTagSoftHandleContainer(const FAvaTagHandle& InTagHandle)
	: Source(InTagHandle.Source)
	, TagIds({ InTagHandle.TagId })
{
}

bool FAvaTagSoftHandleContainer::ContainsTag(const FAvaTagHandle& InTagHandle) const
{
	if (ContainsTagHandle(InTagHandle))
	{
		return true;
	}

	const UAvaTagCollection* const ResolvedSource = ResolveSource();
	if (!ResolvedSource)
	{
		return false;
	}

	// Populate the Tag Set with resolved Tags
	TSet<FAvaTag> OtherTagSet;
	{
		FAvaTagList OtherTagList = InTagHandle.GetTags();
		if (OtherTagList.Tags.IsEmpty())
		{
			return false;
		}

		OtherTagSet.Reserve(OtherTagList.Tags.Num());
		for (const FAvaTag* OtherTag : OtherTagList)
		{
			OtherTagSet.Add(*OtherTag);
		}
	}

	for (const FAvaTagId& TagId : TagIds)
	{
		for (const FAvaTag* Tag : ResolvedSource->GetTags(TagId))
		{
			if (OtherTagSet.Contains(*Tag))
			{
				return true;
			}
		}
	}

	return false;
}

bool FAvaTagSoftHandleContainer::ContainsTagHandle(const FAvaTagHandle& InTagHandle) const
{
	return Source == InTagHandle.Source && TagIds.Contains(InTagHandle.TagId);
}

FString FAvaTagSoftHandleContainer::ToString() const
{
	const UAvaTagCollection* const ResolvedSource = ResolveSource();
	if (!ResolvedSource)
	{
		return FString();
	}

	TStringBuilder<32> StringBuilder;
	for (const FAvaTagId& TagId : TagIds)
	{
		for (const FAvaTag* Tag : ResolvedSource->GetTags(TagId))
		{
			if (StringBuilder.Len() > 0)
			{
				StringBuilder.Append(TEXT(", "));
			}
			StringBuilder.Append(Tag->ToString());
		}
	}
	return FString(StringBuilder);
}

void FAvaTagSoftHandleContainer::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsSaving())
	{
		for (const FAvaTagId& TagId : TagIds)
		{
			if (TagId.IsValid())
			{
				Ar.MarkSearchableName(FAvaTagId::StaticStruct(), *TagId.ToString());
			}
		}
	}
}

bool FAvaTagSoftHandleContainer::SerializeFromMismatchedTag(const FPropertyTag& InPropertyTag, FStructuredArchive::FSlot InSlot)
{
	if (InPropertyTag.GetType().IsStruct(FAvaTagSoftHandle::StaticStruct()->GetFName()))
	{
		FAvaTagSoftHandle TagSoftHandle;
		FAvaTagSoftHandle::StaticStruct()->SerializeItem(InSlot, &TagSoftHandle, nullptr);

		if (TagSoftHandle.IsValid())
		{
			Source = TagSoftHandle.Source;
			TagIds = { TagSoftHandle.TagId };
		}
		return true;
	}

	if (InPropertyTag.GetType().IsStruct(FAvaTagHandleContainer::StaticStruct()->GetFName()))
	{
		FAvaTagHandleContainer TagHandleContainer;
		FAvaTagHandleContainer::StaticStruct()->SerializeItem(InSlot, &TagHandleContainer, nullptr);

		if (TagHandleContainer.Source)
		{
			Source = TagHandleContainer.Source;
			TagIds = TagHandleContainer.GetTagIds();
		}
		return true;
	}

	return false;
}

bool FAvaTagSoftHandleContainer::AddTagHandle(const FAvaTagHandle& InTagHandle)
{
	if (!Source)
	{
		Source = InTagHandle.Source;
	}

	if (TagIds.Contains(InTagHandle.TagId))
	{
		return false;
	}

	TagIds.Add(InTagHandle.TagId);
	return true;
}

bool FAvaTagSoftHandleContainer::RemoveTagHandle(const FAvaTagHandle& InTagHandle)
{
	return TagIds.Remove(InTagHandle.TagId) > 0;
}

TArray<FAvaTag> FAvaTagSoftHandleContainer::ResolveTags() const
{
	TArray<FAvaTag> Tags;

	const UAvaTagCollection* const ResolvedSource = ResolveSource();
	if (!ResolvedSource)
	{
		return Tags;
	}

	Tags.Reserve(TagIds.Num());

	for (const FAvaTagId& TagId : TagIds)
	{
		for (const FAvaTag* Tag : ResolvedSource->GetTags(TagId))
		{
			Tags.Add(*Tag);
		}
	}

	return Tags;
}

const UAvaTagCollection* FAvaTagSoftHandleContainer::ResolveSource() const
{
	return Source.LoadSynchronous();
}
