// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "AvaTag.generated.h"

/**
 * Struct for a Tag
 * NOTE: Do not use as storage type. Prefer using FAvaTagHandle to then retrieve the FAvaTag it references
 */
USTRUCT(BlueprintType, DisplayName="Motion Design Tag")
struct FAvaTag
{
	GENERATED_BODY()

	bool IsValid() const
	{
		return TagName != NAME_None;
	}

	bool operator==(const FAvaTag& InTag) const
	{
		return TagName == InTag.TagName;
	}

	friend uint32 GetTypeHash(const FAvaTag& InTag)
	{
		return GetTypeHash(InTag.TagName);
	}

	FString ToString() const
	{
		return TagName.ToString();
	}

	UPROPERTY(EditAnywhere, Category="Tag")
	FName TagName;
};
