// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagId.h"
#include "AvaTagAlias.generated.h"

class UAvaTagCollection;

/**
 * An alias represents multiple other Tag Ids.
 *
 * Unlike Tag Containers which would need to be updated in every place it's used when the set of tags it needs to manipulate is added to or removed from,
 * Aliases are a layer of abstraction that allows the set of tags to be added to or removed from without affecting the places where the alias is used.
 */
USTRUCT(BlueprintType, DisplayName="Motion Design Tag Alias")
struct FAvaTagAlias
{
	GENERATED_BODY()

#if WITH_EDITOR
	void SetOwner(UAvaTagCollection* InOwner);

	AVALANCHETAG_API UAvaTagCollection* GetOwner() const;

	AVALANCHETAG_API FString GetTagsAsString() const;
#endif

	UPROPERTY(EditAnywhere, Category="Tag")
	FName AliasName;

	UPROPERTY(EditAnywhere, Category="Tag")
	TArray<FAvaTagId> TagIds;

private:
#if WITH_EDITOR
	/**
	 * Tag Collection used to resolve the Tag Ids
	 * Set by the Tag Collection that holds the alias map on load/change
	 */
	TWeakObjectPtr<UAvaTagCollection> OwnerWeak;
#endif
};
