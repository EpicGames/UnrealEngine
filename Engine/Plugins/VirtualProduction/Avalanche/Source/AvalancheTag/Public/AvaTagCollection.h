// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTag.h"
#include "AvaTagAlias.h"
#include "AvaTagId.h"
#include "Containers/Map.h"
#include "UObject/Object.h"
#include "AvaTagCollection.generated.h"

struct FAvaTagList;

/**
 * Tag Collection that identifies a tag with an underlying Tag Id Guid
 * and provides Tag reference capabilities
 */
UCLASS(MinimalAPI, DisplayName="Motion Design Tag Collection")
class UAvaTagCollection : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Returns an array of valid pointers to the tags that are under the given TagId
	 * Which could be an Alias
	 */
	AVALANCHETAG_API FAvaTagList GetTags(const FAvaTagId& InTagId) const;

	/**
	 * Returns the name of the Tag mapped to the given TagId.
	 * If the TagId is mapped to an Alias, it returns the Alias name instead.
	 */
	AVALANCHETAG_API FName GetTagName(const FAvaTagId& InTagId) const;

	/** Returns the keys of the Tag Map, optionally including aliases */
	AVALANCHETAG_API TArray<FAvaTagId> GetTagIds(bool bInIncludeAliases) const;

	//~ Begin UObject
	AVALANCHETAG_API virtual void PostLoad() override;
#if WITH_EDITOR
	AVALANCHETAG_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	AVALANCHETAG_API static FName GetTagMapName();
	AVALANCHETAG_API static FName GetAliasMapName();

private:
#if WITH_EDITOR
	/** Updates the weak ptr to the tag collection for each alias in the alias map */
	void UpdateAliasOwner();
#endif

	UPROPERTY(EditAnywhere, Category="Tag")
	TMap<FAvaTagId, FAvaTag> Tags;

	UPROPERTY(EditAnywhere, Category="Tag")
	TMap<FAvaTagId, FAvaTagAlias> Aliases;
};
