// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagId.h"
#include "Containers/ContainersFwd.h"
#include "AvaTagHandle.generated.h"

class UAvaTagCollection;
struct FAvaTag;
struct FAvaTagList;

/**
 * Handle to a Tag or Alias (set of Tags) in a particular Source.
 * This should be used by the UStructs/UObjects to properly reference a Tag or Alias (set of Tags).
 */
USTRUCT(BlueprintType, DisplayName="Motion Design Tag Handle")
struct FAvaTagHandle
{
	GENERATED_BODY()

	FAvaTagHandle() = default;

	FAvaTagHandle(const UAvaTagCollection* InSource, const FAvaTagId& InTagId)
		: Source(InSource)
		, TagId(InTagId)
	{
	}

	/**
	 * Returns the resolved tags from the Handle
	 * If the Handle is to a particular Tag, it should return the array with a single element.
	 * If the Handle is to an alias, it should return the array of tags the alias represents.
	 */
	AVALANCHETAG_API FAvaTagList GetTags() const;

	AVALANCHETAG_API FString ToString() const;

	AVALANCHETAG_API FString ToDebugString() const;

	AVALANCHETAG_API FName ToName() const;

	AVALANCHETAG_API void PostSerialize(const FArchive& Ar);

	/** Returns true if the Tag Handles have overlapping FAvaTags, even if the Source or Tag Id is different */
	AVALANCHETAG_API bool Overlaps(const FAvaTagHandle& InOther) const;

	/** Returns true if the Tag Handles is the exact same as the other (Same Source and Tag Id) */
	AVALANCHETAG_API bool MatchesExact(const FAvaTagHandle& InOther) const;

	bool IsValid() const
	{
		return Source && TagId.IsValid();
	}

	friend uint32 GetTypeHash(const FAvaTagHandle& InHandle)
	{
		return HashCombineFast(GetTypeHash(InHandle.Source), GetTypeHash(InHandle.TagId));
	}

	UPROPERTY(EditAnywhere, Category = "Tag")
	TObjectPtr<const UAvaTagCollection> Source;

	UPROPERTY()
	FAvaTagId TagId;
};

template<>
struct TStructOpsTypeTraits<FAvaTagHandle> : public TStructOpsTypeTraitsBase2<FAvaTagHandle>
{
	enum
	{
		WithPostSerialize = true,
	};
};
