// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagId.h"
#include "Containers/ContainersFwd.h"
#include "AvaTagHandleContainer.generated.h"

class UAvaTagCollection;
struct FAvaTag;
struct FAvaTagHandle;

/**
 * Handle to a multiple tags in a particular Source.
 * This should be used by the UStructs/UObjects to properly reference a multiple FAvaTags.
 */
USTRUCT(BlueprintType, DisplayName="Motion Design Tag Handle Container")
struct FAvaTagHandleContainer
{
	GENERATED_BODY()

	FAvaTagHandleContainer() = default;

	AVALANCHETAG_API explicit FAvaTagHandleContainer(const FAvaTagHandle& InTagHandle);

	/** Returns true if the Tag Handles resolve to same valued FAvaTags, even if the Source or Tag Id is different */
	AVALANCHETAG_API bool ContainsTag(const FAvaTagHandle& InTagHandle) const;

	/** Returns true if the Tag Handles is the exact same as the other (Same Source and Tag Id) */
	AVALANCHETAG_API bool ContainsTagHandle(const FAvaTagHandle& InTagHandle) const;

	AVALANCHETAG_API FString ToString() const;

	AVALANCHETAG_API void PostSerialize(const FArchive& Ar);

	AVALANCHETAG_API bool SerializeFromMismatchedTag(const FPropertyTag& InPropertyTag, FStructuredArchive::FSlot InSlot);

	/** Adds the provided Tag Handle to TagIds. Returns true only if it was added as a new entry to TagIds */
	AVALANCHETAG_API bool AddTagHandle(const FAvaTagHandle& InTagHandle);

	/** Removes the provided Tag Handle from TagIds. Returns true only if it existed and was removed from TagIds */
	AVALANCHETAG_API bool RemoveTagHandle(const FAvaTagHandle& InTagHandle);

	/** Returns an array of resolved tags through this container's tag ids and source tag collection */
	AVALANCHETAG_API TArray<FAvaTag> ResolveTags() const;

	TConstArrayView<FAvaTagId> GetTagIds() const
	{
		return TagIds;
	}

	UPROPERTY(EditAnywhere, Category = "Tag")
	TObjectPtr<const UAvaTagCollection> Source;

private:
	UPROPERTY(EditAnywhere, Category = "Tag")
	TArray<FAvaTagId> TagIds;
};

template<>
struct TStructOpsTypeTraits<FAvaTagHandleContainer> : public TStructOpsTypeTraitsBase2<FAvaTagHandleContainer>
{
	enum
	{
		WithPostSerialize = true,
		WithStructuredSerializeFromMismatchedTag = true,
	};
};
