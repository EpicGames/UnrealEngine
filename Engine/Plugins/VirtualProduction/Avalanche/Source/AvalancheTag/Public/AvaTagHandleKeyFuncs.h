// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagHandle.h"
#include "Containers/Set.h"

template<typename InValueType, bool bInAllowDuplicateKeys = false>
struct TAvaTagHandleMapKeyFuncs : BaseKeyFuncs<TPair<FAvaTagHandle, InValueType>, FAvaTagHandle, bInAllowDuplicateKeys>
{
	using KeyInitType     = TTypeTraits<FAvaTagHandle>::ConstPointerType;
	using ElementInitType = const TPairInitializer<TTypeTraits<FAvaTagHandle>::ConstInitType, typename TTypeTraits<InValueType>::ConstInitType>&;

	static KeyInitType GetSetKey(ElementInitType InElement)
	{
		return InElement.Key;
	}

	static bool Matches(KeyInitType A, KeyInitType B)
	{
		return A.MatchesExact(B);
	}

	static uint32 GetKeyHash(KeyInitType InKey)
	{
		return GetTypeHash(InKey);
	}
};

template<bool bInAllowDuplicateKeys = false>
struct TAvaTagHandleSetKeyFuncs : BaseKeyFuncs<FAvaTagHandle, FAvaTagHandle, bInAllowDuplicateKeys>
{
	using KeyInitType     = TTypeTraits<FAvaTagHandle>::ConstPointerType;
	using ElementInitType = TCallTraits<FAvaTagHandle>::ParamType;

	static KeyInitType GetSetKey(ElementInitType InElement)
	{
		return InElement;
	}

	static bool Matches(KeyInitType A, KeyInitType B)
	{
		return A.MatchesExact(B);
	}

	static uint32 GetKeyHash(KeyInitType InKey)
	{
		return GetTypeHash(InKey);
	}
};
