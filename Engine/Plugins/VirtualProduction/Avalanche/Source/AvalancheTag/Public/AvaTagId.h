// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "AvaTagId.generated.h"

/**
 * Struct to identify a Tag. This is used by FAvaTagHandle to reference an FAvaTag
 */
USTRUCT()
struct FAvaTagId
{
	GENERATED_BODY()

	/** Id is initialized to a zero */
	FAvaTagId() = default;

	/**
	 * Force init where Id is initialized to a new guid.
	 * This is used in TCppStructOps::Construct
	 */
	explicit FAvaTagId(EForceInit)
		: Id(FGuid::NewGuid())
	{
	}

	explicit FAvaTagId(const FGuid& InId)
		: Id(InId)
	{
	}

	bool operator==(const FAvaTagId& InOther) const
	{
		return Id == InOther.Id;
	}

	bool IsValid() const
	{
		return Id.IsValid();
	}

	FString ToString() const
	{
		return Id.ToString();
	}

	friend uint32 GetTypeHash(const FAvaTagId& InTagId)
	{
		return GetTypeHash(InTagId.Id);
	}

private:
	UPROPERTY(EditAnywhere, Category="Tag", meta=(IgnoreForMemberInitializationTest))
	FGuid Id;
};

template<>
struct TStructOpsTypeTraits<FAvaTagId> : TStructOpsTypeTraitsBase2<FAvaTagId>
{
	enum
	{
		WithNoInitConstructor = true,
	};
};
