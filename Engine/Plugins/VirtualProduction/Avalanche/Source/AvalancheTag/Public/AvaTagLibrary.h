// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "AvaTagLibrary.generated.h"

struct FAvaTag;
struct FAvaTagHandle;
struct FAvaTagHandleContainer;
struct FAvaTagSoftHandle;

UCLASS(MinimalAPI, DisplayName = "Motion Design Tag Library", meta=(ScriptName = "MotionDesignTagLibrary"))
class UAvaTagLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, Category = "Motion Design|Tag", meta=(CompactNodeTitle = "->", BlueprintAutocast))
	static AVALANCHETAG_API TArray<FAvaTag> ResolveTagHandle(const FAvaTagHandle& InTagHandle);

	UFUNCTION(BlueprintPure, Category = "Motion Design|Tag", meta=(CompactNodeTitle = "->", BlueprintAutocast))
	static AVALANCHETAG_API TArray<FAvaTag> ResolveTagHandles(const FAvaTagHandleContainer& InTagHandleContainer);

	UFUNCTION(BlueprintPure, Category = "Motion Design|Tag", meta=(CompactNodeTitle = "->", BlueprintAutocast))
	static AVALANCHETAG_API FAvaTagHandle ResolveTagSoftHandle(const FAvaTagSoftHandle& InTagSoftHandle);
};
