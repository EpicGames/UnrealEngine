// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

struct FAvaTag;

struct FAvaTagList
{
	TArray<const FAvaTag*, TInlineAllocator<1>> Tags;

	decltype(Tags)::RangedForIteratorType begin() { return Tags.begin(); }
	decltype(Tags)::RangedForIteratorType end() { return Tags.end(); }

	decltype(Tags)::RangedForConstIteratorType begin() const { return Tags.begin(); }
	decltype(Tags)::RangedForConstIteratorType end() const { return Tags.end(); }

	decltype(Tags)::RangedForReverseIteratorType rbegin() { return Tags.rbegin(); }
	decltype(Tags)::RangedForReverseIteratorType rend() { return Tags.rend(); }

	decltype(Tags)::RangedForConstReverseIteratorType rbegin() const { return Tags.rbegin(); }
	decltype(Tags)::RangedForConstReverseIteratorType rend() const { return Tags.rend(); }
};
