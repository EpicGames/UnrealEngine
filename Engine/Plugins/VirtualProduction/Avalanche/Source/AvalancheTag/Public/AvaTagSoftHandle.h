// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagId.h"
#include "UObject/SoftObjectPtr.h"
#include "AvaTagSoftHandle.generated.h"

class UAvaTagCollection;
struct FAvaTagHandle;

/**
 * Soft reference handle to a Tag in a particular Source Tag Collection
 * This should be used when needing to soft reference a particular FAvaTag.
 */
USTRUCT(BlueprintType, DisplayName="Motion Design Tag Soft Handle")
struct FAvaTagSoftHandle
{
	GENERATED_BODY()

	FAvaTagSoftHandle() = default;

	explicit FAvaTagSoftHandle(const TSoftObjectPtr<UAvaTagCollection>& InSource, const FAvaTagId& InTagId)
		: Source(InSource)
		, TagId(InTagId)
	{
	}

	AVALANCHETAG_API explicit FAvaTagSoftHandle(const FAvaTagHandle& InTagHandle);

	/**
	 * Creates a Tag Handle from this Soft Handle.
	 * Warning: This loads in the Source Tag Collection if not already loaded.
	 */
	AVALANCHETAG_API FAvaTagHandle MakeTagHandle() const;

	/** Returns true if this Soft Tag Handle matches the provided Tag Handle (Same Source and Tag Id) */
	AVALANCHETAG_API bool MatchesExact(const FAvaTagHandle& InTagHandle) const;

	AVALANCHETAG_API void PostSerialize(const FArchive& Ar);

	bool IsValid() const
	{
		return !Source.IsNull() && TagId.IsValid();
	}

	UPROPERTY(EditAnywhere, Category = "Tag")
	TSoftObjectPtr<const UAvaTagCollection> Source;

	UPROPERTY()
	FAvaTagId TagId;
};

template<>
struct TStructOpsTypeTraits<FAvaTagSoftHandle> : public TStructOpsTypeTraitsBase2<FAvaTagSoftHandle>
{
	enum
	{
		WithPostSerialize = true,
	};
};
