// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagId.h"
#include "Containers/ContainersFwd.h"
#include "AvaTagSoftHandleContainer.generated.h"

#define UE_API AVALANCHETAG_API

class UAvaTagCollection;
struct FAvaTag;
struct FAvaTagHandle;

/**
 * Soft handle to multiple tags in a particular Source.
 * This should be used when needing to soft reference multiple FAvaTags.
 */
USTRUCT(BlueprintType, DisplayName="Motion Design Tag Soft Handle Container")
struct FAvaTagSoftHandleContainer
{
	GENERATED_BODY()

	FAvaTagSoftHandleContainer() = default;

	UE_API explicit FAvaTagSoftHandleContainer(const FAvaTagHandle& InTagHandle);

	/** Returns true if the Tag Handles resolve to same valued FAvaTags, even if the Source or Tag Id is different */
	UE_API bool ContainsTag(const FAvaTagHandle& InTagHandle) const;

	/** Returns true if the Tag Handles is the exact same as the other (Same Source and Tag Id) */
	UE_API bool ContainsTagHandle(const FAvaTagHandle& InTagHandle) const;

	UE_API FString ToString() const;

	UE_API void PostSerialize(const FArchive& Ar);

	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& InPropertyTag, FStructuredArchive::FSlot InSlot);

	/** Adds the provided Tag Handle to TagIds. Returns true only if it was added as a new entry to TagIds */
	UE_API bool AddTagHandle(const FAvaTagHandle& InTagHandle);

	/** Removes the provided Tag Handle from TagIds. Returns true only if it existed and was removed from TagIds */
	UE_API bool RemoveTagHandle(const FAvaTagHandle& InTagHandle);

	/** Returns an array of resolved tags through this container's tag ids and source tag collection */
	UE_API TArray<FAvaTag> ResolveTags() const;

	/** Resolves the source soft object reference */
	const UAvaTagCollection* ResolveSource() const;

	TConstArrayView<FAvaTagId> GetTagIds() const
	{
		return TagIds;
	}

	UPROPERTY(EditAnywhere, Category = "Tag")
	TSoftObjectPtr<const UAvaTagCollection> Source;

private:
	UPROPERTY(EditAnywhere, Category = "Tag")
	TArray<FAvaTagId> TagIds;
};

template<>
struct TStructOpsTypeTraits<FAvaTagSoftHandleContainer> : public TStructOpsTypeTraitsBase2<FAvaTagSoftHandleContainer>
{
	enum
	{
		WithPostSerialize = true,
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

#undef UE_API
