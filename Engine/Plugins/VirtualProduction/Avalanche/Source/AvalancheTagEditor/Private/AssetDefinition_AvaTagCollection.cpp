// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_AvaTagCollection.h"
#include "AvaTagCollection.h"

FText UAssetDefinition_AvaTagCollection::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AvaTagCollection", "Motion Design Tag Collection");
}

FLinearColor UAssetDefinition_AvaTagCollection::GetAssetColor() const
{
	return FLinearColor(FColor(165, 243, 243));
}

TSoftClassPtr<UObject> UAssetDefinition_AvaTagCollection::GetAssetClass() const
{
	return UAvaTagCollection::StaticClass();
}
