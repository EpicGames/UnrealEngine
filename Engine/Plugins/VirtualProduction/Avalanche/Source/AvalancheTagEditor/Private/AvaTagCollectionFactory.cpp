// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagCollectionFactory.h"
#include "AssetToolsModule.h"
#include "AvaTagCollection.h"

UAvaTagCollectionFactory::UAvaTagCollectionFactory()
{
	SupportedClass = UAvaTagCollection::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

uint32 UAvaTagCollectionFactory::GetMenuCategories() const
{
	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	return AssetTools.FindAdvancedAssetCategory(TEXT("MotionDesignCategory"));
}

UObject* UAvaTagCollectionFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn)
{
	if (!ensure(SupportedClass == InClass))
	{
		return nullptr;
	}
	return NewObject<UAvaTagCollection>(InParent, InName, InFlags);
}
