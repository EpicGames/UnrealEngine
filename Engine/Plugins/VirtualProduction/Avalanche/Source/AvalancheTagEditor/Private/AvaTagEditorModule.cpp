// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagEditorModule.h"
#include "AvaTag.h"
#include "AvaTagCollection.h"
#include "AvaTagEditorStyle.h"
#include "AvaTagHandle.h"
#include "AvaTagHandleContainer.h"
#include "AvaTagSoftHandle.h"
#include "AvaTagSoftHandleContainer.h"
#include "Customization/AvaTagAliasCustomization.h"
#include "Customization/AvaTagCollectionCustomization.h"
#include "Customization/AvaTagHandleCustomization.h"
#include "Customization/TagCustomizers/AvaTagHandleContainerCustomizer.h"
#include "Customization/TagCustomizers/AvaTagHandleCustomizer.h"
#include "Customization/TagCustomizers/AvaTagSoftHandleContainerCustomizer.h"
#include "Customization/TagCustomizers/AvaTagSoftHandleCustomizer.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

namespace UE::AvaTagEditor::Private
{
	template<typename InTagCustomizerType>
	FOnGetPropertyTypeCustomizationInstance CreateTagHandleCustomization()
	{
		return FOnGetPropertyTypeCustomizationInstance::CreateLambda([]
		{
			return MakeShared<FAvaTagHandleCustomization>(MakeShared<InTagCustomizerType>());
		});
	}
}

void FAvalancheTagEditorModule::StartupModule()
{
	RegisterCustomizations();
	FAvaTagEditorStyle::Get();
}

void FAvalancheTagEditorModule::ShutdownModule()
{
    UnregisterCustomizations();
}

void FAvalancheTagEditorModule::RegisterCustomizations()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(CustomizedTypes.Add_GetRef(FAvaTagHandle::StaticStruct()->GetFName())
		, UE::AvaTagEditor::Private::CreateTagHandleCustomization<FAvaTagHandleCustomizer>());

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(CustomizedTypes.Add_GetRef(FAvaTagSoftHandle::StaticStruct()->GetFName())
		, UE::AvaTagEditor::Private::CreateTagHandleCustomization<FAvaTagSoftHandleCustomizer>());

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(CustomizedTypes.Add_GetRef(FAvaTagHandleContainer::StaticStruct()->GetFName())
		, UE::AvaTagEditor::Private::CreateTagHandleCustomization<FAvaTagHandleContainerCustomizer>());

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(CustomizedTypes.Add_GetRef(FAvaTagSoftHandleContainer::StaticStruct()->GetFName())
		, UE::AvaTagEditor::Private::CreateTagHandleCustomization<FAvaTagSoftHandleContainerCustomizer>());

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(CustomizedTypes.Add_GetRef(FAvaTagAlias::StaticStruct()->GetFName())
		, FOnGetPropertyTypeCustomizationInstance::CreateLambda([]
		{
			return MakeShared<FAvaTagAliasCustomization>();
		}));

	PropertyEditorModule.RegisterCustomClassLayout(CustomizedClasses.Add_GetRef(UAvaTagCollection::StaticClass()->GetFName())
		, FOnGetDetailCustomizationInstance::CreateLambda([]
		{
			return MakeShared<FAvaTagCollectionCustomization>();
		}));
}

void FAvalancheTagEditorModule::UnregisterCustomizations()
{
	if (FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		for (FName CustomizedType : CustomizedTypes)
		{
			PropertyEditorModule->UnregisterCustomPropertyTypeLayout(CustomizedType);	
		}
		CustomizedTypes.Reset();

		for (FName CustomizedClass : CustomizedClasses)
		{
			PropertyEditorModule->UnregisterCustomClassLayout(CustomizedClass);
		}
		CustomizedClasses.Reset();
	}
}

IMPLEMENT_MODULE(FAvalancheTagEditorModule, AvalancheTagEditor)
