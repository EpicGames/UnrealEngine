// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Modules/ModuleInterface.h"
#include "UObject/NameTypes.h"

class FAvalancheTagEditorModule : public IModuleInterface
{
public:
    //~ Begin IModuleInterface
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;
    //~ End IModuleInterface

    void RegisterCustomizations();

    void UnregisterCustomizations();

    TArray<FName> CustomizedTypes;

    TArray<FName> CustomizedClasses;
};
