// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagEditorStyle.h"
#include "Brushes/SlateColorBrush.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"

FAvaTagEditorStyle::FAvaTagEditorStyle()
	: FSlateStyleSet(TEXT("AvaTagEditor"))
{
	const FTableRowStyle& TableRowStyle = FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row");

	FSlateColorBrush SelectColorBrush(FStyleColors::Select);

	Set("TagListView.Row", FTableRowStyle(TableRowStyle)
		.SetEvenRowBackgroundHoveredBrush(SelectColorBrush)
		.SetOddRowBackgroundHoveredBrush(SelectColorBrush));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FAvaTagEditorStyle::~FAvaTagEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
