// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagAliasCustomization.h"
#include "AvaTagAlias.h"
#include "Builders/AvaTagElementHelper.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "SAvaTagPicker.h"
#include "TagCustomizers/AvaTagAliasCustomizer.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

FAvaTagAliasCustomization::FAvaTagAliasCustomization()
	: TagElementHelper(MakeShared<FAvaTagElementHelper>())
	, TagCustomizer(MakeShared<FAvaTagAliasCustomizer>())
{
}

void FAvaTagAliasCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	TSharedPtr<IPropertyHandle> TagNameHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaTagAlias, AliasName));

	InHeaderRow
		.WholeRowContent()
		.HAlign(HAlign_Left)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.MinDesiredWidth(150.f)
				[
					TagNameHandle->CreatePropertyValueWidget()	
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.MinDesiredWidth(150.f)
				[
					SNew(SAvaTagPicker, InPropertyHandle, TagCustomizer)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				TagElementHelper->CreatePropertyButtonsWidget(InPropertyHandle)
			]
		];
}
