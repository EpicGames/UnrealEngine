// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class IAvaTagHandleCustomizer;
class FAvaTagElementHelper;

class FAvaTagAliasCustomization : public IPropertyTypeCustomization
{
public:
	FAvaTagAliasCustomization();

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildrenBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override {}
	//~ End IPropertyTypeCustomization

	TSharedRef<FAvaTagElementHelper> TagElementHelper;

	TSharedRef<IAvaTagHandleCustomizer> TagCustomizer;
};
