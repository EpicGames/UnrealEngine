// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagCollectionCustomization.h"
#include "AvaTagCollection.h"
#include "Builders/AvaTagAliasMapBuilder.h"
#include "Builders/AvaTagTagMapBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

void FAvaTagCollectionCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	TSharedRef<IPropertyHandle> TagMapHandle = InDetailBuilder.GetProperty(UAvaTagCollection::GetTagMapName());
	InDetailBuilder.EditCategory(TEXT("Tags"))
		.HeaderContent(BuildHeaderContent(TagMapHandle), /*bWholeRowContent*/true)
		.AddCustomBuilder(MakeShared<FAvaTagTagMapBuilder>(TagMapHandle), /*bForAdvanced*/false);

	TSharedRef<IPropertyHandle> AliasMapHandle = InDetailBuilder.GetProperty(UAvaTagCollection::GetAliasMapName());
	InDetailBuilder.EditCategory(TEXT("Aliases"))
		.HeaderContent(BuildHeaderContent(AliasMapHandle), /*bWholeRowContent*/true)
		.AddCustomBuilder(MakeShared<FAvaTagAliasMapBuilder>(AliasMapHandle), /*bForAdvanced*/false);
}

TSharedRef<SWidget> FAvaTagCollectionCustomization::BuildHeaderContent(const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	return SNew(SBox)
		.HAlign(HAlign_Left)
		[
			SNew(SBox)
			.MinDesiredWidth(250.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				.VAlign(VAlign_Center)
				[
					InPropertyHandle->CreatePropertyNameWidget()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					InPropertyHandle->CreateDefaultPropertyButtonWidgets()
				]
			]
		];
}
