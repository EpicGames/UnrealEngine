// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class IPropertyHandle;
class SWidget;

class FAvaTagCollectionCustomization : public IDetailCustomization
{
public:
	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization

	TSharedRef<SWidget> BuildHeaderContent(const TSharedRef<IPropertyHandle>& InPropertyHandle);
};
