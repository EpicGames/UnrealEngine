// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagHandleCustomization.h"
#include "AvaTagHandle.h"
#include "DetailWidgetRow.h"
#include "SAvaTagPicker.h"
#include "Widgets/SBoxPanel.h"

void FAvaTagHandleCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	InHeaderRow
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(200.f)
		.MaxDesiredWidth(200.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			[
				SNew(SAvaTagPicker, InPropertyHandle, TagCustomizer)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				InPropertyHandle->CreateDefaultPropertyButtonWidgets()
			]
		];
}
