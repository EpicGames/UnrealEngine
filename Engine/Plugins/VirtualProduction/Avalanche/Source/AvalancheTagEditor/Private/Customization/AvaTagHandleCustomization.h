// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class IAvaTagHandleCustomizer;

class FAvaTagHandleCustomization : public IPropertyTypeCustomization
{
public:
	explicit FAvaTagHandleCustomization(TSharedRef<IAvaTagHandleCustomizer>&& InTagCustomizer)
		: TagCustomizer(MoveTemp(InTagCustomizer))
	{
	}

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override {}
	//~ End IPropertyTypeCustomization

private:
	TSharedRef<IAvaTagHandleCustomizer> TagCustomizer;
};
