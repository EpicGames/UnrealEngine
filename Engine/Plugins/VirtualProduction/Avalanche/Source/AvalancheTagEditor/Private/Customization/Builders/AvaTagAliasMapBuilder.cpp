// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagAliasMapBuilder.h"
#include "IDetailChildrenBuilder.h"

void FAvaTagAliasMapBuilder::GenerateChildContent(IDetailChildrenBuilder& InChildrenBuilder)
{
	ForEachChildProperty(
		[&InChildrenBuilder](const TSharedRef<IPropertyHandle>& InChildHandle)
		{
			InChildrenBuilder.AddProperty(InChildHandle);
		});
}
