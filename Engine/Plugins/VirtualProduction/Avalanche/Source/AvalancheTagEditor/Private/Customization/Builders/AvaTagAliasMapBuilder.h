// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagMapBuilder.h"

/** Map builder for the Alias Map */
class FAvaTagAliasMapBuilder : public FAvaTagMapBuilder
{
public:
	explicit FAvaTagAliasMapBuilder(const TSharedRef<IPropertyHandle>& InMapProperty)
		: FAvaTagMapBuilder(InMapProperty)
	{
	}

	//~ Begin IDetailCustomNodeBuilder
	virtual void GenerateChildContent(IDetailChildrenBuilder& InChildrenBuilder) override;
	//~ End IDetailCustomNodeBuilder
};
