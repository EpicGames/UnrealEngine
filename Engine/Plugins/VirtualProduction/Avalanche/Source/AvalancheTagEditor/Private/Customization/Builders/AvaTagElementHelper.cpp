// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagElementHelper.h"
#include "AssetRegistry/AssetIdentifier.h"
#include "AvaTag.h"
#include "AvaTagId.h"
#include "Editor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SComboBox.h"

#define LOCTEXT_NAMESPACE "AvaTagElementHelper"

TSharedRef<SWidget> FAvaTagElementHelper::CreatePropertyButtonsWidget(TSharedPtr<IPropertyHandle> InElementHandle)
{
	FMenuBuilder MenuContentBuilder(/*bInShouldCloseWindowAfterMenuSelection*/true
		, /*InCommandList*/nullptr
		, /*InExtender*/nullptr
		, /*bInCloseSelfOnly*/true);

	if (CanDeleteItem(InElementHandle))
	{
		MenuContentBuilder.AddMenuEntry(LOCTEXT("DeleteButtonLabel", "Delete")
			, FText::GetEmpty()
			, FSlateIcon()
			, FExecuteAction::CreateSP(this, &FAvaTagElementHelper::DeleteItem, InElementHandle));
	}

	MenuContentBuilder.AddMenuEntry(LOCTEXT("SearchForReferencesLabel", "Search for References")
		, FText::GetEmpty()
		, FSlateIcon()
		, FExecuteAction::CreateSP(this, &FAvaTagElementHelper::SearchForReferences, InElementHandle->GetKeyHandle()));

	return SNew(SComboButton)
		.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
		.ContentPadding(2)
		.ForegroundColor(FSlateColor::UseForeground())
		.HasDownArrow(true)
		.MenuContent()
		[
			MenuContentBuilder.MakeWidget()
		];
}

bool FAvaTagElementHelper::CanDeleteItem(const TSharedPtr<IPropertyHandle>& InElementHandle) const
{
	if (!InElementHandle.IsValid())
	{
		return false;
	}

	TSharedPtr<IPropertyHandle> ParentHandle = InElementHandle->GetParentHandle();
	if (!ParentHandle.IsValid())
	{
		return false;
	}

	return ParentHandle->AsArray() || ParentHandle->AsMap() || ParentHandle->AsSet();
}

void FAvaTagElementHelper::DeleteItem(TSharedPtr<IPropertyHandle> InElementHandle)
{
	if (!InElementHandle.IsValid())
	{
		return;
	}

	TSharedPtr<IPropertyHandle> ParentHandle = InElementHandle->GetParentHandle();
	if (!ParentHandle.IsValid())
	{
		return;
	}

	if (TSharedPtr<IPropertyHandleArray> ArrayProperty = ParentHandle->AsArray())
	{
		ArrayProperty->DeleteItem(InElementHandle->GetArrayIndex());
	}
	else if (TSharedPtr<IPropertyHandleMap> MapProperty = ParentHandle->AsMap())
	{
		MapProperty->DeleteItem(InElementHandle->GetArrayIndex());
	}
	else if (TSharedPtr<IPropertyHandleSet> SetProperty = ParentHandle->AsSet())
	{
		SetProperty->DeleteItem(InElementHandle->GetArrayIndex());
	}
}

void FAvaTagElementHelper::SearchForReferences(TSharedPtr<IPropertyHandle> InTagIdHandle)
{
	if (!FEditorDelegates::OnOpenReferenceViewer.IsBound())
	{
		return;
	}

	if (!InTagIdHandle.IsValid())
	{
		return;
	}

	TArray<const void*> TagIdRawData;
	InTagIdHandle->AccessRawData(TagIdRawData);
	if (TagIdRawData.IsEmpty() || !TagIdRawData[0])
	{
		return;
	}

	const FAvaTagId& TagId = *static_cast<const FAvaTagId*>(TagIdRawData[0]);

	TArray<FAssetIdentifier> AssetIdentifiers;
	AssetIdentifiers.Emplace(FAvaTagId::StaticStruct(), *TagId.ToString());
	FEditorDelegates::OnOpenReferenceViewer.Broadcast(AssetIdentifiers, FReferenceViewerParams());
}

#undef LOCTEXT_NAMESPACE
