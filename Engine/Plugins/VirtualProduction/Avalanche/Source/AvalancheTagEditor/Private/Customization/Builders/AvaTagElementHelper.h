// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IPropertyHandle;
class SWidget;

/** Helper class to re-use same functionality across Node Builders and Customizations */
class FAvaTagElementHelper : public TSharedFromThis<FAvaTagElementHelper>
{
public:
	TSharedRef<SWidget> CreatePropertyButtonsWidget(TSharedPtr<IPropertyHandle> InElementHandle);

	bool CanDeleteItem(const TSharedPtr<IPropertyHandle>& InElementHandle) const;

	void DeleteItem(TSharedPtr<IPropertyHandle> InElementHandle);

	void SearchForReferences(TSharedPtr<IPropertyHandle> InTagIdHandle);
};
