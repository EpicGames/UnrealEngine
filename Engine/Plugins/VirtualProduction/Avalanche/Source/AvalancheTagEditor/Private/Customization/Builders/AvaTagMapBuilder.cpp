// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagMapBuilder.h"
#include "PropertyHandle.h"

FAvaTagMapBuilder::FAvaTagMapBuilder(const TSharedRef<IPropertyHandle>& InMapProperty)
	: BaseProperty(InMapProperty)
	, MapProperty(InMapProperty->AsMap().ToSharedRef())
{
	BaseProperty->MarkHiddenByCustomization();

	// Delegate for when the number of children in the array changes
	FSimpleDelegate OnNumChildrenChanged = FSimpleDelegate::CreateRaw(this, &FAvaTagMapBuilder::OnNumChildrenChanged);
	OnNumElementsChangedHandle = MapProperty->SetOnNumElementsChanged(OnNumChildrenChanged);
}

FAvaTagMapBuilder::~FAvaTagMapBuilder()
{
	MapProperty->UnregisterOnNumElementsChanged(OnNumElementsChangedHandle);
	OnNumElementsChangedHandle.Reset();
}

FName FAvaTagMapBuilder::GetName() const
{
	const FProperty* Property = BaseProperty->GetProperty();
	ensure(Property);
	return Property ? Property->GetFName() : NAME_None;
}

TSharedPtr<IPropertyHandle> FAvaTagMapBuilder::GetPropertyHandle() const
{
	return BaseProperty;
}

void FAvaTagMapBuilder::SetOnRebuildChildren(FSimpleDelegate InOnRebuildChildren)
{
	OnRebuildChildren = InOnRebuildChildren;
}

void FAvaTagMapBuilder::ForEachChildProperty(TFunctionRef<void(const TSharedRef<IPropertyHandle>&)> InCallable)
{
	uint32 NumChildren = 0;
	BaseProperty->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		if (TSharedPtr<IPropertyHandle> ChildHandle = BaseProperty->GetChildHandle(ChildIndex))
		{
			InCallable(ChildHandle.ToSharedRef());
		}
	}
}

void FAvaTagMapBuilder::OnNumChildrenChanged()
{
	OnRebuildChildren.ExecuteIfBound();
}
