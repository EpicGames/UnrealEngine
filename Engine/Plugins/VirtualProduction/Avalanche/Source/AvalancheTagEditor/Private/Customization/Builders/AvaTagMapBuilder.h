// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "IDetailCustomNodeBuilder.h"
#include "Templates/SharedPointer.h"

class IPropertyHandleMap;

/** Base class to Build Map properties */
class FAvaTagMapBuilder : public IDetailCustomNodeBuilder, public TSharedFromThis<FAvaTagMapBuilder>
{
public:
	FAvaTagMapBuilder(const TSharedRef<IPropertyHandle>& InMapProperty);

	virtual ~FAvaTagMapBuilder() override;

	//~ Begin IDetailCustomNodeBuilder
	virtual FName GetName() const override;
	virtual bool InitiallyCollapsed() const override { return false; }
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& InNodeRow) override {}
	virtual TSharedPtr<IPropertyHandle> GetPropertyHandle() const override;
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRebuildChildren) override;
	//~ End IDetailCustomNodeBuilder

protected:
	void ForEachChildProperty(TFunctionRef<void(const TSharedRef<IPropertyHandle>&)> InCallable);

	void OnNumChildrenChanged();

	TSharedRef<IPropertyHandle> BaseProperty;

	TSharedRef<IPropertyHandleMap> MapProperty;

	FDelegateHandle OnNumElementsChangedHandle;

	FSimpleDelegate OnRebuildChildren;
};
