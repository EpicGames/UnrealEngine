// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagTagMapBuilder.h"
#include "AvaTag.h"
#include "AvaTagElementHelper.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

FAvaTagTagMapBuilder::FAvaTagTagMapBuilder(const TSharedRef<IPropertyHandle>& InMapProperty)
	: FAvaTagMapBuilder(InMapProperty)
	, TagElementHelper(MakeShared<FAvaTagElementHelper>())
{
}

void FAvaTagTagMapBuilder::GenerateChildContent(IDetailChildrenBuilder& InChildrenBuilder)
{
	ForEachChildProperty(
		[&InChildrenBuilder, this](const TSharedRef<IPropertyHandle>& InChildHandle)
		{
			TSharedPtr<IPropertyHandle> TagNameHandle = InChildHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaTag, TagName));
			check(TagNameHandle.IsValid());

			InChildrenBuilder.AddProperty(InChildHandle)
				.CustomWidget(/*bShowChildren*/false)
				.WholeRowContent()
				.HAlign(HAlign_Left)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SBox)
						.MinDesiredWidth(150.f)
						[
							TagNameHandle->CreatePropertyValueWidget()
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						TagElementHelper->CreatePropertyButtonsWidget(InChildHandle)
					]
				];
		});
}
