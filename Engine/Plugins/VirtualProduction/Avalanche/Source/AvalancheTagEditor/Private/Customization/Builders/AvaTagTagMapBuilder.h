// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagMapBuilder.h"

class FAvaTagElementHelper;

/** Map builder for the Tag Map */
class FAvaTagTagMapBuilder : public FAvaTagMapBuilder
{
public:
	explicit FAvaTagTagMapBuilder(const TSharedRef<IPropertyHandle>& InMapProperty);

	//~ Begin IDetailCustomNodeBuilder
	virtual void GenerateChildContent(IDetailChildrenBuilder& InChildrenBuilder) override;
	//~ End IDetailCustomNodeBuilder

private:
	TSharedRef<FAvaTagElementHelper> TagElementHelper;
};
