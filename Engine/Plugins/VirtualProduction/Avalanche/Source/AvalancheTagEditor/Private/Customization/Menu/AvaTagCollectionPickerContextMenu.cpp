// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagCollectionPickerContextMenu.h"
#include "AvaTagCollection.h"
#include "AvaTagCollectionMenuContext.h"
#include "Editor.h"
#include "PropertyHandle.h"
#include "Styling/AppStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "AvaTagCollectionPickerContextMenu"

FAvaTagCollectionPickerContextMenu& FAvaTagCollectionPickerContextMenu::Get()
{
	static TSharedRef<FAvaTagCollectionPickerContextMenu> ContextMenu = MakeShared<FAvaTagCollectionPickerContextMenu>(FPrivateToken());
	return ContextMenu.Get();
}

TSharedRef<SWidget> FAvaTagCollectionPickerContextMenu::GenerateContextMenuWidget(const TSharedPtr<IPropertyHandle>& InTagCollectionHandle)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return SNullWidget::NullWidget;
	}

	constexpr const TCHAR* ContextMenuName = TEXT("AvaTagCollectionPickerContextMenu");

	if (!ToolMenus->IsMenuRegistered(ContextMenuName))
	{
		UToolMenu* const ToolMenu = ToolMenus->RegisterMenu(ContextMenuName, NAME_None, EMultiBoxType::Menu);
		check(ToolMenu);
		ToolMenu->AddDynamicSection(TEXT("PopulateContextMenu"), FNewToolMenuDelegate::CreateSP(this, &FAvaTagCollectionPickerContextMenu::PopulateContextMenu));
	}

	UAvaTagCollectionMenuContext* MenuContext = NewObject<UAvaTagCollectionMenuContext>();
	MenuContext->TagCollectionHandleWeak = InTagCollectionHandle;

	FToolMenuContext ToolMenuContext(MenuContext);	
	return ToolMenus->GenerateWidget(ContextMenuName, ToolMenuContext);
}

void FAvaTagCollectionPickerContextMenu::PopulateContextMenu(UToolMenu* InToolMenu)
{
	UAvaTagCollectionMenuContext* MenuContext = InToolMenu
		? InToolMenu->FindContext<UAvaTagCollectionMenuContext>()
		: nullptr;

	if (!MenuContext || !MenuContext->TagCollectionHandleWeak.IsValid())
	{
		return;
	}

	FToolMenuSection& AssetActions = InToolMenu->FindOrAddSection(TEXT("AssetActions"), LOCTEXT("AssetSectionLabel", "Asset"));

	AssetActions.AddMenuEntry(TEXT("EditAsset")
		, LOCTEXT("EditAssetLabel", "Edit...")
		, LOCTEXT("EditAssetTooltip", "Opens the Tag Collection asset editor")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit")
		, FExecuteAction::CreateSP(this, &FAvaTagCollectionPickerContextMenu::EditAsset, MenuContext->TagCollectionHandleWeak));

	AssetActions.AddMenuEntry(TEXT("FindInContentBrowser")
		, LOCTEXT("FindInContentBrowserLabel", "Find in Content Browser")
		, LOCTEXT("FindInContentBrowserTooltip", "Navigates to the Tag Collection asset in the Content Browser")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.FindInContentBrowser")
		, FExecuteAction::CreateSP(this, &FAvaTagCollectionPickerContextMenu::FindInContentBrowser, MenuContext->TagCollectionHandleWeak));
}

bool FAvaTagCollectionPickerContextMenu::TryGetAsset(const TWeakPtr<IPropertyHandle>& InTagCollectionHandleWeak, FAssetData& OutAssetData) const
{
	if (TSharedPtr<IPropertyHandle> TagCollectionHandle = InTagCollectionHandleWeak.Pin())
	{
		return TagCollectionHandle->GetValue(OutAssetData) == FPropertyAccess::Success;
	}
	return false;
}

void FAvaTagCollectionPickerContextMenu::EditAsset(TWeakPtr<IPropertyHandle> InTagCollectionHandleWeak)
{
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor ? GEditor->GetEditorSubsystem<UAssetEditorSubsystem>() : nullptr;

	FAssetData TagCollectionAsset;
	if (AssetEditorSubsystem && TryGetAsset(InTagCollectionHandleWeak, TagCollectionAsset))
	{
		AssetEditorSubsystem->OpenEditorForAsset(TagCollectionAsset.GetSoftObjectPath());
	}
}

void FAvaTagCollectionPickerContextMenu::FindInContentBrowser(TWeakPtr<IPropertyHandle> InTagCollectionHandleWeak)
{
	FAssetData TagCollectionAsset;
	if (GEditor && TryGetAsset(InTagCollectionHandleWeak, TagCollectionAsset))
	{
		GEditor->SyncBrowserToObject(TagCollectionAsset);
	}
}

#undef LOCTEXT_NAMESPACE
