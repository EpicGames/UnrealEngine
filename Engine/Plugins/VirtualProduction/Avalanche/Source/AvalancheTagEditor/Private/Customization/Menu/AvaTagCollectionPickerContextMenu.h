// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class IPropertyHandle;
class SWidget;
class UToolMenu;
struct FAssetData;

class FAvaTagCollectionPickerContextMenu : public TSharedFromThis<FAvaTagCollectionPickerContextMenu>
{
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	explicit FAvaTagCollectionPickerContextMenu(FPrivateToken) {}

	static FAvaTagCollectionPickerContextMenu& Get();

	TSharedRef<SWidget> GenerateContextMenuWidget(const TSharedPtr<IPropertyHandle>& InTagCollectionHandle);

private:
	void PopulateContextMenu(UToolMenu* InToolMenu);

	bool TryGetAsset(const TWeakPtr<IPropertyHandle>& InTagCollectionHandleWeak, FAssetData& OutAssetData) const;

	void EditAsset(TWeakPtr<IPropertyHandle> InTagCollectionHandleWeak);

	void FindInContentBrowser(TWeakPtr<IPropertyHandle> InTagCollectionHandleWeak);
};
