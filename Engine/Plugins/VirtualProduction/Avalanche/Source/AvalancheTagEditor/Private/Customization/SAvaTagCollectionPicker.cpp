// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaTagCollectionPicker.h"
#include "AvaTagCollection.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaTagCollectionPicker"

void SAvaTagCollectionPicker::Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InTagCollectionPropertyHandle)
{
	TagCollectionPropertyHandle = InTagCollectionPropertyHandle;

	OnTagCollectionChanged = InArgs._OnTagCollectionChanged;

	ChildSlot
	[
		SAssignNew(TagCollectionPickerButton, SComboButton)
		.OnGetMenuContent(this, &SAvaTagCollectionPicker::MakeTagCollectionPicker)
		.OnMenuOpenChanged(this, &SAvaTagCollectionPicker::OnTagCollectionMenuOpenChanged)
		.ContentPadding(FMargin(6, -1, 0, -1))
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &SAvaTagCollectionPicker::GetTagCollectionTitleText)
			.ToolTipText(this, &SAvaTagCollectionPicker::GetTagCollectionTooltipText)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];
}

bool SAvaTagCollectionPicker::IsOpen() const
{
	return TagCollectionPickerButton->IsOpen();
}

void SAvaTagCollectionPicker::SetIsOpen(bool bInIsOpen)
{
	TagCollectionPickerButton->SetIsOpen(bInIsOpen);
}

FText SAvaTagCollectionPicker::GetTagCollectionTitleText() const
{
	FAssetData AssetData;
	TagCollectionPropertyHandle->GetValue(AssetData);

	if (AssetData.IsValid())
	{
		FString ObjectPath = AssetData.GetObjectPathString();
		if (AssetData.IsTopLevelAsset())
		{
			int32 CharIndex;
			if (ObjectPath.FindLastChar(TEXT('.'), CharIndex))
			{
				ObjectPath.LeftInline(CharIndex, EAllowShrinking::No);
			}
		}
		return FText::FromString(ObjectPath);
	}

	return LOCTEXT("NoTagCollectionTitle", "(no asset)");
}

FText SAvaTagCollectionPicker::GetTagCollectionTooltipText() const
{
	FAssetData AssetData;
	TagCollectionPropertyHandle->GetValue(AssetData);

	if (AssetData.IsValid())
	{
		return FText::FromString(AssetData.GetObjectPathString());	
	}

	return LOCTEXT("NoTagCollectionTooltip", "No Tag Collection set");
}

TSharedRef<SWidget> SAvaTagCollectionPicker::MakeTagCollectionPicker()
{
	constexpr bool bAllowClear = true;
	TArray<const UClass*> AllowedClasses = { UAvaTagCollection::StaticClass() };
	TArray<const UClass*> DisallowedClasses;

	return PropertyCustomizationHelpers::MakeAssetPickerWithMenu(FAssetData()
		, bAllowClear
		, AllowedClasses
		, DisallowedClasses
		, PropertyCustomizationHelpers::GetNewAssetFactoriesForClasses(AllowedClasses)
		, FOnShouldFilterAsset()
		, FOnAssetSelected::CreateSP(this, &SAvaTagCollectionPicker::OnTagCollectionSelected)
		, FSimpleDelegate::CreateSP(this, &SAvaTagCollectionPicker::CloseTagCollectionPicker)
		, TagCollectionPropertyHandle);
}

void SAvaTagCollectionPicker::OnTagCollectionMenuOpenChanged(bool bInIsOpened)
{
	if (!bInIsOpened)
	{
		TagCollectionPickerButton->SetMenuContent(SNullWidget::NullWidget);
	}
}

void SAvaTagCollectionPicker::OnTagCollectionSelected(const FAssetData& InAssetData)
{
	// Once a Tag Collection has been selected, open up the Tag Selector if the assets have changed,
	// as this is going to be the user's most likely next action (and can always click out)
	if (InAssetData.IsValid())
	{
		FAssetData CurrentAssetData;
		TagCollectionPropertyHandle->GetValue(CurrentAssetData);

		if (CurrentAssetData != InAssetData)
		{
			OnTagCollectionChanged.ExecuteIfBound();
		}
	}

	CloseTagCollectionPicker();
	TagCollectionPropertyHandle->SetValue(InAssetData);
}

void SAvaTagCollectionPicker::CloseTagCollectionPicker()
{
	TagCollectionPickerButton->SetIsOpen(false);
}

#undef LOCTEXT_NAMESPACE
