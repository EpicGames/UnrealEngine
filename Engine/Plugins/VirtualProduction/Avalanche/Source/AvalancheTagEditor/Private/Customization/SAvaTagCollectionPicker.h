// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FText;
class IPropertyHandle;
class SComboButton;
struct FAssetData;

class SAvaTagCollectionPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaTagCollectionPicker) {}
		SLATE_EVENT(FSimpleDelegate, OnTagCollectionChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InTagCollectionPropertyHandle);

	bool IsOpen() const;

	void SetIsOpen(bool bInIsOpen);

private:
	FText GetTagCollectionTitleText() const;

	FText GetTagCollectionTooltipText() const;

	TSharedRef<SWidget> MakeTagCollectionPicker();

	void OnTagCollectionMenuOpenChanged(bool bInIsOpened);

	DLLEXPORT void OnTagCollectionSelected(const FAssetData& InAssetData);

	void CloseTagCollectionPicker();

	TSharedPtr<IPropertyHandle> TagCollectionPropertyHandle;

	TSharedPtr<SComboButton> TagCollectionPickerButton;

	FSimpleDelegate OnTagCollectionChanged;

	friend class UAvaTagCollectionHelper;
};
