// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaTagHandleEntry.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::AvaTagEditor::Private
{
	bool IsMousePressEvent(const FPointerEvent& InMouseEvent)
	{
		return InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton || InMouseEvent.IsTouchEvent();
	}
}

void SAvaTagHandleEntry::Construct(const FArguments& InArgs, const FAvaTagHandle& InTagHandle)
{
	TagHandle          = InTagHandle;
	IsSelected         = InArgs._IsSelected;
	OnSelectionChanged = InArgs._OnSelectionChanged;

	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

	if (InArgs._ShowCheckBox)
	{
		HorizontalBox->AddSlot()
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &SAvaTagHandleEntry::GetCheckState)
				.OnCheckStateChanged(this, &SAvaTagHandleEntry::OnCheckStateChanged)
			];
	}

	HorizontalBox->AddSlot()
		.FillWidth(1.f)
		[
			SNew(SBox)
			.MinDesiredWidth(120.f)
			.MaxDesiredWidth(250.f)
			.Padding(5.f, 1.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SAvaTagHandleEntry::GetTagText)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Visibility(EVisibility::SelfHitTestInvisible)
			]
		];

	ChildSlot
	[
		HorizontalBox
	];
}

FReply SAvaTagHandleEntry::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	FReply Reply = FReply::Handled();

	if (IsEnabled() && UE::AvaTagEditor::Private::IsMousePressEvent(InMouseEvent))
	{
		bIsPressed = true;
		Reply.CaptureMouse(AsShared());
	}
	return Reply;
}

FReply SAvaTagHandleEntry::OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (bIsPressed && UE::AvaTagEditor::Private::IsMousePressEvent(InMouseEvent))
	{
		bIsPressed = false;

		if (IsEnabled() && HasMouseCapture())
		{
			bool bEventOverButton = IsHovered();
			if (!bEventOverButton && InMouseEvent.IsTouchEvent())
			{
				bEventOverButton = InGeometry.IsUnderLocation(InMouseEvent.GetScreenSpacePosition());
			}

			if (bEventOverButton)
			{
				// Toggle Behavior: Select if not selected, Deselect if selected
				bool bSelect = !IsSelected.Get(false);
				OnSelectionChanged.ExecuteIfBound(TagHandle, bSelect);
			}
		}
	}

	FReply Reply = FReply::Handled();

	if (HasMouseCapture())
	{
		Reply.ReleaseMouseCapture();
	}

	return Reply;
}

FText SAvaTagHandleEntry::GetTagText() const
{
	return FText::FromName(TagHandle.ToName());
}

ECheckBoxState SAvaTagHandleEntry::GetCheckState() const
{
	return IsSelected.Get(false)
		? ECheckBoxState::Checked
		: ECheckBoxState::Unchecked;
}

void SAvaTagHandleEntry::OnCheckStateChanged(ECheckBoxState InState)
{
	OnSelectionChanged.ExecuteIfBound(TagHandle, InState == ECheckBoxState::Checked);
}
