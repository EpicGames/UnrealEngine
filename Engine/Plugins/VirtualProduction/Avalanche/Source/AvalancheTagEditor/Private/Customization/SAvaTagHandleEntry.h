// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagHandle.h"
#include "Delegates/Delegate.h"
#include "Widgets/SCompoundWidget.h"

enum class ECheckBoxState : uint8;

class SAvaTagHandleEntry : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaTagHandleEntry) {}
		SLATE_ARGUMENT(bool, ShowCheckBox)
		SLATE_ATTRIBUTE(bool, IsSelected)
		SLATE_EVENT(TDelegate<void(const FAvaTagHandle&, bool)>, OnSelectionChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FAvaTagHandle& InTagHandle);

private:
	//~ Begin SWidget
	virtual FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	//~ End SWidget

	FText GetTagText() const;

	ECheckBoxState GetCheckState() const;

	void OnCheckStateChanged(ECheckBoxState InState);

	TAttribute<bool> IsSelected;

	TDelegate<void(const FAvaTagHandle&, bool)> OnSelectionChanged;

	FAvaTagHandle TagHandle;

	bool bIsPressed = true;
};
