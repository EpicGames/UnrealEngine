// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaTagPicker.h"
#include "AvaTag.h"
#include "AvaTagCollection.h"
#include "AvaTagEditorStyle.h"
#include "AvaTagHandle.h"
#include "DetailLayoutBuilder.h"
#include "Menu/AvaTagCollectionPickerContextMenu.h"
#include "SAvaTagCollectionPicker.h"
#include "SAvaTagHandleEntry.h"
#include "Styling/ToolBarStyle.h"
#include "TagCustomizers/IAvaTagHandleCustomizer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "SAvaTagPicker"

void SAvaTagPicker::Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InStructPropertyHandle, const TSharedRef<IAvaTagHandleCustomizer>& InTagCustomizer)
{
	StructPropertyHandle = InStructPropertyHandle;

	TagCustomizer = InTagCustomizer;

	TSharedRef<SVerticalBox> ContentVerticalBox = SNew(SVerticalBox);

	TSharedRef<SHorizontalBox> WrapperHorizontalBox = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
        .FillWidth(1.f)
        [
        	ContentVerticalBox
        ];

	TagCollectionPropertyHandle = InTagCustomizer->GetTagCollectionHandle(InStructPropertyHandle);

	if (TagCollectionPropertyHandle.IsValid())
	{
		ContentVerticalBox->AddSlot()
			.AutoHeight()
			.Padding(0)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SAssignNew(TagCollectionPicker, SAvaTagCollectionPicker, TagCollectionPropertyHandle.ToSharedRef())
				.OnTagCollectionChanged(this, &SAvaTagPicker::OnTagCollectionChanged)
			];

		WrapperHorizontalBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SAssignNew(TagCollectionOptions, SMenuAnchor)
				.Content()
				[
					SNew(SButton)
					.OnClicked(this, &SAvaTagPicker::OpenContextMenu)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText(LOCTEXT("TagCollectionOptionsToolTip", "Tag Collection Options"))
					.ContentPadding(FMargin(4, 6))
					[
						SNew(SImage)
						.Image(&FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar").SettingsComboButton.DownArrowImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			];
	}

	ContentVerticalBox->AddSlot()
		.AutoHeight()
		.Padding(0)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SAssignNew(TagComboButton, SComboButton)
			.HasDownArrow(true)
			.ContentPadding(FMargin(0, -1))
			.OnMenuOpenChanged(this, &SAvaTagPicker::OnTagMenuOpenChanged)
			.MenuContent()
			[
				SAssignNew(TagListView, SListView<TSharedPtr<FAvaTagHandle>>)
				.ListViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("SimpleListView"))
				.OnGenerateRow(this, &SAvaTagPicker::CreateTagTableRow)
				.SelectionMode(ESelectionMode::Single) // Selection Mode set to single, yet preventing Items to be selected via OnIsSelectableOrNavigable, so that hover cue appears
				.ListItemsSource(&TagOptions)
				.IsFocusable(false)
				.HandleGamepadEvents(false)
				.HandleSpacebarSelection(false)
				.HandleDirectionalNavigation(false)
			]
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &SAvaTagPicker::GetValueDisplayText)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];

	ChildSlot
	[
		WrapperHorizontalBox
	];
}

void SAvaTagPicker::Tick(const FGeometry& InGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bRequestOpenTagCollectionPicker)
	{
		TagComboButton->SetIsOpen(false);

		if (TagCollectionPicker.IsValid())
		{
			TagCollectionPicker->SetIsOpen(true);
		}

		bRequestOpenTagCollectionPicker = false;
	}
}

FReply SAvaTagPicker::OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		return OpenContextMenu();
	}
	return FReply::Unhandled();
}

FReply SAvaTagPicker::OpenContextMenu()
{
	if (TagCollectionOptions.IsValid() && !TagCollectionOptions->IsOpen())
	{
		// If TagCollectionOptions is valid, the TagCollectionHandle should also be
		ensure(TagCollectionPropertyHandle.IsValid());

		TagCollectionOptions->SetMenuContent(FAvaTagCollectionPickerContextMenu::Get().GenerateContextMenuWidget(TagCollectionPropertyHandle));
		TagCollectionOptions->SetIsOpen(true);
	}
	return FReply::Handled();
}

const UAvaTagCollection* SAvaTagPicker::GetOrLoadTagCollection() const
{
	if (!StructPropertyHandle.IsValid())
	{
		return nullptr;
	}

	const UAvaTagCollection* TagCollection = nullptr;

	StructPropertyHandle->EnumerateConstRawData(
		[&TagCollection, this](const void* InStructRawData, const int32 InDataIndex, const int32 InNumData)->bool
		{
			if (!InStructRawData)
			{
				return true;
			}
			const UAvaTagCollection* CurrentTagCollection = TagCustomizer->GetOrLoadTagCollection(InStructRawData);
			if (InDataIndex == 0)
			{
				TagCollection = CurrentTagCollection;
			}
			else if (TagCollection != CurrentTagCollection)
			{
				// return null if there's a different collection
				TagCollection = nullptr;
				return false;
			}
			return true;
		});

	return TagCollection;
}

void SAvaTagPicker::RefreshTagOptions()
{
	TagOptions.Reset();

	// Get or Load. Loading is ok at this point because this we're refreshing the tags the Tag Collection has to offer (i.e. we're peeking inside of it)
	if (const UAvaTagCollection* TagCollection = GetOrLoadTagCollection())
	{
		TArray<FAvaTagId> TagIds = TagCollection->GetTagIds(TagCustomizer->AllowAliasTags());

		// Add default (none) TagId as an option, if there's no multiple choice
		if (!TagCustomizer->AllowMultipleTags())
		{
			TagOptions.Add(MakeShared<FAvaTagHandle>(TagCollection, FAvaTagId()));
		}

		for (const FAvaTagId& TagId : TagIds)
		{
			TSharedRef<FAvaTagHandle> TagOption = MakeShared<FAvaTagHandle>(TagCollection, TagId);
			TagOptions.Add(TagOption);
		}
	}
	// if there's no current collection, prioritize selecting a collection first
	else if (TagCollectionPicker.IsValid() && !TagCollectionPicker->IsOpen())
	{
		bRequestOpenTagCollectionPicker = true;
	}

	TagListView->ClearSelection();
	TagListView->RequestListRefresh();
}

void SAvaTagPicker::OnTagCollectionChanged()
{
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSPLambda(this, [this](float)->bool
	{
		if (TagComboButton.IsValid())
		{
			RefreshTagOptions();
			TagComboButton->SetIsOpen(true);
		}
		return false;
	}));
}

void SAvaTagPicker::OnTagMenuOpenChanged(bool bInIsOpen)
{
	if (bInIsOpen)
	{
		RefreshTagOptions();
	}
}

TSharedRef<ITableRow> SAvaTagPicker::CreateTagTableRow(TSharedPtr<FAvaTagHandle> InTagHandle, const TSharedRef<STableViewBase>& InOwnerTable)
{
	check(InTagHandle.IsValid());
	FAvaTagHandle TagHandle = *InTagHandle;

	// Disable Selecting Items as this is handle already by SAvaTagHandleEntry
	class SAvaTagRow : public STableRow<TSharedPtr<FAvaTagHandle>>
	{
	public:
		//~ Begin STableRow
		virtual FReply OnMouseButtonDown(const FGeometry&, const FPointerEvent&) override { return FReply::Unhandled(); }
		virtual FReply OnMouseButtonUp(const FGeometry&, const FPointerEvent&) override { return FReply::Unhandled(); }
		//~ End STableRow
	};

	return SNew(SAvaTagRow, InOwnerTable)
		.Style(&FAvaTagEditorStyle::Get().GetWidgetStyle<FTableRowStyle>("TagListView.Row"))
		.Padding(FMargin(0,2))
		[
			SNew(SAvaTagHandleEntry, TagHandle)
			.IsSelected(this, &SAvaTagPicker::IsTagHandleSelected, TagHandle)
			.OnSelectionChanged(this, &SAvaTagPicker::OnTagHandleSelectionChanged)
			.ShowCheckBox(TagCustomizer->AllowMultipleTags())
		];
}

bool SAvaTagPicker::IsTagHandleSelected(FAvaTagHandle InTagHandle) const
{
	if (!StructPropertyHandle.IsValid())
	{
		return false;
	}

	bool bTagHandleSelected = false;

	StructPropertyHandle->EnumerateConstRawData(
		[&bTagHandleSelected, &InTagHandle, this](const void* InStructRawData, const int32 InDataIndex, const int32 InNumData)->bool
		{
			if (InStructRawData && TagCustomizer->ContainsTagHandle(InStructRawData, InTagHandle))
			{
				bTagHandleSelected = true;
				return false;
			}
			return true;
		});

	return bTagHandleSelected;
}

void SAvaTagPicker::OnTagHandleSelectionChanged(const FAvaTagHandle& InTagHandle, bool bInIsSelected)
{
	if (!InTagHandle.Source)
	{
		return;
	}

	TagCustomizer->SetTagHandleAdded(StructPropertyHandle.ToSharedRef(), InTagHandle, bInIsSelected);

	// If Single Selection close Combo
	if (!TagCustomizer->AllowMultipleTags())
	{
		TagComboButton->SetIsOpen(false);
	}
}

FText SAvaTagPicker::GetValueDisplayText() const
{
	if (!StructPropertyHandle.IsValid())
	{
		return FText::GetEmpty();
	}

	FName ValueName;
	bool bMultipleValues = false;

	StructPropertyHandle->EnumerateConstRawData(
		[&ValueName, &bMultipleValues, this](const void* InStructRawData, const int32 InDataIndex, const int32 InNumData)->bool
		{
			if (!InStructRawData)
			{
				return true;
			}
			FName CurrentValueName = TagCustomizer->GetDisplayValueName(InStructRawData);
			if (InDataIndex == 0)
			{
				ValueName = CurrentValueName;
			}
			else if (ValueName != CurrentValueName)
			{
				bMultipleValues = true;
				return false;
			}
			return true;
		});

	if (bMultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}

	return FText::FromName(ValueName);
}

#undef LOCTEXT_NAMESPACE
