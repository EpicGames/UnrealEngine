// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/SCompoundWidget.h"

class IAvaTagHandleCustomizer;
class IPropertyHandle;
class ITableRow;
class SAvaTagCollectionPicker;
class SComboButton;
class SMenuAnchor;
class STableViewBase;
class UAvaTagCollection;
struct FAvaTagHandle;
template <typename ItemType> class SListView;

class SAvaTagPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaTagPicker) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<IPropertyHandle>& InStructPropertyHandle, const TSharedRef<IAvaTagHandleCustomizer>& InTagCustomizer);

private:
	//~ Begin SWidget
	virtual void Tick(const FGeometry& InGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	//~ End SWidget

	FReply OpenContextMenu();

	const UAvaTagCollection* GetOrLoadTagCollection() const;

	void RefreshTagOptions();

	void OnTagCollectionChanged();

	void OnTagMenuOpenChanged(bool bInIsOpen);

	TSharedRef<ITableRow> CreateTagTableRow(TSharedPtr<FAvaTagHandle> InTagHandle, const TSharedRef<STableViewBase>& InOwnerTable);

	bool IsTagHandleSelected(FAvaTagHandle InTagHandle) const;

	DLLEXPORT void OnTagHandleSelectionChanged(const FAvaTagHandle& InTagHandle, bool bInIsSelected);

	FText GetValueDisplayText() const;

	/** Handle to the struct property */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	/** Handle to the Tag Collection property */
	TSharedPtr<IPropertyHandle> TagCollectionPropertyHandle;

	TSharedPtr<SAvaTagCollectionPicker> TagCollectionPicker;

	TSharedPtr<SMenuAnchor> TagCollectionOptions;

	TSharedPtr<SListView<TSharedPtr<FAvaTagHandle>>> TagListView;

	TSharedPtr<IAvaTagHandleCustomizer> TagCustomizer;

	TSharedPtr<SComboButton> TagComboButton;

	/** The available tag options */
	TArray<TSharedPtr<FAvaTagHandle>> TagOptions;

	bool bRequestOpenTagCollectionPicker = false;

	friend class UAvaTagCollectionHelper;
};
