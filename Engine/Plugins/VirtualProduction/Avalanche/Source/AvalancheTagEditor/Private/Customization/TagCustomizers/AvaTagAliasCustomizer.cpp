// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagAliasCustomizer.h"
#include "AvaTagAlias.h"
#include "AvaTagHandle.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "AvaTagAliasCustomizer"

TSharedPtr<IPropertyHandle> FAvaTagAliasCustomizer::GetTagCollectionHandle(const TSharedRef<IPropertyHandle>& InStructHandle) const
{
	// There's no source property in Tag Alias so do not generate a source picker.
	return nullptr;
}

const UAvaTagCollection* FAvaTagAliasCustomizer::GetOrLoadTagCollection(const void* InStructRawData) const
{
	return static_cast<const FAvaTagAlias*>(InStructRawData)->GetOwner();
}

void FAvaTagAliasCustomizer::SetTagHandleAdded(const TSharedRef<IPropertyHandle>& InContainerProperty, const FAvaTagHandle& InTagHandle, bool bInAdd) const
{
	FScopedTransaction Transaction(bInAdd
		? LOCTEXT("AddTagHandleInAlias", "Add Tag Handle in Alias")
		: LOCTEXT("RemoveTagHandleInAlias", "Remove Tag Handle from Alias"));

	InContainerProperty->NotifyPreChange();
	InContainerProperty->EnumerateRawData(
		[&InTagHandle, bInAdd](void* InStructRawData, const int32, const int32)->bool
		{
			FAvaTagAlias* Alias = static_cast<FAvaTagAlias*>(InStructRawData);
			if (bInAdd)
			{
				Alias->TagIds.AddUnique(InTagHandle.TagId);
			}
			else
			{
				Alias->TagIds.Remove(InTagHandle.TagId);
			}
			return true;
		});

	InContainerProperty->NotifyPostChange(bInAdd ? EPropertyChangeType::ArrayAdd : EPropertyChangeType::ArrayRemove);
	InContainerProperty->NotifyFinishedChangingProperties();
}

bool FAvaTagAliasCustomizer::ContainsTagHandle(const void* InStructRawData, const FAvaTagHandle& InTagHandle) const
{
	return static_cast<const FAvaTagAlias*>(InStructRawData)->TagIds.Contains(InTagHandle.TagId);
}

FName FAvaTagAliasCustomizer::GetDisplayValueName(const void* InStructRawData) const
{
	return *static_cast<const FAvaTagAlias*>(InStructRawData)->GetTagsAsString();
}

#undef LOCTEXT_NAMESPACE
