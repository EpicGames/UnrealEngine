// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagHandleContainerCustomizer.h"
#include "AvaTagCollection.h"
#include "AvaTagHandle.h"
#include "AvaTagHandleContainer.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "AvaTagHandleContainerCustomizer"

TSharedPtr<IPropertyHandle> FAvaTagHandleContainerCustomizer::GetTagCollectionHandle(const TSharedRef<IPropertyHandle>& InStructHandle) const
{
	return InStructHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaTagHandleContainer, Source));
}

const UAvaTagCollection* FAvaTagHandleContainerCustomizer::GetOrLoadTagCollection(const void* InStructRawData) const
{
	return static_cast<const FAvaTagHandleContainer*>(InStructRawData)->Source;
}

void FAvaTagHandleContainerCustomizer::SetTagHandleAdded(const TSharedRef<IPropertyHandle>& InContainerProperty, const FAvaTagHandle& InTagHandle, bool bInAdd) const
{
	FScopedTransaction Transaction(bInAdd
		? LOCTEXT("AddTagHandleInContainer", "Add Tag Handle in Container")
		: LOCTEXT("RemoveTagHandleInContainer", "Remove Tag Handle in Container"));

	InContainerProperty->NotifyPreChange();

	InContainerProperty->EnumerateRawData(
		[&InTagHandle, bInAdd](void* InStructRawData, const int32, const int32)->bool
		{
			FAvaTagHandleContainer* Container = static_cast<FAvaTagHandleContainer*>(InStructRawData);

			ensureMsgf(Container->Source == InTagHandle.Source
				, TEXT("Unexpected result setting tag handle in container: Container Source (%s) doesn't match Tag Handle Source (%s)")
				, *GetNameSafe(Container->Source)
				, *GetNameSafe(InTagHandle.Source));

			if (bInAdd)
			{
				Container->AddTagHandle(InTagHandle);
			}
			else
			{
				Container->RemoveTagHandle(InTagHandle);
			}
			return true;
		});

	InContainerProperty->NotifyPostChange(bInAdd ? EPropertyChangeType::ArrayAdd : EPropertyChangeType::ArrayRemove);
	InContainerProperty->NotifyFinishedChangingProperties();
}

bool FAvaTagHandleContainerCustomizer::ContainsTagHandle(const void* InStructRawData, const FAvaTagHandle& InTagHandle) const
{
	return static_cast<const FAvaTagHandleContainer*>(InStructRawData)->ContainsTagHandle(InTagHandle);
}

FName FAvaTagHandleContainerCustomizer::GetDisplayValueName(const void* InStructRawData) const
{
	return *static_cast<const FAvaTagHandleContainer*>(InStructRawData)->ToString();
}

#undef LOCTEXT_NAMESPACE
