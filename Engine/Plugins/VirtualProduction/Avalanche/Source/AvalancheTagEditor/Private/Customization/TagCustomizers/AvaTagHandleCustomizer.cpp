// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagHandleCustomizer.h"
#include "AvaTagHandle.h"
#include "PropertyHandle.h"

TSharedPtr<IPropertyHandle> FAvaTagHandleCustomizer::GetTagCollectionHandle(const TSharedRef<IPropertyHandle>& InStructHandle) const
{
	return InStructHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaTagHandle, Source));
}

const UAvaTagCollection* FAvaTagHandleCustomizer::GetOrLoadTagCollection(const void* InStructRawData) const
{
	return static_cast<const FAvaTagHandle*>(InStructRawData)->Source;
}

void FAvaTagHandleCustomizer::SetTagHandleAdded(const TSharedRef<IPropertyHandle>& InContainerProperty, const FAvaTagHandle& InTagHandle, bool bInAdd) const
{
	FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(InContainerProperty->GetProperty());
	check(StructProperty);

	FString TextValue;
	StructProperty->Struct->ExportText(TextValue, &InTagHandle, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr);

	ensure(InContainerProperty->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
}

bool FAvaTagHandleCustomizer::ContainsTagHandle(const void* InStructRawData, const FAvaTagHandle& InTagHandle) const
{
	return static_cast<const FAvaTagHandle*>(InStructRawData)->MatchesExact(InTagHandle);
}

FName FAvaTagHandleCustomizer::GetDisplayValueName(const void* InStructRawData) const
{
	return static_cast<const FAvaTagHandle*>(InStructRawData)->ToName();
}
