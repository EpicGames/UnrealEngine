// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagSoftHandleContainerCustomizer.h"
#include "AvaTagCollection.h"
#include "AvaTagHandle.h"
#include "AvaTagSoftHandleContainer.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "AvaTagSoftHandleContainerCustomizer"

TSharedPtr<IPropertyHandle> FAvaTagSoftHandleContainerCustomizer::GetTagCollectionHandle(const TSharedRef<IPropertyHandle>& InStructHandle) const
{
	return InStructHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaTagSoftHandleContainer, Source));
}

const UAvaTagCollection* FAvaTagSoftHandleContainerCustomizer::GetOrLoadTagCollection(const void* InStructRawData) const
{
	return static_cast<const FAvaTagSoftHandleContainer*>(InStructRawData)->Source.LoadSynchronous();
}

void FAvaTagSoftHandleContainerCustomizer::SetTagHandleAdded(const TSharedRef<IPropertyHandle>& InContainerProperty, const FAvaTagHandle& InTagHandle, bool bInAdd) const
{
	FScopedTransaction Transaction(bInAdd
		? LOCTEXT("AddTagHandleInContainer", "Add Tag Handle in Container")
		: LOCTEXT("RemoveTagHandleInContainer", "Remove Tag Handle in Container"));

	InContainerProperty->NotifyPreChange();

	InContainerProperty->EnumerateRawData(
		[&InTagHandle, bInAdd](void* InStructRawData, const int32, const int32)->bool
		{
			FAvaTagSoftHandleContainer* Container = static_cast<FAvaTagSoftHandleContainer*>(InStructRawData);

			ensureMsgf(Container->Source == InTagHandle.Source
				, TEXT("Unexpected result setting tag handle in container: Container Source (%s) doesn't match Tag Handle Source (%s)")
				, *Container->Source->GetName()
				, *GetNameSafe(InTagHandle.Source));

			if (bInAdd)
			{
				Container->AddTagHandle(InTagHandle);
			}
			else
			{
				Container->RemoveTagHandle(InTagHandle);
			}
			return true;
		});

	InContainerProperty->NotifyPostChange(bInAdd ? EPropertyChangeType::ArrayAdd : EPropertyChangeType::ArrayRemove);
	InContainerProperty->NotifyFinishedChangingProperties();
}

bool FAvaTagSoftHandleContainerCustomizer::ContainsTagHandle(const void* InStructRawData, const FAvaTagHandle& InTagHandle) const
{
	return static_cast<const FAvaTagSoftHandleContainer*>(InStructRawData)->ContainsTagHandle(InTagHandle);
}

FName FAvaTagSoftHandleContainerCustomizer::GetDisplayValueName(const void* InStructRawData) const
{
	return *static_cast<const FAvaTagSoftHandleContainer*>(InStructRawData)->ToString();
}

#undef LOCTEXT_NAMESPACE
