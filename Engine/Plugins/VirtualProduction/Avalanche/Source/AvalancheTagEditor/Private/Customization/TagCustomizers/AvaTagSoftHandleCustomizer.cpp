// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTagSoftHandleCustomizer.h"
#include "AvaTagCollection.h"
#include "AvaTagHandle.h"
#include "AvaTagSoftHandle.h"
#include "PropertyHandle.h"

TSharedPtr<IPropertyHandle> FAvaTagSoftHandleCustomizer::GetTagCollectionHandle(const TSharedRef<IPropertyHandle>& InStructHandle) const
{
	return InStructHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaTagSoftHandle, Source));
}

const UAvaTagCollection* FAvaTagSoftHandleCustomizer::GetOrLoadTagCollection(const void* InStructRawData) const
{
	return static_cast<const FAvaTagSoftHandle*>(InStructRawData)->Source.LoadSynchronous();
}

void FAvaTagSoftHandleCustomizer::SetTagHandleAdded(const TSharedRef<IPropertyHandle>& InContainerProperty, const FAvaTagHandle& InTagHandle, bool bInAdd) const
{
	FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(InContainerProperty->GetProperty());
	check(StructProperty);

	FString TextValue;

	FAvaTagSoftHandle SoftHandle(InTagHandle);
	StructProperty->Struct->ExportText(TextValue, &SoftHandle, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr);

	ensure(InContainerProperty->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
}

bool FAvaTagSoftHandleCustomizer::ContainsTagHandle(const void* InStructRawData, const FAvaTagHandle& InTagHandle) const
{
	return static_cast<const FAvaTagSoftHandle*>(InStructRawData)->MatchesExact(InTagHandle);
}

FName FAvaTagSoftHandleCustomizer::GetDisplayValueName(const void* InStructRawData) const
{
	// todo: MakeTagHandle loads the TagCollection. Ideally, instead, Soft Handle should only use the Tag Collection is already available, or use the last cached tag that gets updated everytime it gets the opportunity
	// (e.g. it can get updated when tag collection changes or loads, or tag id changes)
	return static_cast<const FAvaTagSoftHandle*>(InStructRawData)->MakeTagHandle().ToName();
}
