// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaTagHandleCustomizer.h"

/** Customizer for FAvaTagSoftHandle */
class FAvaTagSoftHandleCustomizer : public IAvaTagHandleCustomizer
{
public:
	//~ Begin IAvaTagHandleCustomizer
	virtual TSharedPtr<IPropertyHandle> GetTagCollectionHandle(const TSharedRef<IPropertyHandle>& InStructHandle) const override;
	virtual const UAvaTagCollection* GetOrLoadTagCollection(const void* InStructRawData) const override;
	virtual void SetTagHandleAdded(const TSharedRef<IPropertyHandle>& InContainerProperty, const FAvaTagHandle& InTagHandle, bool bInAdd) const override;
	virtual bool ContainsTagHandle(const void* InStructRawData, const FAvaTagHandle& InTagHandle) const override;
	virtual FName GetDisplayValueName(const void* InStructRawData) const override;
	//~ End IAvaTagHandleCustomizer
};
