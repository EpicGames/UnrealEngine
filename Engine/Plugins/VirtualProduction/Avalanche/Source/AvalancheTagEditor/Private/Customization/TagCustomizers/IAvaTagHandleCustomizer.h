// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

class FName;
class IPropertyHandle;
class UAvaTagCollection;
struct FAvaTagHandle;

/** Abstraction Layer between the Tag Handle Types and the Tag Handle Customization */
class IAvaTagHandleCustomizer
{
public:
	virtual ~IAvaTagHandleCustomizer() = default;

	/**
	 * Gets the Child Property Handle to the Tag Collection Property
	 * If returning nullptr, the picker will not generate a Tag Collection Source Picker
	 */
	virtual TSharedPtr<IPropertyHandle> GetTagCollectionHandle(const TSharedRef<IPropertyHandle>& InStructHandle) const = 0;

	/** Gets or Loads the Tag Collection from the Raw Data Struct */
	virtual const UAvaTagCollection* GetOrLoadTagCollection(const void* InStructRawData) const = 0;

	/** Sets the Tag Handled as added or removed */
	virtual void SetTagHandleAdded(const TSharedRef<IPropertyHandle>& InStructHandle, const FAvaTagHandle& InTagHandle, bool bInAdd) const = 0;

	/** Returns whether the given Handle is contained by (or matches) the given Struct */
	virtual bool ContainsTagHandle(const void* InStructRawData, const FAvaTagHandle& InTagHandle) const = 0;

	/** Gets the Display Value Name of the Tags in the given Struct */
	virtual FName GetDisplayValueName(const void* InStructRawData) const = 0;

	/** Whether multiple tags are allowed to be selected, or just a single one */
	virtual bool AllowMultipleTags() const { return false; }

	/** Whether to show alias tags as options */
	virtual bool AllowAliasTags() const { return true; }
};
