// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheText : ModuleRules
{
	public AvalancheText(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AvalancheCore",
				"AvalancheMaterial",
				"Core",
				"CoreUObject",
				"Engine",
				"SlateCore",
				"Text3D",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ActorModifierCore",
				"Avalanche",
			}
		);

		if (Target.Type == TargetRules.TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"UnrealEd"
			});
		}
	}
}
