// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaText3DComponent.h"
#include "AvaActorUtils.h"
#include "AvaLog.h"
#include "Characters/Text3DCharacterTransform.h"
#include "Engine/Texture2D.h"
#include "Extensions/Text3DDefaultMaterialExtension.h"
#include "Extensions/Text3DLayoutTransformEffect.h"
#include "Extensions/Text3DMaterialExtensionBase.h"
#include "GameFramework/Actor.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Settings/Text3DProjectSettings.h"
#include "Text3DComponentVersion.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

// Sets default values for this component's properties
UAvaText3DComponent::UAvaText3DComponent()
{
	SetExtrude(0.0f);
	SetScaleProportionally(false);
	SetMaxWidth(100.f);
	SetMaxHeight(100.f);
}

void UAvaText3DComponent::Serialize(FArchive& InArchive)
{
	Super::Serialize(InArchive);

	const int32 Version = InArchive.CustomVer(FText3DComponentVersion::GUID);

	if (Version < FText3DComponentVersion::LatestVersion)
	{
		UE_LOGF(LogAva, Log, "AvaText3D : Migrating from %i to %i version", Version, FText3DComponentVersion::LatestVersion)

		if (Version < FText3DComponentVersion::Extensions)
		{
#if WITH_EDITORONLY_DATA
			PRAGMA_DISABLE_DEPRECATION_WARNINGS

			if (const UText3DProjectSettings* TextSettings = UText3DProjectSettings::Get())
			{
				SetFont(TextSettings->GetFallbackFont());
			}

#if WITH_EDITOR
			SetFontByName(MotionDesignFont_DEPRECATED.GetFontNameAsString());
#else
			SetFont(MotionDesignFont.GetFont());
#endif

			if (UText3DDefaultMaterialExtension* DefaultMaterialExtension = GetCastedMaterialExtension<UText3DDefaultMaterialExtension>())
			{
				DefaultMaterialExtension->SetStyle(static_cast<EText3DMaterialStyle>(ColoringStyle_DEPRECATED));
				DefaultMaterialExtension->SetFrontColor(Color_DEPRECATED);
				DefaultMaterialExtension->SetBackColor(Color_DEPRECATED);
				DefaultMaterialExtension->SetExtrudeColor(ExtrudeColor_DEPRECATED);
				DefaultMaterialExtension->SetBevelColor(BevelColor_DEPRECATED);
				DefaultMaterialExtension->SetGradientOffset(GradientSettings_DEPRECATED.Offset);
				DefaultMaterialExtension->SetGradientSmoothness(GradientSettings_DEPRECATED.Smoothness);
				DefaultMaterialExtension->SetGradientColorA(GradientSettings_DEPRECATED.ColorA);
				DefaultMaterialExtension->SetGradientColorB(GradientSettings_DEPRECATED.ColorB);

				if (GradientSettings_DEPRECATED.Direction == EAvaGradientDirection::Vertical)
				{
					DefaultMaterialExtension->SetGradientRotation(0.f);
				}
				else if (GradientSettings_DEPRECATED.Direction == EAvaGradientDirection::Horizontal)
				{
					DefaultMaterialExtension->SetGradientRotation(0.75f);
				}
				else
				{
					DefaultMaterialExtension->SetGradientRotation(GradientSettings_DEPRECATED.Rotation);
				}
				
				DefaultMaterialExtension->SetTextureAsset(MainTexture_DEPRECATED);
				DefaultMaterialExtension->SetTextureTiling(Tiling_DEPRECATED);

				if (TranslucencyStyle_DEPRECATED == EAvaTextTranslucency::None)
				{
					DefaultMaterialExtension->SetBlendMode(EText3DMaterialBlendMode::Opaque);
				}
				else
				{
					DefaultMaterialExtension->SetBlendMode(EText3DMaterialBlendMode::Translucent);
				}

				DefaultMaterialExtension->SetOpacity(Opacity_DEPRECATED);

				if (MaskOrientation_DEPRECATED == EAvaMaterialMaskOrientation::LeftRight)
				{
					DefaultMaterialExtension->SetMaskRotation(0.75f);
				}
				else if (MaskOrientation_DEPRECATED == EAvaMaterialMaskOrientation::RightLeft)
				{
					DefaultMaterialExtension->SetMaskRotation(0.25f);
				}
				else
				{
					DefaultMaterialExtension->SetMaskRotation(MaskRotation_DEPRECATED);
				}

				DefaultMaterialExtension->SetIsUnlit(bIsUnlit_DEPRECATED);
				DefaultMaterialExtension->SetMaskSmoothness(MaskSmoothness_DEPRECATED);

				if (TranslucencyStyle_DEPRECATED == EAvaTextTranslucency::GradientMask)
				{
					DefaultMaterialExtension->SetMaskOffset(MaskOffset_DEPRECATED);
				}
				else
				{
					DefaultMaterialExtension->SetMaskOffset(1.f);
				}

				// Patch materials for other systems like mask
				DefaultMaterialExtension->PreCacheMaterials();
			}

			if (const AActor* Actor = GetOwner())
			{
				if (const UText3DCharacterTransform* CharacterTransform = Actor->FindComponentByClass<UText3DCharacterTransform>())
				{
					UText3DLayoutTransformEffect* TransformExtension = NewObject<UText3DLayoutTransformEffect>(this, TEXT("AvaTransformExtension"));

					TransformExtension->SetLocationEnabled(CharacterTransform->GetLocationEnabled());
					TransformExtension->SetLocationProgress(CharacterTransform->GetLocationProgress());
					TransformExtension->SetLocationOrder(CharacterTransform->GetLocationOrder());
					TransformExtension->SetLocationEnd(CharacterTransform->GetLocationDistance());

					TransformExtension->SetRotationEnabled(CharacterTransform->GetRotationEnabled());
					TransformExtension->SetRotationProgress(CharacterTransform->GetRotationProgress());
					TransformExtension->SetRotationOrder(CharacterTransform->GetRotationOrder());
					TransformExtension->SetRotationBegin(CharacterTransform->GetRotationBegin());
					TransformExtension->SetRotationEnd(CharacterTransform->GetRotationEnd());

					TransformExtension->SetScaleEnabled(CharacterTransform->GetScaleEnabled());
					TransformExtension->SetScaleProgress(CharacterTransform->GetScaleProgress());
					TransformExtension->SetScaleOrder(CharacterTransform->GetScaleOrder());
					TransformExtension->SetScaleBegin(CharacterTransform->GetScaleBegin());
					TransformExtension->SetScaleEnd(CharacterTransform->GetScaleEnd());

					LayoutEffects.Emplace(TransformExtension);
				}
			}

			PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
		}
	}
}
