// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTextActor.h"
#include "AvaDefs.h"
#include "Characters/Text3DCharacterTransform.h"
#include "Extensions/Text3DDefaultMaterialExtension.h"

AAvaTextActor::AAvaTextActor()
{
	Text3DComponent = CreateDefaultSubobject<UAvaText3DComponent>(TEXT("Text3DComponent"));
	RootComponent = Text3DComponent;
}

FAvaColorChangeData AAvaTextActor::GetColorData() const
{
	FAvaColorChangeData ColorData;
	ColorData.ColorStyle = EAvaColorStyle::None;

	if (Text3DComponent)
	{
		if (const UText3DDefaultMaterialExtension* MaterialExtension = Text3DComponent->GetCastedMaterialExtension<UText3DDefaultMaterialExtension>())
		{
			switch (MaterialExtension->GetStyle())
			{
			case EText3DMaterialStyle::Solid:
				ColorData.ColorStyle = EAvaColorStyle::Solid;
				ColorData.PrimaryColor = MaterialExtension->GetFrontColor();
				ColorData.SecondaryColor = MaterialExtension->GetFrontColor();
				break;

			case EText3DMaterialStyle::Gradient:
				ColorData.ColorStyle = EAvaColorStyle::LinearGradient;
				ColorData.PrimaryColor = MaterialExtension->GetGradientColorA();
				ColorData.SecondaryColor = MaterialExtension->GetGradientColorB();
				break;

			default:
				break;
			}
		
			ColorData.bIsUnlit = MaterialExtension->GetIsUnlit();
		}
	}

	return ColorData;
}

void AAvaTextActor::SetColorData(const FAvaColorChangeData& NewColorData)
{
	if (Text3DComponent)
	{
		if (UText3DDefaultMaterialExtension* MaterialExtension = Text3DComponent->GetCastedMaterialExtension<UText3DDefaultMaterialExtension>())
		{
			switch (NewColorData.ColorStyle)
			{
			case EAvaColorStyle::Solid:
				MaterialExtension->SetStyle(EText3DMaterialStyle::Solid);
				MaterialExtension->SetFrontColor(NewColorData.PrimaryColor);
				MaterialExtension->SetBackColor(NewColorData.PrimaryColor);
				MaterialExtension->SetBevelColor(NewColorData.PrimaryColor);
				MaterialExtension->SetExtrudeColor(NewColorData.PrimaryColor);
				break;

			case EAvaColorStyle::LinearGradient:
				MaterialExtension->SetStyle(EText3DMaterialStyle::Gradient);
				MaterialExtension->SetGradientColorA(NewColorData.PrimaryColor);
				MaterialExtension->SetGradientColorB(NewColorData.SecondaryColor);
				break;

			default:
				break;
			}

			MaterialExtension->SetIsUnlit(NewColorData.bIsUnlit);
		}
	}
}

#if WITH_EDITOR
FString AAvaTextActor::GetDefaultActorLabel() const
{
	return TEXT("Text3DActor");
}
#endif
