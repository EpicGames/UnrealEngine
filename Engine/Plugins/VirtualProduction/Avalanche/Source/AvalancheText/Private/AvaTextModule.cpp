// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTextModule.h"
#include "MaterialBridge/AvaMaterialBridgeRegistry.h"
#include "MaterialBridge/AvaText3DComponentMaterialBridge.h"
#include "MaterialBridge/AvaText3DDefaultExtensionMaterialBridge.h"
#include "MaterialBridge/AvaText3DExtensionMaterialBridge.h"
#include "MaterialBridge/AvaText3DManagedComponentMaterialBridge.h"
#include "Modifiers/ActorModifierRenderStateDirtyEvent.h"
#include "Modules/ModuleManager.h"
#include "Text3DDelegates.h"

IMPLEMENT_MODULE(FAvaTextModule, AvalancheText)

void FAvaTextModule::StartupModule()
{
	constexpr uint32 DefaultPriority = 0;

	UE::Ava::FMaterialBridgeRegistry& MaterialBridgeRegistry = UE::Ava::FMaterialBridgeRegistry::GetMutable();
	MaterialBridgeRegistry.Register<UE::Ava::FText3DComponentMaterialBridge>(DefaultPriority);
	MaterialBridgeRegistry.Register<UE::Ava::FText3DExtensionMaterialBridge>(DefaultPriority);
	MaterialBridgeRegistry.Register<UE::Ava::FText3DDefaultExtensionMaterialBridge>(DefaultPriority);

	// Higher than default to be prioritized over registered default material bridges of the same bridged type.
	constexpr uint32 HigherPriority = 100;

	MaterialBridgeRegistry.Register<UE::Ava::FText3DManagedComponentMaterialBridge>(HigherPriority);

	RegisterTextRendererDelegates();
}

void FAvaTextModule::ShutdownModule()
{
	UnregisterTextRendererDelegates();
}

void FAvaTextModule::RegisterTextRendererDelegates()
{
	static TArray<TUniquePtr<UE::ActorModifierCore::FRenderStateDirtyReasonScope>, TInlineAllocator<8>> Scope;

	static auto GetReason = [](const UE::Text3D::FText3DBuildContext& InBuildContext)->UE::ActorModifierCore::ERenderStateDirtyReason
		{
			// If only updating material, use the reason scope. Otherwise, keep it unknown
			if (InBuildContext.UpdateFlags == EText3DRendererFlags::Material)
			{
				return UE::ActorModifierCore::ERenderStateDirtyReason::Material;
			}
			return UE::ActorModifierCore::ERenderStateDirtyReason::Unknown;
		};

	TextPreRendererUpdateHandle = UE::Text3D::OnText3DPreRendererUpdate().AddLambda(
		[](const UE::Text3D::FText3DBuildContext& InBuildContext)
		{
			Scope.Push(MakeUnique<UE::ActorModifierCore::FRenderStateDirtyReasonScope>(GetReason(InBuildContext)));
		});

	TextPostRendererUpdateHandle = UE::Text3D::OnText3DPostRendererUpdate().AddLambda(
		[](const UE::Text3D::FText3DBuildContext& InBuildContext)
		{
			Scope.Pop();
		});
}

void FAvaTextModule::UnregisterTextRendererDelegates()
{
	UE::Text3D::OnText3DPreRendererUpdate().Remove(TextPreRendererUpdateHandle);
	TextPreRendererUpdateHandle.Reset();

	UE::Text3D::OnText3DPostRendererUpdate().Remove(TextPostRendererUpdateHandle);
	TextPostRendererUpdateHandle.Reset();
}
