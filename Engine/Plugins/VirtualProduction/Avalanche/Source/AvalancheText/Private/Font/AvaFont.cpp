// Copyright Epic Games, Inc. All Rights Reserved.

#include "Font/AvaFont.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"

#define LOCTEXT_NAMESPACE "AvaFont"

FAvaFont::FAvaDefaultFontObjects& FAvaFont::GetDefaultFontObjects()
{
	static FAvaDefaultFontObjects DefaultFontObjects;

	return DefaultFontObjects;
}

UFont* FAvaFont::GetDefaultFont()
{
	if (IsRunningDedicatedServer())
	{
		return nullptr;
	}

	if (!GetDefaultFontObjects().AvaDefaultFont)
	{
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

		// Trying to load Roboto font
		const FSoftObjectPath RobotoPath(TEXT("/Script/Engine.Font'/Engine/EngineFonts/Roboto.Roboto'"));
		UFont* DefaultFont = Cast<UFont>(AssetRegistryModule.Get().GetAssetByObjectPath(RobotoPath).GetAsset());

		if (!DefaultFont)
		{
			// no Roboto was found, this is not expected. Trying to load an avalanche font
			TArray<FAssetData> AssetDataArray;
			// avalanche fonts path
			const FName Path = TEXT("/Game/SystemFonts/Fonts");
			AssetRegistryModule.Get().GetAssetsByPath(Path, AssetDataArray);

			// no avalanche font available, trying to load the first engine font available
			if (AssetDataArray.IsEmpty())
			{
				const UClass* Class = UFont::StaticClass();
				const FTopLevelAssetPath AssetPath(Class->GetPathName());
				AssetRegistryModule.Get().GetAssetsByClass(AssetPath, AssetDataArray);
			}

			for (const FAssetData& AssetData : AssetDataArray)
			{
				DefaultFont = CastChecked<UFont>(AssetData.GetAsset());

				if (DefaultFont)
				{
					break;
				}
			}
		}

		if (ensureMsgf(DefaultFont, TEXT("MotionDesignFont: cannot load any font to be used as default.")))
		{
			GetDefaultFontObjects().AvaDefaultFont = DefaultFont;
		}
	}

	return GetDefaultFontObjects().AvaDefaultFont.Get();
}

FString FAvaFont::GenerateFontFormattedString(const FString& InFontName, const FString& InFontObjectPathName)
{
	// e.g. (Property1=Value1,Property2=Value2,Property3=Value3,...)

	FString FormattedString = TEXT("(");
	FormattedString += TEXT("CurrentFont_DEPRECATED=None");
	FormattedString += TEXT(",MotionDesignFontObject=/Script/AvalancheText.AvaFontObject'") + InFontObjectPathName + TEXT("'");
	FormattedString += TEXT(",FontName=\"") + InFontName  + TEXT("\"");
	FormattedString += TEXT(")");

	return FormattedString;
}

bool FAvaFont::GenerateFontFormattedString(const UAvaFontObject* InFontObject, FString& OutFormattedString)
{
	if (!InFontObject)
	{
		return false;
	}

	OutFormattedString = GenerateFontFormattedString(InFontObject->GetFontName(), InFontObject->GetPathName());
	return true;
}

bool FAvaFont::AreSameFont(const FAvaFont* InFontA, const FAvaFont* InFontB)
{
	if (!InFontA || !InFontB)
	{
		return false;
	}

	return *InFontA == *InFontB;
}

void FAvaFont::InitDefaults()
{
	bIsFavorite = false;
	FontAssetState = EFontAssetState::DefaultFont;
}

FAvaFont::FAvaFont()
{
	InitDefaults();
	SetFontObject(GetDefaultAvaFontObject());
}

FAvaFont::FAvaFont(UAvaFontObject* InFontObject)
{
	InitDefaults();

	if (InFontObject)
	{
		SetFontObject(InFontObject);
	}
	else
	{
		SetFontObject(GetDefaultAvaFontObject());
	}
}

UFont* FAvaFont::GetFont()
{
	// make sure FontAssetState value is up to date
	RefreshAssetState();

	if (FontAssetState == EFontAssetState::SelectedFont)
	{
		EnsureUsingCurrentVersion();

		if (MotionDesignFontObject)
		{
			return MotionDesignFontObject->GetFont();
		}
	}

	// if this is the default font, or previously referenced resource is not available, return default font
	return GetDefaultFont();
}

EAvaFontSource FAvaFont::GetFontSource() const
{
	if (MotionDesignFontObject)
	{
		return MotionDesignFontObject->GetSource();
	}

	return EAvaFontSource::Invalid;
}

bool FAvaFont::operator==(const FAvaFont& Other) const
{
	if (CurrentFont_DEPRECATED)
	{
		return (CurrentFont_DEPRECATED == Other.CurrentFont_DEPRECATED) && (GetFontName() == Other.GetFontName());
	}

	const UAvaFontObject* MyFontObject = GetFontObject();
	const UAvaFontObject* OtherFontObject = Other.GetFontObject();

	if (MyFontObject && OtherFontObject && MyFontObject->GetFont() != OtherFontObject->GetFont())
	{
		return false;
	}

	return (GetFontName() == Other.GetFontName());
}

FName FAvaFont::GetFontName() const
{
	return FName(GetFontNameAsString());
}

FString FAvaFont::GetFontNameAsString() const
{
	if (MotionDesignFontObject)
	{
		return MotionDesignFontObject->GetFontName();
	}

	return FontName;
}

bool FAvaFont::IsFavorite() const
{
	return bIsFavorite;
}

bool FAvaFont::IsDefaultFont() const
{
	if (MotionDesignFontObject && MotionDesignFontObject->GetFont())
	{
		return MotionDesignFontObject->GetFont() == GetDefaultFont();
	}

	return true;
}

bool FAvaFont::IsFallbackFont() const
{
	return FontAssetState == EFontAssetState::FallbackFont;
}

bool FAvaFont::IsMonospaced() const
{
	if (MotionDesignFontObject)
	{
		return MotionDesignFontObject->IsMonospaced();
	}

	return false;
}

bool FAvaFont::IsBold() const
{
	if (MotionDesignFontObject)
	{
		return MotionDesignFontObject->IsBold();
	}

	return false;
}

bool FAvaFont::IsItalic() const
{
	if (MotionDesignFontObject)
	{
		return MotionDesignFontObject->IsItalic();
	}

	return false;
}

void FAvaFont::SetFavorite(const bool bFavorite)
{
	bIsFavorite = bFavorite;
}

void FAvaFont::EnsureUsingCurrentVersion()
{
	if (CurrentFont_DEPRECATED)
	{
		MotionDesignFontObject = NewObject<UAvaFontObject>();
		MotionDesignFontObject->InitProjectFont(CurrentFont_DEPRECATED, GetFontName().ToString());
		CurrentFont_DEPRECATED = nullptr;

		RefreshName();
	}
}

bool FAvaFont::HasValidFont() const
{
	const FCompositeFont* CompositeFont = nullptr;

	if (CurrentFont_DEPRECATED)
	{
		CompositeFont = CurrentFont_DEPRECATED->GetCompositeFont();
	}
	else if (MotionDesignFontObject && MotionDesignFontObject->GetFont())
	{
		CompositeFont = MotionDesignFontObject->GetFont()->GetCompositeFont();
	}

	if (CompositeFont)
	{
		if (!CompositeFont->DefaultTypeface.Fonts.IsEmpty())
		{
			if (CompositeFont->DefaultTypeface.Fonts[0].Font.GetFontFaceAsset())
			{
				return true;
			}
		}
	}

	return false;
}

void FAvaFont::SetFontObject(UAvaFontObject* InFontObject)
{
	CurrentFont_DEPRECATED = nullptr;
	MotionDesignFontObject = InFontObject;

	RefreshName();
	RefreshAssetState();
}

void FAvaFont::InitFromFont(UFont* InFont)
{
	if (InFont)
	{
		FString Name;
		UE::Ava::FontUtilities::GetFontName(InFont, Name);

		MotionDesignFontObject = NewObject<UAvaFontObject>();
		MotionDesignFontObject->InitProjectFont(InFont, Name);

		RefreshName();
	}
}

void FAvaFont::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		RefreshName();

		if ((MotionDesignFontObject && MotionDesignFontObject->GetFont()) || CurrentFont_DEPRECATED)
		{
			FontAssetState = EFontAssetState::SelectedFont;
		}
		else
		{
			/*
			 * MotionDesignFontObject might be invalid due to a previously existing issue.
			 * That would cause the Editor only version of MotionDesignFontObject to be serialized, instead of the proper one.
			 * For assets using the font saved like that, the MotionDesignFontObject had the wrong Outer, which worked fine in Editor, but not for Game/Runtime.
			 * The following code tries to find a font based on the font name, and then to create a new MotionDesignFontObject, with the proper Outer
			 */

			bool bFontRecoverySuccess = false;

			TArray<FProperty*> OutProperties;
			Ar.GetSerializedPropertyChain(OutProperties);

			// this should just contain the FAvaFont
			for (const FProperty* Property : OutProperties)
			{
				// let's get the Outer we need to create the MotionDesignFontObject
				if (UObject* const Outer = Property->GetOwner<UObject>())
				{
					// try to get a font just using the name (this might fail!)
					if (UFont* const Font = GetFontByName(FontName))
					{
						MotionDesignFontObject = nullptr;

						// we create and assign a new UAvaFontObject with the proper Outer
						MotionDesignFontObject = NewObject<UAvaFontObject>(Outer, GetFontName(), RF_Public | RF_Standalone);
						MotionDesignFontObject->InitProjectFont(Font, FontName);
						bFontRecoverySuccess = true;
						break;
					}
				}
			}

			// if we managed to retrieve the font, just act as if it was properly loaded
			if (bFontRecoverySuccess)
			{
				FontAssetState = EFontAssetState::SelectedFont;
			}
			else
			{
				MissingFontName = FontName;
				FontAssetState = EFontAssetState::FallbackFont;
			}
		}
	}
}

UAvaFontObject* FAvaFont::GetDefaultAvaFontObject()
{
	if (!GetDefaultFontObjects().AvaDefaultFontObject)
	{
		if (UFont* DefaultFont = GetDefaultFont())
		{
			GetDefaultFontObjects().AvaDefaultFontObject = NewObject<UAvaFontObject>(DefaultFont);
			GetDefaultFontObjects().AvaDefaultFontObject->InitProjectFont(DefaultFont, DefaultFont->GetName());
		}
	}

	return GetDefaultFontObjects().AvaDefaultFontObject.Get();
}

void FAvaFont::RefreshName()
{
	if (MotionDesignFontObject)
	{
		const FString& FontObjectName = MotionDesignFontObject->GetFontName();

		if (FontName != FontObjectName)
		{
			FontName = FontObjectName;
		}
	}
}

void FAvaFont::RefreshAssetState()
{
	if (!IsDefaultFont())
	{
		FontAssetState = EFontAssetState::SelectedFont;
	}
}

UFont* FAvaFont::GetFontByName(const FString& InFontName)
{
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	// no Roboto was found, this is not expected. Trying to load an avalanche font
	TArray<FAssetData> AssetDataArray;

	// no avalanche font available, trying to load the first engine font available
	if (AssetDataArray.IsEmpty())
	{
		const UClass* Class = UFont::StaticClass();
		const FTopLevelAssetPath AssetPath(Class->GetPathName());
		AssetRegistryModule.Get().GetAssetsByClass(AssetPath, AssetDataArray);
	}

	for (const FAssetData& AssetData : AssetDataArray)
	{
		if (UFont* const CurrFont = Cast<UFont>(AssetData.GetAsset()))
		{
			FString CurrFontName;
			UE::Ava::FontUtilities::GetFontName(CurrFont, CurrFontName);

			if (CurrFontName == InFontName)
			{
				return CurrFont;
			}
		}
	}

	return nullptr;
}

// note: this function used to be in the AvalancheEditor module, since it was not needed at Runtime before 
void UE::Ava::FontUtilities::GetFontName(const UFont* InFont, FString& OutFontName)
{
	if (IsValid(InFont))
	{
		FString FontAssetName;
		FString FontImportName = InFont->ImportOptions.FontName;

		if (InFont->GetFName() == NAME_None)
		{
			FontAssetName = InFont->LegacyFontName.ToString();
		}
		else
		{
			FontAssetName = InFont->GetName();
		}

		// Roboto fonts are actually from the Arial family and their import name is "Arial", so we try to list them as well
		// this will likely lead to missing spaces in their names
		if (FontAssetName.Contains(TEXT("Roboto")) || FontImportName == TEXT("Arial"))
		{
			OutFontName = FontAssetName;
		}
		else
		{
			OutFontName = FontImportName;
		}
	}
}

#undef LOCTEXT_NAMESPACE
