// Copyright Epic Games, Inc. All Rights Reserved.

#include "Font/AvaFontObject.h"
#include "Engine/Font.h"
#include "Misc/Paths.h"
#include "UObject/Package.h"

static TMap<int32, FString> CharsetMap
{
	{0,   TEXT("")},              // ANSI_CHARSET
	{1,   TEXT("")},              // DEFAULT_CHARSET
	{2,   TEXT("Symbol")},        // SYMBOL_CHARSET
	{128, TEXT("ShiftJIS")},      // SHIFTJIS_CHARSET
	{129, TEXT("한글")},           // HANGEUL_CHARSET
	{129, TEXT("한글")},           // HANGUL_CHARSET
	{134, TEXT("GB2312")},        // GB2312_CHARSET
	{136, TEXT("ChineseBig5")},   // CHINESEBIG5_CHARSET
	{255, TEXT("Oem")},           // OEM_CHARSET
	{130, TEXT("Johab")},         // JOHAB_CHARSET
	{177, TEXT("אָלֶף־בֵּית עִבְרִי")},  // HEBREW_CHARSET
	{178, TEXT("العربية")},       // ARABIC_CHARSET
	{161, TEXT("αλφάβητο")},      // GREEK_CHARSET
	{162, TEXT("")},              // TURKISH_CHARSET
	{163, TEXT("chữ Quốc ngữ")},  // VIETNAMESE_CHARSET
	{222, TEXT("ักษรไทย")},        // THAI_CHARSET
	{238, TEXT("EastEurope")},    // EASTEUROPE_CHARSET
	{204, TEXT("Russian")},       // RUSSIAN_CHARSET
	{77,  TEXT("Mac")},           // MAC_CHARSET
	{186, TEXT("Baltic")},        // BALTIC_CHARSET
};

void UAvaFontObject::InitProjectFont(UFont* InFont, const FString& InFontName)
{
	Font = InFont;
	FontName = InFontName;

	Source = EAvaFontSource::Project;
}

void UAvaFontObject::InitSystemFont(const FSystemFontsRetrieveParams& FontParams, UFont* InFont)
{
	Font = InFont;
	FontName = FontParams.FontFamilyName;
	Source = EAvaFontSource::System;
	FontParameters = FontParams;
}

void UAvaFontObject::Invalidate()
{
	Font = nullptr;
	FontName = TEXT("");
	Source = EAvaFontSource::Invalid;
}

void UAvaFontObject::SwitchToProjectFont()
{
	InitProjectFont(GetFont(), GetFontName());
}

void UAvaFontObject::InitFromFontObject(const UAvaFontObject* Other)
{
	if (IsValid(Other))
	{
		if (FontName != Other->FontName)
		{
			Font = Other->Font;
			FontName = Other->FontName;
			Source = Other->GetSource();
			Metrics = Other->GetMetrics();
		}
	}
}

bool UAvaFontObject::HasValidFont() const
{
	if (!Font)
	{
		return false;
	}

	const FCompositeFont* CompositeFont = Font->GetCompositeFont();
	if (!CompositeFont)
	{
		return false;
	}

	if (!CompositeFont->DefaultTypeface.Fonts.IsEmpty())
	{
		if (CompositeFont->DefaultTypeface.Fonts[0].Font.GetFontFaceAsset())
		{
			return true;
		}
	}

	return false;
}

FString UAvaFontObject::GetAlternateText() const
{
	if (Metrics.Charsets.Num())
	{
		const int CharsetToRetrieve = Metrics.Charsets.Num() > 1 ? 1 : 0;

		if (CharsetMap.Contains(CharsetToRetrieve))
		{
			return CharsetMap[Metrics.Charsets[CharsetToRetrieve]];
		}
	}

	return TEXT("");
}

bool UAvaFontObject::IsMonospaced() const
{
	return Metrics.bIsMonospaced;
}

bool UAvaFontObject::IsBold() const
{
	return Metrics.bIsBold;
}

bool UAvaFontObject::IsItalic() const
{
	return Metrics.bIsItalic;
}
