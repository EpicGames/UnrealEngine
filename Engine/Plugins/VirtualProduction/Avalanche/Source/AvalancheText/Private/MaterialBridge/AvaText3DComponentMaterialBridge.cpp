// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialBridge/AvaText3DComponentMaterialBridge.h"
#include "AvaDataView.h"
#include "Extensions/Text3DMaterialExtensionBase.h"
#include "MaterialBridge/AvaMaterialBridgeRegistry.h"
#include "MaterialBridge/Context/AvaMaterialBridgeApplyStateContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeReadSlotContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeStoreStateContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeWriteSlotContext.h"
#include "Text3DComponent.h"

namespace UE::Ava
{

const UStruct* FText3DComponentMaterialBridge::OnGetBridgedType() const
{
	return UText3DComponent::StaticClass();
}

EControlFlow FText3DComponentMaterialBridge::OnAccessSlots(const FReadSlotContext& InContext, TFunctionRef<EControlFlow(const FReadSlotContext&, const FReadSlot&)> InFunc, const FReadSlotOptions& InOptions) const
{
	const UText3DComponent& Component = InContext.MaterialContainer.Get<const UText3DComponent>();

	if (UText3DMaterialExtensionBase* MaterialExtension = Component.GetMaterialExtension())
	{
		if (const FMaterialBridge* MaterialBridge = InOptions.MaterialBridgeRegistry->GetMaterialBridge(FConstDataView(MaterialExtension)))
		{
			return MaterialBridge->AccessSlots(FReadSlotContext(MaterialExtension, &InContext), InFunc, InOptions);
		}
	}

	return EControlFlow::Continue;
}

EControlFlow FText3DComponentMaterialBridge::OnAccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const
{
	const UText3DComponent& Component = InContext.MaterialContainer.Get<const UText3DComponent>();

	if (UText3DMaterialExtensionBase* MaterialExtension = Component.GetMaterialExtension())
	{
		if (const FMaterialBridge* MaterialBridge = InOptions.MaterialBridgeRegistry->GetMaterialBridge(FConstDataView(MaterialExtension)))
		{
			return MaterialBridge->AccessSlots(FWriteSlotContext(MaterialExtension, &InContext), InFunc, InOptions);
		}
	}

	return EControlFlow::Continue;
}

TSubScriptStructOf<FAvaMaterialContainerState> FText3DComponentMaterialBridge::OnGetContainerStateType() const
{
	return FContainerState::StaticStruct();
}

void FText3DComponentMaterialBridge::OnApplyState(const FApplyStateContext& InContext, TConstStructView<FAvaMaterialContainerState> InContainerState, const FApplyStateOptions& InOptions) const
{
	UText3DComponent& Component = InContext.MaterialContainer.GetMutable<UText3DComponent>();
	const FContainerState& ContainerState = InContainerState.Get<FContainerState>();

	if (UText3DMaterialExtensionBase* MaterialExtension = Component.GetMaterialExtension())
	{
		if (const FMaterialBridge* MaterialBridge = InOptions.MaterialBridgeRegistry->GetMaterialBridge(FConstDataView(MaterialExtension)))
		{
			FApplyStateContext ApplyContext(MaterialExtension, &InContext);
			MaterialBridge->ApplyState(ApplyContext, ContainerState.MaterialExtensionStateData, InOptions);
		}
	}
}

void FText3DComponentMaterialBridge::OnStoreState(const FStoreStateContext& InContext, TStructView<FAvaMaterialContainerState> InContainerState, const FStoreStateOptions& InOptions) const
{
	const UText3DComponent& Component = InContext.MaterialContainer.Get<const UText3DComponent>();
	FContainerState& StateData = InContainerState.Get<FContainerState>();

	if (UText3DMaterialExtensionBase* MaterialExtension = Component.GetMaterialExtension())
	{
		if (const FMaterialBridge* MaterialBridge = InOptions.MaterialBridgeRegistry->GetMaterialBridge(FConstDataView(MaterialExtension)))
		{
			FStoreStateContext StoreContext(MaterialExtension, &InContext);
			MaterialBridge->StoreState(StoreContext, &StateData.MaterialExtensionStateData, InOptions);
		}
	}
}

} // UE::Ava
