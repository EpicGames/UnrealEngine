// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialBridge/AvaText3DDefaultExtensionMaterialBridge.h"
#include "Extensions/Text3DDefaultMaterialExtension.h"
#include "MaterialBridge/Context/AvaMaterialBridgeApplyStateContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeStoreStateContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeWriteSlotContext.h"
#include "MaterialBridge/Slot/CommonFeatures/AvaMaterialBridgeBlendModeFeature.h"
#include "Text3DComponent.h"

namespace UE::Ava
{

namespace Private
{
// Gets the target Text3D blend mode for a given input blend mode
EText3DMaterialBlendMode GetText3DTargetBlendMode(EBlendMode InBlendMode)
{
	switch (InBlendMode)
	{
	case BLEND_Masked:
	case BLEND_Translucent:
		return EText3DMaterialBlendMode::Translucent;

	default:
		return EText3DMaterialBlendMode::Opaque;
	}
}

// Sets the material extension's blend mode and updating Text3D Component immediately to update any material properties
// returns true if the blend mode was changed, or false if the blend mode was already the requested one.
bool SetText3DBlendMode(TNotNull<UText3DDefaultMaterialExtension*> InMaterialExtension, EText3DMaterialBlendMode InBlendMode)
{
	if (InMaterialExtension->GetBlendMode() != InBlendMode)
	{
		InMaterialExtension->SetBlendMode(InBlendMode);

		// Request immediate material update so that the material changes due to blend mode can take effect.
		if (UText3DComponent* Text3DComponent = InMaterialExtension->GetText3DComponent())
		{
			Text3DComponent->RequestUpdate(EText3DRendererFlags::Material, /*bInImmediate*/true);
		}
		return true;
	}
	return false;
}
	
} // UE::Ava::Private

bool FText3DDefaultMaterialWriteSlot::OnFeatureRequest(TConstStructView<FAvaMaterialBridgeFeature> InFeature)
{
	UText3DDefaultMaterialExtension* MaterialExtension = Cast<UText3DDefaultMaterialExtension>(BaseMaterialExtension);
	if (!ensure(MaterialExtension))
	{
		return false;
	}

	// Blend mode feature
	if (const FAvaMaterialBridgeBlendModeFeature* BlendModeFeature = InFeature.GetPtr<FAvaMaterialBridgeBlendModeFeature>())
	{
		if (Private::SetText3DBlendMode(MaterialExtension, Private::GetText3DTargetBlendMode(BlendModeFeature->BlendMode)))
		{
			SetMaterialInternal(MaterialExtension->GetMaterial(MaterialParameters));
		}
		return true;
	}

	return false;
}

const UStruct* FText3DDefaultExtensionMaterialBridge::OnGetBridgedType() const
{
	return UText3DDefaultMaterialExtension::StaticClass();
}

TSubScriptStructOf<FAvaMaterialContainerState> FText3DDefaultExtensionMaterialBridge::OnGetContainerStateType() const
{
	return FContainerState::StaticStruct();
}

EControlFlow FText3DDefaultExtensionMaterialBridge::OnAccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const
{
	UText3DDefaultMaterialExtension& MaterialExtension = InContext.MaterialContainer.GetMutable<UText3DDefaultMaterialExtension>();

	EControlFlow ControlFlow = EControlFlow::Continue;

	MaterialExtension.ForEachMaterial(
		[&InFunc, &InContext, &ControlFlow, &MaterialExtension](const UE::Text3D::Material::FMaterialParameters& InParameters, UMaterialInterface* InMaterial)->bool
		{
			FText3DDefaultMaterialWriteSlot Slot(&MaterialExtension, InParameters);
			ControlFlow = InFunc(InContext, Slot);
			return ControlFlow == EControlFlow::Continue;
		});

	return ControlFlow;
}

void FText3DDefaultExtensionMaterialBridge::OnApplyState(const FApplyStateContext& InContext, TConstStructView<FAvaMaterialContainerState> InContainerState, const FApplyStateOptions& InOptions) const
{
	UText3DDefaultMaterialExtension& MaterialExtension = InContext.MaterialContainer.GetMutable<UText3DDefaultMaterialExtension>();
	const FContainerState& ContainerState = InContainerState.Get<FContainerState>();

	Private::SetText3DBlendMode(&MaterialExtension, ContainerState.BlendMode);

	// Call parent implementation last to apply the materials after the blend mode update
	Super::OnApplyState(InContext, InContainerState, InOptions);
}

void FText3DDefaultExtensionMaterialBridge::OnStoreState(const FStoreStateContext& InContext, TStructView<FAvaMaterialContainerState> InContainerState, const FStoreStateOptions& InOptions) const
{
	// Call parent implementation to store the materials
	Super::OnStoreState(InContext, InContainerState, InOptions);

	const UText3DDefaultMaterialExtension& MaterialExtension = InContext.MaterialContainer.Get<const UText3DDefaultMaterialExtension>();

	FContainerState& ContainerState = InContainerState.Get<FContainerState>();
	ContainerState.BlendMode = MaterialExtension.GetBlendMode();
}

} // UE::Ava
