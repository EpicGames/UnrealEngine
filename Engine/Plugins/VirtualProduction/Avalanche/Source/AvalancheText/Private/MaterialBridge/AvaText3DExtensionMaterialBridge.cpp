// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialBridge/AvaText3DExtensionMaterialBridge.h"
#include "Extensions/Text3DMaterialExtensionBase.h"
#include "MaterialBridge/Context/AvaMaterialBridgeApplyStateContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeReadSlotContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeStoreStateContext.h"
#include "MaterialBridge/Context/AvaMaterialBridgeWriteSlotContext.h"
#include "MaterialBridge/Slot/AvaMaterialBridgeReadSlot.h"
#include "Text3DComponent.h"

namespace UE::Ava
{

namespace Private
{

// Creates a Slot Id out of the Material Parameters
FAvaMaterialBridgeSlotId CreateSlotId(const Text3D::Material::FMaterialParameters& InParameters)
{
	const int32 GroupIndex = static_cast<int32>(InParameters.Group);
	return FAvaMaterialBridgeSlotId(InParameters.Tag, GroupIndex);
}

} // UE::Ava::Private

FText3DMaterialWriteSlot::FText3DMaterialWriteSlot(TNotNull<UText3DMaterialExtensionBase*> InMaterialExtension, const UE::Text3D::Material::FMaterialParameters& InMaterialParameters)
	: FMaterialBridgeWriteSlot(InMaterialExtension->GetMaterial(InMaterialParameters), Private::CreateSlotId(InMaterialParameters))
	, BaseMaterialExtension(InMaterialExtension)
	, MaterialParameters(InMaterialParameters)
{
}

void FText3DMaterialWriteSlot::OnMaterialChanged()
{
	BaseMaterialExtension->SetMaterial(MaterialParameters, GetMaterial());
}

const UStruct* FText3DExtensionMaterialBridge::OnGetBridgedType() const
{
	return UText3DMaterialExtensionBase::StaticClass();
}

EControlFlow FText3DExtensionMaterialBridge::OnAccessSlots(const FReadSlotContext& InContext, TFunctionRef<EControlFlow(const FReadSlotContext&, const FReadSlot&)> InFunc, const FReadSlotOptions& InOptions) const
{
	const UText3DMaterialExtensionBase& Extension = InContext.MaterialContainer.Get<const UText3DMaterialExtensionBase>();

	EControlFlow ControlFlow = EControlFlow::Continue;

	Extension.ForEachMaterial(
		[&InFunc, &InContext, &ControlFlow](const UE::Text3D::Material::FMaterialParameters& InMaterialParameters, UMaterialInterface* InMaterial)->bool
		{
			const FReadSlot Slot(InMaterial, Private::CreateSlotId(InMaterialParameters));
			ControlFlow = InFunc(InContext, Slot);
			return ControlFlow == EControlFlow::Continue;
		});

	return ControlFlow;
}

EControlFlow FText3DExtensionMaterialBridge::OnAccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const
{
	UText3DMaterialExtensionBase& MaterialExtension = InContext.MaterialContainer.GetMutable<UText3DMaterialExtensionBase>();

	EControlFlow ControlFlow = EControlFlow::Continue;

	MaterialExtension.ForEachMaterial(
		[&InFunc, &InContext, &ControlFlow, &MaterialExtension](const UE::Text3D::Material::FMaterialParameters& InParameters, UMaterialInterface* InMaterial)->bool
		{
			FText3DMaterialWriteSlot Slot(&MaterialExtension, InParameters);
			ControlFlow = InFunc(InContext, Slot);
			return ControlFlow == EControlFlow::Continue;
		});

	return ControlFlow;
}

TSubScriptStructOf<FAvaMaterialContainerState> FText3DExtensionMaterialBridge::OnGetContainerStateType() const
{
	return FContainerState::StaticStruct();
}

void FText3DExtensionMaterialBridge::OnApplyState(const FApplyStateContext& InContext, TConstStructView<FAvaMaterialContainerState> InContainerState, const FApplyStateOptions& InOptions) const
{
	UText3DMaterialExtensionBase& MaterialExtension = InContext.MaterialContainer.GetMutable<UText3DMaterialExtensionBase>();
	const FContainerState& ContainerState = InContainerState.Get<const FContainerState>();

	for (const FAvaText3DExtensionMaterialData& MaterialData : ContainerState.MaterialData)
	{
		UE::Text3D::Material::FMaterialParameters Parameters;
		Parameters.Tag = MaterialData.Tag;
		Parameters.Group = MaterialData.Group;

		MaterialExtension.SetMaterial(Parameters, MaterialData.Material);
	}
}

void FText3DExtensionMaterialBridge::OnStoreState(const FStoreStateContext& InContext, TStructView<FAvaMaterialContainerState> InContainerState, const FStoreStateOptions& InOptions) const
{
	const UText3DMaterialExtensionBase& MaterialExtension = InContext.MaterialContainer.Get<const UText3DMaterialExtensionBase>();
	FContainerState& ContainerState = InContainerState.Get<FContainerState>();

	// Ensure Text3D is up-to-date with the material before storing state
	if (UText3DComponent* Text3DComponent = MaterialExtension.GetText3DComponent())
	{
		Text3DComponent->RequestUpdate(EText3DRendererFlags::Material, /*bInImmediate*/true);
	}

	const int32 MaterialCount = MaterialExtension.GetMaterialCount();
	ContainerState.MaterialData.SetNum(MaterialCount);

	int32 Index = 0;

	MaterialExtension.ForEachMaterial(
		[&ContainerState, &Index, &InContext, &InOptions](const UE::Text3D::Material::FMaterialParameters& InParameters, UMaterialInterface* InMaterial)->bool
		{
			FAvaText3DExtensionMaterialData& MaterialData = ContainerState.MaterialData[Index++];
			MaterialData.Tag = InParameters.Tag;
			MaterialData.Group = InParameters.Group;
			MaterialData.Material = InMaterial;
			return true; // continue;
		});
}

} // UE::Ava
