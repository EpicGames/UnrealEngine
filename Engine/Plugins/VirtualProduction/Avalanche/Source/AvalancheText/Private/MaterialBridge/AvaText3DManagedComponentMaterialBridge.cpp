// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialBridge/AvaText3DManagedComponentMaterialBridge.h"
#include "Components/MeshComponent.h"
#include "Renderers/Text3DRendererBase.h"

namespace UE::Ava
{

const UStruct* FText3DManagedComponentMaterialBridge::OnGetBridgedType() const
{
	return UMeshComponent::StaticClass();
}

bool FText3DManagedComponentMaterialBridge::OnIsMaterialContainerSupported(FConstDataView InMaterialContainer) const
{
	const UMeshComponent& Component = InMaterialContainer.Get<const UMeshComponent>();

	// NOTE: The mesh components get created with renderer extension as outer. 
	// These get attached to the Text3D component directly, but there are other components that could potentially be attached to the Text3D component.
	return !!Cast<UText3DRendererBase>(Component.GetOuter());
}

} // UE::Ava
