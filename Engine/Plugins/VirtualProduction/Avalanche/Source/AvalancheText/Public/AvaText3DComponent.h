// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTextDefs.h"
#include "Font/AvaFont.h"
#include "Text3DComponent.h"
#include "AvaText3DComponent.generated.h"

UCLASS(MinimalAPI
	, ClassGroup="Text3D"
	, PrioritizeCategories="Text Character Layout Geometry Material Rendering Transform"
	, HideCategories=(Replication,Collision,HLOD,Physics,Networking,Input,Actor,Cooking,LevelInstance,Streaming,DataLayers,WorldPartition))
class UAvaText3DComponent : public UText3DComponent
{
	friend class FAvaTextVisualizer;
	friend class FAvaToolboxTextVisualizer;
	friend class FAvaTextComponentCustomization;

	GENERATED_BODY()

public:
	UAvaText3DComponent();

	//~ Begin UText3DComponent
	virtual void Serialize(FArchive& InArchive) override;
	//~ End UText3DComponent

private:
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.6, "Use SetFont instead")
	UPROPERTY()
	FAvaFont MotionDesignFont_DEPRECATED;

	UE_DEPRECATED(5.6, "Use Material Extension instead")
	UPROPERTY()
	EAvaTextColoringStyle ColoringStyle_DEPRECATED = EAvaTextColoringStyle::Solid;

	UE_DEPRECATED(5.6, "Use Material Extension instead")
	UPROPERTY()
	FLinearColor Color_DEPRECATED = FLinearColor::White;

	UE_DEPRECATED(5.6, "Use Material Extension instead")
	UPROPERTY()
	FLinearColor ExtrudeColor_DEPRECATED = FLinearColor::Gray;

	UE_DEPRECATED(5.6, "Use Material Extension instead")
	UPROPERTY()
	FLinearColor BevelColor_DEPRECATED = FLinearColor::Gray;

	UE_DEPRECATED(5.6, "Use Material Extension instead")
	UPROPERTY()
	FAvaLinearGradientSettings GradientSettings_DEPRECATED;

	UE_DEPRECATED(5.6, "Use Material Extension instead")
	UPROPERTY()
	TObjectPtr<UTexture2D> MainTexture_DEPRECATED;

	UE_DEPRECATED(5.6, "Use Material Extension instead")
	UPROPERTY()
	FVector2D Tiling_DEPRECATED = FVector2D::One();

	UE_DEPRECATED(5.6, "Use Material Extension instead")
	UPROPERTY()
	EAvaTextTranslucency TranslucencyStyle_DEPRECATED = EAvaTextTranslucency::None;

	UE_DEPRECATED(5.6, "Use Material Extension instead")
	UPROPERTY()
	float Opacity_DEPRECATED = 1.f;

	UE_DEPRECATED(5.6, "Use Material Extension instead")
	UPROPERTY()
	EAvaMaterialMaskOrientation MaskOrientation_DEPRECATED = EAvaMaterialMaskOrientation::LeftRight;

	UE_DEPRECATED(5.6, "Use Material Extension instead")
	UPROPERTY()
	float MaskSmoothness_DEPRECATED = 0.1f;

	UE_DEPRECATED(5.6, "Use Material Extension instead")
	UPROPERTY()
	float MaskOffset_DEPRECATED = 0.5f;

	UE_DEPRECATED(5.6, "Use Material Extension instead")
	UPROPERTY()
    float MaskRotation_DEPRECATED = 0.0f;

	UE_DEPRECATED(5.6, "Use Material Extension instead")
	UPROPERTY()
	bool bIsUnlit_DEPRECATED = true;
#endif
};
