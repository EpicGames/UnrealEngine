// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaText3DComponent.h"
#include "GameFramework/Actor.h"
#include "AvaTextActor.generated.h"

class UText3DCharacterTransform;
struct FAvaColorChangeData;

/**
 * This actor is getting replaced by AText3DActor, do not use anymore
 */
UCLASS(MinimalAPI, NotPlaceable, ClassGroup = (Text3D), DisplayName = "Motion Design Text", meta = (ComponentWrapperClass))
class AAvaTextActor : public AActor
{
	GENERATED_BODY()

public:
	AAvaTextActor();

	UText3DComponent* GetText3DComponent() const
	{
		return Text3DComponent;
	}

	AVALANCHETEXT_API FAvaColorChangeData GetColorData() const;
	AVALANCHETEXT_API void SetColorData(const FAvaColorChangeData& InColorData);

	//~ Begin AActor
#if WITH_EDITOR
	virtual FString GetDefaultActorLabel() const override;
#endif
	//~ End AActor

private:
	UPROPERTY(Category = "Text", VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UText3DComponent> Text3DComponent;

	UE_DEPRECATED(5.6, "Use Transform Extension instead")
	UPROPERTY()
	TObjectPtr<UText3DCharacterTransform> Text3DCharacterTransform;
};
