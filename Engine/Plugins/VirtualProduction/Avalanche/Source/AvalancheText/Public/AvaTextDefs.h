// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Text3DComponent.h"
#include "AvaTextDefs.generated.h"

UENUM()
enum class EAvaTextColoringStyle : uint8 
{
	Invalid        UMETA(Hidden),
	Solid,
	Gradient,
	FromTexture,
	CustomMaterial
};

UENUM()
enum class EAvaTextTranslucency : uint8 
{
	Invalid        UMETA(Hidden),
	None           UMETA(DisplayName="Opaque"),
	Translucent,
	GradientMask,
};

UENUM()
enum class EAvaMaterialMaskOrientation : uint8 
{
	LeftRight UMETA(DisplayName = "Left to Right"),
	RightLeft UMETA(DisplayName = "Right to Left"),
	Custom
};

UENUM()
enum class EAvaGradientDirection : uint8 
{
	None        UMETA(Hidden),
	Vertical,
	Horizontal,
	Custom,
};

USTRUCT(BlueprintType)
struct FAvaLinearGradientSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design")
	EAvaGradientDirection Direction = EAvaGradientDirection::Vertical;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design")
	FLinearColor ColorA = FLinearColor::White;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design")
	FLinearColor ColorB = FLinearColor::Black;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design", meta = (ClampMin = 0.0, ClampMax = 1.0))
	float Smoothness = 0.1f;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design", meta = (ClampMin = 0.0, ClampMax = 1.0))
	float Offset = 0.5f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Motion Design", meta = (ClampMin = 0.0, ClampMax = 1.0))
	float Rotation = 0.0f;

	bool operator==(const FAvaLinearGradientSettings& Other) const
	{
		return
			Direction == Other.Direction &&
			ColorA == Other.ColorA &&
			ColorB == Other.ColorB &&
			Smoothness == Other.Smoothness &&
			Offset == Other.Offset &&
			Rotation == Other.Rotation;
	}
};
