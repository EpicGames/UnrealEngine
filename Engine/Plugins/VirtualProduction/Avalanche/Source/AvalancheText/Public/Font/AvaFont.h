// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaFontObject.h"
#include "Engine/Font.h"
#include "UObject/StrongObjectPtr.h"

#include "AvaFont.generated.h"

USTRUCT()
struct FAvaFont
{
	friend class UAvaText3DComponent;

	GENERATED_BODY()

	/**
	 * Returns a default font (Roboto). The font will be loaded from asset if needed.
	 * In the unlikely case in which Roboto is not available, the first available font will be returned
	 */
	AVALANCHETEXT_API static UFont* GetDefaultFont();

	/**
	 * Returns a formatted string representing a FAvaFont for the specified FontObjectPathName and FontName
	 * Useful e.g. when setting the value of multiple FAvaFonts using IPropertyHandle->SetPerObjectValues
	 * @param InFontName {The name of the font handled by UAvaFontObject}
	 * @param InFontObjectPathName {The path of the UAvaFontObject}
	 * @return {the formatted string}
	 */
	static FString GenerateFontFormattedString(const FString& InFontName, const FString& InFontObjectPathName);

	/**
	 * Returns a formatted string representing a FAvaFont for the specified UAvaFontObject
	 * Useful e.g. when setting the value of multiple FAvaFonts using IPropertyHandle->SetPerObjectValues
	 * @param InFontObject {The UAvaFontObject used to generated the FAvaFont formatted string}
	 * @param OutFormattedString {will contain the formatted string}
	 * @return {true if successful}
	 */
	AVALANCHETEXT_API static bool GenerateFontFormattedString(const UAvaFontObject* InFontObject, FString& OutFormattedString);

	/**
	 * Checks if the specified Ava Fonts are the same
	 * @param InFontA First font for comparison
	 * @param InFontB Second font for comparison
	 * @return true if Fonts are the same
	 */
	static bool AreSameFont(const FAvaFont* InFontA, const FAvaFont* InFontB);

	/** Default constructor: calling GetFont() right after the constructor will return the default font */
	AVALANCHETEXT_API FAvaFont();

	/** this FAvaFont will be initialized using the provided UAvaFontObject */
	AVALANCHETEXT_API FAvaFont(UAvaFontObject* InFontObject);

	/** Returns the currently stored font, or default font if none available. Preferred getter to just get the UFont and use it */
	UFont* GetFont();

	/** Returns current font name, as FName */
	FName GetFontName() const;

	/** Returns current font name as FString */
	AVALANCHETEXT_API FString GetFontNameAsString() const;

	/** Returns true if this FAvaFont is marked as favorite by the user */
	AVALANCHETEXT_API bool IsFavorite() const;

	/**
	 * Checks if this FAvaFont is referencing the default font. This will be happening also in fallback state.
	 * Use IsFallBackFont() if you explicitly need to know if this font is default because of a missing asset
	 * @return true if referenced font is the default one
	 */
	bool IsDefaultFont() const;

	/**
	 * Checks if this FAvaFont is in fallback state.
	 * This happens when the font referenced by this FAvaFont is not available after de-serialization. This likely means that a UFont asset is missing.
	 * In fallback state, a FAvaFont will return the default font when GetFont() is called.
	 * @return true if this FAvaFont is in fallback state
	 */
	AVALANCHETEXT_API bool IsFallbackFont() const;

	/** Returns true if font is monospaced */
	bool IsMonospaced() const;

	/** Returns true if font is bold */
	bool IsBold() const;

	/** Returns true if font is italic */
	bool IsItalic() const;

	/**
	 * Mark/unmark this font as favorite
	 * @param bFavorite {the flag to mark the font as favorite or not. True means favorite.}
	 */
	AVALANCHETEXT_API void SetFavorite(const bool bFavorite);

	/**
	 * Calling this function will null the now deprecated UFont CurrentFont reference, and use that UFont to initialize the currently used UAvaFontObject
	 */
	void EnsureUsingCurrentVersion();

	/**
	 * In case this FAvaFont is in fallback state, this call can return the name of the missing font asset, if available.
	 * @return the name of the missing font asset.
	 */
	const FString& GetMissingFontName() const { return MissingFontName; }

	/**
	 * Checks if the currently referenced font asset is valid. Also internally checks if there's a composite font available within the UFont.
	 * @return true if font is valid
	 */
	bool HasValidFont() const;

	/**
	 * Returns the currently stored UAvaFontObject. This is useful when dealing with font load/import.
	 * If only the current UFont is needed, use GetFont().
	 * @return Current font object. 
	 */
	UAvaFontObject* GetFontObject() const { return MotionDesignFontObject; }

	/**
	 * Directly sets the current UAvaFontObject, and then updates this FAvaFont accordingly.
	 * If this FAvaFont is from past versions, the internal deprecated CurrentFont field will be set to nullptr.
	 * @param InFontObject {The FontObject which will be assigned to this FAvaFont}
	 */
	void SetFontObject(UAvaFontObject* InFontObject);

	/* Returns current font source (System, Project or Invalid) */
	EAvaFontSource GetFontSource() const;

	/** Fonts are considered equal if they share the same font and font name */
	AVALANCHETEXT_API bool operator==(const FAvaFont& Other) const;

	bool operator!=(const FAvaFont& Other) const
	{
		return !(*this == Other);
	}

	/**
	 * Called after serialization of FAvaFont.
	 * In case of missing font assets, the FAvaFont state will be marked as "FallbackFont"
	 */
	AVALANCHETEXT_API void PostSerialize(const FArchive& Ar);

private:
	/** A struct to hold default values which we want to prevent from being GC'd */
	struct FAvaDefaultFontObjects : public FGCObject
	{
		TObjectPtr<UFont> AvaDefaultFont;
		TObjectPtr<UAvaFontObject> AvaDefaultFontObject;

		//~ Begin FGCObject
		virtual void AddReferencedObjects(FReferenceCollector& OutCollector) override
		{
			OutCollector.AddReferencedObject(AvaDefaultFont);
			OutCollector.AddReferencedObject(AvaDefaultFontObject);
		}
		virtual FString GetReferencerName() const override
		{
			return TEXT("FAvaDefaultFontObjects");
		}
		//~ End FGCObject
	};

	/**
	 * Used to handle font state and possible loading issues
	 */
	enum EFontAssetState
	{
		DefaultFont,   // Font is meant to be the default one
		SelectedFont,  // Font is not the default one and has been explicitly set, either by serialization or UI
		FallbackFont   // Referenced font asset is missing
	};

	/** Returns the default font object for Ava Fonts */
	static UAvaFontObject* GetDefaultAvaFontObject();

	/** Returns a static FAvaDefaultFontObjects struct holding defaults values for fonts */
	static FAvaDefaultFontObjects& GetDefaultFontObjects();

	/** Search for a font asset based on a font name */
	static UFont* GetFontByName(const FString& InFontName);

	/** Initializes default values */
	void InitDefaults();

	/** Ensures the Font Name property is up to date with current font object */
	void RefreshName();

	/** Refreshes asset state */
	void RefreshAssetState();

	/** Initialize this Avalanche Font using a UFont asset */
	void InitFromFont(UFont* InFont);

	/**
	 * Deprecated - used to reference the font used by FAvaFont.
	 * Older assets still have the font stored in this field. Calling GetFont() will move the font to the currently used MotionDesignFontObject property
	 */
	UPROPERTY(Transient, meta=(DeprecatedProperty, DeprecationMessage="MotionDesignFontObject is the property currently handling the font."))
	TObjectPtr<UFont> CurrentFont_DEPRECATED;

	/** Stores the UAvaFontObject holding all the information about the font currently used by FAvaFont */
	UPROPERTY()
	TObjectPtr<UAvaFontObject> MotionDesignFontObject;

	/** Updated when MotionDesignFontObject is set. Mainly used when loading FAvaFont, to help looking for a font in case the current font is null */
	UPROPERTY()
	FString FontName;

	/** The current font asset state, used internally to track the current state of FAvaFont with respect to its referenced font asset */
	EFontAssetState FontAssetState;

	/** FAvaFonts can be marked as favorites by the user */
	bool bIsFavorite;

	/** When a font is missing and its name is available, it will be stored here. Old assets version cannot retrieve the name of the missing font */
	FString MissingFontName;
};

/** We need to intercept FAvaFont serialization to detect if there's a missing font when loading */
template<> struct TStructOpsTypeTraits<FAvaFont> : public TStructOpsTypeTraitsBase2<FAvaFont>
{
	enum
	{
		/** void PostSerialize(const FArchive& Ar) in FAvaFont */
		WithPostSerialize = true 
	};
};

namespace UE::Ava::FontUtilities
{
	AVALANCHETEXT_API void GetFontName(const UFont* InFont, FString& OutFontName);
}
