// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Font.h"
#include "Engine/FontFace.h"
#include "UObject/Object.h"
#include "AvaFontObject.generated.h"

class UFont;
class UFontFace;

UENUM()
enum class EAvaFontSource : uint8 
{
	System = 0,
	Project = 1,
	Invalid = 2
};

struct FAvaSystemFontMetrics
{
	TArray<int32> Charsets; // this will become proper enum

	bool bIsMonospaced;

	bool bIsBold;

	bool bIsItalic;
};

struct FSystemFontsRetrieveParams
{
	FString FontFamilyName;

	void AddFontFace(const FString& InFontFaceName, const FString& InFontFacePath)
	{
		// Avoiding duplicates
		if (FontFaceNames.Contains(InFontFaceName) || GetFontFacePaths().Contains(InFontFacePath))
		{
			return;
		}

		FontFaceNames.Add(InFontFaceName);
		FontFacePaths.Add(InFontFacePath);
	}

	TConstArrayView<FString> GetFontFaceNames() const { return FontFaceNames; }
	TConstArrayView<FString> GetFontFacePaths() const { return FontFacePaths; }

private:
	TArray<FString> FontFaceNames;
	TArray<FString> FontFacePaths;
};

UCLASS()
class AVALANCHETEXT_API UAvaFontObject : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Motion Design")
	TObjectPtr<UFont> Font = nullptr;

private:
	EAvaFontSource Source = EAvaFontSource::System;

	FAvaSystemFontMetrics Metrics;

	FSystemFontsRetrieveParams FontParameters;

public:
	void InitProjectFont(UFont* InFont, const FString& InFontName);
	void InitSystemFont(const FSystemFontsRetrieveParams& FontParams, UFont* InFont);
	void Invalidate();
	void SwitchToProjectFont();
	void InitFromFontObject(const UAvaFontObject* Other);

	bool HasValidFont() const;

	const FString& GetFontName() const { return FontName; }

	TObjectPtr<UFont> GetFont() const { return Font; }

	FAvaSystemFontMetrics GetMetrics() const { return Metrics; }

	void SetMetrics(const FAvaSystemFontMetrics& InMetrics) { Metrics = InMetrics; }

	EAvaFontSource GetSource() const { return Source; }

	FString GetAlternateText() const;

	const FSystemFontsRetrieveParams& GetFontParameters() const { return FontParameters; }

	bool IsMonospaced() const;
	bool IsBold() const;
	bool IsItalic() const;

	TArray<const UFontFace*> GetFontFaces() const
	{
		TArray<const UFontFace*> OutFontFaces;

		for (const FTypefaceEntry& TypefaceEntry : Font->GetCompositeFont()->DefaultTypeface.Fonts)
		{
			if (const UFontFace* FontFace = Cast<UFontFace>(TypefaceEntry.Font.GetFontFaceAsset()))
			{
				OutFontFaces.Add(FontFace);
			}
		}

		return OutFontFaces;
	}

private:
	UPROPERTY()
	FString FontName;
};
