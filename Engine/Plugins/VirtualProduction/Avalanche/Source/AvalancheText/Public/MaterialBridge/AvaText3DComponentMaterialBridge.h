// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialBridge/AvaMaterialBridge.h"
#include "MaterialBridge/AvaMaterialContainerState.h"
#include "AvaText3DComponentMaterialBridge.generated.h"

#define UE_API AVALANCHETEXT_API

/** Material state data for text3d components */
USTRUCT()
struct FAvaText3DComponentMaterialStateData : public FAvaMaterialContainerState
{
	GENERATED_BODY()

	UPROPERTY()
	TInstancedStruct<FAvaMaterialContainerState> MaterialExtensionStateData;
};

namespace UE::Ava
{

/** Material Bridge for Text3D Components */
class FText3DComponentMaterialBridge : public FMaterialBridge
{
public:
	using FContainerState = FAvaText3DComponentMaterialStateData;

protected:
	//~ Begin FMaterialBridge
	UE_API virtual const UStruct* OnGetBridgedType() const override;
	UE_API virtual EControlFlow OnAccessSlots(const FReadSlotContext& InContext, TFunctionRef<EControlFlow(const FReadSlotContext&, const FReadSlot&)> InFunc, const FReadSlotOptions& InOptions) const override;
	UE_API virtual EControlFlow OnAccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const override;
	UE_API virtual TSubScriptStructOf<FAvaMaterialContainerState> OnGetContainerStateType() const override;
	UE_API virtual void OnApplyState(const FApplyStateContext& InContext, TConstStructView<FAvaMaterialContainerState> InContainerState, const FApplyStateOptions& InOptions) const override;
	UE_API virtual void OnStoreState(const FStoreStateContext& InContext, TStructView<FAvaMaterialContainerState> InContainerState, const FStoreStateOptions& InOptions) const override;
	//~ End FMaterialBridge
};

} // UE::Ava

#undef UE_API
