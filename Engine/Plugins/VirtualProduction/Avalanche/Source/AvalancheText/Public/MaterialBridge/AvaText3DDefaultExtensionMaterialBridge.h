// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaText3DExtensionMaterialBridge.h"
#include "AvaText3DDefaultExtensionMaterialBridge.generated.h"

#define UE_API AVALANCHETEXT_API

/** Material state data struct for the Default Text3D Material Extension */
USTRUCT()
struct FAvaText3DDefaultExtensionMaterialContainerState : public FAvaText3DExtensionMaterialContainerState
{
	GENERATED_BODY()

	UPROPERTY()
	EText3DMaterialBlendMode BlendMode = EText3DMaterialBlendMode::Invalid;
};

namespace UE::Ava
{

/** Represents a slot for the Text3D default material extension for write */
class FText3DDefaultMaterialWriteSlot : public FText3DMaterialWriteSlot
{
public:
	using UE::Ava::FText3DMaterialWriteSlot::FText3DMaterialWriteSlot;

protected:
	//~ Begin FMaterialBridgeSlot
	UE_API virtual bool OnFeatureRequest(TConstStructView<FAvaMaterialBridgeFeature> InFeature) override;
	//~ End FMaterialBridgeSlot
};

/** Material Bridge for the Default Text3D Material Extension */
class FText3DDefaultExtensionMaterialBridge : public FText3DExtensionMaterialBridge
{
public:
	using Super = FText3DExtensionMaterialBridge;
	using FContainerState = FAvaText3DDefaultExtensionMaterialContainerState;

protected:
	//~ Begin FMaterialBridge
	UE_API virtual const UStruct* OnGetBridgedType() const override;
	UE_API virtual TSubScriptStructOf<FAvaMaterialContainerState> OnGetContainerStateType() const override;
	UE_API virtual EControlFlow OnAccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const override;
	UE_API virtual void OnApplyState(const FApplyStateContext& InContext, TConstStructView<FAvaMaterialContainerState> InContainerState, const FApplyStateOptions& InOptions) const override;
	UE_API virtual void OnStoreState(const FStoreStateContext& InContext, TStructView<FAvaMaterialContainerState> InContainerState, const FStoreStateOptions& InOptions) const override;
	//~ End FMaterialBridge
};

} // UE::Ava

#undef UE_API
