// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialBridge/AvaMaterialBridge.h"
#include "MaterialBridge/AvaMaterialContainerState.h"
#include "MaterialBridge/Slot/AvaMaterialBridgeWriteSlot.h"
#include "Text3DTypes.h"
#include "AvaText3DExtensionMaterialBridge.generated.h"

#define UE_API AVALANCHETEXT_API

class UText3DMaterialExtensionBase;

/** State data for a material in Text3D material extensions */
USTRUCT()
struct FAvaText3DExtensionMaterialData
{
	GENERATED_BODY()

	/** Tag used to identify a material section, none = default */
	UPROPERTY()
	FName Tag = NAME_None;

	/** Group used to identify an entry in the material section */
	UPROPERTY()
	EText3DGroupType Group = EText3DGroupType::Front;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> Material;
};

/** Base material container state struct for Text3D material extensions */
USTRUCT()
struct FAvaText3DExtensionMaterialContainerState : public FAvaMaterialContainerState
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FAvaText3DExtensionMaterialData> MaterialData;
};

namespace UE::Ava
{

/** Represents a slot in Text3D material extension when accessing the slot with write access */
class FText3DMaterialWriteSlot : public FMaterialBridgeWriteSlot
{
public:
	UE_API explicit FText3DMaterialWriteSlot(TNotNull<UText3DMaterialExtensionBase*> InMaterialExtension, const UE::Text3D::Material::FMaterialParameters& InMaterialParameters);

protected:
	//~ Begin FMaterialBridgeSlot
	UE_API virtual void OnMaterialChanged() override;
	//~ End FMaterialBridgeSlot

	/** The material extension that owns the slot */
	TNotNull<TObjectPtr<UText3DMaterialExtensionBase>> BaseMaterialExtension;

	/** The material parameters identifying the slot */
	UE::Text3D::Material::FMaterialParameters MaterialParameters;
};

/** Base Material Bridge for Text3D material extensions */
class FText3DExtensionMaterialBridge : public FMaterialBridge
{
public:
	using FContainerState = FAvaText3DExtensionMaterialContainerState;

protected:
	//~ Begin FMaterialBridge
	UE_API virtual const UStruct* OnGetBridgedType() const override;
	UE_API virtual EControlFlow OnAccessSlots(const FReadSlotContext& InContext, TFunctionRef<EControlFlow(const FReadSlotContext&, const FReadSlot&)> InFunc, const FReadSlotOptions& InOptions) const override;
	UE_API virtual EControlFlow OnAccessSlots(const FWriteSlotContext& InContext, TFunctionRef<EControlFlow(const FWriteSlotContext&, FWriteSlot&)> InFunc, const FWriteSlotOptions& InOptions) const override;
	UE_API virtual TSubScriptStructOf<FAvaMaterialContainerState> OnGetContainerStateType() const override;
	UE_API virtual void OnApplyState(const FApplyStateContext& InContext, TConstStructView<FAvaMaterialContainerState> InContainerState, const FApplyStateOptions& InOptions) const override;
	UE_API virtual void OnStoreState(const FStoreStateContext& InContext, TStructView<FAvaMaterialContainerState> InContainerState, const FStoreStateOptions& InOptions) const override;
	//~ End FMaterialBridge

};

} // UE::Ava

#undef UE_API
