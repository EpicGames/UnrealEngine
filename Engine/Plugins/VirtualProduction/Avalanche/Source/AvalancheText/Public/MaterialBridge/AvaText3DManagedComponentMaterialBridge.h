// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MaterialBridge/AvaMaterialBridge.h"

#define UE_API AVALANCHETEXT_API

namespace UE::Ava
{
/**
 * Material Bridge for components that are managed by the Text3D 
 * Slot accessing and material container state are not implemented by design as the material slot access for these is managed by the Text3D Component and its extensions. 
 * The Text3DComponent bridge should be used instead.
 */
class FText3DManagedComponentMaterialBridge : public FMaterialBridge
{
protected:
	//~ Begin FMaterialBridge
	UE_API virtual const UStruct* OnGetBridgedType() const override;
	UE_API virtual bool OnIsMaterialContainerSupported(FConstDataView InMaterialContainer) const override;
	//~ End FMaterialBridge
};

} // UE::Ava

#undef UE_API
