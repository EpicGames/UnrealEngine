// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class AvalancheTextEditor : ModuleRules
{
	public AvalancheTextEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateIncludePathModuleNames.AddRange(
			new string[]
			{
				"AvalancheShapesEditor",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Avalanche",
				"AvalancheComponentVisualizers",
				"AvalancheEditorCore",
				"AvalancheInteractiveTools",
				"AvalancheLevelViewport",
				"AvalancheText",
				"ComponentVisualizers",
				"Core",
				"CoreUObject",
				"DynamicMaterialEditor",
				"Engine",
				"EditorScriptingUtilities",
				"InteractiveToolsFramework",
				"InputCore",
				"Projects",
				"PropertyEditor",
				"Slate",
				"SlateCore",
				"Text3D",
				"UnrealEd",
			}
		);
	}
}
