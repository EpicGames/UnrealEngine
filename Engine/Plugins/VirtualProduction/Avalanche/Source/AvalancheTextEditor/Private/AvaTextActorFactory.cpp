// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTextActorFactory.h"

#include "EngineAnalytics.h"
#include "Subsystems/PlacementSubsystem.h"
#include "Text3DActor.h"
#include "Text3DComponent.h"

UAvaTextActorFactory::UAvaTextActorFactory()
{
	NewActorClass = AText3DActor::StaticClass();
}

void UAvaTextActorFactory::PostSpawnActor(UObject* InAsset, AActor* InNewActor)
{
	Super::PostSpawnActor(InAsset, InNewActor);

	/*
	 * Text3D is different from AvaText3D, set its properties to expected default values
	 * Cannot set them on Text3D to avoid affecting other projects
	 */ 
	if (const AText3DActor* Text3DActor = Cast<AText3DActor>(InNewActor))
	{
		UText3DComponent* Component = Text3DActor->GetText3DComponent();
		Component->SetExtrude(0.0f);
		Component->SetScaleProportionally(false);
		Component->SetMaxWidth(100.f);
		Component->SetMaxHeight(100.f);
	}
}

void UAvaTextActorFactory::PostPlaceAsset(TArrayView<const FTypedElementHandle> InHandle, const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions)
{
	Super::PostPlaceAsset(InHandle, InPlacementInfo, InPlacementOptions);
	if (!InPlacementOptions.bIsCreatingPreviewElements && FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> Attributes;
		Attributes.Reserve(2);
		Attributes.Emplace(TEXT("ToolClass"), GetNameSafe(GetClass()));
		Attributes.Emplace(TEXT("ActorClass"), GetNameSafe(NewActorClass));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.MotionDesign.PlaceActor"), Attributes);
	}
}
