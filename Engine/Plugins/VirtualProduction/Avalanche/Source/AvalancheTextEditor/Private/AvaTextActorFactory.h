// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"
#include "AvaTextActorFactory.generated.h"

UCLASS()
class UAvaTextActorFactory : public UActorFactory
{
	GENERATED_BODY()

public:
	UAvaTextActorFactory();

	//~ Begin UActorFactory
	virtual void PostSpawnActor(UObject* InAsset, AActor* InNewActor) override;
	virtual void PostPlaceAsset(TArrayView<const FTypedElementHandle> InHandle, const FAssetPlacementInfo& InPlacementInfo, const FPlacementOptions& InPlacementOptions) override;
	//~ End UActorFactory
};
