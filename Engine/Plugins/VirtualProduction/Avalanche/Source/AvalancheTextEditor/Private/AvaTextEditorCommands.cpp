// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTextEditorCommands.h"
#include "AvaTextEditorStyle.h"

#define LOCTEXT_NAMESPACE "AvaTextEditorCommands"

FAvaTextEditorCommands::FAvaTextEditorCommands()
	: TCommands<FAvaTextEditorCommands>(
		TEXT("AvaTextEditor")
		, LOCTEXT("MotionDesignTextEditor", "Motion Design Text Editor")
		, NAME_None
		, FAvaTextEditorStyle::Get().GetStyleSetName()
	)
{
}

void FAvaTextEditorCommands::RegisterCommands()
{
	UI_COMMAND(Tool_Actor_Text3D
		, "Text Actor"
		, "Create a Text Actor in the viewport."
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());
}

#undef LOCTEXT_NAMESPACE
