// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTextEditorModule.h"

#include "AvaInteractiveToolsDelegates.h"
#include "AvaText3DComponent.h"
#include "AvaTextActor.h"
#include "AvaTextDefs.h"
#include "AvaTextEditorCommands.h"
#include "ColorPicker/AvaViewportColorPickerActorClassRegistry.h"
#include "ColorPicker/AvaViewportColorPickerAdapter.h"
#include "DMObjectMaterialProperty.h"
#include "Extensions/Text3DDefaultMaterialExtension.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/Actor.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "IDynamicMaterialEditorModule.h"
#include "Material/DynamicMaterialInstance.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Text3DActor.h"
#include "Visualizer/AvaTextVisualizer.h"

namespace UE::AvalancheTextEditor::Private
{
	TConstArrayView<FProperty*> GetText3DProperties()
	{
		static const UClass* MaterialExtensionClass = UText3DDefaultMaterialExtension::StaticClass();
		static TArray<FProperty*> Text3DProperties =
		{
			MaterialExtensionClass->FindPropertyByName(TEXT("FrontMaterial")),
			MaterialExtensionClass->FindPropertyByName(TEXT("BevelMaterial")),
			MaterialExtensionClass->FindPropertyByName(TEXT("ExtrudeMaterial")),
			MaterialExtensionClass->FindPropertyByName(TEXT("BackMaterial"))
		};

		return Text3DProperties;
	}

	/**
	 * Make sure that the Style is set to Custom when a material is set.
	 */
	bool SetTextMaterialProperty(const FDMObjectMaterialProperty& InProperty, UDynamicMaterialInstance* InMaterial)
	{
		if (UText3DDefaultMaterialExtension* MaterialExtension = Cast<UText3DDefaultMaterialExtension>(InProperty.GetOuter()))
		{
			TConstArrayView<FProperty*> Text3DProperties = GetText3DProperties();
			FProperty* PropertyToSet = InProperty.GetProperty();

			if (Text3DProperties.Contains(PropertyToSet))
			{
				// Only set the style if we're actaully seting a material.
				if (InMaterial)
				{
					MaterialExtension->SetStyle(EText3DMaterialStyle::Custom);
				}
			}
		}

		// Return to the original method and continue setting normally
		return false;
	}

	/**
	 * Customize Text actor to only show material extension materials instead of all character slots
	 */
	TArray<FDMObjectMaterialProperty> GetText3DMaterialProperties(UObject* InObject)
	{
		TArray<FDMObjectMaterialProperty> Properties;

		if (const AActor* TextActor = Cast<AActor>(InObject))
		{
			if (const UText3DComponent* TextComponent = TextActor->FindComponentByClass<UText3DComponent>())
			{
				if (UText3DDefaultMaterialExtension* MaterialExtension = TextComponent->GetCastedMaterialExtension<UText3DDefaultMaterialExtension>())
				{
					TConstArrayView<FProperty*> Text3DProperties = GetText3DProperties();
					Properties.Reserve(Text3DProperties.Num());

					for (FProperty* Property : Text3DProperties)
					{
						if (Property)
						{
							FDMObjectMaterialProperty& ObjectMaterialProperty = Properties.Emplace_GetRef(MaterialExtension, Property);
							ObjectMaterialProperty.SetMaterialSetterDelegate(FDMSetMaterialObjectProperty::CreateStatic(&SetTextMaterialProperty));
						}

						if (MaterialExtension->GetUseSingleMaterial())
						{
							break;
						}
					}
				}
			}
		}

		return Properties;
	}
}

void FAvaTextEditorModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FAvaTextEditorModule::RegisterComponentVisualizers);

	FAvaTextEditorCommands::Register();

	RegisterDynamicMaterialPropertyGenerator();

	FAvaViewportColorPickerActorClassRegistry::RegisterDefaultClassAdapter<AAvaTextActor>();
}

void FAvaTextEditorModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	FAvaTextEditorCommands::Unregister();
}

void FAvaTextEditorModule::RegisterComponentVisualizers()
{
	if (FSlateApplication::IsInitialized())
	{
		IAvalancheComponentVisualizersModule& AvaComponentVisualizerModule = IAvalancheComponentVisualizersModule::Get();

		AvaComponentVisualizerModule.RegisterComponentVisualizer<UText3DComponent, FAvaTextVisualizer>(&Visualizers);
	}
}

void FAvaTextEditorModule::RegisterDynamicMaterialPropertyGenerator()
{
	const TArray<UClass*> RegisterClasses
	{
		AText3DActor::StaticClass(),
		AAvaTextActor::StaticClass(),
	};

	for (UClass* RegisterClass : RegisterClasses)
	{
		IDynamicMaterialEditorModule::Get().RegisterCustomMaterialPropertyGenerator(
			RegisterClass,
			FDMGetObjectMaterialPropertiesDelegate::CreateStatic(UE::AvalancheTextEditor::Private::GetText3DMaterialProperties)
		);
	}
}

IMPLEMENT_MODULE(FAvaTextEditorModule, AvalancheTextEditor)
