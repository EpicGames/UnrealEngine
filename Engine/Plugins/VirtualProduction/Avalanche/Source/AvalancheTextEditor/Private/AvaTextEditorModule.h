// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class FComponentVisualizer;
class FName;
class IAvalancheInteractiveToolsModule;
class UEdMode;
struct FAvaInteractiveToolsToolParameters;

class FAvaTextEditorModule : public IModuleInterface
{
public:

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

private:
	TArray<TSharedPtr<FComponentVisualizer>> Visualizers;

	void RegisterComponentVisualizers();

	void RegisterDynamicMaterialPropertyGenerator();
};
