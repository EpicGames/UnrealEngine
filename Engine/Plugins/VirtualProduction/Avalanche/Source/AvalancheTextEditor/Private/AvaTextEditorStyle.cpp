// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTextEditorStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Math/MathFwd.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

FAvaTextEditorStyle::FAvaTextEditorStyle()
	: FSlateStyleSet(TEXT("AvaTextEditor"))
{
	const FVector2D Icon16x16(16.0f, 16.0f);

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	SetContentRoot(Plugin->GetBaseDir() / TEXT("Resources"));
	
	Set("ClassIcon.AvaTextActor", new IMAGE_BRUSH("Icons/ToolboxIcons/3d-text", Icon16x16));
	Set("AvaTextEditor.Tool_Actor_Text3D", new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/Toolbar_Text", Icon16x16));
	Set("Tool_Actor_Text3D", new IMAGE_BRUSH_SVG("Icons/ToolboxIcons/Toolbar_Text", Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FAvaTextEditorStyle::~FAvaTextEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
