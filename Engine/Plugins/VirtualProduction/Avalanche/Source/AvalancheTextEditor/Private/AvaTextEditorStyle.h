// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FAvaTextEditorStyle final : public FSlateStyleSet
{
public:
	static FAvaTextEditorStyle& Get()
	{
		static FAvaTextEditorStyle Instance;
		return Instance;
	}

	FAvaTextEditorStyle();
	virtual ~FAvaTextEditorStyle() override;
};
