// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tool/AvaTextActorTool.h"
#include "AvaInteractiveToolsSettings.h"
#include "AvaTextActorFactory.h"
#include "AvaTextEditorCommands.h"
#include "Builders/AvaInteractiveToolsToolBuilder.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "Text3DActor.h"

UAvaTextActorTool::UAvaTextActorTool()
{
	ActorClass = AText3DActor::StaticClass();
}

void UAvaTextActorTool::OnRegisterTool(IAvalancheInteractiveToolsModule* InAITModule)
{
	Super::OnRegisterTool(InAITModule);

	FAvaInteractiveToolsToolParameters ToolParameters = CreateDefaultToolParameters();
	ToolParameters.UICommand = FAvaTextEditorCommands::Get().Tool_Actor_Text3D;
	ToolParameters.Priority = 1000;
	ToolParameters.ActorFactory.Set<TObjectPtr<UActorFactory>>(CreateActorFactory<UAvaTextActorFactory>());

	InAITModule->RegisterTool(IAvalancheInteractiveToolsModule::CategoryNameActor, MoveTemp(ToolParameters));
}
