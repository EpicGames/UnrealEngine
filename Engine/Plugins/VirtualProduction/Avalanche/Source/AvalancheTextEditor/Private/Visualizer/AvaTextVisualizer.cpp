// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTextVisualizer.h"

#include "AvaField.h"
#include "AvaShapeSprites.h"
#include "Characters/Text3DDefaultCharacter.h"
#include "EditorViewportClient.h"
#include "Engine/Texture2D.h"
#include "Extensions/Text3DDefaultLayoutExtension.h"
#include "Extensions/Text3DDefaultMaterialExtension.h"
#include "IAvalancheComponentVisualizersModule.h"
#include "LevelEditor/AvaLevelEditorUtils.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "Text3DComponent.h"
#include "TextureResource.h"

IMPLEMENT_HIT_PROXY(HAvaTextMaxTextHeightProxy, HAvaHitProxy)
IMPLEMENT_HIT_PROXY(HAvaTextMaxTextHeightHandleProxy, HAvaHitProxy)
IMPLEMENT_HIT_PROXY(HAvaTextMaxTextWidthProxy, HAvaHitProxy)
IMPLEMENT_HIT_PROXY(HAvaTextMaxTextWidthHandleProxy, HAvaHitProxy)
IMPLEMENT_HIT_PROXY(HAvaTextScaleProportionallyProxy, HAvaHitProxy)

IMPLEMENT_HIT_PROXY(HAvaTextEditGradientProxy, HAvaHitProxy)
IMPLEMENT_HIT_PROXY(HAvaTextGradientLineStartHandleProxy, HAvaHitProxy)
IMPLEMENT_HIT_PROXY(HAvaTextGradientLineEndHandleProxy, HAvaHitProxy)
IMPLEMENT_HIT_PROXY(HAvaTextGradientCenterHandleProxy, HAvaHitProxy)
IMPLEMENT_HIT_PROXY(HAvaTextGradientSmoothnessHandleProxy, HAvaHitProxy)
IMPLEMENT_HIT_PROXY(HAvaTextCharacterKerningHandleProxy, HAvaHitProxy)

namespace UE::Ava::Private
{
	struct FTextVisualizerStatics
	{
		inline static constexpr float GradientRotHandleScale = 0.005f;
		inline static constexpr float GradientOffsetHandleScale = 0.01f;
		inline static constexpr float GradientSmoothnessHandleScale = 0.1f;
		inline static constexpr float GradientHandleMaxLength = 50.0f;
		inline static constexpr float GradientSmoothnessHandleOffset = -10;
	};
}

#define LOCTEXT_NAMESPACE "AvaTextVisualizer"

FAvaTextVisualizer::FAvaTextVisualizer()
	: FAvaVisualizerBase()
{
	using namespace UE::AvaCore;

	HasMaxHeightProperty        = GetProperty<UText3DDefaultLayoutExtension>(UText3DDefaultLayoutExtension::GetUseMaxHeightPropertyName());
	HasMaxWidthProperty         = GetProperty<UText3DDefaultLayoutExtension>(UText3DDefaultLayoutExtension::GetUseMaxWidthPropertyName());
	MaxHeightProperty           = GetProperty<UText3DDefaultLayoutExtension>(UText3DDefaultLayoutExtension::GetMaxHeightPropertyName());
	MaxWidthProperty            = GetProperty<UText3DDefaultLayoutExtension>(UText3DDefaultLayoutExtension::GetMaxWidthPropertyName());
	ScaleProportionallyProperty = GetProperty<UText3DDefaultLayoutExtension>(UText3DDefaultLayoutExtension::GetScaleProportionallyPropertyName());
	CharacterKerningProperty    = GetProperty<UText3DDefaultCharacter>(UText3DDefaultCharacter::GetKerningPropertyName());

	ResetEditingFlags();
	bShowGradientControls = false;
	bEditingKerning = false;
}

UActorComponent* FAvaTextVisualizer::GetEditedComponent() const
{
	return TextComponent.Get();
}

TMap<UObject*, TArray<FProperty*>> FAvaTextVisualizer::GatherEditableProperties(UObject* InObject) const
{
	if (const UText3DComponent* Component = Cast<UText3DComponent>(InObject))
	{
		if (UText3DDefaultLayoutExtension* DefaultLayoutExtension = Component->GetCastedLayoutExtension<UText3DDefaultLayoutExtension>())
		{
			return {{
				DefaultLayoutExtension,
				{
					HasMaxHeightProperty,
					MaxWidthProperty,
					HasMaxHeightProperty,
					MaxHeightProperty,
					ScaleProportionallyProperty
				}
			}};
		}
	}

	return {};
}

bool FAvaTextVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* VisProxy, const FViewportClick& Click)
{
	if (Click.GetKey() != EKeys::LeftMouseButton || !VisProxy)
	{
		EndEditing();
		return Super::VisProxyHandleClick(InViewportClient, VisProxy, Click);
	}	
	
	if (VisProxy->IsA(HAvaTextMaxTextHeightProxy::StaticGetType()))
	{
		if (const HAvaTextMaxTextHeightProxy* const MaxHeightProxy = static_cast<HAvaTextMaxTextHeightProxy*>(VisProxy))
		{
			EndEditing();

			if (UText3DComponent* const Text3DComponent = Cast<UText3DComponent>(const_cast<UActorComponent*>(MaxHeightProxy->Component.Get())))
			{
				FScopedTransaction Transaction(LOCTEXT("ToggleText3DHasMaxHeight", "Toggle Text 3D Has Max Height"));

				if (AActor* const Text3DActor = Text3DComponent->GetOwner())
				{
					Text3DActor->Modify();
					Text3DComponent->Modify();

					// If max height is off, we will toggle it on later.
					if (!Text3DComponent->HasMaxHeight() && Text3DComponent->GetMaxHeight() == GetDefault<UText3DComponent>()->GetMaxHeight())
					{
						// Set a sensible default value
						Bounds = GetComponentBounds(Text3DComponent);
						float NewMaxHeight = (Bounds.Max.Z - Bounds.Min.Z) * 1.25f;
						Text3DComponent->SetMaxHeight(NewMaxHeight);
					}					

					Text3DComponent->SetHasMaxHeight(!Text3DComponent->HasMaxHeight());
				}
			}

			return true;
		}
	}
	else if (VisProxy->IsA(HAvaTextMaxTextHeightHandleProxy::StaticGetType()))
	{
		if (const HAvaTextMaxTextHeightHandleProxy* const MaxHeightHandleProxy = static_cast<HAvaTextMaxTextHeightHandleProxy*>(VisProxy))
		{
			EndEditing();

			if (UText3DComponent* Text3DComponent = Cast<UText3DComponent>(const_cast<UActorComponent*>(MaxHeightHandleProxy->Component.Get())))
			{
				TextComponent = Text3DComponent;
				bEditingHeight = true;
				StartEditing(InViewportClient, Text3DComponent);
			}

			return true;
		}
	}
	else if (VisProxy->IsA(HAvaTextMaxTextWidthProxy::StaticGetType()))
	{
		if (const HAvaTextMaxTextWidthProxy* const MaxWidthProxy = static_cast<HAvaTextMaxTextWidthProxy*>(VisProxy))
		{
			EndEditing();

			if (UText3DComponent* const Text3DComponent = Cast<UText3DComponent>(const_cast<UActorComponent*>(MaxWidthProxy->Component.Get())))
			{
				FScopedTransaction Transaction(LOCTEXT("ToggleText3DHasMaxWidth", "Toggle Text 3D Has Max Width"));

				if (AActor* const Text3DActor = Text3DComponent->GetOwner())
				{
					Text3DActor->Modify();
					Text3DComponent->Modify();

					// If max width is off, we will toggle it on later.
					if (!Text3DComponent->HasMaxWidth() && Text3DComponent->GetMaxWidth() == GetDefault<UText3DComponent>()->GetMaxWidth())
					{
						// Set a sensible default value
						Bounds = GetComponentBounds(Text3DComponent);
						float NewMaxWidth = (Bounds.Max.Y - Bounds.Min.Y) * 1.25f;
						Text3DComponent->SetMaxWidth(NewMaxWidth);
					}

					Text3DComponent->SetHasMaxWidth(!Text3DComponent->HasMaxWidth());
				}
			}

			return true;
		}
	}
	else if (VisProxy->IsA(HAvaTextMaxTextWidthHandleProxy::StaticGetType()))
	{
		if (const HAvaTextMaxTextWidthHandleProxy* const MaxWidthHandleProxy = static_cast<HAvaTextMaxTextWidthHandleProxy*>(VisProxy))
		{
			EndEditing();

			if (UText3DComponent* const Text3DComponent = Cast<UText3DComponent>(const_cast<UActorComponent*>(MaxWidthHandleProxy->Component.Get())))
			{
				TextComponent = Text3DComponent;
				bEditingWidth = true;
				StartEditing(InViewportClient, Text3DComponent);
			}

			return true;
		}
	}
	else if (VisProxy->IsA(HAvaTextScaleProportionallyProxy::StaticGetType()))
	{
		if (const HAvaTextScaleProportionallyProxy* const ScaleProportionallyProxy = static_cast<HAvaTextScaleProportionallyProxy*>(VisProxy))
		{
			EndEditing();

			if (UText3DComponent* const Text3DComponent = Cast<UText3DComponent>(const_cast<UActorComponent*>(ScaleProportionallyProxy->Component.Get())))
			{
				FScopedTransaction Transaction(LOCTEXT("ToggleText3DScaleProportionally", "Toggle Text 3D Scale Proportionally"));

				if (AActor* const Text3DActor = Text3DComponent->GetOwner())
				{
					Text3DActor->Modify();
					Text3DComponent->Modify();
					Text3DComponent->SetScaleProportionally(!Text3DComponent->ScalesProportionally());
				}
			}
			
			return true;
		}
	}
	// this one just toggles gradient control on/off, in case the actor has it
	else if (VisProxy->IsA(HAvaTextEditGradientProxy::StaticGetType()))
	{
		EndEditing();			
		bShowGradientControls = !bShowGradientControls;

		return true;
	}
	else if (VisProxy->IsA(HAvaTextGradientLineStartHandleProxy::StaticGetType()))
	{
		if (const HAvaTextGradientLineStartHandleProxy* const GradientStartHandleProxy = static_cast<HAvaTextGradientLineStartHandleProxy*>(VisProxy))
		{
			EndEditing();

			if (UText3DComponent* Text3DComponent = Cast<UText3DComponent>(const_cast<UActorComponent*>(GradientStartHandleProxy->Component.Get())))
			{
				TextComponent = Text3DComponent;
				bEditingGradientRotation_StartHandle = true;
				GradientEditBeginLocation_StartHandle = GetGradientStartHandleLocation(Text3DComponent);
				GradientEditBeginLocation_Center = GetGradientCenterHandleLocation(Text3DComponent);
				StartEditing(InViewportClient, Text3DComponent);
			}

			return true;
		}
	}
	else if (VisProxy->IsA(HAvaTextGradientLineEndHandleProxy::StaticGetType()))
	{
		if (const HAvaTextGradientLineEndHandleProxy* const GradientEndHandleProxy = static_cast<HAvaTextGradientLineEndHandleProxy*>(VisProxy))
		{
			EndEditing();

			if (UText3DComponent* Text3DComponent = Cast<UText3DComponent>(const_cast<UActorComponent*>(GradientEndHandleProxy->Component.Get())))
			{
				TextComponent = Text3DComponent;
				bEditingGradientRotation_EndHandle = true;
				GradientEditBeginLocation_EndHandle = GetGradientEndHandleLocation(Text3DComponent);
				GradientEditBeginLocation_Center = GetGradientCenterHandleLocation(Text3DComponent);
				StartEditing(InViewportClient, Text3DComponent);
			}

			return true;
		}
	}
	else if (VisProxy->IsA(HAvaTextGradientCenterHandleProxy::StaticGetType()))
	{
		if (const HAvaTextGradientCenterHandleProxy* const GradientCenterHandleProxy = static_cast<HAvaTextGradientCenterHandleProxy*>(VisProxy))
		{
			EndEditing();

			if (UText3DComponent* Text3DComponent = Cast<UText3DComponent>(const_cast<UActorComponent*>(GradientCenterHandleProxy->Component.Get())))
			{
				TextComponent = Text3DComponent;
				bEditingGradientOffset = true;
				StartEditing(InViewportClient, Text3DComponent);
			}

			return true;
		}
	}
	else if (VisProxy->IsA(HAvaTextGradientSmoothnessHandleProxy::StaticGetType()))
	{
		if (const HAvaTextGradientSmoothnessHandleProxy* const GradientSmoothnessHandleProxy = static_cast<HAvaTextGradientSmoothnessHandleProxy*>(VisProxy))
		{
			EndEditing();

			if (UText3DComponent* Text3DComponent = Cast<UText3DComponent>(const_cast<UActorComponent*>(GradientSmoothnessHandleProxy->Component.Get())))
			{
				TextComponent = Text3DComponent;
				bEditingGradientSmoothness = true;
				StartEditing(InViewportClient, Text3DComponent);
				return true;
			}			
		}
	}
	else if (VisProxy->IsA(HAvaTextCharacterKerningHandleProxy::StaticGetType()))
	{
		if (const HAvaTextCharacterKerningHandleProxy* const KerningHandleProxy = static_cast<HAvaTextCharacterKerningHandleProxy*>(VisProxy))
		{
			EndEditing();

			if (KerningHandleProxy->Index != std::numeric_limits<uint16>::max())
			{
				if (UText3DComponent* Text3DComponent = Cast<UText3DComponent>(const_cast<UActorComponent*>(KerningHandleProxy->Component.Get())))
				{
					TextComponent = Text3DComponent;
					EditingKerningIndex = KerningHandleProxy->Index;
					StartEditing(InViewportClient, Text3DComponent);
				}
			}
			else
			{
				bEditingKerning = !bEditingKerning;
				EditingKerningIndex = INDEX_NONE;
			}

			return true;
		}
	}

	return false;
}

FVector FAvaTextVisualizer::GetGradientSmoothnessHandleLocation(const UText3DComponent* InTextComponent) const
{
	using namespace UE::Ava::Private;
	
	return GetGradientCenterHandleLocation(InTextComponent) + FVector(0, FTextVisualizerStatics::GradientSmoothnessHandleOffset, 0);
}

bool FAvaTextVisualizer::GetWidgetLocation(const FEditorViewportClient* ViewportClient, FVector& OutLocation) const
{
	if (const UText3DComponent* const Text3DComponent = Cast<UText3DComponent>(GetEditedComponent()))
	{
		if (bEditingWidth)
		{
			OutLocation = GetWidthHandleLocation(Text3DComponent);
			return true;
		}

		if (bEditingHeight)
		{
			OutLocation = GetHeightHandleLocation(Text3DComponent);
			return true;
		}

		if (bEditingGradientRotation_StartHandle)
		{
			OutLocation = GetGradientStartHandleLocation(Text3DComponent);
			return true;
		}
		
		if (bEditingGradientRotation_EndHandle)
		{
			OutLocation = GetGradientEndHandleLocation(Text3DComponent);
			return true;
		}
		
		if (bEditingGradientOffset)
		{
			OutLocation = GetGradientCenterHandleLocation(Text3DComponent);
			return true;
		}

		if (bEditingGradientSmoothness)
		{
			OutLocation = GetGradientSmoothnessHandleLocation(Text3DComponent);
			return true;
		}

		if (EditingKerningIndex != INDEX_NONE)
		{
			OutLocation = GetKerningHandleLocation(Text3DComponent, EditingKerningIndex);
			return true;
		}
	}

	return Super::GetWidgetLocation(ViewportClient, OutLocation);
}

bool FAvaTextVisualizer::GetWidgetMode(const FEditorViewportClient* ViewportClient, UE::Widget::EWidgetMode& Mode) const
{
	if (bEditingWidth || bEditingHeight)
	{
		Mode = UE::Widget::WM_Translate;
		return true;
	}

	if (bEditingGradientRotation_EndHandle || bEditingGradientRotation_StartHandle)
	{
		Mode = UE::Widget::WM_Translate;
		return true;
	}

	if (bEditingGradientOffset)
	{
		Mode = UE::Widget::WM_Translate;
		return true;
	}

	if (bEditingGradientSmoothness)
	{
		Mode = UE::Widget::WM_Scale;
		return true;
	}

	if (EditingKerningIndex != INDEX_NONE)
	{
		Mode = UE::Widget::WM_Translate;
		return true;
	}

	return Super::GetWidgetMode(ViewportClient, Mode);
}

bool FAvaTextVisualizer::GetWidgetAxisList(const FEditorViewportClient* ViewportClient, UE::Widget::EWidgetMode WidgetMode, 
	EAxisList::Type& AxisList) const
{
	if (bEditingWidth)
	{
		AxisList = EAxisList::Y;
		return true;
	}

	if (bEditingHeight)
	{
		AxisList = EAxisList::Z;
		return true;
	}

	if (bEditingGradientRotation_EndHandle || bEditingGradientRotation_StartHandle)
	{
		AxisList = EAxisList::Type::Screen;
		return true;
	}

	if (bEditingGradientOffset)
	{
		AxisList = EAxisList::Type::Screen;
		return true;
	}

	if (bEditingGradientSmoothness)
	{
		AxisList = EAxisList::Type::Z;
		return true;
	}
	
	if (EditingKerningIndex != INDEX_NONE)
	{
		AxisList = EAxisList::Y;
		return true;
	}

	return Super::GetWidgetAxisList(ViewportClient, WidgetMode, AxisList);
}

bool FAvaTextVisualizer::GetWidgetAxisListDragOverride(const FEditorViewportClient* ViewportClient,
	UE::Widget::EWidgetMode WidgetMode, EAxisList::Type& AxisList) const
{
	if (bEditingGradientOffset)
	{
		AxisList = EAxisList::YZ;
		return true;
	}

	if (bEditingGradientRotation_EndHandle || bEditingGradientRotation_StartHandle)
	{
		AxisList = EAxisList::YZ;
		return true;
	}

	return Super::GetWidgetAxisListDragOverride(ViewportClient, WidgetMode, AxisList);
}

bool FAvaTextVisualizer::HandleInputDeltaInternal(FEditorViewportClient* InViewportClient, FViewport* InViewport, const FVector& InAccumulatedTranslation,
	const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale)
{
	using namespace UE::Ava::Private;
	
	if (bEditingWidth)
	{
		if (UText3DComponent* const Text3DComponent = Cast<UText3DComponent>(GetEditedComponent()))
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
				{
					float MaxWidth = InitialMaxWidth;

					switch (Text3DComponent->GetHorizontalAlignment())
					{
						case EText3DHorizontalTextAlignment::Left:
							MaxWidth += InAccumulatedTranslation.Y;
							break;

						case EText3DHorizontalTextAlignment::Center:
							MaxWidth += InAccumulatedTranslation.Y * 2.f;
							break;

						case EText3DHorizontalTextAlignment::Right:
							MaxWidth -= InAccumulatedTranslation.Y;
							break;
					}

					MaxWidth = FMath::Max(1.f, MaxWidth);

					if (MaxWidth != InitialMaxWidth)
					{
						bHasBeenModified = true;
					}

					if (AActor* const Text3DActor = Text3DComponent->GetOwner())
					{
						Text3DActor->Modify();
						Text3DComponent->Modify();
						Text3DComponent->SetMaxWidth(MaxWidth);
					}

					return true;
				}
			}
		}

		EndEditing();
		return true;
	}
	else if (bEditingHeight)
	{
		if (UText3DComponent* const Text3DComponent = Cast<UText3DComponent>(GetEditedComponent()))
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
				{
					float MaxHeight = InitialMaxHeight;

					switch (Text3DComponent->GetVerticalAlignment())
					{
						case EText3DVerticalTextAlignment::FirstLine:
							MaxHeight += InAccumulatedTranslation.Z / TopHeightFraction;
							break;

						case EText3DVerticalTextAlignment::Top:
							MaxHeight -= InAccumulatedTranslation.Z;
							break;

						case EText3DVerticalTextAlignment::Center:
							MaxHeight += InAccumulatedTranslation.Z * 2.f;
							break;

						case EText3DVerticalTextAlignment::Bottom:
							MaxHeight += InAccumulatedTranslation.Z;
							break;
					}

					MaxHeight = FMath::Max(1.f, MaxHeight);

					if (MaxHeight != InitialMaxHeight)
					{
						bHasBeenModified = true;
					}

					if (AActor* const Text3DActor = Text3DComponent->GetOwner())
					{
						Text3DActor->Modify();
						Text3DComponent->Modify();
						Text3DComponent->SetMaxHeight(MaxHeight);
					}

					return true;
				}
			}
		}

		EndEditing();
		return true;
	}
	else if (bEditingGradientRotation_EndHandle || bEditingGradientRotation_StartHandle)
	{
		if (UText3DComponent* const Text3DComponent = Cast<UText3DComponent>(GetEditedComponent()))
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
				{
					float GradientRot = InitialGradientRotation;
					float RotIncrZ = InAccumulatedTranslation.Z * FTextVisualizerStatics::GradientRotHandleScale;

					FVector HandleToCenter = FVector::ZeroVector;

					if (bEditingGradientRotation_EndHandle)
					{
					    HandleToCenter = GradientEditBeginLocation_EndHandle - GradientEditBeginLocation_Center;
					}
					else if (bEditingGradientRotation_StartHandle)
					{
					    HandleToCenter = GradientEditBeginLocation_StartHandle - GradientEditBeginLocation_Center;
					}
					
					if (HandleToCenter.Y < 0)
                    {
                        RotIncrZ *= -1;
                    }
					
					GradientRot -= RotIncrZ;
					
					if (GradientRot != InitialGradientRotation)
					{
						bHasBeenModified = true;
					}
					
					if (UText3DDefaultMaterialExtension* MaterialExtension = Text3DComponent->GetCastedMaterialExtension<UText3DDefaultMaterialExtension>())
					{
						MaterialExtension->Modify();
						MaterialExtension->SetGradientRotation(GradientRot);
						return true;
					}
				}
			}
		}

		EndEditing();
		return true;
	}
	else if (bEditingGradientOffset)
	{
		if (UText3DComponent* const Text3DComponent = Cast<UText3DComponent>(GetEditedComponent()))
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::YZ)
				{
					const AActor* Text3DActor = Text3DComponent->GetOwner();
					UText3DDefaultMaterialExtension* MaterialExtension = Text3DComponent->GetCastedMaterialExtension<UText3DDefaultMaterialExtension>();

					if (Text3DActor && MaterialExtension)
					{
						MaterialExtension->SetFlags(RF_Transactional);
						MaterialExtension->Modify();

						float GradientOffset = InitialGradientOffset;
						
						const float OffsetIncr = InAccumulatedTranslation.Length() * FTextVisualizerStatics::GradientOffsetHandleScale;

						FVector GradientDir = Text3DActor->GetActorUpVector().RotateAngleAxis(-MaterialExtension->GetGradientRotation() * 360, Text3DActor->GetActorForwardVector());
						GradientDir.Normalize();

						const float DeltaDotGradDir = InAccumulatedTranslation.Dot(GradientDir);
						
						GradientOffset -= OffsetIncr * FMath::Sign(DeltaDotGradDir);
						GradientOffset = FMath::Clamp(GradientOffset, 0.0, 1.0);

						if (GradientOffset != InitialGradientOffset)
						{
							bHasBeenModified = true;
						}
						
						MaterialExtension->SetGradientOffset(GradientOffset);
						return true;
					}
				}
			}
		}

		EndEditing();
		return true;
	}
	else if (bEditingGradientSmoothness)
	{
		if (UText3DComponent* const Text3DComponent = Cast<UText3DComponent>(GetEditedComponent()))
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Scale)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Z)
				{
					if (UText3DDefaultMaterialExtension* MaterialExtension = Text3DComponent->GetCastedMaterialExtension<UText3DDefaultMaterialExtension>())
					{
						MaterialExtension->SetFlags(RF_Transactional);
						MaterialExtension->Modify();

						float GradientSmoothness = InitialGradientSmoothness;
						
						const float SmoothnessIncr = InAccumulatedScale.Z * FTextVisualizerStatics::GradientSmoothnessHandleScale;
						GradientSmoothness += SmoothnessIncr;
						GradientSmoothness = FMath::Clamp(GradientSmoothness, 0.0, 1.0);

						if (GradientSmoothness != InitialGradientSmoothness)
						{
							bHasBeenModified = true;
						}

						MaterialExtension->SetGradientSmoothness(GradientSmoothness);
						return true;
					}
				}
			}
		}

		EndEditing();
		return true;
	}
	else if (EditingKerningIndex != INDEX_NONE)
	{
		if (const UText3DComponent* const Text3DComponent = Cast<UText3DComponent>(GetEditedComponent()))
		{
			if (GetViewportWidgetMode(InViewportClient) == UE::Widget::WM_Translate)
			{
				if (GetViewportWidgetAxisList(InViewportClient) & EAxisList::Y)
				{
					if (UText3DDefaultCharacter* Character = Text3DComponent->GetCastedCharacter<UText3DDefaultCharacter>(EditingKerningIndex))
					{
						Character->SetFlags(RF_Transactional);
						Character->Modify();

						const float Kerning = InitialCharacterKerning + InAccumulatedTranslation.Y;

						if (!FMath::IsNearlyEqual(Kerning, InitialCharacterKerning))
						{
							bHasBeenModified = true;
						}

						Character->SetKerning(Kerning);
						return true;
					}
				}
			}
		}

		EndEditing();
		return true;
	}

	return Super::HandleInputDeltaInternal(InViewportClient, InViewport, InAccumulatedTranslation, InAccumulatedRotation, InAccumulatedScale);
}

void FAvaTextVisualizer::TrackingStopped(FEditorViewportClient* InViewportClient, bool bInDidMove)
{
	if (bEditingGradientRotation_EndHandle || bEditingGradientRotation_StartHandle)
	{
		if (const UText3DComponent* const Text3DComponent = Cast<UText3DComponent>(GetEditedComponent()))
		{
			// handles positions need to be refreshed every time we release the mouse, so that clockwise vs. counterclockwise interaction works properly
			GradientEditBeginLocation_StartHandle = GetGradientStartHandleLocation(Text3DComponent);
			GradientEditBeginLocation_EndHandle = GetGradientEndHandleLocation(Text3DComponent);
			GradientEditBeginLocation_Center = GetGradientCenterHandleLocation(Text3DComponent);
		}
	}

	FAvaVisualizerBase::TrackingStopped(InViewportClient, bInDidMove);
}

bool FAvaTextVisualizer::ResetValue(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy)
{
	if (!HitProxy->IsA(HAvaTextMaxTextHeightProxy::StaticGetType())
		&& !HitProxy->IsA(HAvaTextMaxTextWidthProxy::StaticGetType())
		&& !HitProxy->IsA(HAvaTextScaleProportionallyProxy::StaticGetType())
		&& !HitProxy->IsA(HAvaTextEditGradientProxy::StaticGetType()))
	{
		return Super::ResetValue(InViewportClient, HitProxy);
	}

	UText3DComponent* Text3DComponent = TextComponent.Get();

	if (!Text3DComponent)
	{
		return Super::ResetValue(InViewportClient, HitProxy);
	}

	Text3DComponent->SetFlags(RF_Transactional);

	if (HitProxy->IsA(HAvaTextMaxTextWidthProxy::StaticGetType()))
	{
		FScopedTransaction Transaction(NSLOCTEXT("AvaText3DVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
		Text3DComponent->Modify();

		Text3DComponent->SetHasMaxWidth(false);
		Text3DComponent->SetMaxWidth(GetDefault<UText3DDefaultLayoutExtension>()->GetMaxWidth());

		FAvaVisualizerBase::NotifyPropertyModified(Text3DComponent, MaxWidthProperty, EPropertyChangeType::ValueSet);
	}
	else if (HitProxy->IsA(HAvaTextMaxTextHeightProxy::StaticGetType()))
	{
		FScopedTransaction Transaction(NSLOCTEXT("AvaTextVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
		Text3DComponent->Modify();

		Text3DComponent->SetHasMaxHeight(false);
		Text3DComponent->SetMaxHeight(GetDefault<UText3DDefaultLayoutExtension>()->GetMaxHeight());

		FAvaVisualizerBase::NotifyPropertyModified(Text3DComponent, MaxHeightProperty, EPropertyChangeType::ValueSet);
	}
	else if (HitProxy->IsA(HAvaTextScaleProportionallyProxy::StaticGetType()))
	{
		FScopedTransaction Transaction(NSLOCTEXT("AvaTextVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));
		Text3DComponent->Modify();

		Text3DComponent->SetScaleProportionally(false);

		FAvaVisualizerBase::NotifyPropertyModified(Text3DComponent, ScaleProportionallyProperty, EPropertyChangeType::ValueSet);
	}
	else if (HitProxy->IsA(HAvaTextCharacterKerningHandleProxy::StaticGetType()))
	{
		FScopedTransaction Transaction(NSLOCTEXT("AvaTextVisualizer", "VisualizerResetValue", "Visualizer Reset Value"));

		if (UText3DDefaultCharacter* Character = Text3DComponent->GetCastedCharacter<UText3DDefaultCharacter>(EditingKerningIndex))
		{
			Character->Modify();
			const UText3DDefaultCharacter* CDO = GetDefault<UText3DDefaultCharacter>();
			Character->SetKerning(CDO->GetKerning());
			FAvaVisualizerBase::NotifyPropertyModified(Character, CharacterKerningProperty, EPropertyChangeType::ValueSet);
		}
	}

	return true;
}

bool FAvaTextVisualizer::IsEditing() const
{
	if (bEditingWidth
		|| bEditingHeight 
		|| bEditingGradientRotation_StartHandle
		|| bEditingGradientRotation_EndHandle
		|| bEditingGradientOffset
		|| bEditingGradientSmoothness
		|| EditingKerningIndex != INDEX_NONE)
	{
		return true;
	}

	return Super::IsEditing();
}

void FAvaTextVisualizer::ResetEditingFlags()
{
	bEditingWidth = false;
	bEditingHeight = false;	
	bEditingGradientRotation_StartHandle = false;
	bEditingGradientRotation_EndHandle = false;
	bEditingGradientOffset = false;
	bEditingGradientSmoothness = false;
	EditingKerningIndex = INDEX_NONE;
}

void FAvaTextVisualizer::EndEditing()
{
	Super::EndEditing();
	
	ResetEditingFlags();
}

FVector FAvaTextVisualizer::GetWidthHandleLocation(const UText3DComponent* InText3DComp) const
{
	const FTransform Transform = InText3DComp->GetComponentTransform();
	const FBox BoundsMax = GetBoundsMax(InText3DComp);
	const float CoordZ = (BoundsMax.Min.Z + BoundsMax.Max.Z) * 0.5f;
	FVector IconLocation;

	switch (InText3DComp->GetHorizontalAlignment())
	{
		default:
		case EText3DHorizontalTextAlignment::Left:
			IconLocation = Transform.TransformPositionNoScale(FVector(0.f, BoundsMax.Max.Y + 10.f, CoordZ));
			break;

		case EText3DHorizontalTextAlignment::Center:
			IconLocation = Transform.TransformPositionNoScale(FVector(0.f, BoundsMax.Max.Y + 10.f, CoordZ));
			break;

		case EText3DHorizontalTextAlignment::Right:
			IconLocation = Transform.TransformPositionNoScale(FVector(0.f, BoundsMax.Min.Y - 10.f, CoordZ));
			break;
	}

	return IconLocation;
}

FVector FAvaTextVisualizer::GetHeightHandleLocation(const UText3DComponent* InText3DComp) const
{
	const FTransform Transform = InText3DComp->GetComponentTransform();
	const FBox BoundsMax = GetBoundsMax(InText3DComp);
	const float CoordY = (InText3DComp->GetHorizontalAlignment() != EText3DHorizontalTextAlignment::Right)
		? BoundsMax.Min.Y - 10.f
		: BoundsMax.Max.Y + 10.f;

	FVector IconLocation = FVector::ZeroVector;

	switch (InText3DComp->GetVerticalAlignment())
	{
		case EText3DVerticalTextAlignment::FirstLine:
			IconLocation = Transform.TransformPositionNoScale(FVector(0.f, CoordY, BoundsMax.Max.Z));
			break;

		case EText3DVerticalTextAlignment::Top:
			IconLocation = Transform.TransformPositionNoScale(FVector(0.f, CoordY, BoundsMax.Min.Z));
			break;

		case EText3DVerticalTextAlignment::Center:
			IconLocation = Transform.TransformPositionNoScale(FVector(0.f, CoordY, BoundsMax.Max.Z));
			break;

		case EText3DVerticalTextAlignment::Bottom:
			IconLocation = Transform.TransformPositionNoScale(FVector(0.f, CoordY, BoundsMax.Max.Z));
			break;
	}

	return IconLocation;
}

void FAvaTextVisualizer::GetTextActorGradientControlsLocations(const UText3DComponent* InTextComponent, FVector& OutGradientCenterLocation, FVector& OutGradientStartLocation, FVector& OutGradientEndLocation) const
{
	using namespace UE::Ava::Private;
	if (IsValid(InTextComponent))
	{
		const AActor* const Text3DActor = InTextComponent->GetOwner();
		const UText3DDefaultMaterialExtension* MaterialExtension = InTextComponent->GetCastedMaterialExtension<UText3DDefaultMaterialExtension>();

		if (Text3DActor && MaterialExtension)
		{
			FVector BoundsOrigin;
			FVector BoundsExtent;
			
			Text3DActor->GetActorBounds(false, BoundsOrigin, BoundsExtent);
			const float GradientLineHalfLength = FMath::Min(BoundsExtent.Length(), FTextVisualizerStatics::GradientHandleMaxLength);
			FVector GradientCenterLocation = BoundsOrigin;

			const FVector GradientDir = MaterialExtension->GetGradientDirection();
			OutGradientStartLocation = GradientCenterLocation - GradientDir * GradientLineHalfLength;
			OutGradientEndLocation = GradientCenterLocation + GradientDir * GradientLineHalfLength;	

			// we need to move gradient center along its direction, base on offset amount
			const float GradientAsNormalizedPos = MaterialExtension->GetGradientOffset() * 2.0 - 1.0;

			GradientCenterLocation -= GradientDir * GradientAsNormalizedPos * GradientLineHalfLength;
			OutGradientCenterLocation = GradientCenterLocation;
		}
	}
}

FVector FAvaTextVisualizer::GetGradientEndHandleLocation(const UText3DComponent* InTextComponent) const
{
	if (IsValid(InTextComponent))
	{
		FVector GradientCenter;
		FVector GradientStart;
		FVector GradientEnd;
		
		GetTextActorGradientControlsLocations(TextComponent.Get(), GradientCenter, GradientStart, GradientEnd);

		return GradientEnd;
	}

	return FVector::Zero();
}

FVector FAvaTextVisualizer::GetGradientCenterHandleLocation(const UText3DComponent* InTextComponent) const
{
	if (IsValid(InTextComponent))
	{
		FVector GradientCenter;
		FVector GradientStart;
		FVector GradientEnd;
		
		GetTextActorGradientControlsLocations(TextComponent.Get(), GradientCenter, GradientStart, GradientEnd);

		return GradientCenter;
	}

	return FVector::Zero();
}

FVector FAvaTextVisualizer::GetGradientStartHandleLocation(const UText3DComponent* InTextComponent) const
{
	if (IsValid(InTextComponent))
	{
		FVector GradientCenter;
		FVector GradientStart;
		FVector GradientEnd;
		
		GetTextActorGradientControlsLocations(TextComponent.Get(), GradientCenter, GradientStart, GradientEnd);

		return GradientStart;
	}

	return FVector::Zero();
}

FVector FAvaTextVisualizer::GetKerningHandleLocation(const UText3DComponent* InTextComponent, int32 InIndex) const
{
	if (IsValid(InTextComponent))
	{
		if (UText3DCharacterBase* Character = InTextComponent->GetCharacter(InIndex))
		{
			FVector CharacterLocation = Character->GetTransform(/** Reset */false).GetLocation();

			if (const UText3DLayoutExtensionBase* LayoutExtension = InTextComponent->GetLayoutExtension())
			{
				CharacterLocation *= LayoutExtension->GetTextScale();
			}

			return InTextComponent->GetComponentTransform().TransformPosition(CharacterLocation);
		}
	}

	return FVector::ZeroVector;
}

void FAvaTextVisualizer::StoreTextMetrics(const UText3DComponent* InText3DComp)
{
	Bounds = GetComponentBounds(InText3DComp);
	TArray<FString> Lines;
	InText3DComp->GetFormattedText().ToString().ParseIntoArrayLines(Lines);
	LineCount = Lines.Num();
	LineHeight = (Bounds.GetSize().Z - (InText3DComp->GetLineSpacing() * (static_cast<float>(LineCount) - 1.f))) / static_cast<float>(LineCount);
}

void FAvaTextVisualizer::DrawVisualizationNotEditing(const UActorComponent* InComponent, const FSceneView* InView, 
	FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationNotEditing(InComponent, InView, InPDI, InOutIconIndex);

	const UText3DComponent* const Text3DComponent = Cast<UText3DComponent>(InComponent);

	if (!Text3DComponent)
	{
		return;
	}
	
	StoreTextMetrics(Text3DComponent);

	DrawMaxTextSizeVisualization(Text3DComponent, InView, InPDI);

	DrawMaxTextWidthButton(Text3DComponent, InView, InPDI, InOutIconIndex, Text3DComponent->HasMaxWidth() ? Enabled : Disabled);
	++InOutIconIndex;

	DrawMaxTextHeightButton(Text3DComponent, InView, InPDI, InOutIconIndex, Text3DComponent->HasMaxHeight() ? Enabled : Disabled);
	++InOutIconIndex;

	DrawScaleProportionallyButton(Text3DComponent, InView, InPDI, InOutIconIndex, Text3DComponent->ScalesProportionally() ? Enabled : Disabled);
	++InOutIconIndex;

	if (Text3DComponent->HasMaxWidth())
	{
		DrawMaxTextWidthHandle(Text3DComponent, InView, InPDI, Inactive);
	}

	if (Text3DComponent->HasMaxHeight())
	{
		DrawMaxTextHeightHandle(Text3DComponent, InView, InPDI, Inactive);
	}
	
	if (const UText3DDefaultMaterialExtension* MaterialExtension = Text3DComponent->GetCastedMaterialExtension<UText3DDefaultMaterialExtension>())
	{
		if (MaterialExtension->GetStyle() == EText3DMaterialStyle::Gradient)
		{
			DrawEditGradientButton(Text3DComponent, InView, InPDI, InOutIconIndex, bShowGradientControls ? Enabled : Disabled);
			++InOutIconIndex;

			if (bShowGradientControls)
			{
				DrawGradientHandles(Text3DComponent, InView, InPDI);
			}
		}
	}

	DrawCharacterKerningButton(Text3DComponent, InView, InPDI, InOutIconIndex, bEditingKerning ? Enabled : Disabled);
	++InOutIconIndex;

	if (bEditingKerning)
	{
		Text3DComponent->ForEachCharacter([this, Text3DComponent, InView, InPDI](UText3DCharacterBase* InCharacter, uint16 InIndex, uint16)
		{
			const FVector CharacterLocation = GetKerningHandleLocation(Text3DComponent, InIndex);
			DrawCharacterKerningHandle(Text3DComponent, InView, InPDI, CharacterLocation, InIndex, Inactive);
		});
	}
}

void FAvaTextVisualizer::DrawVisualizationEditing(const UActorComponent* InComponent, const FSceneView* InView, 
	FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex)
{
	Super::DrawVisualizationNotEditing(InComponent, InView, InPDI, InOutIconIndex);

	const UText3DComponent* const Text3DComponent = Cast<UText3DComponent>(InComponent);

	if (!Text3DComponent)
	{
		return;
	}

	StoreTextMetrics(Text3DComponent);

	DrawMaxTextSizeVisualization(Text3DComponent, InView, InPDI);

	if (bEditingWidth)
	{
		DrawMaxTextWidthButton(Text3DComponent, InView, InPDI, InOutIconIndex, Text3DComponent->HasMaxWidth() ? Enabled : Disabled);
		DrawMaxTextWidthHandle(Text3DComponent, InView, InPDI, Inactive);
		++InOutIconIndex;
	}
	
	if (bEditingHeight)
	{
		DrawMaxTextHeightButton(Text3DComponent, InView, InPDI, InOutIconIndex, Text3DComponent->HasMaxHeight() ? Enabled : Disabled);
		DrawMaxTextHeightHandle(Text3DComponent, InView, InPDI, Inactive);
		++InOutIconIndex;
	}

	if (bEditingWidth || bEditingHeight)
	{
		DrawScaleProportionallyButton(Text3DComponent, InView, InPDI, InOutIconIndex, Text3DComponent->ScalesProportionally() ? Enabled : Disabled);
		++InOutIconIndex;
	}

	if (bEditingGradientOffset || bEditingGradientRotation_EndHandle || bEditingGradientRotation_StartHandle)
	{
		if (const UText3DDefaultMaterialExtension* MaterialExtension = TextComponent->GetCastedMaterialExtension<UText3DDefaultMaterialExtension>())
		{
			if (MaterialExtension->GetStyle() == EText3DMaterialStyle::Gradient)
			{
				DrawEditGradientButton(Text3DComponent, InView, InPDI, InOutIconIndex, bShowGradientControls ? Enabled : Disabled);
				++InOutIconIndex;

				if (bShowGradientControls)
				{
					DrawGradientHandles(Text3DComponent, InView, InPDI);
				}
			}
		}
	}

	DrawCharacterKerningButton(Text3DComponent, InView, InPDI, InOutIconIndex, bEditingKerning ? Enabled : Disabled);
	++InOutIconIndex;

	if (EditingKerningIndex != INDEX_NONE)
	{
		Text3DComponent->ForEachCharacter([this, Text3DComponent, InView, InPDI](UText3DCharacterBase* InCharacter, uint16 InIndex, uint16)
		{
			const FVector CharacterLocation = GetKerningHandleLocation(Text3DComponent, InIndex);
			DrawCharacterKerningHandle(Text3DComponent, InView, InPDI, CharacterLocation, InIndex, InIndex == EditingKerningIndex ? Active : Inactive);
		});
	}
}

void FAvaTextVisualizer::StoreInitialValues()
{
	Super::StoreInitialValues();

	if (!TextComponent.IsValid())
	{
		return;
	}

	bInitialMaxWidthEnabled = TextComponent->HasMaxHeight();
	bInitialMaxHeightEnabled = TextComponent->HasMaxHeight();
	InitialMaxWidth = TextComponent->GetMaxWidth();
	InitialMaxHeight = TextComponent->GetMaxHeight();
	bInitialScaleProportionally = TextComponent->ScalesProportionally();

	if (const UText3DDefaultMaterialExtension* MaterialExtension = TextComponent->GetCastedMaterialExtension<UText3DDefaultMaterialExtension>())
	{
		InitialGradientRotation = MaterialExtension->GetGradientRotation();
		InitialGradientOffset = MaterialExtension->GetGradientOffset();
		InitialGradientSmoothness = MaterialExtension->GetGradientSmoothness();
	}

	if (EditingKerningIndex != INDEX_NONE)
	{
		if (const UText3DCharacterBase* Character = TextComponent->GetCharacter(EditingKerningIndex))
		{
			InitialCharacterKerning = Character->GetCharacterKerning();
		}
	}
}

void FAvaTextVisualizer::DrawMaxTextSizeVisualization(const UText3DComponent* InTextComponent, const FSceneView* InView, 
	FPrimitiveDrawInterface* InPDI) const
{
	if (!InTextComponent || (!InTextComponent->HasMaxWidth() && !InTextComponent->HasMaxHeight()))
	{
		return;
	}

	const FTransform Transform = InTextComponent->GetComponentTransform();
	const FBox BoundsMax = GetBoundsMax(InTextComponent);
	const FVector TopLeft = Transform.TransformPositionNoScale(FVector(0.f, BoundsMax.Min.Y, BoundsMax.Max.Z));
	const FVector TopRight = Transform.TransformPositionNoScale(FVector(0.f, BoundsMax.Max.Y, BoundsMax.Max.Z));
	const FVector BottomLeft = Transform.TransformPositionNoScale(FVector(0.f, BoundsMax.Min.Y, BoundsMax.Min.Z));
	const FVector BottomRight = Transform.TransformPositionNoScale(FVector(0.f, BoundsMax.Max.Y, BoundsMax.Min.Z));

	InPDI->DrawLine(TopLeft, TopRight, FLinearColor::Yellow, SDPG_World, 0.5f);
	InPDI->DrawLine(TopRight, BottomRight, FLinearColor::Yellow, SDPG_World, 0.5f);
	InPDI->DrawLine(BottomRight, BottomLeft, FLinearColor::Yellow, SDPG_World, 0.5f);
	InPDI->DrawLine(BottomLeft, TopLeft, FLinearColor::Yellow, SDPG_World, 0.5f);
}

bool FAvaTextVisualizer::DrawGradientCenterHandle(const UText3DComponent* InTextComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI, FVector InGradientCenter)
{
	static constexpr float BaseSize = 1.f;

	UTexture2D* UVSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::SizeSprite);

	if (!UVSprite || !UVSprite->GetResource())
	{
		return true;
	}

	const FVector IconLocation = InGradientCenter;
	const float IconSize = BaseSize * GetIconSizeScale(InView, IconLocation);

	InPDI->SetHitProxy(new HAvaTextGradientCenterHandleProxy(InTextComponent));
	InPDI->DrawSprite(InGradientCenter, IconSize, IconSize, UVSprite->GetResource(), FLinearColor::White,
	                SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);		
	return false;
}

bool FAvaTextVisualizer::DrawGradientSmoothnessHandle(const UText3DComponent* InTextComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI, FVector InHandleLocation)
{
	static constexpr float BaseSize = 1.5f;
		
	const UTexture2D* const UVSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::UVSprite);

	if (!UVSprite || !UVSprite->GetResource())
	{
		return true;
	}

	const FVector IconLocation = InHandleLocation;
	const float IconSize = BaseSize * GetIconSizeScale(InView, IconLocation);

	InPDI->SetHitProxy(new HAvaTextGradientSmoothnessHandleProxy(InTextComponent));
	InPDI->DrawSprite(InHandleLocation, IconSize, IconSize, UVSprite->GetResource(), FLinearColor::White,
					SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);		
	return false;
}

void FAvaTextVisualizer::DrawGradientHandles(const UText3DComponent* InTextComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI) const
{
	using namespace UE::Ava::Private;
	if (!InTextComponent)
	{
		return;
	}

	FVector GradientCenter;
	FVector GradientLineStart;
	FVector GradientLineEnd;

	GetTextActorGradientControlsLocations(InTextComponent, GradientCenter, GradientLineStart, GradientLineEnd);
	
	InPDI->DrawLine(GradientLineStart, GradientLineEnd, FLinearColor::White,SDPG_Foreground );

	if (UText3DDefaultMaterialExtension* MaterialExtension = TextComponent->GetCastedMaterialExtension<UText3DDefaultMaterialExtension>())
	{
		DrawGradientLineStartHandle(InTextComponent, InView, InPDI, GradientLineStart, MaterialExtension->GetGradientColorB());
		DrawGradientLineEndHandle(InTextComponent, InView, InPDI, GradientLineEnd, MaterialExtension->GetGradientColorA());
	}

	DrawGradientCenterHandle(InTextComponent, InView, InPDI, GradientCenter);
	DrawGradientSmoothnessHandle(InTextComponent, InView, InPDI, GradientCenter + FVector(0, FTextVisualizerStatics::GradientSmoothnessHandleOffset, 0));
	
}

FBox FAvaTextVisualizer::GetBoundsMax(const UText3DComponent* InText3DComp) const
{
	const FVector Scale = InText3DComp->GetComponentTransform().GetScale3D();
	FBox BoundsMax = Bounds;

	if (InText3DComp->HasMaxWidth())
	{
		switch (InText3DComp->GetHorizontalAlignment())
		{
			case EText3DHorizontalTextAlignment::Left:
				BoundsMax.Max.Y = InText3DComp->GetMaxWidth() * Scale.Y;
				break;

			case EText3DHorizontalTextAlignment::Center:
				BoundsMax.Min.Y = InText3DComp->GetMaxWidth() * -0.5f * Scale.Y;
				BoundsMax.Max.Y = InText3DComp->GetMaxWidth() * 0.5f * Scale.Y;
				break;

			case EText3DHorizontalTextAlignment::Right:
				BoundsMax.Min.Y = InText3DComp->GetMaxWidth() * -Scale.Y;
				break;
		}
	}

	if (InText3DComp->HasMaxHeight())
	{
		switch (InText3DComp->GetVerticalAlignment())
		{
			case EText3DVerticalTextAlignment::FirstLine:
				if (LineCount <= 1)
				{
					BoundsMax.Max.Z = FMath::Max(BoundsMax.Max.Z, InText3DComp->GetMaxHeight() * TopHeightFraction * Scale.Z);
				}
				else
				{
					BoundsMax.Max.Z = FMath::Max(BoundsMax.Max.Z, LineHeight * TopHeightFraction * Scale.Z * InText3DComp->GetMaxHeight() / Bounds.GetSize().Z);
				}
				break;

			case EText3DVerticalTextAlignment::Top:
				BoundsMax.Min.Z = -InText3DComp->GetMaxHeight() * Scale.Z;
				break;

			case EText3DVerticalTextAlignment::Center:
				BoundsMax.Min.Z = InText3DComp->GetMaxHeight() / -2.f * Scale.Z;
				BoundsMax.Max.Z = InText3DComp->GetMaxHeight() / 2.f * Scale.Z;
				break;

			case EText3DVerticalTextAlignment::Bottom:
				BoundsMax.Max.Z = InText3DComp->GetMaxHeight() * Scale.Z;
				break;
		}
	}

	return BoundsMax;
}

void FAvaTextVisualizer::DrawMaxTextWidthButton(const UText3DComponent* InTextComponent, const FSceneView* InView, 
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	if (!InTextComponent)
	{
		return;
	}

	UTexture2D* UVSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::TextMaxWidthSprite);

	if (!UVSprite || !UVSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaTextMaxTextWidthProxy(InTextComponent));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, UVSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaTextVisualizer::DrawMaxTextHeightButton(const UText3DComponent* InTextComponent, const FSceneView* InView, 
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	if (!InTextComponent)
	{
		return;
	}

	const UTexture2D* const UVSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::TextMaxHeightSprite);

	if (!UVSprite || !UVSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaTextMaxTextHeightProxy(InTextComponent));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, UVSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaTextVisualizer::DrawScaleProportionallyButton(const UText3DComponent* InTextComponent, const FSceneView* InView, 
	FPrimitiveDrawInterface* InPDI, int32 InIconIndex, const FLinearColor& InColor) const
{
	if (!InTextComponent)
	{
		return;
	}

	const UTexture2D* const UVSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::TextScaleProportionallySprite);

	if (!UVSprite || !UVSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaTextScaleProportionallyProxy(InTextComponent));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, UVSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaTextVisualizer::DrawEditGradientButton(
	const UText3DComponent* InTextComponent
	, const FSceneView* InView
	, FPrimitiveDrawInterface* InPDI
	, int32 InIconIndex
	, const FLinearColor& InColor) const
{
	if (!InTextComponent)
	{
		return;
	}

	// todo: proper sprite
	const UTexture2D* const UVSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::LinearGradientSprite);

	if (!UVSprite || !UVSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);
	InPDI->SetHitProxy(new HAvaTextEditGradientProxy(InTextComponent));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, UVSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaTextVisualizer::DrawMaxTextWidthHandle(
	const UText3DComponent* InTextComponent
	, const FSceneView* InView
	, FPrimitiveDrawInterface* InPDI
	, const FLinearColor& InColor) const
{
	static constexpr float BaseSize = 1.f;

	if (!InTextComponent)
	{
		return;
	}

	const UTexture2D* const UVSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::SizeSprite);

	if (!UVSprite || !UVSprite->GetResource())
	{
		return;
	}

	const FVector IconLocation = GetWidthHandleLocation(InTextComponent);
	const float IconSize = BaseSize * GetIconSizeScale(InView, IconLocation);

	InPDI->SetHitProxy(new HAvaTextMaxTextWidthHandleProxy(InTextComponent));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, UVSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaTextVisualizer::DrawMaxTextHeightHandle(const UText3DComponent* InTextComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
	const FLinearColor& InColor) const
{
	static constexpr  float BaseSize = 1.f;

	if (!InTextComponent)
	{
		return;
	}

	const UTexture2D* const UVSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::SizeSprite);

	if (!UVSprite || !UVSprite->GetResource())
	{
		return;
	}

	const FVector IconLocation = GetHeightHandleLocation(InTextComponent);
	const float IconSize = BaseSize * GetIconSizeScale(InView, IconLocation);

	InPDI->SetHitProxy(new HAvaTextMaxTextHeightHandleProxy(InTextComponent));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, UVSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaTextVisualizer::DrawGradientLineStartHandle(const UText3DComponent* InTextComponent, const FSceneView* InView, 
	FPrimitiveDrawInterface* InPDI, const FVector& InLocation, const FLinearColor& InColor) const
{
	static constexpr float BaseSize = 1.5f;

	if (!InTextComponent)
	{
		return;
	}

	const UTexture2D* const UVSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::ColorSelectionSprite);

	if (!UVSprite || !UVSprite->GetResource())
	{
		return;
	}
	
	const FVector IconLocation = InLocation;
	const float IconSize = BaseSize * GetIconSizeScale(InView, IconLocation);

	// in case gradient center handle is on top of this handle, or almost on top of it, don't enable interaction
	if (FVector::Distance(InLocation, GetGradientCenterHandleLocation(InTextComponent)) > BaseSize/2.0)
	{
		InPDI->SetHitProxy(new HAvaTextGradientLineStartHandleProxy(InTextComponent));
	}
	
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, UVSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaTextVisualizer::DrawGradientLineEndHandle(const UText3DComponent* InTextComponent, const FSceneView* InView, 
	FPrimitiveDrawInterface* InPDI, const FVector& InLocation, const FLinearColor& InColor) const
{
	static constexpr float BaseSize = 1.5f;

	if (!InTextComponent)
	{
		return;
	}

	const UTexture2D* const UVSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::ColorSelectionSprite);

	if (!UVSprite || !UVSprite->GetResource())
	{
		return;
	}

	const FVector IconLocation = InLocation;
	const float IconSize = BaseSize * GetIconSizeScale(InView, IconLocation);

	// in case gradient center handle is on top of this handle, or almost on top of it, don't enable interaction
	if (FVector::Distance(InLocation, GetGradientCenterHandleLocation(InTextComponent)) > BaseSize/2.0)
	{
		InPDI->SetHitProxy(new HAvaTextGradientLineEndHandleProxy(InTextComponent));
	}
	
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, UVSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaTextVisualizer::DrawCharacterKerningButton(const UText3DComponent* InTextComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
	int32 InIconIndex, const FLinearColor& InColor)
{
	if (!InTextComponent)
	{
		return;
	}

	UTexture2D* KerningSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::InnerSizeSprite);

	if (!KerningSprite || !KerningSprite->GetResource())
	{
		return;
	}

	FVector IconLocation;
	float IconSize;
	GetIconMetrics(InView, InIconIndex, IconLocation, IconSize);

	InPDI->SetHitProxy(new HAvaTextCharacterKerningHandleProxy(InTextComponent, std::numeric_limits<uint16>::max()));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, KerningSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

void FAvaTextVisualizer::DrawCharacterKerningHandle(const UText3DComponent* InTextComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
	const FVector& InLocation, uint16 InCharacterIndex, const FLinearColor& InColor) const
{
	static constexpr float BaseSize = 1.f;

	if (!InTextComponent)
	{
		return;
	}

	const UTexture2D* const KerningSprite = IAvalancheComponentVisualizersModule::Get().GetSettings()->GetVisualizerSprite(UE::AvaShapes::SizeSprite);

	if (!KerningSprite || !KerningSprite->GetResource())
	{
		return;
	}

	const FVector IconLocation = InLocation;
	const float IconSize = BaseSize * GetIconSizeScale(InView, IconLocation);

	InPDI->SetHitProxy(new HAvaTextCharacterKerningHandleProxy(InTextComponent, InCharacterIndex));
	InPDI->DrawSprite(IconLocation, IconSize, IconSize, KerningSprite->GetResource(), InColor,
		SDPG_Foreground, 0, 0, 0, 0, SE_BLEND_Opaque);
	InPDI->SetHitProxy(nullptr);
}

#undef LOCTEXT_NAMESPACE
