// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaVisBase.h"

class FProperty;
class UText3DComponent;
class UText3DComponent;

struct HAvaTextMaxTextHeightProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaTextMaxTextHeightProxy(const UActorComponent* InComponent)
		: HAvaHitProxy(InComponent)
	{}
};

struct HAvaTextMaxTextHeightHandleProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaTextMaxTextHeightHandleProxy(const UActorComponent* InComponent)
		: HAvaHitProxy(InComponent)
	{}
};

struct HAvaTextMaxTextWidthProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaTextMaxTextWidthProxy(const UActorComponent* InComponent)
		: HAvaHitProxy(InComponent)
	{}
};

struct HAvaTextMaxTextWidthHandleProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaTextMaxTextWidthHandleProxy(const UActorComponent* InComponent)
		: HAvaHitProxy(InComponent)
	{}
};

struct HAvaTextScaleProportionallyProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaTextScaleProportionallyProxy(const UActorComponent* InComponent)
		: HAvaHitProxy(InComponent)
	{}
};

struct HAvaTextEditGradientProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaTextEditGradientProxy(const UActorComponent* InComponent)
		: HAvaHitProxy(InComponent)
	{}
};

struct HAvaTextGradientLineStartHandleProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaTextGradientLineStartHandleProxy(const UActorComponent* InComponent)
		: HAvaHitProxy(InComponent)
	{}
};

struct HAvaTextGradientLineEndHandleProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaTextGradientLineEndHandleProxy(const UActorComponent* InComponent)
		: HAvaHitProxy(InComponent)
	{}
};

struct HAvaTextGradientCenterHandleProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaTextGradientCenterHandleProxy(const UActorComponent* InComponent)
		: HAvaHitProxy(InComponent)
	{}
};

struct HAvaTextGradientSmoothnessHandleProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaTextGradientSmoothnessHandleProxy(const UActorComponent* InComponent)
		: HAvaHitProxy(InComponent)
	{}
};

struct HAvaTextCharacterKerningHandleProxy : HAvaHitProxy
{
	DECLARE_HIT_PROXY();

	HAvaTextCharacterKerningHandleProxy(const UActorComponent* InComponent, uint16 InIndex)
		: HAvaHitProxy(InComponent)
		, Index(InIndex)
	{}

	uint16 Index;
};

class FAvaTextVisualizer : public FAvaVisualizerBase
{
public:
	using Super = FAvaVisualizerBase;

	void ResetEditingFlags();

	FAvaTextVisualizer();

	//~ Begin FAvaVisualizerBase
	virtual UActorComponent* GetEditedComponent() const override;
	virtual TMap<UObject*, TArray<FProperty*>> GatherEditableProperties(UObject* InObject) const override;
	virtual bool VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* VisProxy,
		const FViewportClick& Click) override;
	virtual bool GetWidgetLocation(const FEditorViewportClient* ViewportClient, FVector& OutLocation) const override;
	virtual bool GetWidgetMode(const FEditorViewportClient* ViewportClient, UE::Widget::EWidgetMode& Mode) const override;
	virtual bool GetWidgetAxisList(const FEditorViewportClient* ViewportClient, UE::Widget::EWidgetMode WidgetMode,
		EAxisList::Type& AxisList) const override;
	virtual bool GetWidgetAxisListDragOverride(const FEditorViewportClient* ViewportClient, UE::Widget::EWidgetMode WidgetMode,
		EAxisList::Type& AxisList) const override;
	virtual bool ResetValue(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy) override;
	virtual bool IsEditing() const override;
	virtual void EndEditing() override;
	//~ End FAvaVisualizerBase

protected:
	static inline constexpr float TopHeightFraction = 0.6f;
	static inline constexpr float BotHeightFraction = 0.17f;

	FProperty* HasMaxWidthProperty;
	FProperty* HasMaxHeightProperty;
	FProperty* MaxWidthProperty;
	FProperty* MaxHeightProperty;
	FProperty* ScaleProportionallyProperty;
	FProperty* CharacterKerningProperty;

	TWeakObjectPtr<UText3DComponent> TextComponent;
	mutable FBox Bounds;
	mutable int32 LineCount;
	float LineHeight;
	bool bInitialMaxWidthEnabled;
	bool bInitialMaxHeightEnabled;
	float InitialMaxWidth;
	float InitialMaxHeight;
	bool bInitialScaleProportionally;
	bool bEditingWidth;
	bool bEditingHeight;

	bool bShowGradientControls;
	bool bEditingGradientRotation_StartHandle;
	bool bEditingGradientRotation_EndHandle;
	bool bEditingGradientOffset;
	bool bEditingGradientSmoothness;
	bool bEditingKerning;
	int32 EditingKerningIndex = INDEX_NONE;
	
	float InitialGradientRotation;
	float InitialGradientOffset;
	float InitialGradientSmoothness;
	float InitialCharacterKerning;

	FVector GradientEditBeginLocation_StartHandle;
	FVector GradientEditBeginLocation_EndHandle;
	FVector GradientEditBeginLocation_Center;
	
	virtual void DrawVisualizationNotEditing(const UActorComponent* InComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;
	
	virtual void DrawVisualizationEditing(const UActorComponent* InComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, int32& InOutIconIndex) override;
	
	virtual bool HandleInputDeltaInternal(FEditorViewportClient* InViewportClient, FViewport* InViewport, const FVector& InAccumulatedTranslation,
		const FRotator& InAccumulatedRotation, const FVector& InAccumulatedScale) override;
	
	virtual void StoreInitialValues() override;
	void StoreTextMetrics(const UText3DComponent* InTextComp);
	
	virtual void TrackingStopped(FEditorViewportClient* InViewportClient, bool bInDidMove) override;

	void DrawMaxTextSizeVisualization(const UText3DComponent* InTextComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI) const;

	static bool DrawGradientCenterHandle(const UText3DComponent* InTextComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, FVector InGradientCenter);

	static bool DrawGradientSmoothnessHandle(const UText3DComponent* InTextComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, FVector InHandleLocation);

	void DrawGradientHandles(const UText3DComponent* InText3DComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI) const;

	FBox GetBoundsMax(const UText3DComponent* InTextComp) const;
	
	FVector GetWidthHandleLocation(const UText3DComponent* InTextComp) const;
	
	FVector GetHeightHandleLocation(const UText3DComponent* InTextComp) const;
	
	void GetTextActorGradientControlsLocations(const UText3DComponent* InText3DComponent,
		FVector& OutGradientCenterLocation, FVector& OutGradientStartLocation, FVector& OutGradientEndLocation) const;
	
	FVector GetGradientEndHandleLocation(const UText3DComponent* InTextComponent) const;
	FVector GetGradientCenterHandleLocation(const UText3DComponent* InTextComponent) const;
	FVector GetGradientStartHandleLocation(const UText3DComponent* InTextComponent) const;
	FVector GetGradientSmoothnessHandleLocation(const UText3DComponent* InText3DComponent) const;
	FVector GetKerningHandleLocation(const UText3DComponent* InTextComponent, int32 InIndex) const;

	void DrawMaxTextWidthButton(const UText3DComponent* InTextComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		int32 InIconIndex, const FLinearColor& InColor) const;

	void DrawMaxTextHeightButton(const UText3DComponent* InTextComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		int32 InIconIndex, const FLinearColor& InColor) const;

	void DrawScaleProportionallyButton(const UText3DComponent* InTextComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		int32 InIconIndex, const FLinearColor& InColor) const;
	
	void DrawEditGradientButton(const UText3DComponent* InTextComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		int32 InIconIndex, const FLinearColor& InColor) const;

	void DrawMaxTextWidthHandle(const UText3DComponent* InTextComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		const FLinearColor& InColor) const;

	void DrawMaxTextHeightHandle(const UText3DComponent* InTextComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		const FLinearColor& InColor) const;

	void DrawGradientLineStartHandle(const UText3DComponent* InTextComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, const FVector& InLocation, const FLinearColor& InColor) const;

	void DrawGradientLineEndHandle(const UText3DComponent* InTextComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, const FVector& InLocation, const FLinearColor& InColor) const;

	void DrawCharacterKerningButton(const UText3DComponent* InTextComponent, const FSceneView* InView, FPrimitiveDrawInterface* InPDI,
		int32 InIconIndex, const FLinearColor& InColor);

	void DrawCharacterKerningHandle(const UText3DComponent* InTextComponent, const FSceneView* InView,
		FPrimitiveDrawInterface* InPDI, const FVector& InLocation, uint16 InCharacterIndex, const FLinearColor& InColor) const;
};
