// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheTransition : ModuleRules
{
    public AvalancheTransition(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "AvalancheTag",
                "Core",
                "CoreUObject",
                "Engine",
                "StateTreeModule",
            }
        );

        if (Target.Type == TargetRules.TargetType.Editor)
        {
            PrivateDependencyModuleNames.AddRange(new string[]
            {
                "StateTreeEditorModule",
                "UnrealEd",
            });
        }
    }
}
