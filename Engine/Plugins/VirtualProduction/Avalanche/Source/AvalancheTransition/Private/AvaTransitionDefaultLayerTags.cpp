// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionDefaultLayerTags.h"

namespace UE::AvaTransition
{

FDefaultTags::FDefaultTags()
{
	TSoftObjectPtr<UAvaTagCollection> TagCollection(FSoftObjectPath(TEXT("/Avalanche/Tags/DefaultTransitionLayerTags.DefaultTransitionLayerTags")));
	DefaultLayer = FAvaTagSoftHandle(TagCollection, FAvaTagId(FGuid(0x89084F7E, 0x4D80443E, 0xAC48609D, 0x7A348C8A)));
}

const FDefaultTags& FDefaultTags::Get()
{
	static const FDefaultTags DefaultTags;
	return DefaultTags;
}

} // UE::AvaTransition
