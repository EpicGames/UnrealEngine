// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionLayer.h"
#include "Behavior/AvaTransitionBehaviorInstance.h"

bool FAvaTransitionLayerComparator::Compare(const FAvaTagHandle& InLayer) const
{
	switch (LayerCompareType)
	{
	case EAvaTransitionLayerCompareType::None:
		return false;

	// Matching Tag and Same use the same logic since the LayerContext should've been set to the Matching Tag
	case EAvaTransitionLayerCompareType::Same:
	case EAvaTransitionLayerCompareType::MatchingTag:
		return LayerContext.ContainsTag(InLayer);

	case EAvaTransitionLayerCompareType::Different:
		return !LayerContext.ContainsTag(InLayer);

	// Currently, any Tag/Name is valid to be used as a layer id
	case EAvaTransitionLayerCompareType::Any:
		return true;
	}

	checkNoEntry();
	return false;
}

bool FAvaTransitionLayerComparator::Compare(const FAvaTransitionBehaviorInstance& InInstance) const
{
	// Transition Contexts are passed by reference / pointers (see FStructView),
	// so for a Behavior Instance running a State Tree, the nodes retrieving the Transition Context are effectively
	// retrieving the Behavior Instance's Transition Context property
	if (ExcludedContext == &InInstance.GetTransitionContext())
	{
		return false;
	}

	return Compare(InInstance.GetTransitionLayer());
}
