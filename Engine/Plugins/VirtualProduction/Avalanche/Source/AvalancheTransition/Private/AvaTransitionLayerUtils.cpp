// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionLayerUtils.h"
#include "AvaTag.h"
#include "AvaTransitionContext.h"
#include "AvaTransitionSubsystem.h"
#include "Containers/Array.h"
#include "Execution/IAvaTransitionExecutor.h"
#include "StateTreePropertyBindings.h"

#define LOCTEXT_NAMESPACE "AvaTransitionLayerUtils"

TArray<const FAvaTransitionBehaviorInstance*> FAvaTransitionLayerUtils::QueryBehaviorInstances(UAvaTransitionSubsystem& InTransitionSubsystem, const FAvaTransitionLayerComparator& InComparator)
{
	TArray<const FAvaTransitionBehaviorInstance*> BehaviorInstances;

	// Get all the Behavior Instances that match the Layer Query
	InTransitionSubsystem.ForEachTransitionExecutor(
		[&BehaviorInstances, &InComparator](IAvaTransitionExecutor& InExecutor)->EAvaTransitionIterationResult
		{
			BehaviorInstances.Append(InExecutor.GetBehaviorInstances(InComparator));
			return EAvaTransitionIterationResult::Continue;
		});

	return BehaviorInstances;
}

FAvaTransitionLayerComparator FAvaTransitionLayerUtils::BuildComparator(const FAvaTransitionContext& InTransitionContext, EAvaTransitionLayerCompareType InCompareType, const FAvaTagHandle& InSpecificLayer)
{
	return BuildComparator(InTransitionContext, InCompareType, FAvaTagHandleContainer(InSpecificLayer));
}

FAvaTransitionLayerComparator FAvaTransitionLayerUtils::BuildComparator(const FAvaTransitionContext& InTransitionContext, EAvaTransitionLayerCompareType InCompareType, const FAvaTagHandleContainer& InSpecificLayers)
{
	FAvaTransitionLayerComparator LayerComparator;
	LayerComparator.LayerCompareType = InCompareType;
	LayerComparator.ExcludedContext  = &InTransitionContext;

	if (InCompareType == EAvaTransitionLayerCompareType::MatchingTag)
	{
		LayerComparator.LayerContext = InSpecificLayers;
	}
	else
	{
		LayerComparator.LayerContext = FAvaTagHandleContainer(InTransitionContext.GetTransitionLayer());
	}

	return LayerComparator;
}

#if WITH_EDITOR
FText FAvaTransitionLayerUtils::GetLayerQueryText(FLayerQueryTextParams&& InParams, const FGuid& InId, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting)
{
	FText LayerType = InBindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(InId, InParams.LayerTypePropertyName), InFormatting);

	// If Layer Type is bound assume it might be set to Specific Layer, so always set it
	if (!LayerType.IsEmpty() || InParams.LayerType == EAvaTransitionLayerCompareType::MatchingTag)
	{
		FText SpecificLayer = InBindingLookup.GetBindingSourceDisplayName(FPropertyBindingPath(InId, InParams.SpecificLayerPropertyName), InFormatting);

		if (SpecificLayer.IsEmpty())
		{
			SpecificLayer = FText::FromName(InParams.SpecificLayerName);
		}

		return InFormatting == EStateTreeNodeFormatting::RichText
			? FText::Format(LOCTEXT("SpecificLayerQueryTextRich", "<s>layer</> '<b>{0}</>'"), SpecificLayer)
			: FText::Format(LOCTEXT("SpecificLayerQueryText", "layer '{0}'"), SpecificLayer);
	}

	if (LayerType.IsEmpty())
	{
		LayerType = UEnum::GetDisplayValueAsText(InParams.LayerType).ToLower();
	}

	return InFormatting == EStateTreeNodeFormatting::RichText
		? FText::Format(LOCTEXT("LayerQueryTextRich", "<b>{0}</> <s>layer</>"), LayerType)
		: FText::Format(LOCTEXT("LayerQueryText", "{0} layer"), LayerType);
}
#endif

#undef LOCTEXT_NAMESPACE
