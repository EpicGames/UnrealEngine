// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionLibrary.h"
#include "AvaTransitionContext.h"
#include "AvaTransitionLayer.h"
#include "AvaTransitionLayerUtils.h"
#include "AvaTransitionSubsystem.h"
#include "Behavior/AvaTransitionBehaviorInstance.h"
#include "Behavior/AvaTransitionBehaviorInstanceCache.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Execution/IAvaTransitionExecutor.h"
#include "IAvaTransitionNodeInterface.h"
#include "UObject/Package.h"

namespace UE::AvaTransition::Private
{
	EAvaTransitionType GetTransitionTypeFromFilter(EAvaTransitionTypeFilter InFilterType)
	{
		switch (InFilterType)
		{
		case EAvaTransitionTypeFilter::In:  return EAvaTransitionType::In;
		case EAvaTransitionTypeFilter::Out: return EAvaTransitionType::Out;
		}
		return EAvaTransitionType::None;
	}

	ULevel* GetLevelFromContextObject(UObject* InContextObject)
	{
		if (!InContextObject)
		{
			return nullptr;
		}

		if (ULevel* ContextLevel = Cast<ULevel>(InContextObject))
		{
			return ContextLevel;
		}

		if (ULevel* ContextLevel = InContextObject->GetTypedOuter<ULevel>())
		{
			return ContextLevel;
		}

		UWorld* const World = GEngine ? GEngine->GetWorldFromContextObject(InContextObject, EGetWorldErrorMode::LogAndReturnNull) : nullptr;
		return World ? World->PersistentLevel : nullptr;
	}

	UAvaTransitionSubsystem* GetSubsystemFromLevel(ULevel* InLevelChecked)
	{
		check(InLevelChecked);

		// Use the main world of a level rather than the outer world (in the case it's a streamed in level)
		if (ensure(InLevelChecked->OwningWorld))
		{
			return InLevelChecked->OwningWorld->GetSubsystem<UAvaTransitionSubsystem>();
		}

		return nullptr;
	}
	
	/**
	 * Retrieves the transition contexts corresponding to a given level
	 * The expectation is that there will be only 1 or no transition contexts, but the API allows it to be technically possible to have more than 1.
	 */
	TArray<const FAvaTransitionContext*, TInlineAllocator<1>> GetTransitionContextsFromLevel(ULevel* InLevelChecked)
	{
		check(InLevelChecked);

		UAvaTransitionSubsystem* const TransitionSubsystem = GetSubsystemFromLevel(InLevelChecked);
		if (!TransitionSubsystem)
		{
			return {};
		}

		TArray<const FAvaTransitionContext*, TInlineAllocator<1>> TransitionContexts;

		// Gather all the transition contexts where the transition scene level matches the provided level
		TransitionSubsystem->ForEachTransitionExecutor(
			[InLevelChecked, &TransitionContexts](IAvaTransitionExecutor& InExecutor)->EAvaTransitionIterationResult
			{
				InExecutor.ForEachBehaviorInstance(
					[InLevelChecked, &TransitionContexts](const FAvaTransitionBehaviorInstance& InBehaviorInstance)
					{
						const FAvaTransitionContext& TransitionContext = InBehaviorInstance.GetTransitionContext();
						const FAvaTransitionScene* TransitionScene = TransitionContext.GetTransitionScene();

						if (TransitionScene && TransitionScene->GetLevel() == InLevelChecked)
						{
							TransitionContexts.Add(&TransitionContext);
						}
					});

				return EAvaTransitionIterationResult::Continue;
			});

		return TransitionContexts;
	}
}

const FAvaTransitionContext* UAvaTransitionLibrary::GetTransitionContext(UObject* InContextObject)
{
	using namespace UE::AvaTransition;

	if (const IAvaTransitionNodeInterface* NodeInterface = Cast<IAvaTransitionNodeInterface>(InContextObject))
	{
		return NodeInterface->GetBehaviorInstanceCache().GetTransitionContext();
	}

	if (ULevel* const Level = Private::GetLevelFromContextObject(InContextObject))
	{
		const TArray<const FAvaTransitionContext*, TInlineAllocator<1>> TransitionContexts = Private::GetTransitionContextsFromLevel(Level);
		if (TransitionContexts.IsEmpty())
		{
			return nullptr;
		}

#if UE_BUILD_DEBUG
		// Expectation is that for a given level instance,
		// there should only be one transition occurring, even if the system technically allows multiple instances to run for a behavior
		ensure(TransitionContexts.Num() == 1);
#endif
		return TransitionContexts[0];
	}

	return nullptr;
}

bool UAvaTransitionLibrary::IsTransitionActiveInLayers(UObject* InWorldContextObject
	, EAvaTransitionLayerCompareType InLayerComparisonType
	, const FAvaTagHandleContainer& InSpecificLayers
	, EAvaTransitionTypeFilter InTransitionTypeFilter)
{
	using namespace UE::AvaTransition;

	const FAvaTransitionContext* TransitionContext = GetTransitionContext(InWorldContextObject);
	if (!TransitionContext)
	{
		return false;
	}

	const FAvaTransitionScene* TransitionScene = TransitionContext->GetTransitionScene();
	if (!TransitionScene)
	{
		return false;
	}

	ULevel* TransitionLevel = TransitionScene->GetLevel();
	if (!TransitionLevel || !TransitionLevel->OwningWorld)
	{
		return false;
	}

	UAvaTransitionSubsystem* TransitionSubsystem = TransitionLevel->OwningWorld->GetSubsystem<UAvaTransitionSubsystem>();
	if (!TransitionSubsystem)
	{
		return false;
	}

	const FAvaTransitionLayerComparator Comparator = FAvaTransitionLayerUtils::BuildComparator(*TransitionContext
		, InLayerComparisonType
		, InSpecificLayers);

	const TArray<const FAvaTransitionBehaviorInstance*> BehaviorInstances = FAvaTransitionLayerUtils::QueryBehaviorInstances(*TransitionSubsystem, Comparator);
	if (BehaviorInstances.IsEmpty())
	{
		return false;
	}

	const EAvaTransitionType TransitionType = Private::GetTransitionTypeFromFilter(InTransitionTypeFilter);

	bool bHasMatchingInstance = BehaviorInstances.ContainsByPredicate(
		[TransitionScene, TransitionType](const FAvaTransitionBehaviorInstance* InInstance)
		{
			// if transition type is set, ensure that the behavior instance matches the given transition type
			if (TransitionType != EAvaTransitionType::None && TransitionType != InInstance->GetTransitionType())
			{
				return false;
			}

			const FAvaTransitionScene* OtherTransitionScene = InInstance->GetTransitionContext().GetTransitionScene();

			const EAvaTransitionComparisonResult ComparisonResult = OtherTransitionScene
				? TransitionScene->Compare(*OtherTransitionScene)
				: EAvaTransitionComparisonResult::None;

			return ComparisonResult != EAvaTransitionComparisonResult::None;
		});

	return bHasMatchingInstance;
}

bool UAvaTransitionLibrary::IsTransitionActiveInLayer(UObject* InWorldContextObject
	, EAvaTransitionComparisonResult InSceneComparisonType
	, EAvaTransitionLayerCompareType InLayerComparisonType
	, const FAvaTagHandleContainer& InSpecificLayers)
{
	return IsTransitionActiveInLayers(InWorldContextObject
		, InLayerComparisonType
		, InSpecificLayers
		, EAvaTransitionTypeFilter::Any);
}

EAvaTransitionType UAvaTransitionLibrary::GetTransitionType(UObject* InContextObject)
{
	const FAvaTransitionContext* TransitionContext = GetTransitionContext(InContextObject);
	if (!TransitionContext)
	{
		return EAvaTransitionType::None;
	}

	return TransitionContext->GetTransitionType();
}

bool UAvaTransitionLibrary::AreScenesTransitioning(UObject* InContextObject
	, const FAvaTagHandleContainer& InLayers
	, const TArray<TSoftObjectPtr<UWorld>>& InScenesToIgnore)
{
	const FAvaTransitionContext* TransitionContext = GetTransitionContext(InContextObject);
	if (!TransitionContext)
	{
		return false;
	}

	const FAvaTransitionScene* TransitionScene = TransitionContext->GetTransitionScene();
	if (!TransitionScene)
	{
		return false;
	}

	ULevel* TransitionLevel = TransitionScene->GetLevel();
	if (!TransitionLevel || !TransitionLevel->OwningWorld)
	{
		return false;
	}

	UAvaTransitionSubsystem* TransitionSubsystem = TransitionLevel->OwningWorld->GetSubsystem<UAvaTransitionSubsystem>();
	if (!TransitionSubsystem)
	{
		return false;
	}

	const FAvaTransitionLayerComparator Comparator = FAvaTransitionLayerUtils::BuildComparator(*TransitionContext
		, EAvaTransitionLayerCompareType::Different
		, FAvaTagHandleContainer());

	TArray<const FAvaTransitionBehaviorInstance*> BehaviorInstances = FAvaTransitionLayerUtils::QueryBehaviorInstances(*TransitionSubsystem, Comparator);
	if (BehaviorInstances.IsEmpty())
	{
		return false;
	}

	TSet<FString> ScenesToIgnore;
	ScenesToIgnore.Reserve(InScenesToIgnore.Num());
	for (const TSoftObjectPtr<UWorld>& SceneToIgnore : InScenesToIgnore)
	{
		ScenesToIgnore.Add(SceneToIgnore.GetLongPackageName());
	}

	for (const FAvaTransitionBehaviorInstance* BehaviorInstance : BehaviorInstances)
	{
		if (!InLayers.ContainsTag(BehaviorInstance->GetTransitionLayer()))
		{
			continue;
		}

		const FAvaTransitionScene* const OtherTransitionScene = BehaviorInstance->GetTransitionContext().GetTransitionScene();

		// skip scenes marked as needing discard
		if (!OtherTransitionScene || OtherTransitionScene->HasAnyFlags(EAvaTransitionSceneFlags::NeedsDiscard))
		{
			continue;
		}

		const ULevel* const OtherLevel = OtherTransitionScene->GetLevel();
		if (!OtherLevel)
		{
			continue;
		}

		const UPackage* const OtherPackage = OtherLevel->GetPackage();
		if (!OtherPackage)
		{
			continue;
		}

		// Remove the /Temp from the Package Name
		FString PackageName = OtherPackage->GetName();
		if (PackageName.StartsWith(TEXT("/Temp")))
		{
			PackageName.RightChopInline(-1 + sizeof(TEXT("/Temp")) / sizeof(TCHAR));
		}

		// Remove the _LevelInstance_[Num] from the Package Name
		const int32 LevelInstancePosition = PackageName.Find(TEXT("_LevelInstance_"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		if (LevelInstancePosition != INDEX_NONE)
		{
			PackageName.LeftChopInline(PackageName.Len() - LevelInstancePosition);
		}

		// If this scene isn't part of those to ignore, then it's a valid transitioning scene
		if (!ScenesToIgnore.Contains(PackageName))
		{
			return true;
		}
	}

	return false;
}

TScriptInterface<IAvaTransitionBehavior> UAvaTransitionLibrary::GetTransitionBehavior(UObject* InContextObject)
{
	using namespace UE::AvaTransition;

	if (const IAvaTransitionNodeInterface* NodeInterface = Cast<IAvaTransitionNodeInterface>(InContextObject))
	{
		if (IAvaTransitionBehavior* TransitionBehavior = NodeInterface->GetBehaviorInstanceCache().GetTransitionBehavior())
		{
			return &TransitionBehavior->AsUObject();
		}
	}

	if (ULevel* Level = Private::GetLevelFromContextObject(InContextObject))
	{
		if (UAvaTransitionSubsystem* const TransitionSubsystem = Private::GetSubsystemFromLevel(Level))
		{
			if (IAvaTransitionBehavior* TransitionBehavior = TransitionSubsystem->GetTransitionBehavior(Level))
			{
				return &TransitionBehavior->AsUObject();
			}
		}
	}
	return nullptr;
}

const UAvaTransitionTree* UAvaTransitionLibrary::GetTransitionTree(UObject* InContextObject)
{
	if (const IAvaTransitionBehavior* TransitionBehavior = UAvaTransitionLibrary::GetTransitionBehavior(InContextObject).GetInterface())
	{
		return TransitionBehavior->GetTransitionTree();
	}
	return nullptr;
}
