// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionModule.h"
#include "AvaTransitionLog.h"
#include "Rendering/AvaTransitionSceneViewExtension.h"
#include "SceneViewExtension.h"

DEFINE_LOG_CATEGORY(LogAvaTransition);

void FAvaTransitionModule::StartupModule()
{
	OnPostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FAvaTransitionModule::PostEngineInit);
}

void FAvaTransitionModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().Remove(OnPostEngineInitHandle);
	OnPostEngineInitHandle.Reset();

	TransitionSceneViewExtension.Reset();
}

IAvaTransitionModule::FOnValidateTransitionTree& FAvaTransitionModule::GetOnValidateTransitionTree()
{
	return OnValidateStateTree;
}

void FAvaTransitionModule::PostEngineInit()
{
	TransitionSceneViewExtension = FSceneViewExtensions::NewExtension<FAvaTransitionSceneViewExtension>();
}

IMPLEMENT_MODULE(FAvaTransitionModule, AvalancheTransition)
