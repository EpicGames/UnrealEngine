// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaTransitionModule.h"
#include "Templates/SharedPointer.h"

class FAvaTransitionSceneViewExtension;

class FAvaTransitionModule : public IAvaTransitionModule
{
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	//~ Begin IAvaTransitionModule
	virtual FOnValidateTransitionTree& GetOnValidateTransitionTree() override;
	//~ End IAvaTransitionModule

	void PostEngineInit();

	TSharedPtr<FAvaTransitionSceneViewExtension> TransitionSceneViewExtension;

	FOnValidateTransitionTree OnValidateStateTree;

	FDelegateHandle OnPostEngineInitHandle;
};
