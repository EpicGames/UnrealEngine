// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionNode.h"
#include "AvaTransitionContext.h"
#include "AvaTransitionSubsystem.h"
#include "StateTreeLinker.h"

bool FAvaTransitionNode::LinkNode(FStateTreeLinker& InLinker)
{
	InLinker.LinkExternalData(TransitionContextHandle);
	InLinker.LinkExternalData(TransitionSubsystemHandle);
	return true;
}
