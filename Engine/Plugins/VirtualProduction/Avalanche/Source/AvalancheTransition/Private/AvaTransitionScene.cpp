// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionScene.h"

FAvaTransitionScene::FAvaTransitionScene(FStateTreeDataView InDataView)
	: DataView(InDataView)
{
}

EAvaTransitionComparisonResult FAvaTransitionScene::Compare(const FAvaTransitionScene& InOther) const
{
	return EAvaTransitionComparisonResult::None;
}

ULevel* FAvaTransitionScene::GetLevel() const
{
	return nullptr;
}

void FAvaTransitionScene::SetFlags(EAvaTransitionSceneFlags InFlags)
{
	if (!HasAllFlags(InFlags))
	{
		Flags |= InFlags;
		OnFlagsChanged();
	}
}

bool FAvaTransitionScene::HasAnyFlags(EAvaTransitionSceneFlags InFlags) const
{
	return EnumHasAnyFlags(Flags, InFlags);
}

bool FAvaTransitionScene::HasAllFlags(EAvaTransitionSceneFlags InFlags) const
{
	return EnumHasAllFlags(Flags, InFlags);
}

const FStateTreeDataView& FAvaTransitionScene::GetDataView() const
{
	return DataView;
}
