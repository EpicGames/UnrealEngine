// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionSubsystem.h"
#include "Behavior/AvaTransitionBehaviorActor.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Execution/IAvaTransitionExecutor.h"
#include "StateTreeExecutionTypes.h"

void UAvaTransitionSubsystem::RegisterTransitionBehavior(const ULevel* InLevel, IAvaTransitionBehavior* InBehavior)
{
	TransitionBehaviors.Add(InLevel, TWeakInterfacePtr<IAvaTransitionBehavior>(InBehavior));
}

IAvaTransitionBehavior* UAvaTransitionSubsystem::GetOrCreateTransitionBehavior(ULevel* InLevel)
{
	UWorld* const World = GetWorld();
	if (!World)
	{
		return nullptr;
	}

	// if passed in nullptr, set it to the persistent level
	if (!InLevel)
	{
		InLevel = World->PersistentLevel;
	}

	// Ensure the provided level belongs to the world of this subsystem
	if (!InLevel || !ensureAlways(InLevel->OwningWorld == World))
	{
		return nullptr;
	}

	// Return Existing instead of creating a new one
	if (IAvaTransitionBehavior* ExistingBehavior = GetTransitionBehavior(InLevel))
	{
		return ExistingBehavior;
	}

	// If no existing transition behavior found, then spawn a new one
	FActorSpawnParameters SpawnParameters;
	SpawnParameters.OverrideLevel = InLevel;
#if WITH_EDITOR
	SpawnParameters.bHideFromSceneOutliner = true;
#endif

	return InLevel->OwningWorld->SpawnActor<AAvaTransitionBehaviorActor>(SpawnParameters);		
}

IAvaTransitionBehavior* UAvaTransitionSubsystem::GetTransitionBehavior(const ULevel* InLevel) const
{
	UWorld* const World = GetWorld();
	if (!World)
	{
		return nullptr;
	}

	// if passed in nullptr, set it to the persistent level
	if (!InLevel)
	{
		InLevel = World->PersistentLevel;
	}

	// Ensure the provided level belongs to the world of this subsystem
	if (!InLevel || !ensureAlways(InLevel->OwningWorld == World))
	{
		return nullptr;
	}

	// Find Behavior through the Registered Entries
	if (const TWeakInterfacePtr<IAvaTransitionBehavior>* FoundInterface = TransitionBehaviors.Find(InLevel))
	{
		return FoundInterface->Get();
	}

	// Find Behavior through the Level itself
	if (IAvaTransitionBehavior* TransitionBehavior = FindTransitionBehavior(InLevel))
	{
		// This transition behavior wasn't registered properly, so register it here
		const_cast<UAvaTransitionSubsystem*>(this)->RegisterTransitionBehavior(InLevel, TransitionBehavior);
		return TransitionBehavior;
	}

	return nullptr;
}

IAvaTransitionBehavior* UAvaTransitionSubsystem::FindTransitionBehavior(const ULevel* InLevel)
{
	// find existing transition behavior actor in level
	AAvaTransitionBehaviorActor* TransitionBehaviorActor;
	if (InLevel->Actors.FindItemByClass(&TransitionBehaviorActor))
	{
		return TransitionBehaviorActor;
	}
	return nullptr;
}

void UAvaTransitionSubsystem::RegisterTransitionExecutor(const TSharedRef<IAvaTransitionExecutor>& InExecutor)
{
	TransitionExecutors.Add(InExecutor);
}

void UAvaTransitionSubsystem::ForEachTransitionExecutor(TFunctionRef<EAvaTransitionIterationResult(IAvaTransitionExecutor&)> InFunc)
{
	for (TArray<TWeakPtr<IAvaTransitionExecutor>>::TIterator Iter(TransitionExecutors); Iter; ++Iter)
	{
		TSharedPtr<IAvaTransitionExecutor> Executor = Iter->Pin();
		if (Executor.IsValid())
		{
			EAvaTransitionIterationResult Result = InFunc(*Executor);
			if (Result == EAvaTransitionIterationResult::Break)
			{
				break;
			}
			check(Result == EAvaTransitionIterationResult::Continue);
		}
		else
		{
			Iter.RemoveCurrent();
		}
	}
}

bool UAvaTransitionSubsystem::DoesSupportWorldType(const EWorldType::Type InWorldType) const
{
	return InWorldType == EWorldType::Game
		|| InWorldType == EWorldType::Editor
		|| InWorldType == EWorldType::PIE
		|| InWorldType == EWorldType::GamePreview;
}

void UAvaTransitionSubsystem::PostInitialize()
{
	Super::PostInitialize();

	UWorld* const World = GetWorld();
	if (!World)
	{
		return;
	}

	// Register Existing Transition State Actors
	for (FConstLevelIterator LevelIterator = World->GetLevelIterator(); LevelIterator; ++LevelIterator)
	{
		ULevel* const Level = *LevelIterator;

		AAvaTransitionBehaviorActor* TransitionStateActor = nullptr;
		if (Level && Level->Actors.FindItemByClass(&TransitionStateActor))
		{
			RegisterTransitionBehavior(Level, TransitionStateActor);
		}
	}
}
