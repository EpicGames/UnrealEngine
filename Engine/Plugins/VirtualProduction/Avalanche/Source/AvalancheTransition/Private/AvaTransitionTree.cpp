// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionTree.h"
#include "StateTreeTaskBase.h"

FAvaTagHandle UAvaTransitionTree::GetTransitionLayer() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return TransitionLayer_DEPRECATED;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UAvaTransitionTree::SetTransitionLayer(FAvaTagHandle InTransitionLayer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TransitionLayer_DEPRECATED = InTransitionLayer;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAvaTransitionTree::IsEnabled() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return bEnabled_DEPRECATED;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UAvaTransitionTree::SetEnabled(bool bInEnabled)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bEnabled_DEPRECATED = bInEnabled;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAvaTransitionTree::ContainsTask(const UScriptStruct* InTaskStruct) const
{
	for (const FCompactStateTreeState& State : GetStates())
	{
		if (!State.bEnabled)
		{
			continue;
		}

		for (int32 TaskIndex = State.TasksBegin; TaskIndex < State.TasksBegin + State.TasksNum; ++TaskIndex)
		{
			const FConstStructView Node = GetNodes()[TaskIndex];

			const UScriptStruct* TaskStruct = Node.GetScriptStruct();
			if (TaskStruct && TaskStruct->IsChildOf(InTaskStruct))
			{
				const FStateTreeTaskBase* TaskNode = Node.GetPtr<const FStateTreeTaskBase>();
				if (TaskNode && TaskNode->bTaskEnabled)
				{
					return true;
				}
			}
		}
	}

	return false;
}

void UAvaTransitionTree::SetInstancingMode(EAvaTransitionInstancingMode InInstancingMode)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InstancingMode_DEPRECATED = InInstancingMode;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

EAvaTransitionInstancingMode UAvaTransitionTree::GetInstancingMode() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return InstancingMode_DEPRECATED;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FName UAvaTransitionTree::GetEnabledPropertyName()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GET_MEMBER_NAME_CHECKED(UAvaTransitionTree, bEnabled_DEPRECATED);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
