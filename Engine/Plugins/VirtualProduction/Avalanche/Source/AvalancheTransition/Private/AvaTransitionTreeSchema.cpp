// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionTreeSchema.h"
#include "AvaTransitionContext.h"
#include "AvaTransitionScene.h"
#include "Conditions/AvaTransitionCondition.h"
#include "Conditions/AvaTransitionConditionBlueprint.h"
#include "StateTreeEvaluatorBase.h"
#include "StateTreePropertyFunctionBase.h"
#include "StateTreeTaskBase.h"
#include "Subsystems/WorldSubsystem.h"
#include "Tasks/AvaTransitionTask.h"
#include "Tasks/AvaTransitionTaskBlueprint.h"

bool UAvaTransitionTreeSchema::IsStructAllowed(const UScriptStruct* InScriptStruct) const
{
	return InScriptStruct->IsChildOf(FAvaTransitionTask::StaticStruct())
		|| InScriptStruct->IsChildOf(FAvaTransitionCondition::StaticStruct())
		|| InScriptStruct->IsChildOf(FStateTreePropertyFunctionCommonBase::StaticStruct());
}

bool UAvaTransitionTreeSchema::IsClassAllowed(const UClass* InClass) const
{
	return InClass->IsChildOf<UAvaTransitionTaskBlueprint>()
		|| InClass->IsChildOf<UAvaTransitionConditionBlueprint>();
}

bool UAvaTransitionTreeSchema::IsExternalItemAllowed(const UStruct& InStruct) const
{
	return InStruct.IsChildOf(FAvaTransitionContext::StaticStruct())
		|| InStruct.IsChildOf(UWorldSubsystem::StaticClass());
}
