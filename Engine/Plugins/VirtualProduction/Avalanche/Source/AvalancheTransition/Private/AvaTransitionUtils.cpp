// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionUtils.h"
#include "Behavior/AvaTransitionBehaviorActor.h"
#include "Execution/AvaTransitionExecutionContext.h"

namespace UE::AvaTransition
{

const FAvaTransitionBehaviorInstance* GetBehaviorInstance(const FStateTreeExecutionContext& InExecutionContext)
{
	UObject* Owner = InExecutionContext.GetOwner();
	if (ensure(Owner && Owner->IsA<AAvaTransitionBehaviorActor>()))
	{
		return static_cast<const FAvaTransitionExecutionContext&>(InExecutionContext).GetBehaviorInstance();
	}
	return nullptr;
}

}
