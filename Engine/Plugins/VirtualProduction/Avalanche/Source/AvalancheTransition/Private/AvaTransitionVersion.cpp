// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionVersion.h"
#include "Serialization/CustomVersion.h"

namespace UE::AvaTransition
{
	const FGuid FBehaviorVersion::Guid(0x57DE9A66, 0x3FB04693, 0x8ED244A4, 0x2FD1D9BD);
	FCustomVersionRegistration GRegisterBehaviorVersion(FBehaviorVersion::Guid, FBehaviorVersion::LatestVersion, TEXT("AvaTransitionBehaviorVersion"));

} // UE::AvaTransition
