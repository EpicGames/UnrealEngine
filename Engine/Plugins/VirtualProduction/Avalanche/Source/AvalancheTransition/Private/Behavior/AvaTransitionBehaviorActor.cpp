// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionBehaviorActor.h"
#include "AvaTransitionContext.h"
#include "AvaTransitionDefaultLayerTags.h"
#include "AvaTransitionEnums.h"
#include "AvaTransitionScene.h"
#include "AvaTransitionSubsystem.h"
#include "AvaTransitionTree.h"
#include "AvaTransitionTreeSchema.h"
#include "AvaTransitionVersion.h"
#include "Engine/World.h"
#include "IAvaTransitionModule.h"
#include "StateTreeDelegates.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeTaskBase.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

AAvaTransitionBehaviorActor::AAvaTransitionBehaviorActor()
{
	// TickBehavior is called by the Subsystem separately  
	PrimaryActorTick.bCanEverTick = false;

	TransitionTree = CreateDefaultSubobject<UAvaTransitionTree>(TEXT("Transition Logic"));
	StateTreeReference.SetStateTree(TransitionTree);
}

UAvaTransitionTree* AAvaTransitionBehaviorActor::GetTransitionTree() const
{
	return Cast<UAvaTransitionTree>(StateTreeReference.GetMutableStateTree());
}

FAvaTagHandle AAvaTransitionBehaviorActor::GetTransitionLayer() const
{
	return TransitionLayer;
}

void AAvaTransitionBehaviorActor::SetTransitionLayer(FAvaTagHandle InTransitionLayer)
{
	TransitionLayer = InTransitionLayer;
}

bool AAvaTransitionBehaviorActor::IsEnabled() const
{
	return bEnabled;
}

void AAvaTransitionBehaviorActor::SetEnabled(bool bInEnabled)
{
	bEnabled = bInEnabled;
}

EAvaTransitionInstancingMode AAvaTransitionBehaviorActor::GetInstancingMode() const
{
	return InstancingMode;
}

void AAvaTransitionBehaviorActor::SetInstancingMode(EAvaTransitionInstancingMode InInstancingMode)
{
	InstancingMode = InInstancingMode;
}

const FStateTreeReference& AAvaTransitionBehaviorActor::GetStateTreeReference() const
{
	return StateTreeReference;
}

#if WITH_EDITOR
void AAvaTransitionBehaviorActor::ForEachDetailsEditableProperty(TFunctionRef<void(const FPropertyContext&)> InFunc) const
{
	IAvaTransitionBehavior::FPropertyContext DefaultPropertyContext;
	auto CallFunc = [&InFunc, &DefaultPropertyContext](FName InPropertyName)
		{
			DefaultPropertyContext.Name = InPropertyName;
			InFunc(DefaultPropertyContext);
		};

	CallFunc(GET_MEMBER_NAME_CHECKED(AAvaTransitionBehaviorActor, bEnabled));
	CallFunc(GET_MEMBER_NAME_CHECKED(AAvaTransitionBehaviorActor, InstancingMode));
	CallFunc(GET_MEMBER_NAME_CHECKED(AAvaTransitionBehaviorActor, TransitionLayer));
	CallFunc(GET_MEMBER_NAME_CHECKED(AAvaTransitionBehaviorActor, StateTreeReference));
}
#endif

void AAvaTransitionBehaviorActor::PostActorCreated()
{
	Super::PostActorCreated();

	ValidateTransitionTree();

	if (UAvaTransitionSubsystem* const TransitionSubsystem = GetTransitionSubsystem())
	{
		TransitionSubsystem->RegisterTransitionBehavior(GetLevel(), this);
	}
}

void AAvaTransitionBehaviorActor::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(UE::AvaTransition::FBehaviorVersion::Guid);
}

void AAvaTransitionBehaviorActor::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerCustomVersion(UE::AvaTransition::FBehaviorVersion::Guid) < UE::AvaTransition::FBehaviorVersion::InstanceProperties)
	{
		if (TransitionTree)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			TransitionLayer = TransitionTree->GetTransitionLayer();
			InstancingMode = TransitionTree->GetInstancingMode();
			bEnabled = TransitionTree->IsEnabled();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	ValidateTransitionTree();
}

#if WITH_EDITOR
void AAvaTransitionBehaviorActor::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);
	UpdateStateTreeReference();
}

void AAvaTransitionBehaviorActor::PostEditImport()
{
	Super::PostEditImport();
	UpdateStateTreeReference();
}

void AAvaTransitionBehaviorActor::PostEditUndo()
{
	Super::PostEditUndo();
	UpdateStateTreeReference();
}
#endif

void AAvaTransitionBehaviorActor::UpdateStateTreeReference()
{
	if (!StateTreeReference.GetStateTree())
	{
		StateTreeReference.SetStateTree(TransitionTree);
	}
}

bool AAvaTransitionBehaviorActor::IsTransitionTreeOverridden() const
{
	return GetTransitionTree() != TransitionTree;
}

UAvaTransitionSubsystem* AAvaTransitionBehaviorActor::GetTransitionSubsystem() const
{
	UWorld* const World = GetWorld();
	return World ? World->GetSubsystem<UAvaTransitionSubsystem>() : nullptr;
}

void AAvaTransitionBehaviorActor::ValidateTransitionTree()
{
	if (IsTransitionTreeOverridden())
	{
		return;
	}

	if (!ensureAlwaysMsgf(TransitionTree, TEXT("Transition Tree is null. Cannot not validate tree")))
	{
		return;
	}

	IAvaTransitionModule::FOnValidateTransitionTree& OnValidateTransitionTree = IAvaTransitionModule::Get().GetOnValidateTransitionTree();

	if (!OnValidateTransitionTree.IsBound())
	{
#if WITH_EDITOR
		if (GEditor)
		{
			ensureAlwaysMsgf(false, TEXT("OnValidateTransitionTree expected to be bound by FAvaTransitionEditorModule"));
		}
#endif
		return;
	}

	OnValidateTransitionTree.Execute(TransitionTree);
}
