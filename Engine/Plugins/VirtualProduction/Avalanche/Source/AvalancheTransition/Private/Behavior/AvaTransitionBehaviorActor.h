// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionContext.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "GameFramework/Actor.h"
#include "StateTreeExecutionTypes.h"
#include "StateTreeInstanceData.h"
#include "StateTreeReference.h"
#include "AvaTransitionBehaviorActor.generated.h"

class UAvaTransitionBehaviorInstance;
class UAvaTransitionSubsystem;
struct FStateTreeExecutionContext;

UCLASS(NotPlaceable, Hidden, DisplayName = "Motion Design Transition Behavior Actor")
class AAvaTransitionBehaviorActor : public AActor, public IAvaTransitionBehavior
{
	GENERATED_BODY()

public:
	AAvaTransitionBehaviorActor();

protected:
	//~ Begin IAvaTransitionBehavior
	virtual UObject& AsUObject() override { return *this; }
	virtual UAvaTransitionTree* GetTransitionTree() const override;
	virtual FAvaTagHandle GetTransitionLayer() const override;
	virtual void SetTransitionLayer(FAvaTagHandle InTransitionLayer) override;
	virtual bool IsEnabled() const override;
	virtual void SetEnabled(bool bInEnabled) override;
	virtual EAvaTransitionInstancingMode GetInstancingMode() const override;
	virtual void SetInstancingMode(EAvaTransitionInstancingMode InInstancingMode) override;
	virtual const FStateTreeReference& GetStateTreeReference() const override;
#if WITH_EDITOR
	virtual void ForEachDetailsEditableProperty(TFunctionRef<void(const FPropertyContext&)> InFunc) const override;
#endif
	//~ End IAvaTransitionBehavior

	//~ Begin AActor
	virtual void PostActorCreated() override;
#if WITH_EDITOR
	virtual bool IsSelectable() const override { return false; }
	virtual bool SupportsExternalPackaging() const override { return false; }
#endif
	//~ End AActor

	//~ Begin UObject
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	virtual void PostEditImport() override;
	virtual void PostEditUndo() override;
#endif
	//~ End UObject

	/** Updates the underlying state tree reference, setting it to TransitionTree if null. */
	void UpdateStateTreeReference();

	/** Determines whether the transition tree to use in transition logic is the overridden tree instead of the owned one */
	bool IsTransitionTreeOverridden() const;

private:
	UAvaTransitionSubsystem* GetTransitionSubsystem() const;

	void ValidateTransitionTree();

	UPROPERTY()
	TObjectPtr<UAvaTransitionTree> TransitionTree;

	UPROPERTY(EditAnywhere, Category="Transition Logic", DisplayName = "Transition Tree", meta=(Schema="/Script/AvalancheTransition.AvaTransitionTreeSchema"))
	mutable FStateTreeReference StateTreeReference;

	/** Transition Layer to use */
	UPROPERTY(EditAnywhere, Category="Transition Logic")
	FAvaTagHandle TransitionLayer;

	UPROPERTY(EditAnywhere, Category="Transition Logic")
	EAvaTransitionInstancingMode InstancingMode = EAvaTransitionInstancingMode::New;

	/**
	 * Determines whether this Transition Logic is enabled.
	 * Can be overriden by a Transition Instance to force the logic to run regardless
	 */
	UPROPERTY(EditAnywhere, Category="Transition Logic", meta=(DisplayPriority=0))
	bool bEnabled = false;
};
