// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behavior/AvaTransitionBehaviorInstance.h"
#include "AvaTransitionLog.h"
#include "AvaTransitionTree.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Execution/AvaTransitionExecutionContext.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeReference.h"
#include "Subsystems/WorldSubsystem.h"

FAvaTransitionBehaviorInstance& FAvaTransitionBehaviorInstance::SetBehavior(IAvaTransitionBehavior* InBehavior)
{
	BehaviorWeak = InBehavior;
	return *this;
}

bool FAvaTransitionBehaviorInstance::IsEnabled() const
{
	const IAvaTransitionBehavior* TransitionBehavior = GetBehavior();
	return TransitionBehavior && TransitionBehavior->IsEnabled();
}

IAvaTransitionBehavior* FAvaTransitionBehaviorInstance::GetBehavior() const
{
	return BehaviorWeak.Get();
}

const UAvaTransitionTree* FAvaTransitionBehaviorInstance::GetTransitionTree() const
{
	if (const IAvaTransitionBehavior* Behavior = GetBehavior())
	{
		return Behavior->GetTransitionTree();
	}
	return nullptr;
}

FAvaTagHandle FAvaTransitionBehaviorInstance::GetTransitionLayer() const
{
	return TransitionContext.GetTransitionLayer();
}

EAvaTransitionType FAvaTransitionBehaviorInstance::GetTransitionType() const
{
	return TransitionContext.GetTransitionType();
}

bool FAvaTransitionBehaviorInstance::IsRunning() const
{
	return RunStatus == EStateTreeRunStatus::Running;
}

const FAvaTransitionContext& FAvaTransitionBehaviorInstance::GetTransitionContext() const
{
	return TransitionContext;
}

FAvaTransitionContext& FAvaTransitionBehaviorInstance::GetTransitionContext()
{
	return TransitionContext;
}

FAvaTransitionSceneOwner FAvaTransitionBehaviorInstance::GetTransitionSceneOwner() const
{
	return TransitionSceneOwner;
}

void FAvaTransitionBehaviorInstance::SetTransitionType(EAvaTransitionType InTransitionType)
{
	TransitionContext.TransitionType = InTransitionType;
}

bool FAvaTransitionBehaviorInstance::Setup()
{
	// Mark Run Status as running on setup so that on start all the nodes are on this status,
	// even if their state tree hasn't started yet
	RunStatus = EStateTreeRunStatus::Running;

	TOptional<FAvaTransitionExecutionContext> Context;
	return UpdateContext(Context);
}

void FAvaTransitionBehaviorInstance::Start()
{
	// If this Instance is not Enabled for Transition, finish immediately
	if (!IsEnabled())
	{
		RunStatus = EStateTreeRunStatus::Succeeded;
		return;
	}

	TOptional<FAvaTransitionExecutionContext> Context;
	if (UpdateContext(Context))
	{
		const IAvaTransitionBehavior* Behavior = GetBehavior();
		check(Behavior); // Assume that we have valid behavior if we were able to create the context.
		const FStateTreeReference& StateTreeReference = Behavior->GetStateTreeReference();

		RunStatus = Context->Start(&StateTreeReference.GetParameters());
	}
	else
	{
		RunStatus = EStateTreeRunStatus::Failed;
	}

	ConditionallyStop();
}

void FAvaTransitionBehaviorInstance::Tick(float InDeltaSeconds)
{
	if (RunStatus != EStateTreeRunStatus::Running)
	{
		return;
	}

	TOptional<FAvaTransitionExecutionContext> Context;
	if (UpdateContext(Context))
	{
		RunStatus = Context->Tick(InDeltaSeconds);
	}
	else
	{
		RunStatus = EStateTreeRunStatus::Failed;
	}

	ConditionallyStop();
}

void FAvaTransitionBehaviorInstance::Stop()
{
	TOptional<FAvaTransitionExecutionContext> Context;
	if (UpdateContext(Context))
	{
		RunStatus = Context->Stop();
	}
	else
	{
		RunStatus = EStateTreeRunStatus::Stopped;
	}
}

void FAvaTransitionBehaviorInstance::SetOverrideLayer(const FAvaTagHandle& InOverrideLayer)
{
	OverrideLayer = InOverrideLayer;
}

void FAvaTransitionBehaviorInstance::SetLogContext(const FString& InContext)
{
	LogContext = InContext;
}

void FAvaTransitionBehaviorInstance::AddReferencedObjects(FReferenceCollector& InCollector)
{
	InstanceData.AddStructReferencedObjects(InCollector);
}

void FAvaTransitionBehaviorInstance::ConditionallyStop()
{
	// if the run status was set in Start/Tick with anything other than running (e.g. Succeeded/Failed/Stop) it should call stop
	if (RunStatus != EStateTreeRunStatus::Running)
	{
		Stop();
	}
}

bool FAvaTransitionBehaviorInstance::UpdateContext(TOptional<FAvaTransitionExecutionContext>& OutContext)
{
	if (ValidateTransitionScene())
	{
		IAvaTransitionBehavior* Behavior = GetBehavior();
		UpdateTransitionLayers(Behavior);
		return MakeContext(Behavior, OutContext);
	}
	return false;
}

bool FAvaTransitionBehaviorInstance::ValidateTransitionScene()
{
	if (!TransitionSceneOwner.IsValid())
	{
		//todo: support having a behavior instance run without a Transition Scene instanced
		UE_LOGF(LogAvaTransition, Error
			, "SetContextRequirements failed for '%ls'. Transition Scene Owner is invalid!"
			, *LogContext);
		return false;
	}

	FAvaTransitionScene* TransitionScene = TransitionContext.GetTransitionScene();
	if (!TransitionScene)
	{
		UE_LOGF(LogAvaTransition, Error
			, "SetContextRequirements failed for '%ls'. Transition Scene is null"
			, *LogContext);
		return false;
	}

	return true;
}

void FAvaTransitionBehaviorInstance::UpdateTransitionLayers(const IAvaTransitionBehavior* InBehavior)
{
	FAvaTransitionScene& TransitionScene = *TransitionContext.GetTransitionScene();
	TransitionContext.TransitionLayer = InBehavior ? InBehavior->GetTransitionLayer() : FAvaTagHandle();

	// Give opportunity for the Scene to override the layer (if not already overriden)
	if (!OverrideLayer.IsSet())
	{
		FAvaTagHandle TagHandle;
		TransitionScene.GetOverrideTransitionLayer(TagHandle);
		if (TagHandle.IsValid())
		{
			OverrideLayer = TagHandle;
		}
	}

	if (OverrideLayer.IsSet())
	{
		TransitionContext.TransitionLayer = *OverrideLayer;
	}
}

bool FAvaTransitionBehaviorInstance::MakeContext(IAvaTransitionBehavior* InBehavior, TOptional<FAvaTransitionExecutionContext>& OutContext)
{
	FAvaTransitionScene* TransitionScene = TransitionContext.GetTransitionScene();
	if (!InBehavior || !TransitionScene)
	{
		return false;
	}

	const ULevel* Level = TransitionScene->GetLevel();

	const UWorld* World = Level ? Level->OwningWorld.Get() : nullptr;

	if (!IsValid(World))
	{
		UE_LOGF(LogAvaTransition, Error
			, "SetContextRequirements failed for '%ls'. World is invalid"
			, *LogContext);
		return false;
	}

	const FStateTreeReference& StateTreeReference = InBehavior->GetStateTreeReference();
	if (!StateTreeReference.GetStateTree())
	{
		UE_LOGF(LogAvaTransition, Error
			, "SetContextRequirements failed for '%ls'. StateTree is invalid"
			, *LogContext);
		return false;
	}

	FAvaTransitionExecutionContext& Context = OutContext.Emplace(*this, InBehavior->AsUObject(), *StateTreeReference.GetStateTree(), InstanceData);
	if (!Context.IsValid())
	{
		return false;
	}

	Context.SetCollectExternalDataCallback(FOnCollectStateTreeExternalData::CreateLambda(
		[World, &TransitionContext = TransitionContext]
		(const FStateTreeExecutionContext& Context, const UStateTree* StateTree, TArrayView<const FStateTreeExternalDataDesc> ExternalDescs, TArrayView<FStateTreeDataView> OutDataViews)
		{
			check(ExternalDescs.Num() == OutDataViews.Num());
			for (int32 Index = 0; Index < ExternalDescs.Num(); Index++)
			{
				const FStateTreeExternalDataDesc& Desc = ExternalDescs[Index];
				if (!Desc.Struct)
				{
					continue;
				}

				if (Desc.Struct->IsChildOf(FAvaTransitionContext::StaticStruct()))
				{
					OutDataViews[Index] = FStructView::Make(TransitionContext);
				}
				else if (Desc.Struct->IsChildOf(UWorldSubsystem::StaticClass()))
				{
					UWorldSubsystem* Subsystem = World->GetSubsystemBase(Cast<UClass>(const_cast<UStruct*>(Desc.Struct.Get())));
					OutDataViews[Index] = FStateTreeDataView(Subsystem);
				}
			}
				
			return true;
		})
	);

	if (!Context.AreContextDataViewsValid())
	{
		UE_LOGF(LogAvaTransition, Error
			, "SetContextRequirements failed for '%ls'. Missing external data requirements. StateTree will not update."
			, *LogContext);

		return false;
	}

	return true;
}
