// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionUtils.h"
#include "Behavior/AvaTransitionBehaviorInstance.h"
#include "Behavior/AvaTransitionBehaviorInstanceCache.h"

const FAvaTransitionContext* FAvaTransitionBehaviorInstanceCache::GetTransitionContext() const
{
	if (CachedBehaviorInstance && CachedSceneOwner.IsValid())
	{
		return &CachedBehaviorInstance->GetTransitionContext();
	}
	return nullptr;
}

IAvaTransitionBehavior* FAvaTransitionBehaviorInstanceCache::GetTransitionBehavior() const
{
	if (CachedBehaviorInstance && CachedSceneOwner.IsValid())
	{
		return CachedBehaviorInstance->GetBehavior();
	}
	return nullptr;
}

void FAvaTransitionBehaviorInstanceCache::SetCachedInstanceDataFromContext(const FStateTreeExecutionContext& InExecutionContext)
{
	if (const FAvaTransitionBehaviorInstance* BehaviorInstance = UE::AvaTransition::GetBehaviorInstance(InExecutionContext))
	{
		CachedBehaviorInstance = BehaviorInstance;
		CachedSceneOwner = BehaviorInstance->GetTransitionSceneOwner();
	}
	else
	{
		ClearCachedInstanceData();
	}
}

void FAvaTransitionBehaviorInstanceCache::ClearCachedInstanceData()
{
	CachedBehaviorInstance = nullptr;
	CachedSceneOwner = FAvaTransitionSceneOwner(nullptr);
}
