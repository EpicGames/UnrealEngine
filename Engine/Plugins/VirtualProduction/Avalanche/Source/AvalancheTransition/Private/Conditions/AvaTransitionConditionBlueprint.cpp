// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/AvaTransitionConditionBlueprint.h"
#include "Misc/ScopeExit.h"

bool UAvaTransitionConditionBlueprint::TestCondition(FStateTreeExecutionContext& InContext) const
{
	if (bHasTestCondition)
	{
		BehaviorInstanceCache.SetCachedInstanceDataFromContext(InContext);
		ON_SCOPE_EXIT
		{
			BehaviorInstanceCache.ClearCachedInstanceData();
		};
		return Super::TestCondition(InContext);
	}
	return false;
}

const FAvaTransitionBehaviorInstanceCache& UAvaTransitionConditionBlueprint::GetBehaviorInstanceCache() const
{
	return BehaviorInstanceCache;
}
