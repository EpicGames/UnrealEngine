// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/AvaTransitionLayerCondition.h"
#include "AvaTransitionContext.h"
#include "AvaTransitionLayerUtils.h"
#include "AvaTransitionSubsystem.h"
#include "AvaTransitionUtils.h"
#include "StateTreeExecutionContext.h"

#define LOCTEXT_NAMESPACE "AvaTransitionLayerCondition"

#if WITH_EDITOR
FText FAvaTransitionLayerCondition::GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const
{
	const FInstanceDataType& InstanceData = InInstanceDataView.Get<FInstanceDataType>();

	FAvaTransitionLayerUtils::FLayerQueryTextParams Params;
	Params.LayerType = InstanceData.LayerType;
	Params.SpecificLayerName = InstanceData.SpecificLayer.ToName();
	Params.LayerTypePropertyName = GET_MEMBER_NAME_CHECKED(FInstanceDataType, LayerType);
	Params.SpecificLayerPropertyName = GET_MEMBER_NAME_CHECKED(FInstanceDataType, SpecificLayer);

	return FAvaTransitionLayerUtils::GetLayerQueryText(MoveTemp(Params), InId, InBindingLookup, InFormatting);
}
#endif

TArray<const FAvaTransitionBehaviorInstance*> FAvaTransitionLayerCondition::QueryBehaviorInstances(const FStateTreeExecutionContext& InContext) const
{
	const FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);

	UAvaTransitionSubsystem& TransitionSubsystem   = InContext.GetExternalData(TransitionSubsystemHandle);
	const FAvaTransitionContext& TransitionContext = InContext.GetExternalData(TransitionContextHandle);
	const FAvaTransitionLayerComparator Comparator = FAvaTransitionLayerUtils::BuildComparator(TransitionContext, InstanceData.LayerType, InstanceData.SpecificLayer);

	return FAvaTransitionLayerUtils::QueryBehaviorInstances(TransitionSubsystem, Comparator);
}

#undef LOCTEXT_NAMESPACE
