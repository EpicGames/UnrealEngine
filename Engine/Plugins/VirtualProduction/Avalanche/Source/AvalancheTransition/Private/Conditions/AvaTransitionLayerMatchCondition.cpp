// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/AvaTransitionLayerMatchCondition.h"
#include "AvaTransitionContext.h"
#include "AvaTransitionLayerUtils.h"
#include "Behavior/AvaTransitionBehaviorInstance.h"
#include "StateTreeExecutionContext.h"

#define LOCTEXT_NAMESPACE "AvaTransitionLayerMatchCondition"

#if WITH_EDITOR
FText FAvaTransitionLayerMatchCondition::GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const
{
	const FText LayerDesc = Super::GetDescription(InId, InInstanceDataView, InBindingLookup, InFormatting);

	return InFormatting == EStateTreeNodeFormatting::RichText
		? FText::Format(LOCTEXT("DescRich", "<s>scenes transitioning in</> {0}"), LayerDesc)
		: FText::Format(LOCTEXT("Desc", "scenes transitioning in {0}"), LayerDesc);
}
#endif

bool FAvaTransitionLayerMatchCondition::TestCondition(FStateTreeExecutionContext& InContext) const
{
	const FAvaTransitionContext& TransitionContext = InContext.GetExternalData(TransitionContextHandle);
	const FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);

	if (InstanceData.LayerType == EAvaTransitionLayerCompareType::Same && TransitionContext.GetTransitionType() == EAvaTransitionType::In)
	{
		return true;
	}

	TArray<const FAvaTransitionBehaviorInstance*> BehaviorInstances = QueryBehaviorInstances(InContext);
	return BehaviorInstances.ContainsByPredicate([](const FAvaTransitionBehaviorInstance* InInstance)
		{
			return InInstance && InInstance->GetTransitionType() == EAvaTransitionType::In;
		});
}

#undef LOCTEXT_NAMESPACE
