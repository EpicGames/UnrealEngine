// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/AvaTransitionSceneMatchCondition.h"
#include "AvaTransitionContext.h"
#include "AvaTransitionLayerUtils.h"
#include "AvaTransitionScene.h"
#include "AvaTransitionUtils.h"
#include "Behavior/AvaTransitionBehaviorInstance.h"
#include "StateTreeExecutionContext.h"

#define LOCTEXT_NAMESPACE "AvaTransitionSceneMatchCondition"

#if WITH_EDITOR
FText FAvaTransitionSceneMatchCondition::GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const
{
	const FInstanceDataType& InstanceData = InInstanceDataView.Get<FInstanceDataType>();

	const FText ComparisonType = UEnum::GetDisplayValueAsText(InstanceData.SceneComparisonType).ToLower();
	const FText LayerDesc = Super::GetDescription(InId, InInstanceDataView, InBindingLookup, InFormatting);

	return InFormatting == EStateTreeNodeFormatting::RichText
		? FText::Format(LOCTEXT("DescRich", "<b>{0}</> <s>scene in</> {1}"), ComparisonType, LayerDesc)
		: FText::Format(LOCTEXT("Desc", "{0} scene in {1}"), ComparisonType, LayerDesc);
}
#endif

bool FAvaTransitionSceneMatchCondition::TestCondition(FStateTreeExecutionContext& InContext) const
{
	const FAvaTransitionContext& TransitionContext = InContext.GetExternalData(TransitionContextHandle);

	const FAvaTransitionScene* TransitionScene = TransitionContext.GetTransitionScene();
	if (!TransitionScene)
	{
		return false;
	}

	TArray<const FAvaTransitionBehaviorInstance*> BehaviorInstances = QueryBehaviorInstances(InContext);
	if (BehaviorInstances.IsEmpty())
	{
		return false;
	}

	const FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);

	bool bHasMatchingInstance = BehaviorInstances.ContainsByPredicate(
		[TransitionScene, &InstanceData](const FAvaTransitionBehaviorInstance* InInstance)
		{
			const FAvaTransitionScene* OtherTransitionScene = InInstance->GetTransitionContext().GetTransitionScene();

			const EAvaTransitionComparisonResult ComparisonResult = OtherTransitionScene
				? TransitionScene->Compare(*OtherTransitionScene)
				: EAvaTransitionComparisonResult::None;

			return ComparisonResult == InstanceData.SceneComparisonType;
		});

	return bHasMatchingInstance;
}

#undef LOCTEXT_NAMESPACE
