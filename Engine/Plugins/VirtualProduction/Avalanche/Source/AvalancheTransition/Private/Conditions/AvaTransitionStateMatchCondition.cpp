// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/AvaTransitionStateMatchCondition.h"
#include "AvaTransitionLayerUtils.h"
#include "AvaTransitionUtils.h"
#include "Behavior/AvaTransitionBehaviorInstance.h"
#include "StateTreeExecutionContext.h"

#define LOCTEXT_NAMESPACE "AvaTransitionSceneMatchCondition"

#if WITH_EDITOR
FText FAvaTransitionStateMatchCondition::GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const
{
	const FInstanceDataType& InstanceData = InInstanceDataView.Get<FInstanceDataType>();

	const FText TransitionState = UEnum::GetDisplayValueAsText(InstanceData.TransitionState).ToLower();
	const FText LayerDesc = Super::GetDescription(InId, InInstanceDataView, InBindingLookup, InFormatting);

	return InFormatting == EStateTreeNodeFormatting::RichText
		? FText::Format(LOCTEXT("DescRich", "<b>{0}</> <s>scene in</> {1}"), TransitionState, LayerDesc)
		: FText::Format(LOCTEXT("Desc", "{0} scene in {1}"), TransitionState, LayerDesc);
}
#endif

bool FAvaTransitionStateMatchCondition::TestCondition(FStateTreeExecutionContext& InContext) const
{
	TArray<const FAvaTransitionBehaviorInstance*> BehaviorInstances = QueryBehaviorInstances(InContext);

	bool bIsLayerRunning = BehaviorInstances.ContainsByPredicate(
		[](const FAvaTransitionBehaviorInstance* InInstance)
		{
			check(InInstance);
			return InInstance->IsRunning();
		});

	const FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);

	switch (InstanceData.TransitionState)
	{
	case EAvaTransitionRunState::Running:
		return bIsLayerRunning;

	case EAvaTransitionRunState::Finished:
		return !bIsLayerRunning;
	}

	checkNoEntry();
	return false;
}

#undef LOCTEXT_NAMESPACE
