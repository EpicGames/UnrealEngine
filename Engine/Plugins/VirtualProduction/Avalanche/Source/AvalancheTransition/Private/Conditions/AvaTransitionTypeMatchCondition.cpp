// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/AvaTransitionTypeMatchCondition.h"
#include "AvaTransitionContext.h"
#include "AvaTransitionUtils.h"
#include "StateTreeExecutionContext.h"

#define LOCTEXT_NAMESPACE "AvaTransitionTypeMatchCondition"

#if WITH_EDITOR
FText FAvaTransitionTypeMatchCondition::GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const
{
	const FInstanceDataType& InstanceData = InInstanceDataView.Get<FInstanceDataType>();

	const FText TransitionTypeDesc = UEnum::GetDisplayValueAsText(InstanceData.TransitionType).ToLower();

	return InFormatting == EStateTreeNodeFormatting::RichText
		? FText::Format(LOCTEXT("DescRich", "<s>transitioning</> <b>{0}</>"), TransitionTypeDesc)
		: FText::Format(LOCTEXT("Desc", "transitioning {0}"), TransitionTypeDesc);
}
#endif

bool FAvaTransitionTypeMatchCondition::TestCondition(FStateTreeExecutionContext& InContext) const
{
	const FAvaTransitionContext& TransitionContext = InContext.GetExternalData(TransitionContextHandle);
	const FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);

	return TransitionContext.GetTransitionType() == InstanceData.TransitionType;
}

#undef LOCTEXT_NAMESPACE
