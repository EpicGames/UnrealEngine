// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionExecutionContext.h"
#include "AvaTransitionContext.h"
#include "Behavior/AvaTransitionBehaviorInstance.h"


FString FAvaTransitionExecutionExtension::GetInstanceDescription(const FContextParameters& Context) const
{
	if (!SceneDescription.IsEmpty())
	{
		return SceneDescription;
	}
	return FStateTreeExecutionExtension::GetInstanceDescription(Context);
}

FAvaTransitionExecutionContext::FAvaTransitionExecutionContext(const FAvaTransitionBehaviorInstance& InBehaviorInstance, UObject& InOwner, const UStateTree& InStateTree, FStateTreeInstanceData& InInstanceData)
	: FStateTreeExecutionContext(InOwner, InStateTree, InInstanceData)
	, BehaviorInstance(InBehaviorInstance)
{
}

EStateTreeRunStatus FAvaTransitionExecutionContext::Start(const FInstancedPropertyBag* InitialParameters)
{
	FAvaTransitionExecutionExtension Extension;
	const FAvaTransitionContext& TransitionContext = BehaviorInstance.GetTransitionContext();

	// Default Scene Description will be just the Transition Type
	Extension.SceneDescription = *UEnum::GetDisplayValueAsText(TransitionContext.GetTransitionType()).ToString();

	// Opportunity for the Transition Scene Implementation to update or set its own Description
	if (TransitionContext.GetTransitionScene())
	{
		TransitionContext.GetTransitionScene()->UpdateSceneDescription(Extension.SceneDescription);
	}

	return FStateTreeExecutionContext::Start(FStateTreeExecutionContext::FStartParameters
		{
			.InitialGlobalParameters = InitialParameters ? InitialParameters->GetValue() : FConstStructView(),
			.ExecutionExtension = TInstancedStruct<FAvaTransitionExecutionExtension>::Make(MoveTemp(Extension))
		});
}
