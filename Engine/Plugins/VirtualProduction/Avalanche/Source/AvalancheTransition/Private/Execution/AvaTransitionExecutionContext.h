// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "StateTreeExecutionContext.h"
#include "AvaTransitionExecutionContext.generated.h"

struct FAvaTransitionBehaviorInstance;

USTRUCT()
struct FAvaTransitionExecutionExtension : public FStateTreeExecutionExtension
{
	GENERATED_BODY()

	virtual FString GetInstanceDescription(const FContextParameters& Context) const override;

	UPROPERTY()
	FString SceneDescription;
};

struct FAvaTransitionExecutionContext : FStateTreeExecutionContext
{
	FAvaTransitionExecutionContext(const FAvaTransitionBehaviorInstance& InBehaviorInstance, UObject& InOwner, const UStateTree& InStateTree, FStateTreeInstanceData& InInstanceData);

	/** Start executing. */
	EStateTreeRunStatus Start(const FInstancedPropertyBag* InitialParameters);

	const FAvaTransitionBehaviorInstance* GetBehaviorInstance() const
	{
		return &BehaviorInstance;
	}

private:
	const FAvaTransitionBehaviorInstance& BehaviorInstance;
};
