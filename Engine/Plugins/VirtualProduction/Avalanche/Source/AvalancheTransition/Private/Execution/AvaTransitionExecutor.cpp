// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionExecutor.h"
#include "AvaTag.h"
#include "AvaTagHandleKeyFuncs.h"
#include "AvaTagList.h"
#include "AvaTransitionLayer.h"
#include "AvaTransitionSubsystem.h"
#include "AvaTransitionTree.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Execution/AvaTransitionExecutorBuilder.h"
#include "Misc/EnumerateRange.h"

DEFINE_LOG_CATEGORY_STATIC(LogAvaTransitionExecutor, Log, All);

TMulticastDelegate<void(const IAvaTransitionExecutor&)> IAvaTransitionExecutor::OnTransitionStart;

FAvaTransitionExecutor::FAvaTransitionExecutor(FAvaTransitionExecutorBuilder& InBuilder)
	: NullInstance(MoveTemp(InBuilder.NullInstance))
	, ContextName(MoveTemp(InBuilder.ContextName))
	, OnFinished(MoveTemp(InBuilder.OnFinished))
{
	// Add Exit then Enter instances to keep a consistent order of execution
	Instances.Reserve(InBuilder.ExitInstances.Num() + InBuilder.EnterInstances.Num());
	Instances.Append(MoveTemp(InBuilder.ExitInstances));
	Instances.Append(MoveTemp(InBuilder.EnterInstances));
}

FAvaTransitionExecutor::~FAvaTransitionExecutor()
{
	if (IsRunning())
	{
		// Log rather than ensuring because this can still happen when running behaviors and shutting down engine,
		// transitioning to another level, etc
		UE_LOGF(LogAvaTransitionExecutor, Warning
			, "FAvaTransitionExecutor '%p' (in Context %ls) has been destroyed while still running Behaviors!"
			, this
			, *ContextName);
	}
}

void FAvaTransitionExecutor::Setup()
{
	// Do a Setup pass on the Current Instances
	ForEachInstance(
		[this](FAvaTransitionBehaviorInstance& InInstance)
		{
			InInstance.SetLogContext(ContextName);
			InInstance.Setup();
		});

	DetermineTickWorld();

	struct FLayerInfo
	{
		// Enter Instances found for a given Layer
		TArray<int32, TInlineAllocator<1>> EnterInstances;

		// Exit Instances found for a given Layer
		TArray<int32, TInlineAllocator<1>> ExitInstances;

		// The accumulated Transition Type for a given Layer (e.g. combinations could be In, Out or In | Out)
		EAvaTransitionType TransitionType;

		FAvaTagHandle TransitionLayer;
	};

	// Map of the Resolved Tag Layer to the Behavior Instances / Transition type of that Layer
	TMap<FAvaTag, FLayerInfo> TagLayerInfo;
	{
		// Rough Estimate: assume each Instance is in its own single unique tag
		TagLayerInfo.Reserve(Instances.Num());

		// Gather the Layer info for each Instance
		for (TEnumerateRef<FAvaTransitionBehaviorInstance> Instance : EnumerateRange(Instances))
		{
			for (const FAvaTag* Tag : Instance->GetTransitionLayer().GetTags())
			{
				FLayerInfo& LayerInfo = TagLayerInfo.FindOrAdd(*Tag);
				LayerInfo.TransitionType |= Instance->GetTransitionType();
				LayerInfo.TransitionLayer = Instance->GetTransitionLayer();

				switch (Instance->GetTransitionType())
				{
				case EAvaTransitionType::In:
					LayerInfo.EnterInstances.AddUnique(Instance.GetIndex());
					break;

				case EAvaTransitionType::Out:
					LayerInfo.ExitInstances.AddUnique(Instance.GetIndex());
					break;
				}
			}
		}
	}

	// Ensure there's an exiting null instance for every entering Transition Instance in a layer
	for (const TPair<FAvaTag, FLayerInfo>& Pair : TagLayerInfo)
	{
		const FLayerInfo& LayerInfo = Pair.Value;
		if (LayerInfo.TransitionType == EAvaTransitionType::In && !EnumHasAnyFlags(LayerInfo.TransitionType, EAvaTransitionType::Out))
		{
			FAvaTransitionBehaviorInstance& NullInstanceCopy = Instances.Add_GetRef(NullInstance);
			NullInstanceCopy.SetTransitionType(EAvaTransitionType::Out);
			NullInstanceCopy.SetOverrideLayer(LayerInfo.TransitionLayer);
			NullInstanceCopy.Setup();
		}
	}

	// For the Instances that are going out, if they belong in the same Transition Layer as an Instance going In
	// mark them as Needs Discard (this does not mean the scene will be discarded as there could be logic that reverts this flag)
	for (const TPair<FAvaTag, FLayerInfo>& Pair : TagLayerInfo)
	{
		// Skip Layers that have no Enter Transition Instance to replace the existing one 
		if (!EnumHasAnyFlags(Pair.Value.TransitionType, EAvaTransitionType::In))
		{
			continue;
		}

		for (int32 ExitInstanceIndex : Pair.Value.ExitInstances)
		{
			FAvaTransitionBehaviorInstance& ExitInstance = Instances[ExitInstanceIndex];
			check(ExitInstance.GetTransitionType() == EAvaTransitionType::Out);

			if (FAvaTransitionScene* TransitionScene = ExitInstance.GetTransitionContext().GetTransitionScene())
			{
				bool bSceneReused = false;

				const IAvaTransitionBehavior* TransitionBehavior = ExitInstance.GetBehavior();
				if (TransitionBehavior && TransitionBehavior->GetInstancingMode() == EAvaTransitionInstancingMode::Reuse)
				{
					// Scene reused if the Tree Instancing Mode is set to reuse, and if there is an Enter Instance with matching behavior (i.e. Level)
					bSceneReused |= Pair.Value.EnterInstances.ContainsByPredicate(
						[TransitionBehavior, &Instances = Instances](int32 InEnterInstanceIndex)
						{
							const FAvaTransitionBehaviorInstance& EnterInstance = Instances[InEnterInstanceIndex];
							return EnterInstance.GetBehavior() == TransitionBehavior;
						});
				}

				// Only mark for discard if scene not reused
				if (!bSceneReused)
				{
					TransitionScene->SetFlags(EAvaTransitionSceneFlags::NeedsDiscard);
				}
			}
		}
	}
}

void FAvaTransitionExecutor::DetermineTickWorld()
{
	WorldWeak.Reset();

	TArray<UWorld*, TInlineAllocator<1>> CandidateWorlds;

	// Gather all candidate worlds from the instances
	ForEachInstance(
		[&CandidateWorlds](FAvaTransitionBehaviorInstance& InInstance)
		{
			const FAvaTransitionScene* const TransitionScene = InInstance.GetTransitionContext().GetTransitionScene();
			if (!TransitionScene)
			{
				return;
			}
			const ULevel* const Level = TransitionScene->GetLevel();
			if (!Level)
			{
				return;
			}
			if (UWorld* const World = Level->GetWorld())
			{
				CandidateWorlds.AddUnique(World);
			}
		});

	if (CandidateWorlds.IsEmpty())
	{
		return;
	}

	if (GEngine)
	{
		for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
		{
			if (CandidateWorlds.Contains(WorldContext.World()))
			{
				WorldWeak = WorldContext.World();
				return;
			}
		}
	}

	// Fallback, no world in the list was found, use the first candidate.
	WorldWeak = CandidateWorlds[0];
}

void FAvaTransitionExecutor::Start()
{
	if (!ensureAlways(!IsRunning()))
	{
		UE_LOGF(LogAvaTransitionExecutor, Error
			, "Trying to start an already-running FAvaTransitionExecutor '%p' (in Context %ls)!"
			, this
			, *ContextName);
		return;
	}

	Setup();

	GetOnTransitionStart().Broadcast(*this);

	ForEachInstance(
		[](FAvaTransitionBehaviorInstance& InInstance)
		{
			InInstance.Start();
		});

	// All Behaviors might've finished on Start
	ConditionallyFinishBehaviors();
}

bool FAvaTransitionExecutor::IsRunning() const
{
	return Instances.ContainsByPredicate(
		[](const FAvaTransitionBehaviorInstance& InInstance)
		{
			return InInstance.IsRunning();
		});
}

TArray<const FAvaTransitionBehaviorInstance*> FAvaTransitionExecutor::GetBehaviorInstances(const FAvaTransitionLayerComparator& InComparator) const
{
	TArray<const FAvaTransitionBehaviorInstance*> OutInstances;
	OutInstances.Reserve(Instances.Num());

	ForEachInstance(
		[&OutInstances, &InComparator](const FAvaTransitionBehaviorInstance& InBehaviorInstance)
		{
			if (InComparator.Compare(InBehaviorInstance))
			{
				OutInstances.Add(&InBehaviorInstance);
			}
		});

	return OutInstances;
}

void FAvaTransitionExecutor::ForEachBehaviorInstance(TFunctionRef<void(const FAvaTransitionBehaviorInstance&)> InCallable) const
{
	ForEachInstance(InCallable);
}

void FAvaTransitionExecutor::Stop()
{
	ForEachInstance(
		[](FAvaTransitionBehaviorInstance& InInstance)
		{
			InInstance.Stop();
		});

	ensureAlways(!IsRunning());
	ConditionallyFinishBehaviors();
}

TStatId FAvaTransitionExecutor::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FAvaTransitionExecutor, STATGROUP_Tickables);
}

void FAvaTransitionExecutor::Tick(float InDeltaSeconds)
{
	ForEachInstance(
		[InDeltaSeconds](FAvaTransitionBehaviorInstance& InInstance)
		{
			InInstance.Tick(InDeltaSeconds);
		});

	ConditionallyFinishBehaviors();
}

bool FAvaTransitionExecutor::IsTickable() const
{
	return IsRunning();
}

UWorld* FAvaTransitionExecutor::GetTickableGameObjectWorld() const
{
	return WorldWeak.Get();
}

FString FAvaTransitionExecutor::GetReferencerName() const
{
	return TEXT("FAvaTransitionExecutor");
}

void FAvaTransitionExecutor::AddReferencedObjects(FReferenceCollector& InCollector)
{
	ForEachInstance(
		[&InCollector](FAvaTransitionBehaviorInstance& InInstance)
		{
			InInstance.AddReferencedObjects(InCollector);
		});
}

void FAvaTransitionExecutor::ForEachInstance(TFunctionRef<void(FAvaTransitionBehaviorInstance&)> InFunc)
{
	for (FAvaTransitionBehaviorInstance& Instance : Instances)
	{
		InFunc(Instance);
	}
}

void FAvaTransitionExecutor::ForEachInstance(TFunctionRef<void(const FAvaTransitionBehaviorInstance&)> InFunc) const
{
	for (const FAvaTransitionBehaviorInstance& Instance : Instances)
	{
		InFunc(Instance);
	}
}

void FAvaTransitionExecutor::ConditionallyFinishBehaviors()
{
	if (!IsRunning())
	{
		OnFinished.ExecuteIfBound();
	}
}
