// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionEnums.h"
#include "Behavior/AvaTransitionBehaviorInstance.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Execution/IAvaTransitionExecutor.h"
#include "Tickable.h"
#include "UObject/GCObject.h"
#include "UObject/WeakInterfacePtr.h"

class FAvaTransitionExecutorBuilder;
class IAvaTransitionBehavior;

/** Base Implementation of an Executor dealing with multiple behaviors going out (Exit Instances) and multiple behaviors going in (Enter Instances) */
class FAvaTransitionExecutor : public IAvaTransitionExecutor, public FTickableGameObject, public FGCObject
{
public:
	explicit FAvaTransitionExecutor(FAvaTransitionExecutorBuilder& InBuilder);

	virtual ~FAvaTransitionExecutor() override;

	bool IsRunning() const;

protected:
	void Setup();

	/** Gets the world to tick with from the behavior instances */
	void DetermineTickWorld();

	//~ Begin IAvaTransitionExecutor
	virtual TArray<const FAvaTransitionBehaviorInstance*> GetBehaviorInstances(const FAvaTransitionLayerComparator& InComparator) const;
	virtual void ForEachBehaviorInstance(TFunctionRef<void(const FAvaTransitionBehaviorInstance&)> InCallable) const override;
	virtual void Start() override;
	virtual void Stop() override;
	//~ End IAvaTransitionExecutor

	//~ Begin FTickableGameObject
	virtual TStatId GetStatId() const override;
	virtual void Tick(float InDeltaSeconds) override;
	virtual bool IsTickable() const override;
	virtual bool IsTickableInEditor() const override { return true; }
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Conditional; }
	virtual UWorld* GetTickableGameObjectWorld() const override;
	//~ End FTickableGameObject

	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FGCObject

private:
	void ForEachInstance(TFunctionRef<void(FAvaTransitionBehaviorInstance&)> InFunc);
	void ForEachInstance(TFunctionRef<void(const FAvaTransitionBehaviorInstance&)> InFunc) const;

	void ConditionallyFinishBehaviors();

	/** 
	 * First valid world context found.
	 * The transition will tick with this World Tick if valid.
	 */
	TWeakObjectPtr<UWorld> WorldWeak;

	TArray<FAvaTransitionBehaviorInstance> Instances;

	const FAvaTransitionBehaviorInstance NullInstance;

	FString ContextName;

	FSimpleDelegate OnFinished;
};
