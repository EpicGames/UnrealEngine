// Copyright Epic Games, Inc. All Rights Reserved.

#include "Execution/AvaTransitionExecutorBuilder.h"
#include "AvaTransitionExecutor.h"
#include "AvaTransitionSubsystem.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "Templates/SharedPointer.h"

FAvaTransitionExecutorBuilder::FAvaTransitionExecutorBuilder()
	: NullInstance(FAvaTransitionBehaviorInstance()
		.SetBehavior(nullptr)
		.CreateScene<FAvaTransitionScene>({}))
{
}

FAvaTransitionExecutorBuilder& FAvaTransitionExecutorBuilder::SetContextName(const FString& InContextName)
{
	ContextName = InContextName;
	return *this;
}

FAvaTransitionExecutorBuilder& FAvaTransitionExecutorBuilder::AddEnterInstance(FAvaTransitionBehaviorInstance& InInstance)
{
	InInstance.SetTransitionType(EAvaTransitionType::In);
	EnterInstances.Add(InInstance);
	return *this;
}

FAvaTransitionExecutorBuilder& FAvaTransitionExecutorBuilder::AddExitInstance(FAvaTransitionBehaviorInstance& InInstance)
{
	InInstance.SetTransitionType(EAvaTransitionType::Out);
	ExitInstances.Add(InInstance);
	return *this;
}

FAvaTransitionExecutorBuilder& FAvaTransitionExecutorBuilder::SetNullInstance(FAvaTransitionBehaviorInstance& InInstance)
{
	NullInstance = InInstance;
	return *this;
}

FAvaTransitionExecutorBuilder& FAvaTransitionExecutorBuilder::SetOnFinished(FSimpleDelegate InDelegate)
{
	OnFinished = InDelegate;
	return *this;
}

TSharedRef<IAvaTransitionExecutor> FAvaTransitionExecutorBuilder::Build(UAvaTransitionSubsystem& InTransitionSubsystem)
{
	TSharedRef<FAvaTransitionExecutor> Executor = MakeShared<FAvaTransitionExecutor>(*this);
	InTransitionSubsystem.RegisterTransitionExecutor(Executor);
	return Executor;
}
