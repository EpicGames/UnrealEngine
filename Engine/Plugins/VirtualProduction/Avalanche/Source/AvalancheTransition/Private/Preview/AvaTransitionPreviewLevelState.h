// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagHandle.h"
#include "UObject/ObjectPtr.h"
#include "AvaTransitionPreviewLevelState.generated.h"

class ULevelStreaming;

USTRUCT()
struct FAvaTransitionPreviewLevelState
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<ULevelStreaming> LevelStreaming;

	UPROPERTY()
	bool bEnableOverrideTransitionLayer = false;

	UPROPERTY()
	FAvaTagHandle OverrideTransitionLayer;

	/** Indication of whether this Level should be unloaded after finishing transition */
	UPROPERTY()
	bool bShouldUnload = false;
};
