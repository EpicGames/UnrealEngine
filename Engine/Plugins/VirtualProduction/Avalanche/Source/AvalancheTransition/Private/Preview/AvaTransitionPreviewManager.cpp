// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionPreviewManager.h"
#include "AvaTransitionPreviewScene.h"
#include "AvaTransitionSubsystem.h"
#include "AvaTransitionTree.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "Containers/Ticker.h"
#include "Engine/Level.h"
#include "Engine/LevelStreamingDynamic.h"
#include "Engine/World.h"
#include "Execution/AvaTransitionExecutorBuilder.h"
#include "Execution/IAvaTransitionExecutor.h"
#include "Misc/PackageName.h"
#include "StateTreeExecutionTypes.h"
#include "Streaming/LevelStreamingDelegates.h"

#if WITH_EDITOR
#include "EditorLevelUtils.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogAvaTransitionPreview, Log, All);

AAvaTransitionPreviewManager::AAvaTransitionPreviewManager()
{
	if (!IsTemplate())
	{
		FLevelStreamingDelegates::OnLevelStreamingStateChanged.AddUObject(this, &AAvaTransitionPreviewManager::OnLevelStreamingStateChanged);
	}
}

void AAvaTransitionPreviewManager::TakeNext()
{
	// Ignore if there's transition in progress
	if (bTransitionInProgress)
	{
		return;
	}

	bTransitionInProgress = true;

	// If Next Level is null, skip Level Streaming, and Start Transition immediately
	if (NextLevelAsset.IsNull())
	{
		NextLevelState.LevelStreaming = nullptr;
		StartTransition();
		return;
	}

	// If re-using level, try to see if there's an existing level streaming with the next level asset
	if (bReuseLevel && TryReuseLevel())
	{
		StartTransition();
		return;
	}

	ULevelStreamingDynamic::FLoadLevelInstanceParams LevelInstanceParams = MakeLevelInstanceParams();

	constexpr bool bInitiallyVisible = false;

	LevelInstanceParams.bLoadAsTempPackage = true;
	LevelInstanceParams.bInitiallyVisible  = bInitiallyVisible;

	bool bSuccess = false;
	NextLevelState.LevelStreaming = ULevelStreamingDynamic::LoadLevelInstance(LevelInstanceParams, bSuccess);

	if (!bSuccess || !NextLevelState.LevelStreaming)
	{
		UE_LOGF(LogAvaTransitionPreview, Error
			, "[%ls]: Failed to load level instance `%ls`."
			, *GetPathNameSafe(this)
			, *NextLevelAsset.ToString());
		return;
	}

	NextLevelState.LevelStreaming->SetShouldBeLoaded(true);
	NextLevelState.LevelStreaming->SetShouldBeVisible(bInitiallyVisible);
}

void AAvaTransitionPreviewManager::TakeOut()
{
	if (bTransitionInProgress)
	{
		return;
	}

	const FAvaTransitionPreviewLevelState* ExistingLevelState = FindExistingLevelStateForNextLevel();
	UAvaTransitionSubsystem* TransitionSubsystem = GetTransitionSubsystem();

	if (!TransitionSubsystem || !ExistingLevelState || !ExistingLevelState->LevelStreaming)
	{
		return;
	}

	IAvaTransitionBehavior* TransitionBehavior = GetBehavior(ExistingLevelState->LevelStreaming, *TransitionSubsystem);
	if (!TransitionBehavior)
	{
		return;
	}

	bTransitionInProgress = true;
	NextLevelState.LevelStreaming = nullptr;
	NextLevelState.bEnableOverrideTransitionLayer = true;
	NextLevelState.OverrideTransitionLayer = TransitionBehavior->GetTransitionLayer();

	if (UAvaTransitionTree* TransitionTree = TransitionBehavior->GetTransitionTree())
	{
		StartTransition();
	}
	else
	{
		OnTransitionEnded();
	}
}

void AAvaTransitionPreviewManager::TransitionStop()
{
	if (TransitionExecutor.IsValid())
	{
		TransitionExecutor->Stop();
	}

	if (!ensureAlways(!bTransitionInProgress))
	{
		UE_LOGF(LogAvaTransitionPreview, Error
			, "Transition Executor %ls, but transition is marked as still in progress!"
			, TransitionExecutor.IsValid() ? TEXT("was stopped") : TEXT("is invalid"));
	}
}

ULevelStreamingDynamic::FLoadLevelInstanceParams AAvaTransitionPreviewManager::MakeLevelInstanceParams() const
{
	return ULevelStreamingDynamic::FLoadLevelInstanceParams(GetWorld()
		, NextLevelAsset.GetLongPackageName()
		, FTransform::Identity);
}

const FAvaTransitionPreviewLevelState* AAvaTransitionPreviewManager::FindExistingLevelStateForNextLevel() const
{
	ULevelStreamingDynamic::FLoadLevelInstanceParams LevelInstanceParams = MakeLevelInstanceParams();

	FString ShortPackageName = FPackageName::GetShortName(LevelInstanceParams.LongPackageName);
	if (ShortPackageName.StartsWith(LevelInstanceParams.World->StreamingLevelsPrefix))
	{
		ShortPackageName.RightChopInline(LevelInstanceParams.World->StreamingLevelsPrefix.Len(), EAllowShrinking::No);
	}

	const FName PackageName = *(FPackageName::GetLongPackagePath(LevelInstanceParams.LongPackageName)
		+ TEXT("/")
		+ ShortPackageName);

	return LevelStates.FindByPredicate(
		[PackageName](const FAvaTransitionPreviewLevelState& InState)
		{
			return InState.LevelStreaming
				&& InState.LevelStreaming->PackageNameToLoad == PackageName;
		});
}

bool AAvaTransitionPreviewManager::TryReuseLevel()
{
	if (const FAvaTransitionPreviewLevelState* FoundLevelState = FindExistingLevelStateForNextLevel())
	{
		// Only set the LevelStreaming property
		// bShouldUnload is reset in StartTransition
		// and the rest of the properties are user-facing that should not change
		NextLevelState.LevelStreaming = FoundLevelState->LevelStreaming;
		return true;
	}
	return false;
}

IAvaTransitionBehavior* AAvaTransitionPreviewManager::GetBehavior(ULevelStreaming* InLevelStreaming, const UAvaTransitionSubsystem& InTransitionSubsystem)
{
	if (!InLevelStreaming)
	{
		return nullptr;
	}

	if (ULevel* const Level = InLevelStreaming->GetLoadedLevel())
	{
		return InTransitionSubsystem.GetTransitionBehavior(Level);
	}

	return nullptr;
}

UAvaTransitionSubsystem* AAvaTransitionPreviewManager::GetTransitionSubsystem() const
{
	if (UWorld* const World = GetWorld())
	{
		return World->GetSubsystem<UAvaTransitionSubsystem>();
	}
	return nullptr;
}

void AAvaTransitionPreviewManager::StartTransition()
{
	UAvaTransitionSubsystem* const TransitionSubsystem = GetTransitionSubsystem();
	if (!TransitionSubsystem)
	{
		return;
	}

	// Reset bShouldUnload to false, in case it was set to Unload previously (unlikely behavior)
	NextLevelState.bShouldUnload = false;

	FAvaTransitionExecutorBuilder ExecutorBuilder;

	// Enter Instance
	ExecutorBuilder.AddEnterInstance(FAvaTransitionBehaviorInstance()
		.SetBehavior(GetBehavior(NextLevelState.LevelStreaming, *TransitionSubsystem))
		.CreateScene<FAvaTransitionPreviewScene>(this, &NextLevelState));

	// Exit Instances
	for (FAvaTransitionPreviewLevelState& State : LevelStates)
	{
		ExecutorBuilder.AddExitInstance(FAvaTransitionBehaviorInstance()
			.SetBehavior(GetBehavior(State.LevelStreaming, *TransitionSubsystem))
			.CreateScene<FAvaTransitionPreviewScene>(this, &State));
	}

	TransitionExecutor = ExecutorBuilder
		.SetContextName(GetFullName())
		.SetOnFinished(FSimpleDelegate::CreateUObject(this, &AAvaTransitionPreviewManager::OnTransitionEnded))
		.Build(*TransitionSubsystem);

	TransitionExecutor->Start();
}

void AAvaTransitionPreviewManager::OnTransitionEnded()
{
	if (!bTransitionInProgress)
	{
		return;
	}

	UnloadDiscardedLevels();

	LevelStates.Add(NextLevelState);

	// Null out Next Level State
	NextLevelState.LevelStreaming = nullptr;
	NextLevelState.bEnableOverrideTransitionLayer = false;
	NextLevelState.OverrideTransitionLayer = FAvaTagHandle();
	NextLevelState.bShouldUnload = false;

	bTransitionInProgress = false;

	TransitionExecutor.Reset();
}

void AAvaTransitionPreviewManager::OnLevelStreamingStateChanged(UWorld* InWorld
	, const ULevelStreaming* InLevelStreaming
	, ULevel* InLevelIfLoaded
	, ELevelStreamingState InPreviousState
	, ELevelStreamingState InNewState)
{
	// Filter out levels we don't care about.
	if (NextLevelState.LevelStreaming != InLevelStreaming)
	{
		return;
	}

	switch (InNewState)
	{
	case ELevelStreamingState::FailedToLoad:
		UE_LOGF(LogAvaTransitionPreview, Error
			, "Level \"%ls\" failed to load."
			, *InLevelStreaming->PackageNameToLoad.ToString());
		break;

	case ELevelStreamingState::LoadedNotVisible:
		if (!NextLevelState.LevelStreaming->GetShouldBeVisibleFlag())
		{
			NextLevelState.LevelStreaming->SetShouldBeVisible(true);
		}
		break;

	case ELevelStreamingState::LoadedVisible:
		// Next Level Loaded and Visible.. Start Transition/
		StartTransition();
		break;

	default:
		break;
	}
}

void AAvaTransitionPreviewManager::UnloadDiscardedLevels()
{
	// Worst case: All Level States are discarded
	TArray<ULevelStreaming*> InstancesToUnload;
	InstancesToUnload.Reserve(LevelStates.Num());

	for (TArray<FAvaTransitionPreviewLevelState>::TIterator Iter(LevelStates); Iter; ++Iter)
	{
		// Skip Level States that are to keep
		if (!Iter->bShouldUnload)
		{
			continue;
		}

		// if Next Level State is reusing the same level streaming, and did not unload it, then it shouldn't be unloaded,
		// but it should still be removed from the list as it will be replaced by NextLevelState
		if (Iter->LevelStreaming == NextLevelState.LevelStreaming && !NextLevelState.bShouldUnload)
		{
			Iter.RemoveCurrent();
			continue;
		}

		InstancesToUnload.Add(Iter->LevelStreaming);
		Iter.RemoveCurrent();
	}

	UnloadLevelStreamingInstances(InstancesToUnload);
}

void AAvaTransitionPreviewManager::UnloadLevelStreamingInstances(TConstArrayView<ULevelStreaming*> InInstancesToUnload)
{
	if (InInstancesToUnload.IsEmpty())
	{
		return;
	}

	UWorld* const World = GetWorld();
	if (!World)
	{
		return;
	}

	if (World->IsGameWorld())
	{
		for (ULevelStreaming* LevelStreaming : InInstancesToUnload)
		{
			if (LevelStreaming)
			{
				LevelStreaming->SetShouldBeLoaded(false);
				LevelStreaming->SetShouldBeVisible(false);
				LevelStreaming->SetIsRequestingUnloadAndRemoval(true);
			}
		}
	}
#if WITH_EDITOR
	else
	{
		TArray<TWeakObjectPtr<ULevel>> LevelsToUnloadWeak;
		LevelsToUnloadWeak.Reserve(InInstancesToUnload.Num());
		for (ULevelStreaming* InstanceToUnload : InInstancesToUnload)
		{
			if (InstanceToUnload)
			{
				LevelsToUnloadWeak.Add(InstanceToUnload->GetLoadedLevel());
			}
		}

		if (LevelsToUnloadWeak.IsEmpty())
		{
			return;
		}

		bool bResetTransactionBuffer = ShouldResetTransactionBuffer(InInstancesToUnload);

		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
			[LevelsToUnloadWeak, bResetTransactionBuffer](float)->bool
			{
				TArray<ULevel*> LevelsToUnload;
				LevelsToUnload.Reserve(LevelsToUnloadWeak.Num());
				for (const TWeakObjectPtr<ULevel>& LevelWeak : LevelsToUnloadWeak)
				{
					if (ULevel* Level = LevelWeak.Get())
					{
						LevelsToUnload.Add(Level);
					}
				}

				// No need to clear the whole editor selection since actor of this level will be removed from the selection by: UEditorEngine::OnLevelRemovedFromWorld
				UEditorLevelUtils::RemoveLevelsFromWorld(LevelsToUnload, /*bClearSelection*/false, bResetTransactionBuffer);
				return false;
			}));
	}
#endif
}

#if WITH_EDITOR
bool AAvaTransitionPreviewManager::ShouldResetTransactionBuffer(TConstArrayView<ULevelStreaming*> InInstancesToUnload) const
{
	for (ULevelStreaming* InstanceToUnload : InInstancesToUnload)
	{
		if (!InstanceToUnload)
		{
			continue;
		}

		ULevel* Level = InstanceToUnload->GetLoadedLevel();
		if (!Level)
		{
			continue;
		}

		// Check if we need to flush the Trans buffer...
		UWorld* OuterWorld = Level->GetTypedOuter<UWorld>();

		bool bResetTransactionBuffer = false;

		ForEachObjectWithOuterBreakable(OuterWorld,
			[&bResetTransactionBuffer](UObject* InObject)
			{
				if (InObject && InObject->HasAnyFlags(RF_Transactional))
				{
					bResetTransactionBuffer = true;
					UE_LOGF(LogAvaTransitionPreview
						, Warning
						, "Found RF_Transactional object '%ls' while unloading Level Instance."
						, *InObject->GetPathName());
					return false;
				}
				return true;
			}
			, EGetObjectsFlags::IncludeNestedObjects);

		if (bResetTransactionBuffer)
		{
			return true;
		}
	}

	return false;
}
#endif
