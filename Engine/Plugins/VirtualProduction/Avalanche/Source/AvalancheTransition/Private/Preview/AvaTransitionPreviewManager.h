// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionEnums.h"
#include "AvaTransitionPreviewLevelState.h"
#include "Containers/ArrayView.h"
#include "Engine/LevelStreamingDynamic.h"
#include "GameFramework/Actor.h"
#include "UObject/WeakInterfacePtr.h"
#include "AvaTransitionPreviewManager.generated.h"

class IAvaTransitionBehavior;
class IAvaTransitionExecutor;
class UAvaTransitionSubsystem;
class ULevelStreaming;
class UWorld;
enum class ELevelStreamingState : uint8;

/**
 * Actor to preview out Transition Logic via Simple Level Instance Loading and Unloading
 * The Level Instance Loading/Unloading is not set to be 
 */
UCLASS(NotBlueprintable, DisplayName = "Motion Design Transition Preview Manager")
class AAvaTransitionPreviewManager : public AActor
{
	GENERATED_BODY()

public:
	AAvaTransitionPreviewManager();

	UFUNCTION(CallInEditor, Category="Transition Logic Preview")
	void TakeNext();

	UFUNCTION(CallInEditor, Category="Transition Logic Preview")
	void TakeOut();

	UFUNCTION(CallInEditor, Category="Transition Logic Preview")
	void TransitionStop();

private:
	ULevelStreamingDynamic::FLoadLevelInstanceParams MakeLevelInstanceParams() const;

	const FAvaTransitionPreviewLevelState* FindExistingLevelStateForNextLevel() const;

	bool TryReuseLevel();

	static IAvaTransitionBehavior* GetBehavior(ULevelStreaming* InLevelStreaming, const UAvaTransitionSubsystem& InTransitionSubsystem);

	UAvaTransitionSubsystem* GetTransitionSubsystem() const;

	void StartTransition();

	void OnTransitionEnded();

	void OnLevelStreamingStateChanged(UWorld* InWorld
		, const ULevelStreaming* InLevelStreaming
		, ULevel* InLevelIfLoaded
		, ELevelStreamingState InPreviousState
		, ELevelStreamingState InNewState);

	void UnloadDiscardedLevels();

	void UnloadLevelStreamingInstances(TConstArrayView<ULevelStreaming*> InInstancesToUnload);

#if WITH_EDITOR
	bool ShouldResetTransactionBuffer(TConstArrayView<ULevelStreaming*> InInstancesToUnload) const;
#endif

	UPROPERTY(Transient)
	TArray<FAvaTransitionPreviewLevelState> LevelStates;

	UPROPERTY(Transient, VisibleAnywhere, Category="Transition Logic Preview")
	bool bTransitionInProgress = false;

	UPROPERTY(Transient, EditAnywhere, Category="Transition Logic Preview", meta=(EditCondition="!bTransitionInProgress"))
	bool bReuseLevel = false;

	/** The Next Level to Transition to the Scene */
	UPROPERTY(Transient, EditAnywhere, DisplayName = "Next Level", Category="Transition Logic Preview", meta=(EditCondition="!bTransitionInProgress"))
	TSoftObjectPtr<UWorld> NextLevelAsset;

	UPROPERTY(Transient)
	FAvaTransitionPreviewLevelState NextLevelState;

	TSharedPtr<IAvaTransitionExecutor> TransitionExecutor;
};
