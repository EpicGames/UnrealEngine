// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionPreviewScene.h"
#include "AvaTransitionPreviewLevelState.h"
#include "Engine/LevelStreaming.h"

FAvaTransitionPreviewScene::FAvaTransitionPreviewScene(FAvaTransitionPreviewLevelState* InLevelState)
	: FAvaTransitionScene(InLevelState)
{
	if (InLevelState && InLevelState->bEnableOverrideTransitionLayer)
	{
		OverrideTransitionLayer = InLevelState->OverrideTransitionLayer;
	}
}

EAvaTransitionComparisonResult FAvaTransitionPreviewScene::Compare(const FAvaTransitionScene& InOther) const
{
	const FAvaTransitionPreviewLevelState& LevelState      = GetDataView().Get<FAvaTransitionPreviewLevelState>();
	const FAvaTransitionPreviewLevelState* OtherLevelState = InOther.GetDataView().GetPtr<FAvaTransitionPreviewLevelState>();

	if (!OtherLevelState || !LevelState.LevelStreaming || !OtherLevelState->LevelStreaming)
	{
		return EAvaTransitionComparisonResult::None;
	}

	return LevelState.LevelStreaming->PackageNameToLoad == OtherLevelState->LevelStreaming->PackageNameToLoad
		? EAvaTransitionComparisonResult::Same
		: EAvaTransitionComparisonResult::Different;
}

ULevel* FAvaTransitionPreviewScene::GetLevel() const
{
	const FAvaTransitionPreviewLevelState& LevelState = GetDataView().Get<FAvaTransitionPreviewLevelState>();

	return LevelState.LevelStreaming ? LevelState.LevelStreaming->GetLoadedLevel() : nullptr;
}

void FAvaTransitionPreviewScene::GetOverrideTransitionLayer(FAvaTagHandle& OutTransitionLayer) const
{
	if (OverrideTransitionLayer.IsSet())
	{
		OutTransitionLayer = *OverrideTransitionLayer;
	}
}

void FAvaTransitionPreviewScene::OnFlagsChanged()
{
	FAvaTransitionPreviewLevelState& LevelState = GetDataView().GetMutable<FAvaTransitionPreviewLevelState>();

	// In this Preview Scene, discarding means kicking off level unloading
	LevelState.bShouldUnload = HasAnyFlags(EAvaTransitionSceneFlags::NeedsDiscard);
}
