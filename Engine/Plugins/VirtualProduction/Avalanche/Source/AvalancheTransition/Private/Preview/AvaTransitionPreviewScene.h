// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagHandle.h"
#include "AvaTransitionScene.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "AvaTransitionPreviewScene.generated.h"

class ULevelStreaming;
class UWorld;
struct FAvaTransitionPreviewLevelState;

USTRUCT()
struct FAvaTransitionPreviewScene : public FAvaTransitionScene
{
	GENERATED_BODY()

	FAvaTransitionPreviewScene() = default;

	explicit FAvaTransitionPreviewScene(FAvaTransitionPreviewLevelState* InLevelState);

	//~ Begin FAvaTransitionScene
	virtual EAvaTransitionComparisonResult Compare(const FAvaTransitionScene& InOther) const override;
	virtual ULevel* GetLevel() const override;
	virtual void GetOverrideTransitionLayer(FAvaTagHandle& OutTransitionLayer) const override;
	virtual void OnFlagsChanged() override;
	//~ End FAvaTransitionScene

	TOptional<FAvaTagHandle> OverrideTransitionLayer;
};
