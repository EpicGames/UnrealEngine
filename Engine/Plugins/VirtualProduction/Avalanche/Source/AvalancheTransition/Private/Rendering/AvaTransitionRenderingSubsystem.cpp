// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/AvaTransitionRenderingSubsystem.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "SceneView.h"

void UAvaTransitionRenderingSubsystem::ShowLevel(TObjectKey<ULevel> InLevel)
{
	// Defer showing to happen later in the frame or next frame (if past core ticker tick)
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateWeakLambda(this,
		[this, InLevel](float)->bool
		{
			HiddenLevels.RemoveSingle(InLevel);
			return false;
		}));
}

void UAvaTransitionRenderingSubsystem::HideLevel(TObjectKey<ULevel> InLevel)
{
	HiddenLevels.Add(InLevel);
}

void UAvaTransitionRenderingSubsystem::SetupView(FSceneView& InView)
{
	HiddenPrimitives.Reset();
	ProcessedLevels.Reset();

	for (const TObjectKey<ULevel>& LevelKey : HiddenLevels)
	{
		bool bAlreadyProcessedLevel;
		ProcessedLevels.Add(LevelKey, &bAlreadyProcessedLevel);

		if (bAlreadyProcessedLevel)
		{
			continue;
		}

		const ULevel* Level = LevelKey.ResolveObjectPtr();
		if (!Level)
		{
			continue;
		}

		for (AActor* Actor : Level->Actors)
		{
			if (!Actor)
			{
				continue;
			}

			Actor->ForEachComponent<UPrimitiveComponent>(/*bIncludeFromChildActors*/false,
				[&HiddenPrimitives = HiddenPrimitives](UPrimitiveComponent* InComponent)
				{
					if (InComponent->IsRegistered())
					{
						HiddenPrimitives.Add(InComponent->GetPrimitiveSceneId());
					}
				});
		}
	}

	InView.HiddenPrimitives.Append(HiddenPrimitives);
}
