// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionSceneViewExtension.h"
#include "Rendering/AvaTransitionRenderingSubsystem.h"
#include "SceneView.h"

void FAvaTransitionSceneViewExtension::SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView)
{
	if (!InViewFamily.Scene)
	{
		return;
	}

	const UWorld* World = InViewFamily.Scene->GetWorld();
	if (!World)
	{
		return;
	}

	UAvaTransitionRenderingSubsystem* RenderingSubsystem = World->GetSubsystem<UAvaTransitionRenderingSubsystem>();
	if (!RenderingSubsystem)
	{
		return;
	}

	RenderingSubsystem->SetupView(InView);
}
