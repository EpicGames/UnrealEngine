// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneViewExtension.h"

class FAvaTransitionSceneViewExtension : public FSceneViewExtensionBase
{
public:
	FAvaTransitionSceneViewExtension(const FAutoRegister& InAutoRegister)
		: FSceneViewExtensionBase(InAutoRegister)
	{
	}

	//~ Begin FSceneViewExtensionBase
	void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override;
	//~ End FSceneViewExtensionBase
};
