// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/AvaTransitionDelayTask.h"
#include "AvaTransitionContext.h"
#include "AvaTransitionScene.h"
#include "AvaTransitionUtils.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "Engine/Level.h"
#include "Rendering/AvaTransitionRenderingSubsystem.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeLinker.h"

#define LOCTEXT_NAMESPACE "AvaTransitionDelayTask"

#if WITH_EDITOR
FText FAvaTransitionDelayTask::GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const
{
	const FInstanceDataType& InstanceData = InInstanceDataView.Get<FInstanceDataType>();

	const FText DurationDesc = FText::AsNumber(InstanceData.Duration);

	return InFormatting == EStateTreeNodeFormatting::RichText
		? FText::Format(LOCTEXT("DescRich", "Delay <b>{0}</> <s>seconds</>"), DurationDesc)
		: FText::Format(LOCTEXT("Desc", "Delay {0} seconds"), DurationDesc);
}
#endif

bool FAvaTransitionDelayTask::Link(FStateTreeLinker& InLinker)
{
	Super::Link(InLinker);
	InLinker.LinkExternalData(RenderingSubsystemHandle);
	return true;
}

EStateTreeRunStatus FAvaTransitionDelayTask::EnterState(FStateTreeExecutionContext& InContext, const FStateTreeTransitionResult& InTransition) const
{
	FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);
	InstanceData.RemainingTime = InstanceData.Duration;
	return WaitForDelayCompletion(InContext, InstanceData);
}

EStateTreeRunStatus FAvaTransitionDelayTask::Tick(FStateTreeExecutionContext& InContext, const float InDeltaTime) const
{
	FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);
	InstanceData.RemainingTime -= InDeltaTime;
	return WaitForDelayCompletion(InContext, InstanceData);
}

EStateTreeRunStatus FAvaTransitionDelayTask::WaitForDelayCompletion(FStateTreeExecutionContext& InContext, FInstanceDataType& InInstanceData) const
{
	UAvaTransitionRenderingSubsystem& RenderingSubsystem = InContext.GetExternalData(RenderingSubsystemHandle);

	if (InInstanceData.RemainingTime <= 0.f)
	{
		// Restore Level Visibility
		RenderingSubsystem.ShowLevel(InInstanceData.HiddenLevel);
		return EStateTreeRunStatus::Succeeded;
	}

	if (ShouldHideLevel(InContext, InInstanceData))
	{
		if (const FAvaTransitionScene* TransitionScene = InContext.GetExternalData(TransitionContextHandle).GetTransitionScene())
		{
			InInstanceData.HiddenLevel = TransitionScene->GetLevel();
			RenderingSubsystem.HideLevel(InInstanceData.HiddenLevel);
		}
	}

	return EStateTreeRunStatus::Running;
}

bool FAvaTransitionDelayTask::ShouldHideLevel(const FStateTreeExecutionContext& InContext, const FInstanceDataType& InInstanceData) const
{
	if (InInstanceData.HideMode == EAvaTransitionLevelHideMode::NoHide)
	{
		return false;	
	}

	// If Hidden Level is non-null, it means the level has already been hidden / processed. Skip
	if (InInstanceData.HiddenLevel != nullptr)
	{
		return false;
	}

	// If Instancing Mode is set to Reuse, and user set to Hide Level regardless, allow Hiding Reused Level
	const IAvaTransitionBehavior* TransitionBehavior = Cast<IAvaTransitionBehavior>(InContext.GetOwner());
	if (TransitionBehavior && TransitionBehavior->GetInstancingMode() == EAvaTransitionInstancingMode::Reuse)
	{
		return InInstanceData.HideMode == EAvaTransitionLevelHideMode::AlwaysHide;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
