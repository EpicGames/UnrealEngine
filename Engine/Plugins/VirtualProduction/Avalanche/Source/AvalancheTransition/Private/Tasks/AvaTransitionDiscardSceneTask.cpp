// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/AvaTransitionDiscardSceneTask.h"
#include "AvaTransitionContext.h"
#include "StateTreeExecutionContext.h"

EStateTreeRunStatus FAvaTransitionDiscardSceneTask::EnterState(FStateTreeExecutionContext& InContext, const FStateTreeTransitionResult& InTransition) const
{
	FAvaTransitionContext& TransitionContext = InContext.GetExternalData(TransitionContextHandle);

	if (FAvaTransitionScene* TransitionScene = TransitionContext.GetTransitionScene())
	{
		TransitionScene->SetFlags(EAvaTransitionSceneFlags::NeedsDiscard);
		return EStateTreeRunStatus::Succeeded;
	}

	return EStateTreeRunStatus::Failed;
}
