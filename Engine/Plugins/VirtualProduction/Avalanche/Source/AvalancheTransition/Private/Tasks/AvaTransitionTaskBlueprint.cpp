// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/AvaTransitionTaskBlueprint.h"

EStateTreeRunStatus UAvaTransitionTaskBlueprint::EnterState(FStateTreeExecutionContext& InExecutionContext, const FStateTreeTransitionResult& InTransition)
{
	BehaviorInstanceCache.SetCachedInstanceDataFromContext(InExecutionContext);
	return Super::EnterState(InExecutionContext, InTransition);
}

void UAvaTransitionTaskBlueprint::ExitState(FStateTreeExecutionContext& InExecutionContext, const FStateTreeTransitionResult& InTransition)
{
	Super::ExitState(InExecutionContext, InTransition);
	BehaviorInstanceCache.ClearCachedInstanceData();
}

const FAvaTransitionBehaviorInstanceCache& UAvaTransitionTaskBlueprint::GetBehaviorInstanceCache() const
{
	return BehaviorInstanceCache;
}
