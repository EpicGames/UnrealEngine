// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/AvaTransitionWaitForLayerTask.h"
#include "AvaTransitionLayerUtils.h"
#include "Behavior/AvaTransitionBehaviorInstance.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "Engine/Level.h"
#include "Rendering/AvaTransitionRenderingSubsystem.h"
#include "StateTreeExecutionContext.h"
#include "StateTreeLinker.h"

#define LOCTEXT_NAMESPACE "AvaTransitionWaitForLayerTask"

#if WITH_EDITOR
FText FAvaTransitionWaitForLayerTask::GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const
{
	const FText LayerDesc = Super::GetDescription(InId, InInstanceDataView, InBindingLookup, InFormatting);

	return InFormatting == EStateTreeNodeFormatting::RichText
		? FText::Format(LOCTEXT("DescRich", "Wait <s>for others in</> {0} <s>to finish</>"), LayerDesc)
		: FText::Format(LOCTEXT("Desc", "Wait for others in {0} to finish"), LayerDesc);
}
#endif

bool FAvaTransitionWaitForLayerTask::Link(FStateTreeLinker& InLinker)
{
	Super::Link(InLinker);
	InLinker.LinkExternalData(RenderingSubsystemHandle);
	return true;
}

EStateTreeRunStatus FAvaTransitionWaitForLayerTask::EnterState(FStateTreeExecutionContext& InContext, const FStateTreeTransitionResult& InTransition) const
{
	return WaitForLayer(InContext);
}

EStateTreeRunStatus FAvaTransitionWaitForLayerTask::Tick(FStateTreeExecutionContext& InContext, const float InDeltaTime) const
{
	return WaitForLayer(InContext);
}

EStateTreeRunStatus FAvaTransitionWaitForLayerTask::WaitForLayer(FStateTreeExecutionContext& InContext) const
{
	TArray<const FAvaTransitionBehaviorInstance*> BehaviorInstances = QueryBehaviorInstances(InContext);

	bool bIsLayerRunning = BehaviorInstances.ContainsByPredicate(
		[](const FAvaTransitionBehaviorInstance* InInstance)
		{
			check(InInstance);
			return InInstance->IsRunning();
		});

	FAvaTransitionWaitForLayerTask::FInstanceDataType& InstanceData = InContext.GetInstanceData(*this);

	UAvaTransitionRenderingSubsystem& RenderingSubsystem = InContext.GetExternalData(RenderingSubsystemHandle);

	if (bIsLayerRunning)
	{
		if (ShouldHideLevel(InContext, InstanceData))
		{
			if (const FAvaTransitionScene* TransitionScene = InContext.GetExternalData(TransitionContextHandle).GetTransitionScene())
			{
				InstanceData.HiddenLevel = TransitionScene->GetLevel();
				RenderingSubsystem.HideLevel(InstanceData.HiddenLevel);
			}
		}

		return EStateTreeRunStatus::Running;
	}

	// Restore Level Visibility
	RenderingSubsystem.ShowLevel(InstanceData.HiddenLevel);
	return EStateTreeRunStatus::Succeeded;
}

bool FAvaTransitionWaitForLayerTask::ShouldHideLevel(const FStateTreeExecutionContext& InContext, const FAvaTransitionWaitForLayerTask::FInstanceDataType& InInstanceData) const
{
	if (InInstanceData.HideMode == EAvaTransitionLevelHideMode::NoHide)
	{
		return false;
	}

	// If Hidden Level is non-null, it means the level has already been hidden / processed. Skip
	if (InInstanceData.HiddenLevel != nullptr)
	{
		return false;
	}

	// If Instancing Mode is set to Reuse, and user set to Hide Level regardless, allow Hiding Reused Level
	const IAvaTransitionBehavior* TransitionBehavior = Cast<IAvaTransitionBehavior>(InContext.GetOwner());
	if (TransitionBehavior && TransitionBehavior->GetInstancingMode() == EAvaTransitionInstancingMode::Reuse)
	{
		return InInstanceData.HideMode == EAvaTransitionLevelHideMode::AlwaysHide;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
