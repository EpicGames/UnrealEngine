// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagHandle.h"
#include "AvaTransitionEnums.h"
#include "AvaTransitionScene.h"
#include "StructUtils/InstancedStruct.h"
#include "AvaTransitionContext.generated.h"

/**
 * A Transition Context is composed of a Transitioning Scene Instance,
 * an indication whether this is an In or an Out Transition,
 * and in which Layer this Transition is taking place
 */
USTRUCT()
struct FAvaTransitionContext
{
	GENERATED_BODY()

	const FAvaTransitionScene* GetTransitionScene() const
	{
		return TransitionScene.GetPtr<FAvaTransitionScene>();
	}

	FAvaTransitionScene* GetTransitionScene()
	{
		return TransitionScene.GetMutablePtr<FAvaTransitionScene>();
	}

	EAvaTransitionType GetTransitionType() const
	{
		return TransitionType;
	}

	FAvaTagHandle GetTransitionLayer() const
	{
		return TransitionLayer;
	}

private:
	friend struct FAvaTransitionBehaviorInstance;

	/** Instance Struct of the Transition Scene */
	UPROPERTY()
	FInstancedStruct TransitionScene;

	UPROPERTY()
	FAvaTagHandle TransitionLayer;

	UPROPERTY()
	EAvaTransitionType TransitionType = EAvaTransitionType::In;
};
