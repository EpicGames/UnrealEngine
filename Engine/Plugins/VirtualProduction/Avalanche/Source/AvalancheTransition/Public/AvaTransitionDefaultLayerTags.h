// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagSoftHandle.h"

namespace UE::AvaTransition
{

class FDefaultTags
{
	FDefaultTags();

public:
	AVALANCHETRANSITION_API static const FDefaultTags& Get();

	/** Soft handle to the default layer tag */
	FAvaTagSoftHandle DefaultLayer;
};

} // UE::AvaTransition
