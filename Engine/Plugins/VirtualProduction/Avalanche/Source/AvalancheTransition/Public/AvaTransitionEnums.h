// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionEnums.generated.h"

enum class EAvaTransitionSceneFlags : uint8
{
	None = 0,
	NeedsDiscard = 1 << 0, // Flag indicating that the Scene needs to be discarded at the end of the Transition
};
ENUM_CLASS_FLAGS(EAvaTransitionSceneFlags);

enum class EAvaTransitionIterationResult : uint8
{
	Break,
	Continue,
};

UENUM(BlueprintType, DisplayName="Motion Design Transition Type")
enum class EAvaTransitionType : uint8
{
	None = 0 UMETA(Hidden),
	In   = 1 << 0,
	Out  = 1 << 1,
	MAX UMETA(Hidden),
};
ENUM_CLASS_FLAGS(EAvaTransitionType);

UENUM(BlueprintType, DisplayName="Motion Design Transition Filter Type")
enum class EAvaTransitionTypeFilter : uint8
{
	In,
	Out,
	Any,
};

UENUM(BlueprintType, DisplayName="Motion Design Transition Run State")
enum class EAvaTransitionRunState : uint8
{
	Unknown UMETA(Hidden),
	Running,
	Finished,
};

UENUM(BlueprintType, DisplayName="Motion Design Transition Comparison Result")
enum class EAvaTransitionComparisonResult : uint8
{
	None,
	Different,
	Same,
};

UENUM(BlueprintType, DisplayName="Motion Design Transition Scene Type")
enum class EAvaTransitionSceneType : uint8
{
	This,
	Other,
};

UENUM(BlueprintType, DisplayName="Motion Design Transition Layer Compare Type")
enum class EAvaTransitionLayerCompareType : uint8
{
	None UMETA(Hidden),
	Same,
	Different UMETA(DisplayName="Other"),
	MatchingTag UMETA(DisplayName="Specific Layers"),
	Any,
};

UENUM(BlueprintType, DisplayName="Motion Design Transition Instancing Mode")
enum class EAvaTransitionInstancingMode : uint8
{
	/** A new scene instance is added to the parent world and exit instance is discarded. */
	New,
	/** If previous instance exists, reuse it for the entering behavior. */
	Reuse
};

UENUM(BlueprintType, DisplayName="Motion Design Transition Level Hide Mode")
enum class EAvaTransitionLevelHideMode : uint8
{
	/** Do not hide the Actors of the Level */
	NoHide,

	/** Hide the Level's Actors unless it's a Level that is re-used */
	HideUnlessReuse,

	/** Always hide the Level's Actors even if it's re-used */
	AlwaysHide,
};
