// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagHandleContainer.h"
#include "AvaTransitionEnums.h"
#include "UObject/NameTypes.h"

struct FAvaTagHandle;
struct FAvaTransitionBehaviorInstance;
struct FAvaTransitionContext;

struct AVALANCHETRANSITION_API FAvaTransitionLayerComparator
{
	bool Compare(const FAvaTagHandle& InLayer) const;

	bool Compare(const FAvaTransitionBehaviorInstance& InInstance) const;

	FAvaTagHandleContainer LayerContext;

	/** Optional pointer to the Transition Context to exclude from the Comparison */
	const FAvaTransitionContext* ExcludedContext = nullptr;

	/** The type of comparison to perform */
	EAvaTransitionLayerCompareType LayerCompareType = EAvaTransitionLayerCompareType::Same;
};
