// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeTypes.h"
#include "Containers/ContainersFwd.h"
#include "UObject/NameTypes.h"

class UAvaTransitionSubsystem;
enum class EStateTreeNodeFormatting : uint8;
enum class EAvaTransitionLayerCompareType : uint8;
struct FAvaTagHandle;
struct FAvaTagHandleContainer;
struct FAvaTransitionBehaviorInstance;
struct FAvaTransitionContext;
struct FAvaTransitionLayerComparator;
struct IStateTreeBindingLookup;

struct FAvaTransitionLayerUtils
{
	/** Gets all the Behavior Instances that match the Comparator */
	AVALANCHETRANSITION_API static TArray<const FAvaTransitionBehaviorInstance*> QueryBehaviorInstances(UAvaTransitionSubsystem& InTransitionSubsystem, const FAvaTransitionLayerComparator& InComparator);

	/** Builds a Comparator for the given Context (and optionally Layer), excluding the provided Transition Context (assumes it's the transition context calling this) */
	AVALANCHETRANSITION_API static FAvaTransitionLayerComparator BuildComparator(const FAvaTransitionContext& InTransitionContext, EAvaTransitionLayerCompareType InCompareType, const FAvaTagHandle& InSpecificLayer);

	/** Builds a Comparator for the given Context and specific layers, excluding the provided Transition Context (assumes it's the transition context calling this) */
	AVALANCHETRANSITION_API static FAvaTransitionLayerComparator BuildComparator(const FAvaTransitionContext& InTransitionContext, EAvaTransitionLayerCompareType InCompareType, const FAvaTagHandleContainer& InSpecificLayers);

#if WITH_EDITOR
	struct FLayerQueryTextParams
	{
		EAvaTransitionLayerCompareType LayerType;
		FName SpecificLayerName;
		FName LayerTypePropertyName;
		FName SpecificLayerPropertyName;
	};
	AVALANCHETRANSITION_API static FText GetLayerQueryText(FLayerQueryTextParams&& InParams, const FGuid& InId, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting);
#endif
};
