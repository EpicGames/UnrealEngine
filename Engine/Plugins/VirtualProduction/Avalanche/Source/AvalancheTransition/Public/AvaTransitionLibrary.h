// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionEnums.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "AvaTransitionLibrary.generated.h"

#define UE_API AVALANCHETRANSITION_API

class IAvaTransitionBehavior;
class UAvaTransitionTree;
struct FAvaTagHandleContainer;
struct FAvaTransitionContext;

UCLASS(MinimalAPI)
class UAvaTransitionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** Retrieves the transition context from */
	static UE_API const FAvaTransitionContext* GetTransitionContext(UObject* InContextObject);

	/**
	 * Determines whether a transition is currently taking place at a given layer
	 * @param InContextObject the context to retrieve the active transition relating to the context.
	 * @param InLayerComparisonType the type of layer to filter.
	 *      "Same": the same layer as this context's.
	 *      "Other": a layer different from this context's.
	 *      "Specific Layers": layers specified by "InSpecificLayers".
	 *      "Any": any layer found.
	 * @param InSpecificLayers the specific layers to filter. InLayerComparisonType should be set to "Specific Layers" for this to be considered.
	 * @param InTransitionTypeFilter the transition type to filter (whether it should only consider Ins or Outs, or any type is allowed)
	 */
	UFUNCTION(BlueprintPure, Category="Transition Logic", meta=(DefaultToSelf="InContextObject"))
	static UE_API bool IsTransitionActiveInLayers(UObject* InContextObject
		, EAvaTransitionLayerCompareType InLayerComparisonType
		, const FAvaTagHandleContainer& InSpecificLayers
		, EAvaTransitionTypeFilter InTransitionTypeFilter = EAvaTransitionTypeFilter::Any);

	UE_DEPRECATED(5.7, "Use IsTransitionActiveInLayers instead")
	UFUNCTION(BlueprintPure, Category="Transition Logic", meta=(DefaultToSelf="InContextObject", DeprecatedFunction, DeprecationMessage="Use \"IsTransitionActiveInLayers\" instead"))
	static bool IsTransitionActiveInLayer(UObject* InContextObject
		, EAvaTransitionComparisonResult InSceneComparisonType
		, EAvaTransitionLayerCompareType InLayerComparisonType
		, const FAvaTagHandleContainer& InSpecificLayers);

	/** Returns the transition type of the current transition taking place if any */
	UFUNCTION(BlueprintPure, Category="Transition Logic", meta=(DefaultToSelf="InContextObject"))
	static UE_API EAvaTransitionType GetTransitionType(UObject* InContextObject);

	/**
	 * Determines whether there are scenes/levels (other than the one from this context) transitioning.
	 * Note: this ignores scenes that are transitioning out and to be discarded.
	 * @param InContextObject the context to retrieve the active transition relating to the context.
	 * @param InLayers the layers to consider
	 * @param InScenesToIgnore the world assets to ignore
	 * @return true if the given layers have assets that are involved in a transition.
	 */
	UFUNCTION(BlueprintPure, Category="Transition Logic", meta=(DefaultToSelf="InContextObject"))
	static UE_API bool AreScenesTransitioning(UObject* InContextObject
		, const FAvaTagHandleContainer& InLayers
		, const TArray<TSoftObjectPtr<UWorld>>& InScenesToIgnore);

	/**
	 * Retrieves the transition behavior from this context
	 */
	UFUNCTION(BlueprintPure, Category="Transition Logic", meta=(DefaultToSelf="InContextObject"))
	static UE_API TScriptInterface<IAvaTransitionBehavior> GetTransitionBehavior(UObject* InContextObject);

	/**
	 * Retrieves the transition tree from this context
	 */
	UFUNCTION(BlueprintPure, Category="Transition Logic", meta=(DefaultToSelf="InContextObject"))
	static UE_API const UAvaTransitionTree* GetTransitionTree(UObject* InContextObject);
};

#undef UE_API
