// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

AVALANCHETRANSITION_API DECLARE_LOG_CATEGORY_EXTERN(LogAvaTransition, Log, All);
