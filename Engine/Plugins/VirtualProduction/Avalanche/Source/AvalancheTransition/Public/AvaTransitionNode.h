// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeExecutionTypes.h"

class UAvaTransitionSubsystem;
struct FAvaTransitionContext;
struct FStateTreeLinker;

class FAvaTransitionNode
{
public:
	AVALANCHETRANSITION_API bool LinkNode(FStateTreeLinker& InLinker);

	TStateTreeExternalDataHandle<FAvaTransitionContext> TransitionContextHandle;

	TStateTreeExternalDataHandle<UAvaTransitionSubsystem> TransitionSubsystemHandle;
};
