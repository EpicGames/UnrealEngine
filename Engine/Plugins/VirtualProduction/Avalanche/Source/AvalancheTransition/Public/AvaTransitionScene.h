// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionEnums.h"
#include "IAvaTransitionExtension.h"
#include "Misc/TVariant.h"
#include "StateTreeTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "AvaTransitionScene.generated.h"

struct FAvaTagHandle;

/**
 * Owner of the Scene Instance. Used to determine whether a Scene Instance is valid or not.
 */
struct FAvaTransitionSceneOwner
{
	FAvaTransitionSceneOwner()
	{
		SceneOwner.Set<bool>(true);
	}

	FAvaTransitionSceneOwner(UObject* InSceneOwner)
	{
		SceneOwner.Set<FWeakObjectPtr>(InSceneOwner);
	}

	template<typename T, ESPMode InMode>
	FAvaTransitionSceneOwner(TWeakPtr<T, InMode> InSceneOwner)
	{
		SceneOwner.Set<TWeakPtr<void, InMode>>(InSceneOwner);
	}

	template<typename T, ESPMode InMode>
	FAvaTransitionSceneOwner(TSharedPtr<T, InMode> InSceneOwner)
	{
		SceneOwner.Set<TWeakPtr<void, InMode>>(InSceneOwner);
	}

	template<typename T, ESPMode InMode>
	FAvaTransitionSceneOwner(TSharedRef<T, InMode> InSceneOwner)
	{
		SceneOwner.Set<TWeakPtr<void, InMode>>(InSceneOwner);
	}

	bool operator()(bool InSceneOwner) const
	{
		return InSceneOwner;
	}

	bool operator()(const FWeakObjectPtr& InSceneOwner) const
	{
		return InSceneOwner.IsValid();
	}

	template<ESPMode InMode>
	bool operator()(const TWeakPtr<void, InMode>& InSceneOwner) const
	{
		return InSceneOwner.IsValid();
	}

	bool IsValid() const
	{
		return Visit(*this, SceneOwner);
	}

private:
	TVariant<
		bool,
		FWeakObjectPtr,
		TWeakPtr<void, ESPMode::ThreadSafe>,
		TWeakPtr<void, ESPMode::NotThreadSafe>
	> SceneOwner;
};

/**
 * A Transition Scene is a representation of what is transitioning in or out.
 * It's validity is bound to the validity of the scene owner because it does not own the data passed to it (i.e. it holds a data view),
 * and so cannot be used without ensuring (through the scene owner) that the scene data is still valid
 * It can be used to represent a page or, more directly, a level streaming object.
 * For example, an implementation could look something like this:

	USTRUCT()
	struct FSceneData
	{
		GENERATED_BODY()
		...
	};

	USTRUCT()
	struct FMyTransitionScene : public FAvaTransitionScene
	{
		GENERATED_BODY()

		FMyTransitionScene(FSceneData* InData)
			: FAvaTransitionScene(InData)
		{
		}

		virtual EAvaTransitionComparisonResult Compare(const FAvaTransitionScene& InOther) override
		{
			const FSceneData& MySceneData    = GetDataView.Get<FSceneData>();
        	const FSceneData* OtherSceneData = InOther.GetDataView().GetPtr<FSceneData>();
        	...
		}
	};
 */
USTRUCT()
struct AVALANCHETRANSITION_API FAvaTransitionScene
{
	GENERATED_BODY()

	FAvaTransitionScene() = default;

	virtual ~FAvaTransitionScene() = default;

	template<typename InStructType>
		requires (!std::is_const_v<InStructType> && TModels_V<CStaticStructProvider, InStructType>)
	explicit FAvaTransitionScene(InStructType* InStruct)
		: DataView(InStructType::StaticStruct(), reinterpret_cast<uint8*>(InStruct))
	{	
	}

	explicit FAvaTransitionScene(FStateTreeDataView InDataView);

	/** Determines whether this Transition Scene is the same as another */
	virtual EAvaTransitionComparisonResult Compare(const FAvaTransitionScene& InOther) const;

	/** Retrieves the underling Level of the Scene */
	virtual ULevel* GetLevel() const;

	/** Optional override of Transition Layer */
	virtual void GetOverrideTransitionLayer(FAvaTagHandle& OutTransitionLayer) const {}

	/** Called when flags have been added / removed */
	virtual void OnFlagsChanged() {}

	/** Optional override for the passed in Scene Description */
	virtual void UpdateSceneDescription(FString& InOutDescription) const {}

	void SetFlags(EAvaTransitionSceneFlags InFlags);

	bool HasAnyFlags(EAvaTransitionSceneFlags InFlags) const;

	bool HasAllFlags(EAvaTransitionSceneFlags InFlags) const;

	const FStateTreeDataView& GetDataView() const;

	template<typename InExtensionType UE_REQUIRES(IAvaTransitionExtension::TIsValidExtension_V<InExtensionType>)>
	InExtensionType* FindExtension() const
	{
		if (const TSharedRef<IAvaTransitionExtension>* Extension = Extensions.Find(InExtensionType::ExtensionIdentifier))
		{
			return static_cast<InExtensionType*>(&Extension->Get());
		}
		return nullptr;
	}

protected:
	template<typename InExtensionType UE_REQUIRES(IAvaTransitionExtension::TIsValidExtension_V<InExtensionType>)>
	InExtensionType& AddExtension()
	{
		TSharedRef<InExtensionType> Extension = MakeShared<InExtensionType>();
		Extensions.Add(InExtensionType::ExtensionIdentifier, Extension);
		return Extension.Get();
	}

private:
	FStateTreeDataView DataView;

	EAvaTransitionSceneFlags Flags = EAvaTransitionSceneFlags::None;

	TMap<FName, TSharedRef<IAvaTransitionExtension>> Extensions;
};
