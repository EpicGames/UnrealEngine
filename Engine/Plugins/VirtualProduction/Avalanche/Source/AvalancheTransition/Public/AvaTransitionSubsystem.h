// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionEnums.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Subsystems/WorldSubsystem.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectKey.h"
#include "UObject/WeakInterfacePtr.h"
#include "AvaTransitionSubsystem.generated.h"

class IAvaTransitionBehavior;
class IAvaTransitionExecutor;
class ULevel;
enum class EStateTreeRunStatus : uint8;
struct FAvaTransitionLayerComparator;
struct FAvaTransitionScene;

UCLASS(MinimalAPI)
class UAvaTransitionSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	void RegisterTransitionBehavior(const ULevel* InLevel, IAvaTransitionBehavior* InBehavior);

	/**
	 * Gets or Creates a Transition Behavior for the given Level in this Subsystem's World
	 * @param InLevel the level to get or create the transition behavior. if null, will default to the persistent level of the world
	 * @return the behavior for the provided level
	 */
	AVALANCHETRANSITION_API IAvaTransitionBehavior* GetOrCreateTransitionBehavior(ULevel* InLevel = nullptr);

	/**
	 * Gets the Transition Behavior for the given Level in this Subsystem's World
	 * @param InLevel the level containing the transition behavior. if null, will default to the persistent level of the world
	 * @return the behavior for the provided level
	 */
	AVALANCHETRANSITION_API IAvaTransitionBehavior* GetTransitionBehavior(const ULevel* InLevel = nullptr) const;

	AVALANCHETRANSITION_API static IAvaTransitionBehavior* FindTransitionBehavior(const ULevel* InLevel);

	void RegisterTransitionExecutor(const TSharedRef<IAvaTransitionExecutor>& InExecutor);

	/**
	 * Executes the provided function for each Registered valid Transition Executor
	 * @param InFunc the function to execute for each executor reference with option to break iteration
	 */
	AVALANCHETRANSITION_API void ForEachTransitionExecutor(TFunctionRef<EAvaTransitionIterationResult(IAvaTransitionExecutor&)> InFunc);

protected:
	//~ Begin UWorldSubsystem
	virtual bool DoesSupportWorldType(const EWorldType::Type InWorldType) const override;
	virtual void PostInitialize() override;
	//~ End UWorldSubsystem

private:
	TMap<TObjectKey<ULevel>, TWeakInterfacePtr<IAvaTransitionBehavior>> TransitionBehaviors;

	TArray<TWeakPtr<IAvaTransitionExecutor>> TransitionExecutors;
};
