// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagHandle.h"
#include "AvaTransitionEnums.h"
#include "StateTree.h"
#include "AvaTransitionTree.generated.h"

#define UE_API AVALANCHETRANSITION_API

struct FAvaTransitionTask;

/**
 * Motion Design Transition Tree is a State Tree with the purpose of executing user-defined logic
 * when there's a Transition between multiple scenes in multiple layers.
 */
UCLASS(MinimalAPI, DisplayName = "Motion Design Transition Tree")
class UAvaTransitionTree : public UStateTree
{
	GENERATED_BODY()

public:
	UE_DEPRECATED(5.8, "GetTransitionLayer deprecated. Use IAvaTransitionBehavior::GetTransitionLayer instead")
	UE_API FAvaTagHandle GetTransitionLayer() const;

	UE_DEPRECATED(5.8, "SetTransitionLayer deprecated. Use IAvaTransitionBehavior::SetTransitionLayer instead")
	UE_API void SetTransitionLayer(FAvaTagHandle InTransitionLayer);

	UE_DEPRECATED(5.8, "IsEnabled deprecated. Use IAvaTransitionBehavior::IsEnabled instead")
	UE_API bool IsEnabled() const;

	UE_DEPRECATED(5.8, "SetEnabled deprecated. Use IAvaTransitionBehavior::SetEnabled instead")
	UE_API void SetEnabled(bool bInEnabled);

	/** Returns whether a Task of a given type exists and is enabled within an enabled state in the Transition Tree */
	UE_API bool ContainsTask(const UScriptStruct* InTaskStruct) const;

	/** Returns whether a Task of a given type exists and is enabled within an enabled state in the Transition Tree */
	template <class InTaskType UE_REQUIRES(std::is_base_of_v<FAvaTransitionTask, InTaskType>)>
	bool ContainsTask() const
	{
		return ContainsTask(InTaskType::StaticStruct());
	}

	UE_DEPRECATED(5.8, "SetInstancingMode deprecated. Use IAvaTransitionBehavior::SetInstancingMode instead")
	UE_API void SetInstancingMode(EAvaTransitionInstancingMode InInstancingMode);

	UE_DEPRECATED(5.8, "GetInstancingMode deprecated. Use IAvaTransitionBehavior::GetInstancingMode instead")
	UFUNCTION(BlueprintPure, Category = "Transition Logic", meta=(DeprecatedFunction, DeprecationMessage="Use GetInstancingMode from the Transition Behavior Interface (via GetTransitionBehavior)instead."))
	UE_API EAvaTransitionInstancingMode GetInstancingMode() const;

	UE_DEPRECATED(5.8, "GetEnabledPropertyName deprecated. Use enabled via IAvaTransitionBehavior instead")
	UE_API static FName GetEnabledPropertyName();

private:
	UE_DEPRECATED(5.8, "TransitionLayer deprecated. Use IAvaTransitionBehavior instead")
	UPROPERTY()
	FAvaTagHandle TransitionLayer_DEPRECATED;

	UE_DEPRECATED(5.8, "bEnabled deprecated. Use IAvaTransitionBehavior instead")
	UPROPERTY()
	bool bEnabled_DEPRECATED = true;

	UE_DEPRECATED(5.8, "InstancingMode deprecated. Use IAvaTransitionBehavior instead")
	UPROPERTY()
	EAvaTransitionInstancingMode InstancingMode_DEPRECATED = EAvaTransitionInstancingMode::New;
};

#undef UE_API
