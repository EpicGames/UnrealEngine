// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeSchema.h"
#include "StateTreeExecutionTypes.h"
#include "AvaTransitionTreeSchema.generated.h"

UCLASS(MinimalAPI, BlueprintType, EditInlineNew, CollapseCategories, HideDropdown, DisplayName="Motion Design Transition Tree Schema")
class UAvaTransitionTreeSchema : public UStateTreeSchema
{
	GENERATED_BODY()

public:
	UAvaTransitionTreeSchema() = default;

protected:
	//~ Begin UStateTreeSchema
	virtual bool IsStructAllowed(const UScriptStruct* InScriptStruct) const override;
	virtual bool IsClassAllowed(const UClass* InClass) const override;
	virtual bool IsExternalItemAllowed(const UStruct& InStruct) const override;
#if WITH_EDITOR
	virtual bool AllowEnterConditions() const override { return true; }
	virtual bool AllowEvaluators() const override { return true; }
	virtual bool AllowMultipleTasks() const override { return true; }
#endif
	//~ End UStateTreeSchema
};
