// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeTypes.h"

struct FAvaTransitionBehaviorInstance;
struct FStateTreeExecutionContext;

namespace UE::AvaTransition
{
	/** Safely gets the Instance Data if the Struct type matches or nullptr if there is type mismatch */
	template<typename InNodeType>
	static typename InNodeType::FInstanceDataType* TryGetInstanceData(const InNodeType& InNode, FStateTreeDataView InInstanceDataView)
	{
		// Current safety check in case this is being called prior to Instance Data types getting fixed. (e.g. in UStateTree::PostLoad)
		const UStruct* InstanceDataType = InInstanceDataView.GetStruct();
		if (InstanceDataType && InstanceDataType->IsChildOf(InNodeType::FInstanceDataType::StaticStruct()))
		{
			return InInstanceDataView.GetMutablePtr<typename InNodeType::FInstanceDataType>();
		}
		return nullptr;
	}

	/**
	 * Retrieves the Behavior Instance from the Execution Context
	 * The Execution Context *must* have originated from Motion Design to return a valid pointer
	 */
	const FAvaTransitionBehaviorInstance* GetBehaviorInstance(const FStateTreeExecutionContext& InExecutionContext);
}
