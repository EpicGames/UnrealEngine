// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"

#define UE_API AVALANCHETRANSITION_API

namespace UE::AvaTransition
{

/** Describes the versions of transition behaviors */
struct FBehaviorVersion
{
private:
	FBehaviorVersion() = delete;

public:
	enum Type : uint8
	{
		PreVersioning = 0,

		/**
		 * Proper location for instance-only properties like Transition Layer. Moved away from UAvaTransitionTree into IAvaTransitionBehavior 
		 * Transition Trees are designed to be re-usable, so these instance-related properties now live in the Transition Behavior.
		 */
		InstanceProperties,

		/* ------------------------------------------------------ */
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	UE_API const static FGuid Guid;
};

} // UE::AvaTransition

#undef UE_API
