// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagHandle.h"
#include "AvaTransitionContext.h"
#include "AvaTransitionScene.h"
#include "StateTreeInstanceData.h"
#include "UObject/WeakInterfacePtr.h"

class FAvaTransitionExecutorBuilder;
class FReferenceCollector;
class IAvaTransitionBehavior;
class UAvaTransitionTree;
struct FAvaTransitionExecutionContext;

/** Struct containing Instance Data for a Transition Behavior running or about to run */
struct FAvaTransitionBehaviorInstance
{
	FAvaTransitionBehaviorInstance()
		: TransitionSceneOwner(nullptr)
	{
	}

	AVALANCHETRANSITION_API FAvaTransitionBehaviorInstance& SetBehavior(IAvaTransitionBehavior* InBehavior);

	AVALANCHETRANSITION_API bool IsEnabled() const;

	template<typename InTransitionSceneType, typename... InArgTypes
		UE_REQUIRES(TIsDerivedFrom<InTransitionSceneType, FAvaTransitionScene>::Value)>
	FAvaTransitionBehaviorInstance& CreateScene(FAvaTransitionSceneOwner InTransitionSceneOwner, InArgTypes&&... InArgs)
	{
		checkf(InTransitionSceneOwner.IsValid(), TEXT("Scene Instance Owner must be valid/alive when Creating Scene"));
		TransitionSceneOwner = MoveTemp(InTransitionSceneOwner);
		TransitionContext.TransitionScene = FInstancedStruct::Make<InTransitionSceneType>(Forward<InArgTypes>(InArgs)...);
		return *this;
	}

	/** Gets the Behavior this Instance is based on */
	AVALANCHETRANSITION_API IAvaTransitionBehavior* GetBehavior() const;

	AVALANCHETRANSITION_API const UAvaTransitionTree* GetTransitionTree() const;

	AVALANCHETRANSITION_API FAvaTagHandle GetTransitionLayer() const;

	AVALANCHETRANSITION_API EAvaTransitionType GetTransitionType() const;

	AVALANCHETRANSITION_API bool IsRunning() const;

	/** Gets the Transition Context of this instance  */
	AVALANCHETRANSITION_API const FAvaTransitionContext& GetTransitionContext() const;

	FAvaTransitionContext& GetTransitionContext();

	FAvaTransitionSceneOwner GetTransitionSceneOwner() const;

	void SetTransitionType(EAvaTransitionType InTransitionType);

	bool Setup();

	void Start();

	void Tick(float InDeltaSeconds);

	void Stop();

	void SetOverrideLayer(const FAvaTagHandle& InOverrideLayer);

	void SetLogContext(const FString& InContext);

	void AddReferencedObjects(FReferenceCollector& InCollector);

private:
	/** Stops Execution if the Tree is no longer running */
	void ConditionallyStop();

	bool UpdateContext(TOptional<FAvaTransitionExecutionContext>& OutContext);

	bool ValidateTransitionScene();

	void UpdateTransitionLayers(const IAvaTransitionBehavior* InBehavior);

	bool MakeContext(IAvaTransitionBehavior* InBehavior, TOptional<FAvaTransitionExecutionContext>& OutContext);

	/** The Behavior this Instance is based on */
	TWeakInterfacePtr<IAvaTransitionBehavior> BehaviorWeak;

	/** The Instance Data used when running the State Tree */
	FStateTreeInstanceData InstanceData;

	/** Context information on the current Transition taking place */
	FAvaTransitionContext TransitionContext;

	/** The Owner of the Scene Instance. If this Owner is alive, it is assumed that the Scene Instance underlying Struct will also be alive */
	FAvaTransitionSceneOwner TransitionSceneOwner;

	/** The current run status of this Instance */
	EStateTreeRunStatus RunStatus = EStateTreeRunStatus::Unset;

	/** If not none, this is the Transition Layer that will be set */
	TOptional<FAvaTagHandle> OverrideLayer;

	FString LogContext;
};
