// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionScene.h"

class IAvaTransitionBehavior;
struct FAvaTransitionBehaviorInstance;
struct FAvaTransitionContext;
struct FStateTreeExecutionContext;

struct FAvaTransitionBehaviorInstanceCache
{
	AVALANCHETRANSITION_API const FAvaTransitionContext* GetTransitionContext() const;

	IAvaTransitionBehavior* GetTransitionBehavior() const;

	void SetCachedInstanceDataFromContext(const FStateTreeExecutionContext& InExecutionContext);

	void ClearCachedInstanceData();

private:
	FAvaTransitionSceneOwner CachedSceneOwner;

	const FAvaTransitionBehaviorInstance* CachedBehaviorInstance = nullptr;
};
