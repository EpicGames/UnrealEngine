// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionEnums.h"
#include "Containers/ArrayView.h"
#include "Delegates/Delegate.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/Interface.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakInterfacePtr.h"
#include "IAvaTransitionBehavior.generated.h"

class UAvaTransitionTree;
class UObject;
enum class EStateTreeRunStatus : uint8;
struct FAvaTransitionScene;
struct FStateTreeReference;

UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UAvaTransitionBehavior : public UInterface
{
	GENERATED_BODY()
};

class IAvaTransitionBehavior
{
	GENERATED_BODY()

public:
	virtual UObject& AsUObject() = 0;

	/** Gets the underlying Transition Tree that this Transition Behavior runs */
	virtual UAvaTransitionTree* GetTransitionTree() const = 0;

	/** Gets the transition layer that this transition behavior uses */
	UFUNCTION(BlueprintCallable, Category = "Transition Logic")
	virtual FAvaTagHandle GetTransitionLayer() const = 0;

	/** Set the transition layer that this transition behavior will use */
	virtual void SetTransitionLayer(FAvaTagHandle InTransitionLayer) = 0;

	/** Sets whether this transition behavior is enabled */
	virtual void SetEnabled(bool bInEnabled) = 0;

	/** Determines whether the transition behavior is enabled */
	virtual bool IsEnabled() const = 0;

	/** Determines the instancing mode of the transition behavior */
	UFUNCTION(BlueprintCallable, Category = "Transition Logic")
	virtual EAvaTransitionInstancingMode GetInstancingMode() const = 0;

	/** Sets the instancing mode for this transition behavior */
	virtual void SetInstancingMode(EAvaTransitionInstancingMode InInstancingMode) = 0;

	/** Gets the underlying State Tree reference that this transition behavior runs */
	virtual const FStateTreeReference& GetStateTreeReference() const = 0;

#if WITH_EDITOR
	struct FPropertyContext
	{
		/** Name of the property */
		FName Name;
	};
	/** Calls the given functor on each property that is allowed to be visible in an external details panel */
	virtual void ForEachDetailsEditableProperty(TFunctionRef<void(const FPropertyContext&)> InFunc) const = 0;
#endif
};
