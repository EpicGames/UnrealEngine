// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionNode.h"
#include "StateTreeConditionBase.h"
#include "StateTreeExecutionTypes.h"
#include "AvaTransitionCondition.generated.h"

USTRUCT(Category="Transition Logic", meta=(Hidden))
struct FAvaTransitionCondition : public FStateTreeConditionBase
#if CPP
	, public FAvaTransitionNode
#endif
{
	GENERATED_BODY()

	//~ Begin FStateTreeNodeBase
	virtual bool Link(FStateTreeLinker& InLinker) override { return LinkNode(InLinker); }
	//~ End FStateTreeNodeBase
};
