// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behavior/AvaTransitionBehaviorInstanceCache.h"
#include "Blueprint/StateTreeConditionBlueprintBase.h"
#include "IAvaTransitionNodeInterface.h"
#include "AvaTransitionConditionBlueprint.generated.h"

UCLASS(MinimalAPI, Abstract, DisplayName="Motion Design Transition Condition Blueprint")
class UAvaTransitionConditionBlueprint : public UStateTreeConditionBlueprintBase, public IAvaTransitionNodeInterface
{
	GENERATED_BODY()

protected:
	//~ Begin UStateTreeConditionBlueprintBase
	AVALANCHETRANSITION_API virtual bool TestCondition(FStateTreeExecutionContext& InContext) const override;
	//~ End UStateTreeConditionBlueprintBase

	//~ Begin IAvaTransitionNodeInterface
	AVALANCHETRANSITION_API virtual const FAvaTransitionBehaviorInstanceCache& GetBehaviorInstanceCache() const override;
	//~ End IAvaTransitionNodeInterface

private:
	mutable FAvaTransitionBehaviorInstanceCache BehaviorInstanceCache;
};
