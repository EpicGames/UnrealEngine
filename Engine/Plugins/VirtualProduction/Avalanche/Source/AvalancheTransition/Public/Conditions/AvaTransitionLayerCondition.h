// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagHandle.h"
#include "AvaTransitionCondition.h"
#include "AvaTransitionLayer.h"
#include "AvaTransitionLayerCondition.generated.h"

struct FAvaTransitionContext;

USTRUCT()
struct FAvaTransitionLayerConditionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Parameter")
	EAvaTransitionLayerCompareType LayerType = EAvaTransitionLayerCompareType::Same;

	UPROPERTY(EditAnywhere, Category="Parameter", meta=(EditCondition="LayerType==EAvaTransitionLayerCompareType::MatchingTag", EditConditionHides))
	FAvaTagHandle SpecificLayer;
};

USTRUCT(Category="Transition Logic", meta=(Hidden))
struct AVALANCHETRANSITION_API FAvaTransitionLayerCondition : public FAvaTransitionCondition
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaTransitionLayerConditionInstanceData;

	//~ Begin FStateTreeNodeBase
#if WITH_EDITOR
	virtual FText GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const override;
#endif
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	//~ End FStateTreeNodeBase

	/** Gets all the Behavior Instances that match the Layer Query. Always excludes the Instance belonging to this Transition */
	TArray<const FAvaTransitionBehaviorInstance*> QueryBehaviorInstances(const FStateTreeExecutionContext& InContext) const;
};
