// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionLayerCondition.h"
#include "AvaTransitionLayerMatchCondition.generated.h"

USTRUCT()
struct FAvaTransitionLayerMatchConditionInstanceData : public FAvaTransitionLayerConditionInstanceData
{
	GENERATED_BODY()
};

USTRUCT(DisplayName="Layer is Transitioning", Category="Transition Logic")
struct AVALANCHETRANSITION_API FAvaTransitionLayerMatchCondition : public FAvaTransitionLayerCondition
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaTransitionLayerMatchConditionInstanceData;

	//~ Begin FStateTreeNodeBase
#if WITH_EDITOR
	virtual FText GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const override;
#endif
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	//~ End FStateTreeNodeBase

	//~ Begin FStateTreeConditionBase
	virtual bool TestCondition(FStateTreeExecutionContext& InContext) const override;
	//~ End FStateTreeConditionBase
};
