// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionEnums.h"
#include "AvaTransitionLayerCondition.h"
#include "AvaTransitionStateMatchCondition.generated.h"

USTRUCT()
struct FAvaTransitionStateMatchConditionInstanceData : public FAvaTransitionLayerConditionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Parameter")
	EAvaTransitionRunState TransitionState = EAvaTransitionRunState::Running;
};

USTRUCT(DisplayName="Check State of other Scene in Layer", Category="Transition Logic")
struct AVALANCHETRANSITION_API FAvaTransitionStateMatchCondition : public FAvaTransitionLayerCondition
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaTransitionStateMatchConditionInstanceData;

	//~ Begin FStateTreeNodeBase
#if WITH_EDITOR
	virtual FText GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const override;
#endif
	virtual const UStruct* GetInstanceDataType() const override { return FAvaTransitionStateMatchConditionInstanceData::StaticStruct(); }
	//~ End FStateTreeNodeBase

	//~ Begin FStateTreeConditionBase
	virtual bool TestCondition(FStateTreeExecutionContext& InContext) const override;
	//~ End FStateTreeConditionBase
};
