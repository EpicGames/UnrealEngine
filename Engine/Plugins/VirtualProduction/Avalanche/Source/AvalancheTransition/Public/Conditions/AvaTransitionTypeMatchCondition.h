// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionEnums.h"
#include "Conditions/AvaTransitionCondition.h"
#include "AvaTransitionTypeMatchCondition.generated.h"

USTRUCT()
struct FAvaTransitionTypeMatchConditionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Parameter")
	EAvaTransitionType TransitionType = EAvaTransitionType::In;
};

USTRUCT(DisplayName="My Transition Type is", Category="Transition Logic")
struct AVALANCHETRANSITION_API FAvaTransitionTypeMatchCondition : public FAvaTransitionCondition
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaTransitionTypeMatchConditionInstanceData;

	//~ Begin FStateTreeNodeBase
#if WITH_EDITOR
	virtual FText GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const override;
#endif
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	//~ End FStateTreeNodeBase

	//~ Begin FStateTreeConditionBase
	virtual bool TestCondition(FStateTreeExecutionContext& InContext) const override;
	//~ End FStateTreeConditionBase
};
