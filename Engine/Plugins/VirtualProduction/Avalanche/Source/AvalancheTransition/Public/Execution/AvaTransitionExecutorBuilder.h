// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behavior/AvaTransitionBehaviorInstance.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointerFwd.h"

class IAvaTransitionExecutor;
class UAvaTransitionSubsystem;

/** Builder Class for Base Transition Executor Implementation */
class AVALANCHETRANSITION_API FAvaTransitionExecutorBuilder
{
	friend class FAvaTransitionExecutor;

public:
	FAvaTransitionExecutorBuilder();

	FAvaTransitionExecutorBuilder& SetContextName(const FString& InContextName);

	FAvaTransitionExecutorBuilder& AddEnterInstance(FAvaTransitionBehaviorInstance& InInstance);

	FAvaTransitionExecutorBuilder& AddExitInstance(FAvaTransitionBehaviorInstance& InInstance);

	/**
	 * Optional: Sets a null instance to use as "no transition"
	 * This is used as filler if there are missing behavior instances in a layer.
	 * For example if there is an Enter Instance for a layer, but there was no Exit Instance added for that layer,
	 * a Null Instance is created for that Layer
	 */
	FAvaTransitionExecutorBuilder& SetNullInstance(FAvaTransitionBehaviorInstance& InInstance);

	/** Set the delegate to call when the Transition finishes */
	FAvaTransitionExecutorBuilder& SetOnFinished(FSimpleDelegate InDelegate);

	[[nodiscard]] TSharedRef<IAvaTransitionExecutor> Build(UAvaTransitionSubsystem& InTransitionSubsystem);

private:
	TArray<FAvaTransitionBehaviorInstance> EnterInstances;

	TArray<FAvaTransitionBehaviorInstance> ExitInstances;

	FAvaTransitionBehaviorInstance NullInstance;

	FString ContextName;

	FSimpleDelegate OnFinished;
};
