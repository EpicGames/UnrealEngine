// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionLayer.h"
#include "Templates/SharedPointer.h"

struct FAvaTransitionLayerComparator;

/**
 * Responsible for execution of Transition Behaviors and signaling when these Behaviors have completed.
 * NOTE: The Executor shared reference should be kept alive until it completes the behaviors
 */
class IAvaTransitionExecutor : public TSharedFromThis<IAvaTransitionExecutor>
{
public:
	virtual ~IAvaTransitionExecutor() = default;

	/** Gets all the Behavior Instances matching the Layer Comparator */
	virtual TArray<const FAvaTransitionBehaviorInstance*> GetBehaviorInstances(const FAvaTransitionLayerComparator& InComparator) const = 0;

	/** Runs the provided callback for each Behavior Instance */
	virtual void ForEachBehaviorInstance(TFunctionRef<void(const FAvaTransitionBehaviorInstance&)> InCallable) const = 0;

	/** Start Transition. Should only be called once */
	virtual void Start() = 0;

	/** Stop the current Execution */
	virtual void Stop() = 0;

	static TMulticastDelegate<void(const IAvaTransitionExecutor&)>& GetOnTransitionStart()
	{
		return OnTransitionStart;
	}

private:
	AVALANCHETRANSITION_API static TMulticastDelegate<void(const IAvaTransitionExecutor&)> OnTransitionStart;
};
