// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

#include "Templates/Models.h"
#include "Templates/UnrealTypeTraits.h"

class FName;

/**
 * Transition Extensions offers capabilities that connect a transition task/condition to some external logic.
 * The Base Extension must be implemented with a ExtensionIdentifier static variable so that it can be identified as T::ExtensionIdentifier.
 * This Extension Identifier is used as the map key to the extensions in the Transition Scene.
 * Example Base Class:

	class IMyTransitionExtension : public IAvaTransitionExtension
	{
	public:
		static constexpr const TCHAR* ExtensionIdentifier = TEXT("MyExtension");
		virtual bool Compare(...) = 0;
	};

 * @see FAvaTransitionScene::FindExtension, FAvaTransitionScene::AddExtension
 */
class IAvaTransitionExtension
{
	struct CTransitionExtensionProvider
	{
		template<typename T>
		auto Requires(FName& OutIdentifier)->decltype(
			OutIdentifier = T::ExtensionIdentifier
		);
	};

public:
	template<typename T>
	static constexpr bool TIsValidExtension_V = std::is_base_of_v<IAvaTransitionExtension, T> && TModels_V<CTransitionExtensionProvider, T>;
};
