// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class UAvaTransitionTree;

class IAvaTransitionModule : public IModuleInterface
{
	static constexpr const TCHAR* ModuleName = TEXT("AvalancheTransition");

public:
	static bool IsLoaded()
	{
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	static IAvaTransitionModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IAvaTransitionModule>(ModuleName);
	}

	/** Delegate to ensure that the Tree has been Initialized correctly */
	DECLARE_DELEGATE_OneParam(FOnValidateTransitionTree, UAvaTransitionTree*);
	virtual FOnValidateTransitionTree& GetOnValidateTransitionTree() = 0;
};
