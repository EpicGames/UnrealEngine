// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IAvaTransitionNodeInterface.generated.h"

struct FAvaTransitionBehaviorInstanceCache;

/** Interface that enable nodes that store a Behavior Instance Cache to expose this execution cache for others like function libraries to retrieve it */
UINTERFACE(MinimalAPI, meta=(DisplayName="Motion Design Transition Node Interface", CannotImplementInterfaceInBlueprint))
class UAvaTransitionNodeInterface : public UInterface
{
	GENERATED_BODY()
};

class IAvaTransitionNodeInterface
{
	GENERATED_BODY()

public:
	virtual const FAvaTransitionBehaviorInstanceCache& GetBehaviorInstanceCache() const = 0;
};
