// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "PrimitiveComponentId.h"
#include "Subsystems/WorldSubsystem.h"
#include "UObject/ObjectKey.h"
#include "AvaTransitionRenderingSubsystem.generated.h"

class FSceneView;
class ULevel;

UCLASS(MinimalAPI)
class UAvaTransitionRenderingSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	AVALANCHETRANSITION_API void ShowLevel(TObjectKey<ULevel> InLevel);
	AVALANCHETRANSITION_API void HideLevel(TObjectKey<ULevel> InLevel);

	void SetupView(FSceneView& InView);

private:
	/** List of levels that will be hidden. Same level can be repeated to indicate multiple places are actively trying to have the level hidden */
	TArray<TObjectKey<ULevel>> HiddenLevels;

	TSet<FPrimitiveComponentId> HiddenPrimitives;

	/** Temporary Member Variable to keep Track of the Processed Levels when Iterating Hidden Levels */
	TSet<TObjectKey<ULevel>> ProcessedLevels;
};
