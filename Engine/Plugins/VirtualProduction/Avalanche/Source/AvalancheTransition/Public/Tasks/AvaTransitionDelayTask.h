// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionEnums.h"
#include "AvaTransitionTask.h"
#include "UObject/ObjectKey.h"
#include "AvaTransitionDelayTask.generated.h"

class UAvaTransitionRenderingSubsystem;
class ULevel;

USTRUCT()
struct FAvaTransitionDelayTaskInstanceData
{
	GENERATED_BODY()

	/** Delay in seconds before the task ends. */
	UPROPERTY(EditAnywhere, Category="Parameter", meta=(ClampMin="0.0"))
	float Duration = 0.5f;

	/** Hide mode to use while the Wait is taking place */
	UPROPERTY(EditAnywhere, Category="Transition Logic")
	EAvaTransitionLevelHideMode HideMode = EAvaTransitionLevelHideMode::NoHide;

	/** Internal countdown in seconds. */
	float RemainingTime = 0.f;

	TObjectKey<ULevel> HiddenLevel;
};

USTRUCT(DisplayName="Delay", Category="Transition Logic")
struct AVALANCHETRANSITION_API FAvaTransitionDelayTask : public FAvaTransitionTask
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaTransitionDelayTaskInstanceData;

	//~ Begin FStateTreeNodeBase
#if WITH_EDITOR
	virtual FText GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const override;
#endif
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	virtual bool Link(FStateTreeLinker& InLinker) override;
	//~ End FStateTreeNodeBase

	//~ Begin FStateTreeTaskBase
	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& InContext, const FStateTreeTransitionResult& InTransition) const override;
	virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& InContext, const float InDeltaTime) const override;
	//~ End FStateTreeTaskBase

	EStateTreeRunStatus WaitForDelayCompletion(FStateTreeExecutionContext& InContext, FInstanceDataType& InInstanceData) const;

	bool ShouldHideLevel(const FStateTreeExecutionContext& InContext, const FInstanceDataType& InInstanceData) const;

	TStateTreeExternalDataHandle<UAvaTransitionRenderingSubsystem> RenderingSubsystemHandle;
};
