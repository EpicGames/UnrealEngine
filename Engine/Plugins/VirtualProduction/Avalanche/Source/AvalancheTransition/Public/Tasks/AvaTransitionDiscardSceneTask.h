// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionTask.h"
#include "AvaTransitionDiscardSceneTask.generated.h"

USTRUCT()
struct FAvaTransitionDiscardSceneInstanceData
{
	GENERATED_BODY()
};

USTRUCT(DisplayName = "Discard Self", Category="Transition Logic")
struct AVALANCHETRANSITION_API FAvaTransitionDiscardSceneTask : public FAvaTransitionTask
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaTransitionDiscardSceneInstanceData;

	//~ Begin FStateTreeNodeBase
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	//~ End FStateTreeNodeBase

	//~ Begin FStateTreeTaskBase
	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& InContext, const FStateTreeTransitionResult& InTransition) const override;
	//~ End FStateTreeTaskBase
};
