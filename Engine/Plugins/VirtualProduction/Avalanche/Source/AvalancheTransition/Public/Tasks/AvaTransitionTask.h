// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionNode.h"
#include "StateTreeExecutionTypes.h"
#include "StateTreeTaskBase.h"
#include "AvaTransitionTask.generated.h"

USTRUCT(meta=(Hidden))
struct FAvaTransitionTask : public FStateTreeTaskBase
#if CPP
	, public FAvaTransitionNode
#endif
{
	GENERATED_BODY()

	//~ Begin FStateTreeNodeBase
	virtual bool Link(FStateTreeLinker& InLinker) override { return LinkNode(InLinker); }
	//~ End FStateTreeNodeBase
};
