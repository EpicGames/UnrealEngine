// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behavior/AvaTransitionBehaviorInstanceCache.h"
#include "Blueprint/StateTreeTaskBlueprintBase.h"
#include "IAvaTransitionNodeInterface.h"
#include "AvaTransitionTaskBlueprint.generated.h"

UCLASS(MinimalAPI, Abstract, DisplayName="Motion Design Transition Task Blueprint")
class UAvaTransitionTaskBlueprint : public UStateTreeTaskBlueprintBase, public IAvaTransitionNodeInterface
{
	GENERATED_BODY()

protected:
	//~ Begin UStateTreeTaskBlueprintBase
	AVALANCHETRANSITION_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& InExecutionContext, const FStateTreeTransitionResult& InTransition) override;
	AVALANCHETRANSITION_API virtual void ExitState(FStateTreeExecutionContext& InExecutionContext, const FStateTreeTransitionResult& InTransition) override;
	//~ End UStateTreeTaskBlueprintBase

	//~ Begin IAvaTransitionNodeInterface
	AVALANCHETRANSITION_API virtual const FAvaTransitionBehaviorInstanceCache& GetBehaviorInstanceCache() const override;
	//~ End IAvaTransitionNodeInterface

private:
	FAvaTransitionBehaviorInstanceCache BehaviorInstanceCache;
};
