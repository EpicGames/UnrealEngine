// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionLayerTask.h"
#include "UObject/ObjectKey.h"
#include "AvaTransitionWaitForLayerTask.generated.h"

class UAvaTransitionRenderingSubsystem;
class ULevel;

USTRUCT()
struct FAvaTransitionWaitForLayerTaskInstanceData : public FAvaTransitionLayerTaskInstanceData
{
	GENERATED_BODY()

	/** Hide mode to use while the Wait is taking place */
	UPROPERTY(EditAnywhere, Category="Transition Logic")
	EAvaTransitionLevelHideMode HideMode = EAvaTransitionLevelHideMode::HideUnlessReuse;

	TObjectKey<ULevel> HiddenLevel;
};

USTRUCT(DisplayName = "Wait for other Scenes in Layer to Finish", Category="Transition Logic")
struct AVALANCHETRANSITION_API FAvaTransitionWaitForLayerTask : public FAvaTransitionLayerTask
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaTransitionWaitForLayerTaskInstanceData;

	//~ Begin FStateTreeNodeBase
#if WITH_EDITOR
	virtual FText GetDescription(const FGuid& InId, FStateTreeDataView InInstanceDataView, const IStateTreeBindingLookup& InBindingLookup, EStateTreeNodeFormatting InFormatting) const override;
#endif
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	virtual bool Link(FStateTreeLinker& InLinker) override;
	//~ End FStateTreeNodeBase

	//~ Begin FStateTreeTaskBase
	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& InContext, const FStateTreeTransitionResult& InTransition) const override;
	virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& InContext, const float InDeltaTime) const override;
	//~ End FStateTreeTaskBase

	EStateTreeRunStatus WaitForLayer(FStateTreeExecutionContext& InContext) const;

	bool ShouldHideLevel(const FStateTreeExecutionContext& InContext, const FAvaTransitionWaitForLayerTask::FInstanceDataType& InInstanceData) const;

	TStateTreeExternalDataHandle<UAvaTransitionRenderingSubsystem> RenderingSubsystemHandle;
};
