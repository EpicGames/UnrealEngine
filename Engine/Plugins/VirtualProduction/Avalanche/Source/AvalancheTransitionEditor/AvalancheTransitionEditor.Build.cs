// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheTransitionEditor : ModuleRules
{
    public AvalancheTransitionEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "AvalancheTag",
                "Core",
                "CoreUObject",
                "StateTreeEditorModule",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "ApplicationCore",
                "AssetDefinition",
                "AssetTools",
                "AvalancheCore",
                "AvalancheTransition",
                "DeveloperSettings",
                "EditorStyle",
                "Engine",
                "InputCore",
                "MessageLog",
                "Projects",
                "PropertyBindingUtils",
                "PropertyEditor",
                "Slate",
                "SlateCore",
                "StateTreeModule",
                "StructUtilsEditor",
                "ToolMenus",
                "UnrealEd",
            }
        );
    }
}
