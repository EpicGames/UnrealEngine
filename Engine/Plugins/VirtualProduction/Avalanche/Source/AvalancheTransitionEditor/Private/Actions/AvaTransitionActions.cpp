// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionActions.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "ViewModels/AvaTransitionViewModelSharedData.h"

FAvaTransitionActions::FAvaTransitionActions(FAvaTransitionEditorViewModel& InOwner)
	: Owner(InOwner)
{
}

bool FAvaTransitionActions::IsEditMode() const
{
	return !Owner.GetSharedData()->IsReadOnly();
}
