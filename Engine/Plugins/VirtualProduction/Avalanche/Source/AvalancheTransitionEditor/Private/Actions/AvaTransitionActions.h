// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FAvaTransitionEditorViewModel;
class FUICommandList;

class FAvaTransitionActions : public TSharedFromThis<FAvaTransitionActions>
{
public:
	explicit FAvaTransitionActions(FAvaTransitionEditorViewModel& InOwner);

	virtual ~FAvaTransitionActions() = default;

	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandList) = 0;

	bool IsEditMode() const;

protected:
	FAvaTransitionEditorViewModel& Owner;
};
