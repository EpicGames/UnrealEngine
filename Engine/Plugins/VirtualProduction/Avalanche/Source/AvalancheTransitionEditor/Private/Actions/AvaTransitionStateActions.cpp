// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionStateActions.h"
#include "AvaTransitionCommands.h"
#include "AvaTransitionSelection.h"
#include "AvaTransitionTreeEditorData.h"
#include "DragDrop/AvaTransitionStateDragDropOp.h"
#include "Framework/Commands/GenericCommands.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ScopedTransaction.h"
#include "Serialization/AvaTransitionStateSerializer.h"
#include "StateTreeState.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "ViewModels/AvaTransitionViewModel.h"
#include "ViewModels/AvaTransitionViewModelSharedData.h"
#include "ViewModels/AvaTransitionViewModelUtils.h"
#include "ViewModels/Registry/AvaTransitionViewModelRegistryCollection.h"
#include "ViewModels/State/AvaTransitionStateViewModel.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "AvaTransitionStateActions"

namespace UE::AvaTransitionEditor::Private
{
	void RemoveNestedStates(TArray<UStateTreeState*>& InStates)
	{
		TSet<UStateTreeState*> UniqueStates(InStates);

		for (TArray<UStateTreeState*>::TIterator StateIterator(InStates); StateIterator; ++StateIterator)
		{
			UStateTreeState* State = *StateIterator;
			check(State);

			// Walk up the parent states and if the current state exists in any of them, remove it.
			UStateTreeState* StateParent = State->Parent;
			while (StateParent)
			{
				if (UniqueStates.Contains(StateParent))
				{
					StateIterator.RemoveCurrent();
					break;
				}
				StateParent = StateParent->Parent;
			}
		}
	}

	bool IsStateDescendantOf(const UStateTreeState* InState, const UStateTreeState* InParentState)
	{
		TArray<const UStateTreeState*> States = { InParentState };
		while (!States.IsEmpty())
		{
			if (const UStateTreeState* CurrentState = States.Pop(EAllowShrinking::No))
			{
				if (CurrentState == InState)
				{
					return true;
				}
				States.Append(CurrentState->Children);
			}
		}
		return false;
	}

	TSet<UObject*> ModifyObjects(UStateTreeEditorData* InEditorData, UStateTreeState* InParentState, TConstArrayView<UStateTreeState*> InStates)
	{
		TSet<UObject*> ModifiedObjects;
		ModifiedObjects.Reserve(InStates.Num() + 1);

		auto ModifyObject = [&ModifiedObjects](UObject* InObjectChecked)
			{
				bool bIsAlreadyInSet = false;
				ModifiedObjects.Add(InObjectChecked, &bIsAlreadyInSet);

				if (!bIsAlreadyInSet)
				{
					InObjectChecked->Modify();
				}
			};

		TArray<TObjectPtr<UStateTreeState>>* TargetChildren;
		if (InParentState)
		{
			ModifyObject(InParentState);
			TargetChildren = &InParentState->Children;
		}
		else
		{
			ModifyObject(InEditorData);
			TargetChildren = &InEditorData->SubTrees;
		}

		for (UStateTreeState* State : InStates)
		{
			check(State);
			State->Modify();
			if (State->Parent)
			{
				ModifyObject(State->Parent);
			}
			else
			{
				ModifyObject(InEditorData);
			}
		}

		return ModifiedObjects;
	}

	void RefreshViewModels(FAvaTransitionEditorViewModel& InEditorViewModel, const TSet<UObject*>& InObjects)
	{
		TSharedRef<FAvaTransitionViewModelRegistryCollection> RegistryCollection = InEditorViewModel.GetSharedData()->GetRegistryCollection();

		TArray<TSharedRef<FAvaTransitionViewModel>> ViewModels;
		ViewModels.Reserve(InObjects.Num());

		for (UObject* Object : InObjects)
		{
			if (TSharedPtr<FAvaTransitionViewModel> ViewModel = RegistryCollection->FindViewModel(Object))
			{
				ViewModels.Add(ViewModel.ToSharedRef());
			}
		}

		// Remove nested to only refresh the topmost modified view models 
		UE::AvaTransitionEditor::RemoveNestedViewModels(ViewModels);

		for (const TSharedRef<FAvaTransitionViewModel>& ViewModel : ViewModels)
		{
			// Refresh to create the new state view models
			ViewModel->Refresh();
		}
	}

	void OnStatesAdded(FAvaTransitionEditorViewModel& InEditorViewModel, TConstArrayView<UStateTreeState*> InStates)
	{
		TArray<TSharedPtr<FAvaTransitionViewModel>> StateViewModels;
		StateViewModels.Reserve(InStates.Num());

		TSharedRef<FAvaTransitionViewModelRegistryCollection> RegistryCollection = InEditorViewModel.GetSharedData()->GetRegistryCollection();

		for (UStateTreeState* State : InStates)
		{
			// Select the new State View Models
			if (TSharedPtr<FAvaTransitionViewModel> StateViewModel = RegistryCollection->FindViewModel(State))
			{
				StateViewModels.Add(StateViewModel);
			}
		}

		InEditorViewModel.GetSelection()->SetSelectedItems(StateViewModels);
		InEditorViewModel.UpdateTree();
		InEditorViewModel.RefreshTreeView();
	}
}

bool FAvaTransitionStateActions::CanAddStatesFromDrop(const TSharedRef<FAvaTransitionViewModel>& InTarget, const FAvaTransitionStateDragDropOp& InStateDragDropOp)
{
	TConstArrayView<TSharedRef<FAvaTransitionStateViewModel>> StateViewModels = InStateDragDropOp.GetStateViewModels();
	if (StateViewModels.IsEmpty())
	{
		return false;
	}

	// If duplicating States, no need to check for descendants
	if (InStateDragDropOp.ShouldDuplicateStates())
	{
		return true;
	}

	for (const TSharedRef<FAvaTransitionStateViewModel>& StateViewModel : StateViewModels)
	{
		// if Target is descendant of any of the dragged State View Models, don't allow drop
		if (UE::AvaTransitionEditor::IsDescendantOf(InTarget, StateViewModel))
		{
			return false;
		}
	}

	return true;
}

bool FAvaTransitionStateActions::AddStatesFromDrop(const TSharedRef<FAvaTransitionViewModel>& InTarget, EItemDropZone InDropZone, const FAvaTransitionStateDragDropOp& InStateDragDropOp)
{
	TConstArrayView<TSharedRef<FAvaTransitionStateViewModel>> StateViewModels = InStateDragDropOp.GetStateViewModels();
	if (StateViewModels.IsEmpty())
	{
		return false;
	}

	TArray<UStateTreeState*> States;
	States.Reserve(StateViewModels.Num());

	const bool bDuplicateStates = InStateDragDropOp.ShouldDuplicateStates();

	for (const TSharedRef<FAvaTransitionStateViewModel>& StateViewModel : StateViewModels)
	{
		if (!bDuplicateStates && UE::AvaTransitionEditor::IsDescendantOf(InTarget, StateViewModel))
		{
			continue;
		}

		if (UStateTreeState* State = StateViewModel->GetState())
		{
			States.Add(State);
		}
	}

	if (States.IsEmpty())
	{
		return false;
	}

	FScopedTransaction Transaction(LOCTEXT("MoveStates", "Move States"));

	if (bDuplicateStates)
	{
		for (UStateTreeState*& State : States)
		{
			State = DuplicateObject<UStateTreeState>(State, State->GetOuter());
		}
	}

	return AddStates(InTarget, InDropZone, MoveTemp(States));
}

void FAvaTransitionStateActions::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	const FAvaTransitionEditorCommands& Commands = FAvaTransitionEditorCommands::Get();

	const FGenericCommands& GenericCommands = FGenericCommands::Get();

	InCommandList->MapAction(Commands.AddComment
		, FExecuteAction::CreateSP(this, &FAvaTransitionStateActions::AddStateComment)
		, FCanExecuteAction::CreateSP(this, &FAvaTransitionStateActions::CanAddStateComment)
		, FGetActionCheckState()
		, FIsActionButtonVisible::CreateSP(this, &FAvaTransitionStateActions::CanAddStateComment));

	InCommandList->MapAction(Commands.RemoveComment
		, FExecuteAction::CreateSP(this, &FAvaTransitionStateActions::RemoveStateComment)
		, FCanExecuteAction::CreateSP(this, &FAvaTransitionStateActions::CanRemoveStateComment)
		, FGetActionCheckState()
		, FIsActionButtonVisible::CreateSP(this, &FAvaTransitionStateActions::CanRemoveStateComment));

	InCommandList->MapAction(Commands.AddSiblingState
		, FExecuteAction::CreateSP(this, &FAvaTransitionStateActions::AddState, EItemDropZone::BelowItem)
		, FCanExecuteAction::CreateSP(this, &FAvaTransitionActions::IsEditMode));

	InCommandList->MapAction(Commands.AddChildState
		, FExecuteAction::CreateSP(this, &FAvaTransitionStateActions::AddState, EItemDropZone::OntoItem)
		, FCanExecuteAction::CreateSP(this, &FAvaTransitionActions::IsEditMode));

	InCommandList->MapAction(Commands.EnableStates
		, FExecuteAction::CreateSP(this, &FAvaTransitionStateActions::ToggleEnableStates)
		, FCanExecuteAction::CreateSP(this, &FAvaTransitionStateActions::CanEditSelectedStates)
		, FIsActionChecked::CreateSP(this, &FAvaTransitionStateActions::AreStatesEnabled));

	InCommandList->MapAction(GenericCommands.Cut
		, FExecuteAction::CreateSP(this, &FAvaTransitionStateActions::CopyStates, /*bDeleteStatesAfterCopy*/true)
		, FCanExecuteAction::CreateSP(this, &FAvaTransitionStateActions::CanEditSelectedStates));

	// Copy uses HasSelectedStates over CanEditSelectedStates, as it doesn't modify them (only copies the states to clipboard)
	InCommandList->MapAction(GenericCommands.Copy
		, FExecuteAction::CreateSP(this, &FAvaTransitionStateActions::CopyStates, /*bDeleteStatesAfterCopy*/false)
		, FCanExecuteAction::CreateSP(this, &FAvaTransitionStateActions::HasSelectedStates));

	InCommandList->MapAction(GenericCommands.Paste
		, FExecuteAction::CreateSP(this, &FAvaTransitionStateActions::PasteStates));

	InCommandList->MapAction(GenericCommands.Duplicate
		, FExecuteAction::CreateSP(this, &FAvaTransitionStateActions::DuplicateStates)
		, FCanExecuteAction::CreateSP(this, &FAvaTransitionStateActions::CanEditSelectedStates));

	InCommandList->MapAction(GenericCommands.Delete
		, FExecuteAction::CreateSP(this, &FAvaTransitionStateActions::DeleteStates)
		, FCanExecuteAction::CreateSP(this, &FAvaTransitionStateActions::CanEditSelectedStates));
}

TConstArrayView<TSharedRef<FAvaTransitionViewModel>> FAvaTransitionStateActions::GetSelectedViewModels() const
{
	return Owner.GetSelection()->GetSelectedItems();
}

TArray<UStateTreeState*> FAvaTransitionStateActions::GetSelectedStates() const
{
	TConstArrayView<TSharedRef<FAvaTransitionViewModel>> SelectedItems = GetSelectedViewModels();

	TArray<UStateTreeState*> States;
	States.Reserve(SelectedItems.Num());

	for (const TSharedRef<FAvaTransitionViewModel>& SelectedViewModel : SelectedItems)
	{
		if (IAvaTransitionObjectExtension* ObjectExtension = SelectedViewModel->CastTo<IAvaTransitionObjectExtension>())
		{
			if (UStateTreeState* State = Cast<UStateTreeState>(ObjectExtension->GetObject()))
			{
				States.Add(State);
			}
		}
	}

	UE::AvaTransitionEditor::Private::RemoveNestedStates(States);

	return States;
}

TSharedPtr<FAvaTransitionStateViewModel> FAvaTransitionStateActions::GetLastSelectedStateViewModel() const
{
	TConstArrayView<TSharedRef<FAvaTransitionViewModel>> SelectedViewModels = GetSelectedViewModels();

	for (const TSharedRef<FAvaTransitionViewModel>& ViewModel : ReverseIterate(SelectedViewModels))
	{
		if (TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = UE::AvaCore::CastSharedPtr<FAvaTransitionStateViewModel>(ViewModel))
		{
			return StateViewModel;
		}
	}

	return nullptr;
}

bool FAvaTransitionStateActions::HasSelectedStates() const
{
	return GetLastSelectedStateViewModel().IsValid();
}

bool FAvaTransitionStateActions::CanEditSelectedStates() const
{
	return IsEditMode() && HasSelectedStates();
}

bool FAvaTransitionStateActions::CanAddStateComment() const
{
	if (!IsEditMode())
	{
		return false;
	}

	for (const TSharedRef<FAvaTransitionViewModel>& ViewModel : GetSelectedViewModels())
	{
		TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = UE::AvaCore::CastSharedPtr<FAvaTransitionStateViewModel>(ViewModel);
		if (StateViewModel && StateViewModel->GetComment().IsEmpty())
		{
			return true;
		}
	}

	return false;
}

void FAvaTransitionStateActions::AddStateComment()
{
	FScopedTransaction Transaction(LOCTEXT("AddStateComment", "Add State Comment"));

	for (const TSharedRef<FAvaTransitionViewModel>& ViewModel : GetSelectedViewModels())
	{
		TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = UE::AvaCore::CastSharedPtr<FAvaTransitionStateViewModel>(ViewModel);
		if (StateViewModel && StateViewModel->GetComment().IsEmpty())
		{
			FString StateDescription = StateViewModel->GetStateDescription().ToString();

			StateViewModel->SetComment(StateDescription.IsEmpty()
				? TEXT("Default comment")
				: StateDescription);
		}
	}
}

bool FAvaTransitionStateActions::CanRemoveStateComment() const
{
	if (!IsEditMode())
	{
		return false;
	}

	for (const TSharedRef<FAvaTransitionViewModel>& ViewModel : GetSelectedViewModels())
	{
		TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = UE::AvaCore::CastSharedPtr<FAvaTransitionStateViewModel>(ViewModel);
		if (StateViewModel && !StateViewModel->GetComment().IsEmpty())
		{
			return true;
		}
	}

	return false;
}

void FAvaTransitionStateActions::RemoveStateComment()
{
	FScopedTransaction Transaction(LOCTEXT("RemoveStateComment", "Remove State Comment"));

	FString EmptyString;

	for (const TSharedRef<FAvaTransitionViewModel>& ViewModel : GetSelectedViewModels())
	{
		if (TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = UE::AvaCore::CastSharedPtr<FAvaTransitionStateViewModel>(ViewModel))
		{
			return StateViewModel->SetComment(EmptyString);
		}
	}
}

void FAvaTransitionStateActions::AddState(EItemDropZone InDropZone)
{
	TSharedPtr<FAvaTransitionViewModel> TargetViewModel;

	UObject* Outer = nullptr;

	if (TSharedPtr<FAvaTransitionStateViewModel> SelectedStateViewModel = GetLastSelectedStateViewModel())
	{
		Outer = SelectedStateViewModel->GetParentState();
		TargetViewModel = SelectedStateViewModel;
	}
	else
	{
		TargetViewModel = Owner.AsShared();
	}

	if (!Outer)
	{
		Outer = Owner.GetEditorData();
	}

	if (!ensureAlwaysMsgf(Outer, TEXT("Failed to add add new state. Editor Data was unexpectedly null, and could not be used as outer")))
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddState", "Add State"));

	UStateTreeState* NewState = NewObject<UStateTreeState>(Outer, NAME_None, RF_Transactional);

	AddStates(TargetViewModel.ToSharedRef(), InDropZone, { NewState });
}

bool FAvaTransitionStateActions::AreStatesEnabled() const
{
	bool bStateViewModelFound = false;

	for (const TSharedRef<FAvaTransitionViewModel>& ViewModel : GetSelectedViewModels())
	{
		if (TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = UE::AvaCore::CastSharedPtr<FAvaTransitionStateViewModel>(ViewModel))
		{
			if (!StateViewModel->IsStateEnabled())
			{
				return false;
			}

			bStateViewModelFound = true;
		}
	}

	return bStateViewModelFound;
}

void FAvaTransitionStateActions::ToggleEnableStates()
{
	TConstArrayView<TSharedRef<FAvaTransitionViewModel>> SelectedViewModels = GetSelectedViewModels();
	if (SelectedViewModels.IsEmpty())
	{
		return;
	}

	bool bEnableState = !AreStatesEnabled();

	FScopedTransaction Transaction(LOCTEXT("ToggleEnableStates", "Toggle Enable States"));

	for (const TSharedRef<FAvaTransitionViewModel>& ViewModel : SelectedViewModels)
	{
		if (TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = UE::AvaCore::CastSharedPtr<FAvaTransitionStateViewModel>(ViewModel))
		{
			StateViewModel->SetStateEnabled(bEnableState);
		}
	}
}

void FAvaTransitionStateActions::CopyStates(bool bInDeleteStatesAfterCopy)
{
	UAvaTransitionTreeEditorData* EditorData = Owner.GetEditorData();
	if (!EditorData)
	{
		return;
	}

	TArray<UStateTreeState*> States = GetSelectedStates();

	FString ExportedText = FAvaTransitionStateSerializer::ExportText(*EditorData, States);

	FPlatformApplicationMisc::ClipboardCopy(*ExportedText);

	if (bInDeleteStatesAfterCopy)
	{
		FScopedTransaction Transaction(LOCTEXT("CutStates", "Cut States"));
		DeleteStates(*EditorData, States);
	}
}

void FAvaTransitionStateActions::PasteStates()
{
	UAvaTransitionTreeEditorData* EditorData = Owner.GetEditorData();
	if (!EditorData)
	{
		return;
	}

	UStateTreeState* SelectedState = nullptr;
	if (TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = GetLastSelectedStateViewModel())
	{
		SelectedState = StateViewModel->GetState();
	}

	FString ImportedText;
	FPlatformApplicationMisc::ClipboardPaste(ImportedText);

	FScopedTransaction Transaction(LOCTEXT("PasteStates", "Paste States"));

	UObject* Outer = nullptr;
	TArray<UStateTreeState*> NewStates;
	if (FAvaTransitionStateSerializer::ImportText(ImportedText, *EditorData, SelectedState, NewStates, Outer))
	{
		UE::AvaTransitionEditor::Private::RefreshViewModels(Owner, { Outer });
		UE::AvaTransitionEditor::Private::OnStatesAdded(Owner, NewStates);
	}
}

void FAvaTransitionStateActions::DuplicateStates()
{
	UAvaTransitionTreeEditorData* EditorData = Owner.GetEditorData();
	if (!EditorData)
	{
		return;
	}

	TArray<UStateTreeState*> States = GetSelectedStates();
	if (States.IsEmpty())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("DuplicateStates", "Duplicate States"));

	FString ExportedText = FAvaTransitionStateSerializer::ExportText(*EditorData, States);

	UObject* Outer = nullptr;
	TArray<UStateTreeState*> NewStates;
	if (FAvaTransitionStateSerializer::ImportText(ExportedText, *EditorData, States.Last(), NewStates, Outer))
	{
		UE::AvaTransitionEditor::Private::RefreshViewModels(Owner, { Outer });
		UE::AvaTransitionEditor::Private::OnStatesAdded(Owner, NewStates);
	}
}

void FAvaTransitionStateActions::DeleteStates()
{
	UAvaTransitionTreeEditorData* EditorData = Owner.GetEditorData();
	if (!EditorData)
	{
		return;
	}

	TArray<UStateTreeState*> States = GetSelectedStates();
	if (States.IsEmpty())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("DeleteStates", "Delete States"));
	DeleteStates(*EditorData, States);
}

bool FAvaTransitionStateActions::AddStates(const TSharedRef<FAvaTransitionViewModel>& InTarget, EItemDropZone InDropZone, TArray<UStateTreeState*> InStates)
{
	// Remove invalid states
	InStates.RemoveAll(
		[](UStateTreeState* InState)
		{
			return !IsValid(InState);
		});

	// Remove nested states to keep hierarchy
	UE::AvaTransitionEditor::Private::RemoveNestedStates(InStates);

	if (InStates.IsEmpty())
	{
		return false;
	}

	TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = InTarget->GetSharedData()->GetEditorViewModel();
	if (!EditorViewModel.IsValid())
	{
		return false;
	}

	UAvaTransitionTreeEditorData* EditorData = EditorViewModel->GetEditorData();
	if (!EditorData)
	{
		return false;
	}

	UStateTreeState* TargetParentState = nullptr;
	UStateTreeState* TargetSiblingState = nullptr;

	if (TSharedPtr<FAvaTransitionStateViewModel> TargetStateViewModel = UE::AvaCore::CastSharedPtr<FAvaTransitionStateViewModel>(InTarget))
	{
		switch (InDropZone)
		{
		case EItemDropZone::AboveItem:
		case EItemDropZone::BelowItem:
			TargetParentState  = TargetStateViewModel->GetParentState();
			TargetSiblingState = TargetStateViewModel->GetState();
			break;

		case EItemDropZone::OntoItem:
			TargetParentState  = TargetStateViewModel->GetState();
			TargetSiblingState = nullptr;
			break;
		}
	}

	// Remove all the States that contain TargetParentState as a descendant
	if (TargetParentState)
	{
		InStates.RemoveAll(
			[TargetParentState](const UStateTreeState* InState)
			{
				return UE::AvaTransitionEditor::Private::IsStateDescendantOf(TargetParentState, InState);
			});

		if (InStates.IsEmpty())
		{
			return false;
		}
	}

	TSet<UObject*> ModifiedObjects = UE::AvaTransitionEditor::Private::ModifyObjects(EditorData, TargetParentState, InStates);

	TArray<TObjectPtr<UStateTreeState>>& TargetChildren = TargetParentState
		? TargetParentState->Children
		: EditorData->SubTrees;

	// Add in reverse order to keep the original order (as these will be inserted in the same index)
	for (UStateTreeState* State : ReverseIterate(InStates))
	{
		if (!State)
		{
			continue;
		}

		if (State->Parent)
		{
			State->Parent->Children.Remove(State);
		}
		else
		{
			EditorData->SubTrees.Remove(State);
		}

		State->Parent = TargetParentState;

		int32 InsertIndex = INDEX_NONE;

		if (InDropZone != EItemDropZone::OntoItem && TargetSiblingState)
		{
			InsertIndex = TargetChildren.Find(TargetSiblingState);
			// Insert after Target Sibling State
			if (InDropZone == EItemDropZone::BelowItem)
			{
				++InsertIndex;
			}
		}

		if (InsertIndex == INDEX_NONE)
		{
			InsertIndex = 0;
		}

		TargetChildren.Insert(State, InsertIndex);
	}

	UE::AvaTransitionEditor::Private::RefreshViewModels(*EditorViewModel, ModifiedObjects);
	UE::AvaTransitionEditor::Private::OnStatesAdded(*EditorViewModel, InStates);
	return true;
}

void FAvaTransitionStateActions::DeleteStates(UAvaTransitionTreeEditorData& InEditorData, TConstArrayView<UStateTreeState*> InStates)
{
	for (UStateTreeState* StateToRemove : InStates)
	{
		if (!StateToRemove)
		{
			continue;
		}

		StateToRemove->Modify();

		UStateTreeState* ParentState = StateToRemove->Parent;
		TArray<TObjectPtr<UStateTreeState>>* ChildStates;

		if (ParentState)
		{
			ParentState->Modify();
			ChildStates = &ParentState->Children;
		}
		else
		{
			InEditorData.Modify();
			ChildStates = &InEditorData.SubTrees;
		}

		ChildStates->Remove(StateToRemove);
		StateToRemove->Parent = nullptr;
	}

	Owner.GetSelection()->ClearSelectedItems();
	Owner.Refresh();
}

#undef LOCTEXT_NAMESPACE
