// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionActions.h"
#include "Containers/Array.h"

class FAvaTransitionStateDragDropOp;
class FAvaTransitionStateViewModel;
class FAvaTransitionViewModel;
class UAvaTransitionTreeEditorData;
class UStateTreeState;
enum class EItemDropZone;

class FAvaTransitionStateActions : public FAvaTransitionActions
{
public:
	explicit FAvaTransitionStateActions(FAvaTransitionEditorViewModel& InOwner)
		: FAvaTransitionActions(InOwner)
	{
	}

	static bool CanAddStatesFromDrop(const TSharedRef<FAvaTransitionViewModel>& InTarget, const FAvaTransitionStateDragDropOp& InStateDragDropOp);

	static bool AddStatesFromDrop(const TSharedRef<FAvaTransitionViewModel>& InTarget, EItemDropZone InDropZone, const FAvaTransitionStateDragDropOp& InStateDragDropOp);

protected:
	//~ Begin FAvaTransitionActions
	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandList) override;
	//~ End FAvaTransitionActions

	TConstArrayView<TSharedRef<FAvaTransitionViewModel>> GetSelectedViewModels() const;

	TArray<UStateTreeState*> GetSelectedStates() const;

	TSharedPtr<FAvaTransitionStateViewModel> GetLastSelectedStateViewModel() const;

	bool HasSelectedStates() const;

	bool CanEditSelectedStates() const;

	bool CanAddStateComment() const;
	void AddStateComment();

	bool CanRemoveStateComment() const;
	void RemoveStateComment();

	void AddState(EItemDropZone InDropZone);

	bool AreStatesEnabled() const;
	void ToggleEnableStates();

	void CopyStates(bool bInDeleteStatesAfterCopy);

	void PasteStates();

	void DuplicateStates();

	void DeleteStates();

private:
	static bool AddStates(const TSharedRef<FAvaTransitionViewModel>& InTarget, EItemDropZone InDropZone, TArray<UStateTreeState*> InStates);

	void DeleteStates(UAvaTransitionTreeEditorData& InEditorData, TConstArrayView<UStateTreeState*> InStates);
};
