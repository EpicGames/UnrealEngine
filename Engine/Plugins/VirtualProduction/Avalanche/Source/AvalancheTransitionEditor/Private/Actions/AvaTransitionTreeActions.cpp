// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionTreeActions.h"
#include "AvaTransitionCommands.h"
#include "AvaTransitionEditorLog.h"
#include "AvaTransitionEditorUtils.h"
#include "AvaTransitionTree.h"
#include "AvaTransitionTreeEditorData.h"
#include "IAssetTools.h"
#include "ScopedTransaction.h"
#include "StateTreeEditorSettings.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "ViewModels/AvaTransitionViewModelSharedData.h"

#if UE_AVA_WITH_TRANSITION_DEBUG
#include "Debugger/AvaTransitionDebugger.h"
#endif

#define LOCTEXT_NAMESPACE "AvaTransitionTreeActions"

void FAvaTransitionTreeActions::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	const FAvaTransitionEditorCommands& Commands = FAvaTransitionEditorCommands::Get();

	InCommandList->MapAction(Commands.ImportTransitionTree
		, FExecuteAction::CreateSP(this, &FAvaTransitionTreeActions::ImportTransitionTree)
		, FCanExecuteAction::CreateSP(this, &FAvaTransitionActions::IsEditMode));

	InCommandList->MapAction(Commands.ExportTransitionTree
		, FExecuteAction::CreateSP(this, &FAvaTransitionTreeActions::ExportTransitionTree)
		, FCanExecuteAction()
		, FGetActionCheckState()
		, FIsActionButtonVisible::CreateSP(this, &FAvaTransitionTreeActions::CanShowExportTransitionTree));

	InCommandList->MapAction(Commands.Compile
		, FExecuteAction::CreateSP(&Owner, &FAvaTransitionEditorViewModel::Compile)
		, FCanExecuteAction::CreateSP(&Owner, &FAvaTransitionEditorViewModel::CanCompile));

	InCommandList->MapAction(Commands.SaveOnCompile_Never
		, FExecuteAction::CreateStatic(&FAvaTransitionCompiler::SetSaveOnCompile, EStateTreeSaveOnCompile::Never)
		, FCanExecuteAction()
		, FIsActionChecked::CreateStatic(&FAvaTransitionCompiler::HasSaveOnCompile, EStateTreeSaveOnCompile::Never));

	InCommandList->MapAction(Commands.SaveOnCompile_SuccessOnly
		, FExecuteAction::CreateStatic(&FAvaTransitionCompiler::SetSaveOnCompile, EStateTreeSaveOnCompile::SuccessOnly)
		, FCanExecuteAction()
		, FIsActionChecked::CreateStatic(&FAvaTransitionCompiler::HasSaveOnCompile, EStateTreeSaveOnCompile::SuccessOnly));

#if UE_AVA_WITH_TRANSITION_DEBUG
	InCommandList->MapAction(Commands.ToggleDebug
		, FExecuteAction::CreateSP(this, &FAvaTransitionTreeActions::ToggleDebugger)
		, FCanExecuteAction()
		, FIsActionChecked::CreateSP(this, &FAvaTransitionTreeActions::IsDebuggerActive));
#endif
}

bool FAvaTransitionTreeActions::CanShowExportTransitionTree() const
{
	UAvaTransitionTree* TransitionTree = Owner.GetTransitionTree();
	return TransitionTree && !TransitionTree->IsAsset();
}

void FAvaTransitionTreeActions::ExportTransitionTree()
{
	UAvaTransitionTree* TransitionTree = Owner.GetTransitionTree();
	if (!TransitionTree)
	{
		return;
	}

	UPackage* Package = TransitionTree->GetPackage();
	check(Package);

	IAssetTools& AssetTools = IAssetTools::Get();

	FString TargetName;
	FString TargetPackageName;

	AssetTools.CreateUniqueAssetName(Package->GetName(), TEXT("_TransitionTree"), TargetPackageName, TargetName);
	AssetTools.DuplicateAssetWithDialog(TargetName, TargetPackageName, TransitionTree);
}

void FAvaTransitionTreeActions::ImportTransitionTree()
{
	UAvaTransitionTree* TransitionTree = Owner.GetTransitionTree();
	if (!TransitionTree)
	{
		return;
	}

	using namespace UE::AvaTransitionEditor;

	UAvaTransitionTree* TemplateTree;
	if (PickTransitionTreeAsset(LOCTEXT("ImportTransitionTreeTitle", "Import TransitionTree"), TemplateTree))
	{
		ImportTransitionTree(TemplateTree);
	}
}

void FAvaTransitionTreeActions::ImportTransitionTree(UAvaTransitionTree* InTemplateTree)
{
	if (!InTemplateTree)
	{
		UE_LOGF(LogAvaEditorTransition, Error, "Failed to Import Transition Tree. Template Tree is null");
		return;
	}

	UAvaTransitionTree* TransitionTree = Owner.GetTransitionTree();
	if (!TransitionTree)
	{
		UE_LOGF(LogAvaEditorTransition, Error, "Failed to Import Transition Tree. Target Tree is null");
		return;
	}

	UAvaTransitionTreeEditorData* TemplateEditorData = Cast<UAvaTransitionTreeEditorData>(InTemplateTree->EditorData);
	if (!TemplateEditorData)
	{
		UE_LOGF(LogAvaEditorTransition, Error, "Failed to Import Transition Tree. Editor Data is null in Template '%ls'.", *InTemplateTree->GetFullName());
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ImportTransitionTree", "Import Tree"));
	TransitionTree->Modify();

	if (UAvaTransitionTreeEditorData* NewEditorData = DuplicateObject<UAvaTransitionTreeEditorData>(TemplateEditorData, TransitionTree))
	{
		TransitionTree->EditorData = NewEditorData;
		Owner.UpdateEditorData();
		Owner.Refresh();
	}
}

#if UE_AVA_WITH_TRANSITION_DEBUG
void FAvaTransitionTreeActions::ToggleDebugger()
{
	TSharedRef<FAvaTransitionDebugger> Debugger = Owner.GetSharedData()->GetDebugger();
	if (Debugger->IsActive())
	{
		Debugger->Stop();
	}
	else
	{
		Debugger->Start();
	}
}

bool FAvaTransitionTreeActions::IsDebuggerActive() const
{
	TSharedRef<FAvaTransitionDebugger> Debugger = Owner.GetSharedData()->GetDebugger();
	return Debugger->IsActive();
}
#endif // UE_AVA_WITH_TRANSITION_DEBUG

#undef LOCTEXT_NAMESPACE
