// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debugger/AvaTransitionDebugDefinitions.h"
#include "AvaTransitionActions.h"

class UAvaTransitionTree;

class FAvaTransitionTreeActions : public FAvaTransitionActions
{
public:
	explicit FAvaTransitionTreeActions(FAvaTransitionEditorViewModel& InOwner)
		: FAvaTransitionActions(InOwner)
	{
	}

protected:
	//~ Begin FAvaTransitionActions
	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandList) override;
	//~ End FAvaTransitionActions

	bool CanShowExportTransitionTree() const;

	void ExportTransitionTree();

	void ImportTransitionTree();

	void ImportTransitionTree(UAvaTransitionTree* InTemplateTree);

#if UE_AVA_WITH_TRANSITION_DEBUG
	void ToggleDebugger();

	bool IsDebuggerActive() const;
#endif
};
