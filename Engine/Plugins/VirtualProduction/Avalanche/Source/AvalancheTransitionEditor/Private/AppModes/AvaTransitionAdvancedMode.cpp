// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionAdvancedMode.h"
#include "TabFactories/AvaTransitionCompilerResultsTabFactory.h"
#include "TabFactories/AvaTransitionSelectionDetailsTabFactory.h"
#include "TabFactories/AvaTransitionTreeDetailsTabFactory.h"
#include "TabFactories/AvaTransitionTreeTabFactory.h"

#define LOCTEXT_NAMESPACE "AvaTransitionAdvancedMode"

FAvaTransitionAdvancedMode::FAvaTransitionAdvancedMode(const TSharedRef<FAvaTransitionEditor>& InEditor)
	: FAvaTransitionAppMode(InEditor, EAvaTransitionEditorMode::Advanced)
{
	RegisterDefaultTabFactories();

	WorkspaceMenuCategory = FWorkspaceItem::NewGroup(LOCTEXT("WorkspaceMenuCategory", "Motion Design Transition Advanced"));

	TabLayout = FTabManager::NewLayout("AvaTransitionEditor_Advanced_Layout_V0_1")
	->AddArea
	(
		FTabManager::NewPrimaryArea()
		->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewSplitter()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.2f)
				->AddTab(FAvaTransitionTreeDetailsTabFactory::TabId, ETabState::OpenedTab)
			)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.6f)
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.8f)
					->AddTab(FAvaTransitionTreeTabFactory::TabId, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.2f)
					->AddTab(FAvaTransitionCompilerResultsTabFactory::TabId, ETabState::ClosedTab)
				)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.2f)
				->AddTab(FAvaTransitionSelectionDetailsTabFactory::TabId, ETabState::OpenedTab)
			)
		)
	);
}

#undef LOCTEXT_NAMESPACE
