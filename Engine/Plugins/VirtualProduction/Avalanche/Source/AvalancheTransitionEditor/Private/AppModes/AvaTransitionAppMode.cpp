// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionAppMode.h"
#include "AvaTransitionEditor.h"
#include "TabFactories/AvaTransitionCompilerResultsTabFactory.h"
#include "TabFactories/AvaTransitionSelectionDetailsTabFactory.h"
#include "TabFactories/AvaTransitionTreeDetailsTabFactory.h"
#include "TabFactories/AvaTransitionTreeTabFactory.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "WorkflowOrientedApp/SModeWidget.h"

#define LOCTEXT_NAMESPACE "AvaTransitionAppMode"

namespace UE::AvaTransitionEditor::Private
{
	static const TMap<EAvaTransitionEditorMode, FName> GModeNames
	{
		{ EAvaTransitionEditorMode::Default  , TEXT("Default") },
		{ EAvaTransitionEditorMode::Advanced , TEXT("Advanced") },
		{ EAvaTransitionEditorMode::Parameter, TEXT("Parameter") },
	};

	static const TMap<FName, FText> GLocalizedModeNames
	{
		{ TEXT("Default")  , LOCTEXT("Default", "Default") },
		{ TEXT("Advanced") , LOCTEXT("Advanced", "Advanced") },
		{ TEXT("Parameter"), LOCTEXT("Parameter", "Parameter") },
	};

	FText GetLocalizedMode(FName InModeName)
	{
		return GLocalizedModeNames[InModeName];
	}
}

FName FAvaTransitionAppMode::StaticGetModeName(EAvaTransitionEditorMode InEditorMode)
{
	return UE::AvaTransitionEditor::Private::GModeNames[InEditorMode];
}

FAvaTransitionAppMode::FAvaTransitionAppMode(const TSharedRef<FAvaTransitionEditor>& InEditor, EAvaTransitionEditorMode InEditorMode)
	: FApplicationMode(FAvaTransitionAppMode::StaticGetModeName(InEditorMode), &UE::AvaTransitionEditor::Private::GetLocalizedMode)
	, EditorWeak(InEditor)
	, EditorMode(InEditorMode)
{
}

void FAvaTransitionAppMode::AddToToolbar(const TSharedRef<FExtender>& InToolbarExtender)
{
	ToolbarExtender = InToolbarExtender;
	InToolbarExtender->AddToolBarExtension("Asset"
		, EExtensionHook::After
		, nullptr
		, FToolBarExtensionDelegate::CreateSP(this, &FAvaTransitionAppMode::ExtendToolbar));
}

void FAvaTransitionAppMode::RegisterDefaultTabFactories()
{
	TSharedRef<FAvaTransitionEditor> Editor = EditorWeak.Pin().ToSharedRef();

	TabFactories.RegisterFactory(MakeShared<FAvaTransitionTreeTabFactory>(Editor));
	TabFactories.RegisterFactory(MakeShared<FAvaTransitionCompilerResultsTabFactory>(Editor));
	TabFactories.RegisterFactory(MakeShared<FAvaTransitionTreeDetailsTabFactory>(Editor));
	TabFactories.RegisterFactory(MakeShared<FAvaTransitionSelectionDetailsTabFactory>(Editor, EditorMode));
}

void FAvaTransitionAppMode::ExtendToolbar(FToolBarBuilder& InToolbarBuilder)
{
	if (!EditorWeak.IsValid())
	{
		return;
	}

	TSharedRef<FAvaTransitionEditor> Editor = EditorWeak.Pin().ToSharedRef();

	FText LocalizedModeName = UE::AvaTransitionEditor::Private::GetLocalizedMode(ModeName);

	Editor->AddToolbarWidget(
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.Padding(1)
		[
			SNew(SBox)
			.MinDesiredHeight(24.f)
			[
				SNew(SModeWidget, LocalizedModeName, ModeName)
				.OnGetActiveMode(Editor, &FAvaTransitionEditor::GetCurrentMode)
				.OnSetActiveMode(FOnModeChangeRequested::CreateSP(Editor, &FAvaTransitionEditor::SetCurrentMode))
				.ToolTipText(FText::Format(LOCTEXT("ModeButtonTooltip", "Switch to {0} Mode"), LocalizedModeName))
				.IconImage(ModeIcon)
				.AddMetaData<FTagMetaData>(FTagMetaData(ModeName))
			]
		]
	);
}

void FAvaTransitionAppMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FAvaTransitionEditor> Editor = EditorWeak.Pin();
	check(Editor.IsValid());
	Editor->SetEditorMode(EditorMode);
	Editor->PushTabFactories(TabFactories);
}

#undef LOCTEXT_NAMESPACE
