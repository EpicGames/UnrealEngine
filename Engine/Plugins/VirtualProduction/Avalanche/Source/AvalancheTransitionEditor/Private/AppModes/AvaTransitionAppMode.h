// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionEditor.h"
#include "AvaTransitionEditorEnums.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

struct FSlateBrush;

class FAvaTransitionAppMode : public FApplicationMode
{
public:
	static FName StaticGetModeName(EAvaTransitionEditorMode InEditorMode);

	FAvaTransitionAppMode(const TSharedRef<FAvaTransitionEditor>& InEditor, EAvaTransitionEditorMode InEditorMode);

	void AddToToolbar(const TSharedRef<FExtender>& InToolbarExtender);

protected:
	void RegisterDefaultTabFactories();

	void ExtendToolbar(FToolBarBuilder& InToolbarBuilder);

	//~ Begin FApplicationMode
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override final;
	//~ End FApplicationMode

	TWeakPtr<FAvaTransitionEditor> EditorWeak;

	FWorkflowAllowedTabSet TabFactories;

	TAttribute<const FSlateBrush*> ModeIcon;

	EAvaTransitionEditorMode EditorMode;
};
