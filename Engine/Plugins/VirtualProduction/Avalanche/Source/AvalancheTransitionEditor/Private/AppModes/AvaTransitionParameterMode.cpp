// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionParameterMode.h"
#include "TabFactories/AvaTransitionParameterTabFactory.h"

#define LOCTEXT_NAMESPACE "AvaTransitionParameterMode"

FAvaTransitionParameterMode::FAvaTransitionParameterMode(const TSharedRef<FAvaTransitionEditor>& InEditor)
	: FAvaTransitionAppMode(InEditor, EAvaTransitionEditorMode::Parameter)
{
	WorkspaceMenuCategory = FWorkspaceItem::NewGroup(LOCTEXT("WorkspaceMenuCategory", "Motion Design Transition Parameter"));

	TabFactories.RegisterFactory(MakeShared<FAvaTransitionParameterTabFactory>(InEditor));

	TabLayout = FTabManager::NewLayout("AvaTransitionEditor_Parameter_Layout_V0_1")
	->AddArea
	(
		FTabManager::NewPrimaryArea()
		->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewStack()
			->SetSizeCoefficient(1.f)
			->AddTab(FAvaTransitionParameterTabFactory::TabId, ETabState::OpenedTab)
		)
	);
}

#undef LOCTEXT_NAMESPACE
