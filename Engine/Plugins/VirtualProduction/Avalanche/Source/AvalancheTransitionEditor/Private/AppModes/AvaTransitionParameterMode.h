// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionAppMode.h"

class FAvaTransitionParameterMode : public FAvaTransitionAppMode
{
public:
	FAvaTransitionParameterMode(const TSharedRef<FAvaTransitionEditor>& InEditor);
};
