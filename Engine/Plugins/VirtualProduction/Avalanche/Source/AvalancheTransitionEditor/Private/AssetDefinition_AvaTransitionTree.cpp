// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_AvaTransitionTree.h"
#include "AvaTransitionTree.h"
#include "AvaTransitionEditor.h"
#include "StateTreeEditorModule.h"
#include "Modules/ModuleManager.h"

namespace UE::AvaTransitionEditor::Private
{
	int32 GUseAvaTransitionEditor = 1;
	static FAutoConsoleVariableRef CVarUseAvaTransitionEditor(
		TEXT("AvaTransitionEditor.EnableNewEditor"),
		GUseAvaTransitionEditor,
		TEXT("Whether to use the new editor")
	);
}

FText UAssetDefinition_AvaTransitionTree::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetTypeActions", "FAssetTypeActions_AvaTransitionTree", "Motion Design Transition Tree");
}

TSoftClassPtr<UObject> UAssetDefinition_AvaTransitionTree::GetAssetClass() const
{
	return UAvaTransitionTree::StaticClass();
}

FLinearColor UAssetDefinition_AvaTransitionTree::GetAssetColor() const
{
	return FColor(201, 185, 29);
}

EAssetCommandResult UAssetDefinition_AvaTransitionTree::OpenAssets(const FAssetOpenArgs& InOpenArgs) const
{
	if (UE::AvaTransitionEditor::Private::GUseAvaTransitionEditor)
	{
		const FAvaTransitionEditorInitSettings InitSettings(InOpenArgs);
		for (UAvaTransitionTree* TransitionTree : InOpenArgs.LoadObjects<UAvaTransitionTree>())
		{
			TSharedRef<FAvaTransitionEditor> Editor = MakeShared<FAvaTransitionEditor>();
			Editor->InitEditor(TransitionTree, InitSettings);
		}
	}
	else
	{
		FStateTreeEditorModule& EditorModule = FModuleManager::LoadModuleChecked<FStateTreeEditorModule>("StateTreeEditorModule");
		for (UAvaTransitionTree* TransitionTree : InOpenArgs.LoadObjects<UAvaTransitionTree>())
		{
			EditorModule.CreateStateTreeEditor(EToolkitMode::Standalone, InOpenArgs.ToolkitHost, TransitionTree);
		}
	}

	return EAssetCommandResult::Handled;
}

FAssetOpenSupport UAssetDefinition_AvaTransitionTree::GetAssetOpenSupport(const FAssetOpenSupportArgs& InOpenSupportArgs) const
{
	constexpr bool bIsOpenMethodSupported = true;
	return FAssetOpenSupport(InOpenSupportArgs.OpenMethod, bIsOpenMethodSupported); 
}
