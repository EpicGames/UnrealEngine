// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "AssetDefinition_AvaTransitionTree.generated.h"

UCLASS()
class UAssetDefinition_AvaTransitionTree : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	//~ Begin UAssetDefinition
	virtual FText GetAssetDisplayName() const override;
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual FLinearColor GetAssetColor() const override;
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& InOpenArgs) const override;
	virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& InOpenSupportArgs) const override;
	//~ End UAssetDefinition
};
