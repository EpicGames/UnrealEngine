// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionCommands.h"
#include "Styling/AppStyle.h"
#include "Styling/AvaTransitionEditorStyle.h"

#define LOCTEXT_NAMESPACE "AvaTransitionEditorCommands"

FAvaTransitionEditorCommands::FAvaTransitionEditorCommands() 
	: TCommands(TEXT("AvaTransitionEditor")
	, LOCTEXT("MotionDesignTransitionEditor", "Motion Design Transition Logic Editor")
	, NAME_None
	, FAvaTransitionEditorStyle::Get().GetStyleSetName())
{
}

void FAvaTransitionEditorCommands::RegisterCommands()
{
	UI_COMMAND(AddComment
		, "Add Comment"
		, "Adds a comment to each state selected"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(RemoveComment
		, "Remove Comment"
		, "Removes the comments on the selected states"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(AddSiblingState
		, "Add Sibling State"
		, "Creates a new state next to the selected state, or at the topmost level if nothing is selected"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(AddChildState
		, "Add Child State"
		, "Creates a new state as child of the selected state, or at the topmost level if nothing is selected"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(EnableStates
		, "State Enabled"
		, "Enables Selected States"
		, EUserInterfaceActionType::Check
		, FInputChord());

	UI_COMMAND(ImportTransitionTree
		, "Import Tree"
		, "Imports a Transition Tree asset as an Instance"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(ExportTransitionTree
		, "Export Tree"
		, "Exports the existing Transition Tree as an asset. Only available if it isn't an asset already."
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(Compile
		, "Compile"
		, "Compile the current Transition Tree."
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SaveOnCompile_Never
		, "Never"
		, "Sets the save-on-compile option to 'Never', meaning that your Transition Tree will not be saved when they are compiled"
		, EUserInterfaceActionType::RadioButton
		, FInputChord());

	UI_COMMAND(SaveOnCompile_SuccessOnly
		, "On Success Only"
		, "Sets the save-on-compile option to 'Success Only', meaning that the Transition Tree will be saved whenever they are successfully compiled"
		, EUserInterfaceActionType::RadioButton
		, FInputChord());

	UI_COMMAND(SaveOnCompile_Always
		, "Always"
		, "Sets the save-on-compile option to 'Always', meaning that the Transition Tree will be saved whenever they are compiled (even if there were errors)"
		, EUserInterfaceActionType::RadioButton
		, FInputChord());

#if UE_AVA_WITH_TRANSITION_DEBUG
	UI_COMMAND(ToggleDebug
		, "Debug"
		, "Enables the debugger to analyze how each instance of the State Tree is run"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());
#endif
}

#undef LOCTEXT_NAMESPACE
