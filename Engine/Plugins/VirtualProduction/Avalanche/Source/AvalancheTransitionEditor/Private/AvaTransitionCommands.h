// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debugger/AvaTransitionDebugDefinitions.h"
#include "Framework/Commands/Commands.h"

class FAvaTransitionEditorCommands : public TCommands<FAvaTransitionEditorCommands>
{
public:
	FAvaTransitionEditorCommands();

	//~ Begin TCommands
	virtual void RegisterCommands() override;
	//~ End TCommands

	TSharedPtr<FUICommandInfo> AddComment;
	TSharedPtr<FUICommandInfo> RemoveComment;

	TSharedPtr<FUICommandInfo> AddSiblingState;
	TSharedPtr<FUICommandInfo> AddChildState;
	TSharedPtr<FUICommandInfo> EnableStates;

	TSharedPtr<FUICommandInfo> ImportTransitionTree;
	TSharedPtr<FUICommandInfo> ExportTransitionTree;

	TSharedPtr<FUICommandInfo> Compile;
	TSharedPtr<FUICommandInfo> SaveOnCompile_Never;
	TSharedPtr<FUICommandInfo> SaveOnCompile_SuccessOnly;
	TSharedPtr<FUICommandInfo> SaveOnCompile_Always;

#if UE_AVA_WITH_TRANSITION_DEBUG
	TSharedPtr<FUICommandInfo> ToggleDebug;
#endif
};
