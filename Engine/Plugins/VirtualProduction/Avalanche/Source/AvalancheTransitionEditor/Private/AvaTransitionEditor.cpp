// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionEditor.h"
#include "AppModes/AvaTransitionAdvancedMode.h"
#include "AppModes/AvaTransitionParameterMode.h"
#include "AppModes/AvaTransitionSimpleMode.h"
#include "AvaTransitionTree.h"
#include "AvaTransitionTreeEditorData.h"
#include "Menu/AvaTransitionToolbar.h"
#include "StateTreeDelegates.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "ViewModels/AvaTransitionViewModelSharedData.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AvaTransitionEditor"

const FName FAvaTransitionEditor::ToolkitName(TEXT("FAvaTransitionEditor"));

FAvaTransitionEditor::FAvaTransitionEditor()
	: ToolbarExtender(MakeShared<FExtender>())
{
}

void FAvaTransitionEditor::InitEditor(UAvaTransitionTree* InTransitionTree, const FAvaTransitionEditorInitSettings& InInitSettings)
{
	check(IsValid(InTransitionTree));

	bReadOnly = InInitSettings.OpenMethod == EAssetOpenMethod::View;

	EditorViewModel = MakeShared<FAvaTransitionEditorViewModel>(InTransitionTree, SharedThis(this));
	EditorViewModel->Initialize(/*Parent*/nullptr);
	EditorViewModel->BindCommands(ToolkitCommands);

	ExtendMenus();

	InitAssetEditor(InInitSettings.ToolkitMode
		, InInitSettings.ToolkitHost
		, TEXT("AvaTransitionEditorApp")
		, FTabManager::FLayout::NullLayout
		, /*bCreateDefaultStandaloneMenu*/true
		, /*bCreateDefaultToolbar*/true
		, InTransitionTree
		, /*bToolbarFocusable*/false
		, /*bUseSmallToolbarIcons*/false
		, InInitSettings.OpenMethod);

	RegisterApplicationModes();
}

void FAvaTransitionEditor::SetEditorMode(EAvaTransitionEditorMode InEditorMode)
{
	if (EditorViewModel.IsValid())
	{
		TSharedRef<FAvaTransitionViewModelSharedData> SharedData = EditorViewModel->GetSharedData();
		if (InEditorMode != SharedData->GetEditorMode())
		{
			SharedData->SetEditorMode(InEditorMode);
			EditorViewModel->Refresh();
		}
	}
}

void FAvaTransitionEditor::RegisterApplicationModes()
{
	TSharedRef<FAvaTransitionEditor> This = SharedThis(this);

	const TArray<TSharedRef<FAvaTransitionAppMode>, TFixedAllocator<3>> AppModes = 
	{
		MakeShared<FAvaTransitionParameterMode>(This),
		MakeShared<FAvaTransitionSimpleMode>(This),
		MakeShared<FAvaTransitionAdvancedMode>(This),
	};

	for (const TSharedRef<FAvaTransitionAppMode>& AppMode : AppModes)
	{
		AddApplicationMode(AppMode->GetModeName(), AppMode);
		AppMode->AddToToolbar(ToolbarExtender);
	}

	// set default mode to the first app mode
	SetCurrentMode(FAvaTransitionAppMode::StaticGetModeName(EAvaTransitionEditorMode::Default));
}

void FAvaTransitionEditor::ExtendMenus()
{
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return;
	}

	// Extend Toolbar
	if (UToolMenu* ToolbarMenu = ToolMenus->ExtendMenu(FAssetEditorToolkit::GetToolMenuToolbarName()))
	{
		ToolbarMenu->AddDynamicSection("PopulateToolbar", FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InToolMenu)->void
			{
				if (!InToolMenu)
				{
					return;
				}

				UAssetEditorToolkitMenuContext* ToolbarContext = InToolMenu->FindContext<UAssetEditorToolkitMenuContext>();
				if (!ToolbarContext)
				{
					return;
				}

				TSharedPtr<FAssetEditorToolkit> Toolkit = ToolbarContext->Toolkit.Pin();
				if (Toolkit.IsValid() && Toolkit->GetToolkitFName() == FAvaTransitionEditor::ToolkitName)
				{
					StaticCastSharedPtr<FAvaTransitionEditor>(Toolkit)->ExtendToolbar(InToolMenu);
				}
			})
		);
	}
}

void FAvaTransitionEditor::ExtendToolbar(UToolMenu* InToolMenu)
{
	if (EditorViewModel.IsValid())
	{
		EditorViewModel->GetToolbar()->ExtendEditorToolbar(InToolMenu);
	}
}

void FAvaTransitionEditor::SetupReadOnlyMenuProfiles(FReadOnlyAssetEditorCustomization& InReadOnlyCustomization)
{
	FWorkflowCentricApplication::SetupReadOnlyMenuProfiles(InReadOnlyCustomization);
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	TSharedRef<FAvaTransitionToolbar> ViewModelToolbar = EditorViewModel->GetToolbar();
	ViewModelToolbar->SetReadOnlyProfileName(FAssetEditorToolkit::GetToolMenuToolbarName(), GetReadOnlyMenuProfileName());
	ViewModelToolbar->SetupReadOnlyCustomization(InReadOnlyCustomization);
}

FName FAvaTransitionEditor::GetToolkitFName() const
{
	return FAvaTransitionEditor::ToolkitName;
}

FText FAvaTransitionEditor::GetBaseToolkitName() const
{
	return LOCTEXT("ToolkitName", "Motion Design Transition Editor");
}

FString FAvaTransitionEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix_LevelScript", "Script ").ToString();
}

FLinearColor FAvaTransitionEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor::White;
}

#undef LOCTEXT_NAMESPACE
