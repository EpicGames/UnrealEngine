// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"

class FAvaTransitionEditorViewModel;
class UAvaTransitionTree;
class UToolMenu;
enum class EAvaTransitionEditorMode : uint8;

struct FAvaTransitionEditorInitSettings
{
	explicit FAvaTransitionEditorInitSettings(const FAssetOpenArgs& InOpenArgs)
		: ToolkitHost(InOpenArgs.ToolkitHost)
		, ToolkitMode(InOpenArgs.GetToolkitMode())
		, OpenMethod(InOpenArgs.OpenMethod)
	{
	}

	TSharedPtr<IToolkitHost> ToolkitHost;

	EToolkitMode::Type ToolkitMode;

	EAssetOpenMethod OpenMethod;
};

class FAvaTransitionEditor : public FWorkflowCentricApplication
{
	static const FName ToolkitName;

public:
	FAvaTransitionEditor();

	void InitEditor(UAvaTransitionTree* InTransitionTree, const FAvaTransitionEditorInitSettings& InInitSettings);

	TSharedPtr<FAvaTransitionEditorViewModel> GetEditorViewModel() const
	{
		return EditorViewModel;
	}

	void SetEditorMode(EAvaTransitionEditorMode InEditorMode);

	bool IsReadOnly() const
	{
		return bReadOnly;
	}

protected:
	void RegisterApplicationModes();

	void ExtendMenus();

	void ExtendToolbar(UToolMenu* InToolMenu);

	//~ Begin FAssetEditorToolkit
	virtual void SetupReadOnlyMenuProfiles(FReadOnlyAssetEditorCustomization& InReadOnlyCustomization) override;
	//~ End FAssetEditorToolkit

	//~ Begin IToolkit
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	//~ End IToolkit

private:
	TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel;

	TSharedRef<FExtender> ToolbarExtender;

	bool bReadOnly = false;
};
