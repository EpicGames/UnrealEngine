// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

enum class EAvaTransitionEditorMode : uint8
{
	/** Simplified view of the State Tree. Core features for most Transition Logic Needs */
	Default,

	/** All the features in State Tree */
	Advanced,

	/** Only Parameters shown */
	Parameter,
};
