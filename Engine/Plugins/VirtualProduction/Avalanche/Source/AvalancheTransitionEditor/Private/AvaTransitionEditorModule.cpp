// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionEditorModule.h"
#include "AvaTransitionCommands.h"
#include "AvaTransitionEditorEnums.h"
#include "AvaTransitionEditorLog.h"
#include "AvaTransitionEditorUtils.h"
#include "AvaTransitionTree.h"
#include "AvaTransitionTreeEditorData.h"
#include "AvaTransitionTreeSchema.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "IAvaTransitionModule.h"
#include "StateTreeDelegates.h"
#include "Styling/AvaTransitionEditorStyle.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "Settings/AvaTransitionEditorSettings.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "AvaTransitionEditorModule"

DEFINE_LOG_CATEGORY(LogAvaEditorTransition)

IMPLEMENT_MODULE(FAvaTransitionEditorModule, AvalancheTransitionEditor)

void FAvaTransitionEditorModule::StartupModule()
{
	FAvaTransitionEditorCommands::Register();

	IAvaTransitionModule::Get().GetOnValidateTransitionTree().BindRaw(this, &FAvaTransitionEditorModule::ValidateStateTree);
}

void FAvaTransitionEditorModule::ShutdownModule()
{
	FAvaTransitionEditorCommands::Unregister();

	IAvaTransitionModule::Get().GetOnValidateTransitionTree().Unbind();
}

IAvaTransitionEditorModule::FOnBuildDefaultTransitionTree& FAvaTransitionEditorModule::GetOnBuildDefaultTransitionTree()
{
	return OnBuildDefaultTransitionTree;
}

void FAvaTransitionEditorModule::GenerateTransitionTreeOptionsMenu(UToolMenu* InMenu, IAvaTransitionBehavior* InTransitionBehavior)
{
	if (!InMenu || !InTransitionBehavior)
	{
		return;
	}

	UAvaTransitionTree* TransitionTree = InTransitionBehavior->GetTransitionTree();
	if (!TransitionTree)
	{
		return;
	}

	TWeakInterfacePtr<IAvaTransitionBehavior> TransitionBehaviorWeak = InTransitionBehavior;

	FToolMenuSection& GeneralSection = InMenu->FindOrAddSection(TEXT("GeneralSection"), LOCTEXT("GeneralSectionLabel", "General"));

	GeneralSection.AddMenuEntry(TEXT("TransitionTreeEnabled")
		, LOCTEXT("TransitionTreeEnabledLabel", "Enabled")
		, LOCTEXT("TransitionTreeEnabledTooltip", "Determines whether Transition Tree is Enabled")
		, TAttribute<FSlateIcon>()
		, FUIAction(FExecuteAction::CreateStatic(&UE::AvaTransitionEditor::ToggleTransitionTreeEnabled, TransitionBehaviorWeak)
			, FCanExecuteAction()
			, FIsActionChecked::CreateStatic(&UE::AvaTransitionEditor::IsTransitionTreeEnabled, TransitionBehaviorWeak))
		, EUserInterfaceActionType::ToggleButton);

	if (const TSharedPtr<SWidget> ModeSelector = UE::AvaTransitionEditor::CreateTransitionInstancingModeSelector(InTransitionBehavior))
	{
		GeneralSection.AddEntry(FToolMenuEntry::InitWidget(TEXT("TransitionModeSelector")
			, SNew(SBox)
				[
					ModeSelector.ToSharedRef()
				]
			, FText::GetEmpty()
			, /*bNoIndent*/true
			, /*bSearchable*/false
			, /*bNoPadding*/true));
	}

	if (TSharedPtr<SWidget> LayerPicker = UE::AvaTransitionEditor::CreateTransitionLayerPicker(InTransitionBehavior))
	{
		FToolMenuSection& LayerSection = InMenu->FindOrAddSection(TEXT("LayerSection"), LOCTEXT("LayerSectionLabel", "Layer"));

		LayerSection.AddEntry(FToolMenuEntry::InitWidget(TEXT("TransitionLayerPicker")
			, SNew(SBox)
				.HeightOverride(40.f)
				[
					LayerPicker.ToSharedRef()
				]
			, FText::GetEmpty()
			, /*bNoIndent*/true
			, /*bSearchable*/false
			, /*bNoPadding*/true));
	}
}

void FAvaTransitionEditorModule::ValidateStateTree(UAvaTransitionTree* InTransitionTree)
{
	// Return early if Editor Data is already valid
	if (!InTransitionTree || InTransitionTree->EditorData)
	{
		return;
	}

	UAvaTransitionTreeEditorData* EditorData;

	const UAvaTransitionEditorSettings* TransitionEditorSettings = GetDefault<UAvaTransitionEditorSettings>();
	check(TransitionEditorSettings);

	if (UAvaTransitionTreeEditorData* TemplateEditorData = TransitionEditorSettings->LoadDefaultTemplateEditorData())
	{
		EditorData = DuplicateObject<UAvaTransitionTreeEditorData>(TemplateEditorData, InTransitionTree);
		check(EditorData);
	}
	else
	{
		EditorData = NewObject<UAvaTransitionTreeEditorData>(InTransitionTree, NAME_None, RF_Transactional);
		check(EditorData);

		EditorData->Schema = NewObject<UAvaTransitionTreeSchema>(EditorData);

		if (OnBuildDefaultTransitionTree.IsBound())
		{
			OnBuildDefaultTransitionTree.Execute(*EditorData);
		}
		else
		{
			EditorData->AddRootState();	
		}
	}

	InTransitionTree->EditorData = EditorData;

	// Compile in Advanced Mode here so that no new nodes are generated from outside
	FAvaTransitionCompiler Compiler;
	Compiler.SetTransitionTree(InTransitionTree);
	Compiler.Compile(EAvaTransitionEditorMode::Advanced);
}

#undef LOCTEXT_NAMESPACE
