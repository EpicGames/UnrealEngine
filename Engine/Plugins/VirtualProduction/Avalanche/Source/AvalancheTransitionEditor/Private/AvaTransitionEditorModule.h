// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaTransitionEditorModule.h"

class UAvaTransitionTree;
class UStateTree;

class FAvaTransitionEditorModule : public IAvaTransitionEditorModule
{
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	//~ Begin IAvaTransitionEditorModule
	virtual FOnBuildDefaultTransitionTree& GetOnBuildDefaultTransitionTree() override;
	virtual void GenerateTransitionTreeOptionsMenu(UToolMenu* InMenu, IAvaTransitionBehavior* InTransitionBehavior) override;
	//~ End IAvaTransitionEditorModule

	void ValidateStateTree(UAvaTransitionTree* InTransitionTree);

	FOnBuildDefaultTransitionTree OnBuildDefaultTransitionTree;
};
