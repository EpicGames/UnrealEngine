// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionEditorUtils.h"
#include "AvaTransitionEditorEnums.h"
#include "AvaTransitionEditorLog.h"
#include "AvaTransitionTree.h"
#include "AvaTransitionTreeEditorData.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "Compiler/AvaTransitionCompiler.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "ISinglePropertyView.h"
#include "Menu/Widgets/AvaTransitionTreeStatus.h"
#include "Menu/Widgets/SAvaTransitionTreeInstancingModeSelector.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "Serialization/ArchiveObjectCrc32.h"
#include "StateTreeTaskBase.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AvaTransitionEditorUtils"

namespace UE::AvaTransitionEditor
{
TSharedPtr<SWidget> CreateTransitionStatusWidget(IAvaTransitionBehavior* InTransitionBehavior)
{
	if (!InTransitionBehavior)
	{
		return nullptr;
	}
	return SNew(SAvaTransitionTreeStatus, InTransitionBehavior);
}

TSharedPtr<SWidget> CreateTransitionLayerPicker(IAvaTransitionBehavior* InTransitionBehavior)
{
	if (!InTransitionBehavior)
	{
		return nullptr;
	}

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FSinglePropertyParams SinglePropertyParams;
	SinglePropertyParams.NamePlacement = EPropertyNamePlacement::Hidden;

	TSharedPtr<ISinglePropertyView> PropertyView = PropertyEditorModule.CreateSingleProperty(&InTransitionBehavior->AsUObject()
		, TEXT("TransitionLayer")
		, SinglePropertyParams);

	if (PropertyView.IsValid())
	{
		return SNew(SBox)
			.Padding(0.f, -6.f)
			.HeightOverride(28.f)
			.MaxDesiredWidth(200.f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				// Scale down the widget so as to fit the Toolbar without making the Toolbar Bigger
				SNew(SScaleBox)
				.Stretch(EStretch::ScaleToFitY)
				.StretchDirection(EStretchDirection::DownOnly)
				.HAlign(HAlign_Fill)
				[
					PropertyView.ToSharedRef()
				]
			];
	}
	return nullptr;
}

TSharedPtr<SWidget> CreateTransitionInstancingModeSelector(IAvaTransitionBehavior* InTransitionBehavior)
{
	const FMargin PropertyPadding(2.0f, 0.0f, 2.0f, 0.0f);
	
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(PropertyPadding)
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("TransitionInstancingModeLabel", "Mode"))
		]
		+ SHorizontalBox::Slot()
		.Padding(PropertyPadding)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SAvaTransitionTreeInstancingModeSelector, InTransitionBehavior)
		];
}

void ValidateTree(UAvaTransitionTree& InTransitionTree)
{
	UStateTreeEditorData* EditorData = Cast<UStateTreeEditorData>(InTransitionTree.EditorData);
	if (!EditorData || !EditorData->Schema)
	{
		return;
	}

	const FString TreeDebugName = InTransitionTree.GetName();

	EditorData->ReparentStates();

	// Clear evaluators if not allowed.
	if (!EditorData->Evaluators.IsEmpty() && !EditorData->Schema->AllowEvaluators())
	{
		UE_LOGF(LogAvaEditorTransition, Warning
			, "%ls: Resetting Evaluators due to current schema restrictions."
			, *TreeDebugName);

		EditorData->Evaluators.Reset();
	}

	// Apply Schema Rules to each State 
	EditorData->VisitHierarchy([&TreeDebugName, EditorData](UStateTreeState& State, UStateTreeState*)
	{
		// Clear enter conditions if not allowed.
		if (!State.EnterConditions.IsEmpty() && !EditorData->Schema->AllowEnterConditions())
		{
			UE_LOGF(LogAvaEditorTransition, Warning
				, "%ls: Resetting Enter Conditions in state %ls due to current schema restrictions."
				, *TreeDebugName
				, *State.GetName());

			State.EnterConditions.Reset();
		}

		// Keep single and many tasks based on what is allowed.
		if (!EditorData->Schema->AllowMultipleTasks())
		{
			if (!State.Tasks.IsEmpty())
			{
				State.Tasks.Reset();
				UE_LOGF(LogAvaEditorTransition, Warning
					, "%ls: Resetting Tasks in state %ls due to current schema restrictions."
					, *TreeDebugName
					, *State.GetName());
			}

			// Task name is the same as state name.
			if (FStateTreeTaskBase* Task = State.SingleTask.Node.GetMutablePtr<FStateTreeTaskBase>())
			{
				Task->Name = State.Name;
			}
		}
		else
		{
			if (State.SingleTask.Node.IsValid())
			{
				State.SingleTask.Reset();
				UE_LOGF(LogAvaEditorTransition, Warning
					, "%ls: Resetting Single Task in state %ls due to current schema restrictions."
					, *TreeDebugName
					, *State.GetName());
			}
		}

		return EStateTreeVisitor::Continue;
	});

	// Remove unused Bindings
	{
		TMap<FGuid, const FPropertyBindingDataView> AllStructValues;
		EditorData->GetAllStructValues(AllStructValues);
		EditorData->GetPropertyEditorBindings()->RemoveInvalidBindings(AllStructValues);
	}

	// Validate Linked States
	{
		// Make sure all state links are valid and update the names if needed.
		// Create ID to state name map.
		TMap<FGuid, FName> IdToName;

		EditorData->VisitHierarchy([&IdToName](const UStateTreeState& State, UStateTreeState* /*ParentState*/)
		{
			IdToName.Add(State.ID, State.Name);
			return EStateTreeVisitor::Continue;
		});

		static auto FixChangedStateLinkName = [](FStateTreeStateLink& StateLink, const TMap<FGuid, FName>& IDToName)
		{
			if (StateLink.ID.IsValid())
			{
				const FName* Name = IDToName.Find(StateLink.ID);
				if (Name == nullptr)
				{
					// Missing link, we'll show these in the UI
					return false;
				}
				if (StateLink.Name != *Name)
				{
					// Name changed, fix!
					StateLink.Name = *Name;
					return true;
				}
			}
			return false;
		};

		// Fix changed names.
		EditorData->VisitHierarchy([&IdToName](UStateTreeState& State, UStateTreeState* /*ParentState*/)
		{
			if (State.Type == EStateTreeStateType::Linked)
			{
				FixChangedStateLinkName(State.LinkedSubtree, IdToName);
			}
					
			for (FStateTreeTransition& Transition : State.Transitions)
			{
				FixChangedStateLinkName(Transition.State, IdToName);
			}

			return EStateTreeVisitor::Continue;
		});
	}

	// Update Linked State Parameters
	EditorData->VisitHierarchy([](UStateTreeState& State, UStateTreeState* /*ParentState*/)
	{
		if (State.Type == EStateTreeStateType::Linked)
		{
			State.UpdateParametersFromLinkedSubtree();
		}
		return EStateTreeVisitor::Continue;
	});
}

uint32 CalculateTreeHash(UAvaTransitionTree& InTransitionTree)
{
	if (!InTransitionTree.EditorData)
	{
		return 0;
	}

	static const FName MD_ExcludeFromHash(TEXT("ExcludeFromHash"));

	class : public FArchiveObjectCrc32
	{
		virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
		{
			return !InProperty
				|| FArchiveObjectCrc32::ShouldSkipProperty(InProperty)
				|| InProperty->HasAllPropertyFlags(CPF_Transient)
				|| InProperty->HasMetaData(MD_ExcludeFromHash);
		}
	} Archive;

	return Archive.Crc32(InTransitionTree.EditorData, 0);
}

bool PickTransitionTreeAsset(const FText& InDialogTitle, UAvaTransitionTree*& OutTransitionTree)
{
	FOpenAssetDialogConfig SelectAssetConfig;
	SelectAssetConfig.DialogTitleOverride = InDialogTitle;
	SelectAssetConfig.bAllowMultipleSelection = false;
	SelectAssetConfig.DefaultPath = TEXT("/Game");
	SelectAssetConfig.AssetClassNames.Add(UAvaTransitionTree::StaticClass()->GetClassPathName());

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	TArray<FAssetData> AssetData = ContentBrowserModule.Get().CreateModalOpenAssetDialog(SelectAssetConfig);
	if (AssetData.IsEmpty())
	{
		// return false as user selected no assets
		return false;
	}

	OutTransitionTree = Cast<UAvaTransitionTree>(AssetData[0].GetAsset());
	return true;
}

void ToggleTransitionTreeEnabled(TWeakInterfacePtr<IAvaTransitionBehavior> InTransitionBehaviorWeak)
{
	if (IAvaTransitionBehavior* const TransitionBehavior = InTransitionBehaviorWeak.Get())
	{
		FScopedTransaction Transaction(LOCTEXT("ToggleTransitionTreeEnabled", "Toggle Transition Tree Enabled"));
		TransitionBehavior->AsUObject().Modify();
		TransitionBehavior->SetEnabled(!TransitionBehavior->IsEnabled());
	}
}

bool IsTransitionTreeEnabled(TWeakInterfacePtr<IAvaTransitionBehavior> InTransitionBehaviorWeak)
{
	IAvaTransitionBehavior* const TransitionBehavior = InTransitionBehaviorWeak.Get();
	return TransitionBehavior && TransitionBehavior->IsEnabled();
}

}

#undef LOCTEXT_NAMESPACE
