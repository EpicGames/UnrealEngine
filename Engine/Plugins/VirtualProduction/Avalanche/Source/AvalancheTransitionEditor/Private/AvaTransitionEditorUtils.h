// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behavior/IAvaTransitionBehavior.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/WeakObjectPtr.h"

class SWidget;
class UAvaTransitionTree;
class UAvaTransitionTreeEditorData;

namespace UE::AvaTransitionEditor
{
	TSharedPtr<SWidget> CreateTransitionStatusWidget(IAvaTransitionBehavior* InTransitionBehavior);

	TSharedPtr<SWidget> CreateTransitionLayerPicker(IAvaTransitionBehavior* InTransitionBehavior);

	TSharedPtr<SWidget> CreateTransitionInstancingModeSelector(IAvaTransitionBehavior* InTransitionBehavior);
	
	void ValidateTree(UAvaTransitionTree& InTransitionTree);

	uint32 CalculateTreeHash(UAvaTransitionTree& InTransitionTree);

	bool PickTransitionTreeAsset(const FText& InDialogTitle, UAvaTransitionTree*& OutTransitionTree);

	void ToggleTransitionTreeEnabled(TWeakInterfacePtr<IAvaTransitionBehavior> InTransitionBehaviorWeak);

	bool IsTransitionTreeEnabled(TWeakInterfacePtr<IAvaTransitionBehavior> InTransitionBehaviorWeak);
}
