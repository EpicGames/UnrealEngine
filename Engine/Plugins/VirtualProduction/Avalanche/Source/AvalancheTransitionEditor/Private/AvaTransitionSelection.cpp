// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionSelection.h"
#include "AvaTypeSharedPointer.h"
#include "Extensions/IAvaTransitionSelectableExtension.h"
#include "ViewModels/AvaTransitionViewModel.h"

void FAvaTransitionSelection::SetSelectedItems(TConstArrayView<TSharedPtr<FAvaTransitionViewModel>> InSelectedItems)
{
	// Default Items to Deselect to all Previously Selected Items
	TArray<TSharedRef<FAvaTransitionViewModel>> ItemsToDeselect = MoveTemp(SelectedViewModels);
	TArray<TSharedRef<FAvaTransitionViewModel>> ItemsToSelect;

	SelectedViewModels.Empty(InSelectedItems.Num());

	// assume that all items are newly selected
	ItemsToSelect.Reserve(InSelectedItems.Num());

	for (TSharedPtr<FAvaTransitionViewModel> Item : InSelectedItems)
	{
		if (Item->IsA<IAvaTransitionSelectableExtension>())
		{
			TSharedRef<FAvaTransitionViewModel> ItemRef = Item.ToSharedRef();

			SelectedViewModels.Add(ItemRef);

			// Try removing the item from the Previously Selected Items list
			// If remove count > 0, it was an item that is still selected after this selection change
			// If remove count == 0, it was an item that wasn't previously selected --- so it's a newly selected item
			if (ItemsToDeselect.Remove(ItemRef) == 0)
			{
				ItemsToSelect.Add(ItemRef);
			}
		}
	}

	auto SetSelection = [](const TSharedRef<FAvaTransitionViewModel>& InItem, bool bInSelected)
	{
		IAvaTransitionSelectableExtension* Selectable = InItem->CastTo<IAvaTransitionSelectableExtension>();
		check(Selectable);
		Selectable->SetSelected(bInSelected);
	};

	for (const TSharedRef<FAvaTransitionViewModel>& Item : ItemsToDeselect)
	{
		SetSelection(Item, false);
	}

	for (const TSharedRef<FAvaTransitionViewModel>& Item : ItemsToSelect)
	{
		SetSelection(Item, true);
	}

	OnSelectionChangedDelegate.Broadcast(SelectedViewModels);
}

void FAvaTransitionSelection::ClearSelectedItems()
{
	SetSelectedItems({});
}
