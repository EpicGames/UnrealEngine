// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"

class FAvaTransitionEditorViewModel;
class FAvaTransitionViewModel;
class IAvaTransitionSelectableExtension;

class FAvaTransitionSelection : public TSharedFromThis<FAvaTransitionSelection>
{
public:
	void SetSelectedItems(TConstArrayView<TSharedPtr<FAvaTransitionViewModel>> InSelectedItems);

	void ClearSelectedItems();

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSelectionChanged, TConstArrayView<TSharedRef<FAvaTransitionViewModel>>);
	FOnSelectionChanged& OnSelectionChanged()
	{
		return OnSelectionChangedDelegate;
	}

	TConstArrayView<TSharedRef<FAvaTransitionViewModel>> GetSelectedItems() const
	{
		return SelectedViewModels;
	}

private:
	TArray<TSharedRef<FAvaTransitionViewModel>> SelectedViewModels;

	FOnSelectionChanged OnSelectionChangedDelegate;
};
