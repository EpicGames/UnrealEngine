// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionTreeEditorData.h"
#include "AvaTransitionDefaultLayerTags.h"

UAvaTransitionTreeEditorData::UAvaTransitionTreeEditorData()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TransitionLayer_DEPRECATED = UE::AvaTransition::FDefaultTags::Get().DefaultLayer.MakeTagHandle();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FStateTreeEditorColor DefaultColor;
	DefaultColor.ColorRef = FStateTreeEditorColorRef(UE::AvaTransitionEditor::ColorId_Default);
	DefaultColor.Color = FLinearColor(0, 0.15f, 0.2f);
	DefaultColor.DisplayName = TEXT("Default Color");

	FStateTreeEditorColor InColor;
	InColor.ColorRef = FStateTreeEditorColorRef(UE::AvaTransitionEditor::ColorId_In);
	InColor.Color = FLinearColor(0, 0.2f, 0);
	InColor.DisplayName = TEXT("In Color");

	FStateTreeEditorColor OutColor;
	OutColor.ColorRef = FStateTreeEditorColorRef(UE::AvaTransitionEditor::ColorId_Out);
	OutColor.Color = FLinearColor(0.2f, 0, 0.15f);
	OutColor.DisplayName = TEXT("Out Color");

	Colors.Empty(3);
	Colors.Add(MoveTemp(DefaultColor));
	Colors.Add(MoveTemp(InColor));
	Colors.Add(MoveTemp(OutColor));
}

UStateTreeState& UAvaTransitionTreeEditorData::CreateState(const UStateTreeState& InSiblingState, bool bInAfter)
{
	UObject* Outer = this;
	if (InSiblingState.Parent)
	{
		Outer = InSiblingState.Parent;
	}

	check(Outer);
	UStateTreeState* State = NewObject<UStateTreeState>(Outer, NAME_None, RF_Transactional);
	check(State);

	State->Parent = InSiblingState.Parent;

	TArray<TObjectPtr<UStateTreeState>>& Children = State->Parent
		? State->Parent->Children
		: SubTrees;

	int32 ChildIndex = Children.IndexOfByKey(&InSiblingState);
	ChildIndex = FMath::Clamp(ChildIndex, 0, Children.Num() - 1);

	if (bInAfter)
	{
		++ChildIndex;
	}

	Children.Insert(State, ChildIndex);
	return *State;
}

bool UAvaTransitionTreeEditorData::Compare(const UAvaTransitionTreeEditorData* InEditorData) const
{
	// Root Parameters guids are unique per tree.
	// When duplicating trees, even though the parameter itself isn't duplicate transient, it will assign a new guid in UStateTreeEditorData::DuplicateIDs
	return InEditorData && (this == InEditorData || GetRootParametersGuid() == InEditorData->GetRootParametersGuid());
}

FAvaTagHandle UAvaTransitionTreeEditorData::GetTransitionLayer() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return TransitionLayer_DEPRECATED;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UAvaTransitionTreeEditorData::SetTransitionLayer(const FAvaTagHandle& InTransitionLayer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TransitionLayer_DEPRECATED = InTransitionLayer;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FName UAvaTransitionTreeEditorData::GetTransitionLayerPropertyName()
{
	return NAME_None;
}
