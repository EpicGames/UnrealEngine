// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionTreeFactory.h"
#include "AssetToolsModule.h"
#include "AvaTransitionTree.h"
#include "IAssetTools.h"
#include "IAvaTransitionModule.h"

UAvaTransitionTreeFactory::UAvaTransitionTreeFactory()
{
	SupportedClass = UAvaTransitionTree::StaticClass();
	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

uint32 UAvaTransitionTreeFactory::GetMenuCategories() const
{
	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	return AssetTools.FindAdvancedAssetCategory(TEXT("MotionDesignCategory"));
}

UObject* UAvaTransitionTreeFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn)
{
	if (ensure(SupportedClass == InClass))
	{
		UAvaTransitionTree* TransitionTree = NewObject<UAvaTransitionTree>(InParent, InName, InFlags);
		if (TransitionTree)
		{
			IAvaTransitionModule::Get().GetOnValidateTransitionTree().ExecuteIfBound(TransitionTree);
		}
		return TransitionTree;
	}
	return nullptr;
}
