// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionCompiler.h"
#include "AvaTransitionCommands.h"
#include "AvaTransitionEditor.h"
#include "AvaTransitionEditorUtils.h"
#include "AvaTransitionSelection.h"
#include "AvaTransitionTree.h"
#include "AvaTransitionTreeEditorData.h"
#include "Extensions/IAvaTransitionObjectExtension.h"
#include "FileHelpers.h"
#include "IAvaTransitionEditorModule.h"
#include "IMessageLogListing.h"
#include "MessageLogModule.h"
#include "Misc/UObjectToken.h"
#include "StateTreeCompiler.h"
#include "StateTreeCompilerLog.h"
#include "StateTreeDelegates.h"
#include "StateTreeEditorSettings.h"
#include "StateTreeState.h"
#include "ToolMenu.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "ViewModels/AvaTransitionViewModelSharedData.h"
#include "ViewModels/Registry/AvaTransitionViewModelRegistryCollection.h"

#define LOCTEXT_NAMESPACE "AvaTransitionCompiler"

namespace UE::AvaTransitionEditor::Private
{
	TSharedRef<IMessageLogListing> CreateCompilerResultsListing()
	{
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");

		// Show Pages so that user is never allowed to clear log messages
		FMessageLogInitializationOptions LogOptions;
		LogOptions.bShowPages   = false;
		LogOptions.bShowFilters = false;
		LogOptions.bAllowClear  = false;
		LogOptions.MaxPageCount = 1;

		return MessageLogModule.CreateLogListing("AvaTransitionTreeCompiler", LogOptions);
	}
}

FAvaTransitionCompiler::FAvaTransitionCompiler()
	: CompilerResultsListing(UE::AvaTransitionEditor::Private::CreateCompilerResultsListing())
{
}

void FAvaTransitionCompiler::SetTransitionTree(UAvaTransitionTree* InTransitionTree)
{
	TransitionTreeWeak = InTransitionTree;
}

IMessageLogListing& FAvaTransitionCompiler::GetCompilerResultsListing()
{
	return CompilerResultsListing.Get();
}

FSimpleDelegate& FAvaTransitionCompiler::GetOnCompileFailed()
{
	return OnCompileFail;
}

bool FAvaTransitionCompiler::Compile(EAvaTransitionEditorMode InCompileMode)
{
	UAvaTransitionTree* TransitionTree = TransitionTreeWeak.Get();
	if (!TransitionTree)
	{
		return false;
	}

	if (UAvaTransitionTreeEditorData* EditorData = Cast<UAvaTransitionTreeEditorData>(TransitionTree->EditorData))
	{
		// Remove all invalid Sub Trees, if any
		EditorData->SubTrees.RemoveAll(
			[](const UStateTreeState* InState)
			{
				return !IsValid(InState);
			});
	}

	UpdateTree();

	FStateTreeCompilerLog Log;
	FStateTreeCompiler Compiler(Log);

	bLastCompileSucceeded = Compiler.Compile(*TransitionTree);

	CompilerResultsListing->ClearMessages();
	Log.AppendToLog(&CompilerResultsListing.Get());

	if (bLastCompileSucceeded)
	{
		// Success
		TransitionTree->LastCompiledEditorDataHash = EditorDataHash;
		UE::StateTree::Delegates::OnPostCompile.Broadcast(*TransitionTree);
	}
	else
	{
		// Make sure not to leave stale data on failed compile.
		TransitionTree->ResetCompiled();
		OnCompileFail.ExecuteIfBound();
	}

	const UStateTreeEditorSettings* Settings = GetMutableDefault<UStateTreeEditorSettings>();

	const bool bShouldSaveOnCompile = Settings->SaveOnCompile == EStateTreeSaveOnCompile::Always
		|| (Settings->SaveOnCompile == EStateTreeSaveOnCompile::SuccessOnly && bLastCompileSucceeded);

	if (bShouldSaveOnCompile)
	{
		FEditorFileUtils::PromptForCheckoutAndSave({ TransitionTree->GetPackage() }
			, /*bCheckDirty*/true
			, /*bPromptToSave*/false);
	}

	return true;
}

void FAvaTransitionCompiler::UpdateTree()
{
	UAvaTransitionTree* TransitionTree = TransitionTreeWeak.Get();
	if (!TransitionTree)
	{
		return;
	}

	UE::AvaTransitionEditor::ValidateTree(*TransitionTree);

	EditorDataHash = UE::AvaTransitionEditor::CalculateTreeHash(*TransitionTree);
}

FSlateIcon FAvaTransitionCompiler::GetCompileStatusIcon() const
{
	static const FName CompileStatusBackground("Blueprint.CompileStatus.Background");
	static const FName CompileStatusUnknown("Blueprint.CompileStatus.Overlay.Unknown");
	static const FName CompileStatusError("Blueprint.CompileStatus.Overlay.Error");
	static const FName CompileStatusGood("Blueprint.CompileStatus.Overlay.Good");

	UAvaTransitionTree* TransitionTree = TransitionTreeWeak.Get();
	if (!TransitionTree)
	{
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusUnknown);
	}

	if (TransitionTree->IsEditorDataDirty())
	{
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusUnknown);
	}
	else if (!TransitionTree->IsReadyToRun())
	{
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusError);
	}

	return FSlateIcon(FAppStyle::GetAppStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusGood);
}

TSharedRef<SWidget> FAvaTransitionCompiler::CreateCompilerResultsWidget() const
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	return MessageLogModule.CreateLogListingWidget(CompilerResultsListing);
}

void FAvaTransitionCompiler::SetSaveOnCompile(EStateTreeSaveOnCompile InSaveOnCompileType)
{
	UStateTreeEditorSettings* Settings = GetMutableDefault<UStateTreeEditorSettings>();
	check(Settings);
	Settings->SaveOnCompile = InSaveOnCompileType;
	Settings->SaveConfig();
}

bool FAvaTransitionCompiler::HasSaveOnCompile(EStateTreeSaveOnCompile InSaveOnCompileType)
{
	const UStateTreeEditorSettings* Settings = GetDefault<UStateTreeEditorSettings>();
	check(Settings);
	return Settings->SaveOnCompile == InSaveOnCompileType;
}

void FAvaTransitionCompiler::GenerateCompileOptionsMenu(UToolMenu* InMenu)
{
	auto MakeSaveOnCompileMenu = [](UToolMenu* InSubMenu)
	{
		FToolMenuSection& Section = InSubMenu->AddSection("CompileOptions");

		const FAvaTransitionEditorCommands& Commands = FAvaTransitionEditorCommands::Get();
		Section.AddMenuEntry(Commands.SaveOnCompile_Never);
		Section.AddMenuEntry(Commands.SaveOnCompile_SuccessOnly);
		Section.AddMenuEntry(Commands.SaveOnCompile_Always);
	};

	FToolMenuSection& Section = InMenu->AddSection(TEXT("CompileOptions"));
	Section.AddSubMenu("SaveOnCompile"
		, LOCTEXT("SaveOnCompileSubMenu", "Save on Compile")
		, LOCTEXT("SaveOnCompileSubMenu_ToolTip", "Determines how the StateTree is saved whenever you compile it.")
		, FNewToolMenuDelegate::CreateLambda(MakeSaveOnCompileMenu));
}

#undef LOCTEXT_NAMESPACE
