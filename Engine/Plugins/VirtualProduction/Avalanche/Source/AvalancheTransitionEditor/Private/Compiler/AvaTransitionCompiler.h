// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class IMessageLogListing;
class SWidget;
class UAvaTransitionTree;
class UToolMenu;
enum class EAvaTransitionEditorMode : uint8;
enum class EStateTreeSaveOnCompile : uint8;
struct FSlateIcon;

class FAvaTransitionCompiler
{
public:
	FAvaTransitionCompiler();

	void SetTransitionTree(UAvaTransitionTree* InTransitionTree);

	IMessageLogListing& GetCompilerResultsListing();

	FSimpleDelegate& GetOnCompileFailed();

	bool Compile(EAvaTransitionEditorMode InCompileMode);

	void UpdateTree();

	FSlateIcon GetCompileStatusIcon() const;

	TSharedRef<SWidget> CreateCompilerResultsWidget() const;

	static void SetSaveOnCompile(EStateTreeSaveOnCompile InSaveOnCompileType);

	static bool HasSaveOnCompile(EStateTreeSaveOnCompile InSaveOnCompileType);

	static void GenerateCompileOptionsMenu(UToolMenu* InMenu);

private:
	TWeakObjectPtr<UAvaTransitionTree> TransitionTreeWeak;

	TSharedRef<IMessageLogListing> CompilerResultsListing;

	FSimpleDelegate OnCompileFail;

	uint32 EditorDataHash = 0;

	bool bLastCompileSucceeded = true;
};
