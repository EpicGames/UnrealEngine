// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaStateTreeStateCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "StateTreeState.h"
#include "Widgets/SAvaTransitionTransitionType.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AvaStateTreeStateCustomization"

TSharedRef<IDetailCustomization> FAvaStateTreeStateCustomization::MakeInstance()
{
	return MakeShared<FAvaStateTreeStateCustomization>();
}

bool FAvaStateTreeStateCustomization::IsPropertyVisible(const FPropertyAndParent& InPropertyAndParent)
{
	const FName PropertyName = InPropertyAndParent.Property.GetFName();

	return PropertyName != GET_MEMBER_NAME_CHECKED(UStateTreeState, bCheckPrerequisitesWhenActivatingChildDirectly)
		&& PropertyName != GET_MEMBER_NAME_CHECKED(UStateTreeState, RequiredEventToEnter);
}

void FAvaStateTreeStateCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	if (TSharedPtr<IDetailCustomization> Customization = GetDefaultCustomization())
	{
		Customization->CustomizeDetails(InDetailBuilder);
	}

	InDetailBuilder.HideCategory(TEXT("State"));
	InDetailBuilder.HideCategory(TEXT("Transitions"));
	InDetailBuilder.HideCategory(TEXT("Parameters"));
	InDetailBuilder.HideCategory(TEXT("Selection Utility"));

	InDetailBuilder.EditCategory(TEXT("Enter Conditions")).InitiallyCollapsed(false);
	InDetailBuilder.EditCategory(TEXT("Task")).InitiallyCollapsed(false);
	InDetailBuilder.EditCategory(TEXT("Tasks")).InitiallyCollapsed(false);

	IDetailCategoryBuilder& GeneralCategory = InDetailBuilder.EditCategory(TEXT("General"), LOCTEXT("GeneralCategory", "General"), ECategoryPriority::Important);
	GeneralCategory.SetSortOrder(0);
	GeneralCategory.AddProperty(InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, ColorRef)));

	IDetailCategoryBuilder& TransitionCategory = InDetailBuilder.EditCategory(TEXT("Transition"), LOCTEXT("TransitionCategory", "Transition"), ECategoryPriority::Important);
	TransitionCategory.SetSortOrder(10);

	TSharedPtr<IPropertyHandleArray> TransitionArrayHandle;
	if (TSharedPtr<IPropertyHandle> TransitionsHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UStateTreeState, Transitions)))
	{
		TransitionArrayHandle = TransitionsHandle->AsArray();
	}

	const FText TransitionType = LOCTEXT("TransitionType", "Transition Type");
	TransitionCategory.AddCustomRow(TransitionType)
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(TransitionType)
		]
		.ValueContent()
		[
			SNew(SAvaTransitionTransitionType, TransitionArrayHandle)
		];
}

TSharedPtr<IDetailCustomization> FAvaStateTreeStateCustomization::GetDefaultCustomization() const
{
	const FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	const FCustomDetailLayoutNameMap& CustomizationMap = PropertyModule.GetClassNameToDetailLayoutNameMap();
	const FDetailLayoutCallback* DefaultLayoutCallback = CustomizationMap.Find(UStateTreeState::StaticClass()->GetFName());

	if (DefaultLayoutCallback && DefaultLayoutCallback->DetailLayoutDelegate.IsBound())
	{
		return DefaultLayoutCallback->DetailLayoutDelegate.Execute();
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
