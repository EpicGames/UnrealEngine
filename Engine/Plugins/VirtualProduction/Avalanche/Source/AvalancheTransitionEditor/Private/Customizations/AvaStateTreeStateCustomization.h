// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

struct FPropertyAndParent;

/** Customization that re-uses the module-registered Customization for UStateTreeState, and tweaks a few settings for it */
class FAvaStateTreeStateCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	static bool IsPropertyVisible(const FPropertyAndParent& InPropertyAndParent);

protected:
	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization

private:
	TSharedPtr<IDetailCustomization> GetDefaultCustomization() const;
};
