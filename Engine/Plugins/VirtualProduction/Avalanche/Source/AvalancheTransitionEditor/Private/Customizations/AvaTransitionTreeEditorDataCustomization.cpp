// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionTreeEditorDataCustomization.h"
#include "AvaTransitionEditorEnums.h"
#include "AvaTransitionTreeEditorData.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Modules/ModuleManager.h"
#include "PropertyBagDetails.h"
#include "PropertyEditorModule.h"
#include "ViewModels/AvaTransitionViewModelSharedData.h"

TSharedRef<IDetailCustomization> FAvaTransitionTreeEditorDataCustomization::MakeInstance(TWeakPtr<FAvaTransitionViewModelSharedData> InSharedDataWeak)
{
	return MakeShared<FAvaTransitionTreeEditorDataCustomization>(InSharedDataWeak);
}

FAvaTransitionTreeEditorDataCustomization::FAvaTransitionTreeEditorDataCustomization(const TWeakPtr<FAvaTransitionViewModelSharedData>& InSharedDataWeak)
	: SharedDataWeak(InSharedDataWeak)
{
}

void FAvaTransitionTreeEditorDataCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	const EAvaTransitionEditorMode EditorMode = GetEditorMode();

	// Parameters don't need customization as its only purpose is to show the Parameters in a fixed layout way (FAvaTransitionTreeEditorDataCustomization::CustomizeParameters)
	if (EditorMode != EAvaTransitionEditorMode::Parameter)
	{
		if (TSharedPtr<IDetailCustomization> Customization = GetDefaultCustomization())
		{
			Customization->CustomizeDetails(InDetailBuilder);
		}
	}

	if (EditorMode != EAvaTransitionEditorMode::Advanced)
	{
		InDetailBuilder.HideCategory(TEXT("Evaluators"));
		InDetailBuilder.HideCategory(TEXT("Global Tasks"));
	}

	if (EditorMode == EAvaTransitionEditorMode::Parameter)
	{
		InDetailBuilder.HideCategory(TEXT("Theme"));
		CustomizeParameters(InDetailBuilder);
	}
}

void FAvaTransitionTreeEditorDataCustomization::CustomizeParameters(IDetailLayoutBuilder& InDetailBuilder)
{
	TSharedPtr<IPropertyHandle> RootParametersHandle = InDetailBuilder.GetProperty(TEXT("RootParameterPropertyBag"));
	check(RootParametersHandle);
	RootParametersHandle->MarkHiddenByCustomization();

	TSharedRef<FPropertyBagInstanceDataDetails> InstanceDetails = MakeShared<FPropertyBagInstanceDataDetails>(RootParametersHandle
		, InDetailBuilder.GetPropertyUtilities()
		, /*bFixedLayout*/true);

	InDetailBuilder.EditCategory(TEXT("Parameters"))
		.AddCustomBuilder(InstanceDetails);
}

EAvaTransitionEditorMode FAvaTransitionTreeEditorDataCustomization::GetEditorMode() const
{
	if (TSharedPtr<FAvaTransitionViewModelSharedData> SharedData = SharedDataWeak.Pin())
	{
		return SharedData->GetEditorMode();
	}

	// Fallback to Advanced as it is the one with most features
	return EAvaTransitionEditorMode::Advanced;
}

TSharedPtr<IDetailCustomization> FAvaTransitionTreeEditorDataCustomization::GetDefaultCustomization() const
{
	const FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	const FCustomDetailLayoutNameMap& CustomizationMap = PropertyModule.GetClassNameToDetailLayoutNameMap();
	const FDetailLayoutCallback* DefaultLayoutCallback = CustomizationMap.Find(UStateTreeEditorData::StaticClass()->GetFName());

	if (DefaultLayoutCallback && DefaultLayoutCallback->DetailLayoutDelegate.IsBound())
	{
		return DefaultLayoutCallback->DetailLayoutDelegate.Execute();
	}

	return nullptr;
}
