// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class FAvaTransitionViewModelSharedData;
enum class EAvaTransitionEditorMode : uint8;

/** Customization that re-uses the module-registered Customization for UStateTreeEditorData, and tweaks a few settings for it */
class FAvaTransitionTreeEditorDataCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance(TWeakPtr<FAvaTransitionViewModelSharedData> InSharedDataWeak);

	FAvaTransitionTreeEditorDataCustomization(const TWeakPtr<FAvaTransitionViewModelSharedData>& InSharedDataWeak);

protected:
	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization

private:
	void CustomizeParameters(IDetailLayoutBuilder& InDetailBuilder);

	EAvaTransitionEditorMode GetEditorMode() const;

	TSharedPtr<IDetailCustomization> GetDefaultCustomization() const;

	TWeakPtr<FAvaTransitionViewModelSharedData> SharedDataWeak;
};
