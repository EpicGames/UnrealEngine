// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaTransitionTransitionType.h"
#include "PropertyHandle.h"
#include "StateTreeState.h"
#include "StateTreeTypes.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaTransitionTransitionType"

void SAvaTransitionTransitionType::Construct(const FArguments& InArgs, const TSharedPtr<IPropertyHandleArray>& InTransitionsHandle)
{
	TransitionArrayHandle = InTransitionsHandle;

	UpdateTransition();

	TSharedRef<SHorizontalBox> TransitionOptions = SNew(SHorizontalBox);

	auto AddTransitionType = [&TransitionOptions, this](EStateTreeTransitionType InTransitionType, const FText& InLabel)
	{
		TransitionOptions->AddSlot()
		.Padding(1.f, 1.f)
		.AutoWidth()
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "DetailsView.SectionButton")
			.OnCheckStateChanged(this, &SAvaTransitionTransitionType::SetTransitionType, InTransitionType)
			.IsChecked(this, &SAvaTransitionTransitionType::GetTransitionCheckState, InTransitionType)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "SmallText")
				.Text(InLabel)
				.Justification(ETextJustify::Center)
			]
		];
	};

	AddTransitionType(EStateTreeTransitionType::None, LOCTEXT("TransitionTypeNone", "None"));
	AddTransitionType(EStateTreeTransitionType::NextSelectableState, LOCTEXT("TransitionTypeNextSelectableState", "Next"));
	AddTransitionType(EStateTreeTransitionType::Succeeded, LOCTEXT("TransitionTypeSucceeded", "Succeed"));

	ChildSlot
	[
		TransitionOptions
	];
}

void SAvaTransitionTransitionType::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(InAllottedGeometry, InCurrentTime, InDeltaTime);
	UpdateTransition();
}

void SAvaTransitionTransitionType::UpdateTransition()
{
	TransitionType.Reset();
	TransitionHandle.Reset();

	if (!TransitionArrayHandle.IsValid())
	{
		return;
	}

	// Return early if the count isn't 1. The expectation here is that there's only 1 transition in the array.
	uint32 TransitionCount = 0;
	if (TransitionArrayHandle->GetNumElements(TransitionCount) != FPropertyAccess::Result::Success)
	{
		return;
	}

	// If more than 1 transition entries are present, early return without setting the transition type, as all the current types deal with only having single transition
	if (TransitionCount > 1)
	{
		return;
	}

	// No transition type present, so TransitionType is none.
	if (TransitionCount == 0)
	{
		TransitionType = EStateTreeTransitionType::None;
		return;
	}

	TransitionHandle = TransitionArrayHandle->GetElement(0);

	TArray<const void*> TransitionRawData;
	TransitionHandle->AccessRawData(TransitionRawData);

	// No transition type found/set, so TransitionType is none.
	if (TransitionRawData.IsEmpty())
	{
		TransitionType = EStateTreeTransitionType::None;
		return;
	}

	for (const void* TransitionRaw : TransitionRawData)
	{
		const FStateTreeTransition* Transition = static_cast<const FStateTreeTransition*>(TransitionRaw);
		if (!Transition
			|| Transition->bDelayTransition
			|| !Transition->Conditions.IsEmpty()
			|| Transition->Trigger != EStateTreeTransitionTrigger::OnStateCompleted
			|| !Transition->bTransitionEnabled)
		{
			continue;
		}

		if (!TransitionType.IsSet())
		{
			TransitionType = Transition->State.LinkType;
		}

		// If Transition Type defers from the current iterated type, then transition types are mixed. Reset and return
		if (*TransitionType != Transition->State.LinkType)
		{
			TransitionType.Reset();
			return;
		}
	}
}

ECheckBoxState SAvaTransitionTransitionType::GetTransitionCheckState(EStateTreeTransitionType InTransitionType) const
{
	return TransitionType.IsSet() && *TransitionType == InTransitionType
		? ECheckBoxState::Checked
		: ECheckBoxState::Unchecked;
}

void SAvaTransitionTransitionType::SetTransitionType(ECheckBoxState InCheckState, EStateTreeTransitionType InTransitionType)
{
	if (InCheckState != ECheckBoxState::Checked)
	{
		return;
	}

	// Checking "None" is straight forward: clear the array and update
	if (InTransitionType == EStateTreeTransitionType::None)
	{
		if (TransitionArrayHandle.IsValid())
		{
			TransitionArrayHandle->EmptyArray();
			UpdateTransition();
		}
		return;
	}

	// For the rest of the entries, Transition Handle must be valid,
	// if it isn't Attempt to fix this setting a single element in the Transition Array
	if (!TransitionHandle.IsValid())
	{
		if (TransitionArrayHandle.IsValid())
		{
			TransitionArrayHandle->EmptyArray();
			TransitionArrayHandle->AddItem();
			UpdateTransition();
		}

		// If it's still not valid, then return
		if (!TransitionHandle.IsValid())
		{
			return;
		}
	}

	FStateTreeTransition Transition(EStateTreeTransitionTrigger::OnStateCompleted, InTransitionType);
	Transition.ID = FGuid::NewGuid();

	FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(TransitionHandle->GetProperty());

	FString TextValue;
	StructProperty->Struct->ExportText(TextValue, &Transition, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr);
	ensure(TransitionHandle->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
}

#undef LOCTEXT_NAMESPACE
