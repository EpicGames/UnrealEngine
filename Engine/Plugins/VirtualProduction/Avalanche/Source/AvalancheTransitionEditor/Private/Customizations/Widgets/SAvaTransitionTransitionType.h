// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "Widgets/SCompoundWidget.h"

class IPropertyHandle;
class IPropertyHandleArray;
enum class ECheckBoxState : uint8;
enum class EStateTreeTransitionType : uint8;

class SAvaTransitionTransitionType : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaTransitionTransitionType) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<IPropertyHandleArray>& InTransitionsHandle);

protected:
	//~ Begin SWidget
	virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

private:
	void UpdateTransition();

	ECheckBoxState GetTransitionCheckState(EStateTreeTransitionType InTransitionType) const;

	void SetTransitionType(ECheckBoxState InCheckState, EStateTreeTransitionType InTransitionType);

	TSharedPtr<IPropertyHandleArray> TransitionArrayHandle;

	TSharedPtr<IPropertyHandle> TransitionHandle;

	TOptional<EStateTreeTransitionType> TransitionType;
};
