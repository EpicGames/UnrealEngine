// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeTypes.h"

#ifndef UE_AVA_WITH_TRANSITION_DEBUG
#define UE_AVA_WITH_TRANSITION_DEBUG (WITH_STATETREE_DEBUG && WITH_STATETREE_TRACE_DEBUGGER)
#endif
