// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionDebugDefinitions.h"

#if UE_AVA_WITH_TRANSITION_DEBUG

#include "Containers/UnrealString.h"
#include "Math/Color.h"
#include "StateTreeExecutionTypes.h"

/** Debug Information to identify a Tree Debug Instance */
struct FAvaTransitionDebugInfo
{
	FStateTreeInstanceDebugId Id;

	FString Name;

	FLinearColor Color;
};

#endif // UE_AVA_WITH_TRANSITION_DEBUG
