// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionDebugger.h"

#if UE_AVA_WITH_TRANSITION_DEBUG

#include "AvaTransitionEditorLog.h"
#include "AvaTransitionTree.h"
#include "AvaTransitionTreeEditorData.h"
#include "StateTreeExecutionContext.h"
#include "Debugger/StateTreeDebug.h"
#include "Extensions/IAvaTransitionDebuggableExtension.h"
#include "StateTreeExecutionTypes.h"
#include "Debugger/StateTreeDebugger.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "ViewModels/AvaTransitionViewModelSharedData.h"
#include "ViewModels/Registry/AvaTransitionViewModelRegistryCollection.h"

FAvaTransitionDebugger::FAvaTransitionDebugger()
{
}

FAvaTransitionDebugger::~FAvaTransitionDebugger()
{
	Stop();
}

void FAvaTransitionDebugger::Initialize(const TSharedRef<FAvaTransitionEditorViewModel>& InEditorViewModel)
{
	EditorViewModelWeak = InEditorViewModel;
}

void FAvaTransitionDebugger::Start()
{
	if (IsActive())
	{
		return;
	}

	bActive = true;
	RegisterDelegates();
}

void FAvaTransitionDebugger::Stop()
{
	UnregisterDelegates();
	TreeDebugInstances.Reset();
	bActive = false;
}

bool FAvaTransitionDebugger::IsActive() const
{
	return bActive;
}

void FAvaTransitionDebugger::OnTreeInstanceStarted(const FStateTreeInstanceDebugId& InInstanceDebugId, const FString& InInstanceName)
{
	// Ensure that a debug instance of the given id doesn't exist already
	const int32 Index = TreeDebugInstances.IndexOfByKey(InInstanceDebugId);
	if (Index == INDEX_NONE)
	{
		TreeDebugInstances.Emplace(InInstanceDebugId, InInstanceName);
	}
}

void FAvaTransitionDebugger::OnTreeInstanceStopped(const FStateTreeInstanceDebugId& InInstanceDebugId)
{
	const int32 Index = TreeDebugInstances.IndexOfByKey(InInstanceDebugId);
	if (Index != INDEX_NONE)
	{
		TreeDebugInstances.RemoveAt(Index);
	}
}

void FAvaTransitionDebugger::OnNodeEntered(const FGuid& InNodeId, const FStateTreeInstanceDebugId& InInstanceDebugId)
{
	if (FAvaTransitionTreeDebugInstance* TreeDebugInstance = TreeDebugInstances.FindByKey(InInstanceDebugId))
	{
		TSharedPtr<IAvaTransitionDebuggableExtension> Debuggable = FindDebuggable(InNodeId);
		TreeDebugInstance->EnterDebuggable(Debuggable);
	}
}

void FAvaTransitionDebugger::OnNodeExited(const FGuid& InNodeId, const FStateTreeInstanceDebugId& InInstanceDebugId)
{
	if (FAvaTransitionTreeDebugInstance* TreeDebugInstance = TreeDebugInstances.FindByKey(InInstanceDebugId))
	{
		TSharedPtr<IAvaTransitionDebuggableExtension> Debuggable = FindDebuggable(InNodeId);
		TreeDebugInstance->ExitDebuggable(Debuggable);
	}
}

void FAvaTransitionDebugger::RegisterDelegates()
{
	UnregisterDelegates();
	OnBeginUpdatePhaseHandle = UE::StateTree::Debug::OnBeginUpdatePhase_AnyThread.AddSP(this, &FAvaTransitionDebugger::OnBeginUpdatePhase);
	OnStateEventHandle = UE::StateTree::Debug::OnStateEvent_AnyThread.AddSP(this, &FAvaTransitionDebugger::OnStateEvent);
}

void FAvaTransitionDebugger::UnregisterDelegates()
{
	UE::StateTree::Debug::OnBeginUpdatePhase_AnyThread.Remove(OnBeginUpdatePhaseHandle);
	OnBeginUpdatePhaseHandle.Reset();

	UE::StateTree::Debug::OnStateEvent_AnyThread.Remove(OnStateEventHandle);
	OnStateEventHandle.Reset();
}

void FAvaTransitionDebugger::OnBeginUpdatePhase(const FStateTreeExecutionContext& InExecutionContext, EStateTreeUpdatePhase InPhase, FStateTreeStateHandle InStateHandle)
{
	constexpr EStateTreeUpdatePhase StartTreePhase = EStateTreeUpdatePhase::StartTree;
	constexpr EStateTreeUpdatePhase StopTreePhase = EStateTreeUpdatePhase::StopTree;

	// This function is only concerned with Start/Stop tree for now.
	if (InPhase != StartTreePhase && InPhase != StopTreePhase)
	{
		return;
	}

	TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = EditorViewModelWeak.Pin();
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	// Return early if the State Tree of this debugger doesn't match the execution context's
	const UAvaTransitionTreeEditorData* const EditorData = EditorViewModel->GetEditorData();
	if (!EditorData || !EditorData->Compare(Cast<UAvaTransitionTreeEditorData>(InExecutionContext.GetStateTree()->EditorData)))
	{
		return;
	}

	switch (InPhase)
	{
	case StartTreePhase:
		OnTreeInstanceStarted(InExecutionContext.GetInstanceDebugId(), InExecutionContext.GetInstanceDebugDescription());
		break;

	case StopTreePhase:
		OnTreeInstanceStopped(InExecutionContext.GetInstanceDebugId());
		break;
	}
}

void FAvaTransitionDebugger::OnStateEvent(const FStateTreeExecutionContext& InExecutionContext, FStateTreeStateHandle InStateHandle, EStateTreeTraceEventType InEventType)
{
	constexpr EStateTreeTraceEventType EnterEvent = EStateTreeTraceEventType::OnEntering;
	constexpr EStateTreeTraceEventType ExitEvent = EStateTreeTraceEventType::OnExited;

	// This function is only concerned with enter/exit state for now.
	if (InEventType != EnterEvent && InEventType != ExitEvent)
	{
		return;
	}

	TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = EditorViewModelWeak.Pin();
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	const UAvaTransitionTree* const TransitionTree = EditorViewModel->GetTransitionTree();
	if (!TransitionTree)
	{
		return;
	}

	// Return early if the State Tree of this debugger doesn't match the execution context's
	const UAvaTransitionTreeEditorData* const EditorData = EditorViewModel->GetEditorData();
	if (!EditorData || !EditorData->Compare(Cast<UAvaTransitionTreeEditorData>(InExecutionContext.GetStateTree()->EditorData)))
	{
		return;
	}

	switch (InEventType)
	{
	case EnterEvent:
		OnNodeEntered(TransitionTree->GetStateIdFromHandle(InStateHandle), InExecutionContext.GetInstanceDebugId());
		break;

	case ExitEvent:
		OnNodeExited(TransitionTree->GetStateIdFromHandle(InStateHandle), InExecutionContext.GetInstanceDebugId());
		break;
	}
}

TSharedPtr<IAvaTransitionDebuggableExtension> FAvaTransitionDebugger::FindDebuggable(const FGuid& InNodeId) const
{
	if (TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = EditorViewModelWeak.Pin())
	{
		TSharedRef<FAvaTransitionViewModelRegistryCollection> RegistryCollection = EditorViewModel->GetSharedData()->GetRegistryCollection();
		return UE::AvaCore::CastSharedPtr<IAvaTransitionDebuggableExtension>(RegistryCollection->FindViewModel(InNodeId));
	}
	return nullptr;
}

#endif // UE_AVA_WITH_TRANSITION_DEBUG
