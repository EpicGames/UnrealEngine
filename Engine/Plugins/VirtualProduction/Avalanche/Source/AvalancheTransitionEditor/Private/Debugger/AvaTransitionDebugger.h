// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionDebugDefinitions.h"

#if UE_AVA_WITH_TRANSITION_DEBUG

#include "AvaTransitionTreeDebugInstance.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class FAvaTransitionEditorViewModel;
class IAvaTransitionDebuggableExtension;
class UStateTree;
enum class EStateTreeTraceEventType : uint8;
enum class EStateTreeUpdatePhase : uint8;
struct FStateTreeInstanceDebugId;

/** Handles forwarding the debug events from state tree execution to the debuggables */
class FAvaTransitionDebugger : public TSharedFromThis<FAvaTransitionDebugger>
{
public:
	FAvaTransitionDebugger();

	~FAvaTransitionDebugger();

	/** Called once to set the editor view model */
	void Initialize(const TSharedRef<FAvaTransitionEditorViewModel>& InEditorViewModel);

	/** Starts the debugger */
	void Start();

	/** Stops the debugger */
	void Stop();

	/** Returns whether the debugger is currently active */
	bool IsActive() const;

	/** Called when an instance of the transition tree has started */
	void OnTreeInstanceStarted(const FStateTreeInstanceDebugId& InInstanceDebugId, const FString& InInstanceName);

	/** Called when an instance of the transition tree has stopped */
	void OnTreeInstanceStopped(const FStateTreeInstanceDebugId& InInstanceDebugId);

	/** Called when a node with the given id was started / entered */
	void OnNodeEntered(const FGuid& InNodeId, const FStateTreeInstanceDebugId& InInstanceDebugId);

	/** Called when a node with the given id was stopped / exited */
	void OnNodeExited(const FGuid& InNodeId, const FStateTreeInstanceDebugId& InInstanceDebugId);

private:
	void RegisterDelegates();
	void UnregisterDelegates();

	/** Called when an update phase has started */
	void OnBeginUpdatePhase(const FStateTreeExecutionContext& InExecutionContext, EStateTreeUpdatePhase InPhase, FStateTreeStateHandle InStateHandle);

	/** Called when a state event has triggered */
	void OnStateEvent(const FStateTreeExecutionContext& InExecutionContext, FStateTreeStateHandle InStateHandle, EStateTreeTraceEventType InEventType);

	/** Finds the view model with the matching id and returns it as debuggable extension, or null if the view model doesn't exist or doesn't have a debuggable extension */
	TSharedPtr<IAvaTransitionDebuggableExtension> FindDebuggable(const FGuid& InNodeId) const;

	/** the view model owning this debugger */
	TWeakPtr<FAvaTransitionEditorViewModel> EditorViewModelWeak;

	/** All the tree instances that are in execution and being debugged */
	TArray<FAvaTransitionTreeDebugInstance> TreeDebugInstances;

	/** Handle to the delegate when an update phase has begun */
	FDelegateHandle OnBeginUpdatePhaseHandle;

	/** Handle to the delegate when a state event has triggered */
	FDelegateHandle OnStateEventHandle;

	/** Determines whether the debugger is currently active */
	bool bActive = false;
};

#endif // UE_AVA_WITH_TRANSITION_DEBUG
