// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionStateDebugInstance.h"

#if UE_AVA_WITH_TRANSITION_DEBUG

namespace UE::AvaTransitionEditor::Private
{
	constexpr static float DebugEnteringTime = 0.25f;
	constexpr static float DebugGracePeriod  = 2.0f;
	constexpr static float DebugExitingTime  = 0.75f;
}

FAvaTransitionStateDebugInstance::FAvaTransitionStateDebugInstance(const FAvaTransitionDebugInfo& InDebugInfo)
	: DebugInfo(InDebugInfo)
{
}

FAvaTransitionStateDebugInstance::~FAvaTransitionStateDebugInstance()
{
	SetDebugState(EAvaTransitionStateDebugStatus::Exited);
}

void FAvaTransitionStateDebugInstance::Enter()
{
	SetDebugState(EAvaTransitionStateDebugStatus::Entering);
	UpdateActivationState();
}

void FAvaTransitionStateDebugInstance::Exit()
{
	SetDebugState(EAvaTransitionStateDebugStatus::ExitingGrace);
	UpdateActivationState();
}

void FAvaTransitionStateDebugInstance::Tick(float InDeltaTime)
{
	StatusElapsedTime += InDeltaTime;
	UpdateActivationState();
}

float FAvaTransitionStateDebugInstance::GetStatusProgress() const
{
	using namespace UE::AvaTransitionEditor;

	switch (DebugStatus)
	{
	case EAvaTransitionStateDebugStatus::Entering:
		return FMath::Clamp(FMath::GetRangePct(0.f, Private::DebugEnteringTime, StatusElapsedTime), 0.f, 1.f);

	case EAvaTransitionStateDebugStatus::Entered:
	case EAvaTransitionStateDebugStatus::ExitingGrace:
		return 1.f;

	case EAvaTransitionStateDebugStatus::Exiting:
		return FMath::Clamp(FMath::GetRangePct(0.f, Private::DebugExitingTime, Private::DebugExitingTime - StatusElapsedTime), 0.f, 1.f);

	case EAvaTransitionStateDebugStatus::Exited:
		return 0.f;
	}

	checkNoEntry();
	return 0.f;
}

void FAvaTransitionStateDebugInstance::UpdateActivationState()
{
	using namespace UE::AvaTransitionEditor;

	switch (DebugStatus)
	{
	case EAvaTransitionStateDebugStatus::Entering:
		if (StatusElapsedTime >= Private::DebugEnteringTime)
		{
			SetDebugState(EAvaTransitionStateDebugStatus::Entered);
		}
		break;

	case EAvaTransitionStateDebugStatus::ExitingGrace:
		if (StatusElapsedTime >= Private::DebugGracePeriod)
		{
			SetDebugState(EAvaTransitionStateDebugStatus::Exiting);
		}
		break;

	case EAvaTransitionStateDebugStatus::Exiting:
		if (StatusElapsedTime >= Private::DebugExitingTime)
		{
			SetDebugState(EAvaTransitionStateDebugStatus::Exited);
		}
		break;
	}
}

void FAvaTransitionStateDebugInstance::SetDebugState(EAvaTransitionStateDebugStatus InDebugState)
{
	if (DebugStatus != InDebugState)
	{
		DebugStatus = InDebugState;
		StatusElapsedTime = 0.f;
	}
}

#endif // UE_AVA_WITH_TRANSITION_DEBUG
