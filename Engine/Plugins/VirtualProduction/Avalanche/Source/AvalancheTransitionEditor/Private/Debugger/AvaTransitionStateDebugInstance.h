// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionDebugDefinitions.h"

#if UE_AVA_WITH_TRANSITION_DEBUG

#include "Debugger/AvaTransitionDebugInfo.h"
#include "StateTreeExecutionTypes.h"

enum class EAvaTransitionStateDebugStatus : uint8
{
	/** Instance started to enter */
	Entering,

	/** Instance finished entering */
	Entered,

	/** Instance is in grace period before it can start exiting */
	ExitingGrace,

	/** Instance started exiting */
	Exiting,

	/** Instance ended exiting and can now be removed */
	Exited,
};

/** Represents a Debug Instance active in a State */
class FAvaTransitionStateDebugInstance
{
public:
	explicit FAvaTransitionStateDebugInstance(const FAvaTransitionDebugInfo& InDebugInfo);

	~FAvaTransitionStateDebugInstance();

	void Enter();

	void Exit();

	void Tick(float InDeltaTime);

	/** Returns 0 if Exited, 1 if Entered, or anything in between if switching status */
	float GetStatusProgress() const;

	EAvaTransitionStateDebugStatus GetDebugStatus() const
	{
		return DebugStatus;
	}

	const FAvaTransitionDebugInfo& GetDebugInfo() const
	{
		return DebugInfo;
	}

	bool operator==(const FStateTreeInstanceDebugId& InDebugInstanceId) const
	{
		return DebugInfo.Id == InDebugInstanceId;
	}

private:
	void UpdateActivationState();

	void SetDebugState(EAvaTransitionStateDebugStatus InDebugState);

	FAvaTransitionDebugInfo DebugInfo;

	EAvaTransitionStateDebugStatus DebugStatus = EAvaTransitionStateDebugStatus::Exited;

	/** Elapsed Time in the current status */
	float StatusElapsedTime = 0.f;
};

#endif // UE_AVA_WITH_TRANSITION_DEBUG
