// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionTreeDebugInstance.h"

#if UE_AVA_WITH_TRANSITION_DEBUG

#include "Extensions/IAvaTransitionDebuggableExtension.h"
#include "StateTreeExecutionTypes.h"

namespace UE::AvaTransitionEditor::Private
{
	FLinearColor GenerateDebugColor()
	{
		constexpr float DeltaHue = 100.f;

		static float Hue = -DeltaHue;
		Hue += DeltaHue;

		// Keep Hue in 0 to 360 range
		if (Hue >= 360.f)
		{
			Hue -= 360.f;
		}

		return FLinearColor(Hue, /*Saturation*/0.7f, /*Value*/0.2f).HSVToLinearRGB();
	}
}

FAvaTransitionTreeDebugInstance::FAvaTransitionTreeDebugInstance(const FStateTreeInstanceDebugId& InId, const FString& InName)
{
	DebugInfo.Id    = InId;
	DebugInfo.Name  = InName;
	DebugInfo.Color = UE::AvaTransitionEditor::Private::GenerateDebugColor();
}

FAvaTransitionTreeDebugInstance::~FAvaTransitionTreeDebugInstance()
{
	Reset();
}

bool FAvaTransitionTreeDebugInstance::operator==(const FStateTreeInstanceDebugId& InId) const
{
	return DebugInfo.Id == InId;
}

void FAvaTransitionTreeDebugInstance::EnterDebuggable(const TSharedPtr<IAvaTransitionDebuggableExtension>& InDebuggable)
{
	if (InDebuggable.IsValid() && !Debuggables.Contains(InDebuggable))
	{
		Debuggables.Add(InDebuggable);
		InDebuggable->DebugEnter(DebugInfo);
	}
}

void FAvaTransitionTreeDebugInstance::ExitDebuggable(const TSharedPtr<IAvaTransitionDebuggableExtension>& InDebuggable)
{
	if (InDebuggable.IsValid() && Debuggables.Remove(InDebuggable) > 0)
	{
		InDebuggable->DebugExit(DebugInfo);
	}
}

void FAvaTransitionTreeDebugInstance::Reset()
{
	// Exit current debuggables
	for (const TWeakPtr<IAvaTransitionDebuggableExtension>& DebuggableWeak : Debuggables)
	{
		if (TSharedPtr<IAvaTransitionDebuggableExtension> Debuggable = DebuggableWeak.Pin())
		{
			Debuggable->DebugExit(DebugInfo);
		}
	}
	Debuggables.Reset();
}

#endif // UE_AVA_WITH_TRANSITION_DEBUG
