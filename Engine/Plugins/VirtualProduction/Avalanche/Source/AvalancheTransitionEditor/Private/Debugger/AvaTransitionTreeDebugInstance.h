// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionDebugDefinitions.h"

#if UE_AVA_WITH_TRANSITION_DEBUG

#include "AvaTransitionDebugInfo.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

class IAvaTransitionDebuggableExtension;
struct FStateTreeInstanceDebugId;

/** Active State Tree Instance running in debug mode */
class FAvaTransitionTreeDebugInstance
{
public:
	explicit FAvaTransitionTreeDebugInstance(const FStateTreeInstanceDebugId& InId, const FString& InName);

	~FAvaTransitionTreeDebugInstance();

	bool operator==(const FStateTreeInstanceDebugId& InId) const;

	void EnterDebuggable(const TSharedPtr<IAvaTransitionDebuggableExtension>& InDebuggable);

	void ExitDebuggable(const TSharedPtr<IAvaTransitionDebuggableExtension>& InDebuggable);

private:
	void Reset();

	TArray<TWeakPtr<IAvaTransitionDebuggableExtension>> Debuggables;

	FAvaTransitionDebugInfo DebugInfo;
};

#endif // UE_AVA_WITH_TRANSITION_DEBUG
