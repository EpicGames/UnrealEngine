// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionStateDragDropOp.h"
#include "AvaTransitionEditorUtils.h"
#include "AvaTransitionSelection.h"
#include "Extensions/IAvaTransitionDragDropExtension.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "ViewModels/AvaTransitionViewModel.h"
#include "ViewModels/AvaTransitionViewModelSharedData.h"
#include "ViewModels/AvaTransitionViewModelUtils.h"
#include "ViewModels/State/AvaTransitionStateViewModel.h"

#define LOCTEXT_NAMESPACE "AvaTransitionStateDragDropOp"

TSharedRef<FAvaTransitionStateDragDropOp> FAvaTransitionStateDragDropOp::New(const TSharedRef<FAvaTransitionStateViewModel>& InStateViewModel, bool bInDuplicateStates)
{
	TSharedRef<FAvaTransitionStateDragDropOp> DragDropOp = MakeShared<FAvaTransitionStateDragDropOp>();
	DragDropOp->Init(InStateViewModel, bInDuplicateStates);
	return DragDropOp;
}

void FAvaTransitionStateDragDropOp::Init(const TSharedRef<FAvaTransitionStateViewModel>& InStateViewModel, bool bInDuplicateStates)
{
	TConstArrayView<TSharedRef<FAvaTransitionViewModel>> SelectedViewModels = InStateViewModel->GetSharedData()->GetSelection()->GetSelectedItems();

	StateViewModels.Reset(SelectedViewModels.Num() + 1);
	StateViewModels.Append(UE::AvaTransitionEditor::GetViewModelsOfType<FAvaTransitionStateViewModel>(SelectedViewModels));
	StateViewModels.AddUnique(InStateViewModel);

	bDuplicateStates = bInDuplicateStates;
	MouseCursor      = EMouseCursor::GrabHandClosed;
	CurrentIconBrush = nullptr;
	CurrentHoverText = FText::Format(LOCTEXT("HoverText", "Dragging {0} states"), StateViewModels.Num());

	CurrentIconColorAndOpacity = FSlateColor::UseForeground();

	SetupDefaults();
	Construct();
}

#undef LOCTEXT_NAMESPACE
