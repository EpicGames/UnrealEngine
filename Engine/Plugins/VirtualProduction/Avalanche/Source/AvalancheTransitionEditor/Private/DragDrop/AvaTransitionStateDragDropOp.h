// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Templates/SharedPointer.h"

class FAvaTransitionStateViewModel;
enum class EItemDropZone;

class FAvaTransitionStateDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FAvaTransitionStateDragDropOp, FDecoratedDragDropOp)

	static TSharedRef<FAvaTransitionStateDragDropOp> New(const TSharedRef<FAvaTransitionStateViewModel>& InStateViewModel, bool bInDuplicateStates);

	TConstArrayView<TSharedRef<FAvaTransitionStateViewModel>> GetStateViewModels() const
	{
		return StateViewModels;
	}

	bool ShouldDuplicateStates() const
	{
		return bDuplicateStates;
	}

protected:
	void Init(const TSharedRef<FAvaTransitionStateViewModel>& InStateViewModel, bool bInDuplicateStates);

private:
	TArray<TSharedRef<FAvaTransitionStateViewModel>> StateViewModels;

	bool bDuplicateStates = false;
};
