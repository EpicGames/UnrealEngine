// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debugger/AvaTransitionDebugDefinitions.h"
#include "AvaType.h"

#if UE_AVA_WITH_TRANSITION_DEBUG
struct FAvaTransitionDebugInfo;
#endif

/** Extension for  View Model that can be debugged */
class IAvaTransitionDebuggableExtension
{
public:
	UE_AVA_TYPE(IAvaTransitionDebuggableExtension)

#if UE_AVA_WITH_TRANSITION_DEBUG
	virtual void DebugEnter(const FAvaTransitionDebugInfo& InDebugInfo) = 0;

	virtual void DebugExit(const FAvaTransitionDebugInfo& InDebugInfo) = 0;
#endif
};
