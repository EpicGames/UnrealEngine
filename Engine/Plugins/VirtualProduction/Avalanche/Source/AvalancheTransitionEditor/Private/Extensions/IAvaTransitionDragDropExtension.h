// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"

class FDragDropEvent;
class FReply;
enum class EItemDropZone;
struct FGeometry;
struct FPointerEvent;
template<typename OptionalType> struct TOptional;

/** Extension for a View Model that can handle drag / drop operations */
class IAvaTransitionDragDropExtension
{
public:
	UE_AVA_TYPE(IAvaTransitionDragDropExtension)

	virtual TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) = 0;

	virtual FReply OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) = 0;

	virtual FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) = 0;
};
