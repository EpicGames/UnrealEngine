// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"

struct FGuid;

/** Extension for a View Model that can be handles an underlying Guid */
class IAvaTransitionGuidExtension
{
public:
	UE_AVA_TYPE(IAvaTransitionGuidExtension)

	virtual const FGuid& GetGuid() const = 0;
};
