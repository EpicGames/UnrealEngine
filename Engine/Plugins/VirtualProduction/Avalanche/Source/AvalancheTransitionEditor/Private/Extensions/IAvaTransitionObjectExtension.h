// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"

class UObject;
struct FPropertyChangedEvent;

/** Extension for a View Model that represents a UObject */
class IAvaTransitionObjectExtension
{
public:
	UE_AVA_TYPE(IAvaTransitionObjectExtension)

	virtual UObject* GetObject() const = 0;

	virtual void OnPropertiesChanged(const FPropertyChangedEvent& InPropertyChangedEvent) = 0;
};
