// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"

/** Extension for a View Model that supports being selected */
class IAvaTransitionSelectableExtension
{
public:
	UE_AVA_TYPE(IAvaTransitionSelectableExtension)

	virtual void SetSelected(bool bInIsSelected) = 0;

	virtual bool IsSelected() const = 0;
};
