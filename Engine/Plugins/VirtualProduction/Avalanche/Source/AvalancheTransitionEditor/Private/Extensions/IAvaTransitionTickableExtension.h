// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"

/** Extension for a View Model that supports ticking */
class IAvaTransitionTickableExtension
{
public:
	UE_AVA_TYPE(IAvaTransitionTickableExtension)

	virtual void Tick(float InDeltaTime) = 0;
};
