// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointerFwd.h"

class ITableRow;
class STableViewBase;

/** Extension for a View Model that supports being a Tree Row */
class IAvaTransitionTreeRowExtension
{
public:
	UE_AVA_TYPE(IAvaTransitionTreeRowExtension)

	virtual bool CanGenerateRow() const
	{
		return true;
	}

	virtual TSharedRef<ITableRow> GenerateRow(const TSharedRef<STableViewBase>& InOwningTableView) = 0;

	/** Indication whether the View Model is Expanded */
	virtual bool IsExpanded() const = 0;

	/** Sets the Expansion state of the View Model */
	virtual void SetExpanded(bool bInIsExpanded) = 0;
};
