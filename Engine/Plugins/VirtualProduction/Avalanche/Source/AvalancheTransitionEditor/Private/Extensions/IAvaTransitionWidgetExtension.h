// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "Templates/SharedPointerFwd.h"

class SWidget;

/** Extension for a View Model that can supports creating its own widget */
class IAvaTransitionWidgetExtension
{
public:
	UE_AVA_TYPE(IAvaTransitionWidgetExtension)

	virtual TSharedRef<SWidget> CreateWidget() = 0;
};
