// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "AvaTransitionMenuContext.generated.h"

class FAvaTransitionEditorViewModel;

UCLASS(MinimalAPI)
class UAvaTransitionMenuContext : public UObject
{
	GENERATED_BODY()

public:
	UAvaTransitionMenuContext() = default;

	TSharedPtr<FAvaTransitionEditorViewModel> GetEditorViewModel() const
	{
		return EditorViewModelWeak.Pin();
	}

	void SetEditorViewModel(const TSharedRef<FAvaTransitionEditorViewModel>& InEditorViewModel)
	{
		EditorViewModelWeak = InEditorViewModel;
	}

private:
	TWeakPtr<FAvaTransitionEditorViewModel> EditorViewModelWeak;
};
