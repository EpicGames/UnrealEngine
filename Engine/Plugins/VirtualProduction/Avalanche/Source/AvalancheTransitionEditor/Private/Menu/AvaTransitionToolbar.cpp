// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionToolbar.h"
#include "AvaTransitionCommands.h"
#include "AvaTransitionEditorUtils.h"
#include "AvaTransitionMenuContext.h"
#include "AvaTransitionTree.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "ViewModels/AvaTransitionViewModelSharedData.h"
#include "Widgets/AvaTransitionTreeStatus.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AvaTransitionToolbar"

namespace UE::AvaTransitionEditor::Private
{
	FToolMenuEntry& AddLabelEntry(FToolMenuSection& InSection, FName InEntryName, const FText& InLabel)
	{
		TSharedRef<SWidget> LabelWidget = SNew(SBox)
			.Padding(8.f, 0.f, 2.f, 0.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(InLabel)
				.Justification(ETextJustify::Right)
			];
		return InSection.AddEntry(FToolMenuEntry::InitWidget(InEntryName, LabelWidget, FText::GetEmpty()));
	}
}

FAvaTransitionToolbar::FAvaTransitionToolbar(FAvaTransitionEditorViewModel& InOwner)
	: Owner(InOwner)
{
}

void FAvaTransitionToolbar::SetReadOnlyProfileName(FName InToolMenuToolbarName, FName InReadOnlyProfileName)
{
	ToolMenuToolbarName = InToolMenuToolbarName;
	ReadOnlyProfileName = InReadOnlyProfileName;
}

void FAvaTransitionToolbar::ExtendEditorToolbar(UToolMenu* InToolbarMenu)
{
	using namespace UE::AvaTransitionEditor;

	if (!InToolbarMenu)
	{
		return;
	}

	const bool bReadOnly = Owner.GetSharedData()->IsReadOnly();

	const FName PermissionOwner = TEXT("FAvaTransitionToolbar");

	FNamePermissionList ReadOnlyPermissionList;

	auto AllowInReadOnly = [&ReadOnlyPermissionList, &PermissionOwner](FToolMenuEntry& InEntry)
		{
			ReadOnlyPermissionList.AddAllowListItem(PermissionOwner, InEntry.Name);;
		};

	FToolMenuSection& Section = InToolbarMenu->FindOrAddSection(TEXT("TransitionLogic"));

	const FAvaTransitionEditorCommands& Commands = FAvaTransitionEditorCommands::Get();

	FToolMenuEntry& CompileButton = Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.Compile
		, TAttribute<FText>()
		, TAttribute<FText>()
		, TAttribute<FSlateIcon>::Create(TAttribute<FSlateIcon>::FGetter::CreateSPLambda(this,
			[this]
			{
				return Owner.GetCompiler().GetCompileStatusIcon();
			}))));

	FToolMenuEntry& CompileOptions = Section.AddEntry(FToolMenuEntry::InitComboButton(TEXT("CompileComboButton")
		, FUIAction()
		, FNewToolMenuDelegate::CreateStatic(&FAvaTransitionCompiler::GenerateCompileOptionsMenu)
		, LOCTEXT("CompileOptions_ToolbarTooltip", "Options to customize how State Trees compile")));

	CompileOptions.ToolBarData.bSimpleComboBox = true;

	int32 SeparatorIndex = 0;
	auto MakeSeparator = [&SeparatorIndex](FToolMenuSection& InSection)->FToolMenuEntry&
		{
			return InSection.AddSeparator(FName(TEXT("Separator"), SeparatorIndex++));
		};

	UAvaTransitionTree* const TransitionTree = Owner.GetTransitionTree();

	IAvaTransitionBehavior* const TransitionBehavior = TransitionTree ? TransitionTree->GetImplementingOuter<IAvaTransitionBehavior>() : nullptr;

	// Tree Status
	if (TSharedPtr<SWidget> TreeStatusWidget = CreateTransitionStatusWidget(TransitionBehavior))
	{
		TreeStatusWidget->SetEnabled(!bReadOnly);

		AllowInReadOnly(MakeSeparator(Section));

		AllowInReadOnly(Private::AddLabelEntry(Section, TEXT("TreeStatusLabel"), LOCTEXT("StatusLabel", "Status")));

		AllowInReadOnly(Section.AddEntry(FToolMenuEntry::InitWidget(TEXT("TreeStatusWidget")
			, TreeStatusWidget.ToSharedRef()
			, FText::GetEmpty())));
	}

	// Layer Picker
	if (TSharedPtr<SWidget> LayerPicker = CreateTransitionLayerPicker(TransitionBehavior))
	{
		LayerPicker->SetEnabled(!bReadOnly);

		AllowInReadOnly(MakeSeparator(Section));

		AllowInReadOnly(Private::AddLabelEntry(Section, TEXT("LayerPickerLabel"), LOCTEXT("LayerLabel", "Layer")));

		AllowInReadOnly(Section.AddEntry(FToolMenuEntry::InitWidget(TEXT("LayerPickerWidget")
			, LayerPicker.ToSharedRef()
			, FText::GetEmpty())));
	}

#if UE_AVA_WITH_TRANSITION_DEBUG
	AllowInReadOnly(MakeSeparator(Section));
	AllowInReadOnly(Section.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.ToggleDebug)));
#endif

	ApplyReadOnlyPermissionList(ReadOnlyPermissionList);
}

void FAvaTransitionToolbar::ExtendTreeToolbar(UToolMenu* InToolbarMenu)
{
	if (!InToolbarMenu)
	{
		return;
	}

	const bool bReadOnly = Owner.GetSharedData()->IsReadOnly();

	const FAvaTransitionEditorCommands& Commands = FAvaTransitionEditorCommands::Get();

	if (!bReadOnly)
	{
		FToolMenuSection& StateSection = InToolbarMenu->FindOrAddSection(TEXT("State"), LOCTEXT("StateActions", "State Actions"));
		StateSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.AddSiblingState, LOCTEXT("AddStateLabel", "Add State")));
	}

	FToolMenuSection& TreeSection = InToolbarMenu->FindOrAddSection(TEXT("Tree"), LOCTEXT("TreeActions", "Tree Actions"));
	if (!bReadOnly)
	{
		TreeSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.ImportTransitionTree));	
	}
	TreeSection.AddEntry(FToolMenuEntry::InitToolBarButton(Commands.ExportTransitionTree));
}

void FAvaTransitionToolbar::SetupReadOnlyCustomization(FReadOnlyAssetEditorCustomization& InReadOnlyCustomization)
{
	// Read Only Customizations not added here, as this is called before the Tool Menu is extended, and so the names are not known at this point
}

TSharedRef<SWidget> FAvaTransitionToolbar::GenerateTreeToolbarWidget()
{
	UToolMenus* const ToolMenus = UToolMenus::Get();
	check(ToolMenus);

	const FName TreeToolbarName = GetTreeToolbarName();

	if (!ToolMenus->IsMenuRegistered(TreeToolbarName))
	{
		UToolMenu* const Toolbar = ToolMenus->RegisterMenu(TreeToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		Toolbar->StyleName = "CalloutToolbar";
		Toolbar->AddDynamicSection("PopulateToolbar", FNewToolMenuDelegate::CreateStatic([](UToolMenu* InToolMenu)
		{
			if (InToolMenu)
			{
				if (UAvaTransitionMenuContext* MenuContext = InToolMenu->FindContext<UAvaTransitionMenuContext>())
				{
					if (TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = MenuContext->GetEditorViewModel())
					{
						EditorViewModel->GetToolbar()->ExtendTreeToolbar(InToolMenu);
					}
				}
			}
		}));
	}

	TSharedPtr<FExtender> Extender;

	UAvaTransitionMenuContext* const ContextObject = NewObject<UAvaTransitionMenuContext>();
	ContextObject->SetEditorViewModel(StaticCastSharedRef<FAvaTransitionEditorViewModel>(Owner.AsShared()));

	FToolMenuContext Context(Owner.GetCommandList(), Extender, ContextObject);
	return ToolMenus->GenerateWidget(TreeToolbarName, Context);
}

void FAvaTransitionToolbar::ApplyReadOnlyPermissionList(const FNamePermissionList& InPermissionList)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return;
	}

	if (FToolMenuProfile* ToolbarProfile = ToolMenus->FindRuntimeMenuProfile(ToolMenuToolbarName, ReadOnlyProfileName))
	{
		ToolbarProfile->MenuPermissions.Append(InPermissionList);
	}
}

#undef LOCTEXT_NAMESPACE
