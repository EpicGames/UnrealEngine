// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FAvaTransitionEditorViewModel;
class FNamePermissionList;
class SWidget;
class UToolMenu;
struct FReadOnlyAssetEditorCustomization;

class FAvaTransitionToolbar : public TSharedFromThis<FAvaTransitionToolbar>
{
public:
	explicit FAvaTransitionToolbar(FAvaTransitionEditorViewModel& InOwner);

	static FName GetTreeToolbarName()
	{
		return TEXT("AvaTransitionTreeToolbar");
	}

	void SetReadOnlyProfileName(FName InToolMenuToolbarName, FName InReadOnlyProfileName);

	void ExtendEditorToolbar(UToolMenu* InToolbarMenu);

	void ExtendTreeToolbar(UToolMenu* InToolbarMenu);

	void SetupReadOnlyCustomization(FReadOnlyAssetEditorCustomization& InReadOnlyCustomization);

	TSharedRef<SWidget> GenerateTreeToolbarWidget();

private:
	void ApplyReadOnlyPermissionList(const FNamePermissionList& InPermissionList);

	FAvaTransitionEditorViewModel& Owner;

	FName ToolMenuToolbarName;
	FName ReadOnlyProfileName;
};
