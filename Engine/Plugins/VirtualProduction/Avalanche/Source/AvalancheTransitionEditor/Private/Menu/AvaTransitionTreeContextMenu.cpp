// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionTreeContextMenu.h"
#include "AvaTransitionCommands.h"
#include "AvaTransitionMenuContext.h"
#include "Framework/Commands/GenericCommands.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "ViewModels/AvaTransitionViewModelSharedData.h"

#define LOCTEXT_NAMESPACE "AvaTransitionTreeContextMenu"

TSharedRef<SWidget> FAvaTransitionTreeContextMenu::GenerateTreeContextMenuWidget()
{
	UToolMenus* const ToolMenus = UToolMenus::Get();
	check(ToolMenus);

	const FName ContextMenuName = GetTreeContextMenuName();

	if (!ToolMenus->IsMenuRegistered(ContextMenuName))
	{
		UToolMenu* const Menu = ToolMenus->RegisterMenu(ContextMenuName, NAME_None, EMultiBoxType::Menu);
		Menu->AddDynamicSection("PopulateContextMenu", FNewToolMenuDelegate::CreateStatic([](UToolMenu* InToolMenu)
		{
			if (InToolMenu)
			{
				if (UAvaTransitionMenuContext* MenuContext = InToolMenu->FindContext<UAvaTransitionMenuContext>())
				{
					if (TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = MenuContext->GetEditorViewModel())
					{
						EditorViewModel->GetContextMenu()->ExtendTreeContextMenu(InToolMenu);
					}
				}
			}
		}));
	}

	TSharedPtr<FExtender> Extender;

	UAvaTransitionMenuContext* const ContextObject = NewObject<UAvaTransitionMenuContext>();
	ContextObject->SetEditorViewModel(StaticCastSharedRef<FAvaTransitionEditorViewModel>(Owner.AsShared()));

	FToolMenuContext Context(Owner.GetCommandList(), Extender, ContextObject);
	return ToolMenus->GenerateWidget(ContextMenuName, Context);
}

void FAvaTransitionTreeContextMenu::ExtendTreeContextMenu(UToolMenu* InContextMenu)
{
	if (!InContextMenu)
	{
		return;
	}

	const bool bReadOnly = Owner.GetSharedData()->IsReadOnly();

	const FAvaTransitionEditorCommands& TransitionEditorCommands = FAvaTransitionEditorCommands::Get();

	// State Actions (only if not read-only) 
	if (!bReadOnly)
	{
		FToolMenuSection& StateActionSection = InContextMenu->FindOrAddSection(TEXT("StateActions"), LOCTEXT("StateActions", "State Actions"));

		StateActionSection.AddMenuEntry(TransitionEditorCommands.AddComment);
		StateActionSection.AddMenuEntry(TransitionEditorCommands.RemoveComment);

		StateActionSection.AddSeparator(NAME_None);

		StateActionSection.AddSubMenu(TEXT("AddState")
			, LOCTEXT("AddState", "Add State")
			, FText::GetEmpty()
			, FNewToolMenuDelegate::CreateSPLambda(this,
				[this](UToolMenu* InMenu)
				{
					if (InMenu)
					{
						const FAvaTransitionEditorCommands& TransitionEditorCommands = FAvaTransitionEditorCommands::Get();

						FToolMenuSection& AddStateSection = InMenu->FindOrAddSection(TEXT("AddStateSection"));
						AddStateSection.AddMenuEntry(TransitionEditorCommands.AddSiblingState);
						AddStateSection.AddMenuEntry(TransitionEditorCommands.AddChildState);
					}
				})
			, /*bInOpenSubMenuOnClick*/false);

		StateActionSection.AddMenuEntry(TransitionEditorCommands.EnableStates);
	}

	const FGenericCommands& GenericCommands = FGenericCommands::Get();

	FToolMenuSection& GenericActionSection = InContextMenu->FindOrAddSection(TEXT("GenericActions"), LOCTEXT("GenericActions", "Generic Actions"));

	if (bReadOnly)
	{
		GenericActionSection.AddMenuEntry(GenericCommands.Copy);
	}
	else
	{
		GenericActionSection.AddMenuEntry(GenericCommands.Cut);
		GenericActionSection.AddMenuEntry(GenericCommands.Copy);
		GenericActionSection.AddMenuEntry(GenericCommands.Paste);
		GenericActionSection.AddMenuEntry(GenericCommands.Duplicate);

		GenericActionSection.AddSeparator(NAME_None);

		GenericActionSection.AddMenuEntry(GenericCommands.Delete);
	}
}

#undef LOCTEXT_NAMESPACE
