// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FAvaTransitionEditorViewModel;
class SWidget;
class UToolMenu;

class FAvaTransitionTreeContextMenu : public TSharedFromThis<FAvaTransitionTreeContextMenu>
{
public:
	explicit FAvaTransitionTreeContextMenu(FAvaTransitionEditorViewModel& InOwner)
		: Owner(InOwner)
	{
	}

	static FName GetTreeContextMenuName()
	{
		return TEXT("AvaTransitionTreeContextMenu");
	}

	TSharedRef<SWidget> GenerateTreeContextMenuWidget();

private:
	void ExtendTreeContextMenu(UToolMenu* InContextMenu);

	FAvaTransitionEditorViewModel& Owner;
};
