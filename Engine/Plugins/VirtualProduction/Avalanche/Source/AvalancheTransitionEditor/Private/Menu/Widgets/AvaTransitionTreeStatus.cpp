// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionTreeStatus.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "ScopedTransaction.h"
#include "Styling/AvaTransitionEditorStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "SAvaTransitionTreeStatus"

namespace UE::AvaTransitionEditor::Private
{
	// Image that always display the brush with no Draw Effects even if disabled
	class SStatusImage : public SImage
	{
		//~ Begin SWidget
		virtual int32 OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InCullingRect, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const override
		{
			constexpr bool bIsParentEnabled = true;
			return SImage::OnPaint(InArgs, InAllottedGeometry, InCullingRect, OutDrawElements, InLayerId, InWidgetStyle, bIsParentEnabled);
		}
		//~ End SWidget
	};
}

void SAvaTransitionTreeStatus::Construct(const FArguments& InArgs, IAvaTransitionBehavior* InTransitionBehavior)
{
	TransitionBehaviorWeak = InTransitionBehavior;

	ChildSlot
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(2.f, 0.f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.IsEnabled(this, &SAvaTransitionTreeStatus::IsStatusButtonEnabled)
			.OnClicked(this, &SAvaTransitionTreeStatus::OnStatusButtonClicked)
			.ToolTipText(this, &SAvaTransitionTreeStatus::GetStatusTooltipText)
			.IsFocusable(false)
			[
				SNew(UE::AvaTransitionEditor::Private::SStatusImage)
				.Image(this, &SAvaTransitionTreeStatus::GetStatusIcon)
			]
		];
}

bool SAvaTransitionTreeStatus::IsStatusButtonEnabled() const
{
	return GetStatus() == EStatus::Disabled;
}

FReply SAvaTransitionTreeStatus::OnStatusButtonClicked()
{
	IAvaTransitionBehavior* const TransitionBehavior = TransitionBehaviorWeak.Get();
	if (TransitionBehavior && !TransitionBehavior->IsEnabled())
	{
		FScopedTransaction Transaction(LOCTEXT("EnableTransitionTree", "Enable Transition Tree"));
		TransitionBehavior->AsUObject().Modify();
		TransitionBehavior->SetEnabled(true);
	}
	return FReply::Handled();
}

SAvaTransitionTreeStatus::EStatus SAvaTransitionTreeStatus::GetStatus() const
{
	IAvaTransitionBehavior* const TransitionBehavior = TransitionBehaviorWeak.Get();
	if (!TransitionBehavior)
	{
		return EStatus::Invalid;
	}

	return TransitionBehavior->IsEnabled()
		? EStatus::Enabled
		: EStatus::Disabled;
}

FText SAvaTransitionTreeStatus::GetStatusText(EStatus InStatus) const
{
	switch (InStatus)
	{
	case EStatus::Invalid:
		return LOCTEXT("InvalidText", "invalid");

	case EStatus::Disabled:
		return LOCTEXT("DisabledText", "disabled");

	case EStatus::Enabled:
		return LOCTEXT("EnabledText", "enabled");
	}

	checkNoEntry()
	return FText::GetEmpty();
}

FText SAvaTransitionTreeStatus::GetStatusTooltipText() const
{
	EStatus Status = GetStatus();

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("Status"), GetStatusText(Status));

	if (Status == EStatus::Disabled)
	{
		if (IsEnabled())
		{
			Arguments.Add(TEXT("Suggestion"), LOCTEXT("DisableSuggestionLabel_ReadWrite", "\nThe tree will never run. Click on this status icon to enable the transition tree."));
		}
		else
		{
			Arguments.Add(TEXT("Suggestion"), LOCTEXT("DisableSuggestionLabel_ReadOnly", "\nThe tree will never run."));
		}
	}
	else
	{
		Arguments.Add(TEXT("Suggestion"), FText::GetEmpty());
	}

	return FText::Format(LOCTEXT("StatusTooltipTextFormat", "Tree is {Status}.{Suggestion}"), Arguments);
}

const FSlateBrush* SAvaTransitionTreeStatus::GetStatusIcon() const
{
	EStatus Status = GetStatus();
	switch (Status)
	{
	case EStatus::Invalid:
		return FAppStyle::Get().GetBrush(TEXT("Icons.ErrorWithColor"));

	case EStatus::Disabled:
		return FAppStyle::Get().GetBrush(TEXT("Icons.WarningWithColor"));

	case EStatus::Enabled:
		return FAppStyle::Get().GetBrush(TEXT("Icons.SuccessWithColor"));
	}

	checkNoEntry();
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
