// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakInterfacePtr.h"
#include "Widgets/SCompoundWidget.h"

class IAvaTransitionBehavior;

class SAvaTransitionTreeStatus : public SCompoundWidget
{
	enum class EStatus : uint8
	{
		Invalid,
		Disabled,
		Enabled,
	};

public:
	SLATE_BEGIN_ARGS(SAvaTransitionTreeStatus) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, IAvaTransitionBehavior* InTransitionBehavior);

private:
	bool IsStatusButtonEnabled() const;

	FReply OnStatusButtonClicked();

	EStatus GetStatus() const;

	FText GetStatusText(EStatus InStatus) const;

	FText GetStatusTooltipText() const;

	const FSlateBrush* GetStatusIcon() const;

	TWeakInterfacePtr<IAvaTransitionBehavior> TransitionBehaviorWeak;
};
