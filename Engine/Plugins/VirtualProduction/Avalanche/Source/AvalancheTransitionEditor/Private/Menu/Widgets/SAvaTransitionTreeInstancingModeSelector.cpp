// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaTransitionTreeInstancingModeSelector.h"
#include "AvaTransitionEnums.h"
#include "Behavior/IAvaTransitionBehavior.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SComboBox.h"

#define LOCTEXT_NAMESPACE "AvaTransitionTreeInstancingModeSelector"

void SAvaTransitionTreeInstancingModeSelector::Construct(const FArguments& InArgs, IAvaTransitionBehavior* InTransitionBehavior)
{
	UpdateItems();
	TransitionBehaviorWeak = InTransitionBehavior;

	ChildSlot
	[
		SAssignNew(Combo, SComboBox<FName>)
		.InitiallySelectedItem(GetItemFromProperty())
		.OptionsSource(&Items)
		.OnGenerateWidget(this, &SAvaTransitionTreeInstancingModeSelector::GenerateWidget)
		.OnSelectionChanged(this, &SAvaTransitionTreeInstancingModeSelector::HandleSelectionChanged)
		[
			SNew(STextBlock)
			.Text(this, &SAvaTransitionTreeInstancingModeSelector::GetDisplayTextFromProperty)
		]
	];
}
	
TSharedRef<SWidget> SAvaTransitionTreeInstancingModeSelector::GenerateWidget(FName InItem) const
{
	return SNew(STextBlock)
		.Text(GetDisplayTextFromItem(InItem));
}

void SAvaTransitionTreeInstancingModeSelector::HandleSelectionChanged(FName InProposedSelection, ESelectInfo::Type InSelectInfo)
{
	if (IAvaTransitionBehavior* TransitionBehavior = TransitionBehaviorWeak.Get())
	{
		FScopedTransaction Transaction(LOCTEXT("SetTransitionTreeInstancingMode", "Set Transition Tree Instancing Mode")); 
		TransitionBehavior->AsUObject().Modify();
		int64 Value = StaticEnum<EAvaTransitionInstancingMode>()->GetValueByName(InProposedSelection); 
		TransitionBehavior->SetInstancingMode(static_cast<EAvaTransitionInstancingMode>(Value));
	}
}

FText SAvaTransitionTreeInstancingModeSelector::GetDisplayTextFromProperty() const
{
	if (const IAvaTransitionBehavior* TransitionBehavior = TransitionBehaviorWeak.Get())
	{
		return StaticEnum<EAvaTransitionInstancingMode>()->GetDisplayNameTextByValue(static_cast<int64>(TransitionBehavior->GetInstancingMode()));
	}
	return FText::GetEmpty();
}

FName SAvaTransitionTreeInstancingModeSelector::GetItemFromProperty() const
{
	if (const IAvaTransitionBehavior* TransitionBehavior = TransitionBehaviorWeak.Get())
	{
		return StaticEnum<EAvaTransitionInstancingMode>()->GetNameByValue(static_cast<int64>(TransitionBehavior->GetInstancingMode()));
	}
	return NAME_None;
}

FText SAvaTransitionTreeInstancingModeSelector::GetDisplayTextFromItem(FName InItem) const
{
	const int64 Value = StaticEnum<EAvaTransitionInstancingMode>()->GetValueByName(InItem);
	return StaticEnum<EAvaTransitionInstancingMode>()->GetDisplayNameTextByValue(Value);
}

void SAvaTransitionTreeInstancingModeSelector::UpdateItems()
{
	const UEnum* InstancingModeEnum = StaticEnum<EAvaTransitionInstancingMode>();

	Items.Empty(InstancingModeEnum->NumEnums() - 1);
	for (int32 Index = 0; Index < InstancingModeEnum->NumEnums() - 1; ++Index)
	{
		Items.Add(InstancingModeEnum->GetNameByIndex(Index));
	}
}

#undef LOCTEXT_NAMESPACE