// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakInterfacePtr.h"
#include "Widgets/SCompoundWidget.h"

class IAvaTransitionBehavior;
template <typename OptionType> class SComboBox;

class SAvaTransitionTreeInstancingModeSelector : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaTransitionTreeInstancingModeSelector){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, IAvaTransitionBehavior* InTransitionBehavior);

protected:
	TSharedRef<SWidget> GenerateWidget(FName InItem) const;
	void HandleSelectionChanged(FName InProposedSelection, ESelectInfo::Type InSelectInfo);
	FText GetDisplayTextFromProperty() const;
	FName GetItemFromProperty() const;
	FText GetDisplayTextFromItem(FName InItem) const;
	void UpdateItems();

protected:
	TWeakInterfacePtr<IAvaTransitionBehavior> TransitionBehaviorWeak;
	TSharedPtr<SComboBox<FName>> Combo;
	TArray<FName> Items;
};
