// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "StateTreePropertyBindings.h"
#include "UObject/Object.h"
#include "AvaTransitionStateExportData.generated.h"

UCLASS(Hidden)
class UAvaTransitionStateExportData : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TArray<FStateTreePropertyPathBinding> Bindings;
};
