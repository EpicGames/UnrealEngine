// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionStateSerializer.h"
#include "AvaTransitionStateExportData.h"
#include "AvaTransitionTreeEditorData.h"
#include "Exporters/Exporter.h"
#include "Misc/StringOutputDevice.h"
#include "Factories.h"
#include "StateTreeState.h"
#include "UnrealExporter.h"

namespace UE::AvaTransitionEditor::Private
{
	class FAvaStateTreeStateTextFactory : public FCustomizableTextObjectFactory
	{
	public:
		FAvaStateTreeStateTextFactory()
			: FCustomizableTextObjectFactory(GWarn)
		{
		}

		virtual bool CanCreateClass(UClass* InObjectClass, bool& bOmitSubObjs) const override
		{
			return InObjectClass->IsChildOf(UStateTreeState::StaticClass())
				|| InObjectClass->IsChildOf(UAvaTransitionStateExportData::StaticClass());
		}

		virtual void ProcessConstructedObject(UObject* InNewObject) override
		{
			if (UStateTreeState* State = Cast<UStateTreeState>(InNewObject))
			{
				States.Add(State);
			}
			else if (UAvaTransitionStateExportData* CopyData = Cast<UAvaTransitionStateExportData>(InNewObject))
			{
				StateCopyData = CopyData;
			}
		}

		TArray<UStateTreeState*> States;
		UAvaTransitionStateExportData* StateCopyData = nullptr;
	};

	void CollectCopyData(UAvaTransitionTreeEditorData& InEditorData, const UStateTreeState* InState, UAvaTransitionStateExportData& InCopyData)
	{
		if (!InState)
		{
			return;
		}

		InEditorData.VisitStateNodes(*InState,
			[&InEditorData, &InCopyData](const UStateTreeState*, const FStateTreeBindableStructDesc& InDesc, const FStateTreeDataView)
			{
				TArray<const FPropertyBindingBinding*> NodeBindings;
				InEditorData.GetPropertyEditorBindings()->GetBindingsFor(InDesc.ID, NodeBindings);
				for (const FPropertyBindingBinding* Binding : NodeBindings)
				{
					const FStateTreePropertyPathBinding* StateTreeBinding = static_cast<const FStateTreePropertyPathBinding*>(Binding);
					InCopyData.Bindings.Add(*StateTreeBinding);
				}
				return EStateTreeVisitor::Continue;
			});

		for (const UStateTreeState* ChildState : InState->Children)
		{
			CollectCopyData(InEditorData, ChildState, InCopyData);
		}
	}

	void FixPastedNodes(TArrayView<FStateTreeEditorNode> InNodes, TMap<FGuid, FGuid>& InOldToNewIdMap, TArray<FStateTreeStateLink*>& OutLinks)
	{
		auto CollectStateLinks = [](const UStruct* InStruct, void* InStructMemory, TArray<FStateTreeStateLink*>& OutLinks)
		{
			UScriptStruct* StateLinkStruct = TBaseStructure<FStateTreeStateLink>::Get();

			for (const TPair<const FStructProperty*, const void*>& Pair : TPropertyValueRange<FStructProperty>(InStruct, InStructMemory))
			{
				if (Pair.Key && Pair.Key->Struct == StateLinkStruct)
				{
					FStateTreeStateLink* StateLink = static_cast<FStateTreeStateLink*>(const_cast<void*>(Pair.Value));
					OutLinks.Add(StateLink);
				}
			}
		};

		for (FStateTreeEditorNode& Node : InNodes)
		{
			Node.ID = InOldToNewIdMap.Emplace(Node.ID, FGuid::NewGuid());

			if (Node.Node.IsValid())
			{
				CollectStateLinks(Node.Node.GetScriptStruct(), Node.Node.GetMutableMemory(), OutLinks);
			}
			if (Node.Instance.IsValid())
			{
				CollectStateLinks(Node.Instance.GetScriptStruct(), Node.Instance.GetMutableMemory(), OutLinks);
			}
			if (Node.InstanceObject)
			{
				CollectStateLinks(Node.InstanceObject->GetClass(), Node.InstanceObject, OutLinks);
			}
		}
	}
}

FString FAvaTransitionStateSerializer::ExportText(UAvaTransitionTreeEditorData& InEditorData, TConstArrayView<UStateTreeState*> InStates)
{
	if (InStates.IsEmpty())
	{
		return FString();
	}

	// Clear the mark state for saving.
	UnMarkAllObjects(static_cast<EObjectMark>(OBJECTMARK_TagExp | OBJECTMARK_TagImp));

	FStringOutputDevice StringOutputDevice;

	const FExportObjectInnerContext Context;

	UAvaTransitionStateExportData* CopyData = NewObject<UAvaTransitionStateExportData>();
	check(CopyData);

	for (UStateTreeState* State : InStates)
	{
		if (State)
		{
			UObject* ThisOuter = State->GetOuter();
			UExporter::ExportToOutputDevice(&Context, State, nullptr, StringOutputDevice, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, ThisOuter);
			UE::AvaTransitionEditor::Private::CollectCopyData(InEditorData, State, *CopyData);
		}
	}

	UExporter::ExportToOutputDevice(&Context, CopyData, nullptr, StringOutputDevice, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false);
	return *StringOutputDevice;
}

bool FAvaTransitionStateSerializer::ImportText(const FString& InText
	, UAvaTransitionTreeEditorData& InEditorData
	, UStateTreeState* InState
	, TArray<UStateTreeState*>& OutNewStates
	, UObject*& OutParent)
{
	using namespace UE::AvaTransitionEditor;

	if (InText.IsEmpty())
	{
		return false;
	}

	TArray<TObjectPtr<UStateTreeState>>* Children;

	UStateTreeState* const ParentState = InState
		? InState->Parent.Get()
		: nullptr;

	if (ParentState)
	{
		OutParent = ParentState;
		Children  = &ParentState->Children;
	}
	else
	{
		OutParent = &InEditorData;
		Children  = &InEditorData.SubTrees;
	}

	OutParent->Modify();

	Private::FAvaStateTreeStateTextFactory Factory;
	Factory.ProcessBuffer(OutParent, RF_Transactional, InText);

	int32 InsertIndex = INDEX_NONE;
	if (InState)
	{
		// Find the State, and set the Insert index one after it.
		InsertIndex = Children->Find(InState);
	}

	if (InsertIndex == INDEX_NONE)
	{
		InsertIndex = Children->Num();
	}
	else
	{
		++InsertIndex;
	}

	Children->Insert(Factory.States, InsertIndex);

	TArray<FStateTreeStateLink*> Links;

	// Map containing old guids to their new guids (of states, parameters, etc)
	TMap<FGuid, FGuid> OldToNewIdMap;
	OldToNewIdMap.Reserve(Factory.States.Num() * 2);

	TArray<UStateTreeState*> StatesToFix(Factory.States);

	// Fix pasted states' ids
	while (!StatesToFix.IsEmpty())
	{
		UStateTreeState* State = StatesToFix.Pop();
		if (!State)
		{
			continue;
		}

		State->Modify();
		State->Parent = ParentState;

		State->ID = OldToNewIdMap.Emplace(State->ID, FGuid::NewGuid());
		State->Parameters.ID = OldToNewIdMap.Emplace(State->Parameters.ID, FGuid::NewGuid());;

		if (State->Type == EStateTreeStateType::Linked)
		{
			Links.Emplace(&State->LinkedSubtree);
		}

		Private::FixPastedNodes(MakeArrayView(&State->SingleTask, 1), OldToNewIdMap, Links);
		Private::FixPastedNodes(State->Tasks, OldToNewIdMap, Links);
		Private::FixPastedNodes(State->EnterConditions, OldToNewIdMap, Links);

		for (FStateTreeTransition& Transition : State->Transitions)
		{
			// Transition Ids are not used by nodes so no need to add to the Old to New Id Map
			Transition.ID = FGuid::NewGuid();

			Private::FixPastedNodes(Transition.Conditions, OldToNewIdMap, Links);
			Links.Emplace(&Transition.State);
		}

		StatesToFix.Append(State->Children);
	}

	if (Factory.StateCopyData)
	{
		for (const TPair<FGuid, FGuid>& OldToNewPair : OldToNewIdMap)
		{
			const FGuid& OldId = OldToNewPair.Key;
			const FGuid& NewId = OldToNewPair.Value;

			// Copy property bindings for the duplicated states.
			for (const FStateTreePropertyPathBinding& Binding : Factory.StateCopyData->Bindings)
			{
				if (Binding.GetTargetPath().GetStructID() == OldId)
				{
					FPropertyBindingPath TargetPath(Binding.GetTargetPath());
					TargetPath.SetStructID(NewId);

					FPropertyBindingPath SourcePath(Binding.GetSourcePath());
					if (const FGuid* NewSourceID = OldToNewIdMap.Find(Binding.GetSourcePath().GetStructID()))
					{
						SourcePath.SetStructID(*NewSourceID);
					}

					InEditorData.GetPropertyEditorBindings()->AddBinding(SourcePath, TargetPath);
				}
			}
		}
	}

	// Patch IDs in state links.
	for (FStateTreeStateLink* Link : Links)
	{
		if (const FGuid* NewId = OldToNewIdMap.Find(Link->ID))
		{
			Link->ID = *NewId;
		}
	}

	OutNewStates = MoveTemp(Factory.States);
	return true;
}
