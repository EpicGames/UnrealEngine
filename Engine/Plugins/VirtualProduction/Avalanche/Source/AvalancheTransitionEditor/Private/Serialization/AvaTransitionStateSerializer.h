// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"

class FString;
class UAvaTransitionTreeEditorData;
class UStateTreeState;

class FAvaTransitionStateSerializer
{
public:
	static FString ExportText(UAvaTransitionTreeEditorData& InEditorData, TConstArrayView<UStateTreeState*> InStates);

	static bool ImportText(const FString& InText, UAvaTransitionTreeEditorData& InEditorData, UStateTreeState* InState, TArray<UStateTreeState*>& OutNewStates, UObject*& OutParent);
};
