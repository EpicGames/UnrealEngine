// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/AvaTransitionEditorSettings.h"
#include "AvaTransitionTree.h"
#include "AvaTransitionTreeEditorData.h"

UAvaTransitionEditorSettings::UAvaTransitionEditorSettings()
{
	CategoryName = TEXT("Motion Design");
	SectionName  = TEXT("Transition Logic");
	DefaultTemplate = FSoftObjectPath(TEXT("/Avalanche/TransitionLogic/TL_TemplateTree.TL_TemplateTree"));
}

UAvaTransitionTreeEditorData* UAvaTransitionEditorSettings::LoadDefaultTemplateEditorData() const
{
	if (UAvaTransitionTree* TemplateTree = DefaultTemplate.LoadSynchronous())
	{
		return Cast<UAvaTransitionTreeEditorData>(TemplateTree->EditorData);
	}
	return nullptr;
}

void UAvaTransitionEditorSettings::ToggleCreateTransitionLogicDefaultScene()
{
	bCreateTransitionLogicDefaultScene = !bCreateTransitionLogicDefaultScene;
}
