// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionEditorStyle.h"
#include "AvaTransitionTree.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Layout/Margin.h"
#include "Misc/Paths.h"
#include "StateTreeEditorStyle.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Textures/SlateIcon.h"

FAvaTransitionEditorStyle::FAvaTransitionEditorStyle()
	: FSlateStyleSet(TEXT("AvaTransitionEditor"))
{
	ParentStyleName = FStateTreeEditorStyle::Get().GetStyleSetName();

	const FVector2f Icon16(16.f);
	const FVector2f Icon20(20.f);
	const FVector2f Icon64(64.f);

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	ContentRootDir = FPaths::EngineContentDir() / TEXT("Editor/Slate");
	CoreContentRootDir = FPaths::EngineContentDir() / TEXT("Slate");

	// Re-use the Behavior Tree Icon
	Set("ClassIcon.AvaTransitionTree"     , new IMAGE_BRUSH("Icons/AssetIcons/BehaviorTree_16x", Icon16));
	Set("ClassThumbnail.AvaTransitionTree", new IMAGE_BRUSH("Icons/AssetIcons/BehaviorTree_64x", Icon64));

	Set("Throbber.CircleChunk", new CORE_IMAGE_BRUSH("Common/Throbber_Piece", FVector2f(1.5f)));

	// Editor Commands
	Set("AvaTransitionEditor.AddSiblingState"       , new CORE_IMAGE_BRUSH_SVG("Starship/Common/plus"     , Icon20));
	Set("AvaTransitionEditor.AddChildState"         , new CORE_IMAGE_BRUSH_SVG("Starship/Common/plus"     , Icon20));
	Set("AvaTransitionEditor.ImportTransitionTree"  , new CORE_IMAGE_BRUSH_SVG("Starship/Common/import_20", Icon20));
	Set("AvaTransitionEditor.ReimportTransitionTree", new CORE_IMAGE_BRUSH_SVG("Starship/Common/import_20", Icon20));
	Set("AvaTransitionEditor.ExportTransitionTree"  , new CORE_IMAGE_BRUSH_SVG("Starship/Common/export_20", Icon20));
	Set("AvaTransitionEditor.ToggleDebug"           , new IMAGE_BRUSH_SVG("Starship/Common/Debug"         , Icon20));

	Set("DebugIndicatorBorder", new BOX_BRUSH("Images/NamespaceBorder", FMargin(0.25f)));

	FLinearColor OperandBoxColor = FStyleColors::AccentGreen.GetSpecifiedColor().Desaturate(0.3f);
	OperandBoxColor.A = 1.f;
	Set("OperandBox", new FSlateRoundedBoxBrush(OperandBoxColor, 4.0f));

	SetupFonts();

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FAvaTransitionEditorStyle::~FAvaTransitionEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FLinearColor FAvaTransitionEditorStyle::LerpColorSRGB(const FLinearColor& InA, const FLinearColor& InB, float InAlpha)
{
	const FColor A = InA.ToFColorSRGB();
	const FColor B = InB.ToFColorSRGB();

	return FLinearColor(FColor(
		static_cast<uint8>(FMath::RoundToInt(static_cast<float>(A.R) * (1.f - InAlpha) + static_cast<float>(B.R) * InAlpha)),
		static_cast<uint8>(FMath::RoundToInt(static_cast<float>(A.G) * (1.f - InAlpha) + static_cast<float>(B.G) * InAlpha)),
		static_cast<uint8>(FMath::RoundToInt(static_cast<float>(A.B) * (1.f - InAlpha) + static_cast<float>(B.B) * InAlpha)),
		static_cast<uint8>(FMath::RoundToInt(static_cast<float>(A.A) * (1.f - InAlpha) + static_cast<float>(B.A) * InAlpha))));
}

FSlateIcon FAvaTransitionEditorStyle::ParseIcon(FName InIconName)
{
	FString IconPath = InIconName.ToString();

	constexpr int32 NumOfIconPathNames = 4;

	FName IconPathNames[NumOfIconPathNames] = {
		NAME_None, // StyleSetName
		NAME_None, // StyleName
		NAME_None, // SmallStyleName
		NAME_None  // StatusOverlayStyleName
	};

	int32 NameIndex = 0;
	while (!IconPath.IsEmpty() && NameIndex < NumOfIconPathNames)
	{
		FString Left;
		FString Right;

		if (!IconPath.Split(TEXT("|"), &Left, &Right))
		{
			Left = IconPath;
		}

		IconPathNames[NameIndex] = FName(*Left);

		NameIndex++;
		IconPath = Right;
	}

	return FSlateIcon(IconPathNames[0], IconPathNames[1], IconPathNames[2], IconPathNames[3]);
}

void FAvaTransitionEditorStyle::SetupFonts()
{
	const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

	const FTextBlockStyle TitleText = FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Bold", 12));

	Set("StateTree.State.Title", FTextBlockStyle(TitleText)
		.SetColorAndOpacity(FLinearColor(FColor(230, 230, 230, 205))));

	Set("StateTree.State.Title.Bold", FTextBlockStyle(TitleText)
		.SetColorAndOpacity(FLinearColor(FColor(230, 230, 230, 230))));

	Set("StateTree.State.Title.Subdued", FTextBlockStyle(TitleText)
		.SetColorAndOpacity(FLinearColor(FColor(230, 230, 230, 180))));

	Set("StateTree.State.Operand", FTextBlockStyle(NormalText)
		.SetFont(FAppStyle::GetFontStyle(TEXT("PropertyWindow.BoldFont")))
		.SetFontSize(10));
}
