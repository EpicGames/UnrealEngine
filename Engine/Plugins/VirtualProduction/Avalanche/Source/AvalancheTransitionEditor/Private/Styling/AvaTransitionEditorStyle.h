// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

struct FSlateIcon;

class FAvaTransitionEditorStyle final : public FSlateStyleSet
{
public:
	static FAvaTransitionEditorStyle& Get()
	{
		static FAvaTransitionEditorStyle Instance;
		return Instance;
	}

	FAvaTransitionEditorStyle();
	virtual ~FAvaTransitionEditorStyle() override;

	static FLinearColor LerpColorSRGB(const FLinearColor& InA, const FLinearColor& InB, float InAlpha);

	static FSlateIcon ParseIcon(FName InIconName);

private:
	void SetupFonts();
};
