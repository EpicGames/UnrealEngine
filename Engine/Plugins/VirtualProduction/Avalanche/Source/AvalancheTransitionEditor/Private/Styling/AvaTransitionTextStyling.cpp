// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionTextStyling.h"
#include "Framework/Text/SlateTextRun.h"
#include "Styling/SlateTypes.h"

TSharedRef<FAvaTransitionTextStyleDecorator> FAvaTransitionTextStyleDecorator::Create(FString InName, const FTextBlockStyle& InTextStyle)
{
	TSharedRef<FAvaTransitionTextStyleDecorator> TextStyleDecorator = MakeShared<FAvaTransitionTextStyleDecorator>();
	TextStyleDecorator->DecoratorName = MoveTemp(InName);
	TextStyleDecorator->TextStyle = InTextStyle;
	return TextStyleDecorator;
}

bool FAvaTransitionTextStyleDecorator::Supports(const FTextRunParseResults& InRunInfo, const FString& InText) const
{
	return InRunInfo.Name == DecoratorName;
}

TSharedRef<ISlateRun> FAvaTransitionTextStyleDecorator::Create(const TSharedRef<class FTextLayout>& InTextLayout, const FTextRunParseResults& InRunParseResult, const FString& InOriginalText, const TSharedRef<FString>& InModelText, const ISlateStyle* InStyle)
{
	FRunInfo RunInfo(InRunParseResult.Name);
	for (const TPair<FString, FTextRange>& Pair : InRunParseResult.MetaData)
	{
		RunInfo.MetaData.Add(Pair.Key, InOriginalText.Mid(Pair.Value.BeginIndex, Pair.Value.EndIndex - Pair.Value.BeginIndex));
	}

	FString Run;
	if (!InRunParseResult.ContentRange.IsEmpty())
	{
		Run = InOriginalText.Mid(InRunParseResult.ContentRange.BeginIndex, InRunParseResult.ContentRange.Len());
	}
	else
	{
		// Handles the case when the decorator name is empty (matches the runs without any tags).
		Run = InOriginalText.Mid(InRunParseResult.OriginalRange.BeginIndex, InRunParseResult.OriginalRange.Len());
	}

	// Remove any formatting in between (e.g. when using nested tags).
	int32 TagStartIndex = INDEX_NONE;
	for (int32 Index = 0; Index < Run.Len(); Index++)
	{
		if (TagStartIndex == INDEX_NONE)
		{
			if (Run[Index] == TEXT('<'))
			{
				TagStartIndex = Index;
			}
		}
		else
		{
			if (Run[Index] == TEXT('>'))
			{
				Run.RemoveAt(TagStartIndex, Index - TagStartIndex + 1);
				Index = TagStartIndex;
				TagStartIndex = INDEX_NONE;
			}
		}
	}

	InModelText->Append(Run);
	return FSlateTextRun::Create(RunInfo, InModelText, TextStyle);
}
