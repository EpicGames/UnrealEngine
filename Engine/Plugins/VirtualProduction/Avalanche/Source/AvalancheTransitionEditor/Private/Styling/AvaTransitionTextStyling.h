// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Text/ITextDecorator.h"
#include "Styling/SlateTypes.h"

/**
 * Simple decorator for SRichTextBlock, which allows to map simple/shorthand tags to specific style names.
 * The decorator also strips away any tags, e.g. in case the initial text came with nested tags.
 *
 * Sourced from FTextStyleDecorator
 */
class FAvaTransitionTextStyleDecorator : public ITextDecorator
{
public:
	static TSharedRef<FAvaTransitionTextStyleDecorator> Create(FString InName, const FTextBlockStyle& InTextStyle);

	//~ Begin ITextDecorator
	virtual bool Supports(const FTextRunParseResults& InRunInfo, const FString& InText) const override;
	virtual TSharedRef<ISlateRun> Create(const TSharedRef<FTextLayout>& InTextLayout,const FTextRunParseResults& InRunParseResult,const FString& InOriginalText,const TSharedRef<FString>& InModelText,const ISlateStyle* InStyle) override;
	//~ End ITextDecorator

private:
	/** Name of this decorator */
	FString DecoratorName;

	/** Color of this decorator */
	FTextBlockStyle TextStyle;
};
