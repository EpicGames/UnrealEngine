// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionWidgetStyling.h"
#include "AvaTransitionEditorStyle.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/ITextDecorator.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

FSlateWidgetRun::FWidgetRunInfo FAvaTransitionWidgetStyling::CreateOperandWidget(const FTextRunInfo& InRunInfo, const ISlateStyle* InStyle)
{
	FSlateColor BackgroundColor(FStyleColors::AccentGreen);
	if (const FString* ColorString = InRunInfo.MetaData.Find(TEXT("color")))
	{
		BackgroundColor = FAppStyle::GetSlateColor(**ColorString);
	}

	const FTextBlockStyle& OperandStyle = FAvaTransitionEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.State.Operand");
	const FTextBlockStyle& TitleStyle = FAvaTransitionEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.State.Title");

	constexpr int32 VPadding = 2;

	TSharedRef<SWidget> Widget = SNew(SBorder)
		.BorderImage(FAvaTransitionEditorStyle::Get().GetBrush("OperandBox"))
		.BorderBackgroundColor(BackgroundColor)
		.Padding(3, VPadding)
		[
			SNew(STextBlock)
			.Text(InRunInfo.Content)
			.TextStyle(&OperandStyle)
			.ColorAndOpacity(FLinearColor::White)
		];

	TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	int16 Baseline = FontMeasure->GetBaseline(OperandStyle.Font);
	int16 TitleBaseline = FontMeasure->GetBaseline(TitleStyle.Font);

	// ensure the resulting widget remains at the center of the widget
	return FSlateWidgetRun::FWidgetRunInfo(Widget, (TitleBaseline - Baseline) - VPadding);
}
