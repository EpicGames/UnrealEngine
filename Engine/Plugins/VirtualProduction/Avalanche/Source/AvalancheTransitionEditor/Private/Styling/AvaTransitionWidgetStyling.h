// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Text/SlateWidgetRun.h"

class ISlateStyle;
struct FTextRunInfo;

struct FAvaTransitionWidgetStyling
{
	static FSlateWidgetRun::FWidgetRunInfo CreateOperandWidget(const FTextRunInfo& InRunInfo, const ISlateStyle* InStyle);
};
