// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionCompilerResultsTabFactory.h"
#include "Styling/SlateIconFinder.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "AvaTransitionCompilerResultsTabFactory"

const FName FAvaTransitionCompilerResultsTabFactory::TabId(TEXT("AvaTransitionCompilerResults"));

FAvaTransitionCompilerResultsTabFactory::FAvaTransitionCompilerResultsTabFactory(const TSharedRef<FAvaTransitionEditor>& InEditor)
	: FAvaTransitionTabFactory(TabId, InEditor)
{
	TabIcon             = FSlateIcon(FAppStyle::GetAppStyleSetName(), "MessageLog.TabIcon");
	TabLabel            = LOCTEXT("TabLabel", "Compiler Results");
	ViewMenuTooltip     = LOCTEXT("ViewMenuTooltip", "Compiler Results");
	ViewMenuDescription = LOCTEXT("ViewMenuDescription", "Compiler Results");
	bIsSingleton        = true;
	ReadOnlyBehavior    = ETabReadOnlyBehavior::Hidden;
}

TSharedRef<SWidget> FAvaTransitionCompilerResultsTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FAvaTransitionEditor> Editor = GetEditor();
	if (!ensure(Editor.IsValid()))
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = Editor->GetEditorViewModel();
	if (!ensure(EditorViewModel.IsValid()))
	{
		return SNullWidget::NullWidget;
	}

	return EditorViewModel->GetCompiler().CreateCompilerResultsWidget();
}

#undef LOCTEXT_NAMESPACE
