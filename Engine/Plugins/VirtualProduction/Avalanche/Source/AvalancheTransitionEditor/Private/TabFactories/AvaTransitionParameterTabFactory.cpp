// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionParameterTabFactory.h"
#include "Styling/SlateIconFinder.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "Views/SAvaTransitionParameterDetails.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "AvaTransitionParameterTabFactory"

const FName FAvaTransitionParameterTabFactory::TabId(TEXT("AvaTransitionParameter"));

FAvaTransitionParameterTabFactory::FAvaTransitionParameterTabFactory(const TSharedRef<FAvaTransitionEditor>& InEditor)
	: FAvaTransitionTabFactory(TabId, InEditor)
{
	TabIcon             = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");
	TabLabel            = LOCTEXT("TabLabel", "Parameters");
	ViewMenuTooltip     = LOCTEXT("ViewMenuTooltip", "State Tree Parameters");
	ViewMenuDescription = LOCTEXT("ViewMenuDescription", "State Tree Parameters");
	bIsSingleton        = true;
	ReadOnlyBehavior    = ETabReadOnlyBehavior::Custom;
}

TSharedRef<SWidget> FAvaTransitionParameterTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FAvaTransitionEditor> Editor = GetEditor();
	if (!ensure(Editor.IsValid()))
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = Editor->GetEditorViewModel();
	if (!ensure(EditorViewModel.IsValid()))
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SAvaTransitionParameterDetails, EditorViewModel.ToSharedRef());
}

#undef LOCTEXT_NAMESPACE
