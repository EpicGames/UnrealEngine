// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionSelectionDetailsTabFactory.h"
#include "AvaTransitionEditorEnums.h"
#include "AvaTypeSharedPointer.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "ViewModels/AvaTransitionViewModelSharedData.h"
#include "Views/SAvaTransitionSelectionDetails.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "AvaTransitionSelectionDetailsTabFactory"

const FName FAvaTransitionSelectionDetailsTabFactory::TabId(TEXT("AvaTransitionSelectionDetails"));

FAvaTransitionSelectionDetailsTabFactory::FAvaTransitionSelectionDetailsTabFactory(const TSharedRef<FAvaTransitionEditor>& InEditor, EAvaTransitionEditorMode InEditorMode)
	: FAvaTransitionTabFactory(TabId, InEditor)
	, EditorMode(InEditorMode)
{
	TabIcon             = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");
	TabLabel            = LOCTEXT("TabLabel", "Details");
	ViewMenuTooltip     = LOCTEXT("ViewMenuTooltip", "Details");
	ViewMenuDescription = LOCTEXT("ViewMenuDescription", "Details");
	bIsSingleton        = true;
	ReadOnlyBehavior    = ETabReadOnlyBehavior::Custom;
}

TSharedRef<SWidget> FAvaTransitionSelectionDetailsTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& InInfo) const
{
	TSharedPtr<FAvaTransitionEditor> Editor = GetEditor();
	if (!ensure(Editor.IsValid()))
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = Editor->GetEditorViewModel();
	if (!ensure(EditorViewModel.IsValid()))
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SAvaTransitionSelectionDetails, EditorViewModel->GetSelection())
		.AdvancedView(EditorMode == EAvaTransitionEditorMode::Advanced)
		.ReadOnly(EditorViewModel->GetSharedData()->IsReadOnly());
}

#undef LOCTEXT_NAMESPACE
