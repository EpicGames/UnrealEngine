// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionTabFactory.h"

FTabSpawnerEntry& FAvaTransitionTabFactory::RegisterTabSpawner(TSharedRef<FTabManager> InTabManager, const FApplicationMode* InCurrentApplicationMode) const
{
	return FWorkflowTabFactory::RegisterTabSpawner(InTabManager, InCurrentApplicationMode)
		.SetReadOnlyBehavior(ReadOnlyBehavior);
}
