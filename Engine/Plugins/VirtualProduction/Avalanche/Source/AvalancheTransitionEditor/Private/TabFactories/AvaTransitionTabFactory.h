// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionEditor.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

/** Base class for all Tab Factories in Motion Design Transition Editor */
class FAvaTransitionTabFactory : public FWorkflowTabFactory
{
public:
	explicit FAvaTransitionTabFactory(FName InTabId, const TSharedRef<FAvaTransitionEditor>& InEditor)
		: FWorkflowTabFactory(InTabId, InEditor)
	{
	}

	TSharedPtr<FAvaTransitionEditor> GetEditor() const
	{
		return StaticCastSharedPtr<FAvaTransitionEditor>(HostingApp.Pin());
	}

protected:
	//~ Begin FWorkflowTabFactory
	virtual FTabSpawnerEntry& RegisterTabSpawner(TSharedRef<FTabManager> InTabManager, const FApplicationMode* InCurrentApplicationMode) const override;
	//~ End FWorkflowTabFactory

	ETabReadOnlyBehavior ReadOnlyBehavior = ETabReadOnlyBehavior::Disabled;
};
