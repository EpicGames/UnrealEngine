// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionTreeDetailsTabFactory.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "Views/SAvaTransitionTreeDetails.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "AvaTransitionTreeDetailsTabFactory"

const FName FAvaTransitionTreeDetailsTabFactory::TabId(TEXT("AvaTransitionTreeDetails"));

FAvaTransitionTreeDetailsTabFactory::FAvaTransitionTreeDetailsTabFactory(const TSharedRef<FAvaTransitionEditor>& InEditor)
	: FAvaTransitionTabFactory(TabId, InEditor)
{
	TabIcon             = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");
	TabLabel            = LOCTEXT("TabLabel", "State Tree");
	ViewMenuTooltip     = LOCTEXT("ViewMenuTooltip", "State Tree Details");
	ViewMenuDescription = LOCTEXT("ViewMenuDescription", "State Tree Details");
	bIsSingleton        = true;
	ReadOnlyBehavior    = ETabReadOnlyBehavior::Custom;
}

TSharedRef<SWidget> FAvaTransitionTreeDetailsTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& InInfo) const
{
	TSharedPtr<FAvaTransitionEditor> Editor = GetEditor();
	if (!ensure(Editor.IsValid()))
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = Editor->GetEditorViewModel();
	if (!ensure(EditorViewModel.IsValid()))
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SAvaTransitionTreeDetails, EditorViewModel.ToSharedRef());
}

#undef LOCTEXT_NAMESPACE
