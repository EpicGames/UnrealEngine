// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionTabFactory.h"

class FAvaTransitionTreeDetailsTabFactory : public FAvaTransitionTabFactory
{
public:
	static const FName TabId;

	explicit FAvaTransitionTreeDetailsTabFactory(const TSharedRef<FAvaTransitionEditor>& InEditor);

	//~ Begin FWorkflowTabFactory
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& InInfo) const override;
	//~ End FWorkflowTabFactory
};
