// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionTreeTabFactory.h"
#include "AvaTransitionTree.h"
#include "AvaTypeSharedPointer.h"
#include "Styling/SlateIconFinder.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "AvaTransitionTreeTabFactory"

const FName FAvaTransitionTreeTabFactory::TabId(TEXT("AvaTransitionTree"));

FAvaTransitionTreeTabFactory::FAvaTransitionTreeTabFactory(const TSharedRef<FAvaTransitionEditor>& InEditor)
	: FAvaTransitionTabFactory(TabId, InEditor)
{
	TabIcon             = FSlateIconFinder::FindIconForClass(UAvaTransitionTree::StaticClass());
	TabLabel            = LOCTEXT("TabLabel", "Transition Tree");
	ViewMenuTooltip     = LOCTEXT("ViewMenuTooltip", "Motion Design Transition Tree");
	ViewMenuDescription = LOCTEXT("ViewMenuDescription", "Motion Design Transition Tree");
	bIsSingleton        = true;
	ReadOnlyBehavior    = ETabReadOnlyBehavior::Custom;
}

TSharedRef<SWidget> FAvaTransitionTreeTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedPtr<FAvaTransitionEditor> Editor = GetEditor();
	if (!ensure(Editor.IsValid()))
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = Editor->GetEditorViewModel();
	if (!ensure(EditorViewModel.IsValid()))
	{
		return SNullWidget::NullWidget;
	}

	return EditorViewModel->GetTreeWidget();
}

#undef LOCTEXT_NAMESPACE
