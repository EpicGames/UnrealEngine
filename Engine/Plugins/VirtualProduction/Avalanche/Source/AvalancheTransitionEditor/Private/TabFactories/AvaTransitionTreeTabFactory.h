// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionTabFactory.h"

class FAvaTransitionTreeTabFactory : public FAvaTransitionTabFactory
{
public:
	static const FName TabId;

	explicit FAvaTransitionTreeTabFactory(const TSharedRef<FAvaTransitionEditor>& InEditor);

	//~ Begin FWorkflowTabFactory
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	//~ End FWorkflowTabFactory
};
