// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "Templates/SharedPointer.h"

/** Casted View Model that has both the Base View Model & the Casted Instance */
template<typename T UE_REQUIRES(TIsValidAvaType<T>::Value)>
struct TAvaTransitionCastedViewModel
{
	TSharedRef<class FAvaTransitionViewModel> Base;
	TSharedRef<T> Casted;
};
