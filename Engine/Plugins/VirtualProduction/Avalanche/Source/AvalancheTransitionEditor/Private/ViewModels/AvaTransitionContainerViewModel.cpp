// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionContainerViewModel.h"
#include "AvaTransitionEditorViewModel.h"
#include "AvaTransitionTreeEditorData.h"
#include "AvaTransitionViewModelSharedData.h"
#include "StateTreeState.h"

FAvaTransitionContainerViewModel::FAvaTransitionContainerViewModel(UStateTreeState* InState)
	: StateWeak(InState)
{
}

UStateTreeState* FAvaTransitionContainerViewModel::GetState() const
{
	return StateWeak.Get();
}

UAvaTransitionTreeEditorData* FAvaTransitionContainerViewModel::GetEditorData() const
{
	if (TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = GetSharedData()->GetEditorViewModel())
	{
		return EditorViewModel->GetEditorData();
	}
	return nullptr;
}

bool FAvaTransitionContainerViewModel::IsValid() const
{
	return StateWeak.IsValid();
}
