// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionViewModel.h"
#include "UObject/WeakObjectPtr.h"

class UAvaTransitionTreeEditorData;
class UStateTreeState;

/** View Model serving as a container for a particular group of elements in a State */
class FAvaTransitionContainerViewModel : public FAvaTransitionViewModel
{
public:
	UE_AVA_INHERITS(FAvaTransitionContainerViewModel, FAvaTransitionViewModel)

	explicit FAvaTransitionContainerViewModel(UStateTreeState* InState);

	UStateTreeState* GetState() const;

	UAvaTransitionTreeEditorData* GetEditorData() const;

	//~ Begin FAvaTransitionViewModel
	virtual bool IsValid() const override;
	//~ End FAvaTransitionViewModel

private:
	TWeakObjectPtr<UStateTreeState> StateWeak;
};
