// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionEditorViewModel.h"
#include "Actions/AvaTransitionStateActions.h"
#include "Actions/AvaTransitionTreeActions.h"
#include "AvaTransitionCommands.h"
#include "AvaTransitionEditor.h"
#include "AvaTransitionEditorLog.h"
#include "AvaTransitionEditorUtils.h"
#include "AvaTransitionSelection.h"
#include "AvaTransitionTree.h"
#include "AvaTransitionTreeEditorData.h"
#include "AvaTransitionViewModelChildren.h"
#include "AvaTransitionViewModelSharedData.h"
#include "IMessageLogListing.h"
#include "Menu/AvaTransitionToolbar.h"
#include "Menu/AvaTransitionTreeContextMenu.h"
#include "Misc/UObjectToken.h"
#include "Registry/AvaTransitionViewModelRegistryCollection.h"
#include "State/AvaTransitionStateViewModel.h"
#include "StateTreeDelegates.h"
#include "StateTreeEditorSettings.h"
#include "TabFactories/AvaTransitionCompilerResultsTabFactory.h"
#include "Views/SAvaTransitionTreeView.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "AvaTransitionEditorViewModel"

FAvaTransitionEditorViewModel::FAvaTransitionEditorViewModel(UAvaTransitionTree* InTransitionTree, const TSharedPtr<FAvaTransitionEditor>& InEditor)
	: Toolbar(MakeShared<FAvaTransitionToolbar>(*this))
	, ContextMenu(MakeShared<FAvaTransitionTreeContextMenu>(*this))
	, TransitionTreeWeak(InTransitionTree)
	, EditorWeak(InEditor)
	, CommandList(MakeShared<FUICommandList>())
{
	Compiler.SetTransitionTree(InTransitionTree);
}

FAvaTransitionEditorViewModel::~FAvaTransitionEditorViewModel()
{
	UnbindDelegates();
}

void FAvaTransitionEditorViewModel::BindCommands(const TSharedRef<FUICommandList>& InCommandList)
{
	Actions =
		{
			MakeShared<FAvaTransitionTreeActions>(*this),
			MakeShared<FAvaTransitionStateActions>(*this),
		};

	InCommandList->Append(CommandList);

	for (const TSharedRef<FAvaTransitionActions>& Action : Actions)
	{
		Action->BindCommands(CommandList);
	}
}

const FAvaTransitionCompiler& FAvaTransitionEditorViewModel::GetCompiler() const
{
	return Compiler;
}

bool FAvaTransitionEditorViewModel::CanCompile() const
{
	UAvaTransitionTree* TransitionTree = GetTransitionTree();

	bool bReadOnly = GetSharedData()->IsReadOnly();

	return !bReadOnly
		&& !GEditor->IsPlayingSessionInEditor()
		&& TransitionTree;
}

void FAvaTransitionEditorViewModel::Compile()
{
	if (CanCompile())
	{
		UpdateTree();
		Compiler.Compile(GetSharedData()->GetEditorMode());
	}
}

UAvaTransitionTree* FAvaTransitionEditorViewModel::GetTransitionTree() const
{
	return TransitionTreeWeak.Get();
}

UAvaTransitionTreeEditorData* FAvaTransitionEditorViewModel::GetEditorData() const
{
	return EditorDataWeak.Get();
}

bool FAvaTransitionEditorViewModel::UpdateEditorData(bool bInCreateIfNotFound)
{
	UAvaTransitionTree* TransitionTree = GetTransitionTree();
	if (!TransitionTree)
	{
		return false;
	}

	bool bCreatedNewEditorData = false;

	UAvaTransitionTreeEditorData* EditorData = Cast<UAvaTransitionTreeEditorData>(TransitionTree->EditorData);
	if (!EditorData && bInCreateIfNotFound)
	{
		EditorData = NewObject<UAvaTransitionTreeEditorData>(TransitionTree, NAME_None, RF_Transactional);
		EditorData->AddRootState();
		TransitionTree->EditorData = EditorData;
		bCreatedNewEditorData = true;
	}

	EditorDataWeak = EditorData;
	return bCreatedNewEditorData;
}

void FAvaTransitionEditorViewModel::UpdateTree()
{
	Compiler.UpdateTree();
}

TSharedPtr<FAvaTransitionEditor> FAvaTransitionEditorViewModel::GetEditor() const
{
	return EditorWeak.Pin();
}

TSharedRef<FAvaTransitionSelection> FAvaTransitionEditorViewModel::GetSelection() const
{
	return GetSharedData()->GetSelection();
}

TSharedRef<SWidget> FAvaTransitionEditorViewModel::GetTreeWidget()
{
	if (!TreeView.IsValid())
	{
		TreeView = SNew(SAvaTransitionTreeView, SharedThis(this));
	}

	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			Toolbar->GenerateTreeToolbarWidget()
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SScrollBox)
				.Orientation(Orient_Horizontal)
				+ SScrollBox::Slot()
				.FillSize(1.f)
				[
					TreeView.ToSharedRef()
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				TreeView->GetVerticalScrollbar()
			]
		];
}

void FAvaTransitionEditorViewModel::OnInitialize()
{
	FAvaTransitionViewModel::OnInitialize();

	BindDelegates();

	if (UpdateEditorData(/*bCreateIfNotFound*/true))
	{
		Compile();
	}

	TSharedRef<FAvaTransitionEditorViewModel> This = SharedThis(this);

	TSharedRef<FAvaTransitionViewModelSharedData> ViewModelSharedData = GetSharedData();

	ViewModelSharedData->Initialize(This);

	if (TSharedPtr<FAvaTransitionEditor> Editor = GetEditor())
	{
		ViewModelSharedData->SetReadOnly(Editor->IsReadOnly());
	}

	UE::StateTree::Delegates::OnPostCompile.AddSP(This, &FAvaTransitionEditorViewModel::OnPostCompile);
}

void FAvaTransitionEditorViewModel::PostRefresh()
{
	FAvaTransitionViewModel::PostRefresh();

	RefreshTreeView();
	UpdateTree();

	OnPostRefreshDelegate.Broadcast();
}

void FAvaTransitionEditorViewModel::RefreshTreeView()
{
	if (TreeView.IsValid())
	{
		TreeView->Refresh();
	}
}

void FAvaTransitionEditorViewModel::GatherChildren(FAvaTransitionViewModelChildren& OutChildren)
{
	UAvaTransitionTreeEditorData* EditorData = GetEditorData();
	if (!EditorData)
	{
		return;
	}

	OutChildren.Reserve(EditorData->SubTrees.Num());

	for (UStateTreeState* State : EditorData->SubTrees)
	{
		OutChildren.Add<FAvaTransitionStateViewModel>(State);
	}
}

void FAvaTransitionEditorViewModel::PostRedo(bool bInSuccess)
{
	UpdateEditorData();
	Refresh();
}

void FAvaTransitionEditorViewModel::PostUndo(bool bInSuccess)
{
	UpdateEditorData();
	Refresh();
}

UObject* FAvaTransitionEditorViewModel::GetObject() const
{
	return GetEditorData();
}

void FAvaTransitionEditorViewModel::BindDelegates()
{
	UnbindDelegates();

	if (UAvaTransitionTreeEditorData* EditorData = GetEditorData())
	{
		EditorData->GetOnTreeRequestRefresh().AddSP(this, &FAvaTransitionEditorViewModel::Refresh);
	}

	Compiler.GetCompilerResultsListing().OnMessageTokenClicked().AddSP(this, &FAvaTransitionEditorViewModel::OnMessageTokenClicked);
	Compiler.GetOnCompileFailed().BindSP(this, &FAvaTransitionEditorViewModel::OnCompileFailed);

	UE::StateTree::Delegates::OnIdentifierChanged.AddSP(this, &FAvaTransitionEditorViewModel::OnIdentifierChanged);
	UE::StateTree::Delegates::OnSchemaChanged.AddSP(this, &FAvaTransitionEditorViewModel::OnSchemaChanged);
}

void FAvaTransitionEditorViewModel::UnbindDelegates()
{
	if (UAvaTransitionTreeEditorData* EditorData = GetEditorData())
	{
		EditorData->GetOnTreeRequestRefresh().RemoveAll(this);
	}

	Compiler.GetCompilerResultsListing().OnMessageTokenClicked().RemoveAll(this);

	UE::StateTree::Delegates::OnIdentifierChanged.RemoveAll(this);
	UE::StateTree::Delegates::OnSchemaChanged.RemoveAll(this);
}

void FAvaTransitionEditorViewModel::OnPostCompile(const UStateTree& InStateTree)
{
	if (&InStateTree == GetTransitionTree())
	{
		// Full refresh on compile
		Refresh();
	}
}

void FAvaTransitionEditorViewModel::OnIdentifierChanged(const UStateTree& InStateTree)
{
	if (&InStateTree == GetTransitionTree())
	{
		UpdateTree();
	}
}

void FAvaTransitionEditorViewModel::OnSchemaChanged(const UStateTree& InStateTree)
{
	if (&InStateTree == GetTransitionTree())
	{
		UpdateTree();
	}
}

void FAvaTransitionEditorViewModel::OnCompileFailed()
{
	if (TSharedPtr<FAvaTransitionEditor> Editor = GetEditor())
	{
		if (TSharedPtr<FTabManager> TabManager = Editor->GetTabManager())
		{
			TabManager->TryInvokeTab(FAvaTransitionCompilerResultsTabFactory::TabId);
		}
	}
}

void FAvaTransitionEditorViewModel::OnMessageTokenClicked(const TSharedRef<IMessageToken>& InMessageToken)
{
	if (InMessageToken->GetType() != EMessageToken::Object)
	{
		return;
	}

	TSharedRef<FAvaTransitionViewModelSharedData> ViewModelSharedData = GetSharedData();

	if (UStateTreeState* State = Cast<UStateTreeState>(StaticCastSharedRef<FUObjectToken>(InMessageToken)->GetObject().Get()))
	{
		if (TSharedPtr<FAvaTransitionViewModel> FoundViewModel = ViewModelSharedData->GetRegistryCollection()->FindViewModel(State))
		{
			ViewModelSharedData->GetSelection()->SetSelectedItems({ FoundViewModel });
		}
	}
}

#undef LOCTEXT_NAMESPACE
