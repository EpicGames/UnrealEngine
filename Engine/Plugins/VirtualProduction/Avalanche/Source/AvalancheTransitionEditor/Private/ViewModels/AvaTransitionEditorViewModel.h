// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionViewModel.h"
#include "Compiler/AvaTransitionCompiler.h"
#include "EditorUndoClient.h"
#include "Extensions/IAvaTransitionObjectExtension.h"
#include "Extensions/IAvaTransitionWidgetExtension.h"
#include "UObject/WeakObjectPtr.h"

class FAvaTransitionActions;
class FAvaTransitionEditor;
class FAvaTransitionSelection;
class FAvaTransitionToolbar;
class FAvaTransitionTreeContextMenu;
class FUICommandList;
class IAvaTransitionSelectableExtension;
class IMessageToken;
class SAvaTransitionTreeView;
class UAvaTransitionTree;
class UAvaTransitionTreeEditorData;
class UStateTree;
class UStateTreeState;
class UToolMenu;

/** View Model for the Tree Editor Data */
class FAvaTransitionEditorViewModel : public FAvaTransitionViewModel, public FSelfRegisteringEditorUndoClient, public IAvaTransitionObjectExtension
{
public:
	UE_AVA_INHERITS(FAvaTransitionEditorViewModel, FAvaTransitionViewModel, IAvaTransitionObjectExtension)

	DECLARE_MULTICAST_DELEGATE(FOnPostRefresh);
	FOnPostRefresh::RegistrationType& GetOnPostRefresh() const
	{
		return OnPostRefreshDelegate;
	}

	explicit FAvaTransitionEditorViewModel(UAvaTransitionTree* InTransitionTree, const TSharedPtr<FAvaTransitionEditor>& InEditor);

	virtual ~FAvaTransitionEditorViewModel() override;

	void BindCommands(const TSharedRef<FUICommandList>& InCommandList);

	TSharedRef<FUICommandList> GetCommandList() const
	{
		return CommandList;
	}

	const FAvaTransitionCompiler& GetCompiler() const;

	bool CanCompile() const;

	void Compile();

	UAvaTransitionTree* GetTransitionTree() const;

	UAvaTransitionTreeEditorData* GetEditorData() const;

	bool UpdateEditorData(bool bInCreateIfNotFound = false);

	void UpdateTree();

	void RefreshTreeView();

	TSharedPtr<FAvaTransitionEditor> GetEditor() const;

	TSharedRef<FAvaTransitionSelection> GetSelection() const;

	TSharedRef<FAvaTransitionToolbar> GetToolbar() const
	{
		return Toolbar;
	}

	TSharedRef<FAvaTransitionTreeContextMenu> GetContextMenu() const
	{
		return ContextMenu;
	}

	TSharedRef<SWidget> GetTreeWidget();

	//~ Begin FAvaTransitionViewModel
	virtual void OnInitialize() override;
	virtual void PostRefresh() override;
	virtual void GatherChildren(FAvaTransitionViewModelChildren& OutChildren) override;
	//~ End FAvaTransitionViewModel

	//~ Begin FEditorUndoClient
	virtual void PostRedo(bool bInSuccess) override;
	virtual void PostUndo(bool bInSuccess) override;
	//~ End FEditorUndoClient

	//~ Begin IAvaTransitionObjectExtension
	virtual UObject* GetObject() const override;
	virtual void OnPropertiesChanged(const FPropertyChangedEvent& InPropertyChangedEvent) override {}
	//~ End IAvaTransitionObjectExtension

private:
	void BindDelegates();

	void UnbindDelegates();

	void OnPostCompile(const UStateTree& InStateTree);

	void OnIdentifierChanged(const UStateTree& InStateTree);

	void OnSchemaChanged(const UStateTree& InStateTree);

	void OnCompileFailed();

	void OnMessageTokenClicked(const TSharedRef<IMessageToken>& InMessageToken);

	TSharedRef<FAvaTransitionToolbar> Toolbar;

	TSharedRef<FAvaTransitionTreeContextMenu> ContextMenu;

	TWeakObjectPtr<UAvaTransitionTree> TransitionTreeWeak;

	TWeakObjectPtr<UAvaTransitionTreeEditorData> EditorDataWeak;

	TWeakPtr<FAvaTransitionEditor> EditorWeak;

	FAvaTransitionCompiler Compiler;

	TSharedRef<FUICommandList> CommandList;

	TSharedPtr<SAvaTransitionTreeView> TreeView;

	TArray<TSharedRef<FAvaTransitionActions>> Actions;

	mutable FOnPostRefresh OnPostRefreshDelegate;
};
