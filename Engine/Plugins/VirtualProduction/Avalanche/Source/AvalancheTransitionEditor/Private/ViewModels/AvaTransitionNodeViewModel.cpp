// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionNodeViewModel.h"
#include "AvaTransitionEditorViewModel.h"
#include "AvaTransitionTreeEditorData.h"
#include "AvaTransitionViewModelSharedData.h"
#include "AvaTransitionViewModelUtils.h"
#include "State/AvaTransitionStateViewModel.h"
#include "StateTreeEditorNode.h"
#include "StateTreeEditorTypes.h"
#include "Styling/AvaTransitionEditorStyle.h"

FAvaTransitionNodeViewModel::FAvaTransitionNodeViewModel(const FStateTreeEditorNode& InEditorNode)
	: NodeId(InEditorNode.ID)
{
}

UStateTreeState* FAvaTransitionNodeViewModel::GetState() const
{
	if (TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = UE::AvaTransitionEditor::FindAncestorOfType<FAvaTransitionStateViewModel>(*this))
	{
		return StateViewModel->GetState();
	}
	return nullptr;
}

FSlateColor FAvaTransitionNodeViewModel::GetStateColor() const
{
	if (TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = UE::AvaTransitionEditor::FindAncestorOfType<FAvaTransitionStateViewModel>(*this))
	{
		return StateViewModel->GetStateColor();
	}
	return FAvaTransitionStateViewModel::GetDefaultStateColor();
}

UAvaTransitionTreeEditorData* FAvaTransitionNodeViewModel::GetEditorData() const
{
	if (TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = GetSharedData()->GetEditorViewModel())
	{
		return EditorViewModel->GetEditorData();
	}
	return nullptr;
}

FStateTreeEditorNode* FAvaTransitionNodeViewModel::GetEditorNode() const
{
	if (UStateTreeState* State = GetState())
	{
		TArrayView<FStateTreeEditorNode> Nodes = GetNodes(*State);
		return Nodes.FindByPredicate([this](const FStateTreeEditorNode& InNode)
		{
			return NodeId == InNode.ID;
		});
	}
	return nullptr;
}

const FInstancedStruct* FAvaTransitionNodeViewModel::GetNode() const
{
	if (const FStateTreeEditorNode* EditorNode = GetEditorNode())
	{
		return &EditorNode->Node;
	}
	return nullptr;
}

bool FAvaTransitionNodeViewModel::IsValid() const
{
	if (NodeId.IsValid())
	{
		const FInstancedStruct* Node = GetNode();
		return Node && Node->IsValid();
	}
	return false;
}

const FGuid& FAvaTransitionNodeViewModel::GetGuid() const
{
	return NodeId;
}
