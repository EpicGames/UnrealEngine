// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionViewModel.h"
#include "Extensions/IAvaTransitionGuidExtension.h"
#include "Misc/Guid.h"
#include "StateTreeEditorNode.h"

class UAvaTransitionTreeEditorData;
class UStateTreeState;
struct FSlateColor;
struct FStateTreeEditorNode;
struct FStateTreeNodeBase;

/** View Model for an Editor Node (base of Task or Condition) */
class FAvaTransitionNodeViewModel : public FAvaTransitionViewModel, public IAvaTransitionGuidExtension
{
public:
	UE_AVA_INHERITS(FAvaTransitionNodeViewModel, FAvaTransitionViewModel, IAvaTransitionGuidExtension)

	explicit FAvaTransitionNodeViewModel(const FStateTreeEditorNode& InEditorNode);

	UStateTreeState* GetState() const;

	FSlateColor GetStateColor() const;

	UAvaTransitionTreeEditorData* GetEditorData() const;

	FStateTreeEditorNode* GetEditorNode() const;

	const FInstancedStruct* GetNode() const;

	template<typename T = FStateTreeNodeBase UE_REQUIRES(TIsDerivedFrom<T, FStateTreeNodeBase>::Value)>
	const T* GetTypedNode() const
	{
		if (const FInstancedStruct* Node = GetNode())
		{
			return Node->GetPtr<T>();
		}
		return nullptr;
	}

	/** Get the Nodes for a given State */
	virtual TArrayView<FStateTreeEditorNode> GetNodes(UStateTreeState& InState) const = 0;

	const FGuid& GetNodeId() const
	{
		return NodeId;
	}

	//~ Begin FAvaTransitionViewModel
	virtual bool IsValid() const override;
	//~ End FAvaTransitionViewModel

	//~ Begin IAvaTransitionGuidExtension
	virtual const FGuid& GetGuid() const override;
	//~ End IAvaTransitionGuidExtension

private:
	FGuid NodeId;
};
