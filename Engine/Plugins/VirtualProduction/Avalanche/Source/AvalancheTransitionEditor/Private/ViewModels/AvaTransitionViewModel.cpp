// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionViewModel.h"
#include "AvaTransitionViewModelChildren.h"
#include "AvaTransitionViewModelSharedData.h"
#include "AvaTransitionViewModelUtils.h"
#include "Registry/AvaTransitionViewModelRegistryCollection.h"

FAvaTransitionViewModel::FAvaTransitionViewModel()
	: Children(/*Parent*/*this)
{
}

void FAvaTransitionViewModel::Initialize(const TSharedPtr<FAvaTransitionViewModel>& InParent)
{
	// Initialized shouldn't be called twice
	checkfSlow(!SharedData.IsValid(), TEXT("FAvaTransitionViewModel::Initialize called twice for the same instance"));

	ParentWeak = InParent;

	// Re-use the Parent Shared Data instead of making a new one
	if (InParent.IsValid())
	{
		SharedData = InParent->GetSharedData();
		ensureMsgf(SharedData.IsValid(), TEXT("Parent View Model found, but does not have a valid shared data instance."));
	}

	// If no parent (i.e. a topmost view model) or parent erroneously had no valid view model shared data (unlikely), allocate a new instance
	if (!SharedData.IsValid())
	{
		SharedData = MakeShared<FAvaTransitionViewModelSharedData>();
	}

	OnInitialize();

	// Register View Model after allowing derived implementations to fully initialize
	SharedData->GetRegistryCollection()->RegisterViewModel(SharedThis(this));

	Refresh();
}

void FAvaTransitionViewModel::Refresh()
{
	TArray<TSharedRef<FAvaTransitionViewModel>> ViewModels = { SharedThis(this) };

	// Refresh this View Model and all its children
	while (!ViewModels.IsEmpty())
	{
		TSharedRef<FAvaTransitionViewModel> ViewModel = ViewModels.Pop();

		ViewModel->Children.Reset();
		ViewModel->GatherChildren(ViewModel->Children);

		ViewModels.Append(ViewModel->GetChildren());
	}

	SharedData->Refresh();

	PostRefresh();

	UE::AvaTransitionEditor::ForEachChild(*this,
		[](const TSharedRef<FAvaTransitionViewModel>& InViewModel, EAvaTransitionIterationResult&)
		{
			InViewModel->PostRefresh();
		}
		, /*bInRecursive*/true);
}
