// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionViewModelChildren.h"
#include "AvaType.h"
#include "AvaTypeSharedPointer.h"
#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointer.h"

class FAvaTransitionViewModelSharedData;

class FAvaTransitionViewModel : public IAvaTypeCastable, public TSharedFromThis<FAvaTransitionViewModel>
{
public:
	UE_AVA_INHERITS(FAvaTransitionViewModel, IAvaTypeCastable)

	FAvaTransitionViewModel();

	void Initialize(const TSharedPtr<FAvaTransitionViewModel>& InParent);

	virtual void OnInitialize()
	{
	}

	/** Returns true if the View Model is in a valid state */
	virtual bool IsValid() const
	{
		return true;
	}

	/** Resets and gathers all the children for this View Model and all its descendants iteratively */
	void Refresh();

	/** Called after all the Children have been gathered recursively */
	virtual void PostRefresh()
	{
	}

	TSharedRef<FAvaTransitionViewModelSharedData> GetSharedData() const
	{
		return SharedData.ToSharedRef();
	}

	TSharedPtr<FAvaTransitionViewModel> GetParent() const
	{
		return ParentWeak.Pin();
	}

	TConstArrayView<TSharedRef<FAvaTransitionViewModel>> GetChildren() const
	{
		return Children.GetViewModels();
	}

	/** Gathers only the top-level children for the View Model */
	virtual void GatherChildren(FAvaTransitionViewModelChildren& OutChildren)
	{
	}

private:
	TSharedPtr<FAvaTransitionViewModelSharedData> SharedData;

	FAvaTransitionViewModelChildren Children;

	TWeakPtr<FAvaTransitionViewModel> ParentWeak;
};
