// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionViewModelChildren.h"
#include "AvaTransitionViewModel.h"

FAvaTransitionViewModelChildren::FAvaTransitionViewModelChildren(FAvaTransitionViewModel& InParent)
	: Parent(InParent)
{
}

void FAvaTransitionViewModelChildren::Reserve(int32 InExpectedCount)
{
	ViewModels.Reserve(InExpectedCount);
}

void FAvaTransitionViewModelChildren::Reset()
{
	ViewModels.Reset();
}

bool FAvaTransitionViewModelChildren::AddImpl(const TSharedRef<FAvaTransitionViewModel>& InViewModel)
{
	InViewModel->Initialize(Parent.AsShared());

	if (InViewModel->IsValid())
	{
		ViewModels.Add(InViewModel);
		return true;
	}

	return false;
}
