// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

class FAvaTransitionViewModel;

/**
 * Container class for Child View Models 
 * Automatically handles setting the Parent when a Child View Model is allocated
 * and disallows items from being added if they're in an invalid state
 */
class FAvaTransitionViewModelChildren
{
public:
	explicit FAvaTransitionViewModelChildren(FAvaTransitionViewModel& InParent);

	TConstArrayView<TSharedRef<FAvaTransitionViewModel>> GetViewModels() const
	{
		return ViewModels;
	}

	void Reserve(int32 InExpectedCount);

	void Reset();

	template<typename T, typename... InArgTypes UE_REQUIRES(TIsValidAvaType<T>::Value)>
	TSharedPtr<T> Add(InArgTypes&&... InArgs)
	{
		TSharedRef<T> ViewModel = ::MakeShared<T>(Forward<InArgTypes>(InArgs)...);
		if (this->AddImpl(ViewModel))
		{
			return ViewModel;
		}
		return nullptr;
	}

private:
	bool AddImpl(const TSharedRef<FAvaTransitionViewModel>& InViewModel);

	FAvaTransitionViewModel& Parent;

	TArray<TSharedRef<FAvaTransitionViewModel>> ViewModels;
};
