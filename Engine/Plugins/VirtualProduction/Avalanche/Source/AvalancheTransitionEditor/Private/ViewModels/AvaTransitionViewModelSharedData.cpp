// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionViewModelSharedData.h"
#include "AvaTransitionSelection.h"
#include "Debugger/AvaTransitionDebugger.h"
#include "Registry/AvaTransitionViewModelRegistryCollection.h"

FAvaTransitionViewModelSharedData::FAvaTransitionViewModelSharedData()
	: RegistryCollection(MakeShared<FAvaTransitionViewModelRegistryCollection>())
	, Selection(MakeShared<FAvaTransitionSelection>())
#if UE_AVA_WITH_TRANSITION_DEBUG
	, Debugger(MakeShared<FAvaTransitionDebugger>())
#endif
{
}

void FAvaTransitionViewModelSharedData::Initialize(const TSharedRef<FAvaTransitionEditorViewModel>& InEditorViewModel)
{
	EditorViewModelWeak = InEditorViewModel;

#if UE_AVA_WITH_TRANSITION_DEBUG
	Debugger->Initialize(InEditorViewModel);
#endif
}

void FAvaTransitionViewModelSharedData::Refresh()
{
	RegistryCollection->Refresh();
}
