// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionEditorEnums.h"
#include "Debugger/AvaTransitionDebugDefinitions.h"
#include "Templates/SharedPointer.h"

class FAvaTransitionEditorViewModel;
class FAvaTransitionSelection;
class FAvaTransitionViewModel;
class FAvaTransitionViewModelRegistryCollection;

#if UE_AVA_WITH_TRANSITION_DEBUG
class FAvaTransitionDebugger;
#endif

/**
 * Shared Data is designed to house data accessible by any View Model in a hierarchy.
 * A Shared Data instance is created at the topmost view model in a hierarchy (i.e. the view model with no parent), and shared across the rest of the view models
 */
class FAvaTransitionViewModelSharedData : public TSharedFromThis<FAvaTransitionViewModelSharedData>
{
public:
	FAvaTransitionViewModelSharedData();

	void Initialize(const TSharedRef<FAvaTransitionEditorViewModel>& InEditorViewModel);

	void Refresh();

	TSharedPtr<FAvaTransitionEditorViewModel> GetEditorViewModel() const
	{
		return EditorViewModelWeak.Pin();
	}

	TSharedRef<FAvaTransitionViewModelRegistryCollection> GetRegistryCollection() const
	{
		return RegistryCollection;
	}

	TSharedRef<FAvaTransitionSelection> GetSelection() const
	{
		return Selection;
	}

#if UE_AVA_WITH_TRANSITION_DEBUG
	TSharedRef<FAvaTransitionDebugger> GetDebugger() const
	{
		return Debugger;
	}
#endif

	EAvaTransitionEditorMode GetEditorMode() const
	{
		return EditorMode;
	}

	void SetEditorMode(EAvaTransitionEditorMode InEditorMode)
	{
		EditorMode = InEditorMode;
	}

	bool IsReadOnly() const
	{
		return bReadOnly;
	}

	void SetReadOnly(bool bInReadOnly)
	{
		bReadOnly = bInReadOnly;
	}

private:
	TSharedRef<FAvaTransitionViewModelRegistryCollection> RegistryCollection;

	TSharedRef<FAvaTransitionSelection> Selection;

#if UE_AVA_WITH_TRANSITION_DEBUG
	TSharedRef<FAvaTransitionDebugger> Debugger;
#endif

	TWeakPtr<FAvaTransitionEditorViewModel> EditorViewModelWeak;

	EAvaTransitionEditorMode EditorMode = EAvaTransitionEditorMode::Default;

	bool bReadOnly = false;
};
