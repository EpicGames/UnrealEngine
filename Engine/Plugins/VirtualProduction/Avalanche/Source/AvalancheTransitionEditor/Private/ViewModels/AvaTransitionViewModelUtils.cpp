// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionViewModelUtils.h"
#include "AvaTransitionEditorViewModel.h"
#include "AvaTransitionViewModel.h"

namespace UE::AvaTransitionEditor
{
	bool IsDescendantOf(const TSharedRef<FAvaTransitionViewModel>& InViewModel, const TSharedRef<FAvaTransitionViewModel>& InParentViewModel)
	{
		TArray<TSharedRef<FAvaTransitionViewModel>> ViewModels = { InParentViewModel };
		while (!ViewModels.IsEmpty())
		{
			TSharedRef<FAvaTransitionViewModel> CurrentViewModel = ViewModels.Pop(EAllowShrinking::No);
			if (CurrentViewModel == InViewModel)
			{
				return true;
			}
			ViewModels.Append(CurrentViewModel->GetChildren());
		}
		return false;
	}

	EAvaTransitionIterationResult ForEachViewModel(TConstArrayView<TSharedRef<FAvaTransitionViewModel>> InViewModels, TFunctionRef<void(const TSharedRef<FAvaTransitionViewModel>&, EAvaTransitionIterationResult&)> InCallable, bool bInRecursive)
	{
		EAvaTransitionIterationResult IterationResult = EAvaTransitionIterationResult::Continue;
		for (const TSharedRef<FAvaTransitionViewModel>& ViewModel : InViewModels)
		{
			InCallable(ViewModel, IterationResult);
			if (IterationResult == EAvaTransitionIterationResult::Break)
			{
				break;
			}

			if (bInRecursive)
			{
				IterationResult = ForEachViewModel(ViewModel->GetChildren(), InCallable, bInRecursive);
				if (IterationResult == EAvaTransitionIterationResult::Break)
				{
					break;
				}
			}
		}
		return IterationResult;
	}

	EAvaTransitionIterationResult ForEachChild(const FAvaTransitionViewModel& InViewModel, TFunctionRef<void(const TSharedRef<FAvaTransitionViewModel>&, EAvaTransitionIterationResult&)> InCallable, bool bInRecursive)
	{
		return UE::AvaTransitionEditor::ForEachViewModel(InViewModel.GetChildren(), InCallable, bInRecursive);
	}

	void RemoveNestedViewModels(TArray<TSharedRef<FAvaTransitionViewModel>>& InViewModels)
	{
		const TSet<TSharedRef<FAvaTransitionViewModel>> UniqueViewModels(InViewModels);

		for (TArray<TSharedRef<FAvaTransitionViewModel>>::TIterator ViewModelIterator(InViewModels); ViewModelIterator; ++ViewModelIterator)
		{
			const TSharedRef<FAvaTransitionViewModel>& ViewModel = *ViewModelIterator;

			// Walk up the parent view models and remove if any of the ancestor is contained
			TSharedPtr<FAvaTransitionViewModel> ParentViewModel = ViewModel->GetParent();
			while (ParentViewModel.IsValid())
			{
				if (UniqueViewModels.Contains(ParentViewModel.ToSharedRef()))
				{
					ViewModelIterator.RemoveCurrent();
					break;
				}
				ParentViewModel = ParentViewModel->GetParent();
			}
		}
	}
}
