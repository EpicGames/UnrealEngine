// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionCastedViewModel.h"
#include "AvaTransitionEnums.h"
#include "AvaTransitionViewModel.h"
#include "AvaType.h"
#include "AvaTypeSharedPointer.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"

namespace UE::AvaTransitionEditor
{
	bool IsDescendantOf(const TSharedRef<FAvaTransitionViewModel>& InViewModel, const TSharedRef<FAvaTransitionViewModel>& InParentViewModel);

	/**
	 * Iterates each view model with option to recurse through all descendants and/or break iteration
	 * @param InViewModels the view models to iterate
	 * @param InCallable the function to call for every view model
	 * @param bInRecursive true to recurse all the descendants of each view model
	 * @return the result of the for each flow (e.g. to query whether it was stopped)
	 */
	EAvaTransitionIterationResult ForEachViewModel(TConstArrayView<TSharedRef<FAvaTransitionViewModel>> InViewModels, TFunctionRef<void(const TSharedRef<FAvaTransitionViewModel>&, EAvaTransitionIterationResult&)> InCallable, bool bInRecursive = false);

	/**
	 * Iterates each view model of a given type with option to recurse through all descendants and/or break iteration
	 * @param InViewModels the view models to iterate
	 * @param InCallable the function to call for every view model with matching type
	 * @param bInRecursive true to recurse all the descendants of each view model
	 * @return the result of the for each flow (e.g. to query whether it was stopped)
	 */
	template<typename T UE_REQUIRES(TIsValidAvaType<T>::Value)>
	EAvaTransitionIterationResult ForEachViewModelOfType(TConstArrayView<TSharedRef<FAvaTransitionViewModel>> InViewModels, TFunctionRef<void(const TAvaTransitionCastedViewModel<T>&, EAvaTransitionIterationResult&)> InCallable, bool bInRecursive)
	{
		EAvaTransitionIterationResult IterationResult = EAvaTransitionIterationResult::Continue;
		for (const TSharedRef<FAvaTransitionViewModel>& ViewModel : InViewModels)
		{
			if (TSharedPtr<T> CastedInstance = UE::AvaCore::CastSharedPtr<T>(ViewModel))
			{
				InCallable({ViewModel, CastedInstance.ToSharedRef()}, IterationResult);

				if (IterationResult == EAvaTransitionIterationResult::Break)
				{
					break;
				}
			}
			if (bInRecursive)
			{
				IterationResult = ForEachViewModelOfType<T>(ViewModel->GetChildren(), InCallable, bInRecursive);

				if (IterationResult == EAvaTransitionIterationResult::Break)
				{
					break;
				}
			}
		}
		return IterationResult;
	}

	/**
	 * Iterates each child with options to recurse through all descendants and/or break iteration
	 * @param InViewModel the view model whose children will be iterated
	 * @param InCallable the function to call for every child
	 * @param bInRecursive true to recurse all the descendants, false to only iterate through the immediate children
	 * @return the result of the for each flow (e.g. to query whether it was stopped)
	 */
	EAvaTransitionIterationResult ForEachChild(const FAvaTransitionViewModel& InViewModel, TFunctionRef<void(const TSharedRef<FAvaTransitionViewModel>&, EAvaTransitionIterationResult&)> InCallable, bool bInRecursive = false);

	/**
	 * Iterates each child type with option to recurse through all descendants and/or break iteration
	 * @param InViewModel the view model whose children will be iterated
	 * @param InCallable the function to call for every child with matching type
	 * @param bInRecursive true to recurse all the descendants, false to only iterate through the immediate children
	 * @return the result of the for each flow (e.g. to query whether it was stopped)
	 */
	template<typename T UE_REQUIRES(TIsValidAvaType<T>::Value)>
	EAvaTransitionIterationResult ForEachChildOfType(const FAvaTransitionViewModel& InViewModel, TFunctionRef<void(const TAvaTransitionCastedViewModel<T>&, EAvaTransitionIterationResult&)> InCallable, bool bInRecursive = false)
	{
		return UE::AvaTransitionEditor::ForEachViewModelOfType<T>(InViewModel.GetChildren(), InCallable, bInRecursive);
	}

	/**
 	 * Gets all the view models matching a given type
 	 * @param InViewModels the view models to iterate
 	 * @param bInRecursive true to recurse all the descendants of each view model
 	 * @return the view models casted to the given type
 	 */
	template<typename T UE_REQUIRES(TIsValidAvaType<T>::Value)>
	TArray<TSharedRef<T>> GetViewModelsOfType(TConstArrayView<TSharedRef<FAvaTransitionViewModel>> InViewModels, bool bInRecursive = false)
	{
		TArray<TSharedRef<T>> CastedViewModels;
		CastedViewModels.Reserve(InViewModels.Num());

		ForEachViewModelOfType<T>(InViewModels,
			[&CastedViewModels](const TAvaTransitionCastedViewModel<T>& InCastedViewModel, EAvaTransitionIterationResult&)
			{
				CastedViewModels.Add(InCastedViewModel.Casted);
			}
			, bInRecursive);

		return CastedViewModels;
	}

	/**
	 * Gets all the child view models matching a given type
	 * @param InViewModel the view models whose children will be iterated
	 * @param bInRecursive true to recurse all the descendants of each view model
	 * @return the view models casted to the given type
	 */
	template<typename T UE_REQUIRES(TIsValidAvaType<T>::Value)>
	TArray<TSharedRef<T>> GetChildrenOfType(const FAvaTransitionViewModel& InViewModel, bool bInRecursive = false)
	{
		return UE::AvaTransitionEditor::GetViewModelsOfType<T>(InViewModel.GetChildren(), bInRecursive);
	}

	/**
	 * Finds the first ancestor matching the given type
	 * @param InViewModel the view model in question
	 * @param bIncludeSelf whether to also include the passed in view model if it's of the requested type
	 * @return the view model ancestor that matched the type (or null if not found)
	 */
	template<typename T UE_REQUIRES(TIsValidAvaType<T>::Value)>
	TSharedPtr<T> FindAncestorOfType(const FAvaTransitionViewModel& InViewModel, bool bIncludeSelf = false)
	{
		TSharedPtr<FAvaTransitionViewModel> Ancestor = InViewModel.GetParent();
		if (bIncludeSelf)
		{
			Ancestor = ConstCastSharedRef<FAvaTransitionViewModel>(InViewModel.AsShared());
		}

		while (Ancestor.IsValid())
		{
			if (TSharedPtr<T> CastedAncestor = UE::AvaCore::CastSharedPtr<T>(Ancestor))
			{
				return CastedAncestor;
			}
			Ancestor = Ancestor->GetParent();
		}
		return nullptr;
	}

	/** Removes any view model whose ancestor is in the provided array*/
	void RemoveNestedViewModels(TArray<TSharedRef<FAvaTransitionViewModel>>& InViewModels);
}
