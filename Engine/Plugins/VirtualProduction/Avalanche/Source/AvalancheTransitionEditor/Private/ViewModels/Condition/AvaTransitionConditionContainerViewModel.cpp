// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionConditionContainerViewModel.h"
#include "AvaTransitionConditionViewModel.h"
#include "AvaTransitionTreeEditorData.h"
#include "Conditions/AvaTransitionCondition.h"
#include "StateTreeState.h"
#include "Styling/AvaTransitionEditorStyle.h"
#include "ViewModels/AvaTransitionViewModelUtils.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "AvaTransitionConditionContainerViewModel"

namespace UE::AvaTransitionEditor::Private
{
	FText GetOperandText(int32 InConditionIndex, const FStateTreeEditorNode& InEditorNode)
	{
		// First Conditions or Copy operands should not have any operand displayed
		if (InConditionIndex > 0)
		{
			switch (InEditorNode.ExpressionOperand)
			{
			case EStateTreeExpressionOperand::Copy:
				break;

			case EStateTreeExpressionOperand::And:
				return INVTEXT("<op color=\"Colors.AccentPink\">AND</> ");

			case EStateTreeExpressionOperand::Or:
				return INVTEXT("<op color=\"Colors.AccentBlue\">OR</> ");
			}
		}
		return FText::GetEmpty();
	}

	FText GetParenthesisText(int8 InDeltaIndent)
	{
		if (int8 IndentAmount = FMath::Abs(InDeltaIndent))
		{
			TArray<FText> Parenthesis;
			Parenthesis.Reserve(IndentAmount);

			FText ParenthesisType = InDeltaIndent > 0 ? INVTEXT("( ") : INVTEXT(" )");

			for (int8 IndentCount = 0; IndentCount < IndentAmount; ++IndentCount)
			{
				Parenthesis.Add(ParenthesisType);
			}

			return FText::Join(FText::GetEmpty(), Parenthesis);
		}
		return FText::GetEmpty();
	}
}

FAvaTransitionConditionContainerViewModel::FAvaTransitionConditionContainerViewModel(UStateTreeState* InState)
	: FAvaTransitionContainerViewModel(InState)
{
}

void FAvaTransitionConditionContainerViewModel::OnConditionsChanged()
{
	Refresh();
}

EVisibility FAvaTransitionConditionContainerViewModel::GetVisibility() const
{
	if (GetChildren().IsEmpty())
	{
		return EVisibility::Collapsed;
	}
	return EVisibility::Visible;
}

FText FAvaTransitionConditionContainerViewModel::UpdateStateDescription() const
{
	TArray<TSharedRef<FAvaTransitionConditionViewModel>> ConditionViewModels = UE::AvaTransitionEditor::GetChildrenOfType<FAvaTransitionConditionViewModel>(*this);
	if (ConditionViewModels.IsEmpty())
	{
		return FText::GetEmpty();
	}

	const UAvaTransitionTreeEditorData* EditorData = GetEditorData();
	if (!EditorData)
	{
		return FText::GetEmpty();
	}

	TArray<FText> ConditionDescriptions;
	for (int32 ConditionIndex = 0; ConditionIndex < ConditionViewModels.Num(); ++ConditionIndex)
	{
		const TSharedRef<FAvaTransitionConditionViewModel>& ConditionViewModel = ConditionViewModels[ConditionIndex];
		const FStateTreeEditorNode* EditorNode = ConditionViewModel->GetEditorNode();
		if (!EditorNode)
		{
			continue;
		}

		int8 DeltaIndent = -EditorNode->ExpressionIndent;
		if (ConditionViewModels.IsValidIndex(ConditionIndex + 1))
		{
			if (const FStateTreeEditorNode* NextEditorNode = ConditionViewModels[ConditionIndex + 1]->GetEditorNode())
			{
				DeltaIndent += NextEditorNode->ExpressionIndent;
			}
		}

		using namespace UE::AvaTransitionEditor;

		FTextFormat TextFormat = DeltaIndent > 0
			? LOCTEXT("ConditionFormatOpening", "{Operand}{Parenthesis}{Description}")
			: LOCTEXT("ConditionFormatClosing", "{Operand}{Description}{Parenthesis}");

		// Formatting set to Text as Titles due to the Background color do not visualize Bold/Subdued/Normal very well
		// However, the State Text is still using Rich text for things like Operand
		constexpr EStateTreeNodeFormatting Formatting = EStateTreeNodeFormatting::Text;

		FFormatNamedArguments TextArguments;
		TextArguments.Add(TEXT("Operand"), Private::GetOperandText(ConditionIndex, *EditorNode));
		TextArguments.Add(TEXT("Description"), EditorData->GetNodeDescription(*EditorNode, Formatting));
		TextArguments.Add(TEXT("Parenthesis"), Private::GetParenthesisText(DeltaIndent));

		ConditionDescriptions.Add(FText::Format(TextFormat, TextArguments));
	}

	return FText::Format(LOCTEXT("ConditionFormat", "If {0}"), FText::Join(INVTEXT(" "), ConditionDescriptions));
}

void FAvaTransitionConditionContainerViewModel::GatherChildren(FAvaTransitionViewModelChildren& OutChildren)
{
	UStateTreeState* State = GetState();
	if (!State)
	{
		return;
	}

	OutChildren.Reserve(State->EnterConditions.Num());
	for (const FStateTreeEditorNode& Condition : State->EnterConditions)
	{
		OutChildren.Add<FAvaTransitionConditionViewModel>(Condition);
	}
}

TSharedRef<SWidget> FAvaTransitionConditionContainerViewModel::CreateWidget()
{
	return SNew(SBox)
		.Padding(FMargin(0.f, 0.f, 4.f, 0.f))
		.Visibility(this, &FAvaTransitionConditionContainerViewModel::GetVisibility)
		[
			SNew(SImage)
			.ColorAndOpacity(FLinearColor(1, 1, 1, 0.5f))
			.Image(FAvaTransitionEditorStyle::Get().GetBrush("StateTreeEditor.StateConditions"))
			.ToolTipText(LOCTEXT("StateHasEnterConditions", "State selection is guarded with enter conditions."))
		];
}

#undef LOCTEXT_NAMESPACE
