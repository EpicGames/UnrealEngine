// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extensions/IAvaTransitionWidgetExtension.h"
#include "ViewModels/AvaTransitionContainerViewModel.h"

class UStateTreeState;
struct EVisibility;

/** View Model serving as a container of all the Enter Conditions in a State */
class FAvaTransitionConditionContainerViewModel : public FAvaTransitionContainerViewModel, public IAvaTransitionWidgetExtension
{
public:
	UE_AVA_INHERITS(FAvaTransitionConditionContainerViewModel, FAvaTransitionContainerViewModel, IAvaTransitionWidgetExtension)

	explicit FAvaTransitionConditionContainerViewModel(UStateTreeState* InState);

	void OnConditionsChanged();

	EVisibility GetVisibility() const;

	FText UpdateStateDescription() const; 

	//~ Begin FAvaTransitionViewModel
	virtual void GatherChildren(FAvaTransitionViewModelChildren& OutChildren) override;
	//~ Begin FAvaTransitionViewModel

	//~ Begin IAvaTransitionWidgetExtension
	virtual TSharedRef<SWidget> CreateWidget() override;
	//~ End IAvaTransitionWidgetExtension
};
