// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeEditorNode.h"
#include "ViewModels/AvaTransitionNodeViewModel.h"

struct FAvaTransitionCondition;
struct FStateTreeConditionBase;

/** View Model for an Enter Condition Node */
class FAvaTransitionConditionViewModel : public FAvaTransitionNodeViewModel
{
public:
	UE_AVA_INHERITS(FAvaTransitionConditionViewModel, FAvaTransitionNodeViewModel)

	explicit FAvaTransitionConditionViewModel(const FStateTreeEditorNode& InEditorNode);

	//~ Begin FAvaTransitionNodeViewModel
	virtual TArrayView<FStateTreeEditorNode> GetNodes(UStateTreeState& InState) const override;
	//~ End FAvaTransitionNodeViewModel
};
