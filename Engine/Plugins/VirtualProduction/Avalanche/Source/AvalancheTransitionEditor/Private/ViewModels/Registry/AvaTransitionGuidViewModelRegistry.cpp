// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionGuidViewModelRegistry.h"
#include "Extensions/IAvaTransitionGuidExtension.h"
#include "ViewModels/AvaTransitionViewModel.h"

bool TAvaTransitionViewModelRegistryKey<FGuid>::IsValid(const FGuid& InKey)
{
	return InKey.IsValid();
}

bool TAvaTransitionViewModelRegistryKey<FGuid>::TryGetKey(const TSharedRef<FAvaTransitionViewModel>& InViewModel, FGuid& OutKey)
{
	if (IAvaTransitionGuidExtension* GuidExtension = InViewModel->CastTo<IAvaTransitionGuidExtension>())
	{
		OutKey = GuidExtension->GetGuid();
		return true;
	}
	return false;
}
