// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionViewModelRegistryKey.h"
#include "Misc/Guid.h"

template<>
struct TAvaTransitionViewModelRegistryKey<FGuid>
{
	static constexpr int32 Id = 2;

	static bool IsValid(const FGuid& InKey);

	static bool TryGetKey(const TSharedRef<FAvaTransitionViewModel>& InViewModel, FGuid& OutKey);
};
