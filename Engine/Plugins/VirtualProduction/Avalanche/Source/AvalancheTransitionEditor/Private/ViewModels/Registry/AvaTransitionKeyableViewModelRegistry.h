// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionViewModelRegistryKey.h"
#include "Containers/Map.h"
#include "IAvaTransitionViewModelRegistry.h"
#include "Templates/SharedPointer.h"

/** Registry Template for registries that allow finding a view model via a key */
template<typename InKeyType>
struct TAvaTransitionKeyableViewModelRegistry : IAvaTransitionViewModelRegistry
{
	using FRegistryKey = TAvaTransitionViewModelRegistryKey<InKeyType>;

	TSharedPtr<FAvaTransitionViewModel> FindViewModel(const InKeyType& InKey) const;

	//~ Begin IAvaTransitionViewModelRegistry
	virtual void RegisterViewModel(const TSharedRef<FAvaTransitionViewModel>& InViewModel) override final;
	virtual void Refresh() override final;
	//~ End IAvaTransitionViewModelRegistry

private:
	TMap<InKeyType, TWeakPtr<FAvaTransitionViewModel>> ViewModels;
};

template<typename InKeyType>
TSharedPtr<FAvaTransitionViewModel> TAvaTransitionKeyableViewModelRegistry<InKeyType>::FindViewModel(const InKeyType& InKey) const
{
	if (!FRegistryKey::IsValid(InKey))
	{
		return nullptr;
	}
	if (const TWeakPtr<FAvaTransitionViewModel>* FoundViewModel = ViewModels.Find(InKey))
	{
		return FoundViewModel->Pin();
	}
	return nullptr;
}

template<typename InKeyType>
void TAvaTransitionKeyableViewModelRegistry<InKeyType>::RegisterViewModel(const TSharedRef<FAvaTransitionViewModel>& InViewModel)
{
	InKeyType Key;
	if (FRegistryKey::TryGetKey(InViewModel, Key))
	{
		ViewModels.Add(Key, InViewModel);	
	}
}

template<typename InKeyType>
void TAvaTransitionKeyableViewModelRegistry<InKeyType>::Refresh()
{
	// Remove Stale View Models
	for (typename TMap<InKeyType, TWeakPtr<FAvaTransitionViewModel>>::TIterator Iter(ViewModels); Iter; ++Iter)
	{
		if (!Iter->Value.IsValid())
		{
			Iter.RemoveCurrent();
		}
	}
}
