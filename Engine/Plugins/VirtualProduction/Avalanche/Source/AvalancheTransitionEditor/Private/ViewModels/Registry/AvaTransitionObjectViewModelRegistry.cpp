// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionObjectViewModelRegistry.h"
#include "Extensions/IAvaTransitionObjectExtension.h"
#include "ViewModels/AvaTransitionViewModel.h"

bool TAvaTransitionViewModelRegistryKey<FObjectKey>::IsValid(const FObjectKey& InKey)
{
	return InKey != FObjectKey();
}

bool TAvaTransitionViewModelRegistryKey<FObjectKey>::TryGetKey(const TSharedRef<FAvaTransitionViewModel>& InViewModel, FObjectKey& OutKey)
{
	if (IAvaTransitionObjectExtension* ObjectExtension = InViewModel->CastTo<IAvaTransitionObjectExtension>())
	{
		if (UObject* Object = ObjectExtension->GetObject())
		{
			OutKey = Object;
			return true;
		}
	}
	return false;
}
