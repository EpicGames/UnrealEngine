// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionViewModelRegistryKey.h"
#include "UObject/ObjectKey.h"

template<>
struct TAvaTransitionViewModelRegistryKey<FObjectKey>
{
	static constexpr int32 Id = 1;

	static bool IsValid(const FObjectKey& InKey);

	static bool TryGetKey(const TSharedRef<FAvaTransitionViewModel>& InViewModel, FObjectKey& OutKey);
};
