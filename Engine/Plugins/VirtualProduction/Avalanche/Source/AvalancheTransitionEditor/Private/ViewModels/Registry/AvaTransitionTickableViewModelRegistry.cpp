// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionTickableViewModelRegistry.h"
#include "AvaTypeSharedPointer.h"
#include "Extensions/IAvaTransitionTickableExtension.h"
#include "ViewModels/AvaTransitionViewModel.h"

void FAvaTransitionTickableViewModelRegistry::RegisterViewModel(const TSharedRef<FAvaTransitionViewModel>& InViewModel)
{
	if (TSharedPtr<IAvaTransitionTickableExtension> Tickable = UE::AvaCore::CastSharedPtr<IAvaTransitionTickableExtension>(InViewModel))
	{
		TickablesWeak.Add(Tickable);
	}
}

void FAvaTransitionTickableViewModelRegistry::Refresh()
{
	TickablesWeak.RemoveAll(
		[](const TWeakPtr<IAvaTransitionTickableExtension>& InTickable)
		{
			return !InTickable.IsValid();	
		});
}

TStatId FAvaTransitionTickableViewModelRegistry::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FAvaTransitionTickableViewModelRegistry, STATGROUP_Tickables);
}

void FAvaTransitionTickableViewModelRegistry::Tick(float InDeltaTime)
{
	for (const TWeakPtr<IAvaTransitionTickableExtension>& TickableWeak : TickablesWeak)
	{
		if (TSharedPtr<IAvaTransitionTickableExtension> Tickable = TickableWeak.Pin())
		{
			Tickable->Tick(InDeltaTime);
		}
	}
}
