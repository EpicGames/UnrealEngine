// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IAvaTransitionViewModelRegistry.h"
#include "Templates/SharedPointer.h"
#include "TickableEditorObject.h"

class IAvaTransitionTickableExtension;

/** Registry containing all View Models that are tickable */
struct FAvaTransitionTickableViewModelRegistry : IAvaTransitionViewModelRegistry, FTickableEditorObject
{
	//~ Begin IAvaTransitionViewModelRegistry
	virtual void RegisterViewModel(const TSharedRef<FAvaTransitionViewModel>& InViewModel) override;
	virtual void Refresh() override;
	//~ End IAvaTransitionViewModelRegistry

	//~ Begin FTickableEditorObject
	virtual TStatId GetStatId() const override;
	virtual void Tick(float InDeltaTime) override;
	//~ End FTickableEditorObject

private:
	TArray<TWeakPtr<IAvaTransitionTickableExtension>> TickablesWeak;
};
