// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionViewModelRegistryCollection.h"
#include "AvaTransitionTickableViewModelRegistry.h"

namespace UE::AvaTransitionEditor::Private
{
	template<typename InKeyType>
	TPair<int32, TSharedRef<IAvaTransitionViewModelRegistry>> CreateKeyableRegistry()
	{
		return { TAvaTransitionViewModelRegistryKey<InKeyType>::Id, MakeShared<TAvaTransitionKeyableViewModelRegistry<InKeyType>>() };
	}
}

FAvaTransitionViewModelRegistryCollection::FAvaTransitionViewModelRegistryCollection()
	: KeyableRegistries({
		UE::AvaTransitionEditor::Private::CreateKeyableRegistry<FObjectKey>(),
		UE::AvaTransitionEditor::Private::CreateKeyableRegistry<FGuid>(),
	})
	, PrivateRegistries({
		MakeShared<FAvaTransitionTickableViewModelRegistry>(),
	})
{
}

void FAvaTransitionViewModelRegistryCollection::RegisterViewModel(const TSharedRef<FAvaTransitionViewModel>& InViewModel)
{
	for (const TPair<int32, TSharedRef<IAvaTransitionViewModelRegistry>>& Pair : KeyableRegistries)
	{
		Pair.Value->RegisterViewModel(InViewModel);
	}

	for (const TSharedRef<IAvaTransitionViewModelRegistry>& Registry : PrivateRegistries)
	{
		Registry->RegisterViewModel(InViewModel);
	}
}

void FAvaTransitionViewModelRegistryCollection::Refresh()
{
	for (const TPair<int32, TSharedRef<IAvaTransitionViewModelRegistry>>& Pair : KeyableRegistries)
	{
		Pair.Value->Refresh();
	}

	for (const TSharedRef<IAvaTransitionViewModelRegistry>& Registry : PrivateRegistries)
	{
		Registry->Refresh();
	}
}
