// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTransitionGuidViewModelRegistry.h"
#include "AvaTransitionKeyableViewModelRegistry.h"
#include "AvaTransitionObjectViewModelRegistry.h"
#include "Containers/Map.h"
#include "IAvaTransitionViewModelRegistry.h"
#include "Templates/SharedPointer.h"

class FAvaTransitionViewModel;
struct IAvaTransitionViewModelRegistry;

class FAvaTransitionViewModelRegistryCollection
{
public:
	FAvaTransitionViewModelRegistryCollection();

	void RegisterViewModel(const TSharedRef<FAvaTransitionViewModel>& InViewModel);

	template<typename InKeyType UE_REQUIRES(TAvaTransitionViewModelRegistryKey<InKeyType>::Id != -1)>
	TSharedPtr<FAvaTransitionViewModel> FindViewModel(const InKeyType& InKey) const
	{
		constexpr int32 Id = TAvaTransitionViewModelRegistryKey<InKeyType>::Id;
		return StaticCastSharedRef<TAvaTransitionKeyableViewModelRegistry<InKeyType>>(KeyableRegistries[Id])->FindViewModel(InKey);
	}

	TSharedPtr<FAvaTransitionViewModel> FindViewModel(const UObject* InObject) const
	{
		return FindViewModel(FObjectKey(InObject));
	}

	void Refresh();

private:
	/**
	 * Registries holding Keyable View Models (i.e. view models that can be found via a key)
	 * @see FAvaTransitionViewModelRegistryCollection::FindViewModel
	 */
	const TMap<int32, TSharedRef<IAvaTransitionViewModelRegistry>> KeyableRegistries;

	/** Private registries used for their own contained purpose. These registries aren't used to find a view model */
	const TArray<TSharedRef<IAvaTransitionViewModelRegistry>> PrivateRegistries;
};
