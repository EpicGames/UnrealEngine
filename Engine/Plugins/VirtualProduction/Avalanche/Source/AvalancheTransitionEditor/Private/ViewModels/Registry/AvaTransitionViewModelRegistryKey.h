// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/SharedPointerFwd.h"

class FAvaTransitionViewModel;

/** Template struct to allow for specializations to handle the Registry Key Type */
template<typename InKeyType>
struct TAvaTransitionViewModelRegistryKey
{
	/** Id unique to each specialization */
	static constexpr int32 Id = -1;

	/** Determines whether the provided Key is valid or not */
	static bool IsValid(const InKeyType& InKey);

	/**
	 * Attempts to get a key of the given type from the provided view model
	 * @return true if succeeded in getting the key, false otherwise
	 */
	static bool TryGetKey(const TSharedRef<FAvaTransitionViewModel>& InViewModel, InKeyType& OutKey);
};
