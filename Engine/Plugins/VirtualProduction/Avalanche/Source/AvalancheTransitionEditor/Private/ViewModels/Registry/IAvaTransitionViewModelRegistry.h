// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

class FAvaTransitionViewModel;

/**
 * Interface to allow View Models to be registered via some common interface or trait
 * @see TAvaTransitionViewModelRegistry, TAvaTransitionViewModelRegistryKey
 */
struct IAvaTransitionViewModelRegistry
{
	virtual ~IAvaTransitionViewModelRegistry() = default;

	virtual void RegisterViewModel(const TSharedRef<FAvaTransitionViewModel>& InViewModel) = 0;

	virtual void Refresh() = 0;
};
