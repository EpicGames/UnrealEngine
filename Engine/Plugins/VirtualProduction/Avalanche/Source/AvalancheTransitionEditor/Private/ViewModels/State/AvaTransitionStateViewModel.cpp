// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionStateViewModel.h"
#include "Actions/AvaTransitionStateActions.h"
#include "AvaTransitionSelection.h"
#include "AvaTransitionTreeEditorData.h"
#include "DragDrop/AvaTransitionStateDragDropOp.h"
#include "StateTreeState.h"
#include "Styling/AvaTransitionEditorStyle.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "ViewModels/AvaTransitionViewModelSharedData.h"
#include "ViewModels/AvaTransitionViewModelUtils.h"
#include "ViewModels/Condition/AvaTransitionConditionContainerViewModel.h"
#include "ViewModels/Task/AvaTransitionTaskContainerViewModel.h"
#include "ViewModels/Transition/AvaTransitionTransitionContainerViewModel.h"
#include "Views/SAvaTransitionStateRow.h"
#include "Views/SAvaTransitionStateView.h"
#include "Widgets/SBoxPanel.h"

#if UE_AVA_WITH_TRANSITION_DEBUG
#include "Views/Debugger/SAvaTransitionStateDebugInstanceContainer.h"
#endif

#define LOCTEXT_NAMESPACE "AvaTransitionStateViewModel"

FAvaTransitionStateViewModel::FAvaTransitionStateViewModel(UStateTreeState* InState)
	: StateId(InState ? InState->ID : FGuid())
	, StateWeak(InState)
{
}

bool FAvaTransitionStateViewModel::IsStateEnabled() const
{
	UStateTreeState* State = GetState();
	return State && State->bEnabled;
}

void FAvaTransitionStateViewModel::SetStateEnabled(bool bInEnabled)
{
	UStateTreeState* State = GetState();
	if (State && State->bEnabled != bInEnabled)
	{
		State->Modify();
		State->bEnabled = bInEnabled;
	}
}

UStateTreeState* FAvaTransitionStateViewModel::GetState() const
{
	return StateWeak.Get();
}

UStateTreeState* FAvaTransitionStateViewModel::GetParentState() const
{
	if (TSharedPtr<FAvaTransitionStateViewModel> StateParent = UE::AvaCore::CastSharedPtr<FAvaTransitionStateViewModel>(GetParent()))
	{
		return StateParent->GetState();
	}
	return nullptr;
}

UAvaTransitionTreeEditorData* FAvaTransitionStateViewModel::GetEditorData() const
{
	if (TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = GetSharedData()->GetEditorViewModel())
	{
		return EditorViewModel->GetEditorData();
	}
	return nullptr;
}

FSlateColor FAvaTransitionStateViewModel::GetStateColor() const
{
	UStateTreeState* State = GetState();
	UAvaTransitionTreeEditorData* EditorData = GetEditorData();

	if (State && EditorData)
	{
		if (const FStateTreeEditorColor* FoundColor = EditorData->FindColor(State->ColorRef))
		{
			// Root States don't have Parent
			const bool bIsRootState = !State->Parent;
			if (bIsRootState || State->Type == EStateTreeStateType::Subtree)
			{
				return FAvaTransitionEditorStyle::LerpColorSRGB(FoundColor->Color, FColor::Black, 0.25f);
			}
			return FoundColor->Color;
		}
	}

	return FAvaTransitionStateViewModel::GetDefaultStateColor();
}

bool FAvaTransitionStateViewModel::TryGetSelectionBehavior(EStateTreeStateSelectionBehavior& OutRuntimeBehavior, EStateTreeStateSelectionBehavior& OutStoredBehavior) const
{
	UStateTreeState* State = GetState();
	if (!State)
	{
		return false;
	}

	OutStoredBehavior = OutRuntimeBehavior = State->SelectionBehavior;

	switch(State->SelectionBehavior)
	{
	case EStateTreeStateSelectionBehavior::None:
	case EStateTreeStateSelectionBehavior::TryFollowTransitions:
	case EStateTreeStateSelectionBehavior::TryEnterState:
		return true;

	case EStateTreeStateSelectionBehavior::TrySelectChildrenInOrder:
		if (State->Children.IsEmpty())
		{
			// Backwards compatible behavior
			OutRuntimeBehavior = EStateTreeStateSelectionBehavior::TryEnterState;
		}
		return true;
	}

	checkNoEntry();
	return false;
}

FText FAvaTransitionStateViewModel::GetStateDescription() const
{
	return StateDescription;
}

FText FAvaTransitionStateViewModel::GetStateTooltip() const
{
	if (const UStateTreeState* State = GetState())
	{
		const UEnum* Enum = StaticEnum<EStateTreeStateType>();
		check(Enum);
		const int32 Index = Enum->GetIndexByValue(static_cast<int64>(State->Type));
		return Enum->GetToolTipTextByIndex(Index);
	}
	return FText::GetEmpty();
}

FStringView FAvaTransitionStateViewModel::GetComment() const
{
	if (UAvaTransitionTreeEditorData* EditorData = GetEditorData())
	{
		if (const FAvaTransitionStateMetadata* Metadata = EditorData->FindStateMetadata(StateId))
		{
			return Metadata->Comment;
		}
	}
	return FStringView();
}

void FAvaTransitionStateViewModel::SetComment(const FString& InComment)
{
	UAvaTransitionTreeEditorData* EditorData = GetEditorData();
	if (!EditorData)
	{
		return;
	}

	// Return early if there was no change in comment
	const FAvaTransitionStateMetadata* ExistingMetadata = EditorData->FindStateMetadata(StateId);
	if (ExistingMetadata && ExistingMetadata->Comment == InComment)
	{
		return;
	}

	EditorData->Modify();

	FAvaTransitionStateMetadata& Metadata = EditorData->FindOrAddStateMetadata(StateId);
	Metadata.Comment = InComment;
}

bool FAvaTransitionStateViewModel::HasAnyBreakpoint() const
{
#if UE_AVA_WITH_TRANSITION_DEBUG
	const UStateTreeState* State = GetState();
	const UStateTreeEditorData* EditorData = GetEditorData();
	return State && EditorData && EditorData->HasAnyBreakpoint(State->ID);
#else
	return false;
#endif
}

FText FAvaTransitionStateViewModel::GetBreakpointTooltip() const
{
#if UE_AVA_WITH_TRANSITION_DEBUG
	const UStateTreeState* State = GetState();
	const UStateTreeEditorData* EditorData = GetEditorData();
	if (State && EditorData)
	{
		int32 BreakpointType = 0;
		BreakpointType |= EditorData->HasBreakpoint(State->ID, EStateTreeBreakpointType::OnEnter) << 0;
		BreakpointType |= EditorData->HasBreakpoint(State->ID, EStateTreeBreakpointType::OnExit)  << 1;

		switch (BreakpointType)
		{
		case 1: return LOCTEXT("BreakpointOnEnterTooltip", "Break when entering state");
		case 2: return LOCTEXT("BreakpointOnExitTooltip", "Break when exiting state");
		case 3: return LOCTEXT("BreakpointOnEnterAndExitTooltip", "Break when entering or exiting state");
		}
	}
#endif
	return FText::GetEmpty();
}

void FAvaTransitionStateViewModel::UpdateStateDescription(bool bInSetStateName)
{
	StateDescription = FText::GetEmpty();

	UStateTreeState* State = GetState();
	if (!State)
	{
		return;
	}

	// Advanced Mode: If name not none show State Descriptions from the State Name (Raw)
	if (State->Name != NAME_None && GetSharedData()->GetEditorMode() == EAvaTransitionEditorMode::Advanced)
	{
		StateDescription = FText::FromName(State->Name);
		return;
	}

	if (ConditionContainer.IsValid())
	{
		StateDescription = ConditionContainer->UpdateStateDescription();
	}

	if (StateDescription.IsEmpty() && !State->Parent)
	{
		StateDescription = LOCTEXT("RootState", "Root");
	}

	if (bInSetStateName)
	{
		if (GUndo)
		{
			State->Modify();
		}
		State->Name = *StateDescription.ToString();
	}
}

#if UE_AVA_WITH_TRANSITION_DEBUG
TSharedRef<SWidget> FAvaTransitionStateViewModel::GetOrCreateDebugIndicatorWidget()
{
	if (!DebugIndicatorWidget.IsValid())
	{
		DebugIndicatorWidget = SNew(SAvaTransitionStateDebugInstanceContainer, SharedThis(this));
	}
	return DebugIndicatorWidget.ToSharedRef();
}

const FAvaTransitionStateDebugInstance* FAvaTransitionStateViewModel::FindDebugInstance(const FStateTreeInstanceDebugId& InId) const
{
	return DebugInstances.FindByKey(InId);
}
#endif // UE_AVA_WITH_TRANSITION_DEBUG

void FAvaTransitionStateViewModel::GatherChildren(FAvaTransitionViewModelChildren& OutChildren)
{
	UStateTreeState* State = GetState();
	if (!State)
	{
		return;
	}

	OutChildren.Reserve(State->Children.Num() + 3);

	for (UStateTreeState* ChildState : State->Children)
	{
		OutChildren.Add<FAvaTransitionStateViewModel>(ChildState);
	}

	ConditionContainer  = OutChildren.Add<FAvaTransitionConditionContainerViewModel>(State);
	TaskContainer       = OutChildren.Add<FAvaTransitionTaskContainerViewModel>(State);
	TransitionContainer = OutChildren.Add<FAvaTransitionTransitionContainerViewModel>(State);
}

bool FAvaTransitionStateViewModel::IsValid() const
{
	return StateWeak.IsValid();
}

void FAvaTransitionStateViewModel::SetSelected(bool bInIsSelected)
{
	bIsSelected = bInIsSelected;
}

bool FAvaTransitionStateViewModel::CanGenerateRow() const
{
	return IsValid();
}

TSharedRef<ITableRow> FAvaTransitionStateViewModel::GenerateRow(const TSharedRef<STableViewBase>& InOwningTableView)
{
	UpdateStateDescription(/*bSetStateName*/false);
	return SNew(SAvaTransitionStateRow, InOwningTableView, SharedThis(this));
}

bool FAvaTransitionStateViewModel::IsExpanded() const
{
	UStateTreeState* State = GetState();
	return State && State->bExpanded;
}

void FAvaTransitionStateViewModel::SetExpanded(bool bInIsExpanded)
{
	UStateTreeState* State = GetState();

	if (State && State->bExpanded != bInIsExpanded)
	{
		if (GUndo)
		{
			State->Modify();
		}
		State->bExpanded = bInIsExpanded;
	}
}

UObject* FAvaTransitionStateViewModel::GetObject() const
{
	return GetState();
}

void FAvaTransitionStateViewModel::OnPropertiesChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	// todo: currently any property change event triggers all container changed functions.
	// Ideally only the changed property should trigger when it's adding/removing/modifying order of array

	if (ConditionContainer)
	{
		ConditionContainer->OnConditionsChanged();
	}

	if (TaskContainer.IsValid())
	{
		TaskContainer->OnTasksChanged();
	}

	if (TransitionContainer.IsValid())
	{
		TransitionContainer->OnTransitionsChanged();
	}

	UpdateStateDescription(/*bSetStateName*/false);
}

TOptional<EItemDropZone> FAvaTransitionStateViewModel::OnCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone)
{
	if (TSharedPtr<FAvaTransitionStateDragDropOp> StateDragDropOp = InDragDropEvent.GetOperationAs<FAvaTransitionStateDragDropOp>())
	{
		if (FAvaTransitionStateActions::CanAddStatesFromDrop(SharedThis(this), *StateDragDropOp))
		{
			return InDropZone;
		}
	}
	return TOptional<EItemDropZone>();
}

FReply FAvaTransitionStateViewModel::OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone)
{
	if (TSharedPtr<FAvaTransitionStateDragDropOp> StateDragDropOp = InDragDropEvent.GetOperationAs<FAvaTransitionStateDragDropOp>())
	{
		if (FAvaTransitionStateActions::AddStatesFromDrop(SharedThis(this), InDropZone, *StateDragDropOp))
		{
			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

FReply FAvaTransitionStateViewModel::OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		return FReply::Handled().BeginDragDrop(FAvaTransitionStateDragDropOp::New(SharedThis(this), InMouseEvent.IsAltDown()));
	}
	return FReply::Unhandled();
}

const FGuid& FAvaTransitionStateViewModel::GetGuid() const
{
	return StateId;
}

void FAvaTransitionStateViewModel::Tick(float InDeltaTime)
{
#if UE_AVA_WITH_TRANSITION_DEBUG
	for (FAvaTransitionStateDebugInstance& StateDebugInstance : DebugInstances)
	{
		StateDebugInstance.Tick(InDeltaTime);
	}

	// Remove all Instances that have fully exited
	int32 RemoveCount = DebugInstances.RemoveAll(
		[](const FAvaTransitionStateDebugInstance& InStateDebugInstance)
		{
			return InStateDebugInstance.GetDebugStatus() == EAvaTransitionStateDebugStatus::Exited;	
		});

	if (RemoveCount > 0 && DebugIndicatorWidget.IsValid())
	{
		DebugIndicatorWidget->Refresh();
	}
#endif // UE_AVA_WITH_TRANSITION_DEBUG
}

#if UE_AVA_WITH_TRANSITION_DEBUG
void FAvaTransitionStateViewModel::DebugEnter(const FAvaTransitionDebugInfo& InDebugInfo)
{
	FAvaTransitionStateDebugInstance* DebugInstance = DebugInstances.FindByKey(InDebugInfo.Id);
	if (!DebugInstance)
	{
		DebugInstance = &DebugInstances.Emplace_GetRef(InDebugInfo);

		if (DebugIndicatorWidget.IsValid())
		{
			DebugIndicatorWidget->Refresh();	
		}
	}

	check(DebugInstance);
	DebugInstance->Enter();
}

void FAvaTransitionStateViewModel::DebugExit(const FAvaTransitionDebugInfo& InDebugInfo)
{
	if (FAvaTransitionStateDebugInstance* DebugInstance = DebugInstances.FindByKey(InDebugInfo.Id))
	{
		DebugInstance->Exit();
	}
}
#endif // UE_AVA_WITH_TRANSITION_DEBUG

#undef LOCTEXT_NAMESPACE
