// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debugger/AvaTransitionDebugDefinitions.h"
#include "Extensions/IAvaTransitionDebuggableExtension.h"
#include "Extensions/IAvaTransitionDragDropExtension.h"
#include "Extensions/IAvaTransitionGuidExtension.h"
#include "Extensions/IAvaTransitionObjectExtension.h"
#include "Extensions/IAvaTransitionSelectableExtension.h"
#include "Extensions/IAvaTransitionTickableExtension.h"
#include "Extensions/IAvaTransitionTreeRowExtension.h"
#include "Extensions/IAvaTransitionWidgetExtension.h"
#include "Math/Color.h"
#include "Styling/SlateColor.h"
#include "UObject/WeakObjectPtr.h"
#include "ViewModels/AvaTransitionViewModel.h"

#if UE_AVA_WITH_TRANSITION_DEBUG
#include "Debugger/AvaTransitionStateDebugInstance.h"
#endif

class FAvaTransitionConditionContainerViewModel;
class FAvaTransitionTaskContainerViewModel;
class FAvaTransitionTransitionContainerViewModel;
class UAvaTransitionTreeEditorData;
class UStateTreeState;
enum class EStateTreeStateSelectionBehavior : uint8;
struct FAvaTransitionAutogeneratedStateContext;

#if UE_AVA_WITH_TRANSITION_DEBUG
class SAvaTransitionStateDebugInstanceContainer;
struct FStateTreeInstanceDebugId;
#endif

/** View Model for a State Tree State */
class FAvaTransitionStateViewModel
	: public FAvaTransitionViewModel
	, public IAvaTransitionSelectableExtension
	, public IAvaTransitionTreeRowExtension
	, public IAvaTransitionObjectExtension
	, public IAvaTransitionDragDropExtension
	, public IAvaTransitionGuidExtension
	, public IAvaTransitionTickableExtension
	, public IAvaTransitionDebuggableExtension
{
public:
	UE_AVA_INHERITS(FAvaTransitionStateViewModel
		, FAvaTransitionViewModel
		, IAvaTransitionSelectableExtension
		, IAvaTransitionTreeRowExtension
		, IAvaTransitionObjectExtension
		, IAvaTransitionDragDropExtension
		, IAvaTransitionGuidExtension
		, IAvaTransitionTickableExtension
		, IAvaTransitionDebuggableExtension
	)

	explicit FAvaTransitionStateViewModel(UStateTreeState* InState);

	bool IsStateEnabled() const;

	void SetStateEnabled(bool bInEnabled);

	const FGuid& GetStateId() const
	{
		return StateId;
	}

	UStateTreeState* GetState() const;

	UStateTreeState* GetParentState() const;

	UAvaTransitionTreeEditorData* GetEditorData() const;

	/** Get the Default State Color to use when the State Color is unable to be retrieved */
	static FSlateColor GetDefaultStateColor()
	{
		return FLinearColor(FColor(31, 151, 167));
	}

	/** Gets the color for the State, assigned by the Editor Data */
	FSlateColor GetStateColor() const;

	/**
	 * Returns both the Stored Selection Behavior and the one that will run for a given State.
	 * These two will typically be the same all the conditions to have such behavior are met
	 */
	bool TryGetSelectionBehavior(EStateTreeStateSelectionBehavior& OutRuntimeBehavior, EStateTreeStateSelectionBehavior& OutStoredBehavior) const;

	FText GetStateDescription() const;

	FText GetStateTooltip() const;

	FStringView GetComment() const;

	void SetComment(const FString& InComment);

	bool HasAnyBreakpoint() const;

	FText GetBreakpointTooltip() const;

	void UpdateStateDescription(bool bInSetStateName);

	TSharedPtr<FAvaTransitionConditionContainerViewModel> GetConditionContainer() const
	{
		return ConditionContainer;
	}

	TSharedPtr<FAvaTransitionTaskContainerViewModel> GetTaskContainer() const
	{
		return TaskContainer;
	}

	TSharedPtr<FAvaTransitionTransitionContainerViewModel> GetTransitionContainer() const
	{
		return TransitionContainer;
	}

#if UE_AVA_WITH_TRANSITION_DEBUG
	TConstArrayView<FAvaTransitionStateDebugInstance> GetDebugInstances() const
	{
		return DebugInstances;
	}

	TSharedRef<SWidget> GetOrCreateDebugIndicatorWidget();

	const FAvaTransitionStateDebugInstance* FindDebugInstance(const FStateTreeInstanceDebugId& InId) const;
#endif

	//~ Begin FAvaTransitionViewModel
	virtual void GatherChildren(FAvaTransitionViewModelChildren& OutChildren) override;
	virtual bool IsValid() const override;
	//~ End FAvaTransitionViewModel

	//~ Begin IAvaTransitionSelectableExtension
	virtual void SetSelected(bool bInIsSelected) override;
	virtual bool IsSelected() const override { return bIsSelected; }
	//~ End IAvaTransitionSelectableExtension

	//~ Begin IAvaTransitionTreeRowExtension
	virtual bool CanGenerateRow() const override;
	virtual TSharedRef<ITableRow> GenerateRow(const TSharedRef<STableViewBase>& InOwningTableView) override;
	virtual bool IsExpanded() const override;
	virtual void SetExpanded(bool bInIsExpanded) override;
	//~ End IAvaTransitionTreeRowExtension

	//~ Begin IAvaTransitionObjectExtension
	virtual UObject* GetObject() const override;
	virtual void OnPropertiesChanged(const FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End IAvaTransitionObjectExtension

	//~ Begin IAvaTransitionDragDropExtension
	virtual TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) override;
	virtual FReply OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) override;
	virtual FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	//~ End IAvaTransitionDragDropExtension

	//~ Begin IAvaTransitionGuidExtension
	virtual const FGuid& GetGuid() const override;
	//~ End IAvaTransitionGuidExtension

	//~ Begin IAvaTransitionTickableExtension
	virtual void Tick(float InDeltaTime) override;
	//~ End IAvaTransitionTickableExtension

#if UE_AVA_WITH_TRANSITION_DEBUG
	//~ Begin IAvaTransitionDebuggableExtension
	virtual void DebugEnter(const FAvaTransitionDebugInfo& InDebugInfo) override;
	virtual void DebugExit(const FAvaTransitionDebugInfo& InDebugInfo) override;
	//~ End IAvaTransitionDebuggableExtension
#endif

private:
	/** Cached Guid of the State to avoid having to resolve the State for something that is unchanged */
	FGuid StateId;

	TWeakObjectPtr<UStateTreeState> StateWeak;

	TSharedPtr<FAvaTransitionConditionContainerViewModel> ConditionContainer;

	TSharedPtr<FAvaTransitionTaskContainerViewModel> TaskContainer;

	TSharedPtr<FAvaTransitionTransitionContainerViewModel> TransitionContainer;

#if UE_AVA_WITH_TRANSITION_DEBUG
	TArray<FAvaTransitionStateDebugInstance> DebugInstances;

	TSharedPtr<SAvaTransitionStateDebugInstanceContainer> DebugIndicatorWidget;
#endif

	FText StateDescription;

	bool bIsSelected = false;
};
