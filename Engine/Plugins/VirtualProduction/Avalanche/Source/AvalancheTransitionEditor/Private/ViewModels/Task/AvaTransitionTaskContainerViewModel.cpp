// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionTaskContainerViewModel.h"
#include "AvaTransitionTaskViewModel.h"
#include "StateTreeState.h"
#include "ViewModels/AvaTransitionViewModelUtils.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "AvaTransitionTaskContainerViewModel"

FAvaTransitionTaskContainerViewModel::FAvaTransitionTaskContainerViewModel(UStateTreeState* InState)
	: FAvaTransitionContainerViewModel(InState)
	, TaskBox(SNew(SHorizontalBox))
{
}

void FAvaTransitionTaskContainerViewModel::RefreshTaskBox()
{
	TaskBox->ClearChildren();

	UE::AvaTransitionEditor::ForEachChildOfType<IAvaTransitionWidgetExtension>(*this,
		[this](const TAvaTransitionCastedViewModel<IAvaTransitionWidgetExtension>& InViewModel, EAvaTransitionIterationResult&)
		{
			TaskBox->AddSlot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Fill)
				.Padding(0)
				.AutoWidth()
				[
					InViewModel.Casted->CreateWidget()
				];
		});
}

void FAvaTransitionTaskContainerViewModel::OnTasksChanged()
{
	Refresh();
	RefreshTaskBox();
}

void FAvaTransitionTaskContainerViewModel::GatherChildren(FAvaTransitionViewModelChildren& OutChildren)
{
	UStateTreeState* State = GetState();
	if (!State)
	{
		return;
	}

	OutChildren.Reserve(State->Tasks.Num());

	// todo: Transition Tree Schema is set to support multiple tasks, but would be good to also handle/check that this is the case
	for (const FStateTreeEditorNode& Task : State->Tasks)
	{
		OutChildren.Add<FAvaTransitionTaskViewModel>(Task);
	}
}

TSharedRef<SWidget> FAvaTransitionTaskContainerViewModel::CreateWidget()
{
	RefreshTaskBox();
	return TaskBox;
}

#undef LOCTEXT_NAMESPACE
