// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extensions/IAvaTransitionWidgetExtension.h"
#include "ViewModels/AvaTransitionContainerViewModel.h"

class SHorizontalBox;
class UStateTreeState;

/** View Model serving as a container of all the Tasks in a State */
class FAvaTransitionTaskContainerViewModel : public FAvaTransitionContainerViewModel, public IAvaTransitionWidgetExtension
{
public:
	UE_AVA_INHERITS(FAvaTransitionTaskContainerViewModel, FAvaTransitionContainerViewModel, IAvaTransitionWidgetExtension)

	explicit FAvaTransitionTaskContainerViewModel(UStateTreeState* InState);

	void RefreshTaskBox();

	void OnTasksChanged();

	//~ Begin FAvaTransitionViewModel
	virtual void GatherChildren(FAvaTransitionViewModelChildren& OutChildren) override;
	//~ End FAvaTransitionViewModel

	//~ Begin IAvaTransitionWidgetExtension
	virtual TSharedRef<SWidget> CreateWidget() override;
	//~ End IAvaTransitionWidgetExtension

private:
	TSharedRef<SHorizontalBox> TaskBox;
};
