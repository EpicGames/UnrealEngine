// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionTaskViewModel.h"
#include "AvaTransitionTreeEditorData.h"
#include "StateTreeEditorData.h"
#include "StateTreeState.h"
#include "Styling/AvaTransitionEditorStyle.h"
#include "Styling/AvaTransitionTextStyling.h"
#include "Tasks/AvaTransitionTask.h"
#include "Textures/SlateIcon.h"
#include "ViewModels/AvaTransitionViewModelSharedData.h"
#include "ViewModels/AvaTransitionViewModelUtils.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "AvaTransitionTaskViewModel"

FAvaTransitionTaskViewModel::FAvaTransitionTaskViewModel(const FStateTreeEditorNode& InEditorNode)
	: FAvaTransitionNodeViewModel(InEditorNode)
{
}

FText FAvaTransitionTaskViewModel::GetTaskDescription() const
{
	return TaskDescription;
}

FSlateColor FAvaTransitionTaskViewModel::GetTaskColor() const
{
	FLinearColor TaskColor = FAvaTransitionEditorStyle::LerpColorSRGB(GetStateColor().GetSpecifiedColor(), FColor::White, 0.25f);
	return TaskColor.CopyWithNewOpacity(0.25f);
}

EVisibility FAvaTransitionTaskViewModel::GetTaskIconVisibility() const
{
	if (const FStateTreeNodeBase* Node = GetTypedNode<>())
	{
		if (!Node->GetIconName().IsNone())
		{
			return EVisibility::SelfHitTestInvisible;
		}
	}
	return EVisibility::Collapsed;
}

const FSlateBrush* FAvaTransitionTaskViewModel::GetTaskIcon() const
{
	if (const FStateTreeNodeBase* Node = GetTypedNode<>())
	{
		return FAvaTransitionEditorStyle::ParseIcon(Node->GetIconName()).GetIcon();
	}
	return nullptr;	
}

FSlateColor FAvaTransitionTaskViewModel::GetTaskIconColor() const
{
	if (const FStateTreeNodeBase* Node = GetTypedNode<>())
	{
		return FLinearColor(Node->GetIconColor());
	}
	return FSlateColor::UseForeground();
}

void FAvaTransitionTaskViewModel::UpdateTaskDescription()
{
	const UAvaTransitionTreeEditorData* EditorData = GetEditorData();
	const FStateTreeEditorNode* EditorNode = GetEditorNode();

	if (EditorData && EditorNode)
	{
		TaskDescription = EditorData->GetNodeDescription(*EditorNode, EStateTreeNodeFormatting::RichText);
	}
	else
	{
		TaskDescription = FText::GetEmpty();
	}
}

bool FAvaTransitionTaskViewModel::IsEnabled() const
{
	const FStateTreeTaskBase* Task = GetTypedNode<FStateTreeTaskBase>();
	return Task && Task->bTaskEnabled;
}

EVisibility FAvaTransitionTaskViewModel::GetBreakpointVisibility() const
{
#if UE_AVA_WITH_TRANSITION_DEBUG
	const UAvaTransitionTreeEditorData* EditorData = GetEditorData();
	if (EditorData && EditorData->HasAnyBreakpoint(GetNodeId()))
	{
		return EVisibility::Visible;
	}
#endif
	return EVisibility::Hidden;
}

FText FAvaTransitionTaskViewModel::GetBreakpointTooltip() const
{
#if UE_AVA_WITH_TRANSITION_DEBUG
	if (const UAvaTransitionTreeEditorData* EditorData = GetEditorData())
	{
		const bool bHasBreakpointOnEnter = EditorData->HasBreakpoint(GetNodeId(), EStateTreeBreakpointType::OnEnter);
		const bool bHasBreakpointOnExit  = EditorData->HasBreakpoint(GetNodeId(), EStateTreeBreakpointType::OnExit);
		if (bHasBreakpointOnEnter && bHasBreakpointOnExit)
		{
			return LOCTEXT("BreakpointOnEnterAndOnExitTooltip","Break when entering or exiting task");
		}

		if (bHasBreakpointOnEnter)
		{
			return LOCTEXT("BreakpointOnEnterTooltip","Break when entering task");
		}

		if (bHasBreakpointOnExit)
		{
			return LOCTEXT("BreakpointOnExitTooltip","Break when exiting task");
		}
	}
#endif // UE_AVA_WITH_TRANSITION_DEBUG

	return FText::GetEmpty();
}

TArrayView<FStateTreeEditorNode> FAvaTransitionTaskViewModel::GetNodes(UStateTreeState& InState) const
{
	if (InState.Tasks.IsEmpty())
	{
		return MakeArrayView(&InState.SingleTask, 1);
	}
	return InState.Tasks;
}

TSharedRef<SWidget> FAvaTransitionTaskViewModel::CreateWidget()
{
	UpdateTaskDescription();

	return SNew(SBorder)
		.VAlign(VAlign_Fill)
		.Padding(0)
		.IsEnabled(this, &FAvaTransitionTaskViewModel::IsEnabled)
		.BorderBackgroundColor(this, &FAvaTransitionTaskViewModel::GetTaskColor)
		.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
		[
			SNew(SOverlay)
			// Task Description
			+ SOverlay::Slot()
			.Padding(6.f, 0.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					SNew(SBox)
					.Padding(FMargin(0.f, 0.f, 2.f, 0.f))
					.Visibility(this, &FAvaTransitionTaskViewModel::GetTaskIconVisibility)
					[
						SNew(SImage)
						.Image(this, &FAvaTransitionTaskViewModel::GetTaskIcon)
						.ColorAndOpacity(this, &FAvaTransitionTaskViewModel::GetTaskIconColor)
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				[
					SNew(SRichTextBlock)
					.Margin(FMargin(4.f, 0.f))
					.Text(this, &FAvaTransitionTaskViewModel::GetTaskDescription)
					.ToolTipText(this, &FAvaTransitionTaskViewModel::GetTaskDescription)
					.TextStyle(&FAvaTransitionEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Task.Title"))
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
					+SRichTextBlock::Decorator(FAvaTransitionTextStyleDecorator::Create(TEXT(""), FAvaTransitionEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Task.Title")))
					+SRichTextBlock::Decorator(FAvaTransitionTextStyleDecorator::Create(TEXT("b"), FAvaTransitionEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Task.Title.Bold")))
					+SRichTextBlock::Decorator(FAvaTransitionTextStyleDecorator::Create(TEXT("s"), FAvaTransitionEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.Task.Title.Subdued")))
				]
			]
			// Task Breakpoint
			+ SOverlay::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					SNew(SBox)
					.Padding(FMargin(0.0f, -10.0f, 0.0f, 0.0f))
					[
						SNew(SImage)
						.DesiredSizeOverride(FVector2D(10.f, 10.f))
						.Image(FAvaTransitionEditorStyle::Get().GetBrush("StateTreeEditor.Debugger.Breakpoint.EnabledAndValid"))
						.Visibility(this, &FAvaTransitionTaskViewModel::GetBreakpointVisibility)
						.ToolTipText(this, &FAvaTransitionTaskViewModel::GetBreakpointTooltip)
					]
				]
			]
		];
}

#undef LOCTEXT_NAMESPACE
