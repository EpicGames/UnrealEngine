// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extensions/IAvaTransitionWidgetExtension.h"
#include "ViewModels/AvaTransitionNodeViewModel.h"

struct EVisibility;
struct FSlateBrush;
struct FSlateColor;
struct FStateTreeTaskBase;

/** View Model for a Task Node */
class FAvaTransitionTaskViewModel : public FAvaTransitionNodeViewModel, public IAvaTransitionWidgetExtension
{
public:
	UE_AVA_INHERITS(FAvaTransitionTaskViewModel, FAvaTransitionNodeViewModel, IAvaTransitionWidgetExtension)

	explicit FAvaTransitionTaskViewModel(const FStateTreeEditorNode& InEditorNode);

	FText GetTaskDescription() const;

	FSlateColor GetTaskColor() const;

	EVisibility GetTaskIconVisibility() const;

	const FSlateBrush* GetTaskIcon() const;

	FSlateColor GetTaskIconColor() const;

	bool IsEnabled() const;

	EVisibility GetBreakpointVisibility() const;

	FText GetBreakpointTooltip() const;

	void UpdateTaskDescription();

	//~ Begin FAvaTransitionNodeViewModel
	virtual TArrayView<FStateTreeEditorNode> GetNodes(UStateTreeState& InState) const override;
	//~ End FAvaTransitionNodeViewModel

	//~ Begin IAvaTransitionWidgetExtension
	virtual TSharedRef<SWidget> CreateWidget() override;
	//~ End IAvaTransitionWidgetExtension

private:
	FText TaskDescription;
};
