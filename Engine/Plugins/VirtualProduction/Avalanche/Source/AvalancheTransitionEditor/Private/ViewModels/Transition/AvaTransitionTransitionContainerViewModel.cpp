// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionTransitionContainerViewModel.h"
#include "AvaTransitionTransitionViewModel.h"
#include "StateTreeState.h"
#include "ViewModels/AvaTransitionViewModelUtils.h"
#include "Widgets/SBoxPanel.h"

FAvaTransitionTransitionContainerViewModel::FAvaTransitionTransitionContainerViewModel(UStateTreeState* InState)
	: FAvaTransitionContainerViewModel(InState)
	, TransitionBox(SNew(SHorizontalBox))
{
}

void FAvaTransitionTransitionContainerViewModel::RefreshTransitionBox()
{
	TransitionBox->ClearChildren();

	UE::AvaTransitionEditor::ForEachChildOfType<IAvaTransitionWidgetExtension>(*this,
		[this](const TAvaTransitionCastedViewModel<IAvaTransitionWidgetExtension>& InViewModel, EAvaTransitionIterationResult&)
		{
			TransitionBox->AddSlot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Fill)
				.Padding(0)
				.AutoWidth()
				.MaxWidth(200.f)
				[
					InViewModel.Casted->CreateWidget()
				];
		});
}

void FAvaTransitionTransitionContainerViewModel::OnTransitionsChanged()
{
	Refresh();
	RefreshTransitionBox();
}

void FAvaTransitionTransitionContainerViewModel::GatherChildren(FAvaTransitionViewModelChildren& OutChildren)
{
	UStateTreeState* State = GetState();
	if (!State)
	{
		return;
	}

	OutChildren.Reserve(State->Transitions.Num());

	for (const FStateTreeTransition& Transition : State->Transitions)
	{
		OutChildren.Add<FAvaTransitionTransitionViewModel>(Transition);
	}
}

TSharedRef<SWidget> FAvaTransitionTransitionContainerViewModel::CreateWidget()
{
	RefreshTransitionBox();
	return TransitionBox;
}
