// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extensions/IAvaTransitionWidgetExtension.h"
#include "ViewModels/AvaTransitionContainerViewModel.h"

class SHorizontalBox;
class UStateTreeState;

/** View Model serving as a container of all the Transitions in a State */
class FAvaTransitionTransitionContainerViewModel : public FAvaTransitionContainerViewModel, public IAvaTransitionWidgetExtension
{
public:
	UE_AVA_INHERITS(FAvaTransitionTransitionContainerViewModel, FAvaTransitionContainerViewModel, IAvaTransitionWidgetExtension)

	explicit FAvaTransitionTransitionContainerViewModel(UStateTreeState* InState);

	void RefreshTransitionBox();

	void OnTransitionsChanged();

	//~ Begin FAvaTransitionViewModel
	virtual void GatherChildren(FAvaTransitionViewModelChildren& OutChildren) override;
	//~ End FAvaTransitionViewModel

	//~ Begin IAvaTransitionWidgetExtension
	virtual TSharedRef<SWidget> CreateWidget() override;
	//~ End IAvaTransitionWidgetExtension

private:
	TSharedRef<SHorizontalBox> TransitionBox;
};
