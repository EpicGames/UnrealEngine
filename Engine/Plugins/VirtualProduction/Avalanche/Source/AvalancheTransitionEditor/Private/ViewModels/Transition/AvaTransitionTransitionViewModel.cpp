// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaTransitionTransitionViewModel.h"
#include "AvaTransitionTreeEditorData.h"
#include "EditorFontGlyphs.h"
#include "StateTreeState.h"
#include "Styling/AvaTransitionEditorStyle.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "ViewModels/AvaTransitionViewModelSharedData.h"
#include "ViewModels/AvaTransitionViewModelUtils.h"
#include "ViewModels/Condition/AvaTransitionConditionViewModel.h"
#include "ViewModels/State/AvaTransitionStateViewModel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AvaTransitionTransitionViewModel"

FAvaTransitionTransitionViewModel::FAvaTransitionTransitionViewModel(const FStateTreeTransition& InTransition)
	: TransitionId(InTransition.ID)
{
}

UAvaTransitionTreeEditorData* FAvaTransitionTransitionViewModel::GetEditorData() const
{
	if (TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = GetSharedData()->GetEditorViewModel())
	{
		return EditorViewModel->GetEditorData();
	}
	return nullptr;
}

UStateTreeState* FAvaTransitionTransitionViewModel::GetState() const
{
	if (TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = UE::AvaTransitionEditor::FindAncestorOfType<FAvaTransitionStateViewModel>(*this))
	{
		return StateViewModel->GetState();
	}
	return nullptr;
}

FStateTreeTransition* FAvaTransitionTransitionViewModel::GetTransition() const
{
	if (UStateTreeState* State = GetState())
	{
		return State->Transitions.FindByPredicate([this](const FStateTreeTransition& InTransition)
		{
			return TransitionId == InTransition.ID;
		});
	}
	return nullptr;
}

const FSlateBrush* FAvaTransitionTransitionViewModel::GetIcon() const
{
	const FStateTreeTransition* Transition = GetTransition();
	if (!Transition)
	{
		return nullptr;
	}

	switch (Transition->State.LinkType)
	{
	case EStateTreeTransitionType::None:
		{
			const UStateTreeState* State = GetState();
			if (State && State->Children.IsEmpty() && State->Type == EStateTreeStateType::State && EnumHasAnyFlags(Transition->Trigger, EStateTreeTransitionTrigger::OnStateCompleted))
			{
				return FAvaTransitionEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Parent");
			}
		}
		// falls through

	case EStateTreeTransitionType::Succeeded:
	case EStateTreeTransitionType::Failed:
	case EStateTreeTransitionType::GotoState:
		return FAvaTransitionEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Goto");

	case EStateTreeTransitionType::NextState:
	case EStateTreeTransitionType::NextSelectableState:
		return FAvaTransitionEditorStyle::Get().GetBrush("StateTreeEditor.Transition.Next");

	default:
		ensureMsgf(false, TEXT("Unhandled transition type."));
		break;
	}

	return nullptr;
}

FText FAvaTransitionTransitionViewModel::GetDescription() const
{
	FStateTreeTransition* Transition = GetTransition();
	if (!Transition)
	{
		return FText::GetEmpty();
	}

	switch (Transition->State.LinkType)
	{
	case EStateTreeTransitionType::None:
		return LOCTEXT("TransitionNone", "None");

	case EStateTreeTransitionType::Succeeded:
		return LOCTEXT("TransitionSucceed", "Succeed");

	case EStateTreeTransitionType::Failed:
		return LOCTEXT("TransitionFail", "Fail");

	case EStateTreeTransitionType::NextState:
		return LOCTEXT("TransitionNext", "Next");

	case EStateTreeTransitionType::NextSelectableState:
		return LOCTEXT("TransitionNextSelectable", "Next Selectable");

	case EStateTreeTransitionType::GotoState:
		return FText::FromName(Transition->State.Name);

	default:
		ensureMsgf(false, TEXT("Unhandled transition type."));
		break;
	}

	return FText::GetEmpty();
}

EVisibility FAvaTransitionTransitionViewModel::GetBreakpointVisibility() const
{
#if UE_AVA_WITH_TRANSITION_DEBUG
	const UAvaTransitionTreeEditorData* EditorData = GetEditorData();
	if (EditorData && EditorData->HasBreakpoint(TransitionId, EStateTreeBreakpointType::OnTransition))
	{
		return EVisibility::Visible;
	}
#endif
	return EVisibility::Collapsed;
}

void FAvaTransitionTransitionViewModel::GatherChildren(FAvaTransitionViewModelChildren& OutChildren)
{
	if (FStateTreeTransition* Transition = GetTransition())
	{
		OutChildren.Reserve(Transition->Conditions.Num());
		for (const FStateTreeEditorNode& Condition : Transition->Conditions)
		{
			OutChildren.Add<FAvaTransitionConditionViewModel>(Condition);
		}
	}
}

TSharedRef<SWidget> FAvaTransitionTransitionViewModel::CreateWidget()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(FMargin(8, 0, 0, 0))
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(this, &FAvaTransitionTransitionViewModel::GetIcon)
				.ColorAndOpacity(FLinearColor(1, 1, 1, 0.5f))
			]
			// Breakpoint box
			+ SOverlay::Slot()
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Left)
			.Padding(FMargin(0, -10, 0, 0))
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(10, 10))
				.Image(FAvaTransitionEditorStyle::Get().GetBrush(TEXT("StateTreeEditor.Debugger.Breakpoint.EnabledAndValid")))
				.Visibility(this, &FAvaTransitionTransitionViewModel::GetBreakpointVisibility)
			]
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(FMargin(4, 0, 0, 0))
		[
			SNew(STextBlock)
			.Text(this, &FAvaTransitionTransitionViewModel::GetDescription)
			.TextStyle(FAvaTransitionEditorStyle::Get(), "StateTree.Details")
		];
}

#undef LOCTEXT_NAMESPACE
