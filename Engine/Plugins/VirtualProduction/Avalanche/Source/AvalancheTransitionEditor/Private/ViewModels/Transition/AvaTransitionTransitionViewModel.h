// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extensions/IAvaTransitionWidgetExtension.h"
#include "Misc/Guid.h"
#include "ViewModels/AvaTransitionViewModel.h"

class UAvaTransitionTreeEditorData;
class UStateTreeState;
struct EVisibility;
struct FSlateBrush;
struct FStateTreeTransition;

/** View Model for a State Tree Transition */
class FAvaTransitionTransitionViewModel : public FAvaTransitionViewModel, public IAvaTransitionWidgetExtension
{
public:
	UE_AVA_INHERITS(FAvaTransitionTransitionViewModel, FAvaTransitionViewModel, IAvaTransitionWidgetExtension)

	explicit FAvaTransitionTransitionViewModel(const FStateTreeTransition& InTransition);

	UAvaTransitionTreeEditorData* GetEditorData() const;

	UStateTreeState* GetState() const;

	FStateTreeTransition* GetTransition() const;

	const FSlateBrush* GetIcon() const;

	FText GetDescription() const;

	EVisibility GetBreakpointVisibility() const;

	//~ Begin FAvaTransitionViewModel
	virtual void GatherChildren(FAvaTransitionViewModelChildren& OutChildren) override;
	//~ End FAvaTransitionViewModel

	//~ Begin IAvaTransitionWidgetExtension	
	virtual TSharedRef<SWidget> CreateWidget() override;
	//~ End IAvaTransitionWidgetExtension

private:
	FGuid TransitionId;
};
