// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaTransitionStateDebugInstance.h"

#if UE_AVA_WITH_TRANSITION_DEBUG
#include "Debugger/AvaTransitionStateDebugInstance.h"
#include "Styling/AvaTransitionEditorStyle.h"
#include "ViewModels/State/AvaTransitionStateViewModel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

void SAvaTransitionStateDebugInstance::Construct(const FArguments& InArgs, const TSharedPtr<FAvaTransitionStateViewModel>& InStateViewModel, const FAvaTransitionStateDebugInstance& InDebugInstance)
{
	StateViewModelWeak = InStateViewModel;

	InstanceDebugId = InDebugInstance.GetDebugInfo().Id;

	SetRenderOpacity(0.f);

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAvaTransitionEditorStyle::Get().GetBrush("DebugIndicatorBorder"))
		.BorderBackgroundColor(InDebugInstance.GetDebugInfo().Color)
		.Padding(FMargin(5.f, 2.f, 5.f, 3.f))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SBox)
				.HeightOverride(16.f)
				.WidthOverride(16.f)
				[
					SNew(SWidgetSwitcher)
					.WidgetIndex(this, &SAvaTransitionStateDebugInstance::GetWidgetIndex)
					+ SWidgetSwitcher::Slot()
					.Padding(0)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SCircularThrobber)
						.PieceImage(FAvaTransitionEditorStyle::Get().GetBrush("Throbber.CircleChunk"))
						.NumPieces(6)
						.Radius(6.f)
						.Period(1.0f)
					]
					+ SWidgetSwitcher::Slot()
					.Padding(0)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Success"))
					]
				]
			]
			+ SHorizontalBox::Slot()
			.Padding(5.f, 0.f, 5.f, 0.f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(FText::FromStringView(InDebugInstance.GetDebugInfo().Name))
			]
		]
	];
}

void SAvaTransitionStateDebugInstance::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (const FAvaTransitionStateDebugInstance* DebugInstance = FindDebugInstance())
	{
		float Progress = DebugInstance->GetStatusProgress();
		SetRenderOpacity(Progress);
	}
}

const FAvaTransitionStateDebugInstance* SAvaTransitionStateDebugInstance::FindDebugInstance() const
{
	if (TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = StateViewModelWeak.Pin())
	{
		return StateViewModel->FindDebugInstance(InstanceDebugId);
	}
	return nullptr;
}

int32 SAvaTransitionStateDebugInstance::GetWidgetIndex() const
{
	const FAvaTransitionStateDebugInstance* DebugInstance = FindDebugInstance();
	if (!DebugInstance)
	{
		return 0;
	}

	switch (DebugInstance->GetDebugStatus())
	{
	case EAvaTransitionStateDebugStatus::Entering:
	case EAvaTransitionStateDebugStatus::Entered:
		return 0;

	case EAvaTransitionStateDebugStatus::ExitingGrace:
	case EAvaTransitionStateDebugStatus::Exiting:
	case EAvaTransitionStateDebugStatus::Exited:
		return 1;
	}

	checkNoEntry();
	return 0;
}

#endif // UE_AVA_WITH_TRANSITION_DEBUG
