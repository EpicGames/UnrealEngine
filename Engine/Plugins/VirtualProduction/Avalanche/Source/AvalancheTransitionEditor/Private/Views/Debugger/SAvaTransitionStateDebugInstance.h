// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debugger/AvaTransitionDebugDefinitions.h"

#if UE_AVA_WITH_TRANSITION_DEBUG

#include "StateTreeExecutionTypes.h"
#include "Widgets/SCompoundWidget.h"

class FAvaTransitionStateDebugInstance;
class FAvaTransitionStateViewModel;

class SAvaTransitionStateDebugInstance : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaTransitionStateDebugInstance) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<FAvaTransitionStateViewModel>& InStateViewModel, const FAvaTransitionStateDebugInstance& InDebugInstance);

	//~ Begin SWidget
	virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget

private:
	const FAvaTransitionStateDebugInstance* FindDebugInstance() const;

	int32 GetWidgetIndex() const;

	TWeakPtr<FAvaTransitionStateViewModel> StateViewModelWeak;

	FStateTreeInstanceDebugId InstanceDebugId;
};

#endif // UE_AVA_WITH_TRANSITION_DEBUG
