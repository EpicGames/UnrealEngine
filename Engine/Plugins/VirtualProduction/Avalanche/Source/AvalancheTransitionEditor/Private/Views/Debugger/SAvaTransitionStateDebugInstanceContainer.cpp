// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaTransitionStateDebugInstanceContainer.h"

#if UE_AVA_WITH_TRANSITION_DEBUG

#include "Debugger/AvaTransitionDebugger.h"
#include "SAvaTransitionStateDebugInstance.h"
#include "Styling/AvaTransitionEditorStyle.h"
#include "ViewModels/AvaTransitionViewModelSharedData.h"
#include "ViewModels/State/AvaTransitionStateViewModel.h"
#include "Widgets/SBoxPanel.h"

void SAvaTransitionStateDebugInstanceContainer::Construct(const FArguments& InArgs, const TSharedRef<FAvaTransitionStateViewModel>& InStateViewModel)
{
	StateViewModelWeak = InStateViewModel;

	SetVisibility(TAttribute<EVisibility>::CreateSP(this, &SAvaTransitionStateDebugInstanceContainer::GetDebuggerVisibility));

	ChildSlot
	[
		SAssignNew(DebugInstanceContainer, SHorizontalBox)
	];

	Refresh();
}

void SAvaTransitionStateDebugInstanceContainer::Refresh()
{
	TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = StateViewModelWeak.Pin();
	if (!StateViewModel.IsValid())
	{
		return;
	}

	DebugInstanceContainer->ClearChildren();

	for (const FAvaTransitionStateDebugInstance& DebugInstance : StateViewModel->GetDebugInstances())
	{
		DebugInstanceContainer->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SAvaTransitionStateDebugInstance, StateViewModel, DebugInstance)
			];
	}
}

EVisibility SAvaTransitionStateDebugInstanceContainer::GetDebuggerVisibility() const
{
	if (TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = StateViewModelWeak.Pin())
	{
		TSharedRef<FAvaTransitionDebugger> Debugger = StateViewModel->GetSharedData()->GetDebugger();
		if (Debugger->IsActive())
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

#endif // UE_AVA_WITH_TRANSITION_DEBUG
