// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debugger/AvaTransitionDebugDefinitions.h"

#if UE_AVA_WITH_TRANSITION_DEBUG

#include "StateTreeExecutionTypes.h"
#include "Widgets/SCompoundWidget.h"

class FAvaTransitionStateDebugInstance;
class FAvaTransitionStateViewModel;
class SHorizontalBox;

/** Widget holding all the Debug Instance Widgets */
class SAvaTransitionStateDebugInstanceContainer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaTransitionStateDebugInstanceContainer) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FAvaTransitionStateViewModel>& InStateViewModel);

	void Refresh();

private:
	EVisibility GetDebuggerVisibility() const;

	TSharedPtr<SHorizontalBox> DebugInstanceContainer;

	TWeakPtr<FAvaTransitionStateViewModel> StateViewModelWeak;
};

#endif // UE_AVA_WITH_TRANSITION_DEBUG
