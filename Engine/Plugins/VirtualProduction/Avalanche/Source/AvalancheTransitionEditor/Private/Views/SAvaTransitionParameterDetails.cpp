// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaTransitionParameterDetails.h"
#include "SAvaTransitionTreeDetails.h"

void SAvaTransitionParameterDetails::Construct(const FArguments& InArgs, const TSharedRef<FAvaTransitionEditorViewModel>& InEditorViewModel)
{
	ChildSlot
	[
		SNew(SAvaTransitionTreeDetails, InEditorViewModel)
	];
}
