// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FAvaTransitionEditorViewModel;

class SAvaTransitionParameterDetails : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaTransitionParameterDetails){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FAvaTransitionEditorViewModel>& InEditorViewModel);
};
