// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaTransitionSelectionDetails.h"
#include "AvaTransitionSelection.h"
#include "Customizations/AvaStateTreeStateCustomization.h"
#include "Extensions/IAvaTransitionObjectExtension.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "StateTreeDelegates.h"
#include "StateTreeEditorModule.h"
#include "StateTreeState.h"
#include "ViewModels/AvaTransitionViewModel.h"

void SAvaTransitionSelectionDetails::Construct(const FArguments& InArgs, const TSharedRef<FAvaTransitionSelection>& InSelection)
{
	SelectionWeak = InSelection;

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(nullptr);
	DetailsView->OnFinishedChangingProperties().AddSP(this, &SAvaTransitionSelectionDetails::OnFinishedChangingProperties);

	// Simple mode has a Property Filter
	if (!InArgs._AdvancedView)
	{
		DetailsView->RegisterInstancedCustomPropertyLayout(UStateTreeState::StaticClass()
			, FOnGetDetailCustomizationInstance::CreateStatic(&FAvaStateTreeStateCustomization::MakeInstance));

		DetailsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateStatic(&FAvaStateTreeStateCustomization::IsPropertyVisible));
	}

	// Read-only
	if (InArgs._ReadOnly)
	{
		DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateLambda([]{ return false; }));
	}

	FStateTreeEditorModule::SetDetailPropertyHandlers(*DetailsView);
	OnSelectionChanged(InSelection->GetSelectedItems());

	OnParametersChangedHandle = UE::StateTree::Delegates::OnParametersChanged.AddSP(this, &SAvaTransitionSelectionDetails::Refresh);
	OnSelectionChangedHandle = InSelection->OnSelectionChanged().AddSP(this, &SAvaTransitionSelectionDetails::OnSelectionChanged);

	ChildSlot
	[
		DetailsView.ToSharedRef()
	];
}

SAvaTransitionSelectionDetails::~SAvaTransitionSelectionDetails()
{
	UE::StateTree::Delegates::OnParametersChanged.Remove(OnParametersChangedHandle);
	OnParametersChangedHandle.Reset();

	if (TSharedPtr<FAvaTransitionSelection> Selection = SelectionWeak.Pin())
	{
		Selection->OnSelectionChanged().Remove(OnSelectionChangedHandle);
		OnSelectionChangedHandle.Reset();
	}
}

void SAvaTransitionSelectionDetails::Refresh(const UStateTree& InStateTree)
{
	if (DetailsView.IsValid())
	{
		DetailsView->ForceRefresh();
	}
}

void SAvaTransitionSelectionDetails::OnSelectionChanged(TConstArrayView<TSharedRef<FAvaTransitionViewModel>> InSelectedItems)
{
	if (!DetailsView.IsValid())
	{
		return;
	}

	TArray<UObject*> ObjectsToView;

	for (const TSharedRef<FAvaTransitionViewModel>& Item : InSelectedItems)
	{
		if (IAvaTransitionObjectExtension* ObjectExtension = Item->CastTo<IAvaTransitionObjectExtension>())
		{
			if (UObject* Object = ObjectExtension->GetObject())
			{
				ObjectsToView.Add(Object);
			}
		}
	}

	DetailsView->SetObjects(ObjectsToView);
}

void SAvaTransitionSelectionDetails::OnFinishedChangingProperties(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	TSharedPtr<FAvaTransitionSelection> Selection = SelectionWeak.Pin();
	if (!Selection.IsValid())
	{
		return;
	}

	for (const TSharedRef<FAvaTransitionViewModel>& Item : Selection->GetSelectedItems())
	{
		if (IAvaTransitionObjectExtension* ObjectExtension = Item->CastTo<IAvaTransitionObjectExtension>())
		{
			ObjectExtension->OnPropertiesChanged(InPropertyChangedEvent);
		}
	}
}
