// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Delegates/IDelegateInstance.h"
#include "Widgets/SCompoundWidget.h"

class FAvaTransitionSelection;
class FAvaTransitionViewModel;
class IDetailsView;
class UAvaTransitionTreeEditorData;
class UStateTree;
struct FPropertyChangedEvent;

class SAvaTransitionSelectionDetails : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaTransitionSelectionDetails)
		: _AdvancedView(false)
		, _ReadOnly(false)
	{
	}
	SLATE_ARGUMENT(bool, AdvancedView)
	SLATE_ARGUMENT(bool, ReadOnly)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FAvaTransitionSelection>& InSelection);

	virtual ~SAvaTransitionSelectionDetails() override;

private:
	void Refresh(const UStateTree& InStateTree);

	void OnSelectionChanged(TConstArrayView<TSharedRef<FAvaTransitionViewModel>> InSelectedItems);

	void OnFinishedChangingProperties(const FPropertyChangedEvent& InPropertyChangedEvent);

	TWeakPtr<FAvaTransitionSelection> SelectionWeak;

	TSharedPtr<IDetailsView> DetailsView;

	FDelegateHandle OnParametersChangedHandle;
	FDelegateHandle OnSelectionChangedHandle;
};
