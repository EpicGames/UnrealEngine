// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaTransitionStateMetadata.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "ViewModels/AvaTransitionViewModelSharedData.h"
#include "ViewModels/State/AvaTransitionStateViewModel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SAvaTransitionStateMetadata"

void SAvaTransitionStateMetadata::Construct(const FArguments& InArgs, const TSharedRef<FAvaTransitionStateViewModel>& InStateViewModel)
{
	StateViewModelWeak = InStateViewModel;

	const bool bIsReadOnly = InStateViewModel->GetSharedData()->IsReadOnly();

	ChildSlot
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Fill)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
			.BorderBackgroundColor(InStateViewModel, &FAvaTransitionStateViewModel::GetStateColor)
			.IsEnabled(InStateViewModel, &FAvaTransitionStateViewModel::IsStateEnabled)
			.Padding(0)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.IsEnabled(!bIsReadOnly)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText(this, &SAvaTransitionStateMetadata::GetCommentText)
					.Visibility(this, &SAvaTransitionStateMetadata::GetCommentVisibility)
					.OnClicked(this, &SAvaTransitionStateMetadata::OnCommentButtonClicked)
					.ContentPadding(1)
					[
						SNew(SImage)
						.ColorAndOpacity(FLinearColor(1, 1, 1, 0.5f))
						.Image(FAppStyle::GetBrush("Icons.Comment"))
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SAssignNew(CommentEditableTextBox, SEditableTextBox)
					.Visibility(EVisibility::Collapsed) // set as initially collapsed
					.OnTextCommitted(this, &SAvaTransitionStateMetadata::OnCommentTextCommitted)
					.SelectAllTextWhenFocused(true)
				]
			]
		];
}

EVisibility SAvaTransitionStateMetadata::GetCommentVisibility() const
{
	TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = StateViewModelWeak.Pin();

	return StateViewModel.IsValid() && !StateViewModel->GetComment().IsEmpty()
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

FText SAvaTransitionStateMetadata::GetCommentText() const
{
	if (TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = StateViewModelWeak.Pin())
	{
		return FText::FromStringView(StateViewModel->GetComment());
	}
	return FText::GetEmpty();
}

FReply SAvaTransitionStateMetadata::OnCommentButtonClicked()
{
	if (!bEditingComment && CommentEditableTextBox.IsValid())
	{
		bEditingComment = true;

		CommentEditableTextBox->SetText(GetCommentText());
		CommentEditableTextBox->SetVisibility(EVisibility::Visible);

		return FReply::Handled()
			.SetUserFocus(CommentEditableTextBox.ToSharedRef(), EFocusCause::SetDirectly);
	}
	return FReply::Handled();
}

void SAvaTransitionStateMetadata::OnCommentTextCommitted(const FText& InCommentText, ETextCommit::Type InCommitType)
{
	bEditingComment = false;

	if (CommentEditableTextBox.IsValid())
	{
		CommentEditableTextBox->SetVisibility(EVisibility::Collapsed);
	}

	FScopedTransaction Transaction(LOCTEXT("EditStateComment", "Edit State Comment"));

	if (TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = StateViewModelWeak.Pin())
	{
		StateViewModel->SetComment(InCommentText.ToString());
	}
}

#undef LOCTEXT_NAMESPACE
