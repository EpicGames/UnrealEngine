// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FAvaTransitionStateViewModel;
class SEditableTextBox;

class SAvaTransitionStateMetadata : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaTransitionStateMetadata) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FAvaTransitionStateViewModel>& InStateViewModel);

private:
	EVisibility GetCommentVisibility() const;

	FText GetCommentText() const;

	FReply OnCommentButtonClicked();

	void OnCommentTextCommitted(const FText& InCommentText, ETextCommit::Type InCommitType);

	TWeakPtr<FAvaTransitionStateViewModel> StateViewModelWeak;

	TSharedPtr<SEditableTextBox> CommentEditableTextBox;

	bool bEditingComment = false;
};
