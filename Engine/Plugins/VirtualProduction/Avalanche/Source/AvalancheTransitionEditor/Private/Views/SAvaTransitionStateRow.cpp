// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaTransitionStateRow.h"
#include "Extensions/IAvaTransitionDragDropExtension.h"
#include "SAvaTransitionStateView.h"
#include "Styling/AvaTransitionEditorStyle.h"
#include "ViewModels/AvaTransitionViewModel.h"
#include "ViewModels/AvaTransitionViewModelSharedData.h"
#include "ViewModels/State/AvaTransitionStateViewModel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/SExpanderArrow.h"

void SAvaTransitionStateRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FAvaTransitionStateViewModel>& InStateViewModel)
{
	STableRow::FArguments TableRowArgs = STableRow::FArguments()
		.Style(&FAvaTransitionEditorStyle::Get().GetWidgetStyle<FTableRowStyle>("StateTree.Selection"))
		.Content()
		[
			SNew(SAvaTransitionStateView, InStateViewModel)
		];

	if (!InStateViewModel->GetSharedData()->IsReadOnly())
	{
		TableRowArgs
			.OnCanAcceptDrop(this, &SAvaTransitionStateRow::OnCanAcceptDrop)
			.OnAcceptDrop(this, &SAvaTransitionStateRow::OnAcceptDrop)
			.OnDragDetected(InStateViewModel, &IAvaTransitionDragDropExtension::OnDragDetected);
	}

	ConstructInternal(TableRowArgs, InOwnerTableView);
	ConstructChildren(InOwnerTableView->TableViewMode, TableRowArgs._Padding, TableRowArgs._Content.Widget);
}

void SAvaTransitionStateRow::ConstructChildren(ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent)
{
	SHorizontalBox::FSlot* ContentSlot = nullptr;

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			SAssignNew(ExpanderArrowWidget, SExpanderArrow, SharedThis(this))
			.StyleSet(ExpanderStyleSet)
			.ShouldDrawWires(false)
			.IndentAmount(32)
			.BaseIndentLevel(0)
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Left)
		.Expose(ContentSlot)
		.AutoWidth()
		[
			InContent
		]
	];

	InnerContentSlot = ContentSlot;
}

TOptional<EItemDropZone> SAvaTransitionStateRow::OnCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, TSharedPtr<FAvaTransitionViewModel> InItem)
{
	if (TSharedPtr<IAvaTransitionDragDropExtension> DragDropExtension = UE::AvaCore::CastSharedPtr<IAvaTransitionDragDropExtension>(InItem))
	{
		return DragDropExtension->OnCanAcceptDrop(InDragDropEvent, InDropZone);
	}
	return TOptional<EItemDropZone>();
}

FReply SAvaTransitionStateRow::OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, TSharedPtr<FAvaTransitionViewModel> InItem)
{
	if (TSharedPtr<IAvaTransitionDragDropExtension> DragDropExtension = UE::AvaCore::CastSharedPtr<IAvaTransitionDragDropExtension>(InItem))
	{
		return DragDropExtension->OnAcceptDrop(InDragDropEvent, InDropZone);
	}
	return FReply::Unhandled();
}
