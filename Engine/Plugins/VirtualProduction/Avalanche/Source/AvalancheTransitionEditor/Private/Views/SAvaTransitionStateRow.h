// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

class FAvaTransitionStateViewModel;
class FAvaTransitionViewModel;
class FDragDropEvent;
class FReply;
class IAvaTransitionTreeRowExtension;
enum class EItemDropZone;
template<typename OptionalType> struct TOptional;

class SAvaTransitionStateRow : public STableRow<TSharedPtr<FAvaTransitionViewModel>>
{
public:
	SLATE_BEGIN_ARGS(SAvaTransitionStateRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FAvaTransitionStateViewModel>& InStateViewModel);

protected:
	//~ Begin STableRow
	virtual void ConstructChildren(ETableViewMode::Type InOwnerTableMode, const TAttribute<FMargin>& InPadding, const TSharedRef<SWidget>& InContent) override;
	//~ End STableRow

	TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, TSharedPtr<FAvaTransitionViewModel> InItem);

	FReply OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, TSharedPtr<FAvaTransitionViewModel> InItem);
};
