// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaTransitionStateView.h"
#include "SAvaTransitionStateMetadata.h"
#include "StateTreeTypes.h"
#include "Styling/AvaTransitionEditorStyle.h"
#include "Styling/AvaTransitionWidgetStyling.h"
#include "ViewModels/Condition/AvaTransitionConditionContainerViewModel.h"
#include "ViewModels/State/AvaTransitionStateViewModel.h"
#include "ViewModels/Task/AvaTransitionTaskContainerViewModel.h"
#include "ViewModels/Transition/AvaTransitionTransitionContainerViewModel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SAvaTransitionStateView"

void SAvaTransitionStateView::Construct(const FArguments& InArgs, const TSharedRef<FAvaTransitionStateViewModel>& InStateViewModel)
{
	StateViewModelWeak = InStateViewModel;

	// The inner box of the state
	TSharedRef<SHorizontalBox> InnerStateBox = SNew(SHorizontalBox);

	// Add Condition Container to State Box
	if (TSharedPtr<FAvaTransitionConditionContainerViewModel> ConditionContainer = InStateViewModel->GetConditionContainer())
	{
		InnerStateBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				ConditionContainer->CreateWidget()
			];
	}

	// The Outer State Box that the Selection Outline will cover
	TSharedRef<SHorizontalBox> OuterStateBox = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(0)
		[
			CreateStateSlotWidget(InnerStateBox, InStateViewModel)
		];

	// Add Task Container Slots to the Outer State Box
	if (TSharedPtr<FAvaTransitionTaskContainerViewModel> TaskContainer = InStateViewModel->GetTaskContainer())
	{
		OuterStateBox->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Fill)
			.Padding(0)
			.AutoWidth()
			[
				TaskContainer->CreateWidget()
			];
	}

	// State Metadata
	OuterStateBox->AddSlot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Fill)
		.Padding(0)
		.AutoWidth()
		[
			SNew(SAvaTransitionStateMetadata, InStateViewModel)
		];

	TSharedRef<SHorizontalBox> RowContent = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(0.f, 4.f)
		[
			SNew(SBorder)
			.BorderImage(FAvaTransitionEditorStyle::Get().GetBrush("StateTree.State.Border"))
			.BorderBackgroundColor(this, &SAvaTransitionStateView::GetActiveStateColor)
			[
				OuterStateBox
			]
		];

	if (TSharedPtr<FAvaTransitionTransitionContainerViewModel> TransitionContainer = InStateViewModel->GetTransitionContainer())
	{
		RowContent->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Fill)
			.Padding(0)
			.AutoWidth()
			[
				TransitionContainer->CreateWidget()
			];
	}

#if UE_AVA_WITH_TRANSITION_DEBUG
	// Debug Info
	RowContent->AddSlot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(12.f, 0.f, 0.f, 0.f)
		.AutoWidth()
		[
			InStateViewModel->GetOrCreateDebugIndicatorWidget()
		];
#endif

	ChildSlot
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			RowContent
		];
}

TSharedRef<SWidget> SAvaTransitionStateView::CreateStateSlotWidget(const TSharedRef<SHorizontalBox>& InStateBox, const TSharedRef<FAvaTransitionStateViewModel>& InStateViewModel)
{
	// Selector
	InStateBox->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
			[
				SNew(SImage)
				.Image(this, &SAvaTransitionStateView::GetSelectorIcon)
				.ColorAndOpacity(FLinearColor(1, 1, 1, 0.5f))
				.ToolTipText(this, &SAvaTransitionStateView::GetSelectorTooltip)
			]
		];

	TSharedRef<SAvaTransitionStateView> This = SharedThis(this);

	// State Description
	InStateBox->AddSlot()
		.VAlign(VAlign_Center)
		.FillWidth(1.f)
		[
			SNew(SRichTextBlock)
			.Text(InStateViewModel, &FAvaTransitionStateViewModel::GetStateDescription)
			.ToolTipText(InStateViewModel, &FAvaTransitionStateViewModel::GetStateTooltip)
			.TextStyle(&FAvaTransitionEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("StateTree.State.Title"))
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
			+SRichTextBlock::WidgetDecorator(TEXT("op"), FWidgetDecorator::FCreateWidget::CreateStatic(&FAvaTransitionWidgetStyling::CreateOperandWidget))
		];

	return SNew(SBox)
		.VAlign(VAlign_Fill)
		.HeightOverride(24.f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
			.BorderBackgroundColor(InStateViewModel, &FAvaTransitionStateViewModel::GetStateColor)
			.IsEnabled(InStateViewModel, &FAvaTransitionStateViewModel::IsStateEnabled)
			.Padding(0)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				.Padding(FMargin(4.f, 2.f, 12.f, 2.f))
				[
					InStateBox
				]
				+ SOverlay::Slot()
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Left)
				.Padding(FMargin(-8.f, -8.f, 0.f, 0.f))
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(12.f, 12.f))
					.Image(FAvaTransitionEditorStyle::Get().GetBrush(TEXT("StateTreeEditor.Debugger.Breakpoint.EnabledAndValid")))
					.Visibility(this, &SAvaTransitionStateView::GetStateBreakpointVisibility)
					.ToolTipText(InStateViewModel, &FAvaTransitionStateViewModel::GetBreakpointTooltip)
				]
			]
		];
}

const FSlateBrush* SAvaTransitionStateView::GetSelectorIcon() const
{
	TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = StateViewModelWeak.Pin();

	EStateTreeStateSelectionBehavior RuntimeBehavior, StoredBehavior;
	if (!StateViewModel.IsValid() || !StateViewModel->TryGetSelectionBehavior(RuntimeBehavior, StoredBehavior))
	{
		return nullptr;
	}

	switch(RuntimeBehavior)
	{
	case EStateTreeStateSelectionBehavior::None:
		return FAvaTransitionEditorStyle::Get().GetBrush("StateTreeEditor.SelectNone");

	case EStateTreeStateSelectionBehavior::TryEnterState:
		return FAvaTransitionEditorStyle::Get().GetBrush("StateTreeEditor.TryEnterState");

	case EStateTreeStateSelectionBehavior::TrySelectChildrenInOrder:
		return FAvaTransitionEditorStyle::Get().GetBrush("StateTreeEditor.TrySelectChildrenInOrder");

	case EStateTreeStateSelectionBehavior::TryFollowTransitions:
		return FAvaTransitionEditorStyle::Get().GetBrush("StateTreeEditor.TryFollowTransitions");
	}

	checkNoEntry();
	return nullptr;
}

FText SAvaTransitionStateView::GetSelectorTooltip() const
{
	TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = StateViewModelWeak.Pin();

	EStateTreeStateSelectionBehavior RuntimeBehavior, StoredBehavior;
	if (!StateViewModel.IsValid() || !StateViewModel->TryGetSelectionBehavior(RuntimeBehavior, StoredBehavior))
	{
		return FText::GetEmpty();
	}

	const UEnum* Enum = StaticEnum<EStateTreeStateSelectionBehavior>();
	check(Enum);
	const int32 Index = Enum->GetIndexByValue(static_cast<int64>(RuntimeBehavior));

	if (RuntimeBehavior != StoredBehavior)
	{
		return FText::Format(LOCTEXT("ConvertedToState", "{0}\nAutomatically converted from '{1}' because the State did not satisfy the selection behavior requirements.")
			, Enum->GetToolTipTextByIndex(Index)
			, UEnum::GetDisplayValueAsText(StoredBehavior));
	}

	return Enum->GetToolTipTextByIndex(Index);
}

FSlateColor SAvaTransitionStateView::GetActiveStateColor() const
{
	TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = StateViewModelWeak.Pin();

	if (StateViewModel.IsValid() && StateViewModel->IsSelected())
	{
		return FLinearColor(FColor(236, 134, 39));
	}

	return FLinearColor::Transparent;
}

EVisibility SAvaTransitionStateView::GetStateBreakpointVisibility() const
{
	TSharedPtr<FAvaTransitionStateViewModel> StateViewModel = StateViewModelWeak.Pin();
	if (StateViewModel.IsValid() && StateViewModel->HasAnyBreakpoint())
	{
		return EVisibility::Visible;
	}
	return EVisibility::Hidden;
}

#undef LOCTEXT_NAMESPACE
