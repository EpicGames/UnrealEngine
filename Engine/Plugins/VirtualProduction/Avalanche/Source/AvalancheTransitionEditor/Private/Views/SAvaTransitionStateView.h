// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FAvaTransitionStateViewModel;
class SHorizontalBox;
class UStateTreeState;

class SAvaTransitionStateView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaTransitionStateView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FAvaTransitionStateViewModel>& InStateViewModel);

private:
	TSharedRef<SWidget> CreateStateSlotWidget(const TSharedRef<SHorizontalBox>& InStateBox, const TSharedRef<FAvaTransitionStateViewModel>& InStateViewModel);

	const FSlateBrush* GetSelectorIcon() const;

	FText GetSelectorTooltip() const;

	FSlateColor GetActiveStateColor() const;

	EVisibility GetStateBreakpointVisibility() const;

	TWeakPtr<FAvaTransitionStateViewModel> StateViewModelWeak;
};
