// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaTransitionTreeDetails.h"
#include "AvaTransitionTreeEditorData.h"
#include "Customizations/AvaTransitionTreeEditorDataCustomization.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "ViewModels/AvaTransitionViewModelSharedData.h"

void SAvaTransitionTreeDetails::Construct(const FArguments& InArgs, const TSharedRef<FAvaTransitionEditorViewModel>& InEditorViewModel)
{
	EditorViewModelWeak = InEditorViewModel;

	TSharedRef<FAvaTransitionViewModelSharedData> SharedData = InEditorViewModel->GetSharedData();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;

	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	DetailsView->RegisterInstancedCustomPropertyLayout(UStateTreeEditorData::StaticClass()
		, FOnGetDetailCustomizationInstance::CreateStatic(&FAvaTransitionTreeEditorDataCustomization::MakeInstance, SharedData->AsWeak()));

	// Read-only
	if (SharedData->IsReadOnly())
	{
		DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateLambda([]{ return false; }));
	}

	ChildSlot
	[
		DetailsView.ToSharedRef()
	];

	Refresh();

	OnRefreshHandle = InEditorViewModel->GetOnPostRefresh().AddSP(this, &SAvaTransitionTreeDetails::Refresh);
}

SAvaTransitionTreeDetails::~SAvaTransitionTreeDetails()
{
	if (TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = EditorViewModelWeak.Pin())
	{
		EditorViewModel->GetOnPostRefresh().Remove(OnRefreshHandle);
		OnRefreshHandle.Reset();
	}
}

void SAvaTransitionTreeDetails::Refresh()
{
	TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = EditorViewModelWeak.Pin();
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	UAvaTransitionTreeEditorData* EditorData = EditorViewModel->GetEditorData();
	if (!EditorData)
	{
		return;
	}

	check(DetailsView.IsValid());
	DetailsView->SetObject(EditorData, /*bForceRefresh*/true);
}
