// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FAvaTransitionEditorViewModel;
class IDetailsView;

class SAvaTransitionTreeDetails : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaTransitionTreeDetails){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FAvaTransitionEditorViewModel>& InEditorViewModel);

	virtual ~SAvaTransitionTreeDetails() override;

private:
	void Refresh();

	TWeakPtr<FAvaTransitionEditorViewModel> EditorViewModelWeak;

	TSharedPtr<IDetailsView> DetailsView;

	FDelegateHandle OnRefreshHandle;
};
