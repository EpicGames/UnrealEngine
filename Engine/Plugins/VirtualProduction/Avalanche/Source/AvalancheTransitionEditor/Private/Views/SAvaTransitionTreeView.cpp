// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvaTransitionTreeView.h"
#include "AvaTransitionSelection.h"
#include "Extensions/IAvaTransitionSelectableExtension.h"
#include "Extensions/IAvaTransitionTreeRowExtension.h"
#include "Menu/AvaTransitionTreeContextMenu.h"
#include "ViewModels/AvaTransitionEditorViewModel.h"
#include "ViewModels/AvaTransitionViewModelUtils.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Views/STreeView.h"

void SAvaTransitionTreeView::Construct(const FArguments& InArgs, const TSharedRef<FAvaTransitionEditorViewModel>& InEditorViewModel)
{
	EditorViewModelWeak = InEditorViewModel;

	VerticalScrollbar = SNew(SScrollBar)
		.Orientation(Orient_Vertical)
		.PreventThrottling(true);

	ChildSlot
		[
			SAssignNew(TreeView, STreeView<TSharedPtr<FAvaTransitionViewModel>>)
			.TreeItemsSource(&TopLevelItems)
			.OnGetChildren(this, &SAvaTransitionTreeView::OnGetChildren)
			.OnGenerateRow(this, &SAvaTransitionTreeView::OnGenerateRow)
			.OnSelectionChanged(this, &SAvaTransitionTreeView::OnSelectionChanged)
			.OnExpansionChanged(this, &SAvaTransitionTreeView::OnExpansionChanged)
			.OnSetExpansionRecursive(this, &SAvaTransitionTreeView::OnSetExpansionRecursive)
			.OnContextMenuOpening(this, &SAvaTransitionTreeView::OnContextMenuOpening)
			.SelectionMode(ESelectionMode::Multi)
			.ExternalScrollbar(VerticalScrollbar)
		];

	InEditorViewModel->GetSelection()->OnSelectionChanged().AddSP(this, &SAvaTransitionTreeView::SetSelectedItems);
	Refresh();
}

SAvaTransitionTreeView::~SAvaTransitionTreeView()
{
	if (TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = EditorViewModelWeak.Pin())
	{
		EditorViewModel->GetSelection()->OnSelectionChanged().RemoveAll(this);
	}
}

void SAvaTransitionTreeView::Refresh()
{
	TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = EditorViewModelWeak.Pin();
	if (!EditorViewModel.IsValid())
	{
		return;
	}

	OnGetChildren(EditorViewModel, TopLevelItems);

	SetSelectedItems(EditorViewModel->GetSelection()->GetSelectedItems());

	UE::AvaTransitionEditor::ForEachChildOfType<IAvaTransitionTreeRowExtension>(*EditorViewModel,
		[this](const TAvaTransitionCastedViewModel<IAvaTransitionTreeRowExtension>& InViewModel, EAvaTransitionIterationResult&)
		{
			TreeView->SetItemExpansion(InViewModel.Base, InViewModel.Casted->IsExpanded());
		}
		, /*bInRecurse*/true);

	if (TreeView.IsValid())
	{
		TreeView->RequestTreeRefresh();	
	}
}

void SAvaTransitionTreeView::SetSelectedItems(TConstArrayView<TSharedRef<FAvaTransitionViewModel>> InSelectedItems)
{
	if (bSyncingSelection)
	{
		return;
	}

	if (TreeView.IsValid())
	{
		TArray<TSharedPtr<FAvaTransitionViewModel>> SelectedItems;
		SelectedItems.Append(InSelectedItems);
		TreeView->SetItemSelection(SelectedItems, /*bSelected*/true);
	}
}

void SAvaTransitionTreeView::OnGetChildren(TSharedPtr<FAvaTransitionViewModel> InItem, TArray<TSharedPtr<FAvaTransitionViewModel>>& OutChildren)
{
	if (!InItem.IsValid())
	{
		OutChildren.Reset();
		return;
	}

	OutChildren.Reset(InItem->GetChildren().Num());

	// Only allow View Models with Tree Row Extensions
	UE::AvaTransitionEditor::ForEachChildOfType<IAvaTransitionTreeRowExtension>(*InItem,
		[&OutChildren](const TAvaTransitionCastedViewModel<IAvaTransitionTreeRowExtension>& InViewModel, EAvaTransitionIterationResult&)
		{
			if (InViewModel.Casted->CanGenerateRow())
			{
				OutChildren.Add(InViewModel.Base);
			}
		});
}

TSharedRef<ITableRow> SAvaTransitionTreeView::OnGenerateRow(TSharedPtr<FAvaTransitionViewModel> InItem, const TSharedRef<STableViewBase>& InOwningTableView)
{
	// At this point only View Models with Tree Row Extensions should've been allowed
	check(InItem.IsValid());
	IAvaTransitionTreeRowExtension* TreeRowExtension = InItem->CastTo<IAvaTransitionTreeRowExtension>();
	check(TreeRowExtension);
	return TreeRowExtension->GenerateRow(InOwningTableView);
}

void SAvaTransitionTreeView::OnSelectionChanged(TSharedPtr<FAvaTransitionViewModel> InItem, ESelectInfo::Type InSelectInfo)
{
	TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = EditorViewModelWeak.Pin();
	if (!EditorViewModel)
	{
		return;
	}

	check(TreeView.IsValid());
	TArray<TSharedPtr<FAvaTransitionViewModel>> SelectedItems = TreeView->GetSelectedItems();

	TGuardValue<bool> SyncSelectionGuard(bSyncingSelection, true);
	EditorViewModel->GetSelection()->SetSelectedItems(SelectedItems);
}

void SAvaTransitionTreeView::OnExpansionChanged(TSharedPtr<FAvaTransitionViewModel> InItem, bool bIsExpanded)
{
	if (InItem.IsValid())
	{
		IAvaTransitionTreeRowExtension* TreeRowExtension = InItem->CastTo<IAvaTransitionTreeRowExtension>();
		check(TreeRowExtension);
		TreeRowExtension->SetExpanded(bIsExpanded);
	}
}

void SAvaTransitionTreeView::OnSetExpansionRecursive(TSharedPtr<FAvaTransitionViewModel> InItem, bool bInShouldExpand)
{
	if (!InItem.IsValid())
	{
		return;
	}

	IAvaTransitionTreeRowExtension* TreeRowExtension = InItem->CastTo<IAvaTransitionTreeRowExtension>();
	check(TreeRowExtension);
	TreeRowExtension->SetExpanded(bInShouldExpand);

	// Only allow View Models with Tree Row Extensions
	UE::AvaTransitionEditor::ForEachChildOfType<IAvaTransitionTreeRowExtension>(*InItem,
		[bInShouldExpand](const TAvaTransitionCastedViewModel<IAvaTransitionTreeRowExtension>& InViewModel, EAvaTransitionIterationResult&)
		{
			InViewModel.Casted->SetExpanded(bInShouldExpand);
		});
}

TSharedPtr<SWidget> SAvaTransitionTreeView::OnContextMenuOpening()
{
	if (TSharedPtr<FAvaTransitionEditorViewModel> EditorViewModel = EditorViewModelWeak.Pin())
	{
		return EditorViewModel->GetContextMenu()->GenerateTreeContextMenuWidget();
	}
	return nullptr;
}
