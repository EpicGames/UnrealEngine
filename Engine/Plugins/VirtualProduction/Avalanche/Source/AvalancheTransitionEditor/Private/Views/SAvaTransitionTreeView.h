// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FAvaTransitionEditorViewModel;
class FAvaTransitionViewModel;
class IAvaTransitionSelectableExtension;
class ITableRow;
class SScrollBar;
class STableViewBase;
template<typename ItemType> class STreeView;

class SAvaTransitionTreeView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaTransitionTreeView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FAvaTransitionEditorViewModel>& InEditorViewModel);

	virtual ~SAvaTransitionTreeView() override;

	void Refresh();

	void SetSelectedItems(TConstArrayView<TSharedRef<FAvaTransitionViewModel>> InSelectedItems);

	TSharedRef<SScrollBar> GetVerticalScrollbar() const
	{
		return VerticalScrollbar.ToSharedRef();
	}

private:
	void OnGetChildren(TSharedPtr<FAvaTransitionViewModel> InItem, TArray<TSharedPtr<FAvaTransitionViewModel>>& OutChildren);

	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FAvaTransitionViewModel> InItem, const TSharedRef<STableViewBase>& InOwningTableView);

	void OnSelectionChanged(TSharedPtr<FAvaTransitionViewModel> InItem, ESelectInfo::Type InSelectInfo);

	void OnExpansionChanged(TSharedPtr<FAvaTransitionViewModel> InItem, bool bIsExpanded);

	void OnSetExpansionRecursive(TSharedPtr<FAvaTransitionViewModel> InItem, bool bInShouldExpand);

	TSharedPtr<SWidget> OnContextMenuOpening(); 

	TWeakPtr<FAvaTransitionEditorViewModel> EditorViewModelWeak;

	TSharedPtr<STreeView<TSharedPtr<FAvaTransitionViewModel>>> TreeView;

	TArray<TSharedPtr<FAvaTransitionViewModel>> TopLevelItems;

	TSharedPtr<SScrollBar> VerticalScrollbar;

	bool bSyncingSelection = false;
};
