// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaTagHandle.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "Misc/Guid.h"
#include "State/AvaTransitionStateMetadata.h"
#include "StateTreeEditorData.h"
#include "AvaTransitionTreeEditorData.generated.h"

#define UE_API AVALANCHETRANSITIONEDITOR_API

class UStateTreeState;

namespace UE::AvaTransitionEditor
{
	constexpr FGuid ColorId_Default;
	constexpr FGuid ColorId_In(0x1DDBC788, 0xD5EB400E, 0xBB71E5DA, 0xB27A784D);
	constexpr FGuid ColorId_Out(0xE549EFA0, 0xDEFF45A7, 0xA8D907AF, 0xDB8F7643);
}

UCLASS(MinimalAPI, HideCategories=(Common))
class UAvaTransitionTreeEditorData : public UStateTreeEditorData
{
	GENERATED_BODY()

public:
	UAvaTransitionTreeEditorData();

	UE_API UStateTreeState& CreateState(const UStateTreeState& InSiblingState, bool bInAfter);

	/** Returns true if this has the same data as the given editor data */
	UE_API bool Compare(const UAvaTransitionTreeEditorData* InEditorData) const;

	UE_DEPRECATED(5.8, "GetTransitionLayer is deprecated. Use IAvaTransitionBehavior::GetTransitionLayer instead")
	UE_API FAvaTagHandle GetTransitionLayer() const;

	UE_DEPRECATED(5.8, "SetTransitionLayer is deprecated. Use IAvaTransitionBehavior::SetTransitionLayer instead")
	UE_API void SetTransitionLayer(const FAvaTagHandle& InTransitionLayer);

	UE_DEPRECATED(5.8, "Editor data TransitionLayer is deprecated. Use IAvaTransitionBehavior instead")
	UE_API static FName GetTransitionLayerPropertyName();

	const FAvaTransitionStateMetadata* FindStateMetadata(const FGuid& InStateId) const
	{
		return StateMetadata.Find(InStateId);
	}

	FAvaTransitionStateMetadata& FindOrAddStateMetadata(const FGuid& InStateId)
	{
		return StateMetadata.FindOrAdd(InStateId);
	}

	FSimpleMulticastDelegate& GetOnTreeRequestRefresh()
	{
		return OnTreeRequestRefresh;
	}

private:
	UE_DEPRECATED(5.8, "TransitionLayer is deprecated. Use IAvaTransitionBehavior Set/Get TransitionLayer instead")
	UPROPERTY()
	FAvaTagHandle TransitionLayer_DEPRECATED;

	/** Map of a state's id to its metadata */
	UPROPERTY()
	TMap<FGuid, FAvaTransitionStateMetadata> StateMetadata;

	FSimpleMulticastDelegate OnTreeRequestRefresh;
};

#undef UE_API
