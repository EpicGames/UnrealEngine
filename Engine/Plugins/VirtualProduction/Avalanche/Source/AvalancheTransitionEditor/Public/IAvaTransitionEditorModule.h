// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IAvaTransitionBehavior;
class UAvaTransitionTreeEditorData;
class UToolMenu;

class IAvaTransitionEditorModule : public IModuleInterface
{
	static constexpr const TCHAR* ModuleName = TEXT("AvalancheTransitionEditor");

public:
	static bool IsLoaded()
	{
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	static IAvaTransitionEditorModule& Get()
	{
		return FModuleManager::Get().LoadModuleChecked<IAvaTransitionEditorModule>(ModuleName);
	}

	DECLARE_DELEGATE_OneParam(FOnBuildDefaultTransitionTree, UAvaTransitionTreeEditorData&);
	/** Returns delegate to build the default Transition Tree */
	virtual FOnBuildDefaultTransitionTree& GetOnBuildDefaultTransitionTree() = 0;

	virtual void GenerateTransitionTreeOptionsMenu(UToolMenu* InMenu, IAvaTransitionBehavior* InTransitionBehavior) = 0;
};
