// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "UObject/SoftObjectPtr.h"
#include "AvaTransitionEditorSettings.generated.h"

#define UE_API AVALANCHETRANSITIONEDITOR_API

class UAvaTransitionTree;
class UAvaTransitionTreeEditorData;

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, meta=(DisplayName="Transition Logic"))
class UAvaTransitionEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UAvaTransitionEditorSettings();

	UAvaTransitionTreeEditorData* LoadDefaultTemplateEditorData() const;

	bool ShouldCreateTransitionLogicDefaultScene() const
	{
		return bCreateTransitionLogicDefaultScene;
	}

	UE_API void ToggleCreateTransitionLogicDefaultScene();

private:
	/** The template to use when building new transition trees */
	UPROPERTY(Config, EditAnywhere, Category="Motion Design")
	TSoftObjectPtr<UAvaTransitionTree> DefaultTemplate;

	/** Whether to create the default scene when creating a Motion Design scene for the first time */
	UPROPERTY(Config, EditAnywhere, Category="Motion Design")
	bool bCreateTransitionLogicDefaultScene = true;
};

#undef UE_API
