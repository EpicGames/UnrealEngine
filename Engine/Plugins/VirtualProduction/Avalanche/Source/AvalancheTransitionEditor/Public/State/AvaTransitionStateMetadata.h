// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "AvaTransitionStateMetadata.generated.h"

USTRUCT()
struct FAvaTransitionStateMetadata
{
	GENERATED_BODY()

	UPROPERTY()
	FString Comment;
};
