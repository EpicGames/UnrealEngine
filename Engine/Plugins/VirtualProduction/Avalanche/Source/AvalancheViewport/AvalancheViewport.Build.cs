// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheViewport : ModuleRules
{
	public AvalancheViewport(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Avalanche",
				"AvalancheCore",
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"SlateCore",
				"UnrealEd"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AvalancheEditorCore",
				"AvalancheShapes",
				"InputCore",
				"Json",
				"Projects",
				"SettingsEditor",
				"Slate"
			}
		);
	}
}
