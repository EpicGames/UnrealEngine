// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaScreenAlignmentActorInfo.h"
#include "AvaActorUtils.h"
#include "Bounds/AvaBoundsProviderSubsystem.h"
#include "GameFramework/Actor.h"
#include "Engine/World.h"
#include "Math/Box.h"
#include "Math/OrientedBox.h"
#include "Math/Transform.h"
#include "Math/Vector.h"

namespace UE::AvaEditor::Private
{
	[[nodiscard]] FBox GetActorLocalBounds(AActor& InActor, EAvaAlignmentSizeMode InActorSizeMode)
	{
		if (UWorld* World = InActor.GetWorld())
		{
			if (UAvaBoundsProviderSubsystem* BoundsSubsystem = World->GetSubsystem<UAvaBoundsProviderSubsystem>())
			{
				switch (InActorSizeMode)
				{
					case EAvaAlignmentSizeMode::Self:
						return BoundsSubsystem->GetActorLocalBounds(&InActor);

					case EAvaAlignmentSizeMode::SelfAndChildren:
						return BoundsSubsystem->GetActorAndChildrenLocalBounds(&InActor);
				}
			}
		}

		constexpr bool bIncludeActorsFromChildren = false; // Child actor components
		constexpr bool bMustBeRegistered = false;

		FBox ActorBounds = FAvaActorUtils::GetActorLocalBoundingBox(
			&InActor,
			bIncludeActorsFromChildren,
			bMustBeRegistered
		);

		if (ActorBounds.IsValid)
		{
			if (InActorSizeMode == EAvaAlignmentSizeMode::SelfAndChildren)
			{
				const FTransform WorldToActor = InActor.GetActorTransform().Inverse();

				FVector Vertices[8];

				TArray<AActor*> ChildActors;
				InActor.GetAttachedActors(ChildActors, false, true);

				for (AActor* ChildActor : ChildActors)
				{
					const FBox ChildBounds = FAvaActorUtils::GetActorLocalBoundingBox(
						ChildActor, 
						bIncludeActorsFromChildren,
						bMustBeRegistered
					);

					if (!ChildBounds.IsValid)
					{
						continue;
					}

					const FTransform ChildActorToWorld = ChildActor->GetActorTransform();
					const FOrientedBox ChildOrientedBounds = FAvaActorUtils::MakeOrientedBox(ChildBounds, ChildActorToWorld);

					ChildOrientedBounds.CalcVertices(Vertices);

					for (int32 Index = 0; Index < 8; ++Index)
					{
						const FVector ChildActorWorldVertex = ChildActorToWorld.TransformPosition(Vertices[Index]);
						const FVector LocalActorVertex = WorldToActor.TransformPosition(ChildActorWorldVertex);

						ActorBounds += LocalActorVertex;
					}
				}
			}
		}

		return ActorBounds;
	}
}

FAvaScreenAlignmentActorInfo FAvaScreenAlignmentActorInfo::Create(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter,
	AActor& InActor, EAvaAlignmentSizeMode InActorSizeMode)
{
	struct
	{
		TBounds<double, FCoordinateSpace::World> ActorLocal;
		TBounds<double, FCoordinateSpace::World> World;
		TBounds<double, FCoordinateSpace::World> CameraLocal;
		TBounds<float, FCoordinateSpace::Screen> Screen;
	} Values;

	auto CreateInfo = [&Values, InActorPtr = &InActor]() -> FAvaScreenAlignmentActorInfo
	{
		return {
			InActorPtr,
			Values.ActorLocal,
			Values.World,
			Values.CameraLocal,
			Values.Screen
		};
	};

	const USceneComponent* ActorRoot = InActor.GetRootComponent();

	if (!ActorRoot)
	{
		return CreateInfo();
	}

	const FBox ActorLocalBoundingBox = UE::AvaEditor::Private::GetActorLocalBounds(InActor, InActorSizeMode);

	if (!ActorLocalBoundingBox.IsValid)
	{
		return CreateInfo();
	}

	// Calculate local space coordinates
	const bool bIs3D = !FMath::IsNearlyEqual(ActorLocalBoundingBox.Min.X, ActorLocalBoundingBox.Max.X);
	const int32 VertexCount = bIs3D ? 8 : 4;

	Values.ActorLocal.ToWorld = ActorRoot->GetComponentTransform();
	Values.ActorLocal.FromWorld = Values.ActorLocal.ToWorld.Inverse();
	Values.ActorLocal.Location = FVector::ZeroVector;

	Values.ActorLocal.Vertices.Reserve(bIs3D ? 8 : 4);

	Values.ActorLocal.Vertices.Add({ActorLocalBoundingBox.Min.X, ActorLocalBoundingBox.Min.Y, ActorLocalBoundingBox.Min.Z});
	Values.ActorLocal.Vertices.Add({ActorLocalBoundingBox.Min.X, ActorLocalBoundingBox.Min.Y, ActorLocalBoundingBox.Max.Z});
	Values.ActorLocal.Vertices.Add({ActorLocalBoundingBox.Min.X, ActorLocalBoundingBox.Max.Y, ActorLocalBoundingBox.Min.Z});
	Values.ActorLocal.Vertices.Add({ActorLocalBoundingBox.Min.X, ActorLocalBoundingBox.Max.Y, ActorLocalBoundingBox.Max.Z});

	if (bIs3D)
	{
		Values.ActorLocal.Vertices.Add({ActorLocalBoundingBox.Max.X, ActorLocalBoundingBox.Min.Y, ActorLocalBoundingBox.Min.Z});
		Values.ActorLocal.Vertices.Add({ActorLocalBoundingBox.Max.X, ActorLocalBoundingBox.Min.Y, ActorLocalBoundingBox.Max.Z});
		Values.ActorLocal.Vertices.Add({ActorLocalBoundingBox.Max.X, ActorLocalBoundingBox.Max.Y, ActorLocalBoundingBox.Min.Z});
		Values.ActorLocal.Vertices.Add({ActorLocalBoundingBox.Max.X, ActorLocalBoundingBox.Max.Y, ActorLocalBoundingBox.Max.Z});
	}

	FVector MinD = Values.ActorLocal.Vertices[0];
	FVector MaxD = Values.ActorLocal.Vertices[0];

	for (int32 Index = 1; Index < VertexCount; ++Index)
	{
		MinD.X = FMath::Min(MinD.X, Values.ActorLocal.Vertices[Index].X);
		MinD.Y = FMath::Min(MinD.Y, Values.ActorLocal.Vertices[Index].Y);
		MinD.Z = FMath::Min(MinD.Z, Values.ActorLocal.Vertices[Index].Z);
		MaxD.X = FMath::Max(MaxD.X, Values.ActorLocal.Vertices[Index].X);
		MaxD.Y = FMath::Max(MaxD.Y, Values.ActorLocal.Vertices[Index].Y);
		MaxD.Z = FMath::Max(MaxD.Z, Values.ActorLocal.Vertices[Index].Z);
	}

	Values.ActorLocal.Center = FVector::ZeroVector;
	Values.ActorLocal.Extent = (MaxD * 0.5) - (MinD * 0.5);

	// Calculate world space coordinates
	Values.World.ToWorld = FTransform::Identity;
	Values.World.FromWorld = FTransform::Identity;
	Values.World.Location = Values.ActorLocal.ToWorld.GetLocation();
	Values.World.Extent = Values.ActorLocal.Extent * Values.ActorLocal.ToWorld.GetScale3D();

	Values.World.Vertices.Reserve(VertexCount);

	for (const FVector& Vertex : Values.ActorLocal.Vertices)
	{
		Values.World.Vertices.Add(Values.ActorLocal.ToWorld.TransformPosition(Vertex));
	}

	MinD = Values.World.Vertices[0];
	MaxD = Values.World.Vertices[0];

	for (int32 Index = 1; Index < VertexCount; ++Index)
	{
		MinD.X = FMath::Min(MinD.X, Values.World.Vertices[Index].X);
		MinD.Y = FMath::Min(MinD.Y, Values.World.Vertices[Index].Y);
		MinD.Z = FMath::Min(MinD.Z, Values.World.Vertices[Index].Z);
		MaxD.X = FMath::Max(MaxD.X, Values.World.Vertices[Index].X);
		MaxD.Y = FMath::Max(MaxD.Y, Values.World.Vertices[Index].Y);
		MaxD.Z = FMath::Max(MaxD.Z, Values.World.Vertices[Index].Z);
	}

	Values.World.Center = (MaxD * 0.5) + (MinD * 0.5);

	// Calculate camera space coordinates
	Values.CameraLocal.ToWorld = InCoordinateConverter->GetViewportViewTransform();
	Values.CameraLocal.FromWorld = Values.CameraLocal.ToWorld.Inverse();
	Values.CameraLocal.Location = Values.CameraLocal.FromWorld.TransformPosition(Values.World.Location);

	Values.CameraLocal.Vertices.Reserve(VertexCount);

	for (const FVector& Vertex : Values.World.Vertices)
	{
		Values.CameraLocal.Vertices.Add(Values.CameraLocal.FromWorld.TransformPositionNoScale(Vertex));
	}

	MinD = Values.CameraLocal.Vertices[0];
	MaxD = Values.CameraLocal.Vertices[0];

	for (int32 Index = 1; Index < VertexCount; ++Index)
	{
		MinD.X = FMath::Min(MinD.X, Values.CameraLocal.Vertices[Index].X);
		MinD.Y = FMath::Min(MinD.Y, Values.CameraLocal.Vertices[Index].Y);
		MinD.Z = FMath::Min(MinD.Z, Values.CameraLocal.Vertices[Index].Z);
		MaxD.X = FMath::Max(MaxD.X, Values.CameraLocal.Vertices[Index].X);
		MaxD.Y = FMath::Max(MaxD.Y, Values.CameraLocal.Vertices[Index].Y);
		MaxD.Z = FMath::Max(MaxD.Z, Values.CameraLocal.Vertices[Index].Z);
	}

	Values.CameraLocal.Center = (MaxD * 0.5) + (MinD * 0.5);
	Values.CameraLocal.Extent = Values.World.Extent;

	// Calculate screen space coordinates
	Values.Screen.ToWorld = static_cast<UE::Math::TTransform<float>>(Values.CameraLocal.ToWorld); // Not really accurate, but it's something
	Values.Screen.FromWorld = static_cast<UE::Math::TTransform<float>>(Values.CameraLocal.FromWorld); // Not really accurate, but it's something

	FVector2f ScreenCoordinate = FVector2f::ZeroVector;
	double Distance = 0;

	InCoordinateConverter->WorldPositionToViewportPosition(Values.World.Location, ScreenCoordinate, Distance);

	Values.Screen.Location = {ScreenCoordinate.X, ScreenCoordinate.Y, static_cast<float>(Distance)};

	Values.Screen.Vertices.Reserve(VertexCount);

	for (const FVector& Vertex : Values.World.Vertices)
	{
		InCoordinateConverter->WorldPositionToViewportPosition(Vertex, ScreenCoordinate, Distance);
		Values.Screen.Vertices.Add({ScreenCoordinate.X, ScreenCoordinate.Y, static_cast<float>(Distance)});
	}

	FVector3f MinF = Values.Screen.Vertices[0];
	FVector3f MaxF = Values.Screen.Vertices[0];

	for (int32 Index = 1; Index < VertexCount; ++Index)
	{
		MinF.X = FMath::Min(MinF.X, Values.Screen.Vertices[Index].X);
		MinF.Y = FMath::Min(MinF.Y, Values.Screen.Vertices[Index].Y);
		MinF.Z = FMath::Min(MinF.Z, Values.Screen.Vertices[Index].Z);
		MaxF.X = FMath::Max(MaxF.X, Values.Screen.Vertices[Index].X);
		MaxF.Y = FMath::Max(MaxF.Y, Values.Screen.Vertices[Index].Y);
		MaxF.Z = FMath::Max(MaxF.Z, Values.Screen.Vertices[Index].Z);
	}

	Values.Screen.Center = (MaxF * 0.5f) + (MinF * 0.5f);
	Values.Screen.Extent = Values.Screen.Center - MinF;

	return CreateInfo();
}

const FAvaScreenAlignmentActorInfo::TBounds<double, 0>& FAvaScreenAlignmentActorInfo::GetWorldSpace(ESpace InSpace) const
{
	switch (InSpace)
	{
		case ESpace::Actor:
			return ActorLocal;

		case ESpace::World:
		default:
			return World;

		case ESpace::Camera:
			return CameraLocal;
	}
}
