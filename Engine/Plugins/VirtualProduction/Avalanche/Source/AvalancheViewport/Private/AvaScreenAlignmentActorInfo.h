// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaDefs.h"
#include "AvaScreenAlignmentEnums.h"
#include "ViewportClient/IAvaViewportWorldCoordinateConverter.h"

/**
 * Information about an actor relative to the camera - world locations in camera space and screen locations.
 */
struct FAvaScreenAlignmentActorInfo
{
	struct FCoordinateSpace
	{
		enum Type : uint8
		{
			World = 0,
			Screen = 1
		};
	};

	enum class ESpace : uint8
	{
		Actor,
		World,
		Camera
	};

	template<typename InRealType, uint8 InCoordinateSpace>
	struct TBounds
	{
		using FRealType = InRealType;
		using FVectorType = UE::Math::TVector<FRealType>;
		using FTransformType = UE::Math::TTransform<FRealType>;

		static constexpr uint8 CoordinateSpace = InCoordinateSpace;

		/** Transform from this coordinate space to world space. */
		FTransformType ToWorld;

		/** Transform from world space to this coordinate space. */
		FTransformType FromWorld;

		/** Location of the actor in this coordinate space. */
		FVectorType Location;

		/** Geometric center in this coordinate space. */
		FVectorType Center;

		/** Corner vertices in this coordinate space. */
		TArray<FVectorType> Vertices;

		/** Distance from the center to the edges in this coordinate space. */
		FVectorType Extent;

		[[nodiscard]] FRealType GetComponent(EAvaHorizontalAlignment InHorizontalAlignment) const;

		[[nodiscard]] FRealType GetComponent(EAvaVerticalAlignment InVerticalAlignment) const;

		[[nodiscard]] FRealType GetComponent(EAvaDepthAlignment InDepthAlignment) const;

		[[nodiscard]] FVectorType GetCoordinate(EAvaHorizontalAlignment InHorizontalAlignment, EAvaVerticalAlignment InVerticalAlignment,
			EAvaDepthAlignment InDepthAlignment) const;		
	};

	[[nodiscard]] static FAvaScreenAlignmentActorInfo Create(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, AActor& InActor,
		EAvaAlignmentSizeMode InActorSizeMode);

	/** The actor in question */
	AActor* const Actor;

	const TBounds<double, FCoordinateSpace::World> ActorLocal;

	const TBounds<double, FCoordinateSpace::World> World;

	const TBounds<double, FCoordinateSpace::World> CameraLocal;

	const TBounds<float, FCoordinateSpace::Screen> Screen;

	[[nodiscard]] const TBounds<double, FCoordinateSpace::World>& GetWorldSpace(ESpace InSpace) const;
};

template<typename InRealType, uint8 InCoordinateSpace>
typename FAvaScreenAlignmentActorInfo::TBounds<InRealType, InCoordinateSpace>::FRealType
	FAvaScreenAlignmentActorInfo::TBounds<InRealType, InCoordinateSpace>::
		GetComponent(EAvaHorizontalAlignment InHorizontalAlignment) const
{
	if constexpr (CoordinateSpace == FCoordinateSpace::World)
	{
		switch (InHorizontalAlignment)
		{
		case EAvaHorizontalAlignment::Left:
			return Center.Y - Extent.Y;

		case EAvaHorizontalAlignment::Right:
			return Center.Y + Extent.Y;

		default:
			return Center.Y;
		}
	}
	else if constexpr (CoordinateSpace == FCoordinateSpace::Screen)
	{
		switch (InHorizontalAlignment)
		{
		case EAvaHorizontalAlignment::Left:
			return Center.X - Extent.X;

		case EAvaHorizontalAlignment::Right:
			return Center.X + Extent.X;

		default:
			return Center.X;
		}
	}
	else
	{
		return 0;
	}
}

template<typename InRealType, uint8 InCoordinateSpace>
typename FAvaScreenAlignmentActorInfo::TBounds<InRealType, InCoordinateSpace>::FRealType
	FAvaScreenAlignmentActorInfo::TBounds<InRealType, InCoordinateSpace>::
		GetComponent(EAvaVerticalAlignment InVerticalAlignment) const
{
	if constexpr (CoordinateSpace == FCoordinateSpace::World)
	{
		switch (InVerticalAlignment)
		{
		case EAvaVerticalAlignment::Top:
			return Center.Z + Extent.Z;

		case EAvaVerticalAlignment::Bottom:
			return Center.Z - Extent.Z;

		default:
			return Center.Y;
		}
	}
	else if constexpr (CoordinateSpace == FCoordinateSpace::Screen)
	{
		switch (InVerticalAlignment)
		{
		case EAvaVerticalAlignment::Top:
			return Center.Y - Extent.Y;

		case EAvaVerticalAlignment::Bottom:
			return Center.Y + Extent.Y;

		default:
			return Center.Y;
		}
	}
	else
	{
		return 0;
	}
}

template<typename InRealType, uint8 InCoordinateSpace>
typename FAvaScreenAlignmentActorInfo::TBounds<InRealType, InCoordinateSpace>::FRealType
	FAvaScreenAlignmentActorInfo::TBounds<InRealType, InCoordinateSpace>::
		GetComponent(EAvaDepthAlignment InDepthAlignment) const
{
	if constexpr (CoordinateSpace == FCoordinateSpace::World)
	{
		switch (InDepthAlignment)
		{
		case EAvaDepthAlignment::Front:
			return Center.X - Extent.X;

		case EAvaDepthAlignment::Back:
			return Center.X + Extent.X;

		default:
			return Center.X;
		}
	}
	else if constexpr (CoordinateSpace == FCoordinateSpace::Screen)
	{
		switch (InDepthAlignment)
		{
		case EAvaDepthAlignment::Front:
			return Center.Z - Extent.Z;

		case EAvaDepthAlignment::Back:
			return Center.Z + Extent.Z;

		default:
			return Center.Z;
		}
	}
	else
	{
		return 0;
	}
}

template<typename InRealType, uint8 InCoordinateSpace>
typename FAvaScreenAlignmentActorInfo::TBounds<InRealType, InCoordinateSpace>::FVectorType
	FAvaScreenAlignmentActorInfo::TBounds<InRealType, InCoordinateSpace>::
		GetCoordinate(EAvaHorizontalAlignment InHorizontalAlignment, EAvaVerticalAlignment InVerticalAlignment, 
			EAvaDepthAlignment InDepthAlignment) const
{
	return {
		GetComponent(InHorizontalAlignment),
		GetComponent(InVerticalAlignment),
		GetComponent(InDepthAlignment)
	};
}
