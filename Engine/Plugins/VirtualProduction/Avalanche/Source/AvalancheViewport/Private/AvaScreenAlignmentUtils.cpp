// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaScreenAlignmentUtils.h"

#include "AvalancheViewportModule.h"
#include "AvaScreenAlignmentActorInfo.h"
#include "AvaScreenAlignmentEnums.h"
#include "AvaShapeActor.h"
#include "AvaViewportAxisUtils.h"
#include "AvaViewportUtils.h"
#include "Bounds/AvaBoundsProviderSubsystem.h"
#include "DynamicMeshes/AvaShapeDynMeshBase.h"
#include "GameFramework/Actor.h"
#include "ViewportClient/IAvaViewportWorldCoordinateConverter.h"

namespace UE::AvaViewport::Private
{
	static constexpr float ValueTolerance = 0.01f;
	static constexpr float ExactValueTolerance = 0.001f;

	struct FAvaAxisDetails
	{
		int32 Index;
		EAxis::Type Axis;
		bool bCodirectional;

		FAvaAxisDetails(int32 InIndex, EAxis::Type InAxis, bool bInPositive = true)
			: Index(InIndex)
			, Axis(InAxis)
			, bCodirectional(bInPositive)
		{
		}

		FAvaAxisDetails(const FAvaViewportAxisMap& InViewportAxisMap, int32 InAxisIndex)
		{
			switch (InAxisIndex)
			{
				case FAvaViewportAxisUtils::WorldAxisIndexList.Horizontal.Index:
					Index = FAvaViewportAxisUtils::WorldAxisIndexList.Horizontal.Index;
					Axis = EAxis::Y;
					break;

				case FAvaViewportAxisUtils::WorldAxisIndexList.Vertical.Index:
					Index = FAvaViewportAxisUtils::WorldAxisIndexList.Vertical.Index;
					Axis = EAxis::Z;
					break;

				case FAvaViewportAxisUtils::WorldAxisIndexList.Depth.Index:
					Index = FAvaViewportAxisUtils::WorldAxisIndexList.Depth.Index;
					Axis = EAxis::X;
					break;

				default:
					Index = 0;
					Axis = EAxis::None;
					break;
			}

			if (InAxisIndex == InViewportAxisMap.Horizontal.Index)
			{
				bCodirectional = InViewportAxisMap.Horizontal.bCodirectional;
			}
			else if (InAxisIndex == InViewportAxisMap.Vertical.Index)
			{
				bCodirectional = InViewportAxisMap.Vertical.bCodirectional;
			}
			else
			{
				bCodirectional = InViewportAxisMap.Depth.bCodirectional;
			}
		}
	};

	void SizeActorToScreen(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter,
		const FAvaScreenAlignmentActorInfo& InActorInfo, bool bInStretchToFit)
	{
		if (!IsValid(InActorInfo.Actor))
		{
			return;
		}

		const FAvaViewportAxisMap ViewportAxisMap = FAvaViewportAxisUtils::CreateViewportAxisMap(
			InActorInfo.CameraLocal.FromWorld,
			*InActorInfo.Actor
		);

		const FVector ActorScale = InActorInfo.Actor->GetActorScale3D();

		if (FMath::IsNearlyZero(ActorScale[ViewportAxisMap.Horizontal.Index]) 
			|| FMath::IsNearlyZero(ActorScale[ViewportAxisMap.Vertical.Index]))
		{
			return;
		}

		const FVector2D CurrentSize = {
			InActorInfo.CameraLocal.Extent[ViewportAxisMap.Horizontal.Index] * 2.0,
			InActorInfo.CameraLocal.Extent[ViewportAxisMap.Vertical.Index] * 2.0
		};

		FVector2D TargetSize = InCoordinateConverter->GetFrustumSizeAtDistance(InActorInfo.Screen.Location.Z - InActorInfo.CameraLocal.Extent.X);

		if (TargetSize.IsNearlyZero() || TargetSize.Equals(CurrentSize))
		{
			return;
		}

		if (CurrentSize.IsNearlyZero() && bInStretchToFit)
		{
			// Not possible
			bInStretchToFit = false;
		}

		if (!bInStretchToFit)
		{
			const double CurrentAspectRatio = CurrentSize.X / CurrentSize.Y;
			const double TargetAspectRatio = TargetSize.X / TargetSize.Y;

			if (!FMath::IsNearlyEqual(CurrentAspectRatio, TargetAspectRatio))
			{
				if (CurrentAspectRatio > TargetAspectRatio)
				{
					TargetSize.Y = TargetSize.X / CurrentAspectRatio;
				}
				else
				{
					TargetSize.X = TargetSize.Y * CurrentAspectRatio;
				}
			}
		}

		if (AAvaShapeActor* ShapeActor = Cast<AAvaShapeActor>(InActorInfo.Actor))
		{
			if (UAvaShapeDynamicMeshBase* DynMesh = ShapeActor->GetDynamicMesh())
			{
				if (GUndo)
				{
					DynMesh->Modify();
				}

				TargetSize.X /= ActorScale[ViewportAxisMap.Horizontal.Index];
				TargetSize.Y /= ActorScale[ViewportAxisMap.Vertical.Index];

				FVector MeshSize = DynMesh->GetSize3D();
				MeshSize[ViewportAxisMap.Horizontal.Index] = TargetSize.X;
				MeshSize[ViewportAxisMap.Vertical.Index] = TargetSize.Y;

				DynMesh->SetSize3D(MeshSize);
			}
		}
		else
		{
			const FVector LocalSize = InActorInfo.ActorLocal.Extent * 2.0;

			FVector NewScale = ActorScale;
			NewScale[ViewportAxisMap.Horizontal.Index] = TargetSize.X / LocalSize[ViewportAxisMap.Horizontal.Index];
			NewScale[ViewportAxisMap.Vertical.Index] = TargetSize.Y / LocalSize[ViewportAxisMap.Vertical.Index];

			InActorInfo.Actor->SetActorScale3D(NewScale);
		}
	}

	/**
	 * Takes one of an actor's screen coordinates and works out where the actor would need to be for that
	 * coordinate to be at a particular point on the screen with the same depth as the current coordinate.
	 */
	[[nodiscard]] FVector ScreenVertexToActorLocation(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter,
		const FAvaScreenAlignmentActorInfo& InActorInfo, int32 InVertexIndex, const FVector3f& InScreenCoordinate,
		FAvaScreenAlignmentActorInfo::ESpace InSpace)
	{
		if (!InActorInfo.CameraLocal.Vertices.IsValidIndex(InVertexIndex))
		{
			return FVector::ZeroVector;
		}

		const FVector& CurrentWorldPosition = InActorInfo.World.Vertices[InVertexIndex];

		const FVector NewWorldPosition = InCoordinateConverter->ViewportPositionToWorldPosition(
			{InScreenCoordinate.X, InScreenCoordinate.Y},
			InScreenCoordinate.Z
		);

		const FVector WorldOffset = NewWorldPosition - CurrentWorldPosition;

		if (InSpace == FAvaScreenAlignmentActorInfo::ESpace::World)
		{
			return InActorInfo.World.Location + WorldOffset;
		}

		const FVector TransformedOffset = InActorInfo.GetWorldSpace(InSpace).FromWorld.TransformVector(WorldOffset);;

		return InActorInfo.GetWorldSpace(InSpace).Location + TransformedOffset;
	}

	[[nodiscard]] float GetScreenAxis(EAvaScreenAxis InAxis, const FVector2f& InScreenCoordinate, const double& InDistance)
	{
		switch (InAxis)
		{
			case EAvaScreenAxis::Horizontal:
				return InScreenCoordinate.X;

			case EAvaScreenAxis::Vertical:
				return InScreenCoordinate.Y;

			case EAvaScreenAxis::Depth:
				return static_cast<float>(InDistance);

			default:
				checkNoEntry();
				return 0.f;
		}
	}

	void SetScreenAxis(EAvaScreenAxis InAxis, FVector2f& InScreenCoordinate, double& InDistance, float InValue)
	{
		switch (InAxis)
		{
			case EAvaScreenAxis::Horizontal:
				InScreenCoordinate.X = InValue;
				break;

			case EAvaScreenAxis::Vertical:
				InScreenCoordinate.Y = InValue;
				break;

			case EAvaScreenAxis::Depth:
				InDistance = InValue;
				break;

			default:
				checkNoEntry();
				break;
		}
	}

	void AddToScreenAxis(EAvaScreenAxis InAxis, FVector2f& InScreenCoordinate, double& InDistance, float InValue)
	{
		switch (InAxis)
		{
			case EAvaScreenAxis::Horizontal:
				InScreenCoordinate.X += InValue;
				break;

			case EAvaScreenAxis::Vertical:
				InScreenCoordinate.Y += InValue;
				break;

			case EAvaScreenAxis::Depth:
				InDistance += InValue;
				break;

			default:
				checkNoEntry();
				break;
		}
	}

	/**
	 * Takes the current screen center location of an actor and attempts to change the actor's position to align its
	 * screen position to the given value on the given axis.
	 * @return The new actor location in camera local space.
	 */
	[[nodiscard]] FVector AlignActorCenter(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, 
		const FAvaScreenAlignmentActorInfo& InActorInfo, EAvaScreenAxis InAxis, float InCoordinate)
	{
		if (FMath::IsNearlyEqual(InActorInfo.Screen.Center[static_cast<uint8>(InAxis)], InCoordinate, ValueTolerance))
		{
			return InActorInfo.CameraLocal.Location;
		}

		// Because the center of the shape, relative to the screen, changes when the actor moves, there may be a perspective change,
		// so recalculate vertex positions and the new center and adjust again until it's right. Similar to the Newton-Raphson method.
		const int32 NumVertices = InActorInfo.World.Vertices.Num();

		if (NumVertices == 0)
		{
			return InActorInfo.CameraLocal.Location;
		}

		const FVector ActorCenterLocationOffset = InActorInfo.World.Center - InActorInfo.World.Location;

		TArray<FVector3f> ActorVertices;
		ActorVertices.SetNumUninitialized(NumVertices);

		FTransform ActorWorldTransform = InActorInfo.ActorLocal.ToWorld;

		FVector2f ScreenCoordinate = {InActorInfo.Screen.Center.X, InActorInfo.Screen.Center.Y};
		double Distance = InActorInfo.Screen.Center.Z;

		SetScreenAxis(InAxis, ScreenCoordinate, Distance, InCoordinate);

		FVector NewWorldCenter = InCoordinateConverter->ViewportPositionToWorldPosition(ScreenCoordinate, Distance);

		float ScreenCenterCoordinate;

		constexpr uint8 MaxIterations = 100;
		uint8 Iterations = 0;
		do
		{
			ActorWorldTransform.SetLocation(NewWorldCenter - ActorCenterLocationOffset);

			FVector2f VertexScreenCoordinate;
			double VertexDistance;

			for (int32 Index = 0; Index < NumVertices; ++Index)
			{
				FVector NewLocationActorVertex = ActorWorldTransform.TransformPosition(InActorInfo.ActorLocal.Vertices[Index]);
				InCoordinateConverter->WorldPositionToViewportPosition(NewLocationActorVertex, VertexScreenCoordinate, VertexDistance);
				ActorVertices[Index] = {VertexScreenCoordinate.X, VertexScreenCoordinate.Y, static_cast<float>(VertexDistance)};
			}

			float Min = ActorVertices[0][static_cast<uint8>(InAxis)];
			float Max = ActorVertices[0][static_cast<uint8>(InAxis)];

			for (int32 Index = 1; Index < NumVertices; ++Index)
			{
				Min = FMath::Min(Min, ActorVertices[Index][static_cast<uint8>(InAxis)]);
				Max = FMath::Max(Max, ActorVertices[Index][static_cast<uint8>(InAxis)]);
			}

			ScreenCenterCoordinate = (Min * 0.5) + (Max * 0.5);

			const float DistanceFromTarget = InCoordinate - ScreenCenterCoordinate;

			AddToScreenAxis(InAxis, ScreenCoordinate, Distance, DistanceFromTarget);

			NewWorldCenter = InCoordinateConverter->ViewportPositionToWorldPosition(ScreenCoordinate, Distance);
			++Iterations;
		}
		while (!FMath::IsNearlyEqual(ScreenCenterCoordinate, InCoordinate, ValueTolerance) && Iterations < MaxIterations);

		const FVector WorldCenterOffset = NewWorldCenter - InActorInfo.World.Center;

		FVector CameraLocalOffset = InActorInfo.CameraLocal.FromWorld.TransformVectorNoScale(WorldCenterOffset);

		if (InAxis != EAvaScreenAxis::Horizontal)
		{
			CameraLocalOffset[static_cast<uint8>(EAvaWorldAxis::Horizontal)] = 0;
		}

		if (InAxis != EAvaScreenAxis::Vertical)
		{
			CameraLocalOffset[static_cast<uint8>(EAvaWorldAxis::Vertical)] = 0;
		}

		if (InAxis != EAvaScreenAxis::Depth)
		{
			CameraLocalOffset[static_cast<uint8>(EAvaWorldAxis::Depth)] = 0;
		}

		return InActorInfo.CameraLocal.Location + CameraLocalOffset;
	}

	/**
	 * Takes the current actor position and uses its corner vertices to move it on screen to the given position.
	 * @param bInIsMinSide true: all vertices >= coordinate. false: all vertices <= coordinate.
	 * @return The new actor location in camera local space.
	 */
	[[nodiscard]] FVector AlignActorSide(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, 
		const FAvaScreenAlignmentActorInfo& InActorInfo, bool bInIsMinSide, EAvaScreenAxis InAxis, float InCoordinate)
	{
		const int32 NumVertices = InActorInfo.World.Vertices.Num();

		TArray<FVector> CameraSpaceActorPositions;
		CameraSpaceActorPositions.Reserve(NumVertices);

		for (int32 Index = 0; Index < NumVertices; ++Index)
		{
			FVector3f Coord = InActorInfo.Screen.Vertices[Index];
			Coord[static_cast<uint8>(InAxis)] = InCoordinate;

			CameraSpaceActorPositions.Add(ScreenVertexToActorLocation(
				InCoordinateConverter,
				InActorInfo,
				Index,
				Coord,
				FAvaScreenAlignmentActorInfo::ESpace::Camera
			));
		}

		int32 BestIndex = INDEX_NONE;
		float BestDistance = 0.f;

		FTransform ActorToWorld = InActorInfo.ActorLocal.ToWorld;

		FVector2f ScreenCoordinate;
		double Distance;

		for (const FVector& CameraSpaceActorPosition : CameraSpaceActorPositions)
		{
			const FVector WorldSpaceActorPosition = InActorInfo.CameraLocal.ToWorld.TransformPosition(CameraSpaceActorPosition);
			ActorToWorld.SetLocation(WorldSpaceActorPosition);

			bool bHasGoneOverLine = false;
			int32 LoopBestIndex = INDEX_NONE;
			float LoopBestDistance = 0.f;

			for (int32 Index = 0; Index < NumVertices; ++Index)
			{
				const FVector WorldVertexPosition = ActorToWorld.TransformPosition(InActorInfo.ActorLocal.Vertices[Index]);

				InCoordinateConverter->WorldPositionToViewportPosition(WorldVertexPosition, ScreenCoordinate, Distance);

				const float ScreenAxisValue = GetScreenAxis(InAxis, ScreenCoordinate, Distance);
				bool bValidValue;

				if (FMath::IsNearlyEqual(ScreenAxisValue, InCoordinate, ValueTolerance))
				{
					bValidValue = true;
				}
				else if (bInIsMinSide)
				{
					bValidValue = ScreenAxisValue > InCoordinate;
				}
				else
				{
					bValidValue = ScreenAxisValue < InCoordinate;
				}

				if (bValidValue)
				{
					// If we're slightly past the line (slightly!) treat this as if we're coming from the other direction
					// for the purposes of getting the closest value.
					const float DistanceFromLine = FMath::Abs(ScreenAxisValue - InCoordinate);

					if (LoopBestIndex == INDEX_NONE || DistanceFromLine < LoopBestDistance)
					{
						LoopBestIndex = Index;
						LoopBestDistance = DistanceFromLine;
					}
				}
				else
				{
					bHasGoneOverLine = true;
					break;
				}
			}

			if (bHasGoneOverLine)
			{
				continue;
			}

			if (BestIndex == INDEX_NONE || LoopBestDistance < BestDistance)
			{
				BestIndex = LoopBestIndex;
				BestDistance = LoopBestDistance;

				if (FMath::IsNearlyEqual(BestDistance, InCoordinate, ExactValueTolerance))
				{
					break;
				}
			}
		}

		// Should never happen
		if (BestIndex == INDEX_NONE)
		{
			return InActorInfo.CameraLocal.Location;
		}

		FVector NewCameraSpacePosition = CameraSpaceActorPositions[BestIndex];

		// Remove small aberrations on other axes
		if (InAxis != EAvaScreenAxis::Horizontal)
		{
			NewCameraSpacePosition[static_cast<uint8>(EAvaWorldAxis::Horizontal)]
				= InActorInfo.CameraLocal.Location[static_cast<uint8>(EAvaWorldAxis::Horizontal)];
		}

		if (InAxis != EAvaScreenAxis::Vertical)
		{
			NewCameraSpacePosition[static_cast<uint8>(EAvaWorldAxis::Vertical)]
				= InActorInfo.CameraLocal.Location[static_cast<uint8>(EAvaWorldAxis::Vertical)];
		}

		if (InAxis != EAvaScreenAxis::Depth)
		{
			NewCameraSpacePosition[static_cast<uint8>(EAvaWorldAxis::Depth)]
				= InActorInfo.CameraLocal.Location[static_cast<uint8>(EAvaWorldAxis::Depth)];
		}

		return NewCameraSpacePosition;
	}

	template<typename InAlignmentType>
	struct TAvaAlignment
	{
		static inline constexpr bool bValid = false;
		static inline constexpr InAlignmentType Min;
		static inline constexpr InAlignmentType Center;
		static inline constexpr InAlignmentType Max;
		static inline constexpr EAvaScreenAxis ScreenAxis = EAvaScreenAxis::Horizontal;
		static inline constexpr int32 ViewportCoordinate = -1;
	};

	template<>
	struct TAvaAlignment<EAvaHorizontalAlignment>
	{
		static inline constexpr bool bValid = true;
		static inline constexpr EAvaHorizontalAlignment Min = EAvaHorizontalAlignment::Left;
		static inline constexpr EAvaHorizontalAlignment Center = EAvaHorizontalAlignment::Center;
		static inline constexpr EAvaHorizontalAlignment Max = EAvaHorizontalAlignment::Right;
		static inline constexpr EAvaScreenAxis ScreenAxis = EAvaScreenAxis::Horizontal;
		static inline constexpr int32 ViewportCoordinate = 0; // X
	};

	template<>
	struct TAvaAlignment<EAvaVerticalAlignment>
	{
		static inline constexpr bool bValid = true;
		static inline constexpr EAvaVerticalAlignment Min = EAvaVerticalAlignment::Top;
		static inline constexpr EAvaVerticalAlignment Center = EAvaVerticalAlignment::Center;
		static inline constexpr EAvaVerticalAlignment Max = EAvaVerticalAlignment::Bottom;
		static inline constexpr EAvaScreenAxis ScreenAxis = EAvaScreenAxis::Vertical;
		static inline constexpr int32 ViewportCoordinate = 1; // Y
	};

	template<>
	struct TAvaAlignment<EAvaDepthAlignment>
	{
		static inline constexpr bool bValid = true;
		static inline constexpr EAvaDepthAlignment Min = EAvaDepthAlignment::Front;
		static inline constexpr EAvaDepthAlignment Center = EAvaDepthAlignment::Center;
		static inline constexpr EAvaDepthAlignment Max = EAvaDepthAlignment::Back;
		static inline constexpr EAvaScreenAxis ScreenAxis = EAvaScreenAxis::Depth;
		static inline constexpr int32 ViewportCoordinate = -1;
	};

	template<typename InAlignmentType
		UE_REQUIRES(TAvaAlignment<InAlignmentType>::bValid)>
	void AlignActors(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, const TArray<AActor*>& InActors, 
		InAlignmentType InAlignment, EAvaAlignmentSizeMode InActorSizeMode, EAvaAlignmentContext InContextType)
	{
		TArray<FAvaScreenAlignmentActorInfo> ActorInfos;
		ActorInfos.Reserve(InActors.Num());

		for (AActor* Actor : InActors)
		{
			if (IsValid(Actor))
			{
				ActorInfos.Add(FAvaScreenAlignmentActorInfo::Create(InCoordinateConverter, *Actor, InActorSizeMode));
			}
		}

		switch (InContextType)
		{
			case EAvaAlignmentContext::SelectedActors:
				if (ActorInfos.Num() < 2)
				{
					return;
				}
				break;

			case EAvaAlignmentContext::Screen:
				if (ActorInfos.Num() < 1)
				{
					return;
				}
				break;
		}		

		const int32 NumActors = ActorInfos.Num();

		float Min, Max;

		if (InContextType == EAvaAlignmentContext::Screen && TAvaAlignment<InAlignmentType>::ViewportCoordinate >= 0)
		{
			const FVector2f ViewportSize = InCoordinateConverter->GetViewportSize();
			Min = 0;
			Max = ViewportSize[TAvaAlignment<InAlignmentType>::ViewportCoordinate];
		}
		else
		{
			Min = ActorInfos[0].Screen.GetComponent(TAvaAlignment<InAlignmentType>::Min);
			Max = ActorInfos[0].Screen.GetComponent(TAvaAlignment<InAlignmentType>::Max);

			for (int32 Index = 1; Index < NumActors; ++Index)
			{
				Min = FMath::Min(Min, ActorInfos[Index].Screen.GetComponent(TAvaAlignment<InAlignmentType>::Min));
				Max = FMath::Max(Max, ActorInfos[Index].Screen.GetComponent(TAvaAlignment<InAlignmentType>::Max));
			}
		}

		float Target;

		switch (InAlignment)
		{
			case TAvaAlignment<InAlignmentType>::Min:
				Target = Min;
				break;

			case TAvaAlignment<InAlignmentType>::Center:
				Target = (Min * 0.5) + (Max * 0.5);
				break;

			case TAvaAlignment<InAlignmentType>::Max:
				Target = Max;
				break;

			default:
				checkNoEntry();
				return;
		}

		for (const FAvaScreenAlignmentActorInfo& ActorInfo : ActorInfos)
		{
			FVector NewCameraSpacePosition;

			if (InAlignment == TAvaAlignment<InAlignmentType>::Center)
			{
				NewCameraSpacePosition = AlignActorCenter(
					InCoordinateConverter,
					ActorInfo,
					TAvaAlignment<InAlignmentType>::ScreenAxis,
					Target
				);
			}
			else
			{
				NewCameraSpacePosition = AlignActorSide(
					InCoordinateConverter,
					ActorInfo,
					(InAlignment == TAvaAlignment<InAlignmentType>::Min),
					TAvaAlignment<InAlignmentType>::ScreenAxis,
					Target
				);
			}

			ActorInfo.Actor->SetActorLocation(
				ActorInfo.CameraLocal.ToWorld.TransformPosition(NewCameraSpacePosition)
			);
		}
	}

	template<typename InAlignmentType
		UE_REQUIRES(TAvaAlignment<InAlignmentType>::bValid)>
	void DistributeActors(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, const TArray<AActor*>& InActors, 
		EAvaAlignmentSizeMode InActorSizeMode, EAvaActorDistributionMode InDistributionMode, EAvaAlignmentContext InContextType)
	{
		TArray<FAvaScreenAlignmentActorInfo> ActorInfos;
		ActorInfos.Reserve(InActors.Num());

		for (AActor* Actor : InActors)
		{
			if (IsValid(Actor))
			{
				ActorInfos.Add(FAvaScreenAlignmentActorInfo::Create(InCoordinateConverter, *Actor, InActorSizeMode));
			}
		}

		switch (InContextType)
		{
			case EAvaAlignmentContext::SelectedActors:
				if (ActorInfos.Num() < 3)
				{
					return;
				}
				break;

			case EAvaAlignmentContext::Screen:
				if (ActorInfos.Num() < 2)
				{
					return;
				}
				break;
		}

		const int32 NumActors = ActorInfos.Num();

		float LoopMin = ActorInfos[0].Screen.GetComponent(TAvaAlignment<InAlignmentType>::Min);
		float LoopMax = ActorInfos[0].Screen.GetComponent(TAvaAlignment<InAlignmentType>::Max);
		float TotalSize = LoopMax - LoopMin;
		float Min = LoopMin;
		float Max = LoopMax;

		for (int32 Index = 1; Index < NumActors; ++Index)
		{
			LoopMin = ActorInfos[Index].Screen.GetComponent(TAvaAlignment<InAlignmentType>::Min);
			LoopMax = ActorInfos[Index].Screen.GetComponent(TAvaAlignment<InAlignmentType>::Max);
			TotalSize += LoopMax - LoopMin;
			Min = FMath::Min(Min, LoopMin);
			Max = FMath::Max(Max, LoopMax);
		}

		ActorInfos.Sort(
			[](const FAvaScreenAlignmentActorInfo& InA, const FAvaScreenAlignmentActorInfo& InB)
			{
				const float AMin = InA.Screen.GetComponent(TAvaAlignment<InAlignmentType>::Min);
				const float AMax = InA.Screen.GetComponent(TAvaAlignment<InAlignmentType>::Max);
				const float AMid = (AMin * 0.5f) + (AMax * 0.5f);

				const float BMin = InB.Screen.GetComponent(TAvaAlignment<InAlignmentType>::Min);
				const float BMax = InB.Screen.GetComponent(TAvaAlignment<InAlignmentType>::Max);
				const float BMid = (BMin * 0.5f) + (BMax * 0.5f);

				return AMid < BMid;
			}
		);

		float SpacePerItem;

		if (InContextType == EAvaAlignmentContext::Screen && TAvaAlignment<InAlignmentType>::ViewportCoordinate >= 0)
		{
			const FVector2f ViewportSize = InCoordinateConverter->GetViewportSize();
			Min = 0.f;
			Max = ViewportSize[TAvaAlignment<InAlignmentType>::ViewportCoordinate];
		}

		switch (InDistributionMode)
		{
			case EAvaActorDistributionMode::CenterDistance:
				SpacePerItem = (Max - Min) / static_cast<float>(NumActors - 1);
				break;

			case EAvaActorDistributionMode::EdgeDistance:
				SpacePerItem = (Max - Min - TotalSize) / static_cast<float>(NumActors - 1);
				break;

			default:
				return;
		}

		float Offset = 0.f;

		for (int32 Index = 0; Index < ActorInfos.Num(); ++Index)
		{
			const FAvaScreenAlignmentActorInfo& ActorInfo = ActorInfos[Index];

			LoopMin = ActorInfo.Screen.GetComponent(TAvaAlignment<InAlignmentType>::Min);
			LoopMax = ActorInfo.Screen.GetComponent(TAvaAlignment<InAlignmentType>::Max);

			const float Width = LoopMax - LoopMin;

			const bool bIsEdgeActor = ActorInfo.Actor == ActorInfos[0].Actor || ActorInfo.Actor == ActorInfos.Last().Actor;

			float NewCoordinate = 0.f;

			switch (InDistributionMode)
			{
				case EAvaActorDistributionMode::CenterDistance:
				{
					const float Progress = static_cast<float>(Index) / static_cast<float>(ActorInfos.Num() - 1);
					NewCoordinate = Min + Offset + ((0.5f - Progress) * Width);
					Offset += SpacePerItem;
					break;
				}

				case EAvaActorDistributionMode::EdgeDistance:
				{
					NewCoordinate = Min + Offset + (Width * 0.5f);
					Offset += Width + SpacePerItem;
					break;
				}
			}

			// The first and last actors are already in the correct places if we're using actor alignment.
			if (!bIsEdgeActor || InContextType == EAvaAlignmentContext::Screen)
			{
				const FVector NewCameraSpaceActorLocation = AlignActorCenter(
					InCoordinateConverter,
					ActorInfo,
					TAvaAlignment<InAlignmentType>::ScreenAxis,
					NewCoordinate
				);

				ActorInfo.Actor->SetActorLocation(
					ActorInfo.CameraLocal.ToWorld.TransformPosition(NewCameraSpaceActorLocation)
				);
			}
		}
	}
}

void FAvaScreenAlignmentUtils::SizeActorToScreen(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, 
	AActor& InActor, bool bInStretchToFit)
{
	const FAvaScreenAlignmentActorInfo ActorInfo = FAvaScreenAlignmentActorInfo::Create(InCoordinateConverter,
		InActor, EAvaAlignmentSizeMode::Self);

	UE::AvaViewport::Private::SizeActorToScreen(InCoordinateConverter, ActorInfo, bInStretchToFit);
}

void FAvaScreenAlignmentUtils::FitActorToScreen(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter,
	AActor& InActor, bool bInStretchToFit, bool bInAlignToNearestAxis)
{
	const FTransform CameraTransform = InCoordinateConverter->GetViewportViewTransform();
	const FVector CameraForward = CameraTransform.TransformVectorNoScale(FVector::ForwardVector);
	const FVector CameraLocation = CameraTransform.GetLocation();
	const FVector ActorLocation = InActor.GetActorLocation();

	const double Distance = (ActorLocation - CameraLocation).Size();

	InActor.SetActorLocation(CameraLocation + (CameraForward * Distance));

	AlignActorsCameraRotationAxis(InCoordinateConverter, {&InActor}, EAvaRotationAxis::All, bInAlignToNearestAxis, /* Backwards */ false);

	if (UAvaBoundsProviderSubsystem* BoundsSubsystem = UAvaBoundsProviderSubsystem::Get(&InActor))
	{
		for (int32 Pass = 0; Pass < 2; ++Pass)
		{
			const FAvaScreenAlignmentActorInfo ActorInfo = FAvaScreenAlignmentActorInfo::Create(InCoordinateConverter,
				InActor, EAvaAlignmentSizeMode::Self);

			UE::AvaViewport::Private::SizeActorToScreen(InCoordinateConverter, ActorInfo, bInStretchToFit);

			const FTransform ActorTransform = InActor.GetActorTransform();
			const FBox ActorLocalBounds = BoundsSubsystem->GetActorLocalBounds(&InActor);
			const FVector ActorLocalOrigin = ActorLocalBounds.GetCenter();

			if (!ActorLocalOrigin.IsNearlyZero())
			{
				InActor.SetActorLocation(ActorTransform.GetLocation() - ActorTransform.TransformVector(ActorLocalOrigin) * 0.5);
			}
		}
	}
}

void FAvaScreenAlignmentUtils::AlignActorRotationAxis(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, AActor& InActor,
	EAvaRotationAxis InAxis, const FRotator& InAlignToRotation, bool bInAlignToNearestAxis, bool bInBackwards)
{
	FAvaViewportAxisMap ActorViewportAxisMap(FAvaViewportAxisUtils::WorldAxisIndexList);

	if (bInAlignToNearestAxis)
	{
		ActorViewportAxisMap = FAvaViewportAxisUtils::CreateViewportAxisMap(
			InCoordinateConverter->GetViewportViewTransform(),
			InActor
		);
	}

	using namespace UE::AvaViewport::Private;

	bool bMadeChange = false;
	UE::Math::TMatrix<double> ActorRotationMatrix = UE::Math::TRotationMatrix<double>(InActor.GetActorRotation());
	const UE::Math::TMatrix<double> AlignToRotationMatrix = UE::Math::TRotationMatrix<double>(InAlignToRotation);

	// Rotate the actor, ensuring that it stays in the same plane as defined by its axis of rotation and current location.
	auto AlignActorOnAxis = [&ActorRotationMatrix, &AlignToRotationMatrix, &bInBackwards](const FAvaAxisDetails& InActorAxisDetailsRotationAxis,
		EAxis::Type InAlignToAxisOne, const FAvaAxisDetails& InActorAxisDetailsOne, EAxis::Type InAlignToAxisTwo, const FAvaAxisDetails& InActorAxisDetailsTwo)
		{
			const FPlane Plane = FPlane(FVector::ZeroVector, ActorRotationMatrix.GetUnitAxis(InActorAxisDetailsRotationAxis.Axis));

			auto AlignAxisOnPlane = [&ActorRotationMatrix, &AlignToRotationMatrix, &bInBackwards, &Plane]
				(EAxis::Type InAlignToAxis, const FAvaAxisDetails& InActorAxisDetails)
				{
					FVector Axis = AlignToRotationMatrix.GetUnitAxis(InAlignToAxis) * (bInBackwards ? -1.0 : 1.0) * (InActorAxisDetails.bCodirectional ? 1.0 : -1.0);
					Axis = UE::Math::TPlane<double>::PointPlaneProject(Axis, Plane);
					Axis.Normalize();
					ActorRotationMatrix.SetAxis(InActorAxisDetails.Index, Axis);
				};

			AlignAxisOnPlane(InAlignToAxisOne, InActorAxisDetailsOne);
			AlignAxisOnPlane(InAlignToAxisTwo, InActorAxisDetailsTwo);
		};

	const TAvaAxisList<FAvaAxisDetails> AxisDetails = {
		FAvaAxisDetails(ActorViewportAxisMap, ActorViewportAxisMap.Horizontal.Index),
		FAvaAxisDetails(ActorViewportAxisMap, ActorViewportAxisMap.Vertical.Index),
		FAvaAxisDetails(ActorViewportAxisMap, ActorViewportAxisMap.Depth.Index)
	};

	if (AxisDetails.Horizontal.Axis != EAxis::None && AxisDetails.Vertical.Axis != EAxis::None && AxisDetails.Depth.Axis != EAxis::None)
	{
		if (EnumHasAnyFlags(InAxis, EAvaRotationAxis::Yaw))
		{
			AlignActorOnAxis(
				AxisDetails.Vertical, 
				FAvaViewportAxisUtils::GetRotationAxisForVectorComponent(FAvaViewportAxisUtils::WorldAxisIndexList.Horizontal.Index),
				AxisDetails.Horizontal, 
				FAvaViewportAxisUtils::GetRotationAxisForVectorComponent(FAvaViewportAxisUtils::WorldAxisIndexList.Depth.Index),
				AxisDetails.Depth
			);

			bMadeChange = true;
		}

		if (EnumHasAnyFlags(InAxis, EAvaRotationAxis::Pitch))
		{
			AlignActorOnAxis(
				AxisDetails.Horizontal, 
				FAvaViewportAxisUtils::GetRotationAxisForVectorComponent(FAvaViewportAxisUtils::WorldAxisIndexList.Vertical.Index),
				AxisDetails.Vertical,
				FAvaViewportAxisUtils::GetRotationAxisForVectorComponent(FAvaViewportAxisUtils::WorldAxisIndexList.Depth.Index),
				AxisDetails.Depth
			);

			bMadeChange = true;
		}

		if (EnumHasAnyFlags(InAxis, EAvaRotationAxis::Roll))
		{
			AlignActorOnAxis(
				AxisDetails.Depth,
				FAvaViewportAxisUtils::GetRotationAxisForVectorComponent(FAvaViewportAxisUtils::WorldAxisIndexList.Horizontal.Index),
				AxisDetails.Horizontal,
				FAvaViewportAxisUtils::GetRotationAxisForVectorComponent(FAvaViewportAxisUtils::WorldAxisIndexList.Vertical.Index),
				AxisDetails.Vertical
			);

			bMadeChange = true;
		}
	}

	if (bMadeChange)
	{
		InActor.SetActorRotation(ActorRotationMatrix.Rotator());
	}
}

void FAvaScreenAlignmentUtils::AlignActorsCameraRotationAxis(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, 
	const TArray<AActor*>& InActors, EAvaRotationAxis InAxis, bool bInAlignToNearestAxis, bool bInBackwards)
{
	const FTransform CameraTransform = InCoordinateConverter->GetViewportViewTransform();
	const FRotator CameraRotation = CameraTransform.Rotator();

	for (AActor* Actor : InActors)
	{
		if (IsValid(Actor))
		{
			AlignActorRotationAxis(InCoordinateConverter , *Actor, InAxis, CameraRotation, bInAlignToNearestAxis, bInBackwards);
		}
	}
}

void FAvaScreenAlignmentUtils::AlignActorsHorizontal(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, 
	const TArray<AActor*>& InActors, EAvaHorizontalAlignment InHorizontalAlignment, EAvaAlignmentSizeMode InActorSizeMode, 
	EAvaAlignmentContext InContextType)
{
	UE::AvaViewport::Private::AlignActors<EAvaHorizontalAlignment>(
		InCoordinateConverter,
		InActors,
		InHorizontalAlignment,
		InActorSizeMode,
		InContextType
	);
}

void FAvaScreenAlignmentUtils::AlignActorsVertical(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, 
	const TArray<AActor*>& InActors, EAvaVerticalAlignment InVerticalAlignment, EAvaAlignmentSizeMode InActorSizeMode, 
	EAvaAlignmentContext InContextType)
{
	UE::AvaViewport::Private::AlignActors<EAvaVerticalAlignment>(
		InCoordinateConverter,
		InActors,
		InVerticalAlignment,
		InActorSizeMode,
		InContextType
	);
}

void FAvaScreenAlignmentUtils::AlignActorsDepth(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, 
	const TArray<AActor*>& InActors, EAvaDepthAlignment InDepthAlignment, EAvaAlignmentSizeMode InActorSizeMode)
{
	UE::AvaViewport::Private::AlignActors<EAvaDepthAlignment>(
		InCoordinateConverter,
		InActors,
		InDepthAlignment,
		InActorSizeMode,
		EAvaAlignmentContext::SelectedActors
	);
}

void FAvaScreenAlignmentUtils::DistributeActorsHorizontal(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, 
	const TArray<AActor*>& InActors, EAvaAlignmentSizeMode InActorSizeMode, EAvaActorDistributionMode InDistributionMode, 
	EAvaAlignmentContext InContextType)
{
	UE::AvaViewport::Private::DistributeActors<EAvaHorizontalAlignment>(
		InCoordinateConverter,
		InActors,
		InActorSizeMode,
		InDistributionMode,
		InContextType
	);
}

void FAvaScreenAlignmentUtils::DistributeActorsVertical(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter,
	const TArray<AActor*>& InActors, EAvaAlignmentSizeMode InActorSizeMode, EAvaActorDistributionMode InDistributionMode, 
	EAvaAlignmentContext InContextType)
{
	UE::AvaViewport::Private::DistributeActors<EAvaVerticalAlignment>(
		InCoordinateConverter,
		InActors,
		InActorSizeMode,
		InDistributionMode,
		InContextType
	);
}

void FAvaScreenAlignmentUtils::DistributeActorsDepth(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter,
	const TArray<AActor*>& InActors, EAvaAlignmentSizeMode InActorSizeMode, EAvaActorDistributionMode InDistributionMode)
{
	UE::AvaViewport::Private::DistributeActors<EAvaDepthAlignment>(
		InCoordinateConverter,
		InActors,
		InActorSizeMode,
		InDistributionMode,
		EAvaAlignmentContext::SelectedActors
	);
}
