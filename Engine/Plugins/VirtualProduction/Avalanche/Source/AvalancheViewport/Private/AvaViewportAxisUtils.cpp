// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaViewportAxisUtils.h"
#include "GameFramework/Actor.h"

FAvaViewportAxisMap FAvaViewportAxisUtils::CreateViewportAxisMap(const FTransform& InViewportCameraTransform, const AActor& InActor)
{
	FAvaViewportAxisMap ViewportAxisMap(WorldAxisIndexList);

	if (!IsValid(&InActor))
	{
		return ViewportAxisMap;
	}

	const TAvaAxisList<FVector> CameraAxisVectors =
	{
		InViewportCameraTransform.TransformVectorNoScale(FVector::RightVector),
		InViewportCameraTransform.TransformVectorNoScale(FVector::UpVector),
		InViewportCameraTransform.TransformVectorNoScale(FVector::ForwardVector)
	};

	const TAvaAxisList<FVector> ActorAxisVectors =
	{
		InActor.GetActorRightVector(),
		InActor.GetActorUpVector(),
		InActor.GetActorForwardVector()
	};

	return CreateViewportAxisMap(CameraAxisVectors, ActorAxisVectors);
}

FAvaViewportAxisMap FAvaViewportAxisUtils::CreateViewportAxisMap(const FTransform& InViewportCameraTransform)
{
	static const TAvaAxisList<FVector> WorldAxisVectors = {
		FVector::RightVector,
		FVector::UpVector,
		FVector::ForwardVector
	};

	const TAvaAxisList<FVector> AxisVectors =
	{
		InViewportCameraTransform.TransformVectorNoScale(FVector::RightVector),
		InViewportCameraTransform.TransformVectorNoScale(FVector::UpVector),
		InViewportCameraTransform.TransformVectorNoScale(FVector::ForwardVector)
	};

	return CreateViewportAxisMap(WorldAxisVectors, AxisVectors);
}

FAvaViewportAxisMap FAvaViewportAxisUtils::CreateViewportAxisMap(const TAvaAxisList<FVector>& InReferenceVectors, 
	const TAvaAxisList<FVector>& InAxisVectors)
{
	FAvaViewportAxisMap ViewportAxisMap(WorldAxisIndexList);

	TAvaAxisList<bool> bAxisAssigned = {false, false, false};

	// Calculate horizontal axis
	const TAvaAxisList<double> HorizontalDots = {
		InReferenceVectors.Horizontal.Dot(InAxisVectors.Horizontal),
		InReferenceVectors.Horizontal.Dot(InAxisVectors.Vertical),
		InReferenceVectors.Horizontal.Dot(InAxisVectors.Depth)
	};

	const TAvaAxisList<double> HorizontalAlignment = {
		FMath::Abs<double>(HorizontalDots.Horizontal),
		FMath::Abs<double>(HorizontalDots.Vertical),
		FMath::Abs<double>(HorizontalDots.Depth)
	};

	// The closer to 1, the more parallel they are to that axis.
	if (HorizontalAlignment.Horizontal >= HorizontalAlignment.Vertical
		&& HorizontalAlignment.Horizontal >= HorizontalAlignment.Depth)
	{
		bAxisAssigned.Horizontal = true;
		ViewportAxisMap.Horizontal.Index = WorldAxisIndexList.Horizontal.Index;
		ViewportAxisMap.Horizontal.bCodirectional = HorizontalDots.Horizontal > 0;
	}
	else if (HorizontalAlignment.Vertical >= HorizontalAlignment.Horizontal
		&& HorizontalAlignment.Vertical >= HorizontalAlignment.Depth)
	{
		bAxisAssigned.Vertical = true;
		ViewportAxisMap.Horizontal.Index = WorldAxisIndexList.Vertical.Index;
		ViewportAxisMap.Horizontal.bCodirectional = HorizontalDots.Vertical > 0;
	}
	else
	{
		bAxisAssigned.Depth = true;
		ViewportAxisMap.Horizontal.Index = WorldAxisIndexList.Depth.Index;
		ViewportAxisMap.Horizontal.bCodirectional = HorizontalDots.Depth > 0;
	}

	// Calculate vertical axis
	const TAvaAxisList<double> VerticalDots = {
		bAxisAssigned.Horizontal ? 0.0 : InReferenceVectors.Vertical.Dot(InAxisVectors.Horizontal),
		bAxisAssigned.Vertical ? 0.0 : InReferenceVectors.Vertical.Dot(InAxisVectors.Vertical),
		bAxisAssigned.Depth ? 0.0 : InReferenceVectors.Vertical.Dot(InAxisVectors.Depth)
	};

	const TAvaAxisList<double> VerticalAlignment = {
		bAxisAssigned.Horizontal ? 0.0 : FMath::Abs(VerticalDots.Horizontal),
		bAxisAssigned.Vertical ? 0.0 : FMath::Abs(VerticalDots.Vertical),
		bAxisAssigned.Depth ? 0.0 : FMath::Abs(VerticalDots.Depth)
	};

	if (!bAxisAssigned.Horizontal
		&& (VerticalAlignment.Horizontal >= VerticalAlignment.Vertical
			&& VerticalAlignment.Horizontal >= VerticalAlignment.Depth))
	{
		bAxisAssigned.Horizontal = true;
		ViewportAxisMap.Vertical.Index = WorldAxisIndexList.Horizontal.Index;
		ViewportAxisMap.Vertical.bCodirectional = VerticalDots.Horizontal > 0;
	}
	else if (!bAxisAssigned.Vertical
		&& (VerticalAlignment.Vertical >= VerticalAlignment.Horizontal
			&& VerticalAlignment.Vertical >= VerticalAlignment.Depth))
	{
		bAxisAssigned.Vertical = true;
		ViewportAxisMap.Vertical.Index = WorldAxisIndexList.Vertical.Index;
		ViewportAxisMap.Vertical.bCodirectional = VerticalDots.Vertical > 0;
	}
	else
	{
		bAxisAssigned.Depth = true;
		ViewportAxisMap.Vertical.Index = WorldAxisIndexList.Depth.Index;
		ViewportAxisMap.Vertical.bCodirectional = VerticalDots.Depth > 0;
	}

	// Calculate vertical axis
	if (!bAxisAssigned.Horizontal)
	{
		ViewportAxisMap.Depth.Index = WorldAxisIndexList.Horizontal.Index;
		ViewportAxisMap.Depth.bCodirectional = InReferenceVectors.Depth.Dot(InAxisVectors.Horizontal) > 0;
	}
	else if (!bAxisAssigned.Vertical)
	{
		ViewportAxisMap.Depth.Index = WorldAxisIndexList.Vertical.Index;
		ViewportAxisMap.Depth.bCodirectional = InReferenceVectors.Depth.Dot(InAxisVectors.Vertical) > 0;
	}
	else
	{
		ViewportAxisMap.Depth.Index = WorldAxisIndexList.Depth.Index;
		ViewportAxisMap.Depth.bCodirectional = InReferenceVectors.Depth.Dot(InAxisVectors.Depth) > 0;
	}

	return ViewportAxisMap;
}

EAxis::Type FAvaViewportAxisUtils::GetRotationAxisForVectorComponent(int32 InComponent)
{
	switch (InComponent)
	{
		case WorldAxisIndexList.Depth.Index:
			return EAxis::X;

		case WorldAxisIndexList.Horizontal.Index:
			return EAxis::Y;

		case WorldAxisIndexList.Vertical.Index:
			return EAxis::Z;

		default:
			return EAxis::None;
	}
}
