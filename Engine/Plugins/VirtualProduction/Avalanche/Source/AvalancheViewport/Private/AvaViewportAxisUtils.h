// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "Math/Axis.h"

class AActor;

template<typename T>
struct TAvaAxisList
{
	T Horizontal;
	T Vertical;
	T Depth;
};

struct FAvaViewportAxisOrientation
{
	int32 Index;
	bool bCodirectional;
};

using FAvaViewportAxisMap = TAvaAxisList<FAvaViewportAxisOrientation>;

class FAvaViewportAxisUtils
{
public:
	static constexpr FAvaViewportAxisMap WorldAxisIndexList = {
		{1, true},
		{2, true},
		{0, true}
	};
		
	/* Takes a viewport and actor's rotation and uses it to determine which of the actor's axes most closely match the camera's. */
	[[nodiscard]] static FAvaViewportAxisMap CreateViewportAxisMap(const FTransform& InViewportCameraTransform, const AActor& InActor);

	[[nodiscard]] static FAvaViewportAxisMap CreateViewportAxisMap(const FTransform& InViewportCameraTransform);

	[[nodiscard]] static FAvaViewportAxisMap CreateViewportAxisMap(const TAvaAxisList<FVector>& InReferenceVectors, const TAvaAxisList<FVector>& InAxisVectors);

	static EAxis::Type GetRotationAxisForVectorComponent(int32 InComponent);
};
