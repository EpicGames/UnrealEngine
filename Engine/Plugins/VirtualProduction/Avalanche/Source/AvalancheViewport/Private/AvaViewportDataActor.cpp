// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaViewportDataActor.h"

AAvaViewportDataActor::AAvaViewportDataActor()
{
	ViewportData.PostProcessInfo.Type = EAvaViewportPostProcessType::None;
	ViewportData.PostProcessInfo.Opacity = 1.f;
	ViewportData.VirtualSize = FIntPoint::ZeroValue;
	ViewportData.VirtualSizeAspectRatioState = EAvaViewportVirtualSizeAspectRatioState::LockedToCamera;
}
