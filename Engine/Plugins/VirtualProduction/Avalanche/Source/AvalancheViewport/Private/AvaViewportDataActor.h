// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaViewportGuideInfo.h"
#include "AvaViewportPostProcessManager.h"
#include "AvaViewportDataSubsystem.h"
#include "AvaViewportVirtualSizeEnums.h"
#include "GameFramework/Actor.h"
#include "Math/IntPoint.h"
#include "UObject/SoftObjectPtr.h"
#include "Viewport/Interaction/AvaViewportPostProcessInfo.h"
#include "AvaViewportDataActor.generated.h"

UCLASS(NotPlaceable, Hidden, DisplayName = "Motion Design Viewport Data Actor")
class AAvaViewportDataActor : public AActor
{
	GENERATED_BODY()

public:
	AAvaViewportDataActor();

	virtual ~AAvaViewportDataActor() override = default;

	/** Indicates whether this actor should participate in level bounds calculations. */
	virtual bool IsLevelBoundsRelevant() const override { return false; }
	
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }

	UPROPERTY()
	FAvaViewportData ViewportData;

private:
	virtual bool ActorTypeSupportsDataLayer() const override { return false; }
};
