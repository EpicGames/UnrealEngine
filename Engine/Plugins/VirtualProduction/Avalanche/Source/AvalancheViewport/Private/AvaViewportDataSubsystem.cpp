// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaViewportDataSubsystem.h"
#include "AvaViewportDataActor.h"
#include "EngineUtils.h"

FAvaViewportData* UAvaViewportDataSubsystem::GetData()
{
	if (AAvaViewportDataActor* DataActor = GetDataActor())
	{
		return &DataActor->ViewportData;
	}

	ensureMsgf(false, TEXT("Failed to find or create data actor."));

	return nullptr;
}

void UAvaViewportDataSubsystem::ModifyDataSource()
{
	if (AAvaViewportDataActor* DataActor = GetDataActor())
	{
		DataActor->Modify();
		return;
	}

	ensureMsgf(false, TEXT("Failed to find or create data actor."));
}

FAvaViewportGuidePresetProvider& UAvaViewportDataSubsystem::GetGuidePresetProvider()
{
	return GuidePresetProvider;
}

AAvaViewportDataActor* UAvaViewportDataSubsystem::GetDataActor()
{
	AAvaViewportDataActor* DataActor = DataActorWeak.Get();

	if (!IsValid(DataActor))
	{
		DataActorWeak.Reset();

		if (auto It = TActorIterator<AAvaViewportDataActor>(GetWorld()); It)
		{
			DataActor = *It;
		}

		if (!DataActor)
		{
			FActorSpawnParameters Params;
			Params.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
			Params.bHideFromSceneOutliner = true;
			Params.bNoFail = true;

			DataActor = GetWorld()->SpawnActor<AAvaViewportDataActor>(Params);
		}

		if (DataActor)
		{
			DataActorWeak = DataActor;
		}
	}

	return DataActor;
}

bool UAvaViewportDataSubsystem::DoesSupportWorldType(const EWorldType::Type InWorldType) const
{
	return InWorldType == EWorldType::Editor
		|| InWorldType == EWorldType::Inactive
		|| InWorldType == EWorldType::PIE;
}
