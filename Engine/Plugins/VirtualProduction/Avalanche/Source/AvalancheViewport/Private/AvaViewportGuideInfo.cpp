// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaViewportGuideInfo.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"

namespace UE::AvaViewport::Private
{
	static const FString IsHorizontalFieldName = TEXT("IsHorizontal");
	static const FString OffsetFieldName = TEXT("Offset");
	static const FString EnabledFieldName = TEXT("IsEnabled");
	static const FString LockedFieldName = TEXT("IsLocked");
}

bool FAvaViewportGuideInfo::DeserializeJson(const TSharedRef<FJsonObject>& InJsonObject, FAvaViewportGuideInfo& OutGuideInfo, const FVector2f& InViewportSize)
{
	using namespace UE::AvaViewport::Private;

	bool bValid = false;

	if (InJsonObject->HasField(IsHorizontalFieldName))
	{
		OutGuideInfo.Orientation = InJsonObject->GetBoolField(IsHorizontalFieldName)
			? EOrientation::Orient_Horizontal
			: EOrientation::Orient_Vertical;

		bValid = true;
	}

	if (InJsonObject->HasField(OffsetFieldName))
	{
		const bool bIsHorizontal = OutGuideInfo.Orientation == EOrientation::Orient_Horizontal;

		OutGuideInfo.OffsetFraction = InJsonObject->GetNumberField(OffsetFieldName)
			/ (bIsHorizontal ? InViewportSize.Y : InViewportSize.X);

		bValid = true;
	}

	if (InJsonObject->HasField(EnabledFieldName))
	{
		OutGuideInfo.State = InJsonObject->GetBoolField(EnabledFieldName)
			? EAvaViewportGuideState::Enabled
			: EAvaViewportGuideState::Disabled;

		bValid = true;
	}

	if (InJsonObject->HasField(LockedFieldName))
	{
		OutGuideInfo.bLocked = InJsonObject->GetBoolField(LockedFieldName);
		bValid = true;
	}

	// Tell the call it's a valid bit of json if there's any matching fields.
	return bValid;
}

bool FAvaViewportGuideInfo::SerializeJson(const TSharedRef<FJsonObject>& InJsonObject, const FVector2f& InViewportSize) const
{
	using namespace UE::AvaViewport::Private;

	const bool bIsHorziontal = Orientation == EOrientation::Orient_Horizontal;

	InJsonObject->SetBoolField(IsHorizontalFieldName, bIsHorziontal);
	InJsonObject->SetNumberField(OffsetFieldName, FMath::RoundToInt(OffsetFraction * (bIsHorziontal ? InViewportSize.Y : InViewportSize.X)));
	InJsonObject->SetBoolField(EnabledFieldName, IsEnabled());
	InJsonObject->SetBoolField(LockedFieldName, bLocked);

	return true;
}
