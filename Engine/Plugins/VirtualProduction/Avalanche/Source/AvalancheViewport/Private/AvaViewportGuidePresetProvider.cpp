// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaViewportGuidePresetProvider.h"
#include "AvalancheViewportModule.h"
#include "AvaViewportGuideInfo.h"
#include "AvaViewportSettings.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "HAL/FileManager.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializable.h"
#include "Serialization/JsonWriter.h"

namespace UE::AvaViewport::Private
{
	static const FString GuideListFieldName = TEXT("Guides");

	FString GetGuideConfigPath(bool bAllowCreate = true)
	{
		const FString& BasePath = GetDefault<UAvaViewportSettings>()->GuideConfigPath;

		if (FPaths::DirectoryExists(BasePath))
		{
			return BasePath;
		}

		const FString ProjectPath = FPaths::ProjectDir() / BasePath;

		if (FPaths::DirectoryExists(ProjectPath))
		{
			return ProjectPath;
		}

		if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME))
		{
			const FString PluginPath = Plugin->GetBaseDir() / BasePath;

			if (FPaths::DirectoryExists(PluginPath))
			{
				return PluginPath;
			}
		}

		if (bAllowCreate)
		{
			static const FString ConfigFolderStart = (FString(TEXT(".")) / FString(TEXT("Config")) / FString(TEXT("."))).RightChop(1).LeftChop(1);

			if (BasePath.StartsWith(ConfigFolderStart))
			{
				IFileManager::Get().MakeDirectory(*ProjectPath, /* Create tree */ true);
				return ProjectPath;
			}
		}

		return "";
	}

	FString GetPresetPath(const FString& InPresetName)
	{
		return GetGuideConfigPath() / InPresetName / TEXT(".json");
	}

	bool LoadPreset(const FString& InPath, TArray<FAvaViewportGuideInfo>& OutGuides, const FVector2f InViewportSize)
	{
		FString FileJsonString;

		if (!FFileHelper::LoadFileToString(FileJsonString, *InPath))
		{
			UE_LOGF(AvaViewportLog, Warning, "Motion Design Viewport Guide: LoadPreset() - Unable to load Json file: %ls", *InPath);
			return false;
		}

		const TSharedPtr<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(FileJsonString);
		TSharedPtr<FJsonObject> RootObject;

		if (!FJsonSerializer::Deserialize(JsonReader.ToSharedRef(), RootObject))
		{
			UE_LOGF(AvaViewportLog, Warning, "Motion Design Viewport Guide: LoadPreset() - Unable to parse file [%ls]. Json=[%ls]", *InPath, *FileJsonString);
			return false;
		}

		if (!RootObject->HasTypedField<EJson::Array>(GuideListFieldName))
		{
			UE_LOGF(AvaViewportLog, Warning, "Motion Design Viewport Guide: LoadPreset() - Malformed data in json [%ls]. Json=[%ls]", *InPath, *FileJsonString);
			return false;
		}

		TArray<TSharedPtr<FJsonValue>> JsonGuides = RootObject->GetArrayField(GuideListFieldName);
		TArray<FAvaViewportGuideInfo> Guides = {};

		for (const TSharedPtr<FJsonValue>& JsonGuide : JsonGuides)
		{
			if (TSharedPtr<FJsonObject> JsonGuideObject = JsonGuide->AsObject())
			{
				FAvaViewportGuideInfo GuideInfo;

				if (FAvaViewportGuideInfo::DeserializeJson(JsonGuideObject.ToSharedRef(), GuideInfo, InViewportSize))
				{
					Guides.Add(MoveTemp(GuideInfo));
				}
				else
				{
					UE_LOGF(AvaViewportLog, Warning, "Motion Design Viewport Guide: LoadPreset() - Malformed data in guide json [%ls]. Json=[%ls]", *InPath, *FileJsonString);
					return false;
				}
			}
			else
			{
				UE_LOGF(AvaViewportLog, Warning, "Motion Design Viewport Guide: LoadPreset() - Malformed data in json [%ls]. Json=[%ls]", *InPath, *FileJsonString);
				return false;
			}
		}

		OutGuides = MoveTemp(Guides);
		return true;
	}

	bool SavePreset(const FString& InPath, const TArray<FAvaViewportGuideInfo>& InGuides, const FVector2f InViewportSize)
	{
		TArray<TSharedPtr<FJsonValue>> JsonGuides;
		JsonGuides.Reserve(InGuides.Num());

		for (const FAvaViewportGuideInfo& Guide : InGuides)
		{
			TSharedRef<FJsonObject> JsonGuide = MakeShared<FJsonObject>();

			if (Guide.SerializeJson(JsonGuide, InViewportSize))
			{
				JsonGuides.Add(MakeShared<FJsonValueObject>(JsonGuide));
			}
			else
			{
				UE_LOGF(AvaViewportLog, Warning, "Motion Design Viewport Guide: SavePreset() - Unable to create guide json [%ls]", *InPath);
			}
		}

	 	TSharedRef<FJsonObject> RootObject = MakeShared<FJsonObject>();
		RootObject->SetArrayField(GuideListFieldName, JsonGuides);

		FString FileJsonString;
		const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&FileJsonString);

		if (!FJsonSerializer::Serialize(RootObject, Writer))
		{
			UE_LOGF(AvaViewportLog, Warning, "Motion Design Viewport Guide: SavePreset() - Unable to serialize [%ls]. Json=[%ls]", *InPath, *FileJsonString);
			return false;
		}

		if (!FFileHelper::SaveStringToFile(FileJsonString, *InPath))
		{
			UE_LOGF(AvaViewportLog, Warning, "Motion Design Viewport Guide: SavePreset() - Unable to save Json file: %ls", *InPath);
			return false;
		}

		return true;
	}
}

bool FAvaViewportGuidePresetProvider::SaveGuidePreset(const FString& InPresetName, const TArray<FAvaViewportGuideInfo>& InGuides,
	const FVector2f InViewportSize)
{
	using namespace UE::AvaViewport::Private;

	const FString Path = GetGuideConfigPath();

	if (Path.IsEmpty() || !FPaths::DirectoryExists(Path))
	{
		return false;
	}

	const FString PresetFile = Path / InPresetName + TEXT(".json");

	if (!SavePreset(PresetFile, InGuides, InViewportSize))
	{
		UE_LOGF(AvaViewportLog, Warning, "Motion Design Viewport Guide: SaveGuidePreset() - Failed [%ls]", *InPresetName);
		return false;
	}

	LastAccessedGuidePresetName = InPresetName;
	UE_LOGF(AvaViewportLog, Log, "Motion Design Viewport Guide: SaveGuidePreset() - Success [%ls]", *InPresetName);
	return true;
}

bool FAvaViewportGuidePresetProvider::LoadGuidePreset(const FString& InPresetName, TArray<FAvaViewportGuideInfo>& OutGuides, 
	const FVector2f InViewportSize)
{
	using namespace UE::AvaViewport::Private;

	const FString Path = GetGuideConfigPath();

	if (Path.IsEmpty() || !FPaths::DirectoryExists(Path))
	{
		return false;
	}

	const FString PresetFile = Path / InPresetName + TEXT(".json");

	TArray<FAvaViewportGuideInfo> LoadedGuides;

	if (!LoadPreset(PresetFile, LoadedGuides, InViewportSize))
	{
		UE_LOGF(AvaViewportLog, Warning, "Motion Design Viewport Guide: LoadGuidePreset() - Failed [%ls]", *InPresetName);
		return false;
	}

	OutGuides = MoveTemp(LoadedGuides);
	LastAccessedGuidePresetName = InPresetName;
	UE_LOGF(AvaViewportLog, Log, "Motion Design Viewport Guide: LoadGuidePreset() - Success [%ls]", *InPresetName);
	return true;
}

bool FAvaViewportGuidePresetProvider::RemoveGuidePreset(const FString& InPresetName)
{
	using namespace UE::AvaViewport::Private;

	const FString Path = GetGuideConfigPath(/* Allow Create */ false);

	if (Path.IsEmpty() || !FPaths::DirectoryExists(Path))
	{
		return false;
	}

	const FString PresetFile = Path / InPresetName + TEXT(".json");

	if (!FPaths::FileExists(PresetFile))
	{
		return false;
	}

	return IFileManager::Get().Delete(*PresetFile);
}

TArray<FString> FAvaViewportGuidePresetProvider::GetGuidePresetNames()
{
	using namespace UE::AvaViewport::Private;

	const FString Path = GetGuideConfigPath();

	if (Path.IsEmpty() || !FPaths::DirectoryExists(Path))
	{
		return {};
	}

	static const FVector2f One = FVector2f(1.f, 1.f);

	TArray<FString> JsonFiles;
	IFileManager::Get().FindFiles(JsonFiles, *Path, TEXT(".json"));

	TArray<FString> PresetNames;

	for (const FString& JsonFile : JsonFiles)
	{
		const FString JsonFilePath = Path / JsonFile;
		const FString PresetName = FPaths::GetBaseFilename(JsonFilePath, /* Remove Path */ true);

		TArray<FAvaViewportGuideInfo> PresetGuides;

		// Doesn't need actual viewport size, it's just a load check
		if (!LoadPreset(JsonFilePath, PresetGuides, One))
		{
			continue;
		}

		PresetNames.Add(PresetName);
	}

	return PresetNames;
}

FString FAvaViewportGuidePresetProvider::GetLastAccessedGuidePresetName() const
{
	return LastAccessedGuidePresetName;
}
