// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaViewportPostProcessManager.h"
#include "AvalancheViewportModule.h"
#include "AvaTypeSharedPointer.h"
#include "AvaViewportDataSubsystem.h"
#include "Interaction/AvaCameraZoomController.h"
#include "ViewportClient/IAvaViewportClient.h"
#include "Visualizers/AvaViewportBackgroundVisualizer.h"
#include "Visualizers/AvaViewportChannelVisualizer.h"
#include "Visualizers/AvaViewportCheckerboardVisualizer.h"

FAvaViewportPostProcessManager::FAvaViewportPostProcessManager(TSharedRef<IAvaViewportClient> InAvaViewportClient)
{
	AvaViewportClientWeak = InAvaViewportClient;

	Visualizers.Emplace(EAvaViewportPostProcessType::Background,   MakeShared<FAvaViewportBackgroundVisualizer>(InAvaViewportClient));
	Visualizers.Emplace(EAvaViewportPostProcessType::RedChannel,   MakeShared<FAvaViewportChannelVisualizer>(InAvaViewportClient, EAvaViewportPostProcessType::RedChannel));
	Visualizers.Emplace(EAvaViewportPostProcessType::GreenChannel, MakeShared<FAvaViewportChannelVisualizer>(InAvaViewportClient, EAvaViewportPostProcessType::GreenChannel));
	Visualizers.Emplace(EAvaViewportPostProcessType::BlueChannel,  MakeShared<FAvaViewportChannelVisualizer>(InAvaViewportClient, EAvaViewportPostProcessType::BlueChannel));
	Visualizers.Emplace(EAvaViewportPostProcessType::AlphaChannel, MakeShared<FAvaViewportChannelVisualizer>(InAvaViewportClient, EAvaViewportPostProcessType::AlphaChannel));
	Visualizers.Emplace(EAvaViewportPostProcessType::Checkerboard, MakeShared<FAvaViewportCheckerboardVisualizer>(InAvaViewportClient));
}

FAvaViewportPostProcessInfo* FAvaViewportPostProcessManager::GetPostProcessInfo() const
{
	TSharedPtr<IAvaViewportClient> AvaViewportClient = AvaViewportClientWeak.Pin();

	if (!AvaViewportClient.IsValid())
	{
		UE_LOGF(AvaViewportLog, Warning, "FAvaViewportPostProcessManager::GetPostProcessInfo: Invalid viewport client.");
		return nullptr;
	}

	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(AvaViewportClient->GetViewportWorld());

	if (!DataSubsystem)
	{
		UE_LOGF(AvaViewportLog, Warning, "FAvaViewportPostProcessManager::GetPostProcessInfo: Failed to find data subsystem.");
		return nullptr;
	}

	if (FAvaViewportData* Data = DataSubsystem->GetData())
	{
		return &Data->PostProcessInfo;
	}

	UE_LOGF(AvaViewportLog, Warning, "FAvaViewportPostProcessManager::GetPostProcessInfo: Missing viewport data.");
	return nullptr;
}

void FAvaViewportPostProcessManager::UpdateSceneView(FSceneView* InSceneView)
{
	TSharedPtr<IAvaViewportClient> AvaViewportClient = AvaViewportClientWeak.Pin();

	if (!AvaViewportClient.IsValid())
	{
		return;
	}

	TSharedPtr<FAvaViewportPostProcessVisualizer> Visualizer = UE::AvaCore::CastSharedPtr<FAvaViewportPostProcessVisualizer>(GetActiveVisualizer());

	if (!Visualizer.IsValid())
	{
		return;
	}

	FVector2f PanOffset = FVector2f::ZeroVector;

	if (TSharedPtr<FAvaCameraZoomController> ZoomController = AvaViewportClient->GetZoomController())
	{
		PanOffset = ZoomController->GetPanOffsetFraction() * AvaViewportClient->GetViewportSize() * -1.f;
	}

	Visualizer->UpdateForViewport(
		AvaViewportClient->GetZoomedVisibleArea(),
		AvaViewportClient->GetViewportOffset(),
		AvaViewportClient->GetViewportWidgetSize(),
		PanOffset
	);

	Visualizer->ApplyToSceneView(InSceneView);
}

void FAvaViewportPostProcessManager::LoadPostProcessInfo()
{
	TSharedPtr<FAvaViewportPostProcessVisualizer> Visualizer = UE::AvaCore::CastSharedPtr<FAvaViewportPostProcessVisualizer>(GetActiveVisualizer());

	if (!Visualizer.IsValid())
	{
		UE_LOGF(AvaViewportLog, Warning, "FAvaViewportPostProcessManager::LoadPostProcessInfo: Invalid visualizer.");
		return;
	}

	Visualizer->LoadPostProcessInfo();
}

EAvaViewportPostProcessType FAvaViewportPostProcessManager::GetType() const
{
	if (FAvaViewportPostProcessInfo* PostProcessInfo = GetPostProcessInfo())
	{
		return PostProcessInfo->Type;
	}

	return EAvaViewportPostProcessType::None;
}

void FAvaViewportPostProcessManager::SetType(EAvaViewportPostProcessType InType)
{
	FAvaViewportPostProcessInfo* PostProcessInfo = GetPostProcessInfo();

	if (!PostProcessInfo)
	{
		UE_LOGF(AvaViewportLog, Warning, "AvaViewportPostProcessManager::SetType: Invalid post process info.");
		return;
	}

	if (PostProcessInfo->Type == InType)
	{
		return;
	}

	TSharedPtr<FAvaViewportPostProcessVisualizer> NewVisualizer = UE::AvaCore::CastSharedPtr<FAvaViewportPostProcessVisualizer>(GetVisualizer(InType));

	if (NewVisualizer.IsValid() && !NewVisualizer->CanActivate(/* bInSilent */ false))
	{
		UE_LOGF(AvaViewportLog, Warning, "AvaViewportPostProcessManager::SetType: Cannot activate new visualizer.");
		return;
	}

	if (TSharedPtr<FAvaViewportPostProcessVisualizer> CurrentVisualizer = UE::AvaCore::CastSharedPtr<FAvaViewportPostProcessVisualizer>(GetActiveVisualizer()))
	{
		CurrentVisualizer->OnDeactivate();
	}

	ModifyDataSource();
	PostProcessInfo->Type = InType;

	if (NewVisualizer.IsValid())
	{
		NewVisualizer->OnActivate();
	}
}

float FAvaViewportPostProcessManager::GetOpacity()
{
	if (FAvaViewportPostProcessInfo* PostProcessInfo = GetPostProcessInfo())
	{
		return PostProcessInfo->Opacity;
	}

	UE_LOGF(AvaViewportLog, Warning, "FAvaViewportPostProcessManager::GetOpacity: Missing post process info.");
	return 1.f;
}

void FAvaViewportPostProcessManager::SetOpacity(float InOpacity)
{
	if (FAvaViewportPostProcessInfo* PostProcessInfo = GetPostProcessInfo())
	{
		ModifyDataSource();
		PostProcessInfo->Opacity = InOpacity;
		LoadPostProcessInfo();
	}
	else
	{
		UE_LOGF(AvaViewportLog, Warning, "FAvaViewportPostProcessManager::SetOpacity: Missing post process info.");
	}
}

void FAvaViewportPostProcessManager::ModifyDataSource()
{
	TSharedPtr<IAvaViewportClient> AvaViewportClient = AvaViewportClientWeak.Pin();

	if (!AvaViewportClient.IsValid())
	{
		UE_LOGF(AvaViewportLog, Warning, "FAvaViewportPostProcessManager::ModifyDataSource: Invalid viewport client.");
		return;
	}

	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(AvaViewportClient->GetViewportWorld());

	if (!DataSubsystem)
	{
		UE_LOGF(AvaViewportLog, Warning, "FAvaViewportPostProcessManager::ModifyDataSource: Failed to find data subsystem.");
		return;
	}

	DataSubsystem->ModifyDataSource();
}

TSharedPtr<IAvaViewportPostProcessVisualizer> FAvaViewportPostProcessManager::GetVisualizer(EAvaViewportPostProcessType InType) const
{
	if (const TSharedPtr<IAvaViewportPostProcessVisualizer>* VisualizerPtr = Visualizers.Find(InType))
	{
		return *VisualizerPtr;
	}

	if (InType != EAvaViewportPostProcessType::None)
	{
		UE_LOGF(AvaViewportLog, Warning, "FAvaViewportPostProcessManager::GetVisualizer: Missing visualizer.");
	}

	return nullptr;
}

TSharedPtr<IAvaViewportPostProcessVisualizer> FAvaViewportPostProcessManager::GetActiveVisualizer() const
{
	if (FAvaViewportPostProcessInfo* PostProcessInfo = GetPostProcessInfo())
	{
		return GetVisualizer(PostProcessInfo->Type);
	}

	return nullptr;
}
	