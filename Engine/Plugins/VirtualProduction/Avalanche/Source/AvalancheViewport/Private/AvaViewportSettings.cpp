// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaViewportSettings.h"

UAvaViewportSettings::UAvaViewportSettings()
{
	CategoryName = TEXT("Motion Design");
	SectionName = TEXT("Viewport");

	bEnableViewportOverlay = true;
	bEnableBoundingBoxes = true;
	ViewportBackgroundMaterial = TSoftObjectPtr<UMaterial>(FSoftObjectPath(TEXT("/Script/Engine.Material'/Avalanche/EditorResources/M_Backplate.M_Backplate'")));
	ViewportCheckerboardMaterial = TSoftObjectPtr<UMaterial>(FSoftObjectPath(TEXT("/Script/Engine.Material'/Avalanche/EditorResources/M_Checkerboard.M_Checkerboard'")));
	ViewportCheckerboardColor0 = FLinearColor(FVector3f(0.048172f));
	ViewportCheckerboardColor1 = FLinearColor(FVector3f(0.177888f));
	ViewportCheckerboardSize = 8.0f;

	bGridEnabled = false;
	bGridAlwaysVisible = false;
	GridSize = 50;
	GridColor = FLinearColor(0.6f, 0.0f, 0.0f, 0.4f);
	GridThickness = 1.f;

	bPixelGridEnabled = true;
	PixelGridColor = FLinearColor(0.5f, 0.5f, 0.5f, 0.4f);

	SnapState = static_cast<int32>(EAvaViewportSnapState::Off);
	bSnapIndicatorsEnabled = true;
	SnapIndicatorColor = FLinearColor::Red;
	SnapIndicatorThickness = 2.f;

	bGuidesEnabled = true;
	GuideConfigPath =  (FString(TEXT(".")) / FString(TEXT("Config")) / FString(TEXT("Guides"))).RightChop(1);
	EnabledGuideColor = FLinearColor(0.f, 1.f, 1.f, 1.f);
	EnabledLockedGuideColor = FLinearColor(0.f, 0.5f, 0.5f, 1.f);
	DisabledGuideColor = FLinearColor(1.f, 0.f, 0.f, 1.f);
	DisabledLockedGuideColor = FLinearColor(0.5f, 0.f, 0.f, 1.f);
	DraggedGuideColor = FLinearColor(1.f, 1.f, 0.f, 1.f);
	SnappedToGuideColor = FLinearColor(0.f, 1.f, 0.f, 1.f);
	GuideThickness = 1.f;

	bSafeFramesEnabled = false;

	bEnableShapesEditorOverlay = false;
	ShapeEditorOverlayType = EAvaShapeEditorOverlayType::ComponentVisualizerOnly;

	SafeFrames.SetNum(2);
	SafeFrames[0].ScreenPercentage = 80;
	SafeFrames[0].Color = FLinearColor::Green;
	SafeFrames[0].Color.A = 0.6f;
	SafeFrames[0].Thickness = 1.f;

	SafeFrames[1].ScreenPercentage = 90;
	SafeFrames[1].Color = FLinearColor::Green;
	SafeFrames[1].Color.A = 0.6f;
	SafeFrames[1].Thickness = 1.f;

	CameraBoundsShadeColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.8f);
}

EAvaViewportSnapState UAvaViewportSettings::GetSnapState() const
{
	return static_cast<EAvaViewportSnapState>(SnapState);
}

bool UAvaViewportSettings::HasSnapState(EAvaViewportSnapState InSnapState)
{
	return (static_cast<int32>(InSnapState) & SnapState) > 0;
}

void UAvaViewportSettings::SetSnapState(EAvaViewportSnapState InSnapState)
{
	SnapState = static_cast<int32>(InSnapState);
}

void UAvaViewportSettings::BroadcastSettingChanged(FName InSettingName)
{
	OnChange.Broadcast(this, InSettingName);
}

void UAvaViewportSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	if (MemberName != NAME_None)
	{
		BroadcastSettingChanged(MemberName);
	}
}
