// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaViewportUtils.h"
#include "Containers/Array.h"
#include "EditorViewportClient.h"
#include "SceneView.h"
#include "SEditorViewport.h"
#include "Templates/SharedPointer.h"
#include "ViewportClient.h"
#include "ViewportClient/EditorViewportClientUtilityWrapper.h"
#include "ViewportClient/IAvaViewportClient.h"

namespace UE::AvaViewport::Private
{
	TArray<FAvaViewportClientCastDelegate> ViewportClientCasterRegistry;
	TArray<FAvaViewportCastDelegate> ViewportCasterRegistry;
}

FDelegateHandle FAvaViewportUtils::RegisterViewportClientCaster(FAvaViewportClientCastDelegate::TFuncType InFunction)
{
	using namespace UE::AvaViewport::Private;

	FAvaViewportClientCastDelegate& NewDelegate = ViewportClientCasterRegistry.AddDefaulted_GetRef();
	NewDelegate.BindStatic(InFunction);

	return NewDelegate.GetHandle();
}

void FAvaViewportUtils::UnregisterViewportClientCaster(FDelegateHandle InDelegateHandle)
{
	using namespace UE::AvaViewport::Private;

	ViewportClientCasterRegistry.RemoveAll(
		[InDelegateHandle](const FAvaViewportClientCastDelegate& InElement)
		{
			return InElement.GetHandle() == InDelegateHandle;
		});
}

FDelegateHandle FAvaViewportUtils::RegisterViewportCaster(FAvaViewportCastDelegate::TFuncType InFunction)
{
	using namespace UE::AvaViewport::Private;

	FAvaViewportCastDelegate& NewDelegate = ViewportCasterRegistry.AddDefaulted_GetRef();
	NewDelegate.BindStatic(InFunction);

	return NewDelegate.GetHandle();
}

void FAvaViewportUtils::UnregisterViewportCaster(FDelegateHandle InDelegateHandle)
{
	using namespace UE::AvaViewport::Private;

	ViewportCasterRegistry.RemoveAll(
		[InDelegateHandle](const FAvaViewportCastDelegate& InElement)
		{
			return InElement.GetHandle() == InDelegateHandle;
		});
}

TSharedPtr<IAvaViewportClient> FAvaViewportUtils::GetAsAvaViewportClient(FEditorViewportClient* InViewportClient)
{
	if (!InViewportClient)
	{
		return nullptr;
	}

	using namespace UE::AvaViewport::Private;

	for (const FAvaViewportClientCastDelegate& CastDelegate : ViewportClientCasterRegistry)
	{
		if (CastDelegate.IsBound())
		{
			if (TSharedPtr<IAvaViewportClient> AvaViewportClient = CastDelegate.Execute(InViewportClient))
			{
				return AvaViewportClient;
			}
		}
	}

	if (FEditorViewportClientUtilityWrapper::IsValidLevelEditorViewportClient(InViewportClient))
	{
		if (TSharedPtr<SEditorViewport> ViewportWidget = InViewportClient->GetEditorViewportWidget())
		{
			if (TSharedPtr<FEditorViewportClient> ViewportClientShared = ViewportWidget->GetViewportClient())
			{
				return MakeShared<FEditorViewportClientUtilityWrapper>(ViewportClientShared);
			}
		}
	}

	return nullptr;
}

FEditorViewportClient* FAvaViewportUtils::GetAsEditorViewportClient(FViewport* InViewport)
{
	if (!InViewport)
	{
		return nullptr;
	}

	using namespace UE::AvaViewport::Private;

	for (const FAvaViewportCastDelegate& CastDelegate : ViewportCasterRegistry)
	{
		if (CastDelegate.IsBound())
		{
			if (FEditorViewportClient* EditorViewportClient = CastDelegate.Execute(InViewport))
			{
				return EditorViewportClient;
			}
		}
	}

	return FEditorViewportClientUtilityWrapper::GetValidLevelEditorViewportClient(InViewport);
}

TSharedPtr<IAvaViewportClient> FAvaViewportUtils::GetAvaViewportClient(FViewport* InViewport)
{
	if (FEditorViewportClient* EditorViewportClient = GetAsEditorViewportClient(InViewport))
	{
		return GetAsAvaViewportClient(EditorViewportClient);
	}

	return nullptr;
}

bool FAvaViewportUtils::IsValidViewportSize(const FVector2f& InViewportSize)
{
	return InViewportSize.X > 0 && InViewportSize.Y > 0
		&& !FMath::IsNearlyZero(InViewportSize.X) && !FMath::IsNearlyZero(InViewportSize.Y)
#if ENABLE_NAN_DIAGNOSTIC
		&& !FMath::IsNaN(InViewportSize.X) && !FMath::IsNaN(InViewportSize.Y)
#endif
	;
}

bool FAvaViewportUtils::IsValidViewportSize(const FIntPoint& InViewportSize)
{
	return InViewportSize.X > 0 && InViewportSize.Y > 0;
}

float FAvaViewportUtils::CalcFOV(float InViewportDimension, float InDistance)
{
	if (InViewportDimension < 0 || FMath::IsNearlyZero(InViewportDimension))
	{
		return 0.f;
	}

	return FMath::RadiansToDegrees(2.f * FMath::Atan(InViewportDimension * 0.5f / InDistance));
}

FTransform FAvaViewportUtils::GetViewportViewTransform(FEditorViewportClient* InViewportClient)
{
	if (!InViewportClient)
	{
		return FTransform::Identity;
	}

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(InViewportClient->Viewport, InViewportClient->GetScene(),
		InViewportClient->EngineShowFlags)
		.SetRealtimeUpdate(InViewportClient->IsRealtime()));

	FSceneView* const View = InViewportClient->CalcSceneView(&ViewFamily);

	if (!View)
	{
		return FTransform::Identity;
	}

	return FTransform(
		View->ViewRotation,
		View->ViewLocation,
		FVector::OneVector
	);
}

FVector2D FAvaViewportUtils::GetFrustumSizeAtDistance(float InHorizontalFoVDegrees, float InAspectRatio, double InDistance)
{
	const double HozHalfAngleInRadians = FMath::DegreesToRadians(static_cast<double>(InHorizontalFoVDegrees) * 0.5);
	const double Width = InDistance * FMath::Tan(HozHalfAngleInRadians) * 2.0;
	const double Height = Width / static_cast<double>(InAspectRatio);

	return {Width, Height};
}

FVector FAvaViewportUtils::ViewportPositionToWorldPositionPerspective(const FVector2f& InViewportSize,
	const FVector2f& InViewportPosition, const FVector& InViewLocation, const FRotator& InViewRotation, float InHorizontalFoVDegrees,
	double InDistance)
{
	const FVector2D FrustrumSize = FAvaViewportUtils::GetFrustumSizeAtDistance(InHorizontalFoVDegrees, InViewportSize.X / InViewportSize.Y, InDistance);

	const FVector WorldOffset = {
		InDistance,
		FMath::GetMappedRangeValueUnclamped<double>(
			FVector2D(0, InViewportSize.X),
			{FrustrumSize.X * -0.5, FrustrumSize.X * 0.5},
			InViewportPosition.X
		),
		// Vertical axis is up in the world, but down in the viewport
		FMath::GetMappedRangeValueUnclamped<double>(
			FVector2D(0, InViewportSize.Y),
			{FrustrumSize.Y * 0.5, FrustrumSize.Y * -0.5},
			InViewportPosition.Y
		)
	};

	return InViewLocation + InViewRotation.RotateVector(WorldOffset);
}

FVector FAvaViewportUtils::ViewportPositionToWorldPositionOrthographic(const FVector2f& InViewportSize,
	const FVector2f& InViewportPosition, const FVector& InViewLocation, const FRotator& InViewRotation, float InOrthoWidth,
	double InDistance)
{
	const FVector2f OrthoSize = FVector2f(InOrthoWidth, InOrthoWidth * InViewportSize.Y / InViewportSize.X);

	const FVector WorldOffset = {
		InDistance,
		FMath::GetMappedRangeValueUnclamped<double>(
			FVector2D(0, InViewportSize.X),
			{OrthoSize.X * -0.5, OrthoSize.X * 0.5},
			InViewportPosition.X
		),
		// Vertical axis is up in the world, but down in the viewport
		FMath::GetMappedRangeValueUnclamped<double>(
			FVector2D(0, InViewportSize.Y),
			{OrthoSize.Y * 0.5, OrthoSize.Y * -0.5},
			InViewportPosition.Y
		)
	};

	return InViewLocation + InViewRotation.RotateVector(WorldOffset);
}

void FAvaViewportUtils::WorldPositionToViewportPositionPerspective(const FVector2f& InViewportSize,
	const FVector& InWorldPosition, const FVector& InViewLocation, const FRotator& InViewRotation, float InHorizontalFoVDegrees,
	FVector2f& OutViewportPosition, double& OutDistance)
{
	const FTransform ViewTransform(InViewRotation, InViewLocation);
	const FVector TransformedPosition = ViewTransform.InverseTransformPositionNoScale(InWorldPosition);
	const FVector2D FrustumSize = FAvaViewportUtils::GetFrustumSizeAtDistance(InHorizontalFoVDegrees, InViewportSize.X / InViewportSize.Y, TransformedPosition.X);

	OutViewportPosition.X = FMath::GetMappedRangeValueUnclamped<float>(
		FVector2f(-FrustumSize.X * 0.5, FrustumSize.X * 0.5),
		{0.f, InViewportSize.X},
		TransformedPosition.Y
	);

	OutViewportPosition.Y = FMath::GetMappedRangeValueUnclamped<float>(
		FVector2f(FrustumSize.Y * 0.5, -FrustumSize.Y * 0.5),
		{0.f, InViewportSize.Y},
		TransformedPosition.Z
	);

	OutDistance = TransformedPosition.X;
}

void FAvaViewportUtils::WorldPositionToViewportPositionOrthographic(const FVector2f& InViewportSize,
	const FVector& InWorldPosition, const FVector& InViewLocation, const FRotator& InViewRotation, float InOrthoWidth,
	FVector2f& OutViewportPosition, double& OutDistance)
{
	const FTransform ViewTransform(InViewRotation, InViewLocation);
	const FVector TransformedPosition = ViewTransform.InverseTransformPositionNoScale(InWorldPosition);
	const FVector2f OrthoSize = {InOrthoWidth, InOrthoWidth / InViewportSize.X * InViewportSize.Y};

	OutViewportPosition.X = FMath::GetMappedRangeValueUnclamped<float>(
		FVector2f(-OrthoSize.X * 0.5, OrthoSize.X * 0.5),
		{0.f, InViewportSize.X},
		TransformedPosition.Y
	);

	OutViewportPosition.Y = FMath::GetMappedRangeValueUnclamped<float>(
		FVector2f(OrthoSize.Y * 0.5, -OrthoSize.Y * 0.5),
		{0.f, InViewportSize.Y},
		TransformedPosition.Z
	);

	OutDistance = TransformedPosition.X;
}
