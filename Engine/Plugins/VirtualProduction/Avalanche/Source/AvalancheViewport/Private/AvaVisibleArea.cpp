// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaVisibleArea.h"
#include "AvaViewportUtils.h"

FAvaVisibleArea::FAvaVisibleArea()
	: FAvaVisibleArea(FVector2f::ZeroVector, FVector2f::ZeroVector, FVector2f::ZeroVector, 1.f)
{
}

FAvaVisibleArea::FAvaVisibleArea(const FVector2f& InAbsoluteSize)
	: FAvaVisibleArea(FVector2f::ZeroVector, InAbsoluteSize, InAbsoluteSize, 1.f)
{
}

FAvaVisibleArea::FAvaVisibleArea(const FVector2f& InAbsoluteSize, float InDPIScale)
	: FAvaVisibleArea(FVector2f::ZeroVector, InAbsoluteSize, InAbsoluteSize, InDPIScale)
{
}

FAvaVisibleArea::FAvaVisibleArea(const FVector2f& InOffset, const FVector2f& InVisibleSize,	const FVector2f& InAbsoluteSize, float InDPIScale)
	: Offset(InOffset)
	, VisibleSize(InVisibleSize)
	, AbsoluteSize(InAbsoluteSize)
	, DPIScale(InDPIScale)
{
}

bool FAvaVisibleArea::IsValid() const
{
	return FAvaViewportUtils::IsValidViewportSize(AbsoluteSize)
		&& FAvaViewportUtils::IsValidViewportSize(VisibleSize)
#if ENABLE_NAN_DIAGNOSTIC
		&& !FMath::IsNaN(Offset.X) && !FMath::IsNaN(Offset.Y)
#endif
		&& DPIScale > 0 && !FMath::IsNearlyZero(DPIScale)
#if ENABLE_NAN_DIAGNOSTIC
		&& !FMath::IsNaN(DPIScale)
#endif
		;
}

bool FAvaVisibleArea::IsAbsoluteView() const
{
	return IsAbsoluteSize() && IsCentered();
}

bool FAvaVisibleArea::IsModifiedView() const
{
	return !IsAbsoluteView();
}

bool FAvaVisibleArea::IsAbsoluteSize() const
{
	return FMath::IsNearlyEqual(VisibleSize.X, AbsoluteSize.X)
		&& FMath::IsNearlyEqual(VisibleSize.Y, AbsoluteSize.Y);
}

bool FAvaVisibleArea::IsZoomedView() const
{
	return !IsAbsoluteSize();
}

bool FAvaVisibleArea::IsCentered() const
{
	return FMath::IsNearlyZero(Offset.X)
		&& FMath::IsNearlyZero(Offset.Y);
}

bool FAvaVisibleArea::IsOffset() const
{
	return !IsCentered();
}

float FAvaVisibleArea::GetVisibleAreaFraction() const
{
	if (FMath::IsNearlyZero(AbsoluteSize.X))
	{
		return 1.f;
	}

	return VisibleSize.X / AbsoluteSize.X;
}

float FAvaVisibleArea::GetInvisibleAreaFraction() const
{
	if (FMath::IsNearlyZero(AbsoluteSize.X))
	{
		return 0.f;
	}

	return 1.f - (VisibleSize.X / AbsoluteSize.X);
}

FVector2f FAvaVisibleArea::GetVisiblePosition(const FVector2f& AbsolutePosition) const
{
	if (IsAbsoluteView())
	{
		return AbsolutePosition;
	}

	FVector2f VisiblePosition = AbsolutePosition;
	VisiblePosition -= Offset;
	VisiblePosition /= GetVisibleAreaFraction();

	return VisiblePosition;
}

FVector2f FAvaVisibleArea::GetDPIScaledVisiblePosition(const FVector2f& AbsolutePosition) const
{
	if (IsAbsoluteView())
	{
		return AbsolutePosition * DPIScale;
	}

	FVector2f VisiblePosition = AbsolutePosition;
	VisiblePosition -= Offset / DPIScale;
	VisiblePosition /= GetVisibleAreaFraction();

	return VisiblePosition * DPIScale;
}

FVector2f FAvaVisibleArea::GetAbsolutePosition(const FVector2f& VisiblePosition) const
{
	if (IsAbsoluteView())
	{
		return VisiblePosition;
	}

	FVector2f AbsolutePosition = VisiblePosition;
	AbsolutePosition *= GetVisibleAreaFraction();
	AbsolutePosition += Offset;

	return AbsolutePosition;
}

FVector2f FAvaVisibleArea::GetDPIScaledAbsolutePosition(const FVector2f& VisiblePosition) const
{
	if (IsAbsoluteView())
	{
		return VisiblePosition / DPIScale;
	}

	FVector2f AbsolutePosition = VisiblePosition;
	AbsolutePosition *= GetVisibleAreaFraction();
	AbsolutePosition += Offset * DPIScale;

	return AbsolutePosition / DPIScale;
}

FVector2f FAvaVisibleArea::GetVisibleAreaCenter() const
{
	return VisibleSize * 0.5f;
}

FVector2f FAvaVisibleArea::GetAbsoluteVisibleAreaCenter() const
{
	return Offset + VisibleSize * 0.5f;
}

FVector2f FAvaVisibleArea::GetAbsoluteAreaCenter() const
{
	return AbsoluteSize * 0.5f;
}

FVector2f FAvaVisibleArea::GetInvisibleSize() const
{
	return AbsoluteSize - VisibleSize;
}
