// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvalancheViewportModule.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(AvaViewportLog);

IMPLEMENT_MODULE(FDefaultModuleImpl, AvalancheViewport)