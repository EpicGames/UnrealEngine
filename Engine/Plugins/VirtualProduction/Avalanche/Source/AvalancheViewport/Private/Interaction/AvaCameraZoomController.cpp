// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interaction/AvaCameraZoomController.h"
#include "AvaActorUtils.h"
#include "AvaViewportUtils.h"
#include "AvaVisibleArea.h"
#include "Camera/CameraComponent.h"
#include "Editor/EditorEngine.h"
#include "EditorViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "ViewportClient/IAvaViewportClient.h"

namespace UE::AvaViewport::Private
{
	const TArray<float> ZoomLevels = {
		0.65f,
		0.7f,
		0.75f,
		0.8f,
		0.85f,
		0.9f,
		0.95f,
		1.0f,
		1.25f,
		1.5f,
		2.f,
		2.5f,
		3.f,
		4.f,
		5.f,
		7.5f,
		10.f,
		12.5f,
		15.f,
		17.5f,
		20.f
	};

	constexpr uint8 DefaultZoomLevel = 7;

	/** Returns the FOV angle at the given zoom level. Degrees. */
	float GetFOVForZoomLevel(float InDefaultFOV, uint8 InZoomLevel)
	{
		return 2.f * FMath::RadiansToDegrees(FMath::Atan(
			FMath::Tan(FMath::DegreesToRadians(InDefaultFOV * 0.5f)) / ZoomLevels[InZoomLevel]
		));
	}

	/**
	 * Returns the highest zoom level (lowest FOV) that is >= the given FOV.
	 * Will return ZoomLevel 0 if there isn't a Zoom FOV high enough.
	 */
	uint8 GetZoomLevelForFOV(float InDefaultFOV, float InFOV)
	{
		for (uint8 ZoomLevel = 1; ZoomLevel < ZoomLevels.Num(); ++ZoomLevel)
		{
			const float ZoomFOV = GetFOVForZoomLevel(InDefaultFOV, ZoomLevel);

			if (ZoomFOV < InFOV)
			{
				return ZoomLevel - 1;
			}
		}

		return ZoomLevels.Num() - 1;
	}
}

bool FAvaCameraZoomController::IsCameraZoomPossible()
{
	return !GEditor->PlayWorld;
}

FAvaCameraZoomController::FAvaCameraZoomController(TSharedRef<IAvaViewportClient> InAvaViewportClient, float InFallbackFOV)
	: FallbackFOV(InFallbackFOV)
{
	AvaViewportClientWeak = InAvaViewportClient;
	bIsPanning = false;
	CachedVisibleArea = FAvaVisibleArea();
	CachedZoomedVisibleArea = FAvaVisibleArea();
	Reset();
}

bool FAvaCameraZoomController::CanZoom() const
{
	return IsCameraZoomPossible();
}

void FAvaCameraZoomController::SetZoomLevel(uint8 InZoomLevel)
{
	ZoomLevel = FMath::Clamp(InZoomLevel, 0, UE::AvaViewport::Private::ZoomLevels.Num() - 1);

	UpdateVisibleAreas();
	InvalidateViewport();
}

bool FAvaCameraZoomController::IsZoomed() const
{
	return ZoomLevel != UE::AvaViewport::Private::DefaultZoomLevel;
}

void FAvaCameraZoomController::ZoomIn()
{
	if (!CachedZoomedVisibleArea.IsValid())
	{
		return;
	}

	ZoomInAroundPoint(CachedZoomedVisibleArea.GetAbsoluteVisibleAreaCenter());
}

void FAvaCameraZoomController::ZoomInCursor()
{
	TSharedPtr<IAvaViewportClient> AvaViewportClient = AvaViewportClientWeak.Pin();

	if (!AvaViewportClient.IsValid())
	{
		return;
	}

	ZoomInRelativePoint(AvaViewportClient->GetConstrainedViewportMousePosition());
}

void FAvaCameraZoomController::ZoomInAroundPoint(const FVector2f& InScreenPosition)
{
	ZoomIn_Internal();
	CenterOnPoint(InScreenPosition);
}

void FAvaCameraZoomController::ZoomInRelativePoint(const FVector2f& InViewportPosition)
{
	if (!CachedZoomedVisibleArea.IsValid())
	{
		return;
	}

	// Save old values
	const FAvaVisibleArea OldVisibleArea = CachedZoomedVisibleArea;

	if (!OldVisibleArea.IsValid())
	{
		ZoomInAroundPoint(OldVisibleArea.AbsoluteSize / 2.f);
		return;
	}

	const FVector2f OriginalAbsoluteViewportPosition = OldVisibleArea.GetDPIScaledAbsolutePosition(InViewportPosition);

	ZoomIn_Internal();

	const FAvaVisibleArea NewVisibleArea = CachedZoomedVisibleArea;
	const FVector2f NewAbsoluteViewportPosition = NewVisibleArea.GetDPIScaledAbsolutePosition(InViewportPosition);

	const FVector2f OffsetChange = (OriginalAbsoluteViewportPosition - NewAbsoluteViewportPosition) / NewVisibleArea.AbsoluteSize;

	SetPanOffsetFraction(PanOffsetFraction + OffsetChange);
}

void FAvaCameraZoomController::ZoomOut()
{
	if (!CachedZoomedVisibleArea.IsValid())
	{
		return;
	}

	ZoomOutAroundPoint(CachedZoomedVisibleArea.GetAbsoluteVisibleAreaCenter());
}

void FAvaCameraZoomController::ZoomOutCursor()
{
	TSharedPtr<IAvaViewportClient> AvaViewportClient = AvaViewportClientWeak.Pin();

	if (!AvaViewportClient.IsValid())
	{
		return;
	}

	ZoomOutRelativePoint(AvaViewportClient->GetConstrainedViewportMousePosition());
}

void FAvaCameraZoomController::ZoomOutAroundPoint(const FVector2f& InScreenPosition)
{
	ZoomOut_Internal();
	CenterOnPoint(InScreenPosition);
}

void FAvaCameraZoomController::ZoomOutRelativePoint(const FVector2f& InViewportPosition)
{
	if (!CachedZoomedVisibleArea.IsValid())
	{
		return;
	}

	// Save old values
	const FAvaVisibleArea OldVisibleArea = CachedZoomedVisibleArea;

	if (!OldVisibleArea.IsValid())
	{
		ZoomOutAroundPoint(OldVisibleArea.AbsoluteSize / 2.f);
		return;
	}

	const FVector2f OriginalAbsoluteViewportPosition = OldVisibleArea.GetDPIScaledAbsolutePosition(InViewportPosition);

	ZoomOut_Internal();

	const FAvaVisibleArea NewVisibleArea = CachedZoomedVisibleArea;
	const FVector2f NewAbsoluteViewportPosition = NewVisibleArea.GetDPIScaledAbsolutePosition(InViewportPosition);

	const FVector2f OffsetChange = (OriginalAbsoluteViewportPosition - NewAbsoluteViewportPosition) / NewVisibleArea.AbsoluteSize;

	SetPanOffsetFraction(PanOffsetFraction + OffsetChange);
}

void FAvaCameraZoomController::PanLeft()
{
	PanAdjustZoomed(FVector2f(-0.1f, 0.f));
}

void FAvaCameraZoomController::PanRight()
{
	PanAdjustZoomed(FVector2f(0.1f, 0.f));
}

void FAvaCameraZoomController::PanUp()
{
	PanAdjustZoomed(FVector2f(0.f, -0.1f));
}

void FAvaCameraZoomController::PanDown()
{
	PanAdjustZoomed(FVector2f(0.f, 0.1f));
}

void FAvaCameraZoomController::FrameActor()
{
	// TODO @Implement
}

void FAvaCameraZoomController::Reset()
{
	ZoomLevel = UE::AvaViewport::Private::DefaultZoomLevel;
	PanOffsetFraction = FVector2f::ZeroVector;

	InvalidateViewport();
}

void FAvaCameraZoomController::SetPanOffsetFraction(const FVector2f& InOffsetFraction)
{
	PanOffsetFraction.X = FMath::Clamp(InOffsetFraction.X, -1.f, 1.f);
	PanOffsetFraction.Y = FMath::Clamp(InOffsetFraction.Y, -1.f, 1.f);

	UpdateVisibleAreas();
	InvalidateViewport();
}

void FAvaCameraZoomController::PanAdjust(const FVector2f& InDirection)
{
	SetPanOffsetFraction(PanOffsetFraction + InDirection);
}

void FAvaCameraZoomController::PanAdjustZoomed(const FVector2f& InZoomedDirection)
{
	if (!CachedZoomedVisibleArea.IsValid())
	{
		return;
	}

	PanAdjust(InZoomedDirection / CachedZoomedVisibleArea.AbsoluteSize * CachedZoomedVisibleArea.GetVisibleAreaFraction());
}

void FAvaCameraZoomController::CenterOnPoint(const FVector2f& InPoint)
{
	if (!CachedZoomedVisibleArea.IsValid())
	{
		return;
	}

	const FVector2f OffsetFractionSize = CachedZoomedVisibleArea.AbsoluteSize;

	if (FMath::IsNearlyZero(OffsetFractionSize.X) || FMath::IsNearlyZero(OffsetFractionSize.Y))
	{
		return;
	}

	const FVector2f VisibleAreaPosition = InPoint - OffsetFractionSize / 2.f;

	SetPanOffsetFraction(VisibleAreaPosition / OffsetFractionSize);
}

void FAvaCameraZoomController::CenterOnBox(const FBox& InBoundingBox, const FTransform& InBoxTransform)
{
	// @TODO Update
#if 0

	TSharedPtr<IAvaViewportClient> AvaViewportClient = AvaViewportClientWeak.Pin();

	if (!AvaViewportClient.IsValid())
	{
		return;
	}

	FAvaVisibleArea VisibleArea = AvaViewportClient->GetZoomedVisibleArea();

	if (!VisibleArea.IsValid())
	{
		return;
	}

	const float AspectRatio = VisibleArea.AbsoluteSize.X / VisibleArea.AbsoluteSize.Y;

	const FOrientedBox OrientedBounds = FAvaActorUtils::MakeOrientedBox(InBoundingBox, InBoxTransform);

	FVector BoundsVertices[8];
	OrientedBounds.CalcVertices(BoundsVertices);

	double CameraDistance = TNumericLimits<float>::Max();
	TArray<FVector2f> ScreenPoints;
	ScreenPoints.SetNum(8);
	double Distance;

	for (int32 PointIdx = 0; PointIdx < 8; ++PointIdx)
	{
		AvaViewportClient->WorldPositionToViewportPosition(BoundsVertices[PointIdx], ScreenPoints[PointIdx], Distance);
		CameraDistance = FMath::Min(CameraDistance, Distance);
	}

	const FVector2f& CachedViewportSize = AvaViewportClient->GetViewportSize();

	FBox2f ScreenBoundingBox = FBox2f(ScreenPoints);
	ScreenBoundingBox.Min.X = FMath::Clamp(ScreenBoundingBox.Min.X - 50.f, 0.0, CachedViewportSize.X);
	ScreenBoundingBox.Min.Y = FMath::Clamp(ScreenBoundingBox.Min.Y - 50.f, 0.0, CachedViewportSize.Y);
	ScreenBoundingBox.Max.X = FMath::Clamp(ScreenBoundingBox.Max.X + 50.f, 0.0, CachedViewportSize.X);
	ScreenBoundingBox.Max.Y = FMath::Clamp(ScreenBoundingBox.Max.Y + 50.f, 0.0, CachedViewportSize.Y);

	FVector2f ScreenBoundsSize = ScreenBoundingBox.GetSize();
	const FVector2f ScreenBoundsCenter = ScreenBoundingBox.GetCenter();

	// Adjust screen bounds size so it's in the same space as the camera distance.
	const FVector2D DistanceFrustumSize = FAvaViewportUtils::GetFrustumSizeAtDistance(GetFOV(), AspectRatio, CameraDistance);
	ScreenBoundsSize *= DistanceFrustumSize.X / CachedViewportSize.X;

	const double DefaultFOV = GetDefaultFOV();
	double RequiredHorizontalFOV = FAvaViewportUtils::CalcFOV(ScreenBoundsSize.X, CameraDistance);
	double RequiredVerticalFOV = FAvaViewportUtils::CalcFOV(ScreenBoundsSize.Y, CameraDistance);

	// Convert vertical fov to horizontal fov
	RequiredVerticalFOV = FMath::RadiansToDegrees(2.f * FMath::Atan(FMath::Tan(FMath::DegreesToRadians(RequiredVerticalFOV) / 2.f) * AspectRatio));

	const uint8 RequiredZoomLevel = UE::AvaViewport::Private::GetZoomLevelForFOV(
		DefaultFOV,
		FMath::Max(RequiredHorizontalFOV, RequiredVerticalFOV)
	);

	if (RequiredZoomLevel == UE::AvaViewport::Private::DefaultZoomLevel)
	{
		Reset();
		return;
	}

	// Fake the zoom level being 1 zoom level out and then zoom in.
	SetZoomLevel(FMath::Max<uint8>(RequiredZoomLevel, 1) - 1);
	ZoomIn();
	CenterOnPoint(ScreenBoundsCenter);
#endif
}

void FAvaCameraZoomController::StartPanning()
{
	bIsPanning = true;
}

void FAvaCameraZoomController::EndPanning()
{
	bIsPanning = false;
}

void FAvaCameraZoomController::ZoomIn_Internal()
{
	if (ZoomLevel >= (UE::AvaViewport::Private::ZoomLevels.Num() - 1))
	{
		return;
	}

	SetZoomLevel(ZoomLevel + 1);
}

void FAvaCameraZoomController::ZoomOut_Internal()
{
	if (ZoomLevel <= 0)
	{
		return;
	}

	SetZoomLevel(ZoomLevel - 1);
}

void FAvaCameraZoomController::InvalidateViewport()
{
	TSharedPtr<IAvaViewportClient> AvaViewportClient = AvaViewportClientWeak.Pin();

	if (!AvaViewportClient.IsValid())
	{
		return;
	}

	const FEditorViewportClient* EditorViewportClient = AvaViewportClient->AsEditorViewportClient();

	if (!EditorViewportClient)
	{
		return;
	}

	const_cast<FEditorViewportClient*>(EditorViewportClient)->Invalidate();
}

float FAvaCameraZoomController::GetDefaultFOV() const
{
	if (TSharedPtr<IAvaViewportClient> AvaViewportClient = AvaViewportClientWeak.Pin())
	{
		if (UCameraComponent* CameraComponent = AvaViewportClient->GetCameraComponentViewTarget())
		{
			return CameraComponent->FieldOfView;
		}

		if (const FEditorViewportClient* ViewportClient = AvaViewportClient->AsEditorViewportClient())
		{
			return ViewportClient->FOVAngle;
		}
	}

	return FallbackFOV;
}

float FAvaCameraZoomController::GetFOV() const
{
	const float DefaultFOV = GetDefaultFOV();

	if (!CanZoom())
	{
		return DefaultFOV;
	}

	if (ZoomLevel == UE::AvaViewport::Private::DefaultZoomLevel)
	{
		return DefaultFOV;
	}

	return UE::AvaViewport::Private::GetFOVForZoomLevel(DefaultFOV, ZoomLevel);
}

FVector2f FAvaCameraZoomController::GetCameraProjectionOffset() const
{
	if (!CachedZoomedVisibleArea.IsValid() || !CanZoom())
	{
		return FVector2f::ZeroVector;
	}

	TSharedPtr<IAvaViewportClient> AvaViewportClient = AvaViewportClientWeak.Pin();

	if (!AvaViewportClient.IsValid())
	{
		return FVector2f::ZeroVector;
	}

	FVector2f CameraPanUVLocal = PanOffsetFraction;
	CameraPanUVLocal.Y *= -1.0;

	return CameraPanUVLocal * 2.f
		* CachedZoomedVisibleArea.AbsoluteSize
		/ AvaViewportClient->GetViewportWidgetSize()
		/ CachedZoomedVisibleArea.GetVisibleAreaFraction();
}

const FAvaVisibleArea& FAvaCameraZoomController::GetCachedVisibleArea() const
{
	return CachedVisibleArea;
}

const FAvaVisibleArea& FAvaCameraZoomController::GetCachedZoomedVisibleArea() const
{
	if (!CanZoom())
	{
		return CachedVisibleArea;
	}

	return CachedZoomedVisibleArea;
}

void FAvaCameraZoomController::UpdateVisibleAreas()
{
	TSharedPtr<IAvaViewportClient> AvaViewportClient = AvaViewportClientWeak.Pin();

	if (!AvaViewportClient.IsValid())
	{
		return;
	}

	const FVector2f CachedViewportSize = AvaViewportClient->GetViewportSize();

	if (FMath::IsNearlyZero(CachedViewportSize.X) || FMath::IsNearlyZero(CachedViewportSize.Y))
	{
		CachedVisibleArea = FAvaVisibleArea(CachedViewportSize);
		CachedZoomedVisibleArea = CachedVisibleArea;
	}

	CachedVisibleArea.VisibleSize = CachedViewportSize;
	CachedVisibleArea.AbsoluteSize = CachedViewportSize;
	CachedVisibleArea.DPIScale = AvaViewportClient->GetViewportDPIScale();

	const float VisibleScale = FMath::Tan(FMath::DegreesToRadians(GetFOV() / 2.0))
		/ FMath::Tan(FMath::DegreesToRadians(GetDefaultFOV() / 2.0));

	const FVector2f VisibleSize = CachedViewportSize * VisibleScale;

	CachedZoomedVisibleArea = CachedVisibleArea;
	CachedZoomedVisibleArea.VisibleSize = VisibleSize;

	const FVector2f Center = CachedVisibleArea.AbsoluteSize * (FVector2f(0.5f, 0.5f) + PanOffsetFraction);
	CachedZoomedVisibleArea.Offset = Center - (VisibleSize * 0.5f);
}

