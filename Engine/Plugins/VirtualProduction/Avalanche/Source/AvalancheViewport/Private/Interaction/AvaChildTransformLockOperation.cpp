// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interaction/AvaChildTransformLockOperation.h"
#include "Components/SceneComponent.h"
#include "CoreGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorViewportClient.h"
#include "GameFramework/Actor.h"
#include "Selection/AvaSelectionProviderSubsystem.h"
#include "ViewportClient/IAvaViewportClient.h"

FAvaChildTransformLockOperation::FAvaChildTransformLockOperation(TSharedRef<IAvaViewportClient> InAvaViewportClient)
{
	AvaViewportClientWeak = InAvaViewportClient;
	bAllowModify = true;
	Save();
}

void FAvaChildTransformLockOperation::Save()
{
	TSharedPtr<IAvaViewportClient> AvaViewportClient = AvaViewportClientWeak.Pin();

	if (!ensureMsgf(AvaViewportClient.IsValid(), TEXT("Invalid ava viewport client.")))
	{
		return;
	}

	const FEditorViewportClient* EditorViewportClient = AvaViewportClient->AsEditorViewportClient();

	if (!ensureMsgf(EditorViewportClient, TEXT("Invalid editor viewport client.")))
	{
		return;
	}

	UAvaSelectionProviderSubsystem* SelectionProvider = UAvaSelectionProviderSubsystem::Get(EditorViewportClient, /* bInGenerateErrors */ true);

	if (!SelectionProvider)
	{
		return;
	}

	TConstArrayView<TWeakObjectPtr<AActor>> CachedSelectedActors = SelectionProvider->GetSelectedActors();
	ChildActorTransforms.Reserve(CachedSelectedActors.Num());

	TSet<AActor*> InvalidChildren;
	InvalidChildren.Reserve(CachedSelectedActors.Num());

	for (const TWeakObjectPtr<AActor>& ActorWeak : CachedSelectedActors)
	{
		if (AActor* Actor = ActorWeak.Get())
		{
			InvalidChildren.Add(Actor);
		}
	}

	for (const TWeakObjectPtr<AActor>& ActorWeak : CachedSelectedActors)
	{
		if (AActor* Actor = ActorWeak.Get())
		{
			TConstArrayView<TWeakObjectPtr<AActor>> ChildActors = SelectionProvider->GetAttachedActors(Actor, /* Recursive */ false);

			for (const TWeakObjectPtr<AActor> ChildActorWeak : ChildActors)
			{
				if (AActor* ChildActor = ChildActorWeak.Get())
				{
					if (USceneComponent* RootComponent = ChildActor->GetRootComponent())
					{
						if (!InvalidChildren.Contains(ChildActor))
						{
							ChildActorTransforms.Emplace(ChildActor, RootComponent->GetComponentTransform());
						}
					}
				}
			}
		}
	}

	// Attempt to call modify now - possibility that it won't work!
	AttemptModifyOnLockedActors();
}

void FAvaChildTransformLockOperation::Restore()
{
	for (const TPair<TWeakObjectPtr<AActor>, FTransform>& Pair : ChildActorTransforms)
	{
		if (AActor* Actor = Pair.Key.Get())
		{
			if (USceneComponent* RootComponent = Actor->GetRootComponent())
			{
				RootComponent->SetWorldTransform(Pair.Value);
			}
		}
	}

	// Attempt to modify after the restore so that we save the original state.
	AttemptModifyOnLockedActors();
}

void FAvaChildTransformLockOperation::AttemptModifyOnLockedActors()
{
	if (bAllowModify && GUndo)
	{
		for (const TPair<TWeakObjectPtr<AActor>, FTransform>& Pair : ChildActorTransforms)
		{
			if (AActor* Actor = Pair.Key.Get())
			{
				Actor->Modify();

				if (USceneComponent* RootComponent = Actor->GetRootComponent())
				{
					RootComponent->Modify();
				}
			}
		}

		// If delta modification is disabled, block this from running again.
		if (!GEditor->IsDeltaModificationEnabled())
		{
			bAllowModify = false;
		}
	}
}
