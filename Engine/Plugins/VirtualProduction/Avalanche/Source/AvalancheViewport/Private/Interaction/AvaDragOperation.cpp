// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interaction/AvaDragOperation.h"
#include "Components/PrimitiveComponent.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "GameFramework/Actor.h"
#include "Interaction/AvaSnapOperation.h"
#include "Selection/AvaSelectionProviderSubsystem.h"
#include "ViewportClient/IAvaViewportClient.h"

FAvaDragOperation::FAvaDragOperation(TSharedRef<IAvaViewportClient> InAvaViewportClient, bool bInAllowSnapToChildren)
{
	AvaViewportClientWeak = InAvaViewportClient;
	bValid = false;
	Init(bInAllowSnapToChildren);
}

FAvaDragOperation::~FAvaDragOperation()
{
	Cleanup();
}

void FAvaDragOperation::Init(bool bInAllowSnapToChildren)
{
	TSharedPtr<IAvaViewportClient> AvaViewportClient = AvaViewportClientWeak.Pin();

	if (!ensureMsgf(AvaViewportClient.IsValid(), TEXT("Invalid ava viewport client.")))
	{
		return;
	}
	
	const FEditorViewportClient* EditorViewportClient = AvaViewportClient->AsEditorViewportClient();

	if (!ensureMsgf(EditorViewportClient, TEXT("Invalid editor viewport client.")))
	{
		return;
	}

	UAvaSelectionProviderSubsystem* SelectionProvider = UAvaSelectionProviderSubsystem::Get(EditorViewportClient, /* bInGenerateErrors */ true);

	if (!SelectionProvider)
	{
		return;
	}

	TConstArrayView<TWeakObjectPtr<AActor>> SelectedActors = SelectionProvider->GetSelectedActors();
	TConstArrayView<TWeakObjectPtr<UActorComponent>> SelectedComponents = SelectionProvider->GetSelectedComponents();

	TArray<UPrimitiveComponent*> SelectedPrimitiveComponents;
	SelectedPrimitiveComponents.Reserve(SelectedComponents.Num());

	for (const TWeakObjectPtr<UActorComponent>& SelectedComponentWeak : SelectedComponents)
	{
		if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(SelectedComponentWeak.Get()))
		{
			SelectedPrimitiveComponents.Add(PrimitiveComponent);
		}
	}

	if (SelectedActors.IsEmpty() && SelectedPrimitiveComponents.IsEmpty())
	{
		return;
	}

	SnapOperation = AvaViewportClient->StartSnapOperation();

	if (!ensureMsgf(SnapOperation.IsValid(), TEXT("Unable to create snap operation.")))
	{
		return;
	}

	// We currently only support actor drag stuff.
	if (!SelectedComponents.IsEmpty())
	{
		for (UPrimitiveComponent* PrimitiveComponent : SelectedPrimitiveComponents)
		{
			// Generate snap points for the selected actor
			FAvaSnapOperation::GenerateLocalSnapPoints(PrimitiveComponent, DraggedObjectSnapPoints);
			DraggedObjectStartLocations.Emplace(PrimitiveComponent, PrimitiveComponent->GetComponentLocation());
		}
	}
	else if (!SelectedActors.IsEmpty())
	{
		TArray<TWeakObjectPtr<AActor>> SelectedWeakActorChildren;

		for (const TWeakObjectPtr<AActor>& ActorWeak : SelectedActors)
		{
			if (AActor* Actor = ActorWeak.Get())
			{
				// Generate snap points for the selected actor
				FAvaSnapOperation::GenerateLocalSnapPoints(Actor, DraggedObjectSnapPoints);
				DraggedObjectStartLocations.Emplace(const_cast<AActor*>(Actor), Actor->GetActorLocation());

				if (!bInAllowSnapToChildren)
				{
					// Find child actors
					TConstArrayView<TWeakObjectPtr<AActor>> ChildActors = SelectionProvider->GetAttachedActors(Actor, /* Recursive */ true);
					SelectedWeakActorChildren.Append(ChildActors);
				}
			}
		}

		SnapOperation->GenerateActorSnapPoints(SelectedActors, SelectedWeakActorChildren);
	}

	SnapOperation->FinaliseSnapPoints();

	bValid = true;
}

void FAvaDragOperation::PreMouseUpdate()
{
	if (!bValid)
	{
		return;
	}

	if (!ensureMsgf(SnapOperation.IsValid(), TEXT("Invalid snap operation.")))
	{
		return;
	}

	const FVector DragOffset = SnapOperation->GetDragOffset();

	for (const TPair<TWeakObjectPtr<UObject>, FVector>& Pair : DraggedObjectStartLocations)
	{
		if (UObject* Object = Pair.Key.Get())
		{
			if (AActor* Actor = Cast<AActor>(Object))
			{
				Actor->SetActorLocation(Pair.Value + DragOffset);
			}
			else if (USceneComponent* SceneComponent = Cast<USceneComponent>(Object))
			{
				SceneComponent->SetWorldLocation(Pair.Value + DragOffset);
			}
		}
	}
}

void FAvaDragOperation::PostMouseUpdate()
{
	if (!bValid)
	{
		return;
	}

	if (!ensureMsgf(SnapOperation.IsValid(), TEXT("Invalid snap operation.")))
	{
		return;
	}

	const FVector DragOffset = SnapOperation->GetDragOffset();

	FVector SnapOffset = FVector::ZeroVector;
	SnapOperation->SnapDragLocation(DraggedObjectSnapPoints, SnapOffset);

	if (!SnapOperation->WasSnappedTo())
	{
		return;
	}

	const FVector TotalOffset = DragOffset + SnapOffset;

	for (const TPair<TWeakObjectPtr<UObject>, FVector>& Pair : DraggedObjectStartLocations)
	{
		if (UObject* Object = Pair.Key.Get())
		{
			if (AActor* Actor = Cast<AActor>(Object))
			{
				Actor->SetActorLocation(Pair.Value + TotalOffset);
			}
			else if (USceneComponent* SceneComponent = Cast<USceneComponent>(Object))
			{
				SceneComponent->SetWorldLocation(Pair.Value + TotalOffset);
			}
		}
	}
}

void FAvaDragOperation::Cleanup()
{
	if (!bValid)
	{
		return;
	}

	TSharedPtr<IAvaViewportClient> AvaViewportClient = AvaViewportClientWeak.Pin();

	if (!ensureMsgf(AvaViewportClient.IsValid(), TEXT("Invalid ava viewport client.")))
	{
		return;
	}

	if (SnapOperation.IsValid())
	{
		AvaViewportClient->EndSnapOperation(SnapOperation.Get());
	}

	if (DraggedObjectStartLocations.Num() != 1)
	{
		return;
	}

	const FEditorViewportClient* EditorViewportClient = AvaViewportClient->AsEditorViewportClient();

	if (!ensureMsgf(EditorViewportClient, TEXT("Invalid editor viewport client.")))
	{
		return;
	}

	FEditorModeTools* ModeTools = EditorViewportClient->GetModeTools();

	if (!ensureMsgf(ModeTools, TEXT("Invalid editor mode tools.")))
	{
		return;
	}

	// There's only 1 actor here, so no need to break or anything.
	for (const TPair<TWeakObjectPtr<UObject>, FVector>& Pair : DraggedObjectStartLocations)
	{
		if (UObject* Object = Pair.Key.Get())
		{
			if (AActor* Actor = Cast<AActor>(Object))
			{
				ModeTools->PivotLocation = Actor->GetActorLocation();
			}
			else if (USceneComponent* SceneComponent = Cast<USceneComponent>(Object))
			{
				ModeTools->PivotLocation = SceneComponent->GetComponentLocation();
			}
		}
	}
}
