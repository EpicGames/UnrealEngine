// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interaction/AvaIsolateActorsOperation.h"
#include "AvalancheViewportModule.h"
#include "Components/PrimitiveComponent.h"
#include "EditorViewportClient.h"
#include "Engine/Light.h"
#include "EngineUtils.h"
#include "GameFramework/Actor.h"
#include "GameFramework/Info.h"
#include "GameFramework/Volume.h"
#include "SceneView.h"
#include "Selection/AvaSelectionProviderSubsystem.h"
#include "ViewportClient/IAvaViewportClient.h"

namespace UE::AvaViewport::Private
{
	const TArray<UClass*> PersistentActorClasses = {
		AInfo::StaticClass(),
		ALight::StaticClass(),
		AVolume::StaticClass()
	};
}

FAvaIsolateActorsOperation::FAvaIsolateActorsOperation(TSharedRef<IAvaViewportClient> InAvaViewportClient)
{
	AvaViewportClientWeak = InAvaViewportClient;
}

void FAvaIsolateActorsOperation::AddIsolatedActorPrimitives(FSceneView* InSceneView)
{
	if (!InSceneView || IsolatedActors.IsEmpty())
	{
		return;
	}

	TSet<FPrimitiveComponentId> PrimitivesToShow;
	PrimitivesToShow.Reserve(IsolatedActors.Num() * 2); // Get some slack

	for (const TWeakObjectPtr<AActor>& ActorWeak : IsolatedActors)
	{
		if (AActor* Actor = ActorWeak.Get())
		{
			Actor->ForEachComponent<UPrimitiveComponent>(
				/* bIncludeFromChildActors */ false,
				[&PrimitivesToShow](const UPrimitiveComponent* InPrimitiveComponent)
				{
					PrimitivesToShow.Add(InPrimitiveComponent->GetPrimitiveSceneId());
				}
			);
		}
	}

	InSceneView->ShowOnlyPrimitives.Emplace(PrimitivesToShow);
}

bool FAvaIsolateActorsOperation::CanToggleIsolateActors() const
{
	if (!IsolatedActors.IsEmpty())
	{
		return true;
	}

	TSharedPtr<IAvaViewportClient> AvaViewportClient = AvaViewportClientWeak.Pin();

	if (!ensureMsgf(AvaViewportClient.IsValid(), TEXT("Invalid ava viewport client.")))
	{
		return false;
	}

	const FEditorViewportClient* EditorViewportClient = AvaViewportClient->AsEditorViewportClient();

	if (!EditorViewportClient)
	{
		return false;
	}

	if (!UAvaSelectionProviderSubsystem::SupportsWorldType(EditorViewportClient))
	{
		return false;
	}

	UAvaSelectionProviderSubsystem* SelectionProvider = UAvaSelectionProviderSubsystem::Get(EditorViewportClient, true);

	if (!SelectionProvider)
	{
		return false;
	}

	return !SelectionProvider->GetSelectedActors().IsEmpty();
}

void FAvaIsolateActorsOperation::ToggleIsolateActors()
{
	if (IsIsolatingActors())
	{
		UnisolateActors();
	}
	else
	{
		IsolateActors();
	}
}

void FAvaIsolateActorsOperation::IsolateActors()
{
	TSharedPtr<IAvaViewportClient> AvaViewportClient = AvaViewportClientWeak.Pin();

	if (!ensureMsgf(AvaViewportClient.IsValid(), TEXT("Invalid ava viewport client.")))
	{
		return;
	}

	const FEditorViewportClient* EditorViewportClient = AvaViewportClient->AsEditorViewportClient();

	if (!EditorViewportClient)
	{
		return;
	}

	UAvaSelectionProviderSubsystem* SelectionProvider = UAvaSelectionProviderSubsystem::Get(EditorViewportClient, true);

	if (!SelectionProvider)
	{
		return;
	}

	TConstArrayView<TWeakObjectPtr<AActor>> SelectedActors = SelectionProvider->GetSelectedActors();

	if (SelectedActors.IsEmpty())
	{
		return;
	}

	IsolatedActors.Empty();
	IsolatedActors.Reserve(SelectedActors.Num());

	using namespace UE::AvaViewport::Private;

	for (const TWeakObjectPtr<AActor>& ActorWeak : SelectedActors)
	{
		if (AActor* Actor = ActorWeak.Get())
		{
			bool bValidActor = true;
			UClass* ActorClass = Actor->GetClass();

			// Don't add these twice
			for (UClass* ActorClassFromList : PersistentActorClasses)
			{
				if (ActorClass->IsChildOf(ActorClassFromList))
				{
					bValidActor = false;
					break;
				}
			}

			if (bValidActor)
			{
				IsolatedActors.Add(ActorWeak);
			}
		}
	}

	// No valid or not-always-added-to-the-list actors selected
	if (IsolatedActors.IsEmpty())
	{
		IsolatedActors.Empty();
		UE_LOGF(AvaViewportLog, Log, "No isolateable actors selected.");
		return;
	}

	if (UWorld* World = EditorViewportClient->GetWorld())
	{
		for (UClass* ActorClass : PersistentActorClasses)
		{
			for (AActor* Actor : TActorRange<AActor>(World, ActorClass))
			{
				IsolatedActors.Add(Actor);
			}
		}
	}
}

void FAvaIsolateActorsOperation::UnisolateActors()
{
	IsolatedActors.Empty();
}
