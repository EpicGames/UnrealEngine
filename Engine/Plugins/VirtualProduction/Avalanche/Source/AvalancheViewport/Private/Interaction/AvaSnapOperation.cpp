// Copyright Epic Games, Inc. All Rights Reserved.

#include "Interaction/AvaSnapOperation.h"

#include "AvaViewportDataSubsystem.h"
#include "AvaViewportGuideInfo.h"
#include "AvaViewportSettings.h"
#include "AvaViewportUtils.h"
#include "AvaVisibleArea.h"
#include "Bounds/AvaBoundsProviderSubsystem.h"
#include "Camera/CameraActor.h"
#include "Components/PrimitiveComponent.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "Engine/Brush.h"
#include "Engine/Light.h"
#include "EngineUtils.h"
#include "GameFramework/Volume.h"
#include "GameFramework/WorldSettings.h"
#include "Interaction/AvaSnapDefs.h"
#include "Math/Vector.h"
#include "SceneView.h"
#include "Selection.h"
#include "Selection/AvaSelectionProviderSubsystem.h"
#include "Viewport/Interaction/AvaSnapPoint.h"
#include "Viewport/Interaction/IAvaSnapPointGenerator.h"
#include "ViewportClient/IAvaViewportClient.h"

namespace UE::AvaViewport::Private
{
	EAvaViewportSnapState GetSnapState()
	{
		if (UAvaViewportSettings const* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
		{
			return AvaViewportSettings->GetSnapState();
		}

		return EAvaViewportSnapState::Off;
	};

	bool IsAnySnapEnabled()
	{
		if (UAvaViewportSettings const* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
		{
			if (!EnumHasAnyFlags(AvaViewportSettings->GetSnapState(), EAvaViewportSnapState::Global))
			{
				return false;
			}

			return EnumHasAnyFlags(AvaViewportSettings->GetSnapState(), EAvaViewportSnapState::All);
		}

		return false;
	}

	float GetGridSize()
	{
		if (UAvaViewportSettings const* AvaViewportSettings = GetDefault<UAvaViewportSettings>())
		{
			return AvaViewportSettings->GridSize;
		}

		// Invalid value
		return -1.f;
	}

	// Rather than linking random modules, this is the way!
	const TSet<FString> DisallowedClassNames = {
		TEXT("MassVisualizer"),
		TEXT("SmartObjectSubsystemRenderingActor")
	};

	bool IsActorEligibleForSnapping(const AActor* InActor)
	{
		if (InActor->IsTemporarilyHiddenInEditor(true))
		{
			return false;
		}

		if (InActor->IsA<AWorldSettings>() || InActor->IsA<ABrush>() || InActor->IsA<AVolume>() || InActor->IsA<ACameraActor>()
			|| InActor->IsA<ALight>() || DisallowedClassNames.Contains(InActor->GetClass()->GetName()))
		{
			return false;
		}

		if (!InActor->GetRootComponent() || InActor->GetRootComponent()->IsVisualizationComponent())
		{
			return false;
		}

		if (InActor->GetActorLabel() == "Default Scene")
		{
			return false;
		}

		return true;
	}

	void UnfilteredDragSnap(const FTransform& InViewTransform, const TSharedRef<IAvaViewportClient>& InViewportClient, const FVector2f InViewportSize,
		const float InVisibleAreaFraction, FVector& OutSnapOffset, FVector2f& OutSnappedToLocation,
		const FVector& InWorldSpaceSnapLocation, const FVector2f& InOriginalScreenSpaceSnapLocation, const FVector2f& InSnappedScreenSpaceSnapLocation,
		int32 InScreenSpaceComponent, int32 InWorldSpaceCoordinate, double InOffsetMultiplier)
	{
		const FVector ScreenWorldSpaceSnapLocation = InViewTransform.InverseTransformPosition(InWorldSpaceSnapLocation);
		const FVector2D FrustumSize = InViewportClient->GetZoomedFrustumSizeAtDistance(ScreenWorldSpaceSnapLocation.X);

		// Calculate offset (difference between snapped and non-snapped position)
		OutSnapOffset[InWorldSpaceCoordinate] = InSnappedScreenSpaceSnapLocation[InScreenSpaceComponent] - InOriginalScreenSpaceSnapLocation[InScreenSpaceComponent];

		// Scale size down to offset at the given distance from the camera (closer points are moved less)
		OutSnapOffset[InWorldSpaceCoordinate] *= FrustumSize[InScreenSpaceComponent] / InViewportSize[InScreenSpaceComponent] / InVisibleAreaFraction;
		OutSnapOffset[InWorldSpaceCoordinate] *= InOffsetMultiplier;

		// Set the snap point
		OutSnappedToLocation[InScreenSpaceComponent] = InSnappedScreenSpaceSnapLocation[InScreenSpaceComponent];
	}

	void SingleAxisDragSnap(const TSharedRef<IAvaViewportClient>& InViewportClient, const FVector2f& InSnappedScreenSpaceSnapLocation,
		const FVector2f& InAxisOffset, const FVector& InWorldSpaceSnapLocation, bool& bOutSnappedToAxis, const FVector& InDragDirection,
		FVector& OutSnapOffset)
	{
		// Make sure the coordinates work for perspective and orthographic by calculating them based on the viewport.
		const FVector ScreenWorldSpaceSnapLocation = InViewportClient->ViewportPositionToWorldPosition(
			InSnappedScreenSpaceSnapLocation,
			100.f
		);

		const FVector CloserScreenWorldSpaceSnapLocation = InViewportClient->ViewportPositionToWorldPosition(
			InSnappedScreenSpaceSnapLocation,
			10.f
		);

		const FVector OffsetScreenWorldSpaceLocation = InViewportClient->ViewportPositionToWorldPosition(
			InSnappedScreenSpaceSnapLocation + (InAxisOffset * 100.f),
			100.f
		);

		// A plane representing the snapped-to line on the screen.
		const FPlane SnapPlane = FPlane(
			ScreenWorldSpaceSnapLocation,
			CloserScreenWorldSpaceSnapLocation,
			OffsetScreenWorldSpaceLocation
		);

		// Starting point of the drag for this snap point.
		const FVector& SnapStartLocation = InWorldSpaceSnapLocation;

		bOutSnappedToAxis = false;

		// If the drag direction is orthogonal to the plane normal, no snap is possible.
		if (!FMath::IsNearlyZero(InDragDirection.Dot(SnapPlane.GetNormal())))
		{
			// Calculate the intersection of the dragged point and the plane.
			FVector IntersectPoint = FMath::RayPlaneIntersection(SnapStartLocation, InDragDirection, SnapPlane);

			if (!IntersectPoint.ContainsNaN())
			{
				OutSnapOffset = IntersectPoint - SnapStartLocation;
				bOutSnappedToAxis = true;
			}
		}
	}

	void DoubleAxisDragSnap(const TSharedRef<IAvaViewportClient>& InViewportClient, const FVector2f& InSnappedScreenSpaceSnapLocation,
		const FVector2f& InAxisOffset, const FVector& InWorldSpaceSnapLocation, bool& bOutSnappedToAxis, const FVector& InDragWorldSpaceOffset1,
		const FVector& InDragWorldSpaceOffset2, FVector& OutSnapOffset)
	{
		// Make sure the coordinates work for perspective and orthographic by calculating them based on the viewport.
		const FVector ScreenWorldSpaceSnapLocation = InViewportClient->ViewportPositionToWorldPosition(
			InSnappedScreenSpaceSnapLocation,
			100.f
		);

		const FVector CloserScreenWorldSpaceSnapLocation = InViewportClient->ViewportPositionToWorldPosition(
			InSnappedScreenSpaceSnapLocation,
			10.f
		);

		const FVector OffsetScreenWorldSpaceLocation = InViewportClient->ViewportPositionToWorldPosition(
			InSnappedScreenSpaceSnapLocation + (InAxisOffset * 100.f),
			100.f
		);

		// A plane representing the snapped-to line on the screen.
		const FPlane SnapPlane = FPlane(
			ScreenWorldSpaceSnapLocation,
			CloserScreenWorldSpaceSnapLocation,
			OffsetScreenWorldSpaceLocation
		);

		// Starting point of the drag for this snap point.
		const FVector& SnapStartLocation = InWorldSpaceSnapLocation;

		const FVector& DragWorldSpaceStart = SnapStartLocation;

		const FPlane DragPlane = FPlane(
			DragWorldSpaceStart,
			DragWorldSpaceStart + InDragWorldSpaceOffset1,
			DragWorldSpaceStart + InDragWorldSpaceOffset2
		);

		bOutSnappedToAxis = false;

		// Calculate the intersection of the dragged point and the plane.
		FVector IntersectPoint;
		FVector IntersectDirection;

		if (FMath::IntersectPlanes2(IntersectPoint, IntersectDirection, SnapPlane, DragPlane))
		{
			FVector ClosestPoint = FMath::ClosestPointOnInfiniteLine(
				IntersectPoint - IntersectDirection * 100000.0,
				IntersectPoint + IntersectDirection * 100000.0,
				SnapStartLocation
			);

			if (!ClosestPoint.ContainsNaN())
			{
				OutSnapOffset = ClosestPoint - SnapStartLocation;
				bOutSnappedToAxis = true;
			}
		}
	}
}

FAvaSnapOperation::FAvaSnapOperation(FEditorViewportClient* InEditorViewportClient)
{ 
	EditorViewportClient = InEditorViewportClient;

	ClosestSnapPointLinkIdxX = INDEX_NONE;
	ClosestSnapPointLinkIdxY = INDEX_NONE;
	bSnappedToLinkX = false;
	bSnappedToLinkY = false;
	SnapDistances = FVector2f::ZeroVector;

	GenerateScreenSnapPoints();

	if (InEditorViewportClient && InEditorViewportClient->Viewport)
	{
		CachedSceneViewFamily = MakeShared<FSceneViewFamilyContext>(FSceneViewFamily::ConstructionValues(
			InEditorViewportClient->Viewport
			, InEditorViewportClient->GetScene()
			, InEditorViewportClient->EngineShowFlags)
			.SetRealtimeUpdate(InEditorViewportClient->IsRealtime()));

		CachedSceneView = InEditorViewportClient->CalcSceneView(CachedSceneViewFamily.Get());

		SelectionStartLocation = GetSelectionLocation();
	}
}

FAvaSnapOperation::~FAvaSnapOperation()
{
	if (TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient))
	{
		AvaViewportClient->EndSnapOperation(this);
	}
}

void FAvaSnapOperation::GenerateLocalSnapPoints(AActor* InActor, TArray<FAvaSnapPoint>& InOutSnapPoints)
{
	using namespace UE::AvaViewport::Private;

	if (!InActor || !IsActorEligibleForSnapping(InActor))
	{
		return;
	}

	if (const IAvaSnapPointGenerator* SnapPointGenerator = Cast<IAvaSnapPointGenerator>(InActor))
	{
		InOutSnapPoints.Append(SnapPointGenerator->GetLocalSnapPoints());
	}

	// Generate our locations to try to snap
	if (UAvaBoundsProviderSubsystem* BoundsProvider = UAvaBoundsProviderSubsystem::Get(InActor, /* bInGenerateErrors */ true))
	{
		const FBox LocalBounds = BoundsProvider->GetActorLocalBounds(InActor);

		if (LocalBounds.IsValid)
		{
			GenerateLocalSnapPoints(InActor, LocalBounds.GetCenter(), LocalBounds.GetExtent(), InOutSnapPoints);
		}
	}
}

void FAvaSnapOperation::GenerateLocalSnapPoints(AActor* InActor, const FVector& InOrigin, const FVector& InBoxExtent, TArray<FAvaSnapPoint>& InOutSnapPoints)
{
	// Calculate snap points for the actor's bounds
	InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::TopLeft,     InOrigin + FVector(-InBoxExtent.X, -InBoxExtent.Y, InBoxExtent.Z),  EAvaDepthAlignment::Front)); // TL
	InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::Top,         InOrigin + FVector(-InBoxExtent.X, 0, InBoxExtent.Z),             EAvaDepthAlignment::Front)); // T
	InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::TopRight,    InOrigin + FVector(-InBoxExtent.X, InBoxExtent.Y, InBoxExtent.Z),   EAvaDepthAlignment::Front)); // TR
	InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::Left,        InOrigin + FVector(-InBoxExtent.X, -InBoxExtent.Y, 0),            EAvaDepthAlignment::Front)); // L
	InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::Center,      InOrigin + FVector(-InBoxExtent.X, 0, 0),                       EAvaDepthAlignment::Front)); // M
	InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::Right,       InOrigin + FVector(-InBoxExtent.X, InBoxExtent.Y, 0),             EAvaDepthAlignment::Front)); // R
	InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::BottomLeft,  InOrigin + FVector(-InBoxExtent.X, -InBoxExtent.Y, -InBoxExtent.Z), EAvaDepthAlignment::Front)); // BL
	InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::Bottom,      InOrigin + FVector(-InBoxExtent.X, 0, -InBoxExtent.Z),            EAvaDepthAlignment::Front)); // B
	InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::BottomRight, InOrigin + FVector(-InBoxExtent.X, InBoxExtent.Y, -InBoxExtent.Z),  EAvaDepthAlignment::Front)); // BR

	if (InBoxExtent.X > 0)
	{
		InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::TopLeft,     InOrigin + FVector(0.f, -InBoxExtent.Y, InBoxExtent.Z),  EAvaDepthAlignment::Center)); // TL
		InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::Top,         InOrigin + FVector(0.f, 0, InBoxExtent.Z),             EAvaDepthAlignment::Center)); // T
		InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::TopRight,    InOrigin + FVector(0.f, InBoxExtent.Y, InBoxExtent.Z),   EAvaDepthAlignment::Center)); // TR
		InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::Left,        InOrigin + FVector(0.f, -InBoxExtent.Y, 0),            EAvaDepthAlignment::Center)); // L
		InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::Center,      InOrigin + FVector(0.f, 0, 0),                       EAvaDepthAlignment::Center)); // M
		InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::Right,       InOrigin + FVector(0.f, InBoxExtent.Y, 0),             EAvaDepthAlignment::Center)); // R
		InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::BottomLeft,  InOrigin + FVector(0.f, -InBoxExtent.Y, -InBoxExtent.Z), EAvaDepthAlignment::Center)); // BL
		InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::Bottom,      InOrigin + FVector(0.f, 0, -InBoxExtent.Z),            EAvaDepthAlignment::Center)); // B
		InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::BottomRight, InOrigin + FVector(0.f, InBoxExtent.Y, -InBoxExtent.Z),  EAvaDepthAlignment::Center)); // BR

		InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::TopLeft,     InOrigin + FVector(InBoxExtent.X, -InBoxExtent.Y, InBoxExtent.Z),  EAvaDepthAlignment::Back)); // TL
		InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::Top,         InOrigin + FVector(InBoxExtent.X, 0, InBoxExtent.Z),             EAvaDepthAlignment::Back)); // T
		InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::TopRight,    InOrigin + FVector(InBoxExtent.X, InBoxExtent.Y, InBoxExtent.Z),   EAvaDepthAlignment::Back)); // TR
		InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::Left,        InOrigin + FVector(InBoxExtent.X, -InBoxExtent.Y, 0),            EAvaDepthAlignment::Back)); // L
		InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::Center,      InOrigin + FVector(InBoxExtent.X, 0, 0),                       EAvaDepthAlignment::Back)); // M
		InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::Right,       InOrigin + FVector(InBoxExtent.X, InBoxExtent.Y, 0),             EAvaDepthAlignment::Back)); // R
		InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::BottomLeft,  InOrigin + FVector(InBoxExtent.X, -InBoxExtent.Y, -InBoxExtent.Z), EAvaDepthAlignment::Back)); // BL
		InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::Bottom,      InOrigin + FVector(InBoxExtent.X, 0, -InBoxExtent.Z),            EAvaDepthAlignment::Back)); // B
		InOutSnapPoints.Add(FAvaSnapPoint::CreateActorSnapPoint(InActor, EAvaAnchors::BottomRight, InOrigin + FVector(InBoxExtent.X, InBoxExtent.Y, -InBoxExtent.Z),  EAvaDepthAlignment::Back)); // BR
	}
}

void FAvaSnapOperation::GenerateLocalSnapPoints(UPrimitiveComponent* InComponent, TArray<FAvaSnapPoint>& InOutSnapPoints)
{
	if (!InComponent)
	{
		return;
	}

	// Generate our locations to try to snap
	if (UAvaBoundsProviderSubsystem* BoundsProvider = UAvaBoundsProviderSubsystem::Get(InComponent, /* bInGenerateErrors */ true))
	{
		const FBox LocalBounds = BoundsProvider->GetComponentLocalBounds(InComponent);

		if (LocalBounds.IsValid)
		{
			GenerateLocalSnapPoints(InComponent, LocalBounds.GetCenter(), LocalBounds.GetExtent(), InOutSnapPoints);
		}
	}
}

void FAvaSnapOperation::GenerateLocalSnapPoints(UPrimitiveComponent* InComponent, const FVector& InOrigin, const FVector& InBoxExtent, TArray<FAvaSnapPoint>& InOutSnapPoints)
{
	// Calculate snap points for the component's bounds
	InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::TopLeft,     InOrigin + FVector(-InBoxExtent.X, -InBoxExtent.Y, InBoxExtent.Z),  EAvaDepthAlignment::Front)); // TL
	InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::Top,         InOrigin + FVector(-InBoxExtent.X, 0, InBoxExtent.Z),             EAvaDepthAlignment::Front)); // T
	InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::TopRight,    InOrigin + FVector(-InBoxExtent.X, InBoxExtent.Y, InBoxExtent.Z),   EAvaDepthAlignment::Front)); // TR
	InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::Left,        InOrigin + FVector(-InBoxExtent.X, -InBoxExtent.Y, 0),            EAvaDepthAlignment::Front)); // L
	InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::Center,      InOrigin + FVector(-InBoxExtent.X, 0, 0),                       EAvaDepthAlignment::Front)); // M
	InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::Right,       InOrigin + FVector(-InBoxExtent.X, InBoxExtent.Y, 0),             EAvaDepthAlignment::Front)); // R
	InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::BottomLeft,  InOrigin + FVector(-InBoxExtent.X, -InBoxExtent.Y, -InBoxExtent.Z), EAvaDepthAlignment::Front)); // BL
	InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::Bottom,      InOrigin + FVector(-InBoxExtent.X, 0, -InBoxExtent.Z),            EAvaDepthAlignment::Front)); // B
	InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::BottomRight, InOrigin + FVector(-InBoxExtent.X, InBoxExtent.Y, -InBoxExtent.Z),  EAvaDepthAlignment::Front)); // BR

	if (InBoxExtent.X > 0)
	{
		InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::TopLeft,     InOrigin + FVector(0.f, -InBoxExtent.Y, InBoxExtent.Z),  EAvaDepthAlignment::Center)); // TL
		InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::Top,         InOrigin + FVector(0.f, 0, InBoxExtent.Z),             EAvaDepthAlignment::Center)); // T
		InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::TopRight,    InOrigin + FVector(0.f, InBoxExtent.Y, InBoxExtent.Z),   EAvaDepthAlignment::Center)); // TR
		InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::Left,        InOrigin + FVector(0.f, -InBoxExtent.Y, 0),            EAvaDepthAlignment::Center)); // L
		InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::Center,      InOrigin + FVector(0.f, 0, 0),                       EAvaDepthAlignment::Center)); // M
		InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::Right,       InOrigin + FVector(0.f, InBoxExtent.Y, 0),             EAvaDepthAlignment::Center)); // R
		InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::BottomLeft,  InOrigin + FVector(0.f, -InBoxExtent.Y, -InBoxExtent.Z), EAvaDepthAlignment::Center)); // BL
		InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::Bottom,      InOrigin + FVector(0.f, 0, -InBoxExtent.Z),            EAvaDepthAlignment::Center)); // B
		InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::BottomRight, InOrigin + FVector(0.f, InBoxExtent.Y, -InBoxExtent.Z),  EAvaDepthAlignment::Center)); // BR

		InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::TopLeft,     InOrigin + FVector(InBoxExtent.X, -InBoxExtent.Y, InBoxExtent.Z),  EAvaDepthAlignment::Back)); // TL
		InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::Top,         InOrigin + FVector(InBoxExtent.X, 0, InBoxExtent.Z),             EAvaDepthAlignment::Back)); // T
		InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::TopRight,    InOrigin + FVector(InBoxExtent.X, InBoxExtent.Y, InBoxExtent.Z),   EAvaDepthAlignment::Back)); // TR
		InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::Left,        InOrigin + FVector(InBoxExtent.X, -InBoxExtent.Y, 0),            EAvaDepthAlignment::Back)); // L
		InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::Center,      InOrigin + FVector(InBoxExtent.X, 0, 0),                       EAvaDepthAlignment::Back)); // M
		InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::Right,       InOrigin + FVector(InBoxExtent.X, InBoxExtent.Y, 0),             EAvaDepthAlignment::Back)); // R
		InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::BottomLeft,  InOrigin + FVector(InBoxExtent.X, -InBoxExtent.Y, -InBoxExtent.Z), EAvaDepthAlignment::Back)); // BL
		InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::Bottom,      InOrigin + FVector(InBoxExtent.X, 0, -InBoxExtent.Z),            EAvaDepthAlignment::Back)); // B
		InOutSnapPoints.Add(FAvaSnapPoint::CreateComponentSnapPoint(InComponent, EAvaAnchors::BottomRight, InOrigin + FVector(InBoxExtent.X, InBoxExtent.Y, -InBoxExtent.Z),  EAvaDepthAlignment::Back)); // BR
	}
}

void FAvaSnapOperation::GenerateBoundsSnapPoints(const FVector& InOrigin, const FVector& InBoxExtent, TArray<FAvaSnapPoint>& InOutSnapPoints)
{
	// Calculate snap points for the bounds
	InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::TopLeft,     InOrigin + FVector(-InBoxExtent.X, -InBoxExtent.Y, InBoxExtent.Z),  EAvaDepthAlignment::Front)); // TL
	InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::Top,         InOrigin + FVector(-InBoxExtent.X, 0, InBoxExtent.Z),             EAvaDepthAlignment::Front)); // T
	InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::TopRight,    InOrigin + FVector(-InBoxExtent.X, InBoxExtent.Y, InBoxExtent.Z),   EAvaDepthAlignment::Front)); // TR
	InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::Left,        InOrigin + FVector(-InBoxExtent.X, -InBoxExtent.Y, 0),            EAvaDepthAlignment::Front)); // L
	InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::Center,      InOrigin + FVector(-InBoxExtent.X, 0, 0),                       EAvaDepthAlignment::Front)); // M
	InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::Right,       InOrigin + FVector(-InBoxExtent.X, InBoxExtent.Y, 0),             EAvaDepthAlignment::Front)); // R
	InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::BottomLeft,  InOrigin + FVector(-InBoxExtent.X, -InBoxExtent.Y, -InBoxExtent.Z), EAvaDepthAlignment::Front)); // BL
	InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::Bottom,      InOrigin + FVector(-InBoxExtent.X, 0, -InBoxExtent.Z),            EAvaDepthAlignment::Front)); // B
	InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::BottomRight, InOrigin + FVector(-InBoxExtent.X, InBoxExtent.Y, -InBoxExtent.Z),  EAvaDepthAlignment::Front)); // BR

	if (InBoxExtent.X > 0)
	{
		InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::TopLeft,     InOrigin + FVector(0.f, -InBoxExtent.Y, InBoxExtent.Z),  EAvaDepthAlignment::Center)); // TL
		InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::Top,         InOrigin + FVector(0.f, 0, InBoxExtent.Z),             EAvaDepthAlignment::Center)); // T
		InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::TopRight,    InOrigin + FVector(0.f, InBoxExtent.Y, InBoxExtent.Z),   EAvaDepthAlignment::Center)); // TR
		InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::Left,        InOrigin + FVector(0.f, -InBoxExtent.Y, 0),            EAvaDepthAlignment::Center)); // L
		InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::Center,      InOrigin + FVector(0.f, 0, 0),                       EAvaDepthAlignment::Center)); // M
		InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::Right,       InOrigin + FVector(0.f, InBoxExtent.Y, 0),             EAvaDepthAlignment::Center)); // R
		InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::BottomLeft,  InOrigin + FVector(0.f, -InBoxExtent.Y, -InBoxExtent.Z), EAvaDepthAlignment::Center)); // BL
		InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::Bottom,      InOrigin + FVector(0.f, 0, -InBoxExtent.Z),            EAvaDepthAlignment::Center)); // B
		InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::BottomRight, InOrigin + FVector(0.f, InBoxExtent.Y, -InBoxExtent.Z),  EAvaDepthAlignment::Center)); // BR

		InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::TopLeft,     InOrigin + FVector(InBoxExtent.X, -InBoxExtent.Y, InBoxExtent.Z),  EAvaDepthAlignment::Back)); // TL
		InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::Top,         InOrigin + FVector(InBoxExtent.X, 0, InBoxExtent.Z),             EAvaDepthAlignment::Back)); // T
		InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::TopRight,    InOrigin + FVector(InBoxExtent.X, InBoxExtent.Y, InBoxExtent.Z),   EAvaDepthAlignment::Back)); // TR
		InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::Left,        InOrigin + FVector(InBoxExtent.X, -InBoxExtent.Y, 0),            EAvaDepthAlignment::Back)); // L
		InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::Center,      InOrigin + FVector(InBoxExtent.X, 0, 0),                       EAvaDepthAlignment::Back)); // M
		InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::Right,       InOrigin + FVector(InBoxExtent.X, InBoxExtent.Y, 0),             EAvaDepthAlignment::Back)); // R
		InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::BottomLeft,  InOrigin + FVector(InBoxExtent.X, -InBoxExtent.Y, -InBoxExtent.Z), EAvaDepthAlignment::Back)); // BL
		InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::Bottom,      InOrigin + FVector(InBoxExtent.X, 0, -InBoxExtent.Z),            EAvaDepthAlignment::Back)); // B
		InOutSnapPoints.Add(FAvaSnapPoint::CreateBoundsSnapPoint(EAvaAnchors::BottomRight, InOrigin + FVector(InBoxExtent.X, InBoxExtent.Y, -InBoxExtent.Z),  EAvaDepthAlignment::Back)); // BR
	}
}

void FAvaSnapOperation::GenerateScreenSnapPoints()
{
	using namespace UE::AvaViewport::Private;

	if (!EnumHasAnyFlags(GetSnapState(), EAvaViewportSnapState::Screen))
	{
		return;
	}

	TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient);

	if (!AvaViewportClient.IsValid())
	{
		return;
	}

	const FVector2f ViewportSize = AvaViewportClient->GetViewportSize();

	if (!FAvaViewportUtils::IsValidViewportSize(ViewportSize))
	{
		return;
	}

	ScreenSnapPoints.Empty();

	// Screen anchors
	ScreenSnapPoints.Add(FAvaSnapPoint::CreateScreenSnapPoint(EAvaAnchors::TopLeft, FVector2f(0.f, 0.f)));
	ScreenSnapPoints.Add(FAvaSnapPoint::CreateScreenSnapPoint(EAvaAnchors::Top, FVector2f(ViewportSize.X / 2.f, 0.f)));
	ScreenSnapPoints.Add(FAvaSnapPoint::CreateScreenSnapPoint(EAvaAnchors::TopRight, FVector2f(ViewportSize.X, 0.f)));
	ScreenSnapPoints.Add(FAvaSnapPoint::CreateScreenSnapPoint(EAvaAnchors::Left, FVector2f(0.f, ViewportSize.Y / 2.f)));
	ScreenSnapPoints.Add(FAvaSnapPoint::CreateScreenSnapPoint(EAvaAnchors::Center, FVector2f(ViewportSize.X / 2.f, ViewportSize.Y / 2.f)));
	ScreenSnapPoints.Add(FAvaSnapPoint::CreateScreenSnapPoint(EAvaAnchors::Right, FVector2f(ViewportSize.X, ViewportSize.Y / 2.f)));
	ScreenSnapPoints.Add(FAvaSnapPoint::CreateScreenSnapPoint(EAvaAnchors::BottomLeft, FVector2f(0.f, ViewportSize.Y)));
	ScreenSnapPoints.Add(FAvaSnapPoint::CreateScreenSnapPoint(EAvaAnchors::Bottom, FVector2f(ViewportSize.X / 2.f, ViewportSize.Y)));
	ScreenSnapPoints.Add(FAvaSnapPoint::CreateScreenSnapPoint(EAvaAnchors::BottomRight, FVector2f(ViewportSize.X, ViewportSize.Y)));

	// Guides
	if (UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(AvaViewportClient->GetViewportWorld()))
	{
		if (FAvaViewportData* Data = DataSubsystem->GetData())
		{
			for (const FAvaViewportGuideInfo& GuideInfo : Data->GuideData)
			{
				if (GuideInfo.IsEnabled())
				{
					switch (GuideInfo.Orientation)
					{
						case EOrientation::Orient_Horizontal:
							ScreenSnapPoints.Add(FAvaSnapPoint::CreateGuideSnapPoint(GuideInfo.OffsetFraction * ViewportSize.Y, GuideInfo.Orientation));
							break;

						case EOrientation::Orient_Vertical:
							ScreenSnapPoints.Add(FAvaSnapPoint::CreateGuideSnapPoint(GuideInfo.OffsetFraction * ViewportSize.X, GuideInfo.Orientation));
							break;
					}
				}
			}
		}
	}
}

void FAvaSnapOperation::GenerateComponentSnapPoints(const UActorComponent* InComponent)
{
	using namespace UE::AvaViewport::Private;

	if (!EnumHasAnyFlags(GetSnapState(), EAvaViewportSnapState::Actor))
	{
		return;
	}

	if (!InComponent->GetOwner())
	{
		return;
	}

	AActor* Owner = InComponent->GetOwner();

	if (!Owner)
	{
		return;
	}

	UAvaSelectionProviderSubsystem* SelectionProvider = UAvaSelectionProviderSubsystem::Get(InComponent, /* bInGenerateErrors */ true);

	if (!SelectionProvider)
	{
		return;
	}

	TArray<TWeakObjectPtr<AActor>> ActorArray;
	ActorArray.Add(Owner);

	TConstArrayView<TWeakObjectPtr<AActor>> ChildActors = SelectionProvider->GetAttachedActors(Owner, /* Recursive */ true);

	GenerateActorSnapPoints(ActorArray, ChildActors);
}

void FAvaSnapOperation::GenerateActorSnapPoints(const TConstArrayView<TWeakObjectPtr<AActor>>& InSelectedActors,
	const TConstArrayView<TWeakObjectPtr<AActor>>& InExcludedActors)
{
	using namespace UE::AvaViewport::Private;

	TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient);

	if (!AvaViewportClient.IsValid())
	{
		return;
	}

	ActorSnapPoints.Empty();

	if (!EnumHasAnyFlags(GetSnapState(), EAvaViewportSnapState::Actor))
	{
		return;
	}

	UWorld* World = nullptr;

	if (EditorViewportClient)
	{
		World = EditorViewportClient->GetWorld();
	}

	if (!World)
	{
		for (const TWeakObjectPtr<AActor>& SelectedActorWeak : InSelectedActors)
		{
			if (AActor* SelectedActor = SelectedActorWeak.Get())
			{
				World = SelectedActor->GetWorld();

				if (World)
				{
					break;
				}
			}
		}
	}

	if (!World)
	{
		for (const TWeakObjectPtr<AActor>& ExcludedActorWeak : InExcludedActors)
		{
			if (AActor* ExcludedActor = ExcludedActorWeak.Get())
			{
				World = ExcludedActor->GetWorld();

				if (World)
				{
					break;
				}
			}
		}
	}

	if (World)
	{
		TArray<TWeakObjectPtr<AActor>> IsolatedActors;

		// @TODO Isolated actors

		for (AActor* Actor : TActorRange<AActor>(World))
		{
			if (Actor == nullptr || !IsActorEligibleForSnapping(Actor))
			{
				continue;
			}

			if (InSelectedActors.Contains(Actor) || InExcludedActors.Contains(Actor))
			{
				continue;
			}

			if (IsolatedActors.Num() > 0 && !IsolatedActors.Contains(Actor))
			{
				continue;
			}

			GenerateLocalSnapPoints(Actor, ActorSnapPoints);
		}
	}
}

void FAvaSnapOperation::FinaliseSnapPoints()
{
	SnapPointLinksX.Empty();
	SnapPointLinksY.Empty();

	for (int32 SnapPointIdx = 0; SnapPointIdx < ScreenSnapPoints.Num(); ++SnapPointIdx)
	{
		if (ScreenSnapPoints[SnapPointIdx].Anchor != EAvaAnchors::Custom
			&& ScreenSnapPoints[SnapPointIdx].Anchor != EAvaAnchors::None)
		{
			// Screen snap points
			SnapPointLinksX.Add({static_cast<float>(ScreenSnapPoints[SnapPointIdx].Location.Y), SnapPointIdx});
			SnapPointLinksY.Add({static_cast<float>(ScreenSnapPoints[SnapPointIdx].Location.Z), SnapPointIdx});
		}
		else if (ScreenSnapPoints[SnapPointIdx].Anchor == EAvaAnchors::Custom)
		{
			// Vertical guide (x-offset)
			if (ScreenSnapPoints[SnapPointIdx].Location.Y > 0)
			{
				SnapPointLinksX.Add({static_cast<float>(ScreenSnapPoints[SnapPointIdx].Location.Y), SnapPointIdx});
			}
			// Horizontal guide (y-offset)
			else if (ScreenSnapPoints[SnapPointIdx].Location.Z > 0)
			{
				SnapPointLinksY.Add({static_cast<float>(ScreenSnapPoints[SnapPointIdx].Location.Z), SnapPointIdx});
			}
		}
	}

	TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient);

	if (AvaViewportClient.IsValid())
	{
		const FTransform ViewTransform = FTransform(EditorViewportClient->GetViewRotation(), EditorViewportClient->GetViewLocation(), FVector::OneVector);
		FVector2f ScreenSize = AvaViewportClient->GetViewportSize();

		for (int32 SnapPointIdx = 0; SnapPointIdx < ActorSnapPoints.Num(); ++SnapPointIdx)
		{
			FAvaSnapPoint& ActorSnapPoint = ActorSnapPoints[SnapPointIdx];

			if (ActorSnapPoint.Outer.IsValid())
			{
				if (ActorSnapPoint.Outer->IsA<AActor>())
				{
					ActorSnapPoint.Location = Cast<const AActor>(ActorSnapPoint.Outer)->GetRootComponent()->GetComponentTransform().TransformPositionNoScale(
						ActorSnapPoint.Location);
				}
				else if (ActorSnapPoint.Outer->IsA<UPrimitiveComponent>())
				{
					ActorSnapPoint.Location = Cast<const UPrimitiveComponent>(ActorSnapPoint.Outer)->GetComponentTransform().TransformPositionNoScale(
						ActorSnapPoint.Location);
				}
				else
				{
					checkNoEntry();
				}
			}

			double Distance;
			FVector2f ScreenLocation;
			AvaViewportClient->WorldPositionToViewportPosition(ActorSnapPoint.Location, ScreenLocation, Distance);

			if (Distance < 0)
			{
				continue;
			}

			SnapPointLinksX.Add({ScreenLocation.X, SnapPointIdx + ScreenSnapPoints.Num()});
			SnapPointLinksY.Add({ScreenLocation.Y, SnapPointIdx + ScreenSnapPoints.Num()});
		}
	}

	SnapPointLinksX.Sort();
	SnapPointLinksY.Sort();
}

void FAvaSnapOperation::AddActorSnapPoint(const FAvaSnapPoint& InSnapPoint)
{
	TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient);

	if (!AvaViewportClient.IsValid())
	{
		return;
	}

	const int32 SnapPointIdx = ActorSnapPoints.Add(InSnapPoint);

	FVector2f ScreenPosition;
	double Distance;

	AvaViewportClient->WorldPositionToViewportPosition(InSnapPoint.Location, ScreenPosition, Distance);

	SnapPointLinksX.Add({ScreenPosition.X, SnapPointIdx + ScreenSnapPoints.Num()});
	SnapPointLinksY.Add({ScreenPosition.Y, SnapPointIdx + ScreenSnapPoints.Num()});
}

bool FAvaSnapOperation::SnapX(float& InOutComponent, float InViewportSize, int32& OutClosestSnapPointIdx, float& OutDistanceFromSnap)
{
	const bool bWasSnappedTo = SnapVectorComponent(InOutComponent, SnapPointLinksX, InViewportSize, ScreenSnapPoints.Num(), OutClosestSnapPointIdx, OutDistanceFromSnap);

	if (bWasSnappedTo)
	{
		SnappedToLocation.X = InOutComponent;
	}

	return bWasSnappedTo;
}

bool FAvaSnapOperation::SnapY(float& InOutComponent, float InViewportSize, int32& OutClosestSnapPointIdx, float& OutDistanceFromSnap)
{
	const bool bWasSnappedTo = SnapVectorComponent(InOutComponent, SnapPointLinksY, InViewportSize, ScreenSnapPoints.Num(), OutClosestSnapPointIdx, OutDistanceFromSnap);

	if (bWasSnappedTo)
	{
		SnappedToLocation.Y = InOutComponent;
	}

	return bWasSnappedTo;
}

bool FAvaSnapOperation::SnapVectorComponent(float& InOutComponent, const TArray<FAvaSnapPointLink>& InSnapPointLinks,
	float InViewportSize, int32 InScreenSnapPointCount, int32& OutClosestSnapPointIdx, float& OutDistanceFromSnap)
{
	using namespace UE::AvaViewport::Private;

	OutDistanceFromSnap = -1;	

	if (InSnapPointLinks.Num() > 0 && EnumHasAnyFlags(GetSnapState(), EAvaViewportSnapState::Screen | EAvaViewportSnapState::Actor))
	{
		if (!InSnapPointLinks.IsValidIndex(OutClosestSnapPointIdx))
		{
			OutClosestSnapPointIdx = INDEX_NONE;
		}
		else
		{
			OutDistanceFromSnap = FMath::Abs(InOutComponent - InSnapPointLinks[OutClosestSnapPointIdx].Position);
		}

		if (OutClosestSnapPointIdx > 0)
		{
			// Check lower points
			for (int32 SnapPointIterIdx = OutClosestSnapPointIdx - 1; SnapPointIterIdx >= 0; --SnapPointIterIdx)
			{
				const float NewDistance = FMath::Abs(InOutComponent - InSnapPointLinks[SnapPointIterIdx].Position);

				// Should never get this, really, There should always be a distance if SnapPointIdx isn't INDEX_NONE
				if (OutDistanceFromSnap < 0)
				{
					OutDistanceFromSnap = NewDistance;
					OutClosestSnapPointIdx = SnapPointIterIdx;
					continue;
				}

				// Lower snap indices have priority (assign the snap distance if it's equal or lower (current snap index is higher.))
				if (NewDistance <= OutDistanceFromSnap)
				{
					OutDistanceFromSnap = NewDistance;
					OutClosestSnapPointIdx = SnapPointIterIdx;
					continue;
				}

				if (OutClosestSnapPointIdx != INDEX_NONE)
				{
					break;
				}
			}
		}

		if (OutClosestSnapPointIdx < (InSnapPointLinks.Num() - 1))
		{
			// Check higher points
			for (int32 SnapPointIterIdx = OutClosestSnapPointIdx + 1; SnapPointIterIdx < InSnapPointLinks.Num(); ++SnapPointIterIdx)
			{
				const float NewDistance = FMath::Abs(InOutComponent - InSnapPointLinks[SnapPointIterIdx].Position);

				if (OutDistanceFromSnap < 0)
				{
					OutDistanceFromSnap = NewDistance;
					OutClosestSnapPointIdx = SnapPointIterIdx;
					continue;
				}

				// Lower snap indices have priority (only assign the snap index if the distance is lower (current index is higher))
				if (NewDistance < OutDistanceFromSnap)
				{
					OutDistanceFromSnap = NewDistance;
					OutClosestSnapPointIdx = SnapPointIterIdx;
					continue;
				}

				if (NewDistance == OutDistanceFromSnap)
				{
					continue;
				}

				if (OutClosestSnapPointIdx != INDEX_NONE)
				{
					break;
				}
			}
		}
	}

	if (EnumHasAnyFlags(GetSnapState(), EAvaViewportSnapState::Grid))
	{
		const double GridSize = GetGridSize();

		// Grid size will be below 0 if it's invalid. And 0 isn't really valid either.
		if (GridSize > 0)
		{
			const double GridPosition = FMath::RoundToDouble((InOutComponent - InViewportSize / 2.f) / GridSize) * GridSize + InViewportSize / 2.f;
			const double GridLineDistance = FMath::Abs(GridPosition - InOutComponent);

			if (GridLineDistance <= MaximumSnapDistance && (OutDistanceFromSnap < 0 || GridLineDistance < OutDistanceFromSnap))
			{
				InOutComponent = GridPosition;
				OutDistanceFromSnap = GridLineDistance;
				return true;
			}
		}
	}

	if (InSnapPointLinks.IsValidIndex(OutClosestSnapPointIdx)
		&& FMath::Abs(InOutComponent - InSnapPointLinks[OutClosestSnapPointIdx].Position) < MaximumSnapDistance)
	{
		InOutComponent = InSnapPointLinks[OutClosestSnapPointIdx].Position;
		return true;
	}

	OutDistanceFromSnap = -1;
	return false;
}

void FAvaSnapOperation::SnapScreenLocation(FVector2f& InOutScreenLocation, bool bInSnapX, bool bInSnapY)
{
	using namespace UE::AvaViewport::Private;

	bSnappedToLinkX = false;
	bSnappedToLinkY = false;
	ClosestSnapPointLinkIdxX = INDEX_NONE;
	ClosestSnapPointLinkIdxY = INDEX_NONE;

	if (!IsAnySnapEnabled())
	{
		return;
	}

	TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient);

	if (!AvaViewportClient.IsValid())
	{
		return;
	}

	const FVector2f ViewportSize = AvaViewportClient->GetViewportSize();

	if (!FAvaViewportUtils::IsValidViewportSize(ViewportSize))
	{
		return;
	}

	const EAxisList::Type CurrentAxis = EditorViewportClient->GetCurrentWidgetAxis();

	if (bInSnapX && (CurrentAxis & EAxisList::Y || CurrentAxis & EAxisList::Screen))
	{
		bSnappedToLinkX = SnapX(InOutScreenLocation.X, ViewportSize.X, ClosestSnapPointLinkIdxX, SnapDistances.X);
	}

	if (bInSnapY && (CurrentAxis & EAxisList::Z || CurrentAxis & EAxisList::Screen))
	{
		bSnappedToLinkY = SnapY(InOutScreenLocation.Y, ViewportSize.Y, ClosestSnapPointLinkIdxY, SnapDistances.Y);
	}
}

void FAvaSnapOperation::SnapScreenLocation(FVector2f& InOutScreenLocation)
{
	SnapScreenLocation(InOutScreenLocation, true, true);
}

void FAvaSnapOperation::SnapScreenLocationX(float& InOutScreenLocation)
{
	FVector2f ScreenLocation = FVector2f(InOutScreenLocation, 0.f);
	SnapScreenLocation(ScreenLocation, true, false);
	InOutScreenLocation = ScreenLocation.X;
	bSnappedToLinkY = false;
	SnappedToLocation.X = ScreenLocation.X;
}

void FAvaSnapOperation::SnapScreenLocationY(float& InOutScreenLocation)
{
	FVector2f ScreenLocation = FVector2f(0.f, InOutScreenLocation);
	SnapScreenLocation(ScreenLocation, false, true);
	InOutScreenLocation = ScreenLocation.Y;
	bSnappedToLinkX = false;
	SnappedToLocation.Y = ScreenLocation.Y;
}

bool FAvaSnapOperation::SnapLocation(FVector& Location)
{
	using namespace UE::AvaViewport::Private;

	bSnappedToLinkX = false;
	bSnappedToLinkY = false;

	if (!IsAnySnapEnabled())
	{
		return false;
	}

	TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient);

	if (!AvaViewportClient.IsValid())
	{
		return false;
	}


	FVector2f ScreenLocation;
	double Distance;
	AvaViewportClient->WorldPositionToViewportPosition(Location, ScreenLocation, Distance);

	const FVector2f ViewportSize = AvaViewportClient->GetViewportSize();
	FVector2f SnappedScreenLocation = ScreenLocation;

	// Compare screen anchors to snap points
	struct SnapPointDistance
	{
		int32 ClosestSnapPointIdx;
		bool bSnappedToPoint;
		float SnapDistance;
	};

	float SnapDistanceX;
	float SnapDistanceY;
	int32 AnchorClosestSnappedToPointIdxXLocal;
	int32 AnchorClosestSnappedToPointIdxYLocal;

	SnapX(SnappedScreenLocation.X, ViewportSize.X, AnchorClosestSnappedToPointIdxXLocal, SnapDistanceX);
	SnapY(SnappedScreenLocation.Y, ViewportSize.Y, AnchorClosestSnappedToPointIdxYLocal, SnapDistanceY);

	bSnappedToLinkX = (AnchorClosestSnappedToPointIdxXLocal != INDEX_NONE && SnapDistanceX >= 0.f && SnapDistanceX <= FAvaSnapOperation::MaximumSnapDistance);
	bSnappedToLinkY = (AnchorClosestSnappedToPointIdxYLocal != INDEX_NONE && SnapDistanceY >= 0.f && SnapDistanceY <= FAvaSnapOperation::MaximumSnapDistance);

	// Find world point max/min based on snap anchor
	FVector Offset = FVector::ZeroVector;

	if (bSnappedToLinkX || bSnappedToLinkY)
	{
		const FTransform ViewTransform = FTransform(CachedSceneView->ViewRotation, CachedSceneView->ViewLocation, FVector::OneVector);
		const FAvaVisibleArea VisibleArea = AvaViewportClient->GetZoomedVisibleArea();
		const float VisibleAreaFraction = VisibleArea.GetVisibleAreaFraction();
		const FVector ScreenWorldSpaceSnapLocation = ViewTransform.InverseTransformPosition(Location);
		const FVector2D FrustumSize = AvaViewportClient->GetZoomedFrustumSizeAtDistance(ScreenWorldSpaceSnapLocation.X);

		if (bSnappedToLinkX)
		{
			// Calculate offset (difference between snapped and non-snapped position)
			Offset.Y = SnappedScreenLocation.X - ScreenLocation.X;

			// Scale size down to offset at the given distance from the camera (closer points are moved less)
			Offset.Y *= FrustumSize.X / ViewportSize.X / VisibleAreaFraction;

			// Set the snap location
			SnappedToLocation.X = SnappedScreenLocation.X;
		}

		if (bSnappedToLinkY)
		{
			// Calculate offset (difference between snapped and non-snapped position)
			Offset.Z = SnappedScreenLocation.Y - ScreenLocation.Y;

			// Scale size down to offset at the given distance from the camera (closer points are moved less)
			Offset.Z *= FrustumSize.Y / ViewportSize.Y / VisibleAreaFraction;
			Offset.Z *= -1.f;

			// Set the snap location
			SnappedToLocation.Y = SnappedScreenLocation.Y;
		}

		// Update new location with offset
		Offset = ViewTransform.TransformVector(Offset);
		Location += Offset;
	}

	return true;
}

bool FAvaSnapOperation::SnapDragLocation(const TArray<FAvaSnapPoint>& InDraggedActorSnapPoints, FVector& OutSnapOffset)
{
	/*
	 * Strategy:
	 * - Generate list of world-space snap points
	 * - Convert snap points to screen space
	 * - Snap each point
	 * - Use closest snapped distance to calculate the distance moved in world space (distance changes depending on
	 *     distance from the camera.)
	 * - Move actor
	 */

	using namespace UE::AvaViewport::Private;

	bSnappedToLinkX = false;
	bSnappedToLinkY = false;
	OutSnapOffset = FVector::ZeroVector;

	TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient);

	if (!AvaViewportClient.IsValid())
	{
		return false;
	}

	const EAxisList::Type WidgetAxes = EditorViewportClient->GetCurrentWidgetAxis();
	const FMatrix WidgetTransform = EditorViewportClient->GetWidgetCoordSystem();
	const bool bScreenSpace = WidgetAxes & EAxisList::Screen;
	const bool bIsDraggingX = WidgetAxes & EAxisList::X;
	const bool bIsDraggingY = WidgetAxes & EAxisList::Y;
	const bool bIsDraggingZ = WidgetAxes & EAxisList::Z;
	const bool bIsFilteredMove = !bIsDraggingX || !bIsDraggingY || !bIsDraggingZ;
	bool bSnapX = true;
	bool bSnapY = true;

	if (!bScreenSpace)
	{
		if (bIsFilteredMove)
		{
			const FTransform& CameraTransform = AvaViewportClient->GetViewportViewTransform();

			const FVector CameraYInWorldSpace = CameraTransform.TransformPositionNoScale(FVector::RightVector);
			const FVector CameraZInWorldSpace = CameraTransform.TransformPositionNoScale(FVector::UpVector);

			if (bIsDraggingX)
			{
				const FVector WidgetXInWorldSpace = WidgetTransform.TransformVector(FVector::ForwardVector);

				if (!FMath::IsNearlyZero(WidgetXInWorldSpace.Dot(CameraYInWorldSpace)))
				{
					bSnapX = true;
				}

				if (!FMath::IsNearlyZero(WidgetXInWorldSpace.Dot(CameraZInWorldSpace)))
				{
					bSnapY = true;
				}
			}

			if (bIsDraggingY && (!bSnapX || !bSnapY))
			{
				const FVector WidgetYInWorldSpace = WidgetTransform.TransformVector(FVector::RightVector);

				if (!FMath::IsNearlyZero(WidgetYInWorldSpace.Dot(CameraYInWorldSpace)))
				{
					bSnapX = true;
				}

				if (!FMath::IsNearlyZero(WidgetYInWorldSpace.Dot(CameraZInWorldSpace)))
				{
					bSnapY = true;
				}
			}

			if (bIsDraggingZ && (!bSnapX || !bSnapY))
			{
				const FVector WidgetZInWorldSpace = WidgetTransform.TransformVector(FVector::UpVector);

				if (!FMath::IsNearlyZero(WidgetZInWorldSpace.Dot(CameraYInWorldSpace)))
				{
					bSnapX = true;
				}

				if (!FMath::IsNearlyZero(WidgetZInWorldSpace.Dot(CameraZInWorldSpace)))
				{
					bSnapY = true;
				}
			}
		}
	}

	if (!bSnapX && !bSnapY)
	{
		return false;
	}

	if (!IsAnySnapEnabled())
	{
		return false;
	}

	// Generated on start of tracking
	if (InDraggedActorSnapPoints.Num() == 0)
	{
		return false;
	}

	const FVector2f ViewportSize = AvaViewportClient->GetViewportSize();
	const FVector DragOffset = GetDragOffset();
	
	TArray<FVector> DraggedActorsWorldSpaceSnapLocations;
	DraggedActorsWorldSpaceSnapLocations.Reserve(InDraggedActorSnapPoints.Num());
						
	// Convert snap positions to screen orientation
	for (const FAvaSnapPoint& SnapPoint : InDraggedActorSnapPoints)
	{
		FVector DraggedActorsWorldSpaceSnapLocation = SnapPoint.Location;

		// Selection position
		if (SnapPoint.Outer.IsExplicitlyNull())
		{
			DraggedActorsWorldSpaceSnapLocation += DragOffset;
		}
		else if (SnapPoint.Outer.IsValid())
		{
			if (SnapPoint.Outer->IsA<AActor>())
			{
				DraggedActorsWorldSpaceSnapLocation = Cast<const AActor>(SnapPoint.Outer)->GetRootComponent()->GetComponentTransform().TransformPositionNoScale(
					DraggedActorsWorldSpaceSnapLocation);
			}
			else if (SnapPoint.Outer->IsA<UPrimitiveComponent>())
			{
				DraggedActorsWorldSpaceSnapLocation = Cast<const UPrimitiveComponent>(SnapPoint.Outer)->GetComponentTransform().TransformPositionNoScale(
					DraggedActorsWorldSpaceSnapLocation);
			}
			else
			{
				checkNoEntry();
			}
		}
		else
		{
			continue;
		}

		// Convert from actor local to world space
		DraggedActorsWorldSpaceSnapLocations.Add(DraggedActorsWorldSpaceSnapLocation);
	}

	TArray<FVector2f> DraggedActorsScreenSpaceSnapLocations_Current;
	DraggedActorsScreenSpaceSnapLocations_Current.Reserve(InDraggedActorSnapPoints.Num());

	// Further convert world space snap points into screen space
	for (FVector& WorldSnapLocation : DraggedActorsWorldSpaceSnapLocations)
	{
		double Distance;
		FVector2f ScreenLocation;
		AvaViewportClient->WorldPositionToViewportPosition(WorldSnapLocation, ScreenLocation, Distance);

		DraggedActorsScreenSpaceSnapLocations_Current.Add(ScreenLocation);
	}

	TArray<FVector2f> DraggedActorsScreenSpaceSnapLocations_Snapped(DraggedActorsScreenSpaceSnapLocations_Current);

	// Compare screen anchors to snap points
	struct SnapPointDistance
	{
		int32 ClosestSnapPointIdx;
		bool bSnappedToPoint;
		double SnapDistance;
	};

	int32 ClosestActorScreenSnapPointIdxX = INDEX_NONE;
	int32 ClosestActorScreenSnapPointIdxY = INDEX_NONE;
	float ClosestActorScreenSnapPointDistanceX = -1;
	float ClosestActorScreenSnapPointDistanceY = -1;
	float SnapDistance;
	TArray<bool> bWasAnchorSnappedX;
	TArray<bool> bWasAnchorSnappedY;
	TArray<int32> AnchorClosestSnappedToPointIdxX;
	TArray<int32> AnchorClosestSnappedToPointIdxY;

	bWasAnchorSnappedX.SetNum(InDraggedActorSnapPoints.Num());
	bWasAnchorSnappedY.SetNum(InDraggedActorSnapPoints.Num());
	AnchorClosestSnappedToPointIdxX.SetNum(InDraggedActorSnapPoints.Num());
	AnchorClosestSnappedToPointIdxY.SetNum(InDraggedActorSnapPoints.Num());

	// Calculate snap distances
	for (int32 ActorScreenSnapPointIdx = 0; ActorScreenSnapPointIdx < DraggedActorsScreenSpaceSnapLocations_Snapped.Num(); ++ActorScreenSnapPointIdx)
	{
		if (bSnapX)
		{
			bWasAnchorSnappedX[ActorScreenSnapPointIdx] = SnapX(DraggedActorsScreenSpaceSnapLocations_Snapped[ActorScreenSnapPointIdx].X,
				ViewportSize.X, AnchorClosestSnappedToPointIdxX[ActorScreenSnapPointIdx], SnapDistance);

			if (SnapDistance >= 0 && (ClosestActorScreenSnapPointDistanceX < 0 || SnapDistance < ClosestActorScreenSnapPointDistanceX))
			{
				ClosestActorScreenSnapPointIdxX = ActorScreenSnapPointIdx;
				ClosestActorScreenSnapPointDistanceX = SnapDistance;
			}
		}

		if (bSnapY)
		{
			bWasAnchorSnappedY[ActorScreenSnapPointIdx] = SnapY(DraggedActorsScreenSpaceSnapLocations_Snapped[ActorScreenSnapPointIdx].Y,
				ViewportSize.Y, AnchorClosestSnappedToPointIdxY[ActorScreenSnapPointIdx], SnapDistance);

			if (SnapDistance >= 0 && (ClosestActorScreenSnapPointDistanceY < 0 || SnapDistance < ClosestActorScreenSnapPointDistanceY))
			{
				ClosestActorScreenSnapPointIdxY = ActorScreenSnapPointIdx;
				ClosestActorScreenSnapPointDistanceY = SnapDistance;
			}
		}
	}

	bSnappedToLinkX = (ClosestActorScreenSnapPointIdxX != INDEX_NONE && ClosestActorScreenSnapPointDistanceX >= 0 && ClosestActorScreenSnapPointDistanceX <= FAvaSnapOperation::MaximumSnapDistance);
	bSnappedToLinkY = (ClosestActorScreenSnapPointIdxY != INDEX_NONE && ClosestActorScreenSnapPointDistanceY >= 0 && ClosestActorScreenSnapPointDistanceY <= FAvaSnapOperation::MaximumSnapDistance);

	// Find world point max/min based on snap anchor
	if (bSnappedToLinkX || bSnappedToLinkY)
	{
		const FTransform ViewTransform = FTransform(CachedSceneView->ViewRotation, CachedSceneView->ViewLocation, FVector::OneVector);
		FAvaVisibleArea VisibleArea = AvaViewportClient->GetZoomedVisibleArea();
		const float VisibleAreaFraction = VisibleArea.GetVisibleAreaFraction();

		// Unrestricted movement
		if (bScreenSpace || !bIsFilteredMove)
		{
			if (bSnappedToLinkX)
			{
				UnfilteredDragSnap(ViewTransform, AvaViewportClient.ToSharedRef(), ViewportSize, VisibleAreaFraction, OutSnapOffset, SnappedToLocation,
					DraggedActorsWorldSpaceSnapLocations[ClosestActorScreenSnapPointIdxX],
					DraggedActorsScreenSpaceSnapLocations_Current[ClosestActorScreenSnapPointIdxX],
					DraggedActorsScreenSpaceSnapLocations_Snapped[ClosestActorScreenSnapPointIdxX],
					0, 1, 1.0
				);
			}

			if (bSnappedToLinkY)
			{
				UnfilteredDragSnap(ViewTransform, AvaViewportClient.ToSharedRef(), ViewportSize, VisibleAreaFraction, OutSnapOffset, SnappedToLocation,
					DraggedActorsWorldSpaceSnapLocations[ClosestActorScreenSnapPointIdxY],
					DraggedActorsScreenSpaceSnapLocations_Current[ClosestActorScreenSnapPointIdxY],
					DraggedActorsScreenSpaceSnapLocations_Snapped[ClosestActorScreenSnapPointIdxY],
					1, 2, -1.0
				);
			}

			// Update new location with offset
			OutSnapOffset = ViewTransform.TransformVector(OutSnapOffset);
		}
		// If we're using a filtered drag, we need to use ray/plane or plane/plane intersection
		else
		{
			const int32 DraggedAxisCount = (bIsDraggingX * 1) + (bIsDraggingY * 1) + (bIsDraggingZ * 1);

			FVector DragWorldSpaceOffset1;
			FVector DragWorldSpaceOffset2;

			FVector SnapOffsetX = FVector::ZeroVector;
			FVector SnapOffsetY = FVector::ZeroVector;

			// Single axis movement
			if (DraggedAxisCount == 1)
			{
				FVector DragDirection = FVector::ZeroVector;

				if (bIsDraggingX)
				{
					DragDirection = WidgetTransform.TransformVector(FVector::ForwardVector);
				}
				else if (bIsDraggingY)
				{
					DragDirection = WidgetTransform.TransformVector(FVector::RightVector);
				}
				else if (bIsDraggingZ)
				{
					DragDirection = WidgetTransform.TransformVector(FVector::UpVector);
				}

				DragWorldSpaceOffset1 = DragWorldSpaceOffset2 = DragDirection;

				if (bSnappedToLinkX)
				{
					SingleAxisDragSnap(AvaViewportClient.ToSharedRef(), 
						DraggedActorsScreenSpaceSnapLocations_Snapped[ClosestActorScreenSnapPointIdxX],
						FVector2f(0.f, 1.f), 
						DraggedActorsWorldSpaceSnapLocations[ClosestActorScreenSnapPointIdxX], 
						bSnappedToLinkX, DragDirection, SnapOffsetX);
				}

				if (bSnappedToLinkY)
				{
					SingleAxisDragSnap(AvaViewportClient.ToSharedRef(),
						DraggedActorsScreenSpaceSnapLocations_Snapped[ClosestActorScreenSnapPointIdxY],
						FVector2f(1.f, 0.f),
						DraggedActorsWorldSpaceSnapLocations[ClosestActorScreenSnapPointIdxY],
						bSnappedToLinkY, DragDirection, SnapOffsetY);
				}
			}
			// Double axis movement
			else if (DraggedAxisCount == 2)
			{
				if (!bIsDraggingX)
				{
					DragWorldSpaceOffset1 = WidgetTransform.TransformVector(FVector::RightVector);
					DragWorldSpaceOffset2 = WidgetTransform.TransformVector(FVector::UpVector);
				}
				else if (!bIsDraggingY)
				{
					DragWorldSpaceOffset1 = WidgetTransform.TransformVector(FVector::ForwardVector);
					DragWorldSpaceOffset2 = WidgetTransform.TransformVector(FVector::UpVector);
				}
				else // !bIsDraggingZ
				{
					DragWorldSpaceOffset1 = WidgetTransform.TransformVector(FVector::RightVector);
					DragWorldSpaceOffset2 = WidgetTransform.TransformVector(FVector::ForwardVector);
				}

				if (bSnappedToLinkX)
				{
					DoubleAxisDragSnap(AvaViewportClient.ToSharedRef(),
						DraggedActorsScreenSpaceSnapLocations_Snapped[ClosestActorScreenSnapPointIdxX],
						FVector2f(0.f, 1.f),
						DraggedActorsWorldSpaceSnapLocations[ClosestActorScreenSnapPointIdxX],
						bSnappedToLinkX, DragWorldSpaceOffset1, DragWorldSpaceOffset2, SnapOffsetX);
				}

				if (bSnappedToLinkY)
				{
					DoubleAxisDragSnap(AvaViewportClient.ToSharedRef(),
						DraggedActorsScreenSpaceSnapLocations_Snapped[ClosestActorScreenSnapPointIdxY],
						FVector2f(1.f, 0.f),
						DraggedActorsWorldSpaceSnapLocations[ClosestActorScreenSnapPointIdxY],
						bSnappedToLinkY, DragWorldSpaceOffset1, DragWorldSpaceOffset2, SnapOffsetY);
				}
			}

			if (bSnappedToLinkX && bSnappedToLinkY)
			{
				// We cannot snap to multiple axes and preserve filtered drag movement, so pick the nearest one...
				// Unless they perfectly align with the camera axes.
				const double SnapDistanceX = SnapOffsetX.SizeSquared();
				const double SnapDistanceY = SnapOffsetY.SizeSquared();

				const FVector CameraUp = ViewTransform.TransformVectorNoScale(FVector::UpVector);
				const FVector CameraRight = ViewTransform.TransformVectorNoScale(FVector::RightVector);

				const float CameraUpDotDragOffset1 = CameraUp.Dot(DragWorldSpaceOffset1);
				const float CameraUpDotDragOffset2 = CameraUp.Dot(DragWorldSpaceOffset2);
				const float CameraRightDotDragOffset1 = CameraRight.Dot(DragWorldSpaceOffset1);
				const float CameraRightDotDragOffset2 = CameraRight.Dot(DragWorldSpaceOffset2);

				const bool bCameraUpHasAlignedAxis = FMath::IsNearlyEqual(FMath::Abs(CameraUpDotDragOffset1), 1.0)
					|| FMath::IsNearlyEqual(FMath::Abs(CameraUpDotDragOffset2), 1.0);

				const bool bCameraRightHasAlignedAxis = FMath::IsNearlyEqual(FMath::Abs(CameraRightDotDragOffset1), 1.0)
					|| FMath::IsNearlyEqual(FMath::Abs(CameraRightDotDragOffset2), 1.0);

				if (bCameraUpHasAlignedAxis && bCameraRightHasAlignedAxis)
				{
					OutSnapOffset = SnapOffsetX + SnapOffsetY;
					SnappedToLocation.X = DraggedActorsScreenSpaceSnapLocations_Snapped[ClosestActorScreenSnapPointIdxX].X;
					SnappedToLocation.Y = DraggedActorsScreenSpaceSnapLocations_Snapped[ClosestActorScreenSnapPointIdxY].Y;
				}
				else if (SnapDistanceX <= SnapDistanceY)
				{
					OutSnapOffset = SnapOffsetX;
					SnappedToLocation.X = DraggedActorsScreenSpaceSnapLocations_Snapped[ClosestActorScreenSnapPointIdxX].X;
					bSnappedToLinkY = false;
				}
				else
				{
					OutSnapOffset = SnapOffsetY;
					SnappedToLocation.Y = DraggedActorsScreenSpaceSnapLocations_Snapped[ClosestActorScreenSnapPointIdxY].Y;
					bSnappedToLinkX = false;
				}
			}
			else if (bSnappedToLinkX)
			{
				OutSnapOffset = SnapOffsetX;
				SnappedToLocation.X = DraggedActorsScreenSpaceSnapLocations_Snapped[ClosestActorScreenSnapPointIdxX].X;
			}
			else if (bSnappedToLinkY)
			{
				OutSnapOffset = SnapOffsetY;
				SnappedToLocation.Y = DraggedActorsScreenSpaceSnapLocations_Snapped[ClosestActorScreenSnapPointIdxY].Y;
			}
		}
	}

	return true;
}

const FAvaSnapPoint* const FAvaSnapOperation::GetScreenSnapPoint(int32 InScreenSnapPointIdx) const
{
	if (ScreenSnapPoints.IsValidIndex(InScreenSnapPointIdx))
	{
		return &ScreenSnapPoints[InScreenSnapPointIdx];
	}

	return nullptr;
}

const FAvaSnapPoint* const FAvaSnapOperation::GetActorSnapPoint(int32 InActorSnapPointIdx) const
{
	if (ActorSnapPoints.IsValidIndex(InActorSnapPointIdx))
	{
		return &ActorSnapPoints[InActorSnapPointIdx];
	}

	return nullptr;
}

const FAvaSnapPoint* const FAvaSnapOperation::GetSnapPointByLink(int32 InSnapPointLinkIdx) const
{
	if (InSnapPointLinkIdx < ScreenSnapPoints.Num())
	{
		return GetScreenSnapPoint(InSnapPointLinkIdx);
	}

	return GetActorSnapPoint(InSnapPointLinkIdx - ScreenSnapPoints.Num());
}

void FAvaSnapOperation::SetSnappedToX(bool bInSnappedTo)
{
	bSnappedToLinkX = bInSnappedTo;
}

void FAvaSnapOperation::SetSnappedToY(bool bInSnappedTo)
{
	bSnappedToLinkY = bInSnappedTo;
}

FVector FAvaSnapOperation::GetSelectionLocation() const
{
	if (TSharedPtr<IAvaViewportClient> AvaViewportClient = FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient))
	{
		if (FEditorModeTools* ModeTools = EditorViewportClient->GetModeTools())
		{
			return ModeTools->PivotLocation;
		}
	}

	return FVector::ZeroVector;
}

FVector FAvaSnapOperation::GetDragOffset() const
{
	return GetSelectionLocation() - SelectionStartLocation;
}
