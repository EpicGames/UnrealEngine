// Copyright Epic Games, Inc. All Rights Reserved.

#include "Test/AvaViewportWorldCoordinateConverterProvider.h"
#include "AvaViewportUtils.h"
#include "Math/Vector.h"
#include "Math/Rotator.h"

FAvaViewportWorldCoordinateConverterProvider::FAvaViewportWorldCoordinateConverterProvider(const FTransform& InViewTransform,
	const FVector2f& InViewportSize)
	: ViewTransform(InViewTransform)
	, ViewportSize(InViewportSize)
{
}

FAvaViewportWorldCoordinateConverterProviderPerspective::FAvaViewportWorldCoordinateConverterProviderPerspective(FVector InLocation,
	FRotator InRotation, FVector2f InViewportSize, float InFieldOfView)
	: FAvaViewportWorldCoordinateConverterProvider(FTransform(InRotation, InLocation, FVector::OneVector), InViewportSize)
	, FieldOfView(InFieldOfView)
{
}

FVector2D FAvaViewportWorldCoordinateConverterProviderPerspective::GetFrustumSizeAtDistance(double InDistance) const
{
	if (!FAvaViewportUtils::IsValidViewportSize(ViewportSize))
	{
		return FVector2D::ZeroVector;
	}

	return FAvaViewportUtils::GetFrustumSizeAtDistance(
		FieldOfView,
		ViewportSize.X / ViewportSize.Y,
		InDistance
	);
}

FVector FAvaViewportWorldCoordinateConverterProviderPerspective::ViewportPositionToWorldPosition(const FVector2f& InViewportPosition, 
	double InDistance) const
{
	if (!FAvaViewportUtils::IsValidViewportSize(ViewportSize))
	{
		return FVector::ZeroVector;
	}

	return FAvaViewportUtils::ViewportPositionToWorldPositionPerspective(
		ViewportSize,
		InViewportPosition,
		ViewTransform.GetLocation(),
		ViewTransform.GetRotation().Rotator(),
		FieldOfView,
		InDistance
	);
}

void FAvaViewportWorldCoordinateConverterProviderPerspective::WorldPositionToViewportPosition(const FVector& InWorldPosition, 
	FVector2f& OutViewportPosition, double& OutDistance) const
{
	if (!FAvaViewportUtils::IsValidViewportSize(ViewportSize))
	{
		return;
	}

	OutViewportPosition = FVector2f::ZeroVector;
	OutDistance = 0;

	return FAvaViewportUtils::WorldPositionToViewportPositionPerspective(
		ViewportSize,
		InWorldPosition,
		ViewTransform.GetLocation(),
		ViewTransform.GetRotation().Rotator(),
		FieldOfView,
		OutViewportPosition,
		OutDistance
	);
}

FAvaViewportWorldCoordinateConverterProviderOrthographic::FAvaViewportWorldCoordinateConverterProviderOrthographic(FVector InLocation,
	FRotator InRotation, FVector2f InViewportSize, float InOrthographicWidth)
	: FAvaViewportWorldCoordinateConverterProvider(FTransform(InRotation, InLocation, FVector::OneVector), InViewportSize)
	, OrthographicWidth(InOrthographicWidth)
{
}

FVector2D FAvaViewportWorldCoordinateConverterProviderOrthographic::GetFrustumSizeAtDistance(double InDistance) const
{
	if (!FAvaViewportUtils::IsValidViewportSize(ViewportSize))
	{
		return FVector2D::ZeroVector;
	}

	const float InverseAspectRatio = ViewportSize.Y / ViewportSize.X;

	return {
		OrthographicWidth,
		OrthographicWidth * InverseAspectRatio
	};
}

FVector FAvaViewportWorldCoordinateConverterProviderOrthographic::ViewportPositionToWorldPosition(const FVector2f& InViewportPosition, 
	double InDistance) const
{
	if (!FAvaViewportUtils::IsValidViewportSize(ViewportSize))
	{
		return FVector::ZeroVector;
	}

	return FAvaViewportUtils::ViewportPositionToWorldPositionOrthographic(
		ViewportSize,
		InViewportPosition,
		ViewTransform.GetLocation(),
		ViewTransform.GetRotation().Rotator(),
		OrthographicWidth,
		InDistance
	);
}

void FAvaViewportWorldCoordinateConverterProviderOrthographic::WorldPositionToViewportPosition(const FVector& InWorldPosition, 
	FVector2f& OutViewportPosition, double& OutDistance) const
{
	if (!FAvaViewportUtils::IsValidViewportSize(ViewportSize))
	{
		return;
	}

	OutViewportPosition = FVector2f::ZeroVector;
	OutDistance = 0;

	return FAvaViewportUtils::WorldPositionToViewportPositionOrthographic(
		ViewportSize,
		InWorldPosition,
		ViewTransform.GetLocation(),
		ViewTransform.GetRotation().Rotator(),
		OrthographicWidth,
		OutViewportPosition,
		OutDistance
	);
}
