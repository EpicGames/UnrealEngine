// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportClient/AvaViewportClientUtilityProvider.h"
#include "AvaViewportUtils.h"
#include "Camera/CameraComponent.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"

FVector2f FAvaViewportClientUtilityProvider::GetViewportWidgetSize() const
{
	return GetViewportSize() + 2.f * GetViewportOffset();
}

FVector2f FAvaViewportClientUtilityProvider::GetVirtualViewportScale() const
{
	const FVector2f ViewportSize = GetViewportSize();

	if (!FAvaViewportUtils::IsValidViewportSize(ViewportSize))
	{
		return FVector2f(1.f, 1.f);
	}

	const FIntPoint VirtualViewportSize = GetVirtualViewportSize();

	if (!FAvaViewportUtils::IsValidViewportSize(VirtualViewportSize))
	{
		return FVector2f(1.f, 1.f);
	}

	return {
		static_cast<float>(VirtualViewportSize.X) / ViewportSize.X,
		static_cast<float>(VirtualViewportSize.Y) / ViewportSize.Y
	};
}

float FAvaViewportClientUtilityProvider::GetAverageVirtualViewportScale() const
{
	const FVector2f VirtualScale = GetVirtualViewportScale();

	return VirtualScale.X * 0.5f + VirtualScale.Y * 0.5f;
}

FVector2D FAvaViewportClientUtilityProvider::GetFrustumSizeAtDistance(double InDistance) const
{
	const FVector2f ViewportSize = GetViewportWidgetSize() - (GetViewportOffset() * 2.f);

	if (!FAvaViewportUtils::IsValidViewportSize(ViewportSize))
	{
		return FVector2D::ZeroVector;
	}

	return FAvaViewportUtils::GetFrustumSizeAtDistance(
		GetUnZoomedFOV(),
		ViewportSize.X / ViewportSize.Y,
		InDistance
	);
}

FVector2D FAvaViewportClientUtilityProvider::GetZoomedFrustumSizeAtDistance(double InDistance) const
{
	const FVector2f ViewportSize = GetViewportSize() - (GetViewportOffset() * 2.f);

	if (!FAvaViewportUtils::IsValidViewportSize(ViewportSize))
	{
		return FVector2D::ZeroVector;
	}

	return FAvaViewportUtils::GetFrustumSizeAtDistance(
		GetZoomedFOV(),
		ViewportSize.X / ViewportSize.Y,
		InDistance
	);
}

FVector FAvaViewportClientUtilityProvider::ViewportPositionToWorldPosition(const FVector2f& InViewportPosition, double InDistance) const
{
	const FEditorViewportClient* ViewportClient = AsEditorViewportClient();

	if (!ViewportClient)
	{
		return FVector::ZeroVector;
	}

	if (ViewportClient->IsOrtho())
	{
		if (!ViewportClient->Viewport)
		{
			return FVector::ZeroVector;
		}

		const FVector2f ViewportSize = GetViewportWidgetSize();

		if (!FAvaViewportUtils::IsValidViewportSize(ViewportSize))
		{
			return FVector::ZeroVector;
		}

		return FAvaViewportUtils::ViewportPositionToWorldPositionOrthographic(
			ViewportSize,
			InViewportPosition,
			ViewportClient->GetViewLocation(),
			ViewportClient->GetViewRotation(),
			ViewportSize.X * ViewportClient->GetOrthoUnitsPerPixel(ViewportClient->Viewport),
			InDistance
		);
	}
	else
	{
		const FVector2f ViewportSize = GetViewportSize();

		if (!FAvaViewportUtils::IsValidViewportSize(ViewportSize))
		{
			return FVector::ZeroVector;
		}

		return FAvaViewportUtils::ViewportPositionToWorldPositionPerspective(
			ViewportSize, 
			InViewportPosition, 
			ViewportClient->GetViewLocation(),
			ViewportClient->GetViewRotation(),
			GetUnZoomedFOV(),
			InDistance
		);
	}
}

void FAvaViewportClientUtilityProvider::WorldPositionToViewportPosition(const FVector& InWorldPosition, FVector2f& OutViewportPosition, double& OutDistance) const
{
	OutViewportPosition = FVector2f::ZeroVector;
	OutDistance = 0;

	const FEditorViewportClient* ViewportClient = AsEditorViewportClient();

	if (!ViewportClient)
	{
		return;
	}

	if (ViewportClient->IsOrtho())
	{
		if (!ViewportClient->Viewport)
		{
			return;
		}

		const FVector2f ViewportSize = GetViewportWidgetSize();

		if (!FAvaViewportUtils::IsValidViewportSize(ViewportSize))
		{
			return;
		}

		return FAvaViewportUtils::WorldPositionToViewportPositionOrthographic(
			ViewportSize,
			InWorldPosition,
			ViewportClient->GetViewLocation(),
			ViewportClient->GetViewRotation(),
			ViewportSize.X * ViewportClient->GetOrthoUnitsPerPixel(ViewportClient->Viewport),
			OutViewportPosition,
			OutDistance
		);
	}
	else
	{
		const FVector2f ViewportSize = GetViewportSize();

		if (!FAvaViewportUtils::IsValidViewportSize(ViewportSize))
		{
			return;
		}

		return FAvaViewportUtils::WorldPositionToViewportPositionPerspective(
			ViewportSize,
			InWorldPosition,
			ViewportClient->GetViewLocation(),
			ViewportClient->GetViewRotation(),
			GetUnZoomedFOV(),
			OutViewportPosition,
			OutDistance
		);
	}
}

void FAvaViewportClientUtilityProvider::OnCameraCut(AActor* InTarget, bool bInJumpCut)
{
	if (!IsValid(InTarget))
	{
		return;
	}

	SetViewTarget(InTarget);

	if (bInJumpCut)
	{
		NotifyJumpCut(InTarget);
	}
}

void FAvaViewportClientUtilityProvider::NotifyJumpCut(AActor* InViewTarget)
{
	if (!IsValid(InViewTarget))
	{
		return;
	}

	TArray<UCameraComponent*> CameraComponents;
	InViewTarget->GetComponents<UCameraComponent>(CameraComponents);

	for (UCameraComponent* CameraComponent : CameraComponents)
	{
		if (IsValid(CameraComponent))
		{
			CameraComponent->NotifyCameraCut();
		}
	}
}
