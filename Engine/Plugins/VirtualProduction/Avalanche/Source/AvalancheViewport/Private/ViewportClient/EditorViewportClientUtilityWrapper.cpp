// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewportClient/EditorViewportClientUtilityWrapper.h"
#include "AvaViewportUtils.h"
#include "AvaVisibleArea.h"
#include "EditorViewportClient.h"
#include "LevelEditor.h"
#include "LevelEditor/AvaLevelEditorUtils.h"
#include "Math/Vector2D.h"
#include "SLevelViewport.h"
#include "Templates/SharedPointer.h"

FEditorViewportClientUtilityWrapper::FEditorViewportClientUtilityWrapper(TSharedPtr<const FEditorViewportClient> InEditorViewportClient)
{
	check(InEditorViewportClient.IsValid());

	EditorViewportClientWeak = InEditorViewportClient;
}

float FEditorViewportClientUtilityWrapper::GetUnZoomedFOV() const
{
	const FEditorViewportClient* ViewportClient = AsEditorViewportClient();

	if (!ViewportClient)
	{
		return 0.f;
	}

	return ViewportClient->ViewFOV;
}

float FEditorViewportClientUtilityWrapper::GetZoomedFOV() const
{
	return GetUnZoomedFOV();
}

FVector2f FEditorViewportClientUtilityWrapper::GetViewportSize() const
{
	const FEditorViewportClient* ViewportClient = AsEditorViewportClient();

	if (!ViewportClient)
	{
		return FVector2f::ZeroVector;
	}

	FIntPoint ViewportSize = ViewportClient->Viewport->GetSizeXY();

	if (ViewportSize.X <= 0 || ViewportSize.Y <= 0)
	{
		if (TSharedPtr<SEditorViewport> EditorWidget = ViewportClient->GetEditorViewportWidget())
		{
			const FVector2f SizeF = EditorWidget->GetTickSpaceGeometry().GetAbsoluteSize();
			ViewportSize.X = SizeF.X;
			ViewportSize.Y = SizeF.Y;
		}
	}

	return {static_cast<float>(ViewportSize.X), static_cast<float>(ViewportSize.Y)};
}

FVector2f FEditorViewportClientUtilityWrapper::GetViewportOffset() const
{
	return FVector2f::ZeroVector;
}

FAvaVisibleArea FEditorViewportClientUtilityWrapper::GetVisibleArea() const
{
	return FAvaVisibleArea(GetViewportSize());
}

FTransform FEditorViewportClientUtilityWrapper::GetViewportViewTransform() const
{
	// Uses CalcSceneView which, for some reason, isn't const.
	return FAvaViewportUtils::GetViewportViewTransform(const_cast<FEditorViewportClient*>(AsEditorViewportClient()));
}

FIntPoint FEditorViewportClientUtilityWrapper::GetVirtualViewportSize() const
{
	FVector2f ViewportSize = GetViewportSize();
	return {FMath::RoundToInt(ViewportSize.X), FMath::RoundToInt(ViewportSize.Y)};
}

FVector2f FEditorViewportClientUtilityWrapper::GetViewportWidgetSize() const
{
	return GetViewportSize();
}

float FEditorViewportClientUtilityWrapper::GetViewportDPIScale() const
{
	// @TODO: Make this accurate
	return 1.f;
}

FAvaVisibleArea FEditorViewportClientUtilityWrapper::GetVirtualVisibleArea() const
{
	return GetVisibleArea();
}

FAvaVisibleArea FEditorViewportClientUtilityWrapper::GetZoomedVisibleArea() const
{
	return GetVisibleArea();
}

FAvaVisibleArea FEditorViewportClientUtilityWrapper::GetVirtualZoomedVisibleArea() const
{
	return GetVisibleArea();
}

FVector2f FEditorViewportClientUtilityWrapper::GetUnconstrainedViewportMousePosition() const
{
	const FEditorViewportClient* ViewportClient = AsEditorViewportClient();

	if (!ViewportClient)
	{
		return FVector2f::ZeroVector;
	}

	if (ViewportClient->Viewport)
	{
		return {
			static_cast<float>(ViewportClient->Viewport->GetMouseX()),
			static_cast<float>(ViewportClient->Viewport->GetMouseY())
		};
	}

	return {
		static_cast<float>(ViewportClient->GetCachedMouseX()),
		static_cast<float>(ViewportClient->GetCachedMouseY())
	};
}

FVector2f FEditorViewportClientUtilityWrapper::GetConstrainedViewportMousePosition() const
{
	return GetUnconstrainedViewportMousePosition();
}

FVector2f FEditorViewportClientUtilityWrapper::GetUnconstrainedZoomedViewportMousePosition() const
{
	return GetUnconstrainedViewportMousePosition();
}

FVector2f FEditorViewportClientUtilityWrapper::GetConstrainedZoomedViewportMousePosition() const
{
	return GetUnconstrainedViewportMousePosition();
}

UWorld* FEditorViewportClientUtilityWrapper::GetViewportWorld() const
{
	if (const FEditorViewportClient* ViewportClient = AsEditorViewportClient())
	{
		return ViewportClient->GetWorld();
	}

	return nullptr;
}

const FEditorViewportClient* FEditorViewportClientUtilityWrapper::AsEditorViewportClient() const
{
	return EditorViewportClientWeak.Pin().Get();
}

bool FEditorViewportClientUtilityWrapper::IsValidLevelEditorViewportClient(const FViewportClient* InViewportClient)
{
	if (!InViewportClient)
	{
		return false;
	}

	FLevelEditorModule* LevelEditorModule = FAvaLevelEditorUtils::GetLevelEditorModule();
	if (!LevelEditorModule)
	{
		return false;
	}

	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule->GetFirstLevelEditor();
	if (!LevelEditor.IsValid())
	{
		return false;
	}

	FEditorViewportClient* FocusedClient = LevelEditor->GetEditorModeManager().GetFocusedViewportClient();
	TArray<TSharedPtr<SLevelViewport>> Viewports = LevelEditor->GetViewports();

	for (const TSharedPtr<SLevelViewport>& Viewport : Viewports)
	{
		if (Viewport.IsValid())
		{
			if (Viewport->GetViewportClient().Get() == InViewportClient)
			{
				return true;
			}
		}
	}

	return false;
}

FEditorViewportClient* FEditorViewportClientUtilityWrapper::GetValidLevelEditorViewportClient(const FViewport* InViewport)
{
	if (!InViewport)
	{
		return nullptr;
	}

	FLevelEditorModule* LevelEditorModule = FAvaLevelEditorUtils::GetLevelEditorModule();
	if (!LevelEditorModule)
	{
		return nullptr;
	}

	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule->GetFirstLevelEditor();
	if (!LevelEditor.IsValid())
	{
		return nullptr;
	}

	FEditorViewportClient* FocusedClient = LevelEditor->GetEditorModeManager().GetFocusedViewportClient();
	TArray<TSharedPtr<SLevelViewport>> Viewports = LevelEditor->GetViewports();

	for (const TSharedPtr<SLevelViewport>& Viewport : Viewports)
	{
		if (Viewport.IsValid())
		{
			if (TSharedPtr<FEditorViewportClient> ViewportClient = Viewport->GetViewportClient())
			{
				if (ViewportClient->Viewport == InViewport)
				{
					return ViewportClient.Get();
				}
			}
		}
	}

	return nullptr;
}
