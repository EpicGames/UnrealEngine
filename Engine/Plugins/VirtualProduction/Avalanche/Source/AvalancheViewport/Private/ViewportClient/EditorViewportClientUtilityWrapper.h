// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportClient/AvaViewportClientUtilityProvider.h"
#include "AvaType.h"

class FViewport;
class FViewportClient;

class FEditorViewportClientUtilityWrapper : public FAvaViewportClientUtilityProvider
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FEditorViewportClientUtilityWrapper, FAvaViewportClientUtilityProvider)

	static bool IsValidLevelEditorViewportClient(const FViewportClient* InViewportClient);
	static FEditorViewportClient* GetValidLevelEditorViewportClient(const FViewport* InViewport);

	FEditorViewportClientUtilityWrapper(TSharedPtr<const FEditorViewportClient> InEditorViewportClient);

	/** Changed override default values to false because it doesn't support zoom */

	//~ Begin IAvaViewportWorldCoordinateConverter
	virtual FVector2f GetViewportSize() const override;
	virtual FTransform GetViewportViewTransform() const override;
	//~ End IAvaViewportWorldCoordinateConverter

	//~ Begin IAvaViewportClient
	virtual bool IsMotionDesignViewport() const override { return false; }
	virtual const FEditorViewportClient* AsEditorViewportClient() const override;
	virtual bool SupportsZoom() const override { return false; } // No support
	virtual float GetUnZoomedFOV() const override;
	virtual float GetZoomedFOV() const override;
	virtual FVector2f GetViewportOffset() const override;
	virtual FIntPoint GetVirtualViewportSize() const override;
	virtual FVector2f GetViewportWidgetSize() const override;
	virtual float GetViewportDPIScale() const override;
	virtual FAvaVisibleArea GetVirtualVisibleArea() const override;
	virtual FAvaVisibleArea GetZoomedVisibleArea() const override;
	virtual FAvaVisibleArea GetVirtualZoomedVisibleArea() const override;
	virtual FAvaVisibleArea GetVisibleArea() const override;
	virtual FVector2f GetUnconstrainedViewportMousePosition() const override;
	virtual FVector2f GetConstrainedViewportMousePosition() const override;
	virtual FVector2f GetUnconstrainedZoomedViewportMousePosition() const override;
	virtual FVector2f GetConstrainedZoomedViewportMousePosition() const override;
	virtual TSharedPtr<FAvaSnapOperation> GetSnapOperation() const override { return nullptr; } // No support
	virtual TSharedPtr<FAvaSnapOperation> StartSnapOperation() override { return nullptr; } // No support
	virtual bool EndSnapOperation(FAvaSnapOperation* InSnapOperation) override { return false; } // No support
	virtual void OnActorSelectionChanged() override {} // Nothing to do
	virtual TSharedPtr<FAvaCameraZoomController> GetZoomController() const override { return nullptr; } // No support
	virtual UCameraComponent* GetCameraComponentViewTarget() const override { return nullptr; } // No support
	virtual AActor* GetViewTarget() const override { return nullptr; } // No support
	virtual void SetViewTarget(TWeakObjectPtr<AActor> InViewTarget) override {} // No support
	virtual UWorld* GetViewportWorld() const;
	virtual TSharedPtr<FAvaViewportPostProcessManager> GetPostProcessManager() const { return nullptr; } // No support
	//~ End IAvaViewportClient

protected:
	TWeakPtr<const FEditorViewportClient> EditorViewportClientWeak;
};
