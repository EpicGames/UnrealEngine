// Copyright Epic Games, Inc. All Rights Reserved.

#include "Visualizers/AvaViewportBackgroundVisualizer.h"
#include "AvalancheViewportModule.h"
#include "AvaViewportPostProcessManager.h"
#include "AvaViewportSettings.h"
#include "AvaViewportUtils.h"
#include "AvaVisibleArea.h"
#include "Engine/Texture.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UObject/Package.h"
#include "ViewportClient/IAvaViewportClient.h"

#define LOCTEXT_NAMESPACE "AvaViewportBackgroundVisualizer"

namespace UE::AvaViewport::Private
{
	const FString BackgroundReferencerName = FString(TEXT("AvaViewportBackgroundVisualizer"));
	const FName TextureObjectName = FName(TEXT("TextureObject"));
	const FName TextureOffsetName = FName(TEXT("TextureOffset"));
	const FName TextureScaleName = FName(TEXT("TextureScale"));
}

FAvaViewportBackgroundVisualizer::FAvaViewportBackgroundVisualizer(TSharedRef<IAvaViewportClient> InAvaViewportClient)
	: FAvaViewportPostProcessVisualizer(InAvaViewportClient)
{
	bRequiresTonemapperSetting = true;

	TextureOffset = FVector::ZeroVector;
	TextureScale = FVector::ZeroVector;

	const UAvaViewportSettings* ViewportSettings = GetDefault<UAvaViewportSettings>();

	if (!ViewportSettings)
	{
		UE_LOGF(AvaViewportLog, Warning, "FAvaViewportBackgroundVisualizer::FAvaViewportBackgroundVisualizer: Unable to find viewport settings.");
		return;
	}

	UMaterial* BackgroundMaterial = ViewportSettings->ViewportBackgroundMaterial.LoadSynchronous();

	if (!BackgroundMaterial)
	{
		UE_LOGF(AvaViewportLog, Warning, "FAvaViewportBackgroundVisualizer::FAvaViewportBackgroundVisualizer: Unable to find background material.");
		return;
	}

	PostProcessBaseMaterial = BackgroundMaterial;
	PostProcessMaterial = UMaterialInstanceDynamic::Create(BackgroundMaterial, GetTransientPackage());
}

UTexture* FAvaViewportBackgroundVisualizer::GetTexture() const
{
	return Texture;
}

void FAvaViewportBackgroundVisualizer::SetTexture(UTexture* InTexture)
{
	if (Texture == InTexture)
	{
		return;
	}

	SetTextureInternal(InTexture);

	UpdatePostProcessInfo();
	UpdatePostProcessMaterial();
}

void FAvaViewportBackgroundVisualizer::AddReferencedObjects(FReferenceCollector& InCollector)
{
	Super::AddReferencedObjects(InCollector);

	if (Texture)
	{
		InCollector.AddReferencedObject(Texture);
	}
}

FString FAvaViewportBackgroundVisualizer::GetReferencerName() const
{
	return UE::AvaViewport::Private::BackgroundReferencerName;
}

void FAvaViewportBackgroundVisualizer::UpdateForViewport(const FAvaVisibleArea& InVisibleArea, const FVector2f& InVisibleAreaOffset, 
	const FVector2f& InWidgetSize, const FVector2f& InCameraOffset)
{
	if (FMath::IsNearlyZero(PostProcessOpacity) || !Texture || !PostProcessMaterial)
	{
		return;
	}

	if (!InVisibleArea.IsValid())
	{
		return;
	}

	if (!FAvaViewportUtils::IsValidViewportSize(InWidgetSize))
	{
		return;
	}

	const FVector2f ImageSize = {Texture->GetSurfaceWidth(), Texture->GetSurfaceHeight()};

	if (!FAvaViewportUtils::IsValidViewportSize(ImageSize))
	{
		return;
	}

	using namespace UE::AvaViewport::Private;

	const FVector2f WidgetBasedScale = InVisibleArea.AbsoluteSize / InWidgetSize;
	const float VisibleAreaFraction = InVisibleArea.GetVisibleAreaFraction();
	const FVector2f Scale = WidgetBasedScale / VisibleAreaFraction;

	if (!FMath::IsNearlyEqual(TextureScale.X, Scale.X)
		|| !FMath::IsNearlyEqual(TextureScale.Y, Scale.Y))
	{
		TextureScale.X = Scale.X;
		TextureScale.Y = Scale.Y;
		TextureOffset.Z = 0.f;
		PostProcessMaterial->SetVectorParameterValue(TextureScaleName, TextureScale);
	}

	const FVector2f TopLeft = (InVisibleArea.GetVisiblePosition(FVector2f::ZeroVector) + InVisibleAreaOffset) * InVisibleArea.DPIScale;

	if (!FMath::IsNearlyEqual(TextureOffset.X, TopLeft.X)
		|| !FMath::IsNearlyEqual(TextureOffset.Y, TopLeft.Y))
	{
		TextureOffset.X = TopLeft.X;
		TextureOffset.Y = TopLeft.Y;
		TextureOffset.Z = 0.f;
		PostProcessMaterial->SetVectorParameterValue(TextureOffsetName, TextureOffset);
	}
}

void FAvaViewportBackgroundVisualizer::LoadPostProcessInfo(const FAvaViewportPostProcessInfo& InPostProcessInfo)
{
	Super::LoadPostProcessInfo(InPostProcessInfo);

	SetTextureInternal(InPostProcessInfo.Texture.LoadSynchronous());
}

void FAvaViewportBackgroundVisualizer::UpdatePostProcessInfo(FAvaViewportPostProcessInfo& InPostProcessInfo) const
{
	Super::UpdatePostProcessInfo(InPostProcessInfo);

	InPostProcessInfo.Texture = Texture;
}

void FAvaViewportBackgroundVisualizer::UpdatePostProcessMaterial()
{
	if (!PostProcessMaterial)
	{
		UE_LOGF(AvaViewportLog, Warning, "FAvaViewportBackgroundVisualizer::UpdatePostProcessMaterial: Invalid post process material.");
		return;
	}

	Super::UpdatePostProcessMaterial();

	using namespace UE::AvaViewport::Private;

	PostProcessMaterial->SetTextureParameterValue(TextureObjectName, Texture);
}

bool FAvaViewportBackgroundVisualizer::SetupPostProcessSettings(FPostProcessSettings& InPostProcessSettings) const
{
	if (!IsValid(Texture))
	{
		return false;
	}

	return FAvaViewportPostProcessVisualizer::SetupPostProcessSettings(InPostProcessSettings);
}

void FAvaViewportBackgroundVisualizer::SetTextureInternal(UTexture* InTexture)
{
	if (!IsValid(InTexture))
	{
		Texture = nullptr;
	}
	else
	{
		Texture = InTexture;
	}
}

#undef LOCTEXT_NAMESPACE
