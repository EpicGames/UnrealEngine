// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Visualizers/AvaViewportPostProcessVisualizer.h"
#include "Math/Vector.h"

class UTexture;
struct FAvaViewportPostProcessInfo;

class FAvaViewportBackgroundVisualizer : public FAvaViewportPostProcessVisualizer
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaViewportBackgroundVisualizer, FAvaViewportPostProcessVisualizer)

	FAvaViewportBackgroundVisualizer(TSharedRef<IAvaViewportClient> InAvaViewportClient);
	virtual ~FAvaViewportBackgroundVisualizer() override = default;

	UTexture* GetTexture() const;
	void SetTexture(UTexture* InTexture);

	//~ Begin FAvaViewportPostProcessVisualizer
	virtual void UpdateForViewport(const FAvaVisibleArea& InVisibleArea, const FVector2f& InVisibleAreaOffset, 
		const FVector2f& InWidgetSize, const FVector2f& InCameraOffset) override;
	//~ End FAvaViewportPostProcessVisualizer

	//~ Begin FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	virtual FString GetReferencerName() const override;
	//~ End FGCObject

protected:
	TObjectPtr<UTexture> Texture;

	/** Offset from the top-left of the viewport where the texture should start in pixel space. */
	FVector TextureOffset;

	/** Scale of the texture. Larger values mean a larger texture. */
	FVector TextureScale;

	using FAvaViewportPostProcessVisualizer::UpdatePostProcessInfo;

	void SetTextureInternal(UTexture* InTexture);

	//~ Begin FAvaViewportPostProcessVisualizer
	virtual void LoadPostProcessInfo(const FAvaViewportPostProcessInfo& InPostProcessInfo) override;
	virtual void UpdatePostProcessInfo(FAvaViewportPostProcessInfo& InPostProcessInfo) const override;
	virtual void UpdatePostProcessMaterial() override;
	virtual bool SetupPostProcessSettings(FPostProcessSettings& InPostProcessSettings) const override;
	//~ End FAvaViewportPostProcessVisualizer
};