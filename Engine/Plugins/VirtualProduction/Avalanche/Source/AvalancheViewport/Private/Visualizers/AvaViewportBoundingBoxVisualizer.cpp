// Copyright Epic Games, Inc. All Rights Reserved.

#include "Visualizers/AvaViewportBoundingBoxVisualizer.h"
#include "AvalancheViewportModule.h"
#include "AvaViewportSettings.h"
#include "Bounds/AvaBoundsProviderSubsystem.h"
#include "Containers/Array.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformTime.h"
#include "Math/Color.h"
#include "PrimitiveDrawingUtils.h"
#include "Selection.h"
#include "Selection/AvaSelectionProviderSubsystem.h"

namespace UE::AvaViewport::Private
{
	constexpr float BoundingBoxLineThickness = 0.4f;
}

TSharedRef<IAvaViewportBoundingBoxVisualizer> FAvaViewportBoundingBoxVisualizerProvider::CreateVisualizer()
{
	return MakeShared<FAvaLevelViewportBoundingBoxVisualizer>();
}

namespace UE::AvaLevelViewport::Private
{
	struct FAvaTaskTimer
	{
		FAvaTaskTimer()
		{
			StartTime = FPlatformTime::Seconds();
		}

		double GetTaskLength() const
		{
			return FPlatformTime::Seconds() - StartTime;
		}

	protected:
		double StartTime;
	};

	constexpr FLinearColor ActorBoundsColor = FLinearColor(1.0, 0.0, 0, 1);
	constexpr FLinearColor ActorAndChildrenBoundsColor = FLinearColor(1.0, 0.5, 0, 1);
	constexpr FLinearColor SelectionBoundsColor = FLinearColor(1.0, 1.0, 0, 1);

	constexpr int32 AxisX = 0;
	constexpr int32 AxisY = 1;
	constexpr int32 AxisZ = 2;

	constexpr int32 CubeLines[12][2] = {
		{0, 1}, {0, 2}, {3, 1}, {3, 2},
		{4, 5}, {4, 6}, {7, 5}, {7, 6},
		{0, 4}, {2, 6}, {1, 5}, {3, 7}
	};

	constexpr int32 PlaneLines[3][4][2] = {
		{{0, 1}, {0, 2}, {3, 1}, {3, 2}}, // Z=0, XY Plane
		{{0, 2}, {0, 4}, {6, 4}, {6, 2}}, // Y=0, XZ Plane
		{{0, 4}, {0, 1}, {5, 1}, {5, 4}}  // X=0, YZ Plane
	};

	void DrawOrientedBox(FPrimitiveDrawInterface& InPDI, const FOrientedBox& InOrientedBox, const FLinearColor& InColor)
	{
		const bool FlatAxes[3] = {
			FMath::IsNearlyZero(InOrientedBox.ExtentX),
			FMath::IsNearlyZero(InOrientedBox.ExtentY),
			FMath::IsNearlyZero(InOrientedBox.ExtentZ)
		};

		const uint8 FlatAxisCount = (FlatAxes[AxisX] * 1)
			+ (FlatAxes[AxisY] * 1)
			+ (FlatAxes[AxisZ] * 1);

		// Don't need lines or dots.
		if (FlatAxisCount > 1)
		{
			return;
		}

		using namespace UE::AvaViewport::Private;

		FVector Vertices[8];
		InOrientedBox.CalcVertices(Vertices);

		constexpr bool bScreenSpace = true;
		constexpr uint8 DepthBias = 0;

		if (FlatAxisCount == 0)
		{
			for (int32 Index = 0; Index < 12; ++Index)
			{
				InPDI.DrawLine(Vertices[CubeLines[Index][0]], Vertices[CubeLines[Index][1]], InColor, SDPG_World, BoundingBoxLineThickness, DepthBias, bScreenSpace);
			}
		}
		else
		{
			// Branchless assignment - only one of these bools can be true
			const int32 VertListIndex = (FlatAxes[AxisX] * AxisX)
				+ (FlatAxes[AxisY] * AxisY)
				+ (FlatAxes[AxisZ] * AxisZ);

			for (int32 Index = 0; Index < 4; ++Index)
			{
				InPDI.DrawLine(Vertices[PlaneLines[VertListIndex][Index][0]], Vertices[PlaneLines[VertListIndex][Index][1]], InColor, SDPG_World, BoundingBoxLineThickness, DepthBias, bScreenSpace);
			}
		}
	}
}

FAvaLevelViewportBoundingBoxVisualizer::FAvaLevelViewportBoundingBoxVisualizer()
	: OptimizationState(EAvaViewportBoundingBoxOptimizationState::RenderSelectionBounds)
{
}

void FAvaLevelViewportBoundingBoxVisualizer::ResetOptimizationState()
{
	SetOptimizationRenderSelectionBounds();
}

void FAvaLevelViewportBoundingBoxVisualizer::Draw(UAvaSelectionProviderSubsystem& InSelectionProvider, UAvaBoundsProviderSubsystem& InBoundsProvider, FPrimitiveDrawInterface& InPDI)
{
	if (OptimizationState < EAvaViewportBoundingBoxOptimizationState::RenderSelectedActors)
	{
		return;
	}

	if (const UAvaViewportSettings* ViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		if (!ViewportSettings->bEnableBoundingBoxes)
		{
			return;
		}
	}

	TConstArrayView<TWeakObjectPtr<AActor>> SelectedActors = InSelectionProvider.GetSelectedActors();

	if (SelectedActors.IsEmpty())
	{
		return;
	}

	using namespace UE::AvaLevelViewport::Private;

	constexpr double MaxTime = 0.003;
	constexpr double HalfMaxTime = MaxTime * 0.5;

	{
		const FAvaTaskTimer TaskTimer;

		for (const TWeakObjectPtr<AActor>& ActorWeak : SelectedActors)
		{
			if (AActor* Actor = ActorWeak.Get())
			{
				FOrientedBox OrientedBox;

				if (InBoundsProvider.GetActorOrientedBounds(Actor, OrientedBox))
				{
					DrawOrientedBox(InPDI, MoveTemp(OrientedBox), ActorBoundsColor);
				}
			}
		}

		const double TaskTime = TaskTimer.GetTaskLength();

		if (TaskTime >= MaxTime)
		{
			SetOptimizationRenderNothing(TaskTime);
			return;
		}

		if (TaskTime >= HalfMaxTime)
		{
			SetOptimizationRenderSelectedActors(TaskTime);
			return;
		}
	}

	if (OptimizationState < EAvaViewportBoundingBoxOptimizationState::RenderSelectedActorsAndChildren)
	{
		return;
	}

	{
		const FAvaTaskTimer TaskTimer;

		for (const TWeakObjectPtr<AActor>& ActorWeak : SelectedActors)
		{
			if (AActor* Actor = ActorWeak.Get())
			{
				if (!InSelectionProvider.GetAttachedActors(Actor, true).IsEmpty())
				{
					FOrientedBox OrientedBox;

					if (InBoundsProvider.GetActorAndChildrenOrientedBounds(Actor, OrientedBox))
					{
						DrawOrientedBox(InPDI, MoveTemp(OrientedBox), ActorAndChildrenBoundsColor);
					}
				}
			}
		}

		const double TaskTime = TaskTimer.GetTaskLength();

		if (TaskTime >= MaxTime)
		{
			SetOptimizationRenderSelectedActors(TaskTime);
		}
	}

	if (SelectedActors.Num() < 2 || OptimizationState < EAvaViewportBoundingBoxOptimizationState::RenderSelectionBounds)
	{
		return;
	}

	{
		const FAvaTaskTimer TaskTimer;
		FOrientedBox SelectionBounds;

		if (InBoundsProvider.GetSelectionOrientedBounds(true, SelectionBounds))
		{
			DrawOrientedBox(InPDI, MoveTemp(SelectionBounds), SelectionBoundsColor);
		}

		const double TaskTime = TaskTimer.GetTaskLength();

		if (TaskTime >= MaxTime)
		{
			SetOptimizationRenderSelectedActorsAndChildren(TaskTime);
		}
	}
}

void FAvaLevelViewportBoundingBoxVisualizer::SetOptimizationRenderNothing(double InTaskTimeTaken)
{
	if (OptimizationState != EAvaViewportBoundingBoxOptimizationState::RenderSelectedActors)
	{
		OptimizationState = EAvaViewportBoundingBoxOptimizationState::RenderNothing;

		const int32 TaskTimeInMicroseconds = FMath::RoundToInt(InTaskTimeTaken * 1000000.0);

		UE_LOGF(AvaViewportLog, Warning,
			"Bounding boxes are taking a critical amount of time to calculate (%iÎ¼s) - automatically hidden all bounding boxes.",
			TaskTimeInMicroseconds);
	}
}

void FAvaLevelViewportBoundingBoxVisualizer::SetOptimizationRenderSelectedActors(double InTaskTimeTaken)
{
	if (OptimizationState != EAvaViewportBoundingBoxOptimizationState::RenderSelectedActors)
	{
		OptimizationState = EAvaViewportBoundingBoxOptimizationState::RenderSelectedActors;

		const int32 TaskTimeInMicroseconds = FMath::RoundToInt(InTaskTimeTaken * 1000000.0);

		UE_LOGF(AvaViewportLog, Warning,
			"Bounding boxes are taking a large amount of time to calculate (%iÎ¼s) - automatically hidden child actor and selection bounding boxes.",
			TaskTimeInMicroseconds);
	}
}

void FAvaLevelViewportBoundingBoxVisualizer::SetOptimizationRenderSelectedActorsAndChildren(double InTaskTimeTaken)
{
	if (OptimizationState != EAvaViewportBoundingBoxOptimizationState::RenderSelectedActorsAndChildren)
	{
		OptimizationState = EAvaViewportBoundingBoxOptimizationState::RenderSelectedActorsAndChildren;

		const int32 TaskTimeInMicroseconds = FMath::RoundToInt(InTaskTimeTaken * 1000000.0);

		UE_LOGF(AvaViewportLog, Warning,
			"Bounding boxes are taking a significant amount of time to calculate (%iÎ¼s) - automatically hidden selection bounding boxes.",
			TaskTimeInMicroseconds);
	}
}

void FAvaLevelViewportBoundingBoxVisualizer::SetOptimizationRenderSelectionBounds()
{
	if (OptimizationState != EAvaViewportBoundingBoxOptimizationState::RenderSelectionBounds)
	{
		OptimizationState = EAvaViewportBoundingBoxOptimizationState::RenderSelectionBounds;

		UE_LOGF(AvaViewportLog, Log,
			"All bounding boxes visible.");
	}
}
