// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "Visualizers/IAvaViewportBoundingBoxVisualizer.h"

class AActor;

class FAvaLevelViewportBoundingBoxVisualizer : public IAvaViewportBoundingBoxVisualizer
{
public:
	UE_AVA_INHERITS(FAvaLevelViewportBoundingBoxVisualizer, IAvaViewportBoundingBoxVisualizer)

	FAvaLevelViewportBoundingBoxVisualizer();
	virtual ~FAvaLevelViewportBoundingBoxVisualizer() = default;

	//~ Begin IAvaViewportBoundingBoxVisualizer
	virtual void Draw(UAvaSelectionProviderSubsystem& InSelectionProvider, UAvaBoundsProviderSubsystem& InBoundsProvider, FPrimitiveDrawInterface& InPDI) override;
	virtual EAvaViewportBoundingBoxOptimizationState GetOptimizationState() const override { return OptimizationState; }
	virtual void ResetOptimizationState() override;
	//~ End IAvaViewportBoundingBoxVisualizer

protected:
	EAvaViewportBoundingBoxOptimizationState OptimizationState;

	void SetOptimizationRenderNothing(double InTaskTimeTaken);
	void SetOptimizationRenderSelectedActors(double InTaskTimeTaken);
	void SetOptimizationRenderSelectedActorsAndChildren(double InTaskTimeTaken);
	void SetOptimizationRenderSelectionBounds();
};
