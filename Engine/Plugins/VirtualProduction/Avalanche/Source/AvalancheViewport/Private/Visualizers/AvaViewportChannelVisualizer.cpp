// Copyright Epic Games, Inc. All Rights Reserved.

#include "Visualizers/AvaViewportChannelVisualizer.h"
#include "AvaViewportPostProcessManager.h"
#include "Containers/Map.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/Package.h"
#include "Viewport/Interaction/AvaViewportPostProcessInfo.h"

namespace UE::AvaViewport::Private
{
	const FString ChannelReferencerName = FString(TEXT("AvaViewportChannelVisualizer"));

	const TMap<EAvaViewportPostProcessType, TSoftObjectPtr<UMaterial>> ChannelAssets = {
		{ EAvaViewportPostProcessType::RedChannel,   TSoftObjectPtr<UMaterial>(FSoftObjectPath("/Script/Engine.Material'/Avalanche/EditorResources/M_ChannelVisualizer_Red.M_ChannelVisualizer_Red'")) },
		{ EAvaViewportPostProcessType::GreenChannel, TSoftObjectPtr<UMaterial>(FSoftObjectPath("/Script/Engine.Material'/Avalanche/EditorResources/M_ChannelVisualizer_Green.M_ChannelVisualizer_Green'")) },
		{ EAvaViewportPostProcessType::BlueChannel,  TSoftObjectPtr<UMaterial>(FSoftObjectPath("/Script/Engine.Material'/Avalanche/EditorResources/M_ChannelVisualizer_Blue.M_ChannelVisualizer_Blue'")) },
		{ EAvaViewportPostProcessType::AlphaChannel, TSoftObjectPtr<UMaterial>(FSoftObjectPath("/Script/Engine.Material'/Avalanche/EditorResources/M_ChannelVisualizer_Alpha.M_ChannelVisualizer_Alpha'")) }
	};
}

FAvaViewportChannelVisualizer::FAvaViewportChannelVisualizer(TSharedRef<IAvaViewportClient> InAvaViewportClient, EAvaViewportPostProcessType InChannel)
	: FAvaViewportPostProcessVisualizer(InAvaViewportClient)
{
	bRequiresTonemapperSetting = (InChannel == EAvaViewportPostProcessType::AlphaChannel);

	using namespace UE::AvaViewport::Private;

	if (const TSoftObjectPtr<UMaterial>* BaseMaterialPtr = ChannelAssets.Find(InChannel))
	{
		if (UMaterial* BaseMaterial = BaseMaterialPtr->LoadSynchronous())
		{
			PostProcessBaseMaterial = BaseMaterial;
			PostProcessMaterial = UMaterialInstanceDynamic::Create(BaseMaterial, GetTransientPackage());
		}
	}
}

FString FAvaViewportChannelVisualizer::GetReferencerName() const
{
	return UE::AvaViewport::Private::ChannelReferencerName;
}
