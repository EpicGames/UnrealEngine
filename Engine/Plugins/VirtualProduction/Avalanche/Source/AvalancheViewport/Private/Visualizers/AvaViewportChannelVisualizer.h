// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Visualizers/AvaViewportPostProcessVisualizer.h"
#include "Math/Vector.h"

enum class EAvaViewportPostProcessType : uint8;

class FAvaViewportChannelVisualizer : public FAvaViewportPostProcessVisualizer
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaViewportChannelVisualizer, FAvaViewportPostProcessVisualizer)

	FAvaViewportChannelVisualizer(TSharedRef<IAvaViewportClient> InAvaViewportClient, EAvaViewportPostProcessType InChannel);
	virtual ~FAvaViewportChannelVisualizer() override = default;

	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	//~ End FGCObject
};
