// Copyright Epic Games, Inc. All Rights Reserved.

#include "Visualizers/AvaViewportCheckerboardVisualizer.h"
#include "AvalancheViewportModule.h"
#include "AvaViewportSettings.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "AvaViewportBackgroundVisualizer"

namespace UE::AvaViewport::Private
{
	const FString CheckerboardReferencerName = FString(TEXT("AvaViewportCheckerboardVisualizer"));
	const FName CheckerboardColor0Name = FName(TEXT("CheckerboardColor0"));
	const FName CheckerboardColor1Name = FName(TEXT("CheckerboardColor1"));
	const FName CheckerboardSizeName = FName(TEXT("CheckerboardSize"));
}

FAvaViewportCheckerboardVisualizer::FAvaViewportCheckerboardVisualizer(TSharedRef<IAvaViewportClient> InAvaViewportClient)
	: FAvaViewportPostProcessVisualizer(InAvaViewportClient)
{
	bRequiresTonemapperSetting = true;
	
	if (!UpdateFromViewportSettings())
	{
		CheckerboardColor0 = FVector(0.048172);
		CheckerboardColor1 = FVector(0.177888);
		CheckerboardSize = FVector(8.0);
	}
	
	if (UAvaViewportSettings* ViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		ViewportSettings->OnSettingChanged().AddRaw(this, &FAvaViewportCheckerboardVisualizer::OnSettingChanged);
	}

	InitPostProcessMaterial();
}

FAvaViewportCheckerboardVisualizer::~FAvaViewportCheckerboardVisualizer()
{
	if (UAvaViewportSettings* ViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		ViewportSettings->OnSettingChanged().RemoveAll(this);	
	}
}

void FAvaViewportCheckerboardVisualizer::UpdatePostProcessMaterial()
{
	FAvaViewportPostProcessVisualizer::UpdatePostProcessMaterial();
	
	if (PostProcessMaterial)
	{
		using namespace UE::AvaViewport::Private;
		PostProcessMaterial->SetVectorParameterValue(CheckerboardColor0Name, CheckerboardColor0);
		PostProcessMaterial->SetVectorParameterValue(CheckerboardColor1Name, CheckerboardColor1);
		PostProcessMaterial->SetVectorParameterValue(CheckerboardSizeName, CheckerboardSize);
	}
	else
	{
		UE_LOGF(AvaViewportLog, Warning, "FAvaViewportCheckerboardVisualizer::UpdatePostProcessMaterial: Missing post process material.");
	}
}

FString FAvaViewportCheckerboardVisualizer::GetReferencerName() const
{
	return UE::AvaViewport::Private::CheckerboardReferencerName;
}

void FAvaViewportCheckerboardVisualizer::OnSettingChanged(UObject* InSettings, FPropertyChangedEvent& InPropertyChangeEvent)
{
	static const FName ViewportCheckerboardMaterialName = GET_MEMBER_NAME_CHECKED(UAvaViewportSettings, ViewportCheckerboardMaterial);
	static const FName ViewportCheckerboardColor0Name = GET_MEMBER_NAME_CHECKED(UAvaViewportSettings, ViewportCheckerboardColor0);
	static const FName ViewportCheckerboardColor1Name = GET_MEMBER_NAME_CHECKED(UAvaViewportSettings, ViewportCheckerboardColor1);
	static const FName ViewportCheckerboardSizeName = GET_MEMBER_NAME_CHECKED(UAvaViewportSettings, ViewportCheckerboardSize);

	bool bShouldUpdatePostProcessMaterial = false;
	
	if (InPropertyChangeEvent.GetPropertyName() == ViewportCheckerboardColor0Name
		|| InPropertyChangeEvent.GetPropertyName() == ViewportCheckerboardColor1Name
		|| InPropertyChangeEvent.GetPropertyName() == ViewportCheckerboardSizeName)
	{
		bShouldUpdatePostProcessMaterial = UpdateFromViewportSettings();
	}
	else if (InPropertyChangeEvent.GetPropertyName() == ViewportCheckerboardMaterialName)
	{
		bShouldUpdatePostProcessMaterial = InitPostProcessMaterial();
	}

	if (bShouldUpdatePostProcessMaterial)
	{
		UpdatePostProcessMaterial();
	}
}

bool FAvaViewportCheckerboardVisualizer::InitPostProcessMaterial()
{
	if (const UAvaViewportSettings* ViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		if (UMaterial* CheckerboardMaterial = ViewportSettings->ViewportCheckerboardMaterial.LoadSynchronous())
		{
			PostProcessBaseMaterial = CheckerboardMaterial;
			PostProcessMaterial = UMaterialInstanceDynamic::Create(CheckerboardMaterial, GetTransientPackage());
			return true;
		}
	}

	UE_LOGF(AvaViewportLog, Warning, "FAvaViewportCheckerboardVisualizer::InitPostProcessMaterial: Failed to init post process material.");
	return false;
}

bool FAvaViewportCheckerboardVisualizer::UpdateFromViewportSettings()
{
	if (const UAvaViewportSettings* ViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		CheckerboardColor0 = FVector(ViewportSettings->ViewportCheckerboardColor0);
		CheckerboardColor1 = FVector(ViewportSettings->ViewportCheckerboardColor1);
		CheckerboardSize = FVector(ViewportSettings->ViewportCheckerboardSize);
		return true;
	}

	UE_LOGF(AvaViewportLog, Warning, "FAvaViewportCheckerboardVisualizer::UpdateFromViewportSettings: Failed to update from viewport settings.");
	return false;
}

#undef LOCTEXT_NAMESPACE
