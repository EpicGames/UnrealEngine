// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Visualizers/AvaViewportPostProcessVisualizer.h"
#include "Math/Vector.h"

struct FAvaViewportPostProcessInfo;

class FAvaViewportCheckerboardVisualizer : public FAvaViewportPostProcessVisualizer
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaViewportCheckerboardVisualizer, FAvaViewportPostProcessVisualizer)

	FAvaViewportCheckerboardVisualizer(TSharedRef<IAvaViewportClient> InAvaViewportClient);
	virtual ~FAvaViewportCheckerboardVisualizer() override;
	
	//~ Begin FAvaViewportPostProcessVisualizer
	virtual void UpdatePostProcessMaterial() override;
	//~ End FAvaViewportPostProcessVisualizer

	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	//~ End FGCObject

	void OnSettingChanged(UObject* InSettings, struct FPropertyChangedEvent& InPropertyChangeEvent);

	bool InitPostProcessMaterial();
	bool UpdateFromViewportSettings();
	
protected:
	/** First color of the checkerboard. */
	FVector CheckerboardColor0;

	/** Second color of the checkerboard. */
	FVector CheckerboardColor1;

	/** Size of the checkerboard (in pixels). */
	FVector CheckerboardSize;
};