// Copyright Epic Games, Inc. All Rights Reserved.

#include "Visualizers/AvaViewportPostProcessVisualizer.h"
#include "AvalancheViewportModule.h"
#include "AvaViewportDataSubsystem.h"
#include "AvaViewportPostProcessManager.h"
#include "Editor.h"
#include "Engine/RendererSettings.h"
#include "FinalPostProcessSettings.h"
#include "ISettingsEditorModule.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "SceneView.h"
#include "ViewportClient/IAvaViewportClient.h"

#define LOCTEXT_NAMESPACE "AvaViewportPostProcessVisualizer"

namespace UE::AvaViewport::Private
{
	const FName OpacityName = FName(TEXT("Opacity"));
}

FAvaViewportPostProcessVisualizer::FAvaViewportPostProcessVisualizer(TSharedRef<IAvaViewportClient> InAvaViewportClient)
{
	AvaViewportClientWeak = InAvaViewportClient;
	PostProcessOpacity = 1.f;
	bRequiresTonemapperSetting = false;

	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
}

FAvaViewportPostProcessVisualizer::~FAvaViewportPostProcessVisualizer()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

TSharedPtr<IAvaViewportClient> FAvaViewportPostProcessVisualizer::GetAvaViewportClient() const
{
	return AvaViewportClientWeak.Pin();
}

void FAvaViewportPostProcessVisualizer::SetPostProcessOpacity(float InOpacity)
{
	if (FMath::IsNearlyEqual(PostProcessOpacity, InOpacity))
	{
		return;
	}

	SetPostProcessOpacityInternal(InOpacity);

	UpdatePostProcessInfo();
	UpdatePostProcessMaterial();
}

bool FAvaViewportPostProcessVisualizer::CanActivate(bool bInSilent) const
{
	if (!bRequiresTonemapperSetting || bInSilent)
	{
		return true;
	}

	URendererSettings* RendererSettings = GetMutableDefault<URendererSettings>();
	check(RendererSettings);

	if (RendererSettings->bEnableAlphaChannelInPostProcessing)
	{
		return true;
	}

	const EAppReturnType::Type Response = FMessageDialog::Open(
		EAppMsgType::YesNoCancel,
		LOCTEXT("AlphaChannelInPostProcessingRequiredMessage", "This Post Process effect will not work without enabling Alpha Output via Project Settings > Engine > Rendering > Default Settings > 'Alpha Ouput'. Warning: update can add renderer performance costs.\n\nEnable this setting in DefaultEngine.ini now?"),
		LOCTEXT("AlphaChannelInPostProcessingRequiredTitle", "Project Setting Required")
	);

	switch (Response)
	{
		case EAppReturnType::Yes:
		{
			RendererSettings->bEnableAlphaChannelInPostProcessing = true;

			if (IConsoleVariable* PropagateAlphaCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PostProcessing.PropagateAlpha")))
			{
				PropagateAlphaCVar->Set(true);
			}

			FProperty* Property = RendererSettings->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(URendererSettings, bEnableAlphaChannelInPostProcessing));

			FPropertyChangedEvent PropertyChangedEvent(Property, EPropertyChangeType::ValueSet, {RendererSettings});
			RendererSettings->PostEditChangeProperty(PropertyChangedEvent);

			RendererSettings->UpdateSinglePropertyInConfigFile(Property, RendererSettings->GetDefaultConfigFilename());

			FModuleManager::GetModuleChecked<ISettingsEditorModule>("SettingsEditor").OnApplicationRestartRequired();
			break;
		}

		case EAppReturnType::No:
			// Continue to enable the option, but not the render setting.
			break;

		default:
			// Do nothing
			return false;
	}

	return true;
}

void FAvaViewportPostProcessVisualizer::OnActivate()
{
	LoadPostProcessInfo();
}

void FAvaViewportPostProcessVisualizer::OnDeactivate()
{
}

void FAvaViewportPostProcessVisualizer::UpdateForViewport(const FAvaVisibleArea& InVisibleArea, const FVector2f& InVisibleAreaOffset, 
	const FVector2f& InWidgetSize, const FVector2f& InCameraOffset)
{
}

void FAvaViewportPostProcessVisualizer::ApplyToSceneView(FSceneView* InSceneView) const
{
	if (!InSceneView || FMath::IsNearlyZero(PostProcessOpacity) || !PostProcessMaterial)
	{
		return;
	}

	FPostProcessSettings PostProcessSettings;

	if (!SetupPostProcessSettings(PostProcessSettings))
	{
		return;
	}

	InSceneView->OverridePostProcessSettings(PostProcessSettings, 1.f);
}

void FAvaViewportPostProcessVisualizer::AddReferencedObjects(FReferenceCollector& InCollector)
{
	if (PostProcessBaseMaterial)
	{
		InCollector.AddReferencedObject(PostProcessBaseMaterial);
	}

	if (PostProcessMaterial)
	{
		InCollector.AddReferencedObject(PostProcessMaterial);
	}
}

void FAvaViewportPostProcessVisualizer::PostUndo(bool bSuccess)
{
	FEditorUndoClient::PostUndo(bSuccess);

	LoadPostProcessInfo();
}

void FAvaViewportPostProcessVisualizer::PostRedo(bool bSuccess)
{
	FEditorUndoClient::PostRedo(bSuccess);

	LoadPostProcessInfo();
}

FAvaViewportPostProcessInfo* FAvaViewportPostProcessVisualizer::GetPostProcessInfo() const
{
	TSharedPtr<IAvaViewportClient> AvaViewportClient = AvaViewportClientWeak.Pin();

	if (!AvaViewportClient.IsValid())
	{
		UE_LOGF(AvaViewportLog, Warning, "FAvaViewportPostProcessVisualizer::GetPostProcessInfo: Invalid viewport.");
		return nullptr;
	}

	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(AvaViewportClient->GetViewportWorld());

	if (!DataSubsystem)
	{
		UE_LOGF(AvaViewportLog, Warning, "FAvaViewportPostProcessVisualizer::GetPostProcessInfo: Missing data subsystem.");
		return nullptr;
	}

	if (FAvaViewportData* Data = DataSubsystem->GetData())
	{
		return &Data->PostProcessInfo;
	}

	UE_LOGF(AvaViewportLog, Warning, "FAvaViewportPostProcessVisualizer::GetPostProcessInfo: Missing viewport data.");
	return nullptr;
}

void FAvaViewportPostProcessVisualizer::LoadPostProcessInfo()
{
	if (FAvaViewportPostProcessInfo* PostProcessInfo = GetPostProcessInfo())
	{
		LoadPostProcessInfo(*PostProcessInfo);
		UpdatePostProcessMaterial();
	}
}

void FAvaViewportPostProcessVisualizer::UpdatePostProcessInfo()
{
	TSharedPtr<IAvaViewportClient> AvaViewportClient = AvaViewportClientWeak.Pin();

	if (!AvaViewportClient.IsValid())
	{
		UE_LOGF(AvaViewportLog, Warning, "FAvaViewportPostProcessVisualizer::UpdatePostProcessInfoL Invalid viewport.");
		return;
	}

	UAvaViewportDataSubsystem* DataSubsystem = UAvaViewportDataSubsystem::Get(AvaViewportClient->GetViewportWorld());

	if (!DataSubsystem)
	{
		UE_LOGF(AvaViewportLog, Warning, "FAvaViewportPostProcessVisualizer::UpdatePostProcessInfo: Missing data subsystem.");
		return;
	}

	if (FAvaViewportData* Data = DataSubsystem->GetData())
	{
		DataSubsystem->ModifyDataSource();
		return UpdatePostProcessInfo(Data->PostProcessInfo);
	}

	UE_LOGF(AvaViewportLog, Warning, "FAvaViewportPostProcessVisualizer::UpdatePostProcessInfo: Missing viewport data.");
}

void FAvaViewportPostProcessVisualizer::UpdatePostProcessMaterial()
{
	if (!PostProcessMaterial)
	{
		UE_LOGF(AvaViewportLog, Warning, "FAvaViewportPostProcessVisualizer::UpdatePostProcessMaterial: Missing post process material.");
		return;
	}

	using namespace UE::AvaViewport::Private;

	PostProcessMaterial->SetScalarParameterValue(OpacityName, PostProcessOpacity);
}

void FAvaViewportPostProcessVisualizer::SetPostProcessOpacityInternal(float InOpacity)
{
	PostProcessOpacity = InOpacity;
}

void FAvaViewportPostProcessVisualizer::LoadPostProcessInfo(const FAvaViewportPostProcessInfo& InPostProcessInfo)
{
	SetPostProcessOpacityInternal(InPostProcessInfo.Opacity);
}

void FAvaViewportPostProcessVisualizer::UpdatePostProcessInfo(FAvaViewportPostProcessInfo& InPostProcessInfo) const
{
	InPostProcessInfo.Opacity = PostProcessOpacity;
}

bool FAvaViewportPostProcessVisualizer::SetupPostProcessSettings(FPostProcessSettings& InPostProcessSettings) const
{
	InPostProcessSettings.AddBlendable(PostProcessMaterial, 1.f);

	return true;
}

#undef LOCTEXT_NAMESPACE
