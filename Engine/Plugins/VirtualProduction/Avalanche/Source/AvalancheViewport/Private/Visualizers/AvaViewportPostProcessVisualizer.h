// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "EditorUndoClient.h"
#include "Math/MathFwd.h"
#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"
#include "Visualizers/IAvaViewportPostProcessVisualizer.h"

class FSceneView;
class IAvaViewportClient;
class UMaterial;
class UMaterialInstanceDynamic;
struct FAvaViewportPostProcessInfo;
struct FAvaVisibleArea;
struct FPostProcessSettings;

class FAvaViewportPostProcessVisualizer : public IAvaViewportPostProcessVisualizer, public FGCObject, public FEditorUndoClient
{
public:
	UE_AVA_INHERITS(FAvaViewportPostProcessVisualizer, IAvaViewportPostProcessVisualizer)

	FAvaViewportPostProcessVisualizer(TSharedRef<IAvaViewportClient> InAvaViewportClient);

	virtual ~FAvaViewportPostProcessVisualizer() override;

	TSharedPtr<IAvaViewportClient> GetAvaViewportClient() const;

	float GetPostProcessOpacity() const { return PostProcessOpacity; }
	void SetPostProcessOpacity(float InOpacity);

	FAvaViewportPostProcessInfo* GetPostProcessInfo() const;

	void LoadPostProcessInfo();

	//~ Begin IAvaViewportPostProcessVisualizer
	virtual bool CanActivate(bool bInSilent) const override;
	//~ End IAvaViewportPostProcessVisualizer

	virtual void OnActivate();
	virtual void OnDeactivate();

	virtual void UpdateForViewport(const FAvaVisibleArea& InVisibleArea, const FVector2f& InVisibleAreaOffset, 
		const FVector2f& InWidgetSize, const FVector2f& InCameraOffset);

	void ApplyToSceneView(FSceneView* InSceneView) const;

protected:
	//~ Begin FGCObject
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FGCObject

	// FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	// ~FEditorUndoClient Interface

	TWeakPtr<IAvaViewportClient> AvaViewportClientWeak;

	TObjectPtr<UMaterial> PostProcessBaseMaterial;

	TObjectPtr<UMaterialInstanceDynamic> PostProcessMaterial;

	float PostProcessOpacity;

	bool bRequiresTonemapperSetting;

	void SetPostProcessOpacityInternal(float InOpacity);

	void UpdatePostProcessInfo();

	virtual void LoadPostProcessInfo(const FAvaViewportPostProcessInfo& InPostProcessInfo);
	virtual void UpdatePostProcessInfo(FAvaViewportPostProcessInfo& InPostProcessInfo) const;

	virtual void UpdatePostProcessMaterial();

	virtual bool SetupPostProcessSettings(FPostProcessSettings& InPostProcessSettings) const;
};
