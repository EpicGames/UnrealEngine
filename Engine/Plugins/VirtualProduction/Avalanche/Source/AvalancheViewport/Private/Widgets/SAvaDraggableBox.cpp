// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAvaDraggableBox.h"
#include "AvaViewportSettings.h"
#include "Framework/Application/SlateApplication.h"
#include "Input/DragAndDrop.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SAvaDraggableBoxOverlay.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"

/**
 * A drag/drop operation used by SAvaDraggableBox.
 */
class FAvaDraggableBoxUIDragOperation : public FDragDropOperation
{
public:
	FAvaDraggableBoxUIDragOperation(const TSharedRef<SAvaDraggableBox> InDraggableBox,
		const SAvaDraggableBox::FDragInfo& InDragInfo)
		: DraggableBoxWeak(InDraggableBox)
		, DragInfo(InDragInfo)
	{
	}

	virtual ~FAvaDraggableBoxUIDragOperation() override = default;

	//~ Begin FDragDropOperation
	virtual void OnDragged(const FDragDropEvent& InDragDropEvent)
	{
		if (TSharedPtr<SAvaDraggableBox> DraggableBox = DraggableBoxWeak.Pin())
		{
			DraggableBox->OnDragUpdate(InDragDropEvent, DragInfo, /* Dropped */ false);
		}
	}

	virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& InMouseEvent)
	{
		if (TSharedPtr<SAvaDraggableBox> DraggableBox = DraggableBoxWeak.Pin())
		{
			DraggableBox->OnDragUpdate(InMouseEvent, DragInfo, /* Dropped */ true);
		}
	}
	//~ End FDragDropOperation

protected:
	TWeakPtr<SAvaDraggableBox> DraggableBoxWeak;
	SAvaDraggableBox::FDragInfo DragInfo;
};

void SAvaDraggableBox::Construct(const FArguments& InArgs, const TSharedRef<SAvaDraggableBoxOverlay>& InDraggableOverlay)
{
	DraggableOverlayWeak = InDraggableOverlay;
	InnerWidget = InArgs._Content.Widget;

	ChildSlot
		[
			InArgs._Content.Widget
		];
}

void SAvaDraggableBox::OnDragUpdate(const FPointerEvent& InMouseEvent, const FDragInfo& InDragInfo, bool bInDropped)
{
	TSharedPtr<SAvaDraggableBoxOverlay> DraggableOverlay = DraggableOverlayWeak.Pin();

	if (!DraggableOverlay.IsValid())
	{
		return;
	}

	const FGeometry& MyGeometry = DraggableOverlay->GetTickSpaceGeometry();

	const FVector2f MouseOffset = (InMouseEvent.GetScreenSpacePosition() - InDragInfo.OriginalMousePosition)
		* (MyGeometry.GetLocalSize() / MyGeometry.GetAbsoluteSize());

	FVector2f NewAlignmentOffset = InDragInfo.OriginalWidgetPosition.AlignmentOffset;

	switch (InDragInfo.OriginalWidgetPosition.HorizontalAlignment)
	{
		case EHorizontalAlignment::HAlign_Left:
			NewAlignmentOffset.X += MouseOffset.X;
			break;

		case EHorizontalAlignment::HAlign_Right:
			NewAlignmentOffset.X -= MouseOffset.X;
			break;

		default:
			// Do nothing
			break;
	}

	switch (InDragInfo.OriginalWidgetPosition.VerticalAlignment)
	{
		case EVerticalAlignment::VAlign_Top:
			NewAlignmentOffset.Y += MouseOffset.Y;
			break;

		case EVerticalAlignment::VAlign_Bottom:
			NewAlignmentOffset.Y -= MouseOffset.Y;
			break;

		default:
			// Do nothing
			break;
	}

	DraggableOverlay->SetBoxHorizontalAlignment(InDragInfo.OriginalWidgetPosition.HorizontalAlignment);
	DraggableOverlay->SetBoxVerticalAlignment(InDragInfo.OriginalWidgetPosition.VerticalAlignment);
	DraggableOverlay->SetBoxAlignmentOffset(NewAlignmentOffset);

	if (bInDropped)
	{
		DraggableOverlay->SavePosition();
	}
}

FReply SAvaDraggableBox::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
};

FReply SAvaDraggableBox::OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (const UAvaViewportSettings* ViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		TSharedRef<FAvaDraggableBoxUIDragOperation> DragDropOperation = MakeShared<FAvaDraggableBoxUIDragOperation>(
			SharedThis(this),
			SAvaDraggableBox::FDragInfo{
				ViewportSettings->ShapeEditorViewportControlPosition,
				InMouseEvent.GetScreenSpacePosition()
			}
		);

		return FReply::Handled().BeginDragDrop(DragDropOperation);
	}

	return FReply::Handled();
}
