// Copyright Epic Games, Inc. All Rights Reserved.

/* Copied and modified from ModelingEditorUI */

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "AvaViewportSettings.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SAvaDraggableBoxOverlay;

/**
 * A widget for the draggable box itself, which requires its parent to handle its positioning in
 * response to the drag.
 *
 * Users probably shouldn't use this class directly; rather, they should use SAvaDraggableBoxOverlay,
 * which will put its contents into a draggable box and properly handle the dragging without the
 * user having to set it up.
 */
class SAvaDraggableBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaDraggableBox)
		{}
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<SAvaDraggableBoxOverlay>& InDraggableOverlay);

	struct FDragInfo
	{
		FAvaShapeEditorViewportControlPosition OriginalWidgetPosition;
		FVector2f OriginalMousePosition;
	};

	void OnDragUpdate(const FPointerEvent& InMouseEvent, const FDragInfo& InDragInfo, bool bInDropped);

	//~ Begin SWidget
	FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	//~ End SWidget

protected:
	TWeakPtr<SAvaDraggableBoxOverlay> DraggableOverlayWeak;
	TSharedPtr<SWidget> InnerWidget;
};
