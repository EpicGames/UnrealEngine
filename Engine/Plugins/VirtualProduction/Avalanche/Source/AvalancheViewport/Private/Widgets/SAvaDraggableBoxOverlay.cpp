// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAvaDraggableBoxOverlay.h"
#include "AvaViewportSettings.h"
#include "Framework/Application/SlateApplication.h"
#include "Input/DragAndDrop.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SAvaDraggableBox.h"
#include "Widgets/SWindow.h"

namespace UE::AvaEditor::Private
{
	static const FAvaShapeEditorViewportControlPosition DefaultPosition = {
		EHorizontalAlignment::HAlign_Left,
		EVerticalAlignment::VAlign_Bottom,
		FVector2f::ZeroVector
	};

	static constexpr float DraggableBorder = 3.f;
}

void SAvaDraggableBoxOverlay::Construct(const FArguments& InArgs)
{
	HorizontalAlignment = UE::AvaEditor::Private::DefaultPosition.HorizontalAlignment;
	VerticalAlignment = UE::AvaEditor::Private::DefaultPosition.VerticalAlignment;

	ChildSlot
	[
		SAssignNew(Container, SBox)
		.Visibility(EVisibility::SelfHitTestInvisible)
		.HAlign(HorizontalAlignment)
		.VAlign(VerticalAlignment)
		[
			// The draggable box itself
			SAssignNew(DraggableBox, SAvaDraggableBox, SharedThis(this))
			[
				InArgs._Content.Widget
			]
		]
	];

	if (const UAvaViewportSettings* ViewportSettings = GetDefault<UAvaViewportSettings>())
	{
		SetBoxHorizontalAlignment(ViewportSettings->ShapeEditorViewportControlPosition.HorizontalAlignment);
		SetBoxVerticalAlignment(ViewportSettings->ShapeEditorViewportControlPosition.VerticalAlignment);
		SetBoxAlignmentOffset(ViewportSettings->ShapeEditorViewportControlPosition.AlignmentOffset, /* Normalise */ true);
	}
}

FVector2f SAvaDraggableBoxOverlay::GetBoxAlignmentOffset() const
{
	FVector2f AlignmentOffset = FVector2f::ZeroVector;

	if (Container.IsValid())
	{
		switch (HorizontalAlignment)
		{
			case EHorizontalAlignment::HAlign_Left:
				AlignmentOffset.X = Padding.Left;
				break;

			case EHorizontalAlignment::HAlign_Right:
				AlignmentOffset.X = Padding.Right;
				break;

			default:
				// Do nothing
				break;
		}

		switch (VerticalAlignment)
		{
			case EVerticalAlignment::VAlign_Top:
				AlignmentOffset.Y = Padding.Top;
				break;

			case EVerticalAlignment::VAlign_Bottom:
				AlignmentOffset.Y = Padding.Bottom;
				break;

			default:
				// Do nothing
				break;
		}
	}

	return AlignmentOffset;
}

void SAvaDraggableBoxOverlay::SetBoxAlignmentOffset(const FVector2f& InOffset, bool bInNormalisePosition)
{
	if (!Container.IsValid() || !DraggableBox.IsValid())
	{
		return;
	}

	using namespace UE::AvaEditor::Private;

	FVector2f ConstrainedOffset = {
		FMath::Max(InOffset.X, DraggableBorder),
		FMath::Max(InOffset.Y, DraggableBorder)
	};

	if (bInNormalisePosition)
	{
		const FGeometry& MyGeometry = GetTickSpaceGeometry();

		const FVector2f AvailableSpace = (MyGeometry.GetAbsoluteSize() - DraggableBox->GetTickSpaceGeometry().GetAbsoluteSize())
			* (MyGeometry.GetLocalSize() / MyGeometry.GetAbsoluteSize());

		if (ConstrainedOffset.X > AvailableSpace.X)
		{
			ConstrainedOffset.X = AvailableSpace.X;
		}

		if (ConstrainedOffset.Y > AvailableSpace.Y)
		{
			ConstrainedOffset.Y = AvailableSpace.Y;
		}

		const FVector2f MidPoint = AvailableSpace * 0.5f;

		if (ConstrainedOffset.X > MidPoint.X)
		{
			ConstrainedOffset.X = FMath::Max(AvailableSpace.X - ConstrainedOffset.X, DraggableBorder); // Circle value around

			switch (HorizontalAlignment)
			{
				case EHorizontalAlignment::HAlign_Left:
					SetBoxHorizontalAlignment(EHorizontalAlignment::HAlign_Right);
					break;

				case EHorizontalAlignment::HAlign_Right:
					SetBoxHorizontalAlignment(EHorizontalAlignment::HAlign_Left);
					break;

				default:
					// Do nothing
					break;
			}
		}

		if (ConstrainedOffset.Y > MidPoint.Y)
		{
			ConstrainedOffset.Y = FMath::Max(AvailableSpace.Y - ConstrainedOffset.Y, DraggableBorder); // Circle value around

			switch (VerticalAlignment)
			{
				case EVerticalAlignment::VAlign_Top:
					SetBoxVerticalAlignment(EVerticalAlignment::VAlign_Bottom);
					break;

				case EVerticalAlignment::VAlign_Bottom:
					SetBoxVerticalAlignment(EVerticalAlignment::VAlign_Top);
					break;

				default:
					// Do nothing
					break;
			}
		}
	}

	switch (HorizontalAlignment)
	{
		case EHorizontalAlignment::HAlign_Left:
			Padding.Left = ConstrainedOffset.X;
			Padding.Right = 0.f;
			break;

		case EHorizontalAlignment::HAlign_Right:
			Padding.Left = 0.f;
			Padding.Right = ConstrainedOffset.X;
			break;

		default:
			// Do nothing
			break;
	}

	switch (VerticalAlignment)
	{
		case EVerticalAlignment::VAlign_Top:
			Padding.Top = ConstrainedOffset.Y;
			Padding.Bottom = 0.f;
			break;

		case EVerticalAlignment::VAlign_Bottom:
			Padding.Top = 0.f;
			Padding.Bottom = ConstrainedOffset.Y;
			break;

		default:
			// Do nothing
			break;
	}

	if (Container.IsValid())
	{
		Container->SetPadding(Padding);
	}
}

EHorizontalAlignment SAvaDraggableBoxOverlay::GetBoxHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void SAvaDraggableBoxOverlay::SetBoxHorizontalAlignment(EHorizontalAlignment InAlignment)
{
	HorizontalAlignment = InAlignment;

	if (Container.IsValid())
	{
		Container->SetHAlign(InAlignment);
	}
}

EVerticalAlignment SAvaDraggableBoxOverlay::GetBoxVerticalAlignment() const
{
	return VerticalAlignment;
}

void SAvaDraggableBoxOverlay::SetBoxVerticalAlignment(EVerticalAlignment InAlignment)
{
	VerticalAlignment = InAlignment;

	if (Container.IsValid())
	{
		Container->SetVAlign(InAlignment);
	}
}

void SAvaDraggableBoxOverlay::SavePosition()
{
	if (UAvaViewportSettings* ViewportSettings = GetMutableDefault<UAvaViewportSettings>())
	{
		ViewportSettings->ShapeEditorViewportControlPosition.HorizontalAlignment = HorizontalAlignment;
		ViewportSettings->ShapeEditorViewportControlPosition.VerticalAlignment = VerticalAlignment;
		ViewportSettings->ShapeEditorViewportControlPosition.AlignmentOffset = GetBoxAlignmentOffset();
		ViewportSettings->SaveConfig();
	}
}

FMargin SAvaDraggableBoxOverlay::GetPadding() const
{
	return Padding;
}
