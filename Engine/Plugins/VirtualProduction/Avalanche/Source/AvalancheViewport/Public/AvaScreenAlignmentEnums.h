// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

enum class EAvaRotationAxis : uint8
{
	Yaw = 1 << 0,
	Pitch = 1 << 1,
	Roll = 1 << 2,
	All = Yaw | Pitch | Roll
};

/**
 * In the order of the vector componennt determining that direction.
 * X = Depth
 * Y = Horizontal
 * Z = Vertical
 */
enum class EAvaWorldAxis : uint8
{
	Depth = 0,
	Horizontal = 1,
	Vertical = 2
};

/**
 * In the order of the vector componennt determining that direction.
 * X = Horizontal
 * Y = Vertical
 * Z = Depth
 */
enum class EAvaScreenAxis : uint8
{
	Horizontal = 0,
	Vertical = 1,
	Depth = 2
};

enum class EAvaActorDistributionMode : uint8
{
	/**
	 * Equal distance between the center of each element.
	 *
	 * |===|  |===|   |=|  |=====|
	 */
	CenterDistance,

	/**
	 * Equal distance between the edges of each element.
	 *
	 * |===|  |===|  |=|  |==|
	 */
	EdgeDistance
};

enum class EAvaAlignmentSizeMode : uint8
{
	Self,
	SelfAndChildren
};

enum class EAvaAlignmentContext : uint8
{
	SelectedActors,
	Screen
};
