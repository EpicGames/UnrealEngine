// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Math/MathFwd.h"
#include "Templates/SharedPointerFwd.h"

class AActor;
class IAvaViewportWorldCoordinateConverter;
enum class EAvaActorDistributionMode : uint8;
enum class EAvaAlignmentContext : uint8;
enum class EAvaAlignmentSizeMode : uint8;
enum class EAvaDepthAlignment : uint8;
enum class EAvaHorizontalAlignment : uint8;
enum class EAvaRotationAxis : uint8;
enum class EAvaVerticalAlignment : uint8;

class AVALANCHEVIEWPORT_API FAvaScreenAlignmentUtils
{
public:
	/**
	 * Resizes the actor to the same size as the screen at its current depth.
	 */
	static void SizeActorToScreen(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, AActor& InActor, 
		bool bInStretchToFit);

	/**
	 * Resizes the actor to teh same size as the screen at its current depth and moves it to the center.
	 */
	static void FitActorToScreen(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, AActor& InActor, 
		bool bInStretchToFit, bool bInAlignToNearestAxis);

	/**
	 * Aligns the given actor to the given rotation/axis.
	 */
	static void AlignActorRotationAxis(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, AActor& InActor, 
		EAvaRotationAxis InAxis, const FRotator& InAlignToRotation, bool bInAlignToNearestAxis, bool bInBackwards);

	/**
	 * Aligns the given actor to the given axis of the camera's rotation (looking at the camera).
	 */
	static void AlignActorsCameraRotationAxis(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, 
		const TArray<AActor*>& InActors, 
		EAvaRotationAxis InAxis, bool bInAlignToNearestAxis, bool bInBackwards);

	/**
	 * Takes in a list of actors and aligns them hozitontally (their screen X coordinates will match)
	 */
	static void AlignActorsHorizontal(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, const TArray<AActor*>& InActors,
		EAvaHorizontalAlignment InHorizontalAlignment, EAvaAlignmentSizeMode InActorSizeMode, EAvaAlignmentContext InContextType);

	/**
	 * Takes in a list of actors and aligns them vertically (their screen Y coordinates will match)
	 */
	static void AlignActorsVertical(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, const TArray<AActor*>& InActors,
		EAvaVerticalAlignment InVerticalAlignment, EAvaAlignmentSizeMode InActorSizeMode, EAvaAlignmentContext InContextType);

	/**
	 * Takes in a list of actors and aligns them front to back (their screen z coordinates will match)
	 */
	static void AlignActorsDepth(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, const TArray<AActor*>& InActors,
		EAvaDepthAlignment InDepthAlignment, EAvaAlignmentSizeMode InActorSizeMode);

	/**
	 * Takes in a list of actors and distributes them hozitontally (their screen X coordinates will spread evenly)
	 */
	static void DistributeActorsHorizontal(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, const TArray<AActor*>& InActors,
		EAvaAlignmentSizeMode InActorSizeMode, EAvaActorDistributionMode InDistributionMode, EAvaAlignmentContext InContextType);

	/**
	 * Takes in a list of actors and distributes them vertically (their screen Y coordinates will spread evenly)
	 */
	static void DistributeActorsVertical(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, const TArray<AActor*>& InActors,
		EAvaAlignmentSizeMode InActorSizeMode, EAvaActorDistributionMode InDistributionMode, EAvaAlignmentContext InContextType);

	/**
	 * Takes in a list of actors and distributes them front to back (their screen Z coordinates will spread evenly)
	 */
	static void DistributeActorsDepth(const TSharedRef<IAvaViewportWorldCoordinateConverter>& InCoordinateConverter, const TArray<AActor*>& InActors,
		EAvaAlignmentSizeMode InActorSizeMode, EAvaActorDistributionMode InDistributionMode);
};
