// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaWorldSubsystemUtils.h"
#include "Subsystems/WorldSubsystem.h"
#include "AvaViewportGuidePresetProvider.h"
#include "AvaViewportPostProcessManager.h"
#include "AvaViewportVirtualSizeEnums.h"
#include "Math/IntPoint.h"
#include "UObject/SoftObjectPtr.h"
#include "AvaViewportDataSubsystem.generated.h"

class AAvaViewportDataActor;

USTRUCT()
struct FAvaViewportData
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FAvaViewportGuideInfo> GuideData;

	UPROPERTY()
	FAvaViewportPostProcessInfo PostProcessInfo;

	UPROPERTY()
	FIntPoint VirtualSize = FIntPoint::ZeroValue;

	UPROPERTY()
	EAvaViewportVirtualSizeAspectRatioState VirtualSizeAspectRatioState = EAvaViewportVirtualSizeAspectRatioState::LockedToCamera;

	UPROPERTY()
	TSoftObjectPtr<AActor> PilotedCamera;
};

UCLASS()
class AVALANCHEVIEWPORT_API UAvaViewportDataSubsystem : public UWorldSubsystem, public TAvaWorldSubsystemInterface<UAvaViewportDataSubsystem>
{
	GENERATED_BODY()

public:
	virtual ~UAvaViewportDataSubsystem() override = default;

	FAvaViewportData* GetData();

	void ModifyDataSource();

	FAvaViewportGuidePresetProvider& GetGuidePresetProvider();

protected:
	TWeakObjectPtr<AAvaViewportDataActor> DataActorWeak;

	FAvaViewportGuidePresetProvider GuidePresetProvider;

	AAvaViewportDataActor* GetDataActor();

	//~ Begin UWorldSubsystem
	virtual bool DoesSupportWorldType(const EWorldType::Type InWorldType) const override;
	//~ End UWorldSubsystem
};
