// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Box2D.h"
#include "Math/Vector2D.h"
#include "Math/IntPoint.h"

struct FAvaViewportGeometry
{
	FBox2f CameraBounds = FBox2f(FVector2f::ZeroVector, FVector2f::ZeroVector);
	FVector2f WidgetSize = FVector2f::ZeroVector;
	float WidgetDPIScale = 1.f;
	FIntPoint VirtualSize = FIntPoint::ZeroValue;
};
