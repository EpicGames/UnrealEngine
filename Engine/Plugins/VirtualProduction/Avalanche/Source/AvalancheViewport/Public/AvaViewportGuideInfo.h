// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "Templates/SharedPointerFwd.h"
#include "Types/SlateEnums.h"
#include "AvaViewportGuideInfo.generated.h"

class FJsonObject;

UENUM()
enum class EAvaViewportGuideState : uint8
{
	Disabled,
	Enabled,
	SnappedTo
};

USTRUCT()
struct AVALANCHEVIEWPORT_API FAvaViewportGuideInfo
{
	GENERATED_BODY()

	static bool DeserializeJson(const TSharedRef<FJsonObject>& InJsonObject, FAvaViewportGuideInfo& OutGuideInfo, const FVector2f& InViewportSize);

	bool SerializeJson(const TSharedRef<FJsonObject>& InJsonObject, const FVector2f& InViewportSize) const;

	UPROPERTY()
	TEnumAsByte<EOrientation> Orientation = EOrientation::Orient_Horizontal;

	UPROPERTY()
	float OffsetFraction = 0.0f;

	UPROPERTY()
	EAvaViewportGuideState State = EAvaViewportGuideState::Disabled;

	UPROPERTY()
	bool bLocked = false;

	bool IsEnabled() const { return State != EAvaViewportGuideState::Disabled; }
};
