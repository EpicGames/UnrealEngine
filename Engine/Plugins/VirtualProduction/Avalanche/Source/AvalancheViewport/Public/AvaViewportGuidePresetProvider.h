// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/UnrealString.h"
#include "Math/MathFwd.h"
#include "Subsystems/WorldSubsystem.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AAvaViewportDataActor;
struct FAvaViewportGuideInfo;

struct AVALANCHEVIEWPORT_API FAvaViewportGuidePresetProvider
{
	bool SaveGuidePreset(const FString& InPresetName, const TArray<FAvaViewportGuideInfo>& InGuides, const FVector2f InViewportSize);

	bool LoadGuidePreset(const FString& InPresetName, TArray<FAvaViewportGuideInfo>& OutGuides, const FVector2f InViewportSize);

	bool RemoveGuidePreset(const FString& InPresetName);

	TArray<FString> GetGuidePresetNames();

	FString GetLastAccessedGuidePresetName() const;

protected:
	TWeakObjectPtr<AAvaViewportDataActor> DataActorWeak;

	FString LastAccessedGuidePresetName;
};
