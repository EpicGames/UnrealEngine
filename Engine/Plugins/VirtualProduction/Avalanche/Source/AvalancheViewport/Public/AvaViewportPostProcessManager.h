// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "Templates/SharedPointer.h"
#include "AvaViewportPostProcessManager.generated.h"

class FAvaViewportBackgroundVisualizer;
class FAvaViewportChannelVisualizer;
class FSceneView;
class IAvaViewportClient;
class IAvaViewportPostProcessVisualizer;
class UTexture;

UENUM()
enum class EAvaViewportPostProcessType : uint8
{
	None,
	Background,
	RedChannel,
	GreenChannel,
	BlueChannel,
	AlphaChannel,
	Checkerboard
};

USTRUCT()
struct FAvaViewportPostProcessInfo
{
	GENERATED_BODY()

	UPROPERTY()
	EAvaViewportPostProcessType Type = EAvaViewportPostProcessType::None;

	UPROPERTY()
	TSoftObjectPtr<UTexture> Texture = nullptr;

	UPROPERTY()
	float Opacity = 1.0f;
};

class AVALANCHEVIEWPORT_API FAvaViewportPostProcessManager : public IAvaTypeCastable
{
public:
	UE_AVA_INHERITS(FAvaViewportPostProcessManager, IAvaTypeCastable)

	FAvaViewportPostProcessManager(TSharedRef<IAvaViewportClient> InAvaViewportClient);
	virtual ~FAvaViewportPostProcessManager() override = default;

	TSharedPtr<IAvaViewportClient> GetAvaViewportClient() const { return AvaViewportClientWeak.Pin(); }

	TSharedPtr<IAvaViewportPostProcessVisualizer> GetVisualizer(EAvaViewportPostProcessType InType) const;

	TSharedPtr<IAvaViewportPostProcessVisualizer> GetActiveVisualizer() const;

	FAvaViewportPostProcessInfo* GetPostProcessInfo() const;

	void UpdateSceneView(FSceneView* InSceneView);

	void LoadPostProcessInfo();

	EAvaViewportPostProcessType GetType() const;
	void SetType(EAvaViewportPostProcessType InType);

	float GetOpacity();
	void SetOpacity(float InOpacity);

protected:
	TWeakPtr<IAvaViewportClient> AvaViewportClientWeak;

	TMap<EAvaViewportPostProcessType, TSharedPtr<IAvaViewportPostProcessVisualizer>> Visualizers;

	void ModifyDataSource();
};
