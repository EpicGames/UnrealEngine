// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Interaction/AvaSnapDefs.h"
#include "UObject/SoftObjectPtr.h"
#include "Types/SlateEnums.h"
#include "AvaViewportSettings.generated.h"

class UMaterial;
class UTexture;

USTRUCT(BlueprintType)
struct FAvaLevelViewportSafeFrame
{
	GENERATED_BODY()

	/** Distance from the center of the screen to the edge in percent. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Safe Frames", Meta=(Units=Percent, UIMin=0, UIMax=100, ClampMin=0, ClampMax=100))
	float ScreenPercentage = 50.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Safe Frames")
	FLinearColor Color = FLinearColor(1, 1, 1, 0.6f);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Safe Frames", Meta = (UIMin = 1, UIMax = 8, ClampMin = 1, ClampMax = 8))
	float Thickness = 1.f;
};

UENUM()
enum class EAvaShapeEditorOverlayType : uint8
{
	ComponentVisualizerOnly,
	FullDetails
};

USTRUCT()
struct FAvaShapeEditorViewportControlPosition
{
	GENERATED_BODY()

	UPROPERTY()
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment = EHorizontalAlignment::HAlign_Left;

	UPROPERTY()
	TEnumAsByte<EVerticalAlignment> VerticalAlignment = EVerticalAlignment::VAlign_Bottom;

	UPROPERTY()
	FVector2f AlignmentOffset = FVector2f::ZeroVector;
};

UCLASS(Config = EditorPerProjectUserSettings, meta = (DisplayName = "Viewport"))
class AVALANCHEVIEWPORT_API UAvaViewportSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UAvaViewportSettings();

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Viewport")
	bool bEnableViewportOverlay;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Viewport")
	bool bEnableBoundingBoxes;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Viewport")
	TSoftObjectPtr<UMaterial> ViewportBackgroundMaterial;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Viewport")
	TSoftObjectPtr<UMaterial> ViewportCheckerboardMaterial;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Viewport")
	FLinearColor ViewportCheckerboardColor0;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Viewport")
	FLinearColor ViewportCheckerboardColor1;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Viewport")
	float ViewportCheckerboardSize;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Viewport")
	bool bEnableShapesEditorOverlay;

	/** Whether to show or hide the Shapes In-Viewport controls. */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Viewport")
	EAvaShapeEditorOverlayType ShapeEditorOverlayType;

	UPROPERTY(Config)
	FAvaShapeEditorViewportControlPosition ShapeEditorViewportControlPosition;
	
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Screen Grid")
	bool bGridEnabled;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Screen Grid")
	bool bGridAlwaysVisible;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Screen Grid", Meta = (UIMin = 1, UIMax = 256, ClampMin = 1, ClampMax = 256))
	int32 GridSize;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Screen Grid")
	FLinearColor GridColor;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Screen Grid", Meta = (UIMin = 1, UIMax = 8, ClampMin = 1, ClampMax = 8))
	float GridThickness;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Pixel Grid")
	bool bPixelGridEnabled;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Pixel Grid")
	FLinearColor PixelGridColor;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Snapping", Meta = (Bitmask, BitmaskEnum = "/Script/AvalancheViewport.EAvaViewportSnapState"))
	int32 SnapState;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Snapping")
	bool bSnapIndicatorsEnabled;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Snapping")
	FLinearColor SnapIndicatorColor;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Snapping", Meta = (UIMin = 1, UIMax = 8, ClampMin = 1, ClampMax = 8))
	float SnapIndicatorThickness;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Guides")
	bool bGuidesEnabled;

	/**
	 * Directory used to load and save guide json config files.
	 *
	 * The path will be checked against 3 possible locations:
	 * - Just itself (/Config/Guides)
	 * - Project (/Path/To/Project/Config/Guides)
	 * - Plugin (/Path/To/Plugin/Config/Guides)
	 *
	 * Example values:
	 * - /Config/Guides
	 * - D:\UnrealEngine\Config\Guides
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Guides")
	FString GuideConfigPath;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Guides")
	FLinearColor EnabledGuideColor;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Guides")
	FLinearColor EnabledLockedGuideColor;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Guides")
	FLinearColor DisabledGuideColor;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Guides")
	FLinearColor DisabledLockedGuideColor;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Guides")
	FLinearColor DraggedGuideColor;

	// Not yet implemented
	UPROPERTY(Config)
	FLinearColor SnappedToGuideColor;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Guides", Meta = (UIMin = 1, UIMax = 8, ClampMin = 1, ClampMax = 8))
	float GuideThickness;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Safe Frames")
	bool bSafeFramesEnabled;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Safe Frames")
	TArray<FAvaLevelViewportSafeFrame> SafeFrames;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Camera Bounds")
	FLinearColor CameraBoundsShadeColor;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Texture Overlay")
	bool bEnableTextureOverlay = false;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Texture Overlay")
	TSoftObjectPtr<UTexture> TextureOverlayTexture;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Texture Overlay", Meta = (UIMin = 0.01, UIMax = 1, ClampMin = 0, ClampMax = 1))
	float TextureOverlayOpacity = 0.25;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Texture Overlay")
	bool bTextureOverlayStretch = false;

	UFUNCTION(BlueprintPure, Category = "Motion Design")
	EAvaViewportSnapState GetSnapState() const;

	UFUNCTION(BlueprintCallable, Category = "Motion Design")
	bool HasSnapState(EAvaViewportSnapState InSnapState);

	UFUNCTION(BlueprintCallable, Category = "Motion Design")
	void SetSnapState(EAvaViewportSnapState InSnapState);

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnAvaViewportSettingsChanged, const UAvaViewportSettings* /* This */, FName /* Setting name */)
	FOnAvaViewportSettingsChanged OnChange;

	void BroadcastSettingChanged(FName InSettingName);

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
};
