// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Delegates/IDelegateInstance.h"
#include "Math/MathFwd.h"
#include "Templates/SharedPointer.h"

class FEditorViewportClient;
class FViewport;
class IAvaViewportClient;

DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<IAvaViewportClient>, FAvaViewportClientCastDelegate, FEditorViewportClient*)
DECLARE_DELEGATE_RetVal_OneParam(FEditorViewportClient*, FAvaViewportCastDelegate, FViewport*)

struct AVALANCHEVIEWPORT_API FAvaViewportUtils
{
	static FDelegateHandle RegisterViewportClientCaster(FAvaViewportClientCastDelegate::TFuncType InFunction);
	static void UnregisterViewportClientCaster(FDelegateHandle InDelegateHandle);

	static FDelegateHandle RegisterViewportCaster(FAvaViewportCastDelegate::TFuncType InFunction);
	static void UnregisterViewportCaster(FDelegateHandle InDelegateHandle);

	/**
	 * Uses the viewport client casters to attempt to create an IAvaViewportClient. Failing that, it will create
	 * backup wrapper for FEditorViewportClient if it's a valid Level Editor viewport.
	 */
	[[nodiscard]] static TSharedPtr<IAvaViewportClient> GetAsAvaViewportClient(FEditorViewportClient* InViewportClient);

	/**
	 * Uses the viewport caster  to attempt to get an FEditorViewportClient*.
	 * The level editor does not properly clean up its viewports so dangling pointers remain making this check necessary.
	 */
	[[nodiscard]] static FEditorViewportClient* GetAsEditorViewportClient(FViewport* InViewport);

	/**
	 * Uses the viewport caster  to attempt to get an FEditorViewportClient*.
	 * The level editor does not properly clean up its viewports so dangling pointers remain making this check necessary.
	 */
	[[nodiscard]] static TSharedPtr<IAvaViewportClient> GetAvaViewportClient(FViewport* InViewport);

	static bool IsValidViewportSize(const FVector2f& InViewportSize);
	static bool IsValidViewportSize(const FIntPoint& InViewportSize);

	static float CalcFOV(float InViewportDimension, float InDistance);

	/** Calcs a viewport client's scene view and returns a transform of its rotation and location. */
	[[nodiscard]] static FTransform GetViewportViewTransform(FEditorViewportClient* InViewportClient);

	/** Calculates the size of the frustum plane parallel to the camera's near clip plane at the given distance. */
	static FVector2D GetFrustumSizeAtDistance(float InHorizontalFoVDegrees, float InAspectRatio, double InDistance);

	/**
	 * Translates the viewport position to the corresponding position on the frustum plane parallel to the camera's
	 * near clip plane at the given distance for perspective views.
	 */
	static FVector ViewportPositionToWorldPositionPerspective(const FVector2f& InViewportSize, const FVector2f& InViewportPosition,
		const FVector& InViewLocation, const FRotator& InViewRotation, float InHorizontalFoVDegrees, double InDistance);

	/**
	 * Translates the viewport position to the corresponding position on the frustum plane parallel to the camera's
	 * near clip plane at the given distance for orthographic views.
	 */
	static FVector ViewportPositionToWorldPositionOrthographic(const FVector2f& InViewportSize, const FVector2f& InViewportPosition,
		const FVector& InViewLocation, const FRotator& InViewRotation, float InOrthoWidth, double InDistance);

	/**
	 * Translates the position on the frustum plane parallel to the camera's near clip plane to the corresponding viewport position
	 * for perspective views.
	 */
	static void WorldPositionToViewportPositionPerspective(const FVector2f& InViewportSize, const FVector& InWorldPosition,
		const FVector& InViewLocation, const FRotator& InViewRotation, float InHorizontalFoVDegrees, FVector2f& OutViewportPosition, double& OutDistance);

	/**
	 * Translates the position on the frustum plane parallel to the camera's near clip plane to the corresponding viewport position
	 * for orthographic views.
	 */
	static void WorldPositionToViewportPositionOrthographic(const FVector2f& InViewportSize, const FVector& InWorldPosition,
		const FVector& InViewLocation, const FRotator& InViewRotation, float InOrthoWidth, FVector2f& OutViewportPosition, double& OutDistance);
};
