// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector2D.h"

struct AVALANCHEVIEWPORT_API FAvaVisibleArea
{
	FVector2f Offset;
	FVector2f VisibleSize;
	FVector2f AbsoluteSize;
	float DPIScale;

	FAvaVisibleArea();
	FAvaVisibleArea(const FVector2f& InAbsoluteSize);
	FAvaVisibleArea(const FVector2f& InAbsoluteSize, float InDPIScale);
	FAvaVisibleArea(const FVector2f& InOffset, const FVector2f& InVisibleSize, const FVector2f& InAbsoluteSize, float InDPIScale);

	bool IsValid() const;

	bool IsAbsoluteView() const;
	bool IsModifiedView() const;
	bool IsAbsoluteSize() const;
	bool IsZoomedView() const;
	bool IsCentered() const;
	bool IsOffset() const;

	float GetVisibleAreaFraction() const;
	float GetInvisibleAreaFraction() const;

	FVector2f GetVisiblePosition(const FVector2f& AbsolutePosition) const;
	FVector2f GetDPIScaledVisiblePosition(const FVector2f& AbsolutePosition) const;
	FVector2f GetAbsolutePosition(const FVector2f& VisiblePosition) const;
	FVector2f GetDPIScaledAbsolutePosition(const FVector2f& VisiblePosition) const;
	FVector2f GetVisibleAreaCenter() const;
	FVector2f GetAbsoluteVisibleAreaCenter() const;
	FVector2f GetAbsoluteAreaCenter() const;
	FVector2f GetInvisibleSize() const;
};
