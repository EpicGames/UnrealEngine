// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaVisibleArea.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"

class IAvaViewportClient;
struct FAvaVisibleArea;

class FAvaCameraZoomController
{
public:
	AVALANCHEVIEWPORT_API static bool IsCameraZoomPossible();

	AVALANCHEVIEWPORT_API FAvaCameraZoomController(TSharedRef<IAvaViewportClient> InAvaViewportClient, float InFallbackFOV);

	TSharedPtr<IAvaViewportClient> GetViewportClient() const
	{
		return AvaViewportClientWeak.Pin();
	}

	AVALANCHEVIEWPORT_API bool CanZoom() const;

	uint8 GetZoomLevel() const
	{
		return ZoomLevel;
	}

	void SetZoomLevel(uint8 InZoomLevel);
	bool IsZoomed() const;

	AVALANCHEVIEWPORT_API void ZoomIn();
	AVALANCHEVIEWPORT_API void ZoomInCursor();

	/* Zooms in maintaining the current PanOffsetFraction. Uses the absolute screen position. */
	void ZoomInAroundPoint(const FVector2f& InScreenPosition);

	/* Zooms in maintaining the position of the given viewport position. Uses the absolute screen position. */
	void ZoomInRelativePoint(const FVector2f& InViewportPosition);

	AVALANCHEVIEWPORT_API void ZoomOut();
	AVALANCHEVIEWPORT_API void ZoomOutCursor();

	/* Zooms out maintaining the current PanOffsetFraction. Uses the absolute screen position. */
	void ZoomOutAroundPoint(const FVector2f& OutScreenPosition);

	/* Zooms out maintaining the position of the given viewport position. Uses the absolute screen position. */
	void ZoomOutRelativePoint(const FVector2f& OutViewportPosition);

	AVALANCHEVIEWPORT_API void PanLeft();
	AVALANCHEVIEWPORT_API void PanRight();
	AVALANCHEVIEWPORT_API void PanUp();
	AVALANCHEVIEWPORT_API void PanDown();

	AVALANCHEVIEWPORT_API void FrameActor();

	AVALANCHEVIEWPORT_API void Reset();

	const FVector2f& GetPanOffsetFraction() const
	{
		return PanOffsetFraction;
	}

	void SetPanOffsetFraction(const FVector2f& InOffsetFraction);
	void PanAdjust(const FVector2f& InDirection);

	/** Adjusts zoom pan based on current zoom settings. */
	AVALANCHEVIEWPORT_API void PanAdjustZoomed(const FVector2f& InZoomedDirection);

	void CenterOnPoint(const FVector2f& InPoint);
	void CenterOnBox(const FBox& InBoundingBox, const FTransform& InBoxTransform);

	bool IsPanning() const
	{
		return bIsPanning;
	}

	AVALANCHEVIEWPORT_API void StartPanning();
	AVALANCHEVIEWPORT_API void EndPanning();

	float GetFallbackFOV() const
	{
		return FallbackFOV;
	}

	AVALANCHEVIEWPORT_API float GetDefaultFOV() const;
	AVALANCHEVIEWPORT_API float GetFOV() const;
	AVALANCHEVIEWPORT_API FVector2f GetCameraProjectionOffset() const;

	AVALANCHEVIEWPORT_API const FAvaVisibleArea& GetCachedVisibleArea() const;
	AVALANCHEVIEWPORT_API const FAvaVisibleArea& GetCachedZoomedVisibleArea() const;

	AVALANCHEVIEWPORT_API void UpdateVisibleAreas();

protected:
	TWeakPtr<IAvaViewportClient> AvaViewportClientWeak;

	const float FallbackFOV;

	uint8 ZoomLevel;

	FVector2f PanOffsetFraction;

	bool bIsPanning;

	FAvaVisibleArea CachedVisibleArea;
	FAvaVisibleArea CachedZoomedVisibleArea;

private:
	void ZoomIn_Internal();
	void ZoomOut_Internal();

	void InvalidateViewport();
};