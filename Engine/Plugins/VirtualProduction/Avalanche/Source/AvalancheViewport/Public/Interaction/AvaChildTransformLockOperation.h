// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Math/Transform.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IAvaViewportClient;
class AActor;

/**
 * Resets child actors to their original transforms after the parent actor
 * is moved. Will fail if an actor and its grandchild are both selected.
 * Moving the child actor (actor->child->grandchild) will reset the relative
 * transform of the grandchild. To be honest, I'm not sure how UE would handle
 * this by default anyway...
 */
class FAvaChildTransformLockOperation
{
public:
	AVALANCHEVIEWPORT_API FAvaChildTransformLockOperation(TSharedRef<IAvaViewportClient> InAvaViewportClient);

	AVALANCHEVIEWPORT_API void Save();

	AVALANCHEVIEWPORT_API void Restore();

protected:
	TWeakPtr<IAvaViewportClient> AvaViewportClientWeak;

	TMap<TWeakObjectPtr<AActor>, FTransform> ChildActorTransforms;

	bool bAllowModify;	

	void AttemptModifyOnLockedActors();
};
