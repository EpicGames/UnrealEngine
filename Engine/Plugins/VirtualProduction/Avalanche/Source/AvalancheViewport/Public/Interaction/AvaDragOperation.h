// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/MathFwd.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Viewport/Interaction/AvaSnapPoint.h"

class FAvaSnapOperation;
class IAvaViewportClient;

class FAvaDragOperation
{
public:
	AVALANCHEVIEWPORT_API FAvaDragOperation(TSharedRef<IAvaViewportClient> InAvaViewportClient, bool bInAllowSnapToChildren);
	AVALANCHEVIEWPORT_API virtual ~FAvaDragOperation();

	/** Restore position of actors so they can be accurately dragged. */
	AVALANCHEVIEWPORT_API void PreMouseUpdate();

	/** Snap actors in their post-drag position. */
	AVALANCHEVIEWPORT_API void PostMouseUpdate();

	bool IsValid() const
	{
		return bValid;
	}

protected:
	TWeakPtr<IAvaViewportClient> AvaViewportClientWeak;
	TSharedPtr<FAvaSnapOperation> SnapOperation;
	TArray<FAvaSnapPoint> DraggedObjectSnapPoints;

	/** Not const because we'll be moving them */
	TMap<TWeakObjectPtr<UObject>, FVector> DraggedObjectStartLocations;

	/**
	 * If we fail to init correctly, this will remain false.
	 * The only non-error that would cause this is if we're dragging components.
	 * Component drag-snapping us not currently supported.
	 */
	bool bValid;

	void Init(bool bInAllowSnapToChildren);
	void Cleanup();
};
