// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class FSceneView;
class IAvaViewportClient;

class AVALANCHEVIEWPORT_API FAvaIsolateActorsOperation
{
public:
	FAvaIsolateActorsOperation(TSharedRef<IAvaViewportClient> InAvaViewportClient);

	TConstArrayView<TWeakObjectPtr<AActor>> GetIsolatedActors() const { return IsolatedActors; }

	bool IsIsolatingActors() const { return !IsolatedActors.IsEmpty(); }

	bool CanToggleIsolateActors() const;

	void ToggleIsolateActors();

	void IsolateActors();

	void UnisolateActors();

	void AddIsolatedActorPrimitives(FSceneView* InSceneView);

protected:
	TWeakPtr<IAvaViewportClient> AvaViewportClientWeak;

	TArray<TWeakObjectPtr<AActor>> IsolatedActors;
};
