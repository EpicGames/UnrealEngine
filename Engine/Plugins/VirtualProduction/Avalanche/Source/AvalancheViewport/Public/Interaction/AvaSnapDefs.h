// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSnapDefs.generated.h"

UENUM(BlueprintType, Meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EAvaViewportSnapState : uint8
{
	Off = 0 UMETA(Hidden),
	Global = 1 << 0,
	Screen = 1 << 1,
	Grid = 1 << 2,
	Actor = 1 << 3,
	All = Screen | Grid | Actor
};

ENUM_CLASS_FLAGS(EAvaViewportSnapState);
