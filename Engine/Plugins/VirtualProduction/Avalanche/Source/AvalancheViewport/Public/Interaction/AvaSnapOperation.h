// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Viewport/Interaction/AvaSnapPoint.h"
#include "Containers/Array.h"
#include "Math/MathFwd.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class FEditorViewportClient;
class FSceneView;
class FSceneViewFamilyContext;
class FViewportClient;
class UActorComponent;
class UPrimitiveComponent;

struct FAvaSnapPointLink
{
	float Position; // X or Y screen position
	int32 SnapIdx; // Index in the main snap array

	FAvaSnapPointLink(float InPosition, int32 InSnapIdx)
		: Position(InPosition)
		, SnapIdx(InSnapIdx)
	{ 
	}

	bool operator<(const FAvaSnapPointLink& InOther) const
	{
		if (FMath::IsNearlyEqual(Position, InOther.Position))
		{
			return SnapIdx < InOther.SnapIdx;
		}

		return Position < InOther.Position;
	}
};

class FAvaSnapOperation : public TSharedFromThis<FAvaSnapOperation>
{
public:
	static inline const float MaximumSnapDistance = 10.f;

	// Generates snap points for the given actor
	static void GenerateLocalSnapPoints(AActor* InActor, TArray<FAvaSnapPoint>& InOutSnapPoints);

	// Generates snap points based on a box extent
	static void GenerateLocalSnapPoints(AActor* InActor, const FVector& InOrigin, const FVector& InBoxExtent,
		TArray<FAvaSnapPoint>& InOutSnapPoints);

	// Generates snap points for the given component
	static void GenerateLocalSnapPoints(UPrimitiveComponent* InComponent, TArray<FAvaSnapPoint>& InOutSnapPoints);

	// Generates snap points based on a box extent
	static void GenerateLocalSnapPoints(UPrimitiveComponent* InComponent, const FVector& InOrigin, const FVector& InBoxExtent,
		TArray<FAvaSnapPoint>& InOutSnapPoints);

	// Generates snap points based on a box extent with no actor reference
	static void GenerateBoundsSnapPoints(const FVector& InOrigin, const FVector& InBoxExtent, TArray<FAvaSnapPoint>& InOutSnapPoints);

	AVALANCHEVIEWPORT_API FAvaSnapOperation(FEditorViewportClient* InEditorViewportClient);
	AVALANCHEVIEWPORT_API virtual ~FAvaSnapOperation();

	const TArray<FAvaSnapPoint>& GetScreenSnapPoints() const
	{
		return ScreenSnapPoints;
	}

	void SetScreenSnapPoints(const TArray<FAvaSnapPoint>& InSnapPoints)
	{
		ScreenSnapPoints = InSnapPoints;
	}

	const TArray<FAvaSnapPoint>& GetActorSnapPoints() const
	{
		return ActorSnapPoints;
	}

	// Adds screen- and guide-based snap points
	void GenerateScreenSnapPoints();

	// Generates snap points, excluding the actor for the given Dynamic Mesh
	AVALANCHEVIEWPORT_API void GenerateComponentSnapPoints(const UActorComponent* InComponent);

	// Generates snap points, excluding the list of actors
	AVALANCHEVIEWPORT_API void GenerateActorSnapPoints(const TConstArrayView<TWeakObjectPtr<AActor>>& InSelectedActors,
		const TConstArrayView<TWeakObjectPtr<AActor>>& InExcludedActors);

	// Adds a snap point for a shape to the list of shape snap points
	AVALANCHEVIEWPORT_API void AddActorSnapPoint(const FAvaSnapPoint& InSnapPoint);

	// Sorts snap point links
	AVALANCHEVIEWPORT_API void FinaliseSnapPoints();

	FVector GetSelectionLocation() const;
	FVector GetDragOffset() const;

	const FAvaSnapPoint* const GetScreenSnapPoint(int32 InScreenSnapPointIdx) const;
	const FAvaSnapPoint* const GetActorSnapPoint(int32 InActorSnapPointIdx) const;
	const FAvaSnapPoint* const GetSnapPointByLink(int32 InSnapPointLinkIdx) const;

	// Returns true if we're snapped to a point. We may snap to the grid and not to a point. This returns false.
	bool SnapVectorComponent(float& InOutComponent, const TArray<FAvaSnapPointLink>& InSnapPointLinks,
		float InViewportSize, int32 InScreenSnapPointCount, int32& OutClosestSnapPointIdx, float& OutDistanceFromSnap);
	
	bool SnapX(float& InOutComponent, float InViewportSize, int32& OutClosestSnapPointIdx, float& OutDistanceFromSnap);
	bool SnapY(float& InOutComponent, float InViewportSize, int32& OutClosestSnapPointIdx, float& OutDistanceFromSnap);

	void SnapScreenLocationX(float& InOutScreenLocation);
	void SnapScreenLocationY(float& InOutScreenLocation);
	bool SnapDragLocation(const TArray<FAvaSnapPoint>& InDraggedActorSnapPoints, FVector& OutSnapOffset);

	AVALANCHEVIEWPORT_API void SnapScreenLocation(FVector2f& InOutScreenLocation);
	AVALANCHEVIEWPORT_API void SnapScreenLocation(FVector2f& InOutScreenLocation, bool bInSnapX, bool bInSnapY);
	AVALANCHEVIEWPORT_API bool SnapLocation(FVector& Location);

	bool WasSnappedToX() const
	{
		return bSnappedToLinkX;
	}
	AVALANCHEVIEWPORT_API void SetSnappedToX(bool bInSnappedTo);
	
	bool WasSnappedToY() const
	{
		return bSnappedToLinkY;
	}
	AVALANCHEVIEWPORT_API void SetSnappedToY(bool bInSnappedTo);

	bool WasSnappedTo() const
	{
		return bSnappedToLinkX || bSnappedToLinkY;
	}

	const FVector2f& GetSnappedToLocation() const
	{
		return SnappedToLocation;
	}

	const FVector2f& GetSnapDistances() const
	{
		return SnapDistances;
	}

protected:
	/**
	 * Cannot ensure that this is available as a shared pointer.
	 * You can validate it by using FAvaViewportUtils::GetAsAvaViewportClient(EditorViewportClient)
	 */
	FEditorViewportClient* EditorViewportClient;

	// SceneView used for calculating world/viewport translations
	TSharedPtr<FSceneViewFamilyContext> CachedSceneViewFamily;
	FSceneView* CachedSceneView;

	// Transform of the pivot when 
	FVector SelectionStartLocation;

	// Snap points related to the screen and guides
	TArray<FAvaSnapPoint> ScreenSnapPoints;

	// Snap points related to actors
	TArray<FAvaSnapPoint> ActorSnapPoints;

	// Links to the above 2 arrays
	TArray<FAvaSnapPointLink> SnapPointLinksX;
	TArray<FAvaSnapPointLink> SnapPointLinksY;

	// Snap stuff
	int32 ClosestSnapPointLinkIdxX;
	int32 ClosestSnapPointLinkIdxY;
	bool bSnappedToLinkX;
	bool bSnappedToLinkY;
	FVector2f SnapDistances;
	FVector2f SnappedToLocation;
};
