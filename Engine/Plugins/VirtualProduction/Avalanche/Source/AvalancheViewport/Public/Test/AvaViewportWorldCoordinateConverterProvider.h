// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "Math/Transform.h"
#include "Math/Vector2D.h"
#include "ViewportClient/IAvaViewportWorldCoordinateConverter.h"

/**
 * Test classes that can be passed to the alignment utils.
 * Instantiate with MakeShared<FAvaViewportWorldCoordinateConverterProviderPerspective>(...)
 *   or MakeShared<FAvaViewportWorldCoordinateConverterProviderOrthographic>(...)
 */
struct FAvaViewportWorldCoordinateConverterProvider : public IAvaViewportWorldCoordinateConverter
{
	//~ Begin IAvaViewportWorldCoordinateConverter
	virtual FVector2f GetViewportSize() const override { return ViewportSize; }
	virtual FTransform GetViewportViewTransform() const override { return ViewTransform; }
	//~End IAvaViewportWorldCoordinateConverter

protected:
	FTransform ViewTransform;
	FVector2f ViewportSize;

	FAvaViewportWorldCoordinateConverterProvider(const FTransform& InViewTransform, const FVector2f& InViewportSize);
};

struct FAvaViewportWorldCoordinateConverterProviderPerspective : FAvaViewportWorldCoordinateConverterProvider
{
	AVALANCHEVIEWPORT_API FAvaViewportWorldCoordinateConverterProviderPerspective(FVector InLocation, FRotator InRotation, FVector2f InViewportSize, 
		float InFieldOfView);

protected:
	//~ Begin IAvaViewportWorldCoordinateConverter
	virtual FVector2D GetFrustumSizeAtDistance(double InDistance) const override;
	virtual FVector ViewportPositionToWorldPosition(const FVector2f& InViewportPosition, double InDistance) const override;
	virtual void WorldPositionToViewportPosition(const FVector& InWorldPosition, FVector2f& OutViewportPosition, double& OutDistance) const override;
	//~End IAvaViewportWorldCoordinateConverter

	float FieldOfView;
};

struct FAvaViewportWorldCoordinateConverterProviderOrthographic : public FAvaViewportWorldCoordinateConverterProvider
{
	AVALANCHEVIEWPORT_API FAvaViewportWorldCoordinateConverterProviderOrthographic(FVector InLocation, FRotator InRotation, FVector2f InViewportSize, 
		float InOrthographicWidth);

protected:
	//~ Begin IAvaViewportWorldCoordinateConverter
	virtual FVector2D GetFrustumSizeAtDistance(double InDistance) const override;
	virtual FVector ViewportPositionToWorldPosition(const FVector2f& InViewportPosition, double InDistance) const override;
	virtual void WorldPositionToViewportPosition(const FVector& InWorldPosition, FVector2f& OutViewportPosition, double& OutDistance) const override;
	//~End IAvaViewportWorldCoordinateConverter

	float OrthographicWidth;
};
