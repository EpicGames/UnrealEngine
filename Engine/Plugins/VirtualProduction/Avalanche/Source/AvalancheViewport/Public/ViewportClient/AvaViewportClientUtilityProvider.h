// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportClient/IAvaViewportClient.h"
#include "AvaType.h"
#include "Math/MathFwd.h"

class FEditorViewportClient;

class AVALANCHEVIEWPORT_API FAvaViewportClientUtilityProvider : public IAvaViewportClient
{
public:
	UE_AVA_INHERITS_WITH_SUPER(FAvaViewportClientUtilityProvider, IAvaViewportClient)

	//~ Begin IAvaViewportWorldCoordinateConverter
	virtual FVector2D GetFrustumSizeAtDistance(double InDistance) const override;
	virtual FVector ViewportPositionToWorldPosition(const FVector2f& InViewportPosition, double InDistance) const override;
	virtual void WorldPositionToViewportPosition(const FVector& InWorldPosition, FVector2f& OutViewportPosition, double& OutDistance) const override;
	//~ End IAvaViewportWorldCoordinateConverter

	//~ Begin IAvaViewportClient
	virtual FVector2f GetViewportWidgetSize() const override;
	virtual FVector2f GetVirtualViewportScale() const override;
	virtual float GetAverageVirtualViewportScale() const override;
	virtual FVector2D GetZoomedFrustumSizeAtDistance(double InDistance) const override;
	virtual void OnCameraCut(AActor* InTarget, bool bInJumpCut) override;
	//~ End IAvaViewportClient

protected:
	static void NotifyJumpCut(AActor* InViewTarget);
};
