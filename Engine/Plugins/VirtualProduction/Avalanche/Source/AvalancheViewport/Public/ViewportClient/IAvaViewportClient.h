// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "Math/MathFwd.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "ViewportClient/IAvaViewportWorldCoordinateConverter.h"

class AActor;
class FAvaCameraZoomController;
class FAvaSnapOperation;
class FAvaViewportPostProcessManager;
class FEditorViewportClient;
class IAvaBoundsProviderInterface;
class ILevelEditor;
class UCameraComponent;
struct FAvaVisibleArea;

/**
 * Virtual size is one that scales the viewport's ruler's apparent size to the given size.
 * A virtual size of 1920x1080 on a 960x540 viewport will mean pixels are half as big.
 */
class IAvaViewportClient : public IAvaViewportWorldCoordinateConverter
{
public:
	UE_AVA_INHERITS(IAvaViewportClient, IAvaViewportWorldCoordinateConverter)

	virtual bool IsMotionDesignViewport() const = 0;

	virtual const FEditorViewportClient* AsEditorViewportClient() const = 0;

	/** Returns true if the viewport client supports 2d zoom. */
	virtual bool SupportsZoom() const = 0;

	/** Gets the FOV of the viewport client when zoomed in, if zoom is supported. */
	virtual float GetZoomedFOV() const = 0;

	/** Gets the FOV of the viewport client when not zoomed in. */
	virtual float GetUnZoomedFOV() const = 0;

	/** Gets the full size of the viewport's virtual size when not zoomed in. */
	virtual FIntPoint GetVirtualViewportSize() const = 0;

	/** Returns (virtual viewport size / viewport size) */
	virtual FVector2f GetVirtualViewportScale() const = 0;

	/** Returns the average of the above. */
	virtual float GetAverageVirtualViewportScale() const = 0;

	/** Gets the viewport's offset from the containing widget. */
	virtual FVector2f GetViewportOffset() const = 0;

	/** Gets the total size of the widget that contains the viewport. */
	virtual FVector2f GetViewportWidgetSize() const = 0;

	/** Gets the DPI scale of the viewport. */
	virtual float GetViewportDPIScale() const = 0;

	/** Gets the viewport's non-zoomed visible area. Only checks the full size of the viewport, ignoring zoom settings. */
	virtual FAvaVisibleArea GetVisibleArea() const = 0;

	/** Gets the viewport's non-zoomed virtual visible area. Only checks the full size of the viewport, ignoring zoom settings. */
	virtual FAvaVisibleArea GetVirtualVisibleArea() const = 0;

	/** Gets the viewport's zoomed visible area. Returns the full and calculated-zoom sizes. */
	virtual FAvaVisibleArea GetZoomedVisibleArea() const = 0;

	/** Gets the viewport's zoomed virtual visible area. Returns the full and calculated-zoom sizes. */
	virtual FAvaVisibleArea GetVirtualZoomedVisibleArea() const = 0;

	/** Calculates the size of the frustum plane parallel to the camera's near clip plane at the given distance. */
	virtual FVector2D GetZoomedFrustumSizeAtDistance(double InDistance) const = 0;

	/*
	 * Use the absolute mouse position on the viewport and the zoom settings to return the
	 * position of that point if the view were at the default zoom. Viewport offset and
	 * aspect ratio clamping are ignored.
	 */
	virtual FVector2f GetUnconstrainedViewportMousePosition() const = 0;

	/*
	 * Use the absolute mouse position on the viewport and the zoom settings to return the
	 * position of that point if the view were at the default zoom.
	 */
	virtual FVector2f GetConstrainedViewportMousePosition() const = 0;

	/*
	 * Use the absolute mouse position on the viewport and the zoom settings to return the
	 * position of that point. Viewport offset and aspect ratio clamping are ignored.
	 *
	 * E.g. with a zoom of 50% and a viewport resolution of 1000x600. If the mouse is in the far top left
	 * corner, the mouse position to resolve to 250x150.
	 */
	virtual FVector2f GetUnconstrainedZoomedViewportMousePosition() const = 0;

	/*
	 * Use the absolute mouse position on the viewport and the zoom settings to return the
	 * position of that point.
	 *
	 * E.g. with a zoom of 50% and a viewport resolution of 1000x600. If the mouse is in the far top left
	 * corner, the mouse position to resolve to 250x150.
	 */
	virtual FVector2f GetConstrainedZoomedViewportMousePosition() const = 0;

	/** Gets an active snap operation on this viewport. */
	virtual TSharedPtr<FAvaSnapOperation> GetSnapOperation() const = 0;

	/** Starts an active snap operation on this viewport. Snap operations are only weakly stored. */
	virtual TSharedPtr<FAvaSnapOperation> StartSnapOperation() = 0;

	/**
	 * Ends the current snap operation and resets any references.
	 * If parameter specified, will only cancel the given snap operation.
	 */
	virtual bool EndSnapOperation(FAvaSnapOperation* InSnapOperation = nullptr) = 0;

	/** Called when the actor selection changes. */
	virtual void OnActorSelectionChanged() = 0;

	/** Returns the object responsible for 2d zooming and panning. */
	virtual TSharedPtr<FAvaCameraZoomController> GetZoomController() const = 0;

	/** Returns the currently active camera component view target (if available) */
	virtual UCameraComponent* GetCameraComponentViewTarget() const = 0;

	/** Returns the currently active actor view target (if available) */
	virtual AActor* GetViewTarget() const = 0;

	/** Sets the current view target. */
	virtual void SetViewTarget(TWeakObjectPtr<AActor> InViewTarget) = 0;

	/** Call this when a camera cut occurs. */
	virtual void OnCameraCut(AActor* InCamera, bool bInJumpCut) = 0;

	virtual UWorld* GetViewportWorld() const = 0;

	virtual TSharedPtr<FAvaViewportPostProcessManager> GetPostProcessManager() const = 0;
};
