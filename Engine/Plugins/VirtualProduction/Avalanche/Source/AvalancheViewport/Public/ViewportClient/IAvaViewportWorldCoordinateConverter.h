// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "Math/MathFwd.h"

/**
 * Provider utility for converting to and from viewport/world coordinates and frustrum sizes.
 */
class IAvaViewportWorldCoordinateConverter : public IAvaTypeCastable
{
public:
	UE_AVA_INHERITS(IAvaViewportWorldCoordinateConverter, IAvaTypeCastable)

	/** Gets the full size of the viewport when not zoomed in. */
	virtual FVector2f GetViewportSize() const = 0;

	/** Calculates the size of the frustum plane parallel to the camera's near clip plane at the given distance. */
	virtual FVector2D GetFrustumSizeAtDistance(double InDistance) const = 0;

	/**
	 * Translates the viewport position to the corresponding position on the frustum plane parallel to the camera's
	 * near clip plane at the given distance.
	 */
	virtual FVector ViewportPositionToWorldPosition(const FVector2f& InViewportPosition, double InDistance) const = 0;

	/* Translates the position on the frustum plane parallel to the camera's near clip plane to the corresponding viewport position. */
	virtual void WorldPositionToViewportPosition(const FVector& InWorldPosition, FVector2f& OutViewportPosition, double& OutDistance) const = 0;

	/** Returns the current view transform. */
	virtual FTransform GetViewportViewTransform() const = 0;
};
