// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaType.h"
#include "Templates/SharedPointer.h"

class AActor;
class FPrimitiveDrawInterface;
class UAvaBoundsProviderSubsystem;
class UAvaSelectionProviderSubsystem;

enum class EAvaViewportBoundingBoxOptimizationState : uint8
{
	RenderNothing,
	RenderSelectedActors,
	RenderSelectedActorsAndChildren,
	RenderSelectionBounds
};

class IAvaViewportBoundingBoxVisualizer : public IAvaTypeCastable
{
public:
	UE_AVA_INHERITS(IAvaViewportBoundingBoxVisualizer, IAvaTypeCastable)

	virtual void Draw(UAvaSelectionProviderSubsystem& InSelectionProvider, UAvaBoundsProviderSubsystem& InBoundsProvider, FPrimitiveDrawInterface& InPDI) = 0;
	virtual EAvaViewportBoundingBoxOptimizationState GetOptimizationState() const = 0;
	virtual void ResetOptimizationState() = 0;
};

struct FAvaViewportBoundingBoxVisualizerProvider
{
	AVALANCHEVIEWPORT_API static TSharedRef<IAvaViewportBoundingBoxVisualizer> CreateVisualizer();
};
