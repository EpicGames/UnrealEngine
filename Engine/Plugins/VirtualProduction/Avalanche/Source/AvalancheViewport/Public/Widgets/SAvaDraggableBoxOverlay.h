// Copyright Epic Games, Inc. All Rights Reserved.

/* Copied and modified from ModelingEditorUI */

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "AvaViewportSettings.h"
#include "Layout/Margin.h"
#include "Types/SlateEnums.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SBox;

/**
 * Class that can be used to place a draggable box into a viewport or some other large widget as an
 * overlay. Just place the widget that you want to be draggable as the contents of SAvaDraggableBoxOverlay.
 */
class SAvaDraggableBoxOverlay : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvaDraggableBoxOverlay)  {}
	SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	AVALANCHEVIEWPORT_API void Construct(const FArguments& InArgs);

	/* Returns the current position of the box on the viewport. */
	FVector2f GetBoxAlignmentOffset() const;

	/* Sets the box offset from its alignment side. Bottom offsets from bottom, etc. */
	void SetBoxAlignmentOffset(const FVector2f& InOffset, bool bInNormalisePosition = true);

	EHorizontalAlignment GetBoxHorizontalAlignment() const;

	void SetBoxHorizontalAlignment(EHorizontalAlignment InAlignment);

	EVerticalAlignment GetBoxVerticalAlignment() const;

	void SetBoxVerticalAlignment(EVerticalAlignment InAlignment);

	void SavePosition();

protected:
	TSharedPtr<SBox> Container;
	TSharedPtr<SWidget> DraggableBox;
	EHorizontalAlignment HorizontalAlignment = HAlign_Fill;
	EVerticalAlignment VerticalAlignment = VAlign_Fill;
	FMargin Padding;

	FMargin GetPadding() const;
};
